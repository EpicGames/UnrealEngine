// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Factories/AssetFactoryInterface.h"
#include "Engine/World.h"

#include "ActorFactory.generated.h"

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogActorFactory, Log, All);

class AActor;
struct FActorSpawnParameters;
struct FAssetData;
class UBlueprint;
class ULevel;
class UInstancedPlacemenClientSettings;
class AVolume;
class UBrushBuilder;
UCLASS(collapsecategories, hidecategories=Object, editinlinenew, config=Editor, abstract, transient, MinimalAPI)
class UActorFactory : public UObject, public IAssetFactoryInterface
{
	GENERATED_UCLASS_BODY()

	/** Name used as basis for 'New Actor' menu. */
	UPROPERTY()
	FText DisplayName;

	/** Indicates how far up the menu item should be. The higher the number, the higher up the list.*/
	UPROPERTY(config)
	int32 MenuPriority;

	/** name of actor subclass this actorfactory creates - dynamically loaded.  Overrides NewActorClass. */
	UPROPERTY(config)
	FString NewActorClassName;

	/** should this actor factory auto register with the editor factory system. For factories within a GameFeaturePlugin this should be false. */
	UPROPERTY(config)
	bool bShouldAutoRegister = true;

	/**  AActor  subclass this ActorFactory creates. */
	UPROPERTY()
	TSubclassOf<AActor>  NewActorClass;

	/** Whether to appear in the editor add actor quick menu */
	UPROPERTY()
	uint32 bShowInEditorQuickMenu:1;

	UPROPERTY()
	uint32 bUseSurfaceOrientation:1;

	UPROPERTY()
	uint32 bUsePlacementExtent:1;

	/** Translation applied to the spawn position. */
	UPROPERTY()
	FVector SpawnPositionOffset;

	/** Called to actual create an actor with the supplied transform (scale is ignored), using the properties in the ActorFactory */
	UNREALED_API AActor* CreateActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams = FActorSpawnParameters());

	UNREALED_API virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg );

	/** Name to put on context menu. */
	FText GetDisplayName() const { return DisplayName; }

	/** Initialize NewActorClass if necessary, and return default actor for that class. */
	UNREALED_API virtual AActor* GetDefaultActor( const FAssetData& AssetData );

	/** Initialize NewActorClass if necessary, and return that class. */
	UNREALED_API virtual UClass* GetDefaultActorClass( const FAssetData& AssetData );

	/** Given an instance of an actor, find the wrapped asset object which can be used to create a valid FAssetData.
	 *  Returns nullptr if the given ActorInstance is not valid for this factory.
	 *  Override this function if the factory actor is a different class than the asset data's class which this factory operates on.
	 *  For example, if this is the static mesh actor factory, the class of the asset data is UStaticMesh, but the actor factory's class is AStaticMeshActor
	 */
	UNREALED_API virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance);

	/** Return a quaternion which aligns this actor type to the specified surface normal */
	UNREALED_API virtual FQuat AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation = FQuat::Identity) const;

	// Begin IAssetFactoryInterface Interface
	UNREALED_API virtual bool CanPlaceElementsFromAssetData(const FAssetData& InAssetData) override;
	UNREALED_API virtual bool PrePlaceAsset(FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	UNREALED_API virtual TArray<FTypedElementHandle> PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	UNREALED_API virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	UNREALED_API virtual FAssetData GetAssetDataFromElementHandle(const FTypedElementHandle& InHandle) override;
	UNREALED_API virtual void BeginPlacement(const FPlacementOptions& InPlacementOptions) override;
	UNREALED_API virtual void EndPlacement(TArrayView<const FTypedElementHandle> InPlacedElements, const FPlacementOptions& InPlacementOptions) override;
	UNREALED_API virtual UInstancedPlacemenClientSettings* FactorySettingsObjectForPlacement(const FAssetData& InAssetData, const FPlacementOptions& InPlacementOptions) override;
	// End IAssetFactoryInterface Interface

	static UNREALED_API void CreateBrushForVolumeActor(AVolume* NewActor, UBrushBuilder* BrushBuilder);

protected:

	/** Validates the input params for SpawnActor and returns the appropriate level to use depending on whether InLevel and/or InSpawnParams.OverrideLevel is passed : */
	UNREALED_API ULevel* ValidateSpawnActorLevel(ULevel* InLevel, const FActorSpawnParameters& InSpawnParams) const;

	UNREALED_API virtual bool PreSpawnActor( UObject* Asset, FTransform& InOutLocation);

	UNREALED_API virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams);

	/** Subclasses may implement this to modify the actor after it has been spawned */
	UNREALED_API virtual void PostSpawnActor( UObject* Asset, AActor* NewActor );

	/**
	 * Get the default label that should be used for the actor spawned by the given asset (does not include any numeric suffix).
	 * For classes or BPs that inherit from AActor this will defer to AActor::GetDefaultActorLabel, and for everything else it will use the asset name.
	 */
	UNREALED_API virtual FString GetDefaultActorLabel(UObject* Asset) const;
};

extern UNREALED_API FQuat FindActorAlignmentRotation(const FQuat& InActorRotation, const FVector& InModelAxis, const FVector& InWorldNormal, FQuat* OutDeltaRotation = nullptr);
