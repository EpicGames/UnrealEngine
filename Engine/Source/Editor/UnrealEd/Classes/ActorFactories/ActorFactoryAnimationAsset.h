// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactorySkeletalMesh.h"
#include "ActorFactoryAnimationAsset.generated.h"

class AActor;
struct FAssetData;
class USkeletalMesh;

UCLASS(MinimalAPI, config=Editor, hidecategories=Object)
class UActorFactoryAnimationAsset : public UActorFactorySkeletalMesh
{
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance);
	//~ End UActorFactory Interface

	virtual USkeletalMesh* GetSkeletalMeshFromAsset( UObject* Asset ) override;
};



