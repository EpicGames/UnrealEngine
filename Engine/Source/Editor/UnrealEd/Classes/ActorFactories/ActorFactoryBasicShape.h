// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "ActorFactoryBasicShape.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI,config=Editor)
class UActorFactoryBasicShape : public UActorFactoryStaticMesh
{
	GENERATED_UCLASS_BODY()

	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;

	UNREALED_API static const FSoftObjectPath BasicCube;
	UNREALED_API static const FSoftObjectPath BasicSphere;
	UNREALED_API static const FSoftObjectPath BasicCylinder;
	UNREALED_API static const FSoftObjectPath BasicCone;
	UNREALED_API static const FSoftObjectPath BasicPlane;
};
