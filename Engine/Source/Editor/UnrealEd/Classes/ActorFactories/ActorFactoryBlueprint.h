// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryBlueprint.generated.h"

class AActor;
struct FAssetData;

UCLASS(config=Editor, collapsecategories, hidecategories=Object, MinimalAPI)
class UActorFactoryBlueprint : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	UNREALED_API virtual bool PreSpawnActor( UObject* Asset, FTransform& InOutLocation ) override;
	UNREALED_API virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	UNREALED_API virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	UNREALED_API virtual AActor* GetDefaultActor( const FAssetData& AssetData ) override;
	UNREALED_API virtual UClass* GetDefaultActorClass( const FAssetData& AssetData ) override;
	//~ End UActorFactory Interface	
};



