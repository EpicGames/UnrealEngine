// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactoryVolume.h"
#include "ActorFactoryBoxVolume.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config = Editor, collapsecategories, hidecategories = Object)
class UActorFactoryBoxVolume : public UActorFactoryVolume
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	UNREALED_API virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UNREALED_API virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory Interface
};
