// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryCameraActor.generated.h"

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactoryCameraActor : public UActorFactory
{
	GENERATED_UCLASS_BODY()
};
