// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryCharacter.generated.h"

struct FAssetData;

UCLASS(MinimalAPI,config=Editor)
class UActorFactoryCharacter : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
};
