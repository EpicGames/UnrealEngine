// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryClass.generated.h"

class AActor;
struct FAssetData;
class ULevel;
struct FActorSpawnParameters;

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactoryClass : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual bool PreSpawnActor( UObject* Asset, FTransform& InOutLocation ) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual AActor* GetDefaultActor( const FAssetData& AssetData ) override;
	//~ End UActorFactory Interface	

protected:
	virtual AActor* SpawnActor( UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams) override;
};
