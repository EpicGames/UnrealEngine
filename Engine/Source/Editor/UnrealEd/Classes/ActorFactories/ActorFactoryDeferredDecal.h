// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryDeferredDecal.generated.h"

class AActor;
struct FAssetData;
class UMaterialInterface;

UCLASS(MinimalAPI, config=Editor)
class UActorFactoryDeferredDecal : public UActorFactory
{
public:
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	//~ End UActorFactory Interface

private:
	UMaterialInterface* GetMaterial( UObject* Asset ) const;

};



