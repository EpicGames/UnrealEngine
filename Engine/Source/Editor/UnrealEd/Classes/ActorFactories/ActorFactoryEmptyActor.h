// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryEmptyActor.generated.h"

class AActor;
struct FActorSpawnParameters;
struct FAssetData;
class ULevel;

UCLASS(MinimalAPI,config=Editor)
class UActorFactoryEmptyActor : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	/** If true a sprite will be added to visualize the actor in the world */
	UPROPERTY()
	bool bVisualizeActor;
public:
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;

protected:
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;

};
