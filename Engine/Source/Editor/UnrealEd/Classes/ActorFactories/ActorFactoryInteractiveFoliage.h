// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "ActorFactoryInteractiveFoliage.generated.h"

struct FAssetData;

UCLASS(MinimalAPI, config=Editor)
class UActorFactoryInteractiveFoliage : public UActorFactoryStaticMesh
{
	GENERATED_UCLASS_BODY()


	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override { return false; };
	//~ End UActorFactory Interface
};



