// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "ActorFactoryPawn.generated.h"

struct FAssetData;

UCLASS(MinimalAPI,config=Editor)
class UActorFactoryPawn : public UActorFactoryEmptyActor
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
};
