// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryRectLight.generated.h"

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactoryRectLight : public UActorFactory
{
	GENERATED_UCLASS_BODY()

protected:

	void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
};



