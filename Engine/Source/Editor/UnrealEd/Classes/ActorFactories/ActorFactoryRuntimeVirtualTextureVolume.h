// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryRuntimeVirtualTextureVolume.generated.h"

/** Actor factory for ARuntimeVirtualTextureVolume */
UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactoryRuntimeVirtualTextureVolume : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	//~ End UActorFactory Interface
};
