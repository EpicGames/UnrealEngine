// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactorySkeletalMesh.generated.h"

class AActor;
struct FAssetData;
class USkeletalMesh;

DECLARE_DELEGATE_RetVal_OneParam(USkeletalMesh*, FGetSkeletalMeshFromAssetDelegate, const FAssetData& /* in asset */);
DECLARE_DELEGATE_TwoParams(FPostSkeletalMeshActorSpawnedDelegate, AActor* /* spawned actor */, UObject* /* asset source*/);

UCLASS(MinimalAPI, config=Editor)
class UActorFactorySkeletalMesh : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	/*
	 * Adds support for customizing the actor spawning given a set of delegates.
	 */
	static UNREALED_API void RegisterDelegatesForAssetClass(
		UClass* InAssetClass,
		FGetSkeletalMeshFromAssetDelegate GetSkeletalMeshFromAssetDelegate,
		FPostSkeletalMeshActorSpawnedDelegate PostSkeletalMeshActorSpawnedDelegate
	);

	/*
	 * Removes the registered customization for actor spawning
	 */
	static UNREALED_API void UnregisterDelegatesForAssetClass(UClass* InAssetClass);

protected:

	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual FQuat AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const override;
	//~ End UActorFactory Interface

	virtual USkeletalMesh* GetSkeletalMeshFromAsset( UObject* Asset );

	UClass* ClassUsedForDelegate;
	static TMap<UClass*, FGetSkeletalMeshFromAssetDelegate> GetSkeletalMeshDelegates;
	static TMap<UClass*, FPostSkeletalMeshActorSpawnedDelegate> PostSkeletalMeshActorSpawnedDelegates;
};



