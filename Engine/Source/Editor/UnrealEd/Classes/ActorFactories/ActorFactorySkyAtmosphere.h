// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactorySkyAtmosphere.generated.h"

UCLASS(MinimalAPI, config = Editor, collapsecategories, hidecategories = Object)
class UActorFactorySkyAtmosphere : public UActorFactory
{
	GENERATED_UCLASS_BODY()
};

