// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactorySkyLight.generated.h"

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactorySkyLight : public UActorFactory
{
	GENERATED_UCLASS_BODY()

};


