// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactoryVolume.h"
#include "ActorFactorySphereVolume.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config=Editor, collapsecategories, hidecategories=Object)
class UActorFactorySphereVolume : public UActorFactoryVolume
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
	//~ End UActorFactory Interface
};
