// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "GameFramework/Volume.h"
#include "ActorFactoryVolume.generated.h"

UCLASS(Abstract, MinimalAPI, config=Editor, collapsecategories, hidecategories=Object, Abstract)
class UActorFactoryVolume : public UActorFactory
{
	GENERATED_BODY()

public:
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override
	{
		if (UActorFactory::CanCreateActorFrom(AssetData, OutErrorMsg))
		{
			return true;
		}

		if (AssetData.IsValid() && !AssetData.IsInstanceOf(AVolume::StaticClass()))
		{
			return false;
		}

		return true;
	}
};
