// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// AnalyticsPrivacySettings
//
// A configuration class that holds information for the user's privacy settings.
// Supplied so that the editor 'remembers' the last setup the user had.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/ImportantToggleSettingInterface.h"
#include "AnalyticsPrivacySettings.generated.h"

UCLASS(MinimalAPI, hidecategories=Object, config=EditorSettings)
class UAnalyticsPrivacySettings : public UObject, public IImportantToggleSettingInterface
{
	GENERATED_UCLASS_BODY()

	/** Determines whether the editor sends usage information to Epic Games in order to improve Unreal Engine. Your information will never be shared with 3rd parties. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	bool bSendUsageData;

	/** Determines whether the editor asks for your industry and sends that information to Epic Games in order to improve Unreal Engine. Your information will never be shared with 3rd parties. 
	* As Unreal Engine is used across a range of industries, we want to make sure we are addressing the needs of those verticals. A one-time prompt will show up for a user asking if they 
	* are using the editor for the field of: Game Development, Film & TV, Architecture, Automotive & Manufacturing, Broadcasting & Live Events, Advertising & Marketing, Simulation & Training, or Other.
	* 
	* If you would like to block the prompt from showing up for members of your team, but otherwise send usage data back to Epic Games, you can add the following to your BaseEditorSettings.ini:
	* [/Script/UnrealEd.AnalyticsPrivacySettings]
	* bSuppressIndustryPopup=True
	* 
	* Additionally, for users who do not wish to send this information back, simply close the pop-up and it will not be shown again.
	*/
	UPROPERTY(EditAnywhere, config, Category=Options)
	bool bSuppressIndustryPopup;

public:

	// BEGIN IImportantToggleSettingInterface
	virtual void GetToggleCategoryAndPropertyNames(FName& OutCategory, FName& OutProperty) const override;
	virtual FText GetFalseStateLabel() const override;
	virtual FText GetFalseStateTooltip() const override;
	virtual FText GetFalseStateDescription() const override;
	virtual FText GetTrueStateLabel() const override;
	virtual FText GetTrueStateTooltip() const override;
	virtual FText GetTrueStateDescription() const override;
	virtual FString GetAdditionalInfoUrl() const override;
	virtual FText GetAdditionalInfoUrlLabel() const override;
	// END IImportantToggleSettingInterface

#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	void OnSendFullUsageDataChanged();
#endif
};
