// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CrashReportsPrivacySettings
//
// A configuration class that holds information for the user's privacy settings.
// Supplied so that the editor 'remembers' the last setup the user had.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/ImportantToggleSettingInterface.h"
#include "CrashReportsPrivacySettings.generated.h"

UCLASS(MinimalAPI, hidecategories=Object, config=EditorSettings)
class UCrashReportsPrivacySettings : public UObject, public IImportantToggleSettingInterface
{
	GENERATED_UCLASS_BODY()

	/** Determines whether the editor automatically sends some bug reports Epic Games in order to improve Unreal Engine. Your information will never be shared with 3rd parties. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	bool bSendUnattendedBugReports;

public:

	// BEGIN IImportantToggleSettingInterface
	virtual void GetToggleCategoryAndPropertyNames(FName& OutCategory, FName& OutProperty) const override;
	virtual FText GetFalseStateLabel() const override;
	virtual FText GetFalseStateTooltip() const override;
	virtual FText GetFalseStateDescription() const override;
	virtual FText GetTrueStateLabel() const override;
	virtual FText GetTrueStateTooltip() const override;
	virtual FText GetTrueStateDescription() const override;
	virtual FString GetAdditionalInfoUrl() const override;
	virtual FText GetAdditionalInfoUrlLabel() const override;
	// END IImportantToggleSettingInterface

#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif
};
