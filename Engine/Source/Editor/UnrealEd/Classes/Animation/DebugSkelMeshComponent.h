// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/SkeletalMesh.h"
#include "EngineDefines.h"
#include "Components/SkeletalMeshComponent.h"
#include "Delegates/DelegateCombinations.h"
#include "SkeletalMeshSceneProxy.h"
#include "NaniteSceneProxy.h"
#include "DebugSkelMeshComponent.generated.h"

class Error;

DECLARE_DELEGATE_RetVal(FText, FGetExtendedViewportText);
DECLARE_DELEGATE(FOnDebugForceLODChanged);

USTRUCT()
struct FSelectedSocketInfo
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FSelectedSocketInfo()
		: Socket(nullptr)
		, bSocketIsOnSkeleton( false )
	{

	}

	/** Constructor */
	FSelectedSocketInfo( class USkeletalMeshSocket* InSocket, bool bInSocketIsOnSkeleton )
		: Socket( InSocket )
		, bSocketIsOnSkeleton( bInSocketIsOnSkeleton )
	{

	}

	bool IsValid() const
	{
		return Socket != nullptr;
	}

	void Reset()
	{
		Socket = nullptr;
	}

	/** The socket we have selected */
	class USkeletalMeshSocket* Socket;

	/** true if on skeleton, false if on mesh */
	bool bSocketIsOnSkeleton;
};

/** Different modes for Persona's Turn Table. */
namespace EPersonaTurnTableMode
{
	enum Type
	{
		Stopped,
		Playing,
		Paused
	};
};

/** Different modes for when processing root motion */
UENUM()
enum class EProcessRootMotionMode : uint8
{
	/** Preview mesh will not consume root motion */
	Ignore,

	/** Preview mesh will consume root motion continually */
	Loop,

	/** Preview mesh will consume root motion resetting the position back to the origin every time the animation loops */
	LoopAndReset
};

/** Different modes for visualizing root motion */
UENUM()
enum class EVisualizeRootMotionMode : uint8
{
	/** Preview will not show root motion */
	None,

	/** Preview will show root motion path */
	Trajectory,

	/** Preview will show root motion path and orientation. */
	TrajectoryAndOrientation
};

//////////////////////////////////////////////////////////////////////////
// FDebugSkelMeshSceneProxy

class UDebugSkelMeshComponent;
class FSkeletalMeshRenderData;

class FDebugSkelMeshDynamicData
{
public:

	FDebugSkelMeshDynamicData(UDebugSkelMeshComponent* InComponent);

	bool bDrawMesh;
	bool bDrawNormals;
	bool bDrawTangents;
	bool bDrawBinormals;
	bool bDrawClothPaintPreview;

	bool bFlipNormal;

	bool bDoubleSidedOpaqueCloth;
	int32 ClothingSimDataIndexWhenPainting;
	TArray<uint32> ClothingSimIndices;

	TArray<float> ClothingVisiblePropertyValues;
	float PropertyViewMin;
	float PropertyViewMax;

	TArray<FVector3f> SkinnedPositions;
	TArray<FVector3f> SkinnedNormals;
};

/**
 * Data and interface common to both Nanite and non-Nanite scene proxies
 */
class FDebugSkelMeshSceneProxyCommon
{
public:
	void GetDynamicMeshElements(const FPrimitiveSceneProxy* SceneProxy, const FSkeletalMeshObject* MeshObject, const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const;

	FDebugSkelMeshDynamicData* DynamicData = nullptr;
};

/**
* A skeletal mesh component scene proxy with additional debugging options.
*/
class FDebugSkelMeshSceneProxy : public FSkeletalMeshSceneProxy, public FDebugSkelMeshSceneProxyCommon
{
public:
	FDebugSkelMeshSceneProxy(const UDebugSkelMeshComponent* InComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, FLinearColor InWireframeOverlayColor = FLinearColor::White);
	virtual ~FDebugSkelMeshSceneProxy()	= default;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	SIZE_T GetTypeHash() const override;

	SIZE_T GetAllocatedSize() const
	{
		return FSkeletalMeshSceneProxy::GetAllocatedSize();
	}

	virtual uint32 GetMemoryFootprint() const override
	{
		return sizeof(*this) + GetAllocatedSize();
	}
};

class FDebugSkelMeshSceneProxyNanite : public Nanite::FSkinnedSceneProxy, public FDebugSkelMeshSceneProxyCommon
{
public:
	FDebugSkelMeshSceneProxyNanite(const Nanite::FMaterialAudit& Materials, const UDebugSkelMeshComponent* InComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, FLinearColor InWireframeOverlayColor = FLinearColor::White);
	virtual ~FDebugSkelMeshSceneProxyNanite()	= default;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	SIZE_T GetTypeHash() const override;

	SIZE_T GetAllocatedSize() const
	{
		return Nanite::FSkinnedSceneProxy::GetAllocatedSize();
	}

	virtual uint32 GetMemoryFootprint() const override
	{
		return sizeof(*this) + GetAllocatedSize();
	}
};

/** Generic modes used to render debug skeletons depending on editor-specific context */
UENUM()
enum class ESkeletonDrawMode : uint8
{
	/** Bones are visible and selectable */
	Default,

	/** Bones are completely hidden */
	Hidden,

	/** Bones are visible but non-selectable */
	GreyedOut
};

UCLASS(transient, MinimalAPI)
class UDebugSkelMeshComponent : public USkeletalMeshComponent
{
	GENERATED_UCLASS_BODY()
	
	/** Global drawing mode for this skeleton. Depends on context of specific editor using the component. */
	UPROPERTY()
	ESkeletonDrawMode SkeletonDrawMode = ESkeletonDrawMode::Default;

	/** If true then the skeletal mesh associated with the component is drawn. */
	UPROPERTY()
	uint32 bDrawMesh:1;
	
	/** If true then the bone names associated with the skeletal mesh are displayed */
	UPROPERTY()
	uint32 bShowBoneNames:1;

	/** Bone influences viewing */
	UPROPERTY(transient)
	uint32 bDrawBoneInfluences:1;

	/** Morphtarget viewing */
	UPROPERTY(transient)
	uint32 bDrawMorphTargetVerts : 1;

	/** Vertex normal viewing */
	UPROPERTY(transient)
	uint32 bDrawNormals:1;

	/** Vertex tangent viewing */
	UPROPERTY(transient)
	uint32 bDrawTangents:1;

	/** Vertex binormal viewing */
	UPROPERTY(transient)
	uint32 bDrawBinormals:1;

	/** Socket hit points viewing */
	UPROPERTY(transient)
	uint32 bDrawSockets:1;

	/** Attribute visualization */
	UPROPERTY(transient)
	uint32 bDrawAttributes : 1;

	/** Skeleton sockets visible? */
	UPROPERTY(transient)
	uint32 bSkeletonSocketsVisible:1;

	/** Mesh sockets visible? */
	UPROPERTY(transient)
	uint32 bMeshSocketsVisible:1;

	/** Display raw animation bone transform */
	UPROPERTY(transient)
	uint32 bDisplayRawAnimation:1;

	/** Display non retargeted animation pose */
	UPROPERTY(Transient)
	uint32 bDisplayNonRetargetedPose:1;

	/** Display additive base bone transform */
	UPROPERTY(transient)
	uint32 bDisplayAdditiveBasePose:1;

	/** Display baked animation pose */
	UPROPERTY(Transient)
	uint32 bDisplayBakedAnimation:1;

	/** Display source animation pose */
	UPROPERTY(Transient)
	uint32 bDisplaySourceAnimation:1;

	/** Display Notification visualizations in viewport */
	UPROPERTY(Transient)
	uint32 bShowNotificationVisualizations:1;

	/** Display Metadata visualizations in viewport */
	UPROPERTY(Transient)
	uint32 bShowAssetUserDataVisualizations:1;

	/** Display Bound **/
	UPROPERTY(transient)
	bool bDisplayBound;

	UPROPERTY(transient)
	bool bDisplayVertexColors;

	UPROPERTY(transient)
	FLinearColor WireframeMeshOverlayColor;

	/** Requested Process root motion mode, ProcessRootMotionMode gets set based on requested mode and what is supported. */
	UPROPERTY(transient)
	EProcessRootMotionMode RequestedProcessRootMotionMode;

	/** Process root motion mode */
	UPROPERTY(transient)
	EProcessRootMotionMode ProcessRootMotionMode;

	UPROPERTY(transient)
	EVisualizeRootMotionMode VisualizeRootMotionMode;

	/** Playback time last time ConsumeRootmotion was called */
	UPROPERTY(transient)
	float ConsumeRootMotionPreviousPlaybackTime;
	
	UPROPERTY(transient)
	TEnumAsByte<EAdditiveAnimationType> ConsumeRootMotionPreviousAdditiveAnimType = AAT_MAX;

	UPROPERTY(transient)
	uint32 bShowClothData : 1;

	UPROPERTY(transient)
	float MinClothPropertyView;

	UPROPERTY(transient)
	float MaxClothPropertyView;

	UPROPERTY(transient)
	float ClothMeshOpacity;

	UPROPERTY(transient)
	bool bClothFlipNormal;

	UPROPERTY(transient)
	bool bClothCullBackface;

	UPROPERTY(transient)
	uint32 bRequiredBonesUpToDateDuringTick : 1;

	/* Bounds computed from cloth. */
	FBoxSphereBounds CachedClothBounds;

	/** Non Compressed SpaceBases for when bDisplayRawAnimation == true **/
	TArray<FTransform> UncompressedSpaceBases;

	/** Storage of Additive Base Pose for when bDisplayAdditiveBasePose == true, as they have to be calculated */
	TArray<FTransform> AdditiveBasePoses;

	/** Storage for non retargeted pose. */
	TArray<FTransform> NonRetargetedSpaceBases;

	/** Storage of Baked Animation Pose for when bDisplayBakedAnimation == true, as they have to be calculated */
	TArray<FTransform> BakedAnimationPoses;

	/** Storage of Source Animation Pose for when bDisplaySourceAnimation == true, as they have to be calculated */
	TArray<FTransform> SourceAnimationPoses;

	/** Transform representing the actor transform at the beginning of the animation sequence. */
	FTransform RootMotionReferenceTransform;

	/** Array of bones to render bone weights for */
	UPROPERTY(transient)
	TArray<int32> BonesOfInterest;

	/** Array of morphtargets to render verts for */
	UPROPERTY(transient)
	TArray<TObjectPtr<class UMorphTarget>> MorphTargetOfInterests;

	/** Array of materials to restore when not rendering blend weights */
	UPROPERTY(transient)
	TArray<TObjectPtr<class UMaterialInterface>> SkelMaterials;
	
	UPROPERTY(transient, NonTransactional)
	TObjectPtr<class UAnimPreviewInstance> PreviewInstance;

	UPROPERTY(transient)
	TObjectPtr<class UAnimInstance> SavedAnimScriptInstance;

	/** Does this component use in game bounds or does it use bounds calculated from bones */
	UPROPERTY(transient)
	bool bIsUsingInGameBounds;
	
	/** Does this component use pre-skinned bounds? This overrides other bounds settings */
	UPROPERTY(transient)
	bool bIsUsingPreSkinnedBounds;

	/** Base skel mesh has support for suspending clothing, but single ticks are more of a debug feature when stepping through an animation
	 *  So we control that using this flag
	 */
	UPROPERTY(transient)
	bool bPerformSingleClothingTick;

	UPROPERTY(transient)
	bool bPauseClothingSimulationWithAnim;

	/** Should the LOD of the debug mesh component track the LOD of the instance being debugged */
	UPROPERTY(transient)
	bool bTrackAttachedInstanceLOD;
	
	/** Palettized semi randomized colors for multi-color bone display */
	UNREALED_API FLinearColor GetBoneColor(int32 BoneIndex) const;

	// Helper method that sets the forced lod
	UNREALED_API void SetDebugForcedLOD(int32 InNewForcedLOD);

	//~ Begin USceneComponent Interface.
	UNREALED_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	UNREALED_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	
	// engine only draw bounds IF selected
	// @todo fix this properly
	// this isn't really the best way to do this, but for now
	// we'll just mark as selected
	UNREALED_API virtual bool ShouldRenderSelected() const override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin SkinnedMeshComponent Interface
	UNREALED_API virtual bool ShouldCPUSkin() const override;
	UNREALED_API virtual bool ShouldNaniteSkin() override;
	UNREALED_API virtual void PostInitMeshObject(class FSkeletalMeshObject* MeshObject) override;
	UNREALED_API virtual void RefreshBoneTransforms(FActorComponentTickFunction* TickFunction = NULL) override;
	virtual int32 GetLODBias() const override { return 0; }
	//~ End SkinnedMeshComponent Interface

	//~ Begin SkeletalMeshComponent Interface
	UNREALED_API virtual void InitAnim(bool bForceReinit) override;
	virtual bool IsWindEnabled() const override { return true; }
	UNREALED_API virtual void OnClearAnimScriptInstance() override;
	UNREALED_API virtual void SetSkeletalMesh(USkeletalMesh* InSkelMesh, bool bReinitPose = true) override;
	//~ End SkeletalMeshComponent Interface

	//~ Begin UObject interface
	UNREALED_API virtual void PostInitProperties() override;
	//~ End UObject interface

	// return true if currently preview animation asset is on
	UNREALED_API virtual bool IsPreviewOn() const;

	// @todo document
	UNREALED_API FString GetPreviewText() const;

	// @todo anim : you still need to give asset, so that we know which one to disable
	// we can disable per asset, so that if some other window disabled before me, I don't accidently turn it off
	UNREALED_API virtual void EnablePreview(bool bEnable, class UAnimationAsset * PreviewAsset);

	// Create the preview instance to use (default UAnimPreviewInstance)
	UNREALED_API virtual TObjectPtr<UAnimPreviewInstance> CreatePreviewInstance();

	// reference pose for this component
	// we don't want to use default refpose because you still want to move joint when this mode is on
	UNREALED_API virtual void ShowReferencePose(bool bRefPose);
	UNREALED_API virtual bool IsReferencePoseShown() const;

	/** Called when mirror data table changes on anim instance. */
	UNREALED_API void OnMirrorDataTableChanged();

	/**
	 * Update material information depending on color render mode 
	 * Refresh/replace materials 
	 */
	UNREALED_API void SetShowBoneWeight(bool bNewShowBoneWeight);

	/**
	* Update material information depending on color render mode
	* Refresh/replace materials
	*/
	UNREALED_API void SetShowMorphTargetVerts(bool bNewShowMorphTargetVerts);

	/**
	 * Does it use in-game bounds or bounds calculated from bones
	 */
	UNREALED_API bool IsUsingInGameBounds() const;

	/**
	 * Set to use in-game bounds or bounds calculated from bones
	 */
	UNREALED_API void UseInGameBounds(bool bUseInGameBounds);

	/**
	 * Does it use pre-skinned bounds
	 */
	UNREALED_API bool IsUsingPreSkinnedBounds() const;

	/**
	 * Set to use pre-skinned bounds
	 */
	UNREALED_API void UsePreSkinnedBounds(bool bUsePreSkinnedBounds);

	/**
	 * Test if in-game bounds are as big as preview bounds
	 */
	UNREALED_API bool CheckIfBoundsAreCorrrect();

	/** Get the in-game bounds of the skeleton mesh */
	UNREALED_API FBoxSphereBounds CalcGameBounds(const FTransform& LocalToWorld) const;

	/** 
	 * Update components position based on animation root motion
	 */
	UNREALED_API void ConsumeRootMotion(const FVector& FloorMin, const FVector& FloorMax);

	/** Sets the flag used to determine whether or not the current active cloth sim mesh should be rendered */
	UNREALED_API void SetShowClothProperty(bool bState);

	/** Whether we are processing root motion or not */
	UNREALED_API bool IsProcessingRootMotion() const;

	/** Gets requested process root motion mode, can differ from GetProcessRootMotionMode() if the current asset does not support root motion. */
	UNREALED_API EProcessRootMotionMode GetRequestedProcessRootMotionMode() const;

	/** Gets process root motion mode */
	UNREALED_API EProcessRootMotionMode GetProcessRootMotionMode() const;

	/** Sets process root motion mode, the request may be ignored if current asset does not support the mode. Note: disabling root motion preview resets transform. */
	UNREALED_API void SetProcessRootMotionMode(EProcessRootMotionMode Mode);

	/** Sets how we visualize root motion in the viewport. See EVisualizeRootMotionMode for details. */
	void SetVisualizeRootMotionMode(EVisualizeRootMotionMode Mode) { VisualizeRootMotionMode = Mode; };
	bool IsVisualizeRootMotionMode(EVisualizeRootMotionMode Mode) const { return VisualizeRootMotionMode == Mode; };
	EVisualizeRootMotionMode GetVisualizeRootMotionMode() const { return VisualizeRootMotionMode; };

	/** Whether the supplied root motion mode can be used for the current asset */
	UNREALED_API bool CanUseProcessRootMotionMode(EProcessRootMotionMode Mode) const;

	/** Whether the current asset or animation blueprint is using root motion */
	UNREALED_API bool DoesCurrentAssetHaveRootMotion() const;

	/** Sets flags whether we notification visualizations should be drawn in the viewport. */
	void SetShowNotificationVisualizations(const bool bShow) { bShowNotificationVisualizations = bShow; }
	bool IsNotificationVisualizationsEnabled() const { return bShowNotificationVisualizations; }

	bool IsRootMotionVisualizationsEnabled() const { return VisualizeRootMotionMode != EVisualizeRootMotionMode::None; }

	/** Sets flags whether we AssetUserData visualizations should be drawn in the viewport. */
	void SetShowAssetUserDataVisualizations(const bool bShow) { bShowAssetUserDataVisualizations = bShow; }
	bool IsAssetUserDataVisualizationsEnabled() const { return bShowAssetUserDataVisualizations; }

	/** Whether the current LOD of the debug mesh is being synced with the attached (preview) mesh instance. */
	UNREALED_API bool IsTrackingAttachedLOD() const;

	/** Set the wireframe mesh overlay color, which basically controls the color of the wireframe. */
	void SetWireframeMeshOverlayColor(FLinearColor Color) { WireframeMeshOverlayColor = Color; }

	/** Get the wireframe mesh overlay color, which basically controls the color of the wireframe. */
	FLinearColor GetWireframeMeshOverlayColor() const { return WireframeMeshOverlayColor; }


#if WITH_EDITOR
	//TODO - This is a really poor way to post errors to the user. Work out a better way.
	struct FAnimNotifyErrors
	{
		FAnimNotifyErrors(UObject* InSourceNotify)
		: SourceNotify(InSourceNotify)
		{}
		UObject* SourceNotify;
		TArray<FString> Errors;
	};
	TArray<FAnimNotifyErrors> AnimNotifyErrors;
	UNREALED_API virtual void ReportAnimNotifyError(const FText& Error, UObject* InSourceNotify) override;
	UNREALED_API virtual void ClearAnimNotifyErrors(UObject* InSourceNotify) override;

	/** 
	 * Extended viewport text delegate handling. Registering a delegate allows external
	 * objects to place custom text in the anim tools viewports.
	 */
	UNREALED_API FDelegateHandle RegisterExtendedViewportTextDelegate(const FGetExtendedViewportText& InDelegate);
	UNREALED_API void UnregisterExtendedViewportTextDelegate(const FDelegateHandle& InDelegateHandle);
	const TArray<FGetExtendedViewportText>& GetExtendedViewportTextDelegates() const { return ExtendedViewportTextDelegates; }

	UNREALED_API FDelegateHandle RegisterOnDebugForceLODChangedDelegate(const FOnDebugForceLODChanged& InDelegate);
	UNREALED_API void UnregisterOnDebugForceLODChangedDelegate();

private:
	TArray<FGetExtendedViewportText> ExtendedViewportTextDelegates;
	FOnDebugForceLODChanged OnDebugForceLODChangedDelegate;
public:

#endif

	/**
	 * Force all body instance to not simulate physics regardless of their physic type
	 * To get back in a default state call : ResetAllBodiesSimulatePhysics
	 */
	void DisableAllBodiesSimulatePhysics()
	{
		for (int32 BodyIdx = 0; BodyIdx < Bodies.Num(); ++BodyIdx)
		{
			if (FBodyInstance* BodyInst = Bodies[BodyIdx])
			{
				BodyInst->SetInstanceSimulatePhysics(false);
			}
		}
	}

	/** 
	 * toggle visibility between cloth sections and non-cloth sections for all LODs
	 * if bShowOnlyClothSections is true, shows only cloth sections. On the other hand, 
	 * if bShowOnlyClothSections is false, hides only cloth sections.
	 */
	UNREALED_API void ToggleClothSectionsVisibility(bool bShowOnlyClothSections);
	/** Restore all section visibilities to original states for all LODs */
	UNREALED_API void RestoreClothSectionsVisibility();

	/** 
	 * To normal game/runtime code we don't want to expose a non-const pointer to the simulation, so we can only get
	 * one from this editor-only component. Intended for debug options/visualisations/editor-only code to poke the sim
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
	UNREALED_API IClothingSimulation* GetMutableClothingSimulation();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** to avoid clothing reset while modifying properties in Persona */
	UNREALED_API virtual void CheckClothTeleport() override;

	/** The currently selected asset guid if we're painting, used to build dynamic mesh to paint sim parameters */
	FGuid SelectedClothingGuidForPainting;

	/** The currently selected LOD for painting */
	int32 SelectedClothingLodForPainting;

	/** The currently selected mask inside the above LOD to be painted */
	int32 SelectedClothingLodMaskForPainting;

	/** Find a section using a clothing asset with the given GUID and set its visiblity */
	UNREALED_API void SetMeshSectionVisibilityForCloth(FGuid InClothGuid, bool bVisibility);

	// fixes up the disabled flags so clothing is enabled and originals are disabled as
	// ToggleMeshSectionForCloth will make these get out of sync
	UNREALED_API void ResetMeshSectionVisibility();

	TArray<FVector3f> SkinnedSelectedClothingPositions;
	TArray<FVector3f> SkinnedSelectedClothingNormals;

private:
	// Helper function to generate space bases for current frame
	UNREALED_API void GenSpaceBases(TArray<FTransform>& OutSpaceBases);

	// Helper function to enable overlay material
	UNREALED_API void EnableOverlayMaterial(bool bEnable);

	// Rebuilds the cloth bounds for the asset.
	UNREALED_API void RebuildCachedClothBounds();

	UNREALED_API void SetProcessRootMotionModeInternal(EProcessRootMotionMode Mode);

	void ResetRootTransform();
protected:

	// Overridden to support single clothing ticks
	UNREALED_API virtual bool ShouldRunClothTick() const override;

	UNREALED_API virtual void SendRenderDynamicData_Concurrent() override;

public:

	/** Set TurnTableMode and reset the turntable rotation if needed. */
	UNREALED_API void SetTurnTableMode(EPersonaTurnTableMode::Type NewMode);
	
	/** Current turn table mode */
	EPersonaTurnTableMode::Type TurnTableMode;
	/** Current turn table speed scaling */
	float TurnTableSpeedScaling;

	UNREALED_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	UNREALED_API void RefreshSelectedClothingSkinnedPositions();

	virtual bool CanOverrideCollisionProfile() const { return true; }

	UNREALED_API virtual void GetUsedMaterials(TArray<UMaterialInterface *>& OutMaterials, bool bGetDebugMaterials = false) const override;

	/**
	 * Define Custom Default pose for this component for preview
	 */
	virtual void SetCustomDefaultPose() {};

	/*
	 *	return RefSkeleton for drawing; 
	 */
	virtual const FReferenceSkeleton& GetReferenceSkeleton() const
	{
		if (GetSkeletalMeshAsset())
		{
			return GetSkeletalMeshAsset()->GetRefSkeleton();
		}

		static FReferenceSkeleton EmptySkeleton;
		return EmptySkeleton;
	}

	/*
	*	return bone indices to draw
	*/
	virtual const TArray<FBoneIndexType>& GetDrawBoneIndices() const
	{
		return RequiredBones;
	}

	virtual int32 GetNumDrawTransform() const 
	{
		return GetNumComponentSpaceTransforms();
	}
	/*
	 *	returns the transform of the joint 
	 */
	virtual FTransform GetDrawTransform(int32 BoneIndex) const
	{
		const TArray<FTransform>& SpaceTransforms = GetComponentSpaceTransforms();
		if (SpaceTransforms.IsValidIndex(BoneIndex))
		{
			return SpaceTransforms[BoneIndex];
		}

		return FTransform::Identity;
	}

	/** Whether this component renders the Nanite fallback for previewing */
	bool IsDisplayingNaniteFallback() const
	{
		return bDisplayNaniteFallback;
	}

	/** Sets whether this component should render the Nanite fallback for previewing */
	void SetDisplayNaniteFallback(bool bValue)
	{
		if (bValue != bDisplayNaniteFallback)
		{
			bDisplayNaniteFallback = bValue;
			MarkRenderStateDirty();
		}
	}

private:
	bool bDisplayNaniteFallback = false;
};


/*
 * This class is use to remove the alternate skinning preview from the multiple editor that can show it.
 * Important it should be destroy after the PostEditChange of the skeletalmesh is done and the renderdata have been recreate
 * i.e. FScopedSkeletalMeshPostEditChange should be create after FScopedSuspendAlternateSkinWeightPreview and delete before FScopedSuspendAlternateSkinWeightPreview
 */
class FScopedSuspendAlternateSkinWeightPreview
{
public:
	/*
	 * This constructor suspend the alternate skinning preview for all editor component that use the specified skeletalmesh
	 * Parameters:
	 * @param InSkeletalMesh - SkeletalMesh use to know which preview component we have to suspend the alternate skinning preview.
	 */
	UNREALED_API FScopedSuspendAlternateSkinWeightPreview(class USkeletalMesh* InSkeletalMesh);

	/*
	 * This destructor put back the preview alternate skinning
	 */
	UNREALED_API ~FScopedSuspendAlternateSkinWeightPreview();

private:
	TArray< TTuple<UDebugSkelMeshComponent*, FName> > SuspendedComponentArray;
};
