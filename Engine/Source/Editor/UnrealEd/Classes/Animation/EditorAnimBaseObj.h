// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Abstract base class of animation composite base
 * This contains Composite Section data and some necessary interface to make this work
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EditorAnimBaseObj.generated.h"

class UAnimSequenceBase;
struct FPropertyChangedEvent;

DECLARE_DELEGATE_TwoParams( FOnAnimObjectChange, class UObject*, bool)


UCLASS(abstract, MinimalAPI)
class UEditorAnimBaseObj: public UObject
{
	GENERATED_UCLASS_BODY()
public:
	

	virtual void InitFromAnim(UAnimSequenceBase* AnimObjectIn, FOnAnimObjectChange OnChange);
	virtual bool ApplyChangesToMontage();

	UAnimSequenceBase* AnimObject;
	FOnAnimObjectChange OnChange;

	using Super::PreEditChange;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	virtual bool PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent) { return true;}
};
