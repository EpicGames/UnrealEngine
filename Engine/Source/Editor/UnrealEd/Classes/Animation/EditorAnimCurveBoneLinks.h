// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Editor only class for UI of linking animation curve to joints
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BoneContainer.h"
#include "Animation/SmartName.h"
#include "EditorAnimCurveBoneLinks.generated.h"

class IEditableSkeleton;
struct FPropertyChangedEvent;

DECLARE_DELEGATE_OneParam( FOnAnimCurveBonesChange, class UEditorAnimCurveBoneLinks*)

UCLASS(MinimalAPI)
class UEditorAnimCurveBoneLinks: public UObject
{
	GENERATED_UCLASS_BODY()
public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.3, "Please use Initialize that takes a FName")
	virtual void Initialize(const TWeakPtr<class IEditableSkeleton> InEditableSkeleton, const FSmartName& InCurveName, FOnAnimCurveBonesChange OnChangeIn);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Initialize(UAnimCurveMetaData* InAnimCurveMetaData, const FName& InCurveName, FOnAnimCurveBonesChange OnChangeIn);

	TWeakObjectPtr<UAnimCurveMetaData> AnimCurveMetaData;
	FOnAnimCurveBonesChange OnChange;

	UPROPERTY(VisibleAnywhere, Category = Curve)
	FName CurveName;

	UPROPERTY(EditAnywhere, Category = Curve)
	TArray<FBoneReference> ConnectedBones;

	/** Max (Lowest) LOD to evaluate to curve. 
	 *  Since LOD goes from 0 to high number, we call it Max. 
	 *  For example, if you have 3 LODs (0, 1, 2), and if you want this to work until LOD 1, type 1.  
	 *  Then the curve will be evaluated until LOD1, but not for LOD 2
	 *  Default value is 255 */
	UPROPERTY(EditAnywhere, Category = Curve)
	uint8 MaxLOD;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// refresh current Connected Bones data
	UNREALED_API void Refresh(const FName& InCurveName, const TArray<FBoneReference>& CurrentLinks, uint8 InMaxLOD);
};
