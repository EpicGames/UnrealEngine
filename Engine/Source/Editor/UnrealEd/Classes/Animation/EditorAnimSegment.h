// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Abstract base class of animation composite base
 * This contains Composite Section data and some necessary interface to make this work
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/EditorAnimBaseObj.h"
#include "Animation/AnimCompositeBase.h"
#include "EditorAnimSegment.generated.h"

DECLARE_DELEGATE_OneParam( FOnAnimSegmentChanged, class UEditorAnimSegment*)


UCLASS(hidecategories=UObject, MinimalAPI, BlueprintType)
class UEditorAnimSegment: public UEditorAnimBaseObj
{
	GENERATED_UCLASS_BODY()
public:

	/** Default blend in time. */
	UPROPERTY(EditAnywhere, Category=Montage)
	FAnimSegment AnimSegment;

	int AnimSlotIndex;
	int AnimSegmentIndex;

	virtual void InitAnimSegment(int32 AnimSlotIndex, int32 AnimSegmentIndex);
	virtual bool ApplyChangesToMontage() override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent) override;
};
