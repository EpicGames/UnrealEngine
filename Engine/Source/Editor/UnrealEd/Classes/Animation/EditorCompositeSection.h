// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Abstract base class of animation composite base
 * This contains Composite Section data and some necessary interface to make this work
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Animation/EditorAnimBaseObj.h"
#include "Animation/AnimMontage.h"
#include "EditorCompositeSection.generated.h"

UCLASS(hidecategories=UObject, MinimalAPI, BlueprintType)
class UEditorCompositeSection: public UEditorAnimBaseObj
{
	GENERATED_UCLASS_BODY()
public:

	/** Default blend in time. */
	UPROPERTY(EditAnywhere, Category=Montage, meta=(ShowOnlyInnerProperties))
	FCompositeSection CompositeSection;

	int32 SectionIndex;
	
	virtual void InitSection(int32 SectionIndex);
	virtual bool ApplyChangesToMontage() override;
};
