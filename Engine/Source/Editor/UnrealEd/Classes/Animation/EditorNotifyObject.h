// Copyright Epic Games, Inc. All Rights Reserved.

//////////////////////////////////////////////////////////////////////////
// Proxy object for displaying notifies in the details panel with
// event data included alongside UAnimNotify 
//////////////////////////////////////////////////////////////////////////

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/EditorAnimBaseObj.h"
#include "EditorNotifyObject.generated.h"

UCLASS(MinimalAPI)
class UEditorNotifyObject : public UEditorAnimBaseObj
{
	GENERATED_UCLASS_BODY()

	/** Set up the editor object
	 *	@param InNotify		The notify to modify
	 */
	virtual void InitialiseNotify(const FAnimNotifyEvent& InNotify);
	
	/** Copy changes made to the event object back to the montage asset */
	virtual bool ApplyChangesToMontage() override;
	virtual bool PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** The notify event to modify */
	UPROPERTY(EditAnywhere, Category=Event)
	FAnimNotifyEvent Event;
	
	/** 
	 * Query and cache the generated notify name.
	 * @return true if the name changed from previous cached value.
	 */
	bool TryToCacheNotifyName();
	
	/** Used to determine if we should rebuild the notify ui when its properties are changed. If it affects the name is the exception and a rebuild will be kicked. */
	FName CachedNotifyName;
};
