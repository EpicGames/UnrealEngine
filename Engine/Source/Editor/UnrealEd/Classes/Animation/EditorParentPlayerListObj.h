// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Animation/AnimBlueprint.h"
#include "EditorParentPlayerListObj.generated.h"

class UAnimGraphNode_Base;

UCLASS(MinimalAPI)
class UEditorParentPlayerListObj : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	// Build a possible list of overrides from a blueprint and pull in any existing overrides.
	UNREALED_API void InitialiseFromBlueprint(UAnimBlueprint* Blueprint);

	// Adds an overridable node to the possible override list.
	UNREALED_API FAnimParentNodeAssetOverride& AddOverridableNode(UAnimGraphNode_Base* Node);

	// Gets a pointer to a U-node from the stored GUID for that node.
	UNREALED_API UAnimGraphNode_Base* GetVisualNodeFromGuid(FGuid InGuid) const;

	// Pushes an override into the blueprint we're editing.
	UNREALED_API void ApplyOverrideToBlueprint(FAnimParentNodeAssetOverride& Override);

	// Get the current blueprint
	UNREALED_API const UAnimBlueprint* GetBlueprint() const;

	// List of possible overrides to display.
	UPROPERTY(Category = AnimationGraphOverrides, EditAnywhere)
	TArray<FAnimParentNodeAssetOverride> Overrides;

private:
	UAnimBlueprint* AnimBlueprint;
	TMap<FGuid, UAnimGraphNode_Base*> GuidToVisualNodeMap;
};
