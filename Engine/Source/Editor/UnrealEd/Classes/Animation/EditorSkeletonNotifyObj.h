// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Proxy class for display skeleton notifies in the details panel
 */
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EditorSkeletonNotifyObj.generated.h"

class IEditableSkeleton;

DECLARE_DELEGATE_TwoParams( FOnAnimObjectChange, class UObject*, bool)

UCLASS(MinimalAPI)
class UEditorSkeletonNotifyObj : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	/** Current skeleton to try to grab objects from */
	TWeakPtr<IEditableSkeleton> EditableSkeleton;

	/** The name of the notify we represent */
	UPROPERTY(VisibleAnywhere, Category=SkeletonNotifies)
	FName Name;
};
