// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ConeBuilder: Builds a 3D cone brush, compatible with cylinder of same size.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "ConeBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Cone"))
class UConeBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UConeBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Distance from base to tip of cone */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Z;

	/** Distance from base to the tip of inner cone (when hollow) */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(EditCondition="Hollow"))
	float CapZ;

	/** Radius of cone */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float OuterRadius;

	/** Radius of inner cone (when hollow) */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(EditCondition="Hollow"))
	float InnerRadius;

	/** How many sides this cone should have */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "3", ClampMax = "500"))
	int32 Sides;

	UPROPERTY()
	FName GroupName;

	/** Whether to align the brush to a face */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 AlignToSide:1;

	/** Whether this is a hollow or solid cone */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 Hollow:1;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildCone( int32 Direction, bool InAlignToSide, int32 InSides, float InZ, float Radius, FName Item );
};



