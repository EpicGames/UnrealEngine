// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * CubeBuilder: Builds a 3D cube brush.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "CubeBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Box"))
class UCubeBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UCubeBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** The size of the cube in the X dimension */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float X;

	/** The size of the cube in the Y dimension */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Y;

	/** The size of the cube in the Z dimension */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Z;

	/** The thickness of the cube wall when hollow */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(EditCondition="Hollow"))
	float WallThickness;

	UPROPERTY()
	FName GroupName;

	/** Whether this is a hollow or solid cube */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 Hollow:1;

	/** Whether extra internal faces should be generated for each cube face */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 Tessellated:1;


	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildCube( int32 Direction, float dx, float dy, float dz, bool _tessellated );
};



