// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "CurvedStairBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew,meta=(DisplayName="Curved Stair"))
class UCurvedStairBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UCurvedStairBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** The radius of the inner curve of the stair */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 InnerRadius;

	/** The height of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepHeight;

	/** The width of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepWidth;

	/** The angle of the total arc described by this stair */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "360"))
	int32 AngleOfCurve;

	/** The number of steps */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "100"))
	int32 NumSteps;

	/** The distance below the first step */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	int32 AddToFirstStep;

	UPROPERTY()
	FName GroupName;

	/** Whether the stair curves clockwise or counter-clockwise */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 CounterClockwise:1;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildCurvedStair( int32 Direction );
};



