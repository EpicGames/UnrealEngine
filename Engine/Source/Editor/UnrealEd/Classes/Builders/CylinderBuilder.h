// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */


//~=============================================================================
// CylinderBuilder: Builds a 3D cylinder brush.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "CylinderBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Cylinder"))
class UCylinderBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UCylinderBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Distance from base to tip of cylinder */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Z;

	/** Radius of cylinder */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float OuterRadius;

	/** Radius of inner cylinder (when hollow) */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(EditCondition="Hollow"))
	float InnerRadius;

	/** How many sides this cylinder should have */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "3", ClampMax = "500"))
	int32 Sides;

	UPROPERTY()
	FName GroupName;

	/** Whether to align the brush to a face */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 AlignToSide:1;

	/** Whether this is a hollow or solid cylinder */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 Hollow:1;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildCylinder( int32 Direction, bool InAlignToSide, int32 InSides, float InZ, float Radius );
};



