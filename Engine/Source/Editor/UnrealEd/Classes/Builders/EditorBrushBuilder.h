// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/BrushBuilder.h"
#include "EditorBrushBuilder.generated.h"

class ABrush;

UCLASS(abstract, MinimalAPI)
class UEditorBrushBuilder : public UBrushBuilder
{
public:
	GENERATED_BODY()
public:

	UEditorBrushBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** UBrushBuilder interface */
	virtual void BeginBrush( bool InMergeCoplanars, FName InLayer ) override;
	virtual bool EndBrush( UWorld* InWorld, ABrush* InBrush ) override;
	virtual int32 GetVertexCount() const override;
	virtual FVector GetVertex( int32 i ) const override;
	virtual int32 GetPolyCount() const override;
	virtual bool BadParameters( const FText& msg ) override;
	virtual int32 Vertexv( FVector v ) override;
	virtual int32 Vertex3f( float X, float Y, float Z ) override;
	virtual void Poly3i( int32 Direction, int32 i, int32 j, int32 k, FName ItemName = NAME_None, bool bIsTwoSidedNonSolid = false ) override;
	virtual void Poly4i( int32 Direction, int32 i, int32 j, int32 k, int32 l, FName ItemName = NAME_None, bool bIsTwoSidedNonSolid = false ) override;
	virtual void PolyBegin( int32 Direction, FName ItemName = NAME_None ) override;
	virtual void Polyi( int32 i ) override;
	virtual void PolyEnd() override;
	UNREALED_API virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;

	/** UObject interface */
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
};



