// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LinearStairBuilder: Builds a Linear Staircase.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "LinearStairBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Linear Stair"))
class ULinearStairBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	ULinearStairBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** The length of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepLength;

	/** The height of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepHeight;

	/** The width of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepWidth;

	/** The number of steps */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "2", ClampMax = "45"))
	int32 NumSteps;

	/** The distance below the first step */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	int32 AddToFirstStep;

	UPROPERTY()
	FName GroupName;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface
};



