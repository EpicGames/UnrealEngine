// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SheetBuilder: Builds a simple sheet.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "SheetBuilder.generated.h"

class ABrush;

UENUM()
enum ESheetAxis : int
{
	AX_Horizontal,
	AX_XAxis,
	AX_YAxis,
	AX_MAX,
};

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Plane"))
class USheetBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	USheetBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 X;

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 Y;

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "100", UIMin = "1", UIMax = "20"))
	int32 XSegments;

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "100", UIMin = "1", UIMax = "20"))
	int32 YSegments;

	UPROPERTY(EditAnywhere, Category=BrushSettings)
	TEnumAsByte<enum ESheetAxis> Axis;

	UPROPERTY()
	FName GroupName;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface
};



