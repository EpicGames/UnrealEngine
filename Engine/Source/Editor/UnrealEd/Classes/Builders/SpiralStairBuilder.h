// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SpiralStairBuilder: Builds a spiral staircase.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "SpiralStairBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Spiral Stair"))
class USpiralStairBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	USpiralStairBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** The radius of the inner curve of the stair */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 InnerRadius;

	/** The width of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepWidth;

	/** The height of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepHeight;

	/** The thickness of each step */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1"))
	int32 StepThickness;

	/** The number of steps in one whole spiral rotation */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "3"))
	int32 NumStepsPer360;

	/** The total number of steps */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "100"))
	int32 NumSteps;

	UPROPERTY()
	FName GroupName;

	/** Whether the underside of the spiral is sloped or stepped */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 SlopedCeiling:1;

	/** Whether the surface of the spiral is sloped or stepped */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 SlopedFloor:1;

	/** Whether the stair curves clockwise or counter-clockwise */
	UPROPERTY(EditAnywhere, Category=BrushSettings)
	uint32 CounterClockwise:1;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildCurvedStair( int32 Direction );
};



