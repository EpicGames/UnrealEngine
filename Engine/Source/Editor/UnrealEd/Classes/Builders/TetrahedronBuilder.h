// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TetrahedronBuilder: Builds an octahedron (not tetrahedron) - experimental.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "TetrahedronBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Sphere"))
class UTetrahedronBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UTetrahedronBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** The radius of this sphere */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Radius;

	/** How many iterations this sphere uses to tessellate its geometry */
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "1", ClampMax = "5",DisplayName="Tessellation"))
	int32 SphereExtrapolation;

	UPROPERTY()
	FName GroupName;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void Extrapolate( int32 a, int32 b, int32 c, int32 Count, float InRadius );

	// @todo document
	virtual void BuildTetrahedron( float R, int32 InSphereExtrapolation );
};



