// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// VolumetricBuilder: Builds a volumetric brush (criss-crossed sheets).
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Builders/EditorBrushBuilder.h"
#include "VolumetricBuilder.generated.h"

class ABrush;

UCLASS(MinimalAPI, autoexpandcategories=BrushSettings, EditInlineNew, meta=(DisplayName="Volumetric"))
class UVolumetricBuilder : public UEditorBrushBuilder
{
public:
	GENERATED_BODY()

public:
	UVolumetricBuilder(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Z;

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "0.000001"))
	float Radius;

	UPROPERTY(EditAnywhere, Category=BrushSettings, meta=(ClampMin = "2", ClampMax = "1000"))
	int32 NumSheets;

	UPROPERTY()
	FName GroupName;


	//~ Begin UBrushBuilder Interface
	virtual bool Build( UWorld* InWorld, ABrush* InBrush = NULL ) override;
	//~ End UBrushBuilder Interface

	// @todo document
	virtual void BuildVolumetric( int32 Direction, int32 InNumSheets, float InZ, float InRadius );
};



