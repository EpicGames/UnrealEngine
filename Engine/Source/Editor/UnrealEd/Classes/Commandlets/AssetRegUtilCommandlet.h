// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AssetRegUtilCommandlet.cpp: General-purpose commandlet for anything which
	makes integral use of the asset registry.
=============================================================================*/

#pragma once
#include "Commandlets/Commandlet.h"
#include "AssetRegUtilCommandlet.generated.h"

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogAssetRegUtil, Log, All);

class IAssetRegistry;
struct FSortableDependencyEntry;
struct FFileOpenOrder;

UCLASS(config=Editor)
class UAssetRegUtilCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()
public:

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;
	
	//~ End UCommandlet Interface

protected:

	IAssetRegistry* AssetRegistry;

	void RecursivelyGrabDependencies(TArray<FSortableDependencyEntry>& OutSortableDependencies,
		const int32& DepSet, int32& DepOrder, int32 DepHierarchy, TSet<FName>& ProcessedFiles, const TSet<FName>& OriginalSet, const FName& FilePath, const FName& PackageFName, const TArray<FTopLevelAssetPath>& FilterByClass);

	void ReorderOrderFile(const FString& OrderFilePath, const FString& ReorderFileOutPath);
	// Generate a new FOO which only takes partial update from new FOO files with input of an old FOO file and a new FOO file
	bool GeneratePartiallyUpdatedOrderFile(const FString& OldOrderFilePath, const FString& NewOrderFilePath, const FString& OutOrderFilePath, const float PatchSizePerfBalanceFactor);

	bool MergeOrderFiles(TMap<FString, int64>& NewOrderMap, TMap<FString, int64>& PrevOrderMap);
	bool GenerateOrderFile(TMap<FString, int64>& OutputOrderMap, const FString& ReorderFileOutPath);

private:


	bool LoadOrderFiles(const FString& OrderFilePath, TSet<FName>& OrderFiles);
	bool LoadOrderFile(const FString& OrderFilePath, TMap<FString, int64>& OrderMap);
};

UCLASS()
class UAssetRegistryDumpCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
public:
	virtual int32 Main(const FString& CmdLineParams) override;
};

