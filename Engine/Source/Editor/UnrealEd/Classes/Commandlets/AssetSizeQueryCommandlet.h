// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "AssetSizeQueryCommandlet.generated.h"

UCLASS()
class UAssetSizeQueryCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:

	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface

private:

};
