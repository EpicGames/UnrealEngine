// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "Engine/EngineTypes.h"
// world partition builder includes
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionActorDesc.h"

class FWorldPartitionActorDescInstance;

#include "BaseIteratePackagesCommandlet.generated.h"




// Log category should be accessible by derived classes
UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogIteratePackagesCommandlet, Log, All);

DECLARE_DELEGATE_TwoParams(FBaseIteratePackagesCommandletActorCallback, AActor* /*Actor*/, bool& /*bSavePackage*/);
DECLARE_DELEGATE_TwoParams(FBaseIteratePackagesCommandletObjectCallback, UObject* /*Actor*/, bool& /*bSavePackage*/);


UCLASS(MinimalAPI)
// Added UNREALED_API to expose this to the save packages test
// Base commandlet used to iterate packages provided on a commandline
// has callbacks for processing each package
class UBaseIteratePackagesCommandlet : public UCommandlet
{
    GENERATED_UCLASS_BODY()

protected:

	enum EBrevity
	{
		VERY_VERBOSE,
		INFORMATIVE,
		ONLY_ERRORS
	};

	EBrevity Verbosity;

	/** only packages that have this version or higher will be processed; a value of IGNORE_PACKAGE_VERSION indicates that there is no minimum package version */
	int32 MinResaveUEVersion;

	/**
	 * Limits processing to packages with this UE4 package version or lower.
	 * A value of IGNORE_PACKAGE_VERSION (default) removes this limitation.
	 */
	int32 MaxResaveUEVersion;

	/**
	 * Limits processing to packages with this licensee package version or lower.
	 * A value of IGNORE_PACKAGE_VERSION (default) removes this limitation.
	 */
	int32 MaxResaveLicenseeUEVersion;

	/** 
	 * Maximum number of packages to resave to avoid having a massive sync
	 * A value of -1 (default) removes this limitation.
	 */
	int32 MaxPackagesToResave;

	/** allows users to save only packages with a particular class in them (useful for fixing content) */
	TArray<FString> ResaveClasses;

	// If non-empty, this substring has to be present in the package name for the commandlet to process it
	FString PackageSubstring;

	// strip editor only content
	bool bStripEditorOnlyContent;

	// skip the assert when a package can not be opened
	bool bCanIgnoreFails;

	/** load all packages, and display warnings for those packages which would have been resaved but were read-only */
	bool bVerifyContent;

	/** if we should only save dirty packages **/
	bool bOnlySaveDirtyPackages;

	/** if we should auto checkout packages that need to be saved**/
	bool bAutoCheckOut;

	/** if we should batch together source control operations rather than submit them one at a time */
	bool bBatchSourceControl;

	/** if we should simply skip checked out files rather than error-ing out */
	bool bSkipCheckedOutFiles;

	/** if we should auto checkin packages that were checked out**/
	bool bAutoCheckIn;

	bool bUseWorldPartitionBuilder;

	/** Initialisation for the FScopedEditorWorld if bUseWorldPartitionBuilder is enabled */
	UWorld::InitializationValues WorldInitialisationValues;

	/** Should we build lighting for the packages we are saving? **/
	//bool bShouldBuildLighting;

	/** Should we build reflection captures for the packages we are saving? **/
	//bool bShouldBuildReflectionCaptures;

	/** Should we build texture streaming for the packages we are saving? **/
	//bool bShouldBuildTextureStreaming;

	/** Similar to above, but applies to all packages rather than just maps **/
	//bool bShouldBuildTextureStreamingForAll;

	/** Should the asset registry have been scanned before iterating packages */
	bool bRequireAssetRegistryLoadedBeforeIteration;

	bool bForceFinishAllCompilationBeforeGC;

	/** only process packages containing materials **/
	bool bOnlyMaterials;

	/** Only save packages with a changelist of zero **/
	bool bOnlyUnversioned;

	/** Only save packages that been saved by a licensee **/
	bool bOnlyLicenseed;

	/** Only save packages containing bulkdata payloads that have been virtualized **/
	bool bOnlyVirtualized;

	/** Only save packages containing FPayloadTrailers **/
	bool bOnlyPayloadTrailers;

	/** Should we build navigation data for the packages we are saving? **/
	//bool bShouldBuildNavigationData;

	/** Ignore package version changelist **/
	bool bIgnoreChangelist;

	UE_DEPRECATED(5.6, "KeepPackageGUIDOnSave is no longer supported, because package save no longer records arbitrary guids but instead records a hash of the package's bytes - PackageSavedHash.")
	bool bKeepPackageGUIDOnSave;

	/** Filter packages based on a collection **/
	TSet<FName> CollectionFilter;

	/** Should we update HLODs */
	//bool bShouldBuildHLOD;
	//bool bGenerateClusters;
	//bool bGenerateMeshProxies;
	//bool bForceClusterGeneration;
	//bool bForceProxyGeneration;
	//bool bForceSingleClusterForLevel;
	bool bSkipSubLevels;
	//bool bHLODMapCleanup;
	//FString ForceHLODSetupAsset;
	//FString HLODSkipToMap;
	bool bForceUATEnvironmentVariableSet;
	bool bResaveOnDemand = false;

	/** Running count of packages that got modified and will need to be resaved */
	int32 PackagesConsideredForResave;
	int32 PackagesResaved;
	int32 PackagesDeleted;
	int32 TotalPackagesForResave;

	/** Only collect garbage after N packages */
	int32 GarbageCollectionFrequency;

 	/** Lighting Build Quality(default: Production) */
 	//ELightingBuildQuality LightingBuildQuality;
 
	/** List of files to submit */
	TArray<FString> FilesToSubmit;

	/** The list of switches that were passed on the commandline */
	TArray<FString> Switches;

	/** List of redirector packages that should be fixed up at the end */
	//TArray<FString> RedirectorsToFixup;

	/** A queue containing source control operations to be performed in batches. */
	TPimplPtr<class FQueuedSourceControlOperations> SourceControlQueue;

	/** If running resaveondemand, only packages reported into this set are resaved. */
	TSet<FName> ResaveOnDemandPackages;
	TSet<FName> ResaveOnDemandSystems;
	FCriticalSection ResaveOnDemandPackagesLock;

	/**
	 * Evaluates the command-line to determine which maps to check.  By default all maps are checked
	 * Provides child classes with a chance to initialize any variables, parse the command line, etc.
	 *
	 * @param	Tokens			the list of tokens that were passed to the commandlet
	 * @param	MapPathNames	receives the list of path names for the maps that will be checked.
	 *
	 * @return	0 to indicate that the commandlet should continue; otherwise, the error code that should be returned by Main()
	 */
	UNREALED_API virtual int32 InitializeParameters( const TArray<FString>& Tokens, TArray<FString>& MapPathNames );

	virtual void InitializePackageNames(const TArray<FString>& Tokens, TArray<FString>& MapPathNames, bool& bExplicitPackages) { }

	UNREALED_API void ParseSourceControlOptions(const TArray<FString>& Tokens);

	UNREALED_API void OnAddResaveOnDemandPackage(FName SystemName, FName PackageName);

	/** Loads and saves a single package */
	UNREALED_API virtual void LoadAndSaveOnePackage(const FString& Filename);

	/** Checks if an actor should be loading during WorldPartition loading */
	virtual bool FilterActorDesc(const FWorldPartitionActorDesc*) const { return true; };

	/** Checks to see if a package should be skipped */
	UNREALED_API virtual bool ShouldSkipPackage(const FString& Filename);

	/** Deletes a single package */
	UNREALED_API virtual void DeleteOnePackage(const FString& Filename);

	/**
	 * Allow the commandlet to perform any operations on the export/import table of the package before all objects in the package are loaded.
	 *
	 * @param	PackageLinker	the linker for the package about to be loaded
	 * @param	bSavePackage	[in]	indicates whether the package is currently going to be saved
	 *							[out]	set to true to resave the package
	 */
	UNREALED_API virtual void PerformPreloadOperations( FLinkerLoad* PackageLinker, bool& bSavePackage );

	/**
	 * Allows the commandlet to perform any additional operations on the object before it is resaved.
	 *
	 * @param	Object			the object in the current package that is currently being processed
	 * @param	bSavePackage	[in]	indicates whether the package is currently going to be saved
	 *							[out]	set to true to resave the package
	 */
	virtual void PerformAdditionalOperations( class UObject* Object, bool& bSavePackage ) { }

	/**
	 * Allows the commandlet to perform any additional operations on the package before it is resaved.
	 *
	 * @param	Package			the package that is currently being processed
	 * @param	bSavePackage	[in]	indicates whether the package is currently going to be saved
	 *							[out]	set to true to resave the package
	 */
	virtual void PerformAdditionalOperations( class UPackage* Package, bool& bSavePackage ) { }

	/**
	* Allows the commandlet to perform any additional operations on the world before it is resaved.
	*
	* @param	World			the world that is currently being processed
	* @param	bSavePackage	[in]	indicates whether the package is currently going to be saved
	*							[out]	set to true to resave the package
	*/
	virtual void PerformAdditionalOperations(class UWorld* World, bool& bSavePackage) { }

	virtual void PerformWorldBuilderAdditionalOperations(class AActor* Actor, bool& bSavePackage) { }
	virtual void PerformWorldBuilderAdditionalOperations(class UObject* Object, bool& bSavePackage) { }
	
	UE_DEPRECATED(5.4, "Use WorldBuilderFailedLoadingActor with FWorldPartitionActorDescInstance")
	virtual void WorldBuilderFailedLoadingActor(const FWorldPartitionActorDesc* ActorDesc) { } 
	virtual void WorldBuilderFailedLoadingActor(const FWorldPartitionActorDescInstance* ActorDescInstance) { }

	virtual void PostPerformAdditionalOperations(class UPackage* Package) { }




	/**
	 * Allows the commandlet to do something after all the packages have been processed
	 */
	virtual void PostProcessPackages() { }

	/**
	 * Removes any UClass exports from packages which aren't script packages.
	 *
	 * @param	Package			the package that is currently being processed
	 *
	 * @return	true to resave the package
	 */
	UNREALED_API bool CleanClassesFromContentPackages( class UPackage* Package );

	// Get the changelist description to use if automatically checking packages out
	UNREALED_API virtual FText GetChangelistDescription() const;

	UNREALED_API bool CheckoutFile(const FString& Filename, bool bAddFile = false, bool bIgnoreAlreadyCheckedOut = false);
	UNREALED_API bool RevertFile(const FString& Filename);
	UNREALED_API bool CanCheckoutFile(const FString& Filename, FString& CheckedOutUser);
	UNREALED_API void CheckoutAndSavePackage(UPackage* Package, TArray<FString>& SublevelFilenames, bool bIgnoreAlreadyCheckedOut = false);
	UNREALED_API void CheckInFiles(const TArray<FString>& InFilesToSubmit, const FText& InDescription) const;

	/**
	 * Creates and returns a unique filename in the temporary file directory.
	 *
	 * @return	A unique temporary filename
	 */
	UNREALED_API FString CreateTempFilename();

	/**
	 * Returns the path of the directory we use to store all temporary files for
	 * this commandlet.
	 *
	 * @return Path to the temporary file directory
	 */
	UNREALED_API FString GetTempFilesDirectory();

	/**
	 * Delete the remaining files in the temporary files directory for the commandlet.
	 */
	UNREALED_API void CleanTempFiles();	

	// Print out a message only if running in very verbose mode
	UNREALED_API void VerboseMessage(const FString& Message);

	/** Parse commandline to decide whether resaveondemand is activated for the given system. */
	UNREALED_API TSet<FName> ParseResaveOnDemandSystems();

private:

	UNREALED_API void SavePackages(const TArray<UPackage*>& PackagesToSave);

public:		
	//~ Begin UCommandlet Interface
	UNREALED_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
