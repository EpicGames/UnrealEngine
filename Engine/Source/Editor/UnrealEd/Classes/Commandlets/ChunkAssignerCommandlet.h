// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ChunkAssignerCommandlet.cpp: used to update cook output with chunk assignment.
	Updates asset registry
	Updates shadercode libraries
	Updates loc settings.
=============================================================================*/

#pragma once
#include "Commandlets/Commandlet.h"
#include "ChunkAssignerCommandlet.generated.h"


UCLASS(config=Editor)
class UChunkAssignerCommandlet : public UCommandlet
{
	GENERATED_BODY()
public:

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;
	
	//~ End UCommandlet Interface
};


