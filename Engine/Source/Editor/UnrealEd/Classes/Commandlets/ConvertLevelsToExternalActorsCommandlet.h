// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ISourceControlProvider.h"
#include "ConvertLevelsToExternalActorsCommandlet.generated.h"

UCLASS(MinimalAPI)
class UConvertLevelsToExternalActorsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	UNREALED_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

protected:
	UNREALED_API ULevel* LoadLevel(const FString& LevelToLoad) const;
	UNREALED_API void GetSubLevelsToConvert(ULevel* MainLevel, TSet<ULevel*>& SubLevels, bool bRecursive);

	UNREALED_API bool CheckExternalActors(const FString& Level, bool bRepair);

	bool UseSourceControl() const { return SourceControlProvider != nullptr; }
	ISourceControlProvider& GetSourceControlProvider() { check(UseSourceControl()); return *SourceControlProvider; }
	UNREALED_API bool AddPackageToSourceControl(UPackage* Package);
	UNREALED_API bool CheckoutPackage(UPackage* Package);
	UNREALED_API bool SavePackage(UPackage* Package);
	UNREALED_API bool DeleteFile(const FString& Filename);

protected:
	ISourceControlProvider* SourceControlProvider;
};
