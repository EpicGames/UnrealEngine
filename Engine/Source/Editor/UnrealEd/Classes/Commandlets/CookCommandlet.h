// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CookCommandlet.cpp: Commandlet for cooking content
=============================================================================*/

#pragma once

#include "Commandlets/Commandlet.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"

#include "CookCommandlet.generated.h"

class FSandboxPlatformFile;
class ITargetPlatform;
class UCookOnTheFlyServer;
enum class ECookByTheBookOptions;

UCLASS(config=Editor)
class UCookCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

protected:
	/** List of asset types that will force GC after loading them during cook */
	UE_DEPRECATED(5.2, "No longer used")
	TArray<FString> FullGCAssetClassNames;

	/** If true, legacyiterative cooking is being done */
	bool bLegacyIterativeCooking;
	/** Prototype cook-on-the-fly server */
	bool bCookOnTheFly; 
	/** Is using fast cook */
	bool bFastCook;
	/** Cook everything */
	bool bCookAll;
	/** Skip saving any packages in Engine/Content/Editor* UNLESS TARGET HAS EDITORONLY DATA (in which case it will save those anyway) */
	bool bSkipEditorContent;
	/** Save all cooked packages without versions. These are then assumed to be current version on load. This is dangerous but results in smaller patch sizes. */
	bool bUnversioned;
	/** Produce editor optional package output when cooking. */
	bool bCookEditorOptional;
	/** Generate manifests for building streaming install packages */
	bool bGenerateStreamingInstallManifests;
	/** Error if we access engine content (useful for dlc) */
	bool bErrorOnEngineContentUse;
	/** Only cook packages specified on commandline options (for debugging)*/
	bool bCookSinglePackage;
	/** Modification to bCookSinglePackage - cook transitive hard references in addition to the packages on the commandline */
	bool bKeepSinglePackageRefs;
	/** Should we output additional verbose cooking warnings */
	bool bVerboseCookerWarnings;
	/** only clean up objects which are not in use by the cooker when we gc (false will enable full gc) */
	bool bPartialGC;
	/** Ignore ini settings out of date. */
	bool bIgnoreIniSettingsOutOfDate;
	/** All commandline tokens */
	TArray<FString> Tokens;
	/** All commandline switches */
	TArray<FString> Switches;
	/** All commandline params */
	FString Params;

	/**
	 * Cook on the fly routing for the commandlet
	 *
	 * @param  BindAnyPort					Whether to bind on any port or the default port.
	 * @param  Timeout						Length of time to wait for connections before attempting to close
	 * @param  bForceClose					Whether or not the server should always shutdown after a timeout or after a user disconnects
	 * @param  TargetPlatforms				The list of platforms that should be initialized at startup.  Other platforms will be initialized when first requested
	 *
	 * @return true on success, false otherwise.
	 */
	bool CookOnTheFly( FGuid InstanceId, int32 Port, int32 Timeout = 180, bool bForceClose = false, const TArray<ITargetPlatform*>& TargetPlatforms=TArray<ITargetPlatform*>() );

	/** Cooks for specified targets */
	bool CookByTheBook(const TArray<ITargetPlatform*>& Platforms);

	bool CookAsCookWorker();

	/** Collect garbage if the cooker's TickResults requested it */
	void ConditionalCollectGarbage(uint32 TickResults, UCookOnTheFlyServer& COTFS);

public:

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;
	
	//~ End UCommandlet Interface

private:
	void RunCookByTheBookList(UCookOnTheFlyServer* CookOnTheFlyServer, void* StartupOptionsAsVoid,
		ECookByTheBookOptions CookOptions);
	void RunCookByTheBookCook(UCookOnTheFlyServer* CookOnTheFlyServer, void* StartupOptionsAsVoid,
		ECookByTheBookOptions CookOptions);
};
