// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "Interfaces/ITargetPlatform.h"
#include "CookGlobalShadersCommandlet.generated.h"

struct FFileChangeData;
class UCookGlobalShadersDeviceHelperBase;

UCLASS(config=Editor)
class UCookGlobalShadersCommandlet : public UCommandlet
{
	GENERATED_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

private:
	// All the derived info from command line parameters.
	bool bDeployToDevice = false;
	bool bCopyToStaged = false;
	bool bExecuteReload = false;
	FString DeployFolder;
	FString PlatformName;
	ITargetPlatform* TargetPlatform = nullptr;
	ITargetDevicePtr TargetDevice = nullptr;
	class UCookGlobalShadersDeviceHelperBase* DeviceHelper = nullptr;
	TArray<FName> ShaderFormats;

	// Cook, deploy and reload global shaders
	void CookGlobalShaders() const;

	// Callback when a shader source directory changes.
	void HandleDirectoryChanged(const TArray<FFileChangeData>& InFileChangeDatas);

	// Issues CookGlobalShaders() automatically when a change is detected to shader source directories.
	void CookGlobalShadersOnDirectoriesChanges();

};

UCLASS(abstract, transient, MinimalAPI)
class UCookGlobalShadersDeviceHelperBase : public UObject
{
	GENERATED_BODY()
public:
	virtual bool CopyFilesToDevice(class ITargetDevice* Device, const TArray<TPair<FString, FString>>& FilesToCopy) const { check(false); return false; }
};

UCLASS()
class UCookGlobalShadersDeviceHelperStaged : public UCookGlobalShadersDeviceHelperBase
{
	GENERATED_BODY()
public:
	virtual bool CopyFilesToDevice(class ITargetDevice* Device, const TArray<TPair<FString, FString>>& FilesToCopy) const override;

	FString StagedBuildPath;
};
