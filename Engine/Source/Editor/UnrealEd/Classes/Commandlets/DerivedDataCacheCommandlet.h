// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DerivedDataCacheCommandlet.cpp: Commandlet for DDC maintenence
=============================================================================*/

#pragma once

#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "DerivedDataCacheCommandlet.generated.h"

class ITargetPlatform;
class UPackage;
class UWorld;

UCLASS()
class UDerivedDataCacheCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	UDerivedDataCacheCommandlet(FVTableHelper& Helper); // Declare the FVTableHelper constructor manually so that we can forward-declare-only TUniquePtrs in the header without getting compile error in generated cpp
	~UDerivedDataCacheCommandlet(); // Same reason as ctor above

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	struct FCachingData
	{
		/** Weak object reference to ensure that the object being referenced is still valid */
		TWeakObjectPtr<UObject> ObjectValidityReference;

		/** Element n == IsCachedCookedPlatformDataLoaded has returned true for Commandlet->Platforms[n] */
		TBitArray<> PlatformIsComplete;
		/**
		 * The last time in seconds we tested whether the object is still compiling and/or called
		 * IsCachedCookedPlatformDataLoaded. Used to throttle IsCachedCookedPlatformDataLoaded which can be quite
		 * expensive on some objects
		 */
		double LastTimeTested = 0.;

		/**
		 * If true, we wait to launch additional platforms until the first platform completes.
		 * This is specifically so that shared linear texture encoding can reuse the base texture encode - otherwise
		 * both platforms try to fetch and build the base texture at the same time and benefits are limited.
		 */
		bool bFirstPlatformIsSolo = false;
	};
	TMap<TObjectPtr<UObject>, FCachingData> CachingObjects;
	TSet<FName>    ProcessedPackages;
	TSet<FName>    PackagesToProcess;
	TArray<const ITargetPlatform*> Platforms;
	double FinishCacheTime = 0.0;
	double BeginCacheTime = 0.0;
	bool bSharedLinearTextureEncodingEnabled = false;

	class FPackageListener;
	TUniquePtr<FPackageListener> PackageListener;

	class FObjectReferencer;
	TUniquePtr<FObjectReferencer> ObjectReferencer;

	void MaybeMarkPackageAsAlreadyLoaded(UPackage *Package);

	void CacheLoadedPackages(UPackage* CurrentPackage, uint8 PackageFilter, TSet<FName>& OutNewProcessedPackages);
	void CacheWorldPackages(UWorld* World, uint8 PackageFilter, TSet<FName>& OutNewProcessedPackages);
	bool ProcessCachingObjects();
	void FinishCachingObjects();
};


