// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "Misc/AssetRegistryInterface.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "DiffAssetRegistriesCommandlet.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDiffAssets, Log, All);

class IAssetRegistry;

UCLASS(CustomConstructor, config=Editor)
class UDiffAssetRegistriesCommandlet : public UCommandlet
{
	enum class SortOrder
	{
		BySize,
		ByName,
		ByClass,
		ByChange,
	};

	enum EAssetFlags
	{
		Add = (1 << 0),
		Remove = (1 << 1),
		GuidChange = (1 << 2),
		HashChange = (1 << 3),
		DepGuidChange = (1 << 4),
		DepHashChange = (1 << 5),
	};

	struct FChangeInfo
	{
		enum class EChangeFlags
		{
			ChangeFlagsNone = 0,
			ChangeFlagsDelete	= (1 << 0),
			ChangeFlagsEdit		= (1 << 1),
			ChangeFlagsAdd		= (1 << 2),
		};

		int64 Adds, AddedBytes;
		int64 Changes, ChangedBytes;
		int64 Deletes, DeletedBytes;
		int64 Unchanged, UnchangedBytes;

		FChangeInfo()
		{
			Adds = AddedBytes = 0;
			Changes = ChangedBytes = 0;
			Deletes = DeletedBytes = 0;
			Unchanged = UnchangedBytes = 0;
		}

		float GetChangePercentage() const
		{
			return GetTotalChangeSize() / (float)GetTotalSize();
		}

		int64 GetTotalChangeCount() const
		{
			return Adds + Changes;
		}

		int64 GetTotalChangeSize() const
		{
			return AddedBytes + ChangedBytes;
		}

		int64 GetTotalSize() const
		{
			return GetTotalChangeSize() + UnchangedBytes;
		}

		int32 GetChangeFlags() const
		{
			int32 Flags = int32(EChangeFlags::ChangeFlagsNone);

			if (Adds > 0)
			{
				Flags |= int32(EChangeFlags::ChangeFlagsAdd);
			}

			if (Changes > 0)
			{
				Flags |= int32(EChangeFlags::ChangeFlagsEdit);
			}

			if (Deletes > 0)
			{
				Flags |= int32(EChangeFlags::ChangeFlagsDelete);
			}

			return Flags;
		}

		const FChangeInfo& operator+=(const FChangeInfo& Rhs)
		{
			Adds += Rhs.Adds;
			AddedBytes += Rhs.AddedBytes;
			Changes += Rhs.Changes;
			ChangedBytes += Rhs.ChangedBytes;
			Deletes += Rhs.Deletes;
			DeletedBytes += Rhs.DeletedBytes;
			Unchanged += Rhs.Unchanged;
			UnchangedBytes += Rhs.UnchangedBytes;

			return *this;
		}
	};

	struct FDeterminismInfo
	{
		int64 DirectSize;
		int64 DirectCount;
		int64 IndirectSize;
		int64 IndirectCount;

		TArray<int64> Direct;
		TArray<int64> Indirect;
		
		void AddDirect(const FChangeInfo& Rhs)
		{
			DirectSize += Rhs.GetTotalChangeSize();
			DirectCount += Rhs.GetTotalChangeCount();
		}

		void AddIndirect(const FChangeInfo& Rhs)
		{
			IndirectSize += Rhs.GetTotalChangeSize();
			IndirectCount += Rhs.GetTotalChangeCount();
		}
	};

	struct FChunkChangeInfo
	{
		TSet<FName> IncludedAssets;
		TMap<FName, FChangeInfo> ChangesByClass;
		TMap<FName, FDeterminismInfo> Determinism;
	};

	GENERATED_UCLASS_BODY()

public:
	UDiffAssetRegistriesCommandlet(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		LogToConsole = true;
	}

	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface

	// do the export without creating a commandlet
	void DiffAssetRegistries(const FString& OldPath, const FString& NewPath, bool bUseSourceGuid, bool bEnginePackagesOnly);
	void ConsistencyCheck(const FString& OldPath, const FString& NewPath);
private:

	void	RecordAdd(FName InAssetPath, const FAssetPackageData& InNewData);
	void	RecordEdit(FName InAssetPath, const FAssetPackageData& InNewData, const FAssetPackageData& InOldData);
	void	RecordDelete(FName InAssetPath, const FAssetPackageData& InOldData);
	void	RecordNoChange(FName InAssetPath, const FAssetPackageData& InData);

	FName	GetClassName(FAssetRegistryState& InRegistryState, FName InAssetPath);

	TArray<int32> GetAssetChunks(FAssetRegistryState& InRegistryState, FName InAssetPath);

	bool	IsInRelevantChunk(FAssetRegistryState& InRegistryState, FName InAssetPath);

	void	LogChangedFiles(FArchive *CSVFile, const FString &OldPath, const FString &NewPath);
	void	LogClassSummary(FArchive *CSVFile, const FString& HeaderPrefix, const TMap<FName, FChangeInfo>& InChangeInfoByAsset, bool bDoWarnings, TMap<FName, FDeterminismInfo> DeterminismInfo = TMap<FName, FDeterminismInfo>());
	void	SummarizeDeterminism();

	void	PopulateChangelistMap(const FString &Branch, const FString &CL, bool bEnginePackages);
	void	FillChangelists(FString Branch, FString CL, FString BasePath, FString AssetPath);
	bool	LaunchP4(const FString& Args, TArray<FString>& Output, int32& OutReturnCode) const;

	bool				bIsVerbose;

	bool				bSaveCSV;

	bool				bMatchChangelists;

	bool				bGroupByChunk;

	FString				CSVFilename;

	// Don't report any classes of assets with less than this number of changes
	int32				MinChangeCount;

	// Don't report any classes of assets with less than this number of changes
	int32				MinChangeSizeMB;

	// Warn when any class of assets has changed by this amount (0=disabled)
	int32				WarnPercentage;

	// Don't warn any class of assets with a total change size lower than this amount (0=disabled)
	int32				WarnSizeMinMB;

	// Warn when the total changes are greater than this amount (0=disabled)
	int32				WarnTotalChangedSizeMB;

	// Platform we're working on, only used for reporting clarity
	FString				TargetPlatform;

	int32				DiffChunkID;
	FAssetRegistryState OldState;
	FAssetRegistryState NewState;

	SortOrder			ReportedFileOrder;

	FChangeInfo						ChangeSummary;
	FChangeInfo						NondeterministicSummary;
	FChangeInfo						IndirectNondeterministicSummary;
	TMap<FName, FChangeInfo>		ChangeSummaryByClass;
	TMap<FName, FDeterminismInfo>   DeterminismByClass;
	TMap<FName, FChangeInfo>		ChangeInfoByAsset;
	TMap<int32, FChangeInfo>		ChangeSummaryByChangelist;
	TMap<FName, FName>				AssetPathToClassName;
	TMap<FString, int32>			AssetPathToChangelist;
	TMap<FName, int32>				AssetPathFlags;
	TMap<int32, FChunkChangeInfo>	ChangesByChunk;
	TMultiMap<FName, int32>			ChunkIdByAssetPath;

	UPROPERTY(config)
	TArray<FString> AssetRegistrySearchPath;

	UPROPERTY(config)
	FString P4Repository;
	UPROPERTY(config)
	FString P4EngineBasePath;
	UPROPERTY(config)
	FString P4EngineAssetPath;

	UPROPERTY(config)
	FString P4GameBasePath;
	UPROPERTY(config)
	FString P4GameAssetPath;
	UPROPERTY(config)
	FString RegexBranchCL;
};
