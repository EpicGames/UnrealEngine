// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Commandlet to allow diff in P4V, and expose that functionality to the editor
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "DiffAssetsCommandlet.generated.h"

UCLASS()
class UDiffAssetsCommandlet : public UCommandlet
{
    GENERATED_UCLASS_BODY()
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override
	{
		return !ExportFilesToTextAndDiff(Params);
	}
	//~ End UCommandlet Interface
	
	/** 
	 * The meat of the commandlet, this can be called from the editor
	 * Format of commandline is as follow
	 * File1.uasset File2.uasset -DiffCmd="C:/Program Files/Araxis/Araxis Merge/AraxisP4Diff.exe {1} {2}"
	 * @param Params		Command line
	 * @return				true if success
	**/
	static bool ExportFilesToTextAndDiff(const FString& Params);

	/** 
	 * Copies a uasset file or map to a temp location so it can be loaded without disruption to anything 
	 * @param InOutFilename		Both input and output. The original filename as input, output as the temp filename
	 * @return					true if success
	**/
	static bool CopyFileToTempLocation(FString& InOutFilename);

	/** 
	 * Loads a uasset file or map and provides a sorted list of contained objects (but not subobjects as those will get exported anyway)
	 * @param Filename			File to load
	 * @param LoadedObjects		Sorted list of objects
	 * @return					true if success
	**/
	static bool LoadFile(const FString& Filename, TArray<UObject *>& LoadedObjects);

	/** 
	 * Loads a uasset file or map and provides a sorted list of contained objects (but not subobjects as those will get exported anyway)
	 * @param Filename			Name to save the text export as
	 * @param LoadedObjects		List of objects to export
	 * @return					true if success
	**/
	static bool ExportFile(const FString& Filename, const TArray<UObject *>& LoadedObjects);

	/** 
	 * Runs an external diff utility
	 * @param Filename1		First filename
	 * @param Filename1		Second filename
	 * @param DiffCommand	Diff command, with {1} {2} in it....for example: C:/Program Files/Araxis/Araxis Merge/AraxisP4Diff.exe {1} {2}
	 * @return				true if success
	**/
	static bool ExportFilesToTextAndDiff(const FString& Filename1, const FString& Filename2, const FString& DiffCommand);
};

