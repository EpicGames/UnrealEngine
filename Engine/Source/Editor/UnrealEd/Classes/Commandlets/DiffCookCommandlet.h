// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetRegistryState.h"
#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/DirectoryTree.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Logging/LogVerbosity.h"
#include "Misc/PackagePath.h"
#include "Misc/StringBuilder.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"

#include "DiffCookCommandlet.generated.h"

class ITargetPlatform;
class UCookOnTheFlyServer;
namespace UE::DiffCook::DiffDepotUtils { struct FDepotData; }


/**
 * Compares two depots that were written out by the CookCommandlet. Similar to running the cookcommandlet
 * with -diffonly, but compares two depots on disk rather than comparing one depot on disk against the
 * current in-memory results.
 */
UCLASS()
class UDiffCookCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	virtual int32 Main(const FString& CmdLineParams) override;

private:
	/**
	 * List of the string arguments parsed by name from commandline that need interpretation/transform, before
	 * we do the transforms.
	 */
	struct FCommandLineArgs
	{
		FString BasePath;
		FString CompPath;
		TArray<FString> RequestedPackages;
	};
	/**
	 * Holds data about a Plugin's mount point for special-case plugins
	 * @see MountsWithLeafFolderNameNotEqualLongPackageNameRoot
	 */
	struct FMountWithLeafFolderNameNotEqualLongPackageNameRoot
	{
		FString PathStartingWithPlugins;
		FString LongPackageNameRoot;
	};
	/** All the information we collect about a package within a cooked depot. */
	struct FPackageData
	{
		FName PackageName;
		FString CookPath;
		uint32 HasExtensionBitfield = 0;
		EPackageExtension HeaderExtension = EPackageExtension::Count;

		bool HasExtension(EPackageExtension Extension) const;
		void SetHasExtension(EPackageExtension Extension, bool bValue);
	};
	/** All the information we collect about a cooked depot. */
	struct FCookedDepot
	{
		FAssetRegistryState ARState;
		FAssetRegistryState DevARState;
		TMap<FString, FString> LongPackageNameRoots;
		TMap<FString, FString> LocalPathRoots;
		TMap<FName, FPackageData*> PackageDatasByPackageName;
		TArray<TUniquePtr<FPackageData>> PackageDatas;
		FString CookPath;
		FString ProjectName;
		const ITargetPlatform* TargetPlatform = nullptr;
		bool bARStateValid = false;
		bool bDevARStateValid = false;
		bool bValid = false;
	};
	/** Types of diff results after comparing the versions of a package in two cooked depots. */
	enum class EPackageDiffResult
	{
		Identical,
		Removed,
		Added,
		Modified,
		Error,
	};
	/**
	 * Diff result and diagnostic information about the diff after comparing the versions of a package in two
	 * cooked depots.
	 */
	struct FPackageDiffResult
	{
		FName PackageName;
		EPackageDiffResult Result = EPackageDiffResult::Error;
		EPackageExtension Extension = EPackageExtension::Count;
		int64 Offset = -1;
		int64 CombinedOffset = -1;

		FPackageDiffResult(FName InPackageName = NAME_None, EPackageDiffResult InResult = EPackageDiffResult::Error,
			EPackageExtension InExtension = EPackageExtension::Count, int64 InOffset = 1, int64 InCombinedOffset = -1);
	};
	/** All diff results found when comparing two cooked depots. */
	struct FDiffResult
	{
		TArray<FPackageDiffResult> PackageDiffs;
	};

private:
	/** Like a destructor, but called at the end of Main. */
	void Shutdown();
	/**
	 * Read information about plugins from the current project running the commandlet. In some cases this information
	 * is necessary to interpret the data in a cooked depot.
	 */
	void InitializePlugins();
	bool TryParseCommandLine(const FString& CmdLineParams);
	/** Load all information about each cooked depot we will compare, including package lists. */
	bool TryLoadDepots();
	/**
	 * Load the non-package data from each cooked depot we will compare. This is relatively quick since we do not have
	 * to scan the disk for package lists.
	 */
	bool TryLoadDepotSummaries();
	/** Load package lists for each cooked depot we will compare. Scans the disk, can be slow. */
	void LoadDepotPackageLists();
	/** Find all of the /Engine, /Game, and /<PluginName> mount points in the depot. Scans the disk, can be slow. */
	void LoadDepotContentRoots(FCookedDepot& Depot);
	/**
	 * Convert commandline arguments for -package= (and any other filters). Has to be done after loading depots and
	 * might fail, because it interprets the data using packagelist information from the depots.
	 */
	bool TryConstructFilterLists();
	/**
	 * Compares the two depots as sets of binary files, without any interpretation of the bytes within each file. 
	 * All the extensions for a package are grouped together, and it does apply a little reasoning about the
	 * relationships between recognized extensions.
	 * Returns a diffresult that lists just the differences.
	 */
	FDiffResult DiffDepotAsBinary();
	/** Print summary information about the diffed depots, without listing any per-package results. */
	void PrintSummary(FDiffResult& Diff);
	/** Print per-package results about the diffed depots. */
	void PrintPackageDiffs(FDiffResult& Diff);
	/** FPackageName::TryConvertFilenameToLongPackageName, but for a cooked depot. */
	static bool TryConvertCookPathToLongPackageName(FCookedDepot& Depot, FStringView CookPath,
		FString& OutLongPackageName);
	/** FPackageName::TryConvertLongPackageNameToFilename, but for a cooked depot. */
	static bool TryConvertLongPackageNameToCookPath(FCookedDepot& Depot, FStringView LongPackageName,
		EPackageExtension Extension, FString& OutCookPath);
	/**
	 * If the give PackageData was found during LoadDepotPackageLists to have the given extension, return
	 * a pointer to a string for the cookpath to package's file for that extension. Returns &Buffer if the,
	 * file exists, otherwise returns nullptr.
	 */
	static FString* GetFileNameForExtension(UDiffCookCommandlet::FPackageData& PackageData, FString& Buffer,
		EPackageExtension Extension);
	/** Helper for LoadDepotContentRoots. Finds all of the directories with a "Content" subdirectory. */
	static TArray<FString> FindContentRootsFromDepotTree(FCookedDepot& Depot,
		FString& OutEngineLocalPath, FString& OutGameLocalPath);

	// Normalization functions, like FPathViews::NormalizeDirectoryName, but potentially specific to our
	// needs.
	[[nodiscard]] static FString GetNormalizedLocalDir(FStringView Dir);
	static void NormalizeLocalDir(FStringBuilderBase& Dir);
	[[nodiscard]] static FString GetNormalizedLocalPath(FStringView Path);
	static void NormalizeLocalPath(FStringBuilderBase& Path);
	[[nodiscard]] static FString GetNormalizedFlexPath(FStringView Path);
	static void NormalizeFlexPath(FStringBuilderBase& Path);

	static ELogVerbosity::Type ParseDiffVerbosity(const FString& Text);

private:
	FCommandLineArgs Args;
	TArray<ITargetPlatform*> TargetPlatforms;
	TSet<FName> FilterByPackageName;
	/**
	 * Map from leaf name of the plugin's directory to the array of plugins that use that directory.
	 * Only contains plugins that need it because plugin name does not match leaf directory name.
	 * Some Plugins, such as Engine/Plugins/Interchange/Runtime/Interchange.uplugin, have a folder name that does not
	 * match its plugin name. The cooked data does not have yet enough information to map those; we look them up from
	 * the plugins registered with the editor running the commandlet.
	 */
	TMap<FString, TArray<FMountWithLeafFolderNameNotEqualLongPackageNameRoot>>
		MountsWithLeafFolderNameNotEqualLongPackageNameRoot;
	FCookedDepot BaseDepot;
	FCookedDepot CompDepot;
	uint32 NumAdded = 0;
	uint32 NumRemoved = 0;
	uint32 NumModified = 0;
	ELogVerbosity::Type AddedVerbosity = ELogVerbosity::Display;
	ELogVerbosity::Type RemovedVerbosity = ELogVerbosity::Display;
	ELogVerbosity::Type ModifiedVerbosity = ELogVerbosity::Display;
	bool bShowSummary = true;
	bool bShowPackages = true;
	bool bShowAddedPackages = true;
	bool bShowRemovedPackages = true;
	bool bShowModifiedPackages = true;
	bool bShowHeaders = false; // Not yet implemented, change default to true once implemented
	bool bShowSerialize = false;

	friend UE::DiffCook::DiffDepotUtils::FDepotData;
};
