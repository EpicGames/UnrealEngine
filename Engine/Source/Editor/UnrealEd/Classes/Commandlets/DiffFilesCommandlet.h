// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectAnnotation.h"
#include "Commandlets/Commandlet.h"
#include "DiffFilesCommandlet.generated.h"

UCLASS()
class UDiffFilesCommandlet : public UCommandlet
{

	GENERATED_UCLASS_BODY()


	struct FPackageInfo
	{
		FString FullPath;
		FString FriendlyName;
	};


	TArray<FPackageInfo> PackageInfos;

	/** Handled annotation to track which objects we have dealt with **/
	class FUObjectAnnotationSparseBool HandledAnnotation;

	/**
	* Parses the command-line and loads the packages being compared.
	*
	* @param	Parms	the full command-line used to invoke this commandlet
	*
	* @return	true if all parameters were parsed successfully; false if any of the specified packages couldn't be loaded
	*			or the parameters were otherwise invalid.
	*/
	bool Initialize(const TCHAR* Parms);


	/**
	 * Load the packages in the commandline while loading diff them against eachother to track down where serialization has variation in the two packages
	 */
	void LoadAndDiff();

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};


