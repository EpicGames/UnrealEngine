// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

#include "DumpAssetRegistryCommandlet.generated.h"

UCLASS()
class UDumpAssetRegistryCommandlet: public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	// Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	// End UCommandlet Interface

private:
	bool TryParseArgs();
	bool TryDumpAssetRegistry();

	TArray<FString> FormattingArgs;
	int32 LinesPerPage = 0;
	bool bLowerCase = false;
	FString Path;
	FString OutDir;
};
