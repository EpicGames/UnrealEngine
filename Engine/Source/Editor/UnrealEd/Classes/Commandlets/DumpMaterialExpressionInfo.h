// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "DumpMaterialExpressionInfo.generated.h"

/**
  Finds all instances of material expressions and dumps their data to a csv.
  
  # CUSTOM HLSL EXAMPLE
  You can dump all custom hlsl nodes and their code with 
  `<YourProject> -dx12 -run=DumpMaterialExpressionInfo -unattended -expression=MaterialExpressionCustom -columns=Code -csv=C:/output.csv`

  The output can then be loaded into sqlite for querying with 
    .separator ","
    .import C:/output.csv tbl --csv
    .save output.db
  Then you can, for example, order by hlsl snippet length:
    SELECT * FROM tbl ORDER BY length(Code)

  This can also be connected to R to do analysis:
    install.packages("RSQLite")
    library(RSQLite)
    conn <- dbConnect(RSQLite::SQLite(), "output.db")
  For example, a histogram of snippet length:
    result <- dbGetQuery(conn, "SELECT length(Code) FROM tbl")
    hist(result$`length(Code)`)
  Or a treemap showing which folders and assets have the most custom HLSL code nodes
    result <- dbGetQuery(conn, "SELECT Name,length(Code) FROM tbl")
    name <- strsplit(result$Name, "/")
    -- Get maximum number of path segments
    name_max_length <- max(sapply(name, length))
    -- Make every asset path have the same number of segments (columns)
    name_padded_list <- lapply(name, function(x) {
      length(x) <- name_max_length
      return(x)
    })
    -- Convert to matrix, trim empty space from front
    name_matrix <- do.call(rbind, name_padded_list)
    name_matrix <- name_matrix[,-1]
    -- Make treemap
    df <- as.data.frame(name_matrix)
    -- To weight by volume of code instead of number of nodes, use df$Value <- result$`length(Code)`
    df$Value <- rep(1, times=length(result$`length(Code)`))
    treemap(df, index = c("V1", "V2", "V3"), vSize="Value")
  */
UCLASS(config = Editor)
class UDumpMaterialExpressionInfoCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
