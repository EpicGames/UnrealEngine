// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "DumpMaterialInfo.generated.h"

/**
  Writes a CSV that lists many properties of all material (instances).
  Useful to inspect all materials and query them, e.g. for performance characteristics.
  */
UCLASS(config = Editor)
class UDumpMaterialInfoCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
