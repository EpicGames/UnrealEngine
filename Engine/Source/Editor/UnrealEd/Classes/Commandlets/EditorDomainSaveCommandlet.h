// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"

#include "EditorDomainSaveCommandlet.generated.h"

/** Runs the EditorDomainSave server for client Editor and CookCommandlet processes. */
UCLASS(config = Editor)
class UEditorDomainSaveCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& CmdLineParams) override;
	//~ End UCommandlet Interface
};

