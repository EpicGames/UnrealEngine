// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "PackageSourceControlHelper.h"
#include "ExternalActorsCommandlet.generated.h"

UCLASS(MinimalAPI)
class UExternalActorsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	UNREALED_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

protected:
	UNREALED_API UWorld* LoadWorld(const FString& LevelToLoad);

	UPROPERTY()
	bool bRepair;

	UPROPERTY()
	bool bDisable;

	UPROPERTY()
	bool bEnable;

	UPROPERTY()
	bool bForce;

	UPROPERTY()
	bool bReport;

	UPROPERTY()
	bool bListMaps;

	UPROPERTY()
	FString MapListFile;

	UPROPERTY()
	FString DumpCSVFile;
};
