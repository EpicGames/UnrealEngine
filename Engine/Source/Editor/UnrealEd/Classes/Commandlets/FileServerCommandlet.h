// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FileServerCommandlet.h: Declares the UFileServerCommandlet class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Commandlets/Commandlet.h"
#include "FileServerCommandlet.generated.h"

/**
 * Implements a file server commandlet.
 */
UCLASS()
class UFileServerCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()


public:

	//~ Begin UCommandlet Interface

	virtual int32 Main( const FString& Params ) override;
	
	//~ End UCommandlet Interface


private:

	// Holds the instance identifier.
	FGuid InstanceId;
};
