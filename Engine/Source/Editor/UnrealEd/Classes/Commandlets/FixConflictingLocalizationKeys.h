// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "FixConflictingLocalizationKeys.generated.h"

/**
 * Internal commandlet to fix any conflicting localization keys that are found in a manifest.
 * @note Hard-coded to work with the "Game" localization target.
 */
UCLASS()
class UFixConflictingLocalizationKeysCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:

	virtual int32 Main(const FString& Params);
};
