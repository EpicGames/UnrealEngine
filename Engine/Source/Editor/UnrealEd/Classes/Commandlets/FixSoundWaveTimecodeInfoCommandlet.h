// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"

#include "FixSoundWaveTimecodeInfoCommandlet.generated.h"

/**
 * Internal commandlet to fix any USoundWaves with corrupt TimecodeInfo.
 */
UCLASS()
class UFixSoundWaveTimecodeInfoCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	UFixSoundWaveTimecodeInfoCommandlet();

	virtual int32 Main(const FString& Params);
};
