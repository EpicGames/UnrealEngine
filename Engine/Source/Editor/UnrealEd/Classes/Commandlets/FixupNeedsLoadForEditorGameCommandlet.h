// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/ResavePackagesCommandlet.h"
#include "FixupNeedsLoadForEditorGameCommandlet.generated.h"

UCLASS()
class UFixupNeedsLoadForEditorGameCommandlet : public UResavePackagesCommandlet
{
	GENERATED_BODY()

	/** Map of class names and their default NeedsLoadForEditorGame() return values. */
	TMap<FString, bool> ResaveClassNeedsLoadForEditorGameValues;

protected:

	//~ Begin UResavePackagesCommandlet Interface
	virtual int32 InitializeResaveParameters(const TArray<FString>& Tokens, TArray<FString>& MapPathNames) override;
	virtual void PerformPreloadOperations(FLinkerLoad* PackageLinker, bool& bSavePackage) override;
	//~ End UCommandlet Interface
};
