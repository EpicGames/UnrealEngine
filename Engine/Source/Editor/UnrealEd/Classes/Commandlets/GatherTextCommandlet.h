// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GatherTextCommandlet.generated.h"

USTRUCT()
struct FGatherTextCommandletTask
{
	GENERATED_BODY();

public:
	/** Params to pass the the Main function of the commandlet */
	UPROPERTY()
	FString CommandletParams;

	/** Commandlet to invoke the Main function of when running this task */
	UPROPERTY()
	TObjectPtr<UGatherTextCommandletBase> Commandlet;
};

USTRUCT()
struct FGatherTextCommandletPhase
{
	GENERATED_BODY();

public:
	/** Array of tasks in this phase that need to run in sequence; these will run before ParallelTasks */
	UPROPERTY()
	TArray<FGatherTextCommandletTask> SequentialTasks;

	/** Array of tasks in this phase that may run in parallel (if enabled); these will run after SequentialTasks */
	UPROPERTY()
	TArray<FGatherTextCommandletTask> ParallelTasks;
};

/**
 *	UGatherTextCommandlet: One commandlet to rule them all. This commandlet loads a config file and then calls other localization commandlets. Allows localization system to be easily extendable and flexible. 
 */
UCLASS(MinimalAPI)
class UGatherTextCommandlet final : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()
public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	/** Internal implementation of Main that can be used to provide additional options when running embedded within another process */
	UNREALED_API int32 Execute(const FString& Params, const TSharedPtr<const FGatherTextCommandletEmbeddedContext>& InEmbeddedContext);

	static const FString UsageText;
//~ Begin UGatherTextCommandletBase  Interface
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const override
	{
		// This commandlet is the driver for other commandlet. This should always run even in preview
		return true;
	}
	//~ End UGatherTextCommandletBase  Interface

private:
	/** Schedule the steps within all gather configs into GatherTextPhases */
	bool ScheduleGatherConfigs(const TArray<FString>& GatherTextConfigPaths, const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo, const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals);

	/** Schedule the steps within this gather config into GatherTextPhases */
	bool ScheduleGatherConfig(const FString& GatherTextConfigPath, const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo, const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals);

	/** Execute the work that was scheduled into GatherTextPhases */
	bool ExecuteGatherTextPhases(const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo);

	/** Clean-up any stale per-platform data in SplitPlatformConfigs */
	void CleanupStalePlatformData();

	// Helper function to generate a changelist description
	FText GetChangelistDescription(const TArray<FString>& GatherTextConfigPaths) const;

	/** The set of tasks this commandlet will run, distributed via a sequential set of phases */
	UPROPERTY()
	FGatherTextCommandletPhase GatherTextPhases[(int32)EGatherTextCommandletPhase::NumPhases];

	/** Set of split-platform root paths that we will process in CleanupStalePlatformData */
	struct FSplitPlatformConfig
	{
		TSet<FString> PlatformsToSplit;
		bool bShouldSplitPlatformData = false;
	};
	TMap<FString, FSplitPlatformConfig> SplitPlatformConfigs;

	bool bLegacyScheduler = false;
	bool bRunningInPreview = false;
};
