// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "LocTextHelper.h"
#include "LocalizationSourceControlUtil.h"
#include "LocalizedAssetUtil.h"
#include "GatherTextCommandletBase.generated.h"

struct FGatherTextContext
{
	enum class EPreferredPathType : uint8
	{
		/** Generic "root" path, eg) the root folder of a plugin */
		Root,
		/** Content path, eg) the content folder of a plugin */
		Content,
	};

	FTopLevelAssetPath CommandletClass;
	EPreferredPathType PreferredPathType = EPreferredPathType::Root;
};

struct FGatherTextDelegates
{
	/** Delegate called during a localization gather to allow code to inject new gather and exclude paths for the given localization target */
	// TODO: Deprecate in favor of GetAdditionalGatherPathsForContext
	using FGetAdditionalGatherPaths = TMulticastDelegate<void(const FString& InLocalizationTargetName, TArray<FString>& InOutIncludePathFilters, TArray<FString>& InOutExcludePathFilters)>;
	static UNREALED_API FGetAdditionalGatherPaths GetAdditionalGatherPaths;

	/** Delegate called during a localization gather to allow code to inject new gather and exclude paths for the given localization target (with context) */
	using FGetAdditionalGatherPathsForContext = TMulticastDelegate<void(const FString& InLocalizationTargetName, const FGatherTextContext& InContext, TArray<FString>& InOutIncludePathFilters, TArray<FString>& InOutExcludePathFilters)>;
	static UNREALED_API FGetAdditionalGatherPathsForContext GetAdditionalGatherPathsForContext;
};

/** Performs fuzzy path matching against a set of include and exclude paths */
class FFuzzyPathMatcher
{
public:
	enum class EPathMatch
	{
		Included,
		Excluded,
		NoMatch,
	};

public:
	UNREALED_API FFuzzyPathMatcher(const TArray<FString>& InIncludePathFilters, const TArray<FString>& InExcludePathFilters);

	UNREALED_API EPathMatch TestPath(const FString& InPathToTest) const;

	/**
	 * The algorithm used to test path matches for a fuzzy path. Defaults to FString::MatchesWildcard
	 */
	enum class EPathTestPolicy : uint8
	{
		/** Performs the path test with FString::MatchesWildcard. This is the default algorithm to use for fuzzy paths that can't be optimized with FString::StartsWith.*/
		MatchesWildcard,
		/** Uses FSTring::StartsWith to perform the path test against this fuzzy path. This is an optimization for fuzzy paths that only contain a single wildcard and the * wildcard only exists at the end of the fuzzy path. */
		StartsWith
	};

	static UNREALED_API EPathTestPolicy CalculatePolicyForPath(const FString& InPath);

private:
	enum class EPathType : uint8
	{
		Include,
		Exclude,
	};

	struct FFuzzyPath
	{
		FFuzzyPath(FString InPathFilter, const EPathType InPathType);

		FString PathFilter;
		EPathType PathType;
		EPathTestPolicy PathTestPolicy;
	};

	TArray<FFuzzyPath> FuzzyPaths;
};

/**
 * Additional options and hooks that can be specified when running the GatherText commandlet embedded within another process
 * @see UGatherTextCommandlet::Execute
 */
struct FGatherTextCommandletEmbeddedContext
{
	/** Optional override for the message of the overall localization slow task */
	TOptional<FText> SlowTaskMessageOverride;

	/** Callback used to perform additional tick tasks during the gather process */
	TFunction<void()> TickCallback;

	/** Callback used to allow user termination of the gather process */
	TFunction<bool()> WasAbortRequestedCallback;

	/** If true, the provided TickCallback and WasAbortRequestedCallback are both thread-safe and we are allowed to run tasks in parallel */
	bool bIsThreadSafe = false;

	bool IsThreadSafe() const
	{
		return bIsThreadSafe || (!TickCallback && !WasAbortRequestedCallback);
	}

	void RunTick() const
	{
		if (TickCallback)
		{
			TickCallback();
		}
	}

	bool ShouldAbort() const
	{
		return WasAbortRequestedCallback && WasAbortRequestedCallback();
	}
};

/**
 * The GatherText commandlet runs its steps over multiple sequential phases that may contain separate sets of sequential or parallel steps.
 * This enum defines which phases are available, and allows the sub-commandlets to define which phase they need to run in.
 * The phases run in the below order, up-to NumPhases. Sequential steps run before parallel steps within the same phase.
 */
enum class EGatherTextCommandletPhase : uint8
{
	/** Runs before the Initialize phase starts */
	PreInitialize = 0,
	/** The main Initialize phase (eg, to perform initialization needed by the other phases) */
	Initialize,
	/** Runs after the Initialize phase ends */
	PostInitialize,

	/** Runs before the UpdateManifests phase starts */
	PreUpdateManifests,
	/** The main UpdateManifests phase, where text is harvested and added to GatherManifestHelper, and the manifest files are updated on-disk */
	UpdateManifests,
	/** Runs after the UpdateManifests phase ends */
	PostUpdateManifests,

	/** Runs before the UpdateArchives phase starts */
	PreUpdateArchives,
	/** The main UpdateArchives phase, where the archive files are updated on-disk from the current manifest state */
	UpdateArchives,
	/** Runs after the UpdateArchives phase ends */
	PostUpdateArchives,

	/** Runs before the Import phase starts */
	PreImport,
	/** The main Import phase, where the archive files are updated on-disk from external file data (eg, PO files) */
	Import,
	/** Runs after the Import phase ends */
	PostImport,

	/** Runs before the Export phase starts */
	PreExport,
	/** The main Export phase, where external file data (eg, PO files), are updated from the archives */
	Export,
	/** Runs after the Export phase ends */
	PostExport,

	/** Runs before the Compile phase starts */
	PreCompile,
	/** The main Compile phase, where runtime files (eg, LocRes files), are updated from the archives */
	Compile,
	/** Runs after the Compile phase ends */
	PostCompile,

	/** Runs before the Finalize phase starts */
	PreFinalize,
	/** The main Finalize phase (eg, to perform clean-up or reporting of work done by the other phases) */
	Finalize,
	/** Runs after the Finalize phase ends */
	PostFinalize,

	/** Internal; do not use */
	NumPhases,

	/**
	 * No specific phase has been specified, and this will be scheduled in whichever phase we are currently building.
	 * You should upgrade your commandlet to use a specific phase to ensure it gets sequenced correctly!
	 */
	Undefined,

	/** Internal; do not use */
	FirstPhase = PreInitialize,
};

UNREALED_API const TCHAR* LexToString(EGatherTextCommandletPhase Phase);

/**
 *	UGatherTextCommandletBase: Base class for localization commandlets. Just to force certain behaviors and provide helper functionality. 
 */
UCLASS(MinimalAPI)
class UGatherTextCommandletBase : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	/** Called once before GetPhase to allow the commandlet to compute which phase it needs to run it. Params are the same as are passed to Main. */
	virtual bool ConfigurePhase(const FString& Params) { return true; }
	/** Which phase does this task need to run in? */
	virtual EGatherTextCommandletPhase GetPhase() const { return EGatherTextCommandletPhase::Undefined; }
	/** Can this task run in parallel with other tasks in its phase? If true, a random thread may invoke Main so any external/shared resources used must be thread-safe. */
	virtual bool CanParallelize() const { return false; }

	UNREALED_API void SetEmbeddedContext(const TSharedPtr<const FGatherTextCommandletEmbeddedContext>& InEmbeddedContext);
	UNREALED_API virtual void Initialize( const TSharedRef< FLocTextHelper >& InGatherManifestHelper, const TSharedPtr< FLocalizationSCC >& InSourceControlInfo );
	UNREALED_API virtual void BeginDestroy() override;

	// Wrappers for extracting config values
	UNREALED_API bool GetBoolFromConfig( const TCHAR* Section, const TCHAR* Key, bool& OutValue, const FString& Filename );
	UNREALED_API bool GetStringFromConfig( const TCHAR* Section, const TCHAR* Key, FString& OutValue, const FString& Filename );
	UNREALED_API bool GetPathFromConfig( const TCHAR* Section, const TCHAR* Key, FString& OutValue, const FString& Filename );
	UNREALED_API int32 GetStringArrayFromConfig( const TCHAR* Section, const TCHAR* Key, TArray<FString>& OutArr, const FString& Filename );
	UNREALED_API int32 GetPathArrayFromConfig( const TCHAR* Section, const TCHAR* Key, TArray<FString>& OutArr, const FString& Filename );

	// Utilities for split platform detection
	UNREALED_API bool IsSplitPlatformName(const FName InPlatformName) const;
	UNREALED_API bool ShouldSplitPlatformForPath(const FString& InPath, FName* OutPlatformName = nullptr) const;
	UNREALED_API FName GetSplitPlatformNameFromPath(const FString& InPath) const;

	// Utility to get the correct base path (engine or project) for the current environment
	static UNREALED_API const FString& GetProjectBasePath();

	/**
	* Returns true if this commandlet should run during a preview run.
	* Override in child classes to conditionally skip a commandlet from being run.
	* Most commandlets that require source control, write to files etc should be skipped for preview runs
	*/
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
	{
		return false;
	}

protected:
	void ResolveLocalizationPath(FString& InOutPath);

	static FName GetSplitPlatformNameFromPath_Static(const FString& InPath, const TMap<FName, FString>& InSplitPlatforms);

	TSharedPtr<const FGatherTextCommandletEmbeddedContext> EmbeddedContext;

	TSharedPtr< FLocTextHelper > GatherManifestHelper;

	TSharedPtr< FLocalizationSCC > SourceControlInfo;

	/** Mapping from platform name to the path marker for that platform */
	TMap<FName, FString> SplitPlatforms;

	// Common params and switches among all text gathering commandlets 
	static UNREALED_API const TCHAR* ConfigParam;
	static UNREALED_API const TCHAR* EnableSourceControlSwitch;
	static UNREALED_API const TCHAR* DisableSubmitSwitch;
	static UNREALED_API const TCHAR* PreviewSwitch;
	static UNREALED_API const TCHAR* GatherTypeParam;
	static UNREALED_API const TCHAR* SkipNestedMacroPrepassSwitch;

private:
	UNREALED_API virtual void CreateCustomEngine(const FString& Params) override ; //Disallow other text commandlets to make their own engine.	
};
