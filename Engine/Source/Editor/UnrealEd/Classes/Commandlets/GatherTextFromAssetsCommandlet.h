// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "Containers/MpscQueue.h"
#include "IMessageContext.h"
#include "Internationalization/GatherableTextData.h"
#include "Templates/PimplPtr.h"

#include "GatherTextFromAssetsCommandlet.generated.h"

class FGatherTextMetaDataHelper;

struct FARFilter;
struct FPackageFileSummary;

class FMessageEndpoint;

USTRUCT()
struct FGatherTextFromAssetsWorkerMessage_Ping
{
	GENERATED_BODY()

	UPROPERTY()
	int32 ProtocolVersion = 0;
};

USTRUCT()
struct FGatherTextFromAssetsWorkerMessage_Pong
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid WorkerId;

	UPROPERTY()
	TOptional<FDateTime> IdleStartTimeUtc;
};

USTRUCT()
struct FGatherTextFromAssetsWorkerMessage_PackageRequest
{
	GENERATED_BODY()

	UPROPERTY()
	FName PackageName;

	UPROPERTY()
	TSet<FName> Dependencies;

	UPROPERTY()
	TSet<FGuid> ExternalActors;

	UPROPERTY()
	TSet<FName> ExternalPackages;

	UPROPERTY()
	bool bIsMapPackage = false;
};

USTRUCT()
struct FGatherTextFromAssetsWorkerMessage_PackageResult
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid WorkerId;

	UPROPERTY()
	FName PackageName;

	UPROPERTY()
	TArray<uint8> GatherableTextData;

	UPROPERTY()
	FString LoadLogCapture;

	UPROPERTY()
	bool bLoadError = false;
};

/**
 *	UGatherTextFromAssetsCommandlet: Localization commandlet that collects all text to be localized from the game assets.
 */
UCLASS()
class UGatherTextFromAssetsCommandlet : public UGatherTextCommandletBase
{
	GENERATED_UCLASS_BODY()

	void StartWorkers(const int32 MinPackagesToUseWorkers);
	void AssignPackagesToWorkers(TConstArrayView<FGuid> IdleWorkerIds); // Note: You must hold PackagesPendingGatherMutex when calling this function
	void IngestPackageResultFromWorker(const FGatherTextFromAssetsWorkerMessage_PackageResult& PackageResult, const bool bSendWorkIfIdle = false);

	void ProcessGatherableTextDataArray(const TArray<FGatherableTextData>& GatherableTextDataArray);
	
	void CalculateDependenciesForPackagesPendingGather();

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;

	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::UpdateManifests; }

	bool GetConfigurationScript(const TMap<FString, FString>& InCommandLineParameters, FString& OutFilePath, FString& OutStepSectionName);
	bool ConfigureFromScript(const FString& GatherTextConfigPath, const FString& SectionName);

	//~ End UCommandlet Interface
	//~ Begin UGatherTextCommandletBase  Interface
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const override;
	//~ End UGatherTextCommandletBase  Interface

	/** Localization cache states of a package */
	enum class EPackageLocCacheState : uint8
	{
		Uncached_TooOld = 0,
		Uncached_NoCache,
		Cached, // Cached must come last as it acts as a count for an array
	};

private:
	/** Parses the command line for the commandlet. Returns true if all required parameters are provided and are correct.*/
	bool ParseCommandLineHelper(const FString& InCommandLine);

// Filtering of asset registry elements
	// Broadly, there is the first pass filter,the exact class filter and the include/exclude path filter that can be applied to filter out asset registry elements.
	// Look at Main() to see how the functions are applied to understand the logic.
	bool PerformFirstPassFilter(TArray<FAssetData>& OutAssetDataArray) const;
	void ApplyFirstPassFilter(const FARFilter& InFilter, TArray<FAssetData>& InOutAssetDataArray) const;
	bool BuildFirstPassFilter(FARFilter& InOutFilter) const;
	bool BuildCollectionFilter(FARFilter& InOutFilter, const TArray<FString>& Collections) const;
	bool BuildPackagePathsFilter(FARFilter& InOutFilter) const;
	bool BuildExcludeDerivedClassesFilter(FARFilter& InOutFilter) const;
	bool PerformExcludeExactClassesFilter(TArray<FAssetData>& InOutAssetDataArray) const;
	bool BuildExcludeExactClassesFilter(FARFilter& InOutFilter) const;
	void ApplyExcludeExactClassesFilter(const FARFilter& InFilter, TArray<FAssetData>& InOutAssetDataArray) const;
	void FilterAssetsBasedOnIncludeExcludePaths(TArray<FAssetData>& InOutAssetDataArray) const;
	
	bool DiscoverExternalPackages(TArray<FAssetData>& InOutAssetDataArray);
	void RemoveExistingExternalPackages(TArray<FAssetData>& InOutAssetDataArray, const TSet<FName>* WorldPackageFilter, TSet<FName>& OutExternalActorsWorldPackageNames, TSet<FName>& OutGameFeatureDataPackageNames) const;

	void ExtractPackageNamesToGather(const TArray<FAssetData>& InAssetDataArray, TSet<FName>& OutPackageNames, TMap<FName, FName>& OutExternalToOuterPackages) const;
	void PopulatePackagesPendingGather(TSet<FName> PackageNamesToGather);
	void ProcessAndRemoveCachedPackages(const TMap<FName, FName>& ExternalToOuterPackages, TMap<FName, TSet<FGuid>>& OutExternalActorsWithStaleOrMissingCaches, TMap<FName, TSet<FName>>& OutExternalPackagesWithStaleOrMissingCaches);
	void MergeInExternalPackagesWithStaleOrMissingCaches(TMap<FName, TSet<FGuid>>& ExternalActorsWithStaleOrMissingCaches, TMap<FName, TSet<FName>>& ExternalPackagesWithStaleOrMissingCaches);

	bool LoadAndProcessUncachedPackages(TArray<FName>& OutPackagesWithStaleGatherCache);

	void ReportStaleGatherCache(TArray<FName>& InPackagesWithStaleGatherCache) const;
	/** Determines the loc cache state for a package. This determines whether the package should be fully loaded for gathering.*/
	EPackageLocCacheState CalculatePackageLocCacheState(const FPackageFileSummary& PackageFileSummary, const FName PackageName, bool bIsExternalActorPackage, bool bRequiresMetaDataGather) const;
	/** Struct containing the data needed by a pending package that we will gather text from */
	struct FPackagePendingGather
	{
		/** The name of the package */
		FName PackageName;

		/** The filename of the package on disk */
		FString PackageFilename;

		/** The complete set of dependencies for the package */
		TSet<FName> Dependencies;

		/** The set of external actors to process for a world partition map package */
		TSet<FGuid> ExternalActors;

		/** The set of external packages to process */
		TSet<FName> ExternalPackages;

		/** The localization ID of this package, if any */
		FString PackageLocalizationId;

		/** True if this package contains a map */
		bool bIsMapPackage = false;

		/** True if this package must also gather metadata (this will also process it locally rather than on a worker) */
		bool bRequiresMetaDataGather = false;

		/** Localization cache state of this package */
		EPackageLocCacheState PackageLocCacheState;

		/** Contains the localization cache data for this package (if cached) */
		TArray<FGatherableTextData> GatherableTextDataArray;
	};

	/** Adds a package to PackagesPendingGather and returns a pointer to the appended package.*/
	FPackagePendingGather* AppendPackagePendingGather(const FName PackageNameToGather);

	static const FString UsageText;

	TArray<FString> ModulesToPreload;
	TArray<FString> IncludePathFilters;
	TArray<FString> CollectionFilters;
	TArray<FString> WorldCollectionFilters;
	TArray<FString> ExcludePathFilters;
	TArray<FString> PackageFileNameFilters;
	TArray<FString> ExcludeClassNames;
	TArray<FString> ManifestDependenciesList;

	TArray<FPackagePendingGather> PackagesPendingGather;
	mutable UE::FMutex PackagesPendingGatherMutex;

	TSet<FName> PackagesWithDuplicateLocalizationIds;

	/** Record of which packages were sent to workers, so that we can re-add them to the main PackagesPendingGather queue if the worker crashes */
	TMap<FName, FPackagePendingGather> PackagesDistributedToWorkers;
	mutable UE::FMutex PackagesDistributedToWorkersMutex;

	mutable UE::FMutex GatherManifestHelperMutex;

	/** Stats for the uncached package loading */
	int32 TotalNumUncachedPackages = 0;
	std::atomic<int32> NumUncachedPackagesProcessedLocally = 0;
	std::atomic<int32> NumUncachedPackagesProcessedRemotely = 0;

	/** Number of worker processes to use */
	int32 NumWorkers = 0;

	/** True if we should start the worker processes immediately, rather than wait until we know we have packages to load */
	bool bStartWorkersImmediately = false;

	/** True if we should start the worker processes always, even if we don't meet the MinPackagesToUseWorkers threshold */
	bool bStartWorkersAlways = false;

	/** Run a GC if the free system memory is less than this value (or zero to disable) */
	uint64 MinFreeMemoryBytes;

	/** Run a GC if the used process memory is greater than this value (or zero to disable) */
	uint64 MaxUsedMemoryBytes;

	uint64 NumPackagesDupLocId;

	/** Path to the directory where output reports etc will be saved.*/
	FString DestinationPath;
	bool bSkipGatherCache;
	bool bReportStaleGatherCache;
	bool bFixStaleGatherCache;
	bool bFixMissingGatherCache;
	bool bSearchAllAssets;
	bool bApplyRedirectorsToCollections;
	bool bShouldGatherFromEditorOnlyData;
	bool bShouldExcludeDerivedClasses;
	bool bFixPackageLocalizationIdConflict;

	/** Helper for gathering text from field metadata (Blueprint/UserDefinedStruct/UserDefinedEnum). Non-null only when InputKeys is configured. */
	TPimplPtr<FGatherTextMetaDataHelper> MetaDataHelper;
};

/**
 *	UGatherTextFromAssetsWorkerCommandlet: Localization commandlet worker process that collects all text to be localized from the requested assets.
 */
UCLASS()
class UGatherTextFromAssetsWorkerCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

private:
	void HandlePingMessage(const FGatherTextFromAssetsWorkerMessage_Ping& Message, const TSharedRef<IMessageContext>& Context);
	void HandlePackageRequestMessage(const FGatherTextFromAssetsWorkerMessage_PackageRequest& Message, const TSharedRef<IMessageContext>& Context);

	struct FPackagePendingGather
	{
		/** The name of the package */
		FName PackageName;

		/** The complete set of dependencies for the package */
		TSet<FName> Dependencies;

		/** The set of external actors to process for a world partition map package */
		TSet<FGuid> ExternalActors;

		/** The set of external packages to process */
		TSet<FName> ExternalPackages;

		/** True if this package contains a map */
		bool bIsMapPackage = false;

		/** Address of the endpoint that requested this package to be gathered */
		FMessageAddress RequestorAddress;
	};
	TArray<FPackagePendingGather> PackagesPendingGather;
	mutable UE::FMutex PackagesPendingGatherMutex;

	/** Run a GC if the free system memory is less than this value (or zero to disable) */
	uint64 MinFreeMemoryBytes = 0;

	/** Run a GC if the used process memory is greater than this value (or zero to disable) */
	uint64 MaxUsedMemoryBytes = 0;

	/** ID of this worker process */
	FGuid WorkerId;

	/** Holds the messaging endpoint we are sending from */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	/** If this worker is currently idle, when did it last enter an idle state? */
	TOptional<FDateTime> IdleStartTimeUtc;
	mutable UE::FMutex IdleStartTimeUtcMutex;
};

/**
 * Director for managing worker processes from the main gather commandlet.
 * @note This is separate from UGatherTextFromAssetsCommandlet as UGatherTextFromAssetsCommandlet may be invoked multiple times within a single gather.
 * 
 * Example usage:
 *	StartWorkers(...);
 *	WaitForWorkersToStart();
 *	AssignPackageToWorker(...);
 *	while (!IsIdle())
 *	{
 *		TickWorkers(); // Process GT messages and detect crashed/hung workers
 *		
 *		while (TOptional<...> Result = IngestPackageResult())
 *		{
 *			// Merge worker result into main result set
 *		}
 * 
 *		... = IngestPackagesFromCrashedWorkers(); // Merge back into main job queue
 *	}
 *	StopWorkers();
 * 
 * Alternatively you may skip the blocking wait, and use GetAvailableWorkerIds to lazily assign jobs to the workers once available:
 *	StartWorkers(...);
 *	for (;;)
 *	{
 *		TickWorkers(); // Process GT messages and detect crashed/hung workers
 *
 *		if (... = GetAvailableWorkerIds(0))
 *		{
 *			AssignPackageToWorker(...);
 *		}
 * 
 *		while (TOptional<...> Result = IngestPackageResult())
 *		{
 *			// Merge worker result into main result set
 *		}
 *
 *		... = IngestPackagesFromCrashedWorkers(); // Merge back into main job queue
 * 
 *		if (IsIdle())
 *		{
 *			break;
 *		}
 *	}
 *	StopWorkers();
 */
class FGatherTextFromAssetsWorkerDirector
{
private:
	FGatherTextFromAssetsWorkerDirector() = default;

public:
	~FGatherTextFromAssetsWorkerDirector();

	/**
	 * Get the singleton director instance.
	 */
	static FGatherTextFromAssetsWorkerDirector& Get();

	/**
	 * Start the given number of worker processes.
	 * @note This function will return immediately; call WaitForWorkersToStart if you need to wait for new workers to enter a state where they can accept work.
	 * 
	 * @param NumWorkers					The number of worker processes to use.
	 * @param bStopAdditionalWorkers		True to stop any additional worker processes (>NumWorkers) that are already running.
	 * @param NumRestartAttemptsIfCrashed	How many times should we attempt to restart the workers if they crash?
	 * 
	 * @return True if the workers could be started, false otherwise.
	 */
	bool StartWorkers(const int32 NumWorkers, const bool bStopAdditionalWorkers = true, const int32 NumRestartAttemptsIfCrashed = 0);

	/**
	 * Block while any workers created by StartWorkers are in state where they can accept work.
	 * 
	 * @param Timeout Optional timeout to wait for, or unset to wait for an unlimited amount of time.
	 * 
	 * @return True if all workers started in the given timeout, false otherwise.
	 */
	bool WaitForWorkersToStart(const TOptional<FTimespan> Timeout = {});

	/**
	 * Stop the current workers (if any), discarding any work that may currently be assigned to them.
	 */
	bool StopWorkers();

	/**
	 * Does this director have workers?
	 * @note They may be in any state and not ready to accept work (see GetAvailableWorkerIds).
	 */
	bool HasWorkers() const;

	/**
	 * Tick workers, to detect which have crashed or hung.
	 */
	void TickWorkers();

	/**
	 * Get the IDs of the available workers (that are ready to accept work), optionally only returning workers currently meet the given idle threshold (have <= IdleThreshold packages assigned).
	 * 
	 * @param IdleThreshold If set, only workers that have <= IdleThreshold packages assigned will be returned.
	 * 
	 * @return The available worker IDs, to be used with AssignPackageToWorker.
	 */
	TArray<FGuid> GetAvailableWorkerIds(const TOptional<int32> IdleThreshold = {}) const;

	/**
	 * Is this director idle?
	 * @note Idle means that all workers have no pending work and there is there no work left to ingest into the main process
	 * 
	 * @return True if idle, false otherwise.
	 */
	bool IsIdle(int32* OutNumPendingWorkerPackages = nullptr) const;

	/**
	 * Assign the given package to the given worker.
	 * 
	 * @param WorkerId			ID of the worker to assign the package to (see GetAvailableWorkerIds).
	 * @param PackageRequest	Information about the package to be assigned.
	 * 
	 * @return True if the assignment was possible (the worker exists), false otherwise.
	 */
	bool AssignPackageToWorker(const FGuid& WorkerId, const FGatherTextFromAssetsWorkerMessage_PackageRequest& PackageRequest);

	/**
	 * Set whether or not worker processes are allowed to read the asset registry cache.
	 */
	void SetWorkersCanReadAssetRegistryCache(const bool bValue)
	{
		bWorkersCanReadAssetRegistryCache = bValue;
	}

	/**
	 * Set the handler used to automatically ingest package results as they arrive (from any thread), rather than add them to PackageResults for deferred processing via IngestPackageResult.
	 * @note The handler may return false if it cannot handle a given result, at which point it will still be added to PackageResults.
	 */
	void SetIngestPackageResultHandler(TFunction<bool(const FGatherTextFromAssetsWorkerMessage_PackageResult&)>&& Handler);

	/**
	 * Clear any currently set handler (@see SetIngestPackageResultHandler).
	 */
	void ClearIngestPackageResultHandler();

	/**
	 * Ingest the next package result received from any worker, if any.
	 * 
	 * @return The package result, or an unset value.
	 */
	TOptional<FGatherTextFromAssetsWorkerMessage_PackageResult> IngestPackageResult();

	/**
	 * Ingest the current queue of packages that failed to be processed due to their worker crashing.
	 * 
	 * @return The array of packages to handle within the main process.
	 */
	TArray<FName> IngestPackagesFromCrashedWorkers();

private:
	/** Utilities for FWorkerInfo; these should be called while CurrentWorkersMutex is held */
	struct FWorkerInfo;
	FString GenerateWorkerCommandLine(const FGuid& WorkerId);
	FProcHandle StartWorker(const FString& WorkerCommandLine);
	bool HandleWorkerCrashed(const FGuid& WorkerId, FWorkerInfo& WorkerInfo);
	void ResetWorkerDiscoveryAndTimeout(FWorkerInfo& WorkerInfo);
	void ResendWorkerPendingPackageRequests(FWorkerInfo& WorkerInfo);

	void BroadcastPingMessage(const bool bIgnoreDelay = false);

	void HandlePongMessage(const FGatherTextFromAssetsWorkerMessage_Pong& Message, const TSharedRef<IMessageContext>& Context);
	void HandlePackageResultMessage(const FGatherTextFromAssetsWorkerMessage_PackageResult& Message, const TSharedRef<IMessageContext>& Context);

	/** Information about the current worker processes, including the work that has been assigned to them */
	struct FWorkerInfo
	{
		FProcHandle WorkerProc;
		FString WorkerCommandLine;
		FMessageAddress EndpointAddress;
		TOptional<FDateTime> LastMessageReceivedUtc;
		TOptional<FDateTime> LastPackageRequestUtc;
		TOptional<FDateTime> IdleStartTimeUtc;
		TMap<FName, FGatherTextFromAssetsWorkerMessage_PackageRequest> PendingPackageRequests;
		int32 NumRestartAttemptsIfCrashed = 0;
		bool bResendPendingPackageRequests = false;
	};
	TMap<FGuid, TSharedPtr<FWorkerInfo>> CurrentWorkers;
	mutable UE::FMutex CurrentWorkersMutex;

	/** The last time that we broadcast a ping message */
	FDateTime LastPingBroadcastUtc = FDateTime::MinValue();

	/** The queue of results received from any worker */
	TMpscQueue<FGatherTextFromAssetsWorkerMessage_PackageResult> PackageResults;

	/** Optional handler that will attempt to process results immediately, rather than add them to PackageResults */
	TFunction<bool(const FGatherTextFromAssetsWorkerMessage_PackageResult&)> PackageResultHandler;

	/** The queue of packages that failed to be processed due to their worker crashing */
	TArray<FName> PackagesFromCrashedWorkers;

	/** Holds the messaging endpoint we are sending from */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	/** Are workers allowed to read the asset registry cache? */
	bool bWorkersCanReadAssetRegistryCache = false;
};
