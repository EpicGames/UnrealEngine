// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GatherTextFromMetadataCommandlet.generated.h"

class FGatherTextMetaDataHelper;

/**
 *	UGatherTextFromMetaDataCommandlet: Localization commandlet that collects all text to be localized from generated metadata.
 */
UCLASS()
class UGatherTextFromMetaDataCommandlet : public UGatherTextCommandletBase
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	//~ Begin UGatherTextCommandletBase  Interface
	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::UpdateManifests; }
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const override;
	//~ End UGatherTextCommandletBase  Interface

private:
	void GatherTextFromUObjects(const TArray<FString>& IncludePaths, const TArray<FString>& ExcludePaths);

	TPimplPtr<FGatherTextMetaDataHelper> MetaDataHelper;
};
