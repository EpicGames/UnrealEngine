// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Framework/Commands/Commands.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "Internationalization/StringTableCore.h"
#include "Misc/EnumClassFlags.h"
#include "GatherTextFromSourceCommandlet.generated.h"

class Error;

enum class EGatherTextSourceFileTypes : uint8
{
	None = 0,
	Cpp = 1 << 0,
	Ini = 1 << 1,
};
ENUM_CLASS_FLAGS(EGatherTextSourceFileTypes);

enum EGatherSourcePasses : uint8
{
	Prepass = 0,
	Mainpass = 1,
};

/**
 *	UGatherTextFromSourceCommandlet: Localization commandlet that collects all text to be localized from the source code.
 */
UCLASS()
class UGatherTextFromSourceCommandlet : public UGatherTextCommandletBase
{
	GENERATED_UCLASS_BODY()

private:
#define LOC_DEFINE_REGION

	enum class EEditorOnlyDefineState : uint8
	{
		Undefined,
		Defined,
	};

	struct FSourceLocation
	{
		FSourceLocation()
			: File()
			, Line(INDEX_NONE)
		{
		}

		FSourceLocation(FString InFile, const int32 InLine)
			: File(MoveTemp(InFile))
			, Line(InLine)
		{
		}

		FString ToString() const
		{
			return (Line == INDEX_NONE) ? File : FString::Printf(TEXT("%s(%d)"), *File, Line);
		}

		FString File;
		int32 Line;
	};

	struct FParsedStringTableEntry
	{
		FString SourceString;
		FSourceLocation SourceLocation;
		FName PlatformName;
		FString DevNotes;
		bool bIsEditorOnly;
	};

	struct FParsedStringTableEntryMetaData
	{
		FString MetaData;
		FSourceLocation SourceLocation;
		bool bIsEditorOnly;
	};

	typedef TMap<FName, FParsedStringTableEntryMetaData> FParsedStringTableEntryMetaDataMap;

	struct FParsedStringTable
	{
		FString TableNamespace;
		FSourceLocation SourceLocation;
		TMap<FString, FParsedStringTableEntry, FDefaultSetAllocator, FLocKeyMapFuncs<FParsedStringTableEntry>> TableEntries;
		TMap<FString, FParsedStringTableEntryMetaDataMap, FDefaultSetAllocator, FLocKeyMapFuncs<FParsedStringTableEntryMetaDataMap>> MetaDataEntries;
	};

	// Macro with nested standard macros, collected in a prepass
	struct FParsedNestedMacro
	{
		FString MacroName;			// Outer macro name
		FString MacroNameNested;	// Which nested macro (LOCTEXT, NSLOCTEXT, UI_COMMAND, UI_COMMAND_EXT) is contained by this macro
		FString Filename;
		FString Content;			// Lines of the macro, including following lines ending with '\' and one more
		int32 LineStart;
		int32 LineCount;
		bool bExclude = false;		// A duplicate macro in a header (.h or .inl) is excluded from parsing, see PrunePrepassResults

		FParsedNestedMacro()
		{}

		FParsedNestedMacro(const FString& InMacroName, const FString& InMacroNameNested, const FString& InFilename, const FString& InContent, int32 InLineStart, int32 InLineCount)
			: MacroName(InMacroName)
			, MacroNameNested(InMacroNameNested)
			, Filename(InFilename)
			, Content(InContent)
			, LineStart(InLineStart)
			, LineCount(InLineCount)
		{}

		inline bool operator==(const FParsedNestedMacro& Other) const
		{
			// It is sufficient to compare a subset to know they match. We can avoid comparing the larger Content field.
			return (MacroName == Other.MacroName &&
				MacroNameNested == Other.MacroNameNested &&
				Filename == Other.Filename &&
				LineStart == Other.LineStart);
		}

		static int32 Size(const FParsedNestedMacro& Result);
	};

	// Results of mainpass to submit to FLocTextHelper once parallel processing completes
	struct FManifestEntryResult
	{
		FLocKey Namespace;
		FString Source;
		FManifestContext Context;
		FString Description;

		FManifestEntryResult(const FLocKey& InNamespace, const FString& InSource, const FManifestContext& InContext, const FString& InDescription)
			: Namespace(InNamespace)
			, Source(InSource)
			, Context(InContext)
			, Description(InDescription)
		{}
	};

	class FMacroArgumentGatherer
	{
	public:
		FMacroArgumentGatherer() {};

		bool Gather(const TCHAR* Arg, int32 Count);
		bool EndArgument();

		// Return the number of argument that are completely resolved.
		int32 GetNumberOfArguments() const;

		void ExtractArguments(TArray<FString>& Arguments);
		
		void OpenDoubleQuotes()
		{
			bInDblQuotes = true;
		}

		void CloseDoubleQuotes()
		{
			bInDblQuotes = false;
		}

		bool IsInDoubleQuotes()  const
		{
			return bInDblQuotes;
		};
		
		void OpenSingleQuotes() 
		{
			bInSglQuotes = true;
		}

		void CloseSingleQuotes()
		{
			bInSglQuotes = false;
		}

		bool IsInSingleQuotes()  const
		{
			return bInSglQuotes;
		};

	private:
		TArray<FString> Args;
		FString CurrentArgument;
		bool bInDblQuotes = false;
		bool bInSglQuotes = false;
	};

	struct FSourceFileParseContext
	{
		void AddManifestText(const FString& Token, const FString& Namespace, const FString& SourceText, const FManifestContext& Context, bool IsNested);

		void PushMacroBlock( const FString& InBlockCtx );

		void PopMacroBlock();

		void FlushMacroStack();

		EEditorOnlyDefineState EvaluateEditorOnlyDefineState() const;

		void SetDefine( const FString& InDefineCtx );

		void RemoveDefine( const FString& InDefineCtx );

		void AddStringTable( const FName InTableId, const FString& InTableNamespace );

		void AddStringTableFromFile( const FName InTableId, const FString& InTableNamespace, const FString& InTableFilename, const FString& InRootPath );

		void AddStringTableEntry( const FName InTableId, const FString& InKey, const FString& InSourceString, const FString& InDevNotes );

		void AddStringTableEntryMetaData( const FName InTableId, const FString& InKey, const FName InMetaDataId, const FString& InMetaData );

		//Working data
		EGatherTextSourceFileTypes FileTypes;
		FString Filename;
		int32 LineIdx;					// Line index that is advanced by more than one in the prepass when collecting macros with nested macros
		int32 LineNumber;				// Log friendly index equal to (LineIdx + 1)
		FName FilePlatformName;
		FString LineText;
		FString Namespace;
		FString RawStringLiteralClosingDelim;
		bool ExcludedRegion;
		bool EndParsingCurrentLine;
		bool WithinBlockComment;
		bool WithinLineComment;
		bool WithinStringLiteral;
		int32 WithinNamespaceDefineLineNumber;
		const TCHAR* WithinStartingLine;

		//Should editor-only data be included in this gather?
		bool ShouldGatherFromEditorOnlyData;

		//Discovered string table data from all files
		TMap<FName, FParsedStringTable> ParsedStringTables;

		TArray<FString> TextLines;

		EGatherSourcePasses Pass;
		TArray<FManifestEntryResult>& MainpassResults;
		bool bIsNested = false;

		FSourceFileParseContext(const TMap<FName, FString>& inSplitPlatforms, TArray<FManifestEntryResult>& IoMainpassResults)
			: FileTypes(EGatherTextSourceFileTypes::None)
			, Filename()
			, LineIdx(0)
			, LineNumber(0)
			, FilePlatformName()
			, LineText()
			, Namespace()
			, ExcludedRegion(false)
			, EndParsingCurrentLine(false)
			, WithinBlockComment(false)
			, WithinLineComment(false)
			, WithinStringLiteral(false)
			, WithinNamespaceDefineLineNumber(INDEX_NONE)
			, WithinStartingLine(nullptr)
			, ShouldGatherFromEditorOnlyData(false)
			, Pass(EGatherSourcePasses::Prepass)
			, MainpassResults(IoMainpassResults)
			, MacroBlockStack()
			, CachedEditorOnlyDefineState()
			, SplitPlatforms(inSplitPlatforms)
		{
		}

	private:
		bool AddStringTableImpl( const FName InTableId, const FString& InTableNamespace );
		bool AddStringTableEntryImpl( const FName InTableId, const FString& InKey, const FString& InSourceString, const FSourceLocation& InSourceLocation, const FName InPlatformName, const FString& InDevNotes );
		bool AddStringTableEntryMetaDataImpl( const FName InTableId, const FString& InKey, const FName InMetaDataId, const FString& InMetaData, const FSourceLocation& InSourceLocation );

		//Working data
		TArray<FString> MacroBlockStack;
		mutable TOptional<EEditorOnlyDefineState> CachedEditorOnlyDefineState;

		TMap<FName, FString> SplitPlatforms;
	};

	class FParsableDescriptor
	{
	public:
		virtual ~FParsableDescriptor() = default;
		virtual const FString& GetToken() const = 0;
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const = 0;

		virtual bool IsApplicableFile(const FString& InFilename) const { return true; }
		bool IsApplicableFileType(const EGatherTextSourceFileTypes InFileTypes) const { return EnumHasAnyFlags(ApplicableFileTypes, InFileTypes); }
		bool OverridesLongerTokens() const { return bOverridesLongerTokens; }

	protected:
		EGatherTextSourceFileTypes ApplicableFileTypes = EGatherTextSourceFileTypes::None;
		bool bOverridesLongerTokens = false;
	};

	class FPreProcessorDescriptor : public FParsableDescriptor
	{
	public:
		FPreProcessorDescriptor()
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp;
			bOverridesLongerTokens = true;
		}

	protected:
		static const FString UndefString;
		static const FString IfString;
		static const FString IfDefString;
		static const FString ElIfString;
		static const FString ElseString;
		static const FString EndIfString;
		static const FString DefinedString;
		static const FString IniNamespaceString;
	};

	class FDefineDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return UGatherTextFromSourceCommandlet::DefineString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FUndefDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::UndefString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FIfDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::IfString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FIfDefDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::IfDefString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FElIfDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::ElIfString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FElseDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::ElseString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FEndIfDescriptor : public FPreProcessorDescriptor
	{
	public:
		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::EndIfString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FMacroDescriptor : public FParsableDescriptor
	{
	public:
		static const FString TextMacroString;

		FMacroDescriptor(FString InName, int32 InMinArgumentNumber)
			: Name(MoveTemp(InName))
			, MinArgumentNumber(InMinArgumentNumber)
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp;
		}

		virtual const FString& GetToken() const override
		{
			return Name;
		}

		int32 GetMinNumberOfArgument() const
		{
			return MinArgumentNumber;
		}

	protected:
		bool ParseArgsFromMacro(const FString& Text, TArray<FString>& Args, FSourceFileParseContext& Context) const;
		bool ParseArgsFromNextLines(FMacroArgumentGatherer& ArgsGatherer, int32& BracketStack, FSourceFileParseContext& Context) const;
		bool ParseArgumentString(const FString& Text, const int32 OpenBracketIdx, int32& BracketStack, const FSourceFileParseContext& Context, FMacroArgumentGatherer& ArgsGatherer) const;

		static bool PrepareArgument(FString& Argument, bool IsAutoText, const FString& IdentForLogging, bool& OutHasQuotes);

	private:
		const FString Name;

		// Minimum number argument for that Macro.
		const int32 MinArgumentNumber;
	};

	class FUICommandMacroDescriptor : public FMacroDescriptor
	{
	public:
		FUICommandMacroDescriptor()
			: FMacroDescriptor(MacroString_UI_COMMAND, 5)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;

	protected:
		FUICommandMacroDescriptor(FString InName, int32 InMinNumberOfArgument)
			: FMacroDescriptor(MoveTemp(InName), InMinNumberOfArgument)
		{
		}

		void TryParseArgs(const FString& Text, FSourceFileParseContext& Context, const TArray<FString>& Arguments, const int32 ArgIndexOffset) const;
	};

	class FUICommandExtMacroDescriptor : public FUICommandMacroDescriptor
	{
	public:
		FUICommandExtMacroDescriptor()
			: FUICommandMacroDescriptor(MacroString_UI_COMMAND_EXT, 5)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	/** This descripter runs in a prepass to collect macros with nested localizable macros
	* Example:
	*	#define METASOUND_PARAM(NAME, NAME_TEXT) \
	*		static const FText NAME##DisplayName = LOCTEXT(#NAME "DisplayName", NAME_TEXT);
	*/
	class FNestedMacroPrepassDescriptor : public FMacroDescriptor
	{
	public:
		FNestedMacroPrepassDescriptor(TArray<FParsedNestedMacro>& InPrepassResults)
			: FMacroDescriptor(UGatherTextFromSourceCommandlet::DefineString, INT_MAX)
			, PrepassResults(InPrepassResults)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;

	private:
		TArray<FParsedNestedMacro>& PrepassResults;
	};

	// This descriptor finds macros that match those found in the prepass (FNestedMacroPrepassDescriptor)
	class FNestedMacroDescriptor : public FMacroDescriptor
	{
	public:
		FNestedMacroDescriptor(FString InMacroName, FString InMacroNameNested, FString InFilename, FString InContent)
			: FMacroDescriptor(MoveTemp(InMacroName), 1)
			, MacroNameNested(MoveTemp(InMacroNameNested))
			, Filename(MoveTemp(InFilename))
			, Content(MoveTemp(InContent))
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;

		virtual bool IsApplicableFile(const FString& InFilename) const override;

		static void TestNestedMacroDescriptorParseArgs();

	private:
		static void TryParseArgs(const FString& MacroInnerParams, FString& ParamsNewAll);

		const FString MacroNameNested;
		const FString Filename;
		const FString Content;
	};

	class FStringMacroDescriptor : public FMacroDescriptor
	{
	public:
		enum EMacroArgSemantic
		{
			MAS_Namespace,
			MAS_Identifier,
			MAS_SourceText,
			MAS_DevNotes,
		};

		struct FMacroArg
		{
			EMacroArgSemantic Semantic;
			bool IsAutoText;

			FMacroArg(EMacroArgSemantic InSema, bool InIsAutoText) : Semantic(InSema), IsAutoText(InIsAutoText) {}
		};

		FStringMacroDescriptor(FString InName, FMacroArg Arg0, FMacroArg Arg1, FMacroArg Arg2, FMacroArg Arg3) : FMacroDescriptor(InName, 4)
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp | EGatherTextSourceFileTypes::Ini;
			Arguments.Add(Arg0);
			Arguments.Add(Arg1);
			Arguments.Add(Arg2);
			Arguments.Add(Arg3);
		}

		FStringMacroDescriptor(FString InName, FMacroArg Arg0, FMacroArg Arg1, FMacroArg Arg2) : FMacroDescriptor(InName, 3)
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp | EGatherTextSourceFileTypes::Ini;
			Arguments.Add(Arg0);
			Arguments.Add(Arg1);
			Arguments.Add(Arg2);
		}

		FStringMacroDescriptor(FString InName, FMacroArg Arg0, FMacroArg Arg1) : FMacroDescriptor(InName, 2)
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp | EGatherTextSourceFileTypes::Ini;
			Arguments.Add(Arg0);
			Arguments.Add(Arg1);
		}

		FStringMacroDescriptor(FString InName, FMacroArg Arg0) : FMacroDescriptor(InName, 1)
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Cpp | EGatherTextSourceFileTypes::Ini;
			Arguments.Add(Arg0);
		}

		virtual void TryParse(const FString& LineText, FSourceFileParseContext& Context) const override;

	private:
		TArray<FMacroArg> Arguments;
	};

	class FStringTableMacroDescriptor : public FMacroDescriptor
	{
	public:
		FStringTableMacroDescriptor()
			: FMacroDescriptor(TEXT("LOCTABLE_NEW"), 2)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FStringTableFromFileMacroDescriptor : public FMacroDescriptor
	{
	public:
		FStringTableFromFileMacroDescriptor(FString InName, FString InRootPath)
			: FMacroDescriptor(MoveTemp(InName), 3), RootPath(MoveTemp(InRootPath)) {}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;

	private:
		FString RootPath;
	};

	class FStringTableEntryMacroDescriptor : public FMacroDescriptor
	{
	public:
		FStringTableEntryMacroDescriptor()
			: FMacroDescriptor(TEXT("LOCTABLE_SETSTRING"), 3)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FStringTableNotedEntryMacroDescriptor : public FMacroDescriptor
	{
	public:
		FStringTableNotedEntryMacroDescriptor()
			: FMacroDescriptor(TEXT("LOCTABLE_SETNOTESTRING"), 4)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FStringTableEntryMetaDataMacroDescriptor : public FMacroDescriptor
	{
	public:
		FStringTableEntryMetaDataMacroDescriptor()
			: FMacroDescriptor(TEXT("LOCTABLE_SETMETA"), 4)
		{
		}

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	class FStructuredLogMacroDescriptor final : public FMacroDescriptor
	{
	public:
		enum class EFlags : int32
		{
			None = 0,
			Condition = 1 << 0,
			Namespace = 1 << 1,
		};

		static int32 CalculateMinimumArgumentCount(EFlags Flags);

		FStructuredLogMacroDescriptor(const TCHAR* Name, EFlags Flags);

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;

	private:
		EFlags Flags;
	};

	FRIEND_ENUM_CLASS_FLAGS(FStructuredLogMacroDescriptor::EFlags);

	class FDeclareErrorDescriptor final : public FMacroDescriptor
	{
	public:
		FDeclareErrorDescriptor(const TCHAR* InMacroName, const TCHAR* InKeyParameterName, int32 InMinArguments, int32 InNamespaceIndex, int32 InKeyIndex, int32 InTextIndex);

		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	protected:
		const FString KeyParameterName;
		// Indices of each localized text component in macro argument list
		int32 NamespaceIndex = -1;
		int32 KeyIndex = -1;
		int32 TextIndex = -1;
	};

	class FIniNamespaceDescriptor : public FPreProcessorDescriptor
	{
	public:
		FIniNamespaceDescriptor()
		{
			ApplicableFileTypes = EGatherTextSourceFileTypes::Ini;
		}

		virtual const FString& GetToken() const override { return FPreProcessorDescriptor::IniNamespaceString; }
		virtual void TryParse(const FString& Text, FSourceFileParseContext& Context) const override;
	};

	static const FString DefineString;
	static const FString MacroString_LOCTEXT;
	static const FString MacroString_NSLOCTEXT;
	static const FString MacroString_NOTELOCTEXT;
	static const FString MacroString_NSNOTELOCTEXT;
	static const FString MacroString_UI_COMMAND;
	static const FString MacroString_UI_COMMAND_EXT;

	void GetFilesToProcess(const TArray<FString>& SearchDirectoryPaths, const TArray<FString>& FileNameFilters, TArray<FString>& IncludePathFilters, TArray<FString>& ExcludePathFilters, TArray<FString>& FilesToProcess, bool bAdditionalGatherPaths) const;
	void GetParsables(TArray<FParsableDescriptor*>& Parsables, EGatherSourcePasses Pass, TArray<FParsedNestedMacro>& PrepassResults);
	void RunPass(EGatherSourcePasses Pass, bool ShouldGatherFromEditorOnlyData, const TArray<FString>& FilesToProcess, const FString& GatheredSourceBasePath, TArray<FParsedNestedMacro>& PrepassResults);

	static FString UnescapeLiteralCharacterEscapeSequences(const FString& InString);
	static FString RemoveStringFromTextMacro(const FString& TextMacro, const FString& IdentForLogging, bool& Error);
	static FString StripCommentsFromToken(const FString& InToken, FSourceFileParseContext& Context);
	static bool ParseSourceText(const FString& Text, const TArray<FParsableDescriptor*>& Parsables, FSourceFileParseContext& ParseCtxt, TArray<FParsedNestedMacro>& PrepassResults);
	static void CountFileTypes(const TArray<FString>& FilesToProcess, EGatherSourcePasses Pass);
	static void PrunePrepassResults(TArray<FParsedNestedMacro>& Results);
	static bool HandledInPrepass(const TArray<FParsedNestedMacro>& Results, const FString& Filename, int32 LineNumber, int32& AdvanceByLines);

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
	//~ Begin UGatherTextCommandletBase  Interface
	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::UpdateManifests; }
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const override;
	//~ End UGatherTextCommandletBase  Interface

	static void LogStats();
#undef LOC_DEFINE_REGION
};

ENUM_CLASS_FLAGS(UGatherTextFromSourceCommandlet::FStructuredLogMacroDescriptor::EFlags);
