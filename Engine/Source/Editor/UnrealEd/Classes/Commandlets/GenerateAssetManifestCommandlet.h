// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "GenerateAssetManifestCommandlet.generated.h"

/** Commandlet for generating a filtered list of assets from the asset registry (intended use is for replacing assets with cooked version) */
UCLASS()
class UGenerateAssetManifestCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};


