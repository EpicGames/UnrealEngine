// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "GenerateBlueprintAPICommandlet.generated.h"

UCLASS()
class UGenerateBlueprintAPICommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:		
	//~ Begin UCommandlet Interface
	virtual int32 Main(FString const& Params) override;
	//~ End UCommandlet Interface
};
