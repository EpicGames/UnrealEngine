// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Find AnimNotify instances that do not have the AnimSeqeunce that 'owns' them as their outer.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "GenerateDistillFileSetsCommandlet.generated.h"

UCLASS()
class UGenerateDistillFileSetsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
