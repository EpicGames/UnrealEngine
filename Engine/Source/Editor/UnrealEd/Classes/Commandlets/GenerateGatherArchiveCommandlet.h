// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GenerateGatherArchiveCommandlet.generated.h"

/**
 *	UGenerateGatherArchiveCommandlet: Generates a localisation archive; generally used as a gather step.
 */
UCLASS()
class UGenerateGatherArchiveCommandlet : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()
#if CPP || UE_BUILD_DOCS
public:
	//~ Begin UCommandlet Interface
	virtual int32 Main( const FString& Params ) override;
	//~ End UCommandlet Interface
	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::UpdateArchives; }
	virtual bool CanParallelize() const { return true; }
#endif
};
