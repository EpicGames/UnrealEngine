// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GenerateGatherManifestCommandlet.generated.h"

/**
 *	UGenerateGatherManifestCommandlet: Generates a localisation manifest; generally used as a gather step.
 */
UCLASS()
class UGenerateGatherManifestCommandlet : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()
#if CPP || UE_BUILD_DOCS
public:
	//~ Begin UCommandlet Interface
	virtual int32 Main( const FString& Params ) override;
	//~ End UCommandlet Interface
		//~ Begin UGatherTextCommandletBase  Interface
	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::UpdateManifests; }
	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const override;
	//~ End UGatherTextCommandletBase  Interface
#endif
};
