// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GenerateTextLocalizationReportCommandlet.generated.h"

UCLASS()
class UGenerateTextLocalizationReportCommandlet : public UGatherTextCommandletBase
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::Finalize; }
	virtual bool CanParallelize() const { return true; }

	virtual bool ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
	{
		return true;
	}
private:
	/**
	 * Collects word count info and writes out a report.
	 * @param SourcePath - Source path to localization files.
	 * @param DestinationPath - The destination folder the report will be write to.
	 * @return True if successful; false oterwise.
	 */
	bool ProcessWordCountReport(const FString& SourcePath, const FString& DestinationPath);

	/**
	 * Writes out a report representing localization conflicts.  These are conflicts where the namespace
	 * and key match but the source text differs.
	 * @param DestinationPath - The destination folder the report will be write to.
	 * @return True if successful; false oterwise.
	 */
	bool ProcessConflictReport(const FString& DestinationPath);
	
private:
	FString GatherTextConfigPath;
	FString SectionName;
	FString CmdlineTimeStamp;

};
