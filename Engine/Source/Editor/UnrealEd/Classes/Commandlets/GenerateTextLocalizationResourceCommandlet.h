// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "GenerateTextLocalizationResourceCommandlet.generated.h"

/**
 *	UGenerateTextLocalizationResourceCommandlet: Localization commandlet that generates a table of FText keys to localized string values.
 */

UCLASS()
class UGenerateTextLocalizationResourceCommandlet : public UGatherTextCommandletBase
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::Compile; }
	virtual bool CanParallelize() const { return true; }
};
