// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "ImportDialogueScriptCommandlet.generated.h"

/**
 *	UImportDialogueScriptCommandlet: Handles importing localized script sheets to update the translated archive text.
 */
UCLASS()
class UImportDialogueScriptCommandlet : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::Import; }

private:
	bool ImportDialogueScriptForCulture(FLocTextHelper& InLocTextHelper, const FString& InDialogueScriptFileName, const FString& InCultureName, const bool bIsNativeCulture);
};
