// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "Factories/SoundFactory.h"
#include "ImportLocalizedDialogueCommandlet.generated.h"

class UDialogueWave;
class USoundWave;

/**
 *	UImportLocalizedDialogueCommandlet: Handles synchronizing localized raw audio files with dialogue and sound wave assets.
 */
UCLASS()
class UImportLocalizedDialogueCommandlet : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	virtual EGatherTextCommandletPhase GetPhase() const override { return EGatherTextCommandletPhase::Import; }

private:
	struct FCultureImportInfo
	{
		FString Name;
		FString AudioPath;
		FString ArchiveFileName;
		FString LocalizedRootContentPath;
		FString LocalizedRootPackagePath;
		FString LocalizedImportedDialoguePackagePath;
		bool bIsNativeCulture;
	};

	bool ImportDialogueForCulture(FLocTextHelper& InLocTextHelper, UDialogueWave* const DialogueWave, const FString& DialogueWaveSubPath, const FCultureImportInfo& InCultureImportInfo, const bool bImportAsSource);

	USoundWave* ConditionalImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const;
	USoundWave* ImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const;

	TSet<FSoftObjectPath> AssetsToKeep;
};
