// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "InternationalizationConditioningCommandlet.generated.h"

/**
 *	UInternationalizationConditioningCommandlet: Commandlet that contains various misc functionality to prepare, modify, and condition Internationalization manifest
 *  and archive data. 
 */
UCLASS()
class UInternationalizationConditioningCommandlet : public UGatherTextCommandletBase
{
    GENERATED_UCLASS_BODY()

	
	/**
	 * Contains all the info we need to create archive/manifest entries from.
	 */
	struct FLocalizationFileEntry
	{
	public:
		FLocalizationFileEntry( const FString& InFile, const FString& InNamespace, const FString& InKey, const FString& InSourceText, const FString& InTranslatedText )
			: File( InFile )
			, Namespace( InNamespace )
			, Key ( InKey )
			, SourceText( InSourceText )
			, TranslatedText( InTranslatedText )
		{

		}

		FString File;
		FString Namespace;
		FString Key;
		FString SourceText;
		FString TranslatedText;
	};

	/**
	 * Contains information about a single localization file, any language.
	 */
	struct FLocalizationFile
	{
	private:
		/**
		 * The filename for the FConfigFile this FLocalizationFile represents.
		 */
		FString LocFilename;

		/** sections that do not exist in the counterpart file. */
		TArray<FString> UnmatchedSections;

		/** properties that are missing from the corresponding section in the other file */
		TArray<FString> UnmatchedProperties;

		/** properties that have identical values in the other file */
		TArray<FLocalizationFileEntry> IdenticalProperties;

		/** Info about translated properties that will be added to the Archive */
		TArray<FLocalizationFileEntry> TranslatedProperties;

		/** the FConfigFile which contains the data for this loc file */
		FConfigFile* LocFile;

	public:

		/**
		 * Standard constructor
		 */
		FLocalizationFile( const FString& InPath, FConfigFile* const InConfigFile );

		/** Copy ctor */
		FLocalizationFile( const TSharedRef<FLocalizationFile> Other );

		/** Dtor */
		~FLocalizationFile();

		/**
		 * Compares the data in this loc file against the data in the specified counterpart file, placing the results in the various tracking arrays.
		 */
		void CompareToCounterpart( TSharedPtr<FLocalizationFile> Other );

		/** Accessors */
		const FString GetFullName()			const	{ return LocFilename; }
		const FString GetDirectoryName()	const	{ return FPaths::GetPath(LocFilename); }
		const FString GetFilename()			const	{ return FPaths::GetBaseFilename(LocFilename); }
		const FString GetExtension()		const	{ return FPaths::GetExtension(LocFilename); }
		class FConfigFile* GetFile()		const	{ return LocFile; }

		/**
		 * Appends all the entries that have matching primary language and translated text to the passed in array.
		 */
		void GetIdenticalProperties( TArray<FLocalizationFileEntry>& OutProperties ) const;

		/**
		 * Appends all the entries that have non-empty translated text to the passed in array.  Note this will not get entries where the primary language text is the same as the translated text, use GetIdenticalProperties() instead.
		 */
		void GetTranslatedProperties( TArray<FLocalizationFileEntry>& OutProperties ) const;
	};

	/**
	 * Contains information about a localization file and its native counterpart.
	 */
	struct FLocalizationFilePair
	{
		TSharedPtr<FLocalizationFile> NativeFile;
		TSharedPtr<FLocalizationFile> ForeignFile;
		
		/** Default ctor */
		FLocalizationFilePair() : NativeFile(NULL), ForeignFile(NULL) {}
		~FLocalizationFilePair();

		/**
		 * Compares the two loc files against each other.
		 */
		void CompareFiles();

		/**
		 * Adds any identical entries to an array.
		 */
		void GetIdenticalProperties( TArray<FLocalizationFileEntry>& Properties );

		/**
		 * Adds entries that differ and are non-empty to an array.
		 */
		void GetTranslatedProperties( TArray<FLocalizationFileEntry>& Properties );

		/**
		 * Assigns the native version of the loc file pair.
		 */
		bool SetNativeFile( const FString& NativeFilename, FConfigFile* const NativeConfigFile );

		/**
		 * Assigns the foreign version of this loc file pair.
		 */
		bool SetForeignFile( const FString& ForeignFilename, FConfigFile* const ForeignConfigFile );

		/** returns the filename (without path or extension info) for this file pair */
		const FString GetFilename();

		bool HasNativeFile();

		bool HasForeignFile();

		bool HasNativeFile( const FString& Filename );

		bool HasForeignFile( const FString& Filename );
	};


	TArray<FLocalizationFilePair> LocPairs;

	/**
	 * Returns the index of the loc file pair that contains the native version of the specified filename, or INDEX_NONE if it isn't found
	 */
	int32 FindNativeIndex( const FString& Filename );

	/**
	 * Returns the index of the loc file pair that contains the native version of the specified filename, or INDEX_NONE if it isn't found
	 */
	int32 FindForeignIndex( const FString& Filename );

	/**
	 * Adds the specified file as the native version for a loc file pair
	 */
	void AddNativeFile( const FString& Filename );

	/**
	 * Adds the specified file as the foreign version for a loc file pair
	 */
	void AddForeignFile( const FString& Filename );

	/**
	 * Initializes the LocPairs arrays using the list of filenames provided.
	 */
	void ReadLocFiles( const TArray<FString>& NativeFilenames, const TArray<FString>& ForeignFilenames );

	bool ProcessManifest( const FString& PrimaryLangExt, const FString& SourcePath, const FString& DestinationPath );

	bool ProcessArchive( const FString& PrimaryLangExt, const FString& SourcePath, const FString& DestinationPath );

	void LoadLegacyLocalizationFiles( const FString& SourcePath, const FString& NativeLanguage, const TArray<FString>& LanguagesToProcess );

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

private: 
	FString GatherTextConfigPath;
	FString SectionName;
	FConfigCacheIni LegacyLocalizationCacheIni;

};
