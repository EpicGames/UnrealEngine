// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/GatherTextCommandletBase.h"
#include "InternationalizationExportCommandlet.generated.h"

/**
 *	UInternationalizationExportCommandlet: Commandlet used to export internationalization data to various standard formats. 
 */
UCLASS()
class UInternationalizationExportCommandlet : public UGatherTextCommandletBase
{
    GENERATED_BODY()

public:
	UInternationalizationExportCommandlet(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{}

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	virtual bool ConfigurePhase(const FString& Params) override;
	virtual EGatherTextCommandletPhase GetPhase() const override;
	virtual bool CanParallelize() const { return true; }

private:
	bool bDoImport = false;
	bool bDoExport = false;
};
