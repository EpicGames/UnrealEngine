// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "IoStoreCommandlet.generated.h"

/*
 * Commandlet for creating container file(s) used by the IoDispatcher.
 */
UCLASS()
class UIoStoreCommandlet  
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);
};
