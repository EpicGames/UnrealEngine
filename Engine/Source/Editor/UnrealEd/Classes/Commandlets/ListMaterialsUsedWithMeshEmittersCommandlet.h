// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ListMaterialsUsedWithMeshEmittersCommandlet.generated.h"

class UParticleSystem;

UCLASS()
class UListMaterialsUsedWithMeshEmittersCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	
	/** Process the given particle system. For mesh emitters, output material paths that don't have bUsedWithParticleSprites flagged.*/
	void ProcessParticleSystem(UParticleSystem* ParticleSystem , TArray<FString> &OutMaterials);

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
