// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "LoadPackageCommandlet.generated.h"

UCLASS()
class ULoadPackageCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	/**
	 *	Parse the given load list file, placing the entries in the given Tokens array.
	 *
	 *	@param	LoadListFilename	The name of the load list file
	 *	@param	Tokens				The array to place the entries into.
	 *	
	 *	@return	bool				true if successful and non-empty, false otherwise
	 */
	bool ParseLoadListFile(FString& LoadListFilename, TArray<FString>& Tokens);

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};


