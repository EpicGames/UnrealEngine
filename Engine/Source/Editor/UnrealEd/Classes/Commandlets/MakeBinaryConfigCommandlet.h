// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "MakeBinaryConfigCommandlet.generated.h"

/*
 * Commandlet for creating a binary GConfig to read in quickly
 */
UCLASS()
class UMakeBinaryConfigCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	virtual int32 Main(const FString& Params) override;
};
