// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"
#include "MaterialLayerUsageCommandlet.generated.h"

UCLASS(config=Editor)
class UMaterialLayerUsageCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
