// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ParticleSystemAuditCommandlet.generated.h"

UCLASS(config=Editor)
class UParticleSystemAuditCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	/** All particle systems w/ a NO LOD levels */
	TSet<FString> ParticleSystemsWithNoLODs;
	/** All particle systems w/ a single LOD level */
	TSet<FString> ParticleSystemsWithSingleLOD;
	/** All particle systems w/out fixed bounds set */
	TSet<FString> ParticleSystemsWithoutFixedBounds;
	/** All particle systems w/ LOD Method of Automatic & a check time of 0.0 */
	TSet<FString> ParticleSystemsWithBadLODCheckTimes;
	/** All particle systems w/ bOrientZAxisTowardCamera enabled */
	TSet<FString> ParticleSystemsWithOrientZAxisTowardCamera;
	/** All particle systems w/ missing materials */
	TSet<FString> ParticleSystemsWithMissingMaterials;
	/** All particle systems w/ no emitters */
	TSet<FString> ParticleSystemsWithNoEmitters;
	/** All particle systems w/ a high spawn rate or burst */
	TSet<FString> ParticleSystemsWithHighSpawnRateOrBurst;
	/** All particle systems w/ a far LODDistance */
	TSet<FString> ParticleSystemsWithFarLODDistance;
	/** All particle systems w/ bone location sources that do not match between LODs */
	TSet<FString> ParticleSystemsWithBoneLocationMismatches;
	/** All particle systems with warmup time on them */
	TSet<FString> ParticleSystemsWithWarmupTime;
	/** All particle systems with lights and the details */
	TSet<FString> ParticleSystemsWithLights;
	/** All particle systems with fixed delta time */
	TSet<FString> ParticleSystemsWithFixedDeltaTime;

	/** If a particle system has a spawn rate or burst count greater than this value, it will be reported */
	UPROPERTY(config)
	float HighSpawnRateOrBurstThreshold;

	/** If a particle system has an LODDistance larger than this value, it will be reported */
	UPROPERTY(config)
	float FarLODDistanceTheshold;

	/** The folder in which the commandlet's output files will be stored */
	FString AuditOutputFolder;

	/** Only assets in this collection will be considered. If this is left blank, no assets will be filtered by collection */
	FString FilterCollection;

	/** Package paths to include */
	TArray<FName> PackagePaths;

	/** Entry point */
	int32 Main(const FString& Params) override;

	/** Process all referenced particle systems */
	bool ProcessParticleSystems();

	/** Dump the results of the audit */
	virtual void DumpResults();

	/**
	 *	Dump the give list of particle systems to an audit CSV file...
	 *
	 *	@param	InPSysSet		The particle system set to dump
	 *	@param	InFilename		The name for the output file (short name)
	 *
	 *	@return	bool			true if successful, false if not
	 */
	bool DumpSimplePSysSet(TSet<FString>& InPSysSet, const TCHAR* InShortFilename, const TCHAR* OptionalHeader = nullptr);

	/** Generic function to handle dumping values to a CSV file */
	bool DumpSimpleSet(TSet<FString>& InSet, const TCHAR* InShortFilename, const TCHAR* InObjectClassName, const TCHAR* OptionalHeader = nullptr);

	/** Gets an archive to write to an output file */
	FArchive* GetOutputFile(const TCHAR* InShortFilename);
};
