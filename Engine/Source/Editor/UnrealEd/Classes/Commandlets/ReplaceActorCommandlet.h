// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ReplaceActorCommandlet.generated.h"

/** Commandlet for replacing one kind of actor with another kind of actor, copying changed properties from the most-derived common superclass */
UCLASS()
class UReplaceActorCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};


