// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ReplaceAssetsCommandlet.generated.h"

/** Commandlet for replacing assets with those from another location (intended use is replacing with cooked assets) */
UCLASS()
class UReplaceAssetsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};


