// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"

#include "SavePackageUtilitiesCommandlet.generated.h"

/*
 * Commandlet used to validate the package saving mechanism. 
 * It can currently compare two back to back saves of a package (or folder of packages)
 */
UCLASS()
class
	UE_DEPRECATED(5.6, "SavePackageUtilitiesCommandlet is no longer needed. Contact Epic if you need this functionality.")
	USavePackageUtilitiesCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	virtual int32 Main(const FString& Params) override;
};
