// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "ShaderCodeLibraryToolsCommandlet.generated.h"

UCLASS()
class UShaderCodeLibraryToolsCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static int32 StaticMain(const FString& Params);
};

