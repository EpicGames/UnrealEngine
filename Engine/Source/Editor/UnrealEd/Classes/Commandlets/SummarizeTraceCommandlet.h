// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SummarizeTraceCommandlet.cpp:
	  Commandlet for summarizing utrace cpu events into to csv
=============================================================================*/

#pragma once
#include "Commandlets/Commandlet.h"
#include "Commandlets/SummarizeTraceUtils.h"
#include "TraceServices/Model/Counters.h"
#include "SummarizeTraceCommandlet.generated.h"

UCLASS(config=Editor)
class USummarizeTraceCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;

	//~ End UCommandlet Interface
private:
	TUniquePtr<IFileHandle> OpenCSVFile(const FString& Name);
	bool GenerateScopesCSV(const TArray<FSummarizeScope>& SortedScopes);
	bool GenerateCountersCSV(const class FSummarizeCountersProvider& CountersProvider);
	bool GenerateBookmarksCSV(const class FSummarizeBookmarksProvider& BookmarksProvider);
	bool GenerateTelemetryCSV(const FString& TestName, bool bAllTelemetry, const TArray<FSummarizeScope>& SortedScopes, const
	                          FSummarizeCountersProvider& CountersProvider, const FSummarizeBookmarksProvider& BookmarksProvider, bool SkipBaseline);

	FString TracePath;
	FString TraceFileBasename;
};
