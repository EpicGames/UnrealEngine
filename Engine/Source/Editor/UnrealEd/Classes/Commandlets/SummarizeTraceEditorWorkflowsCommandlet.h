// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Commandlets/Commandlet.h"
#include "Commandlets/SummarizeTraceUtils.h"
#include "TraceServices/Model/Counters.h"
#include "SummarizeTraceEditorWorkflowsCommandlet.generated.h"

UCLASS(config=Editor)
class USummarizeTraceEditorWorkflowsCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;

	//~ End UCommandlet Interface

private:
	TPair<int32, TArray<int32>> ClipScopes(const TArray<const FSummarizeScope*>& Scopes, double BeginTime, double EndTime);
	TUniquePtr<IFileHandle> OpenCSVFile(const FString& Path);
	bool GenerateTimeCSV(const FString& Path, const TArray<FSummarizeScope>& Scopes, double BeginTime, double EndTime);
};
