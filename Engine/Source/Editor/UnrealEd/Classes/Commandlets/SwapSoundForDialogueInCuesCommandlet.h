// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "SwapSoundForDialogueInCuesCommandlet.generated.h"

UCLASS()
class USwapSoundForDialogueInCuesCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:

	virtual int32 Main(const FString& Params);
};
