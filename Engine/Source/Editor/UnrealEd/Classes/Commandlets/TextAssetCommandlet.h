// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextAssetCommandlet.cpp: Commandlet for saving assets in text asset format
=============================================================================*/

#pragma once
#include "Commandlets/Commandlet.h"
#include "TextAssetCommandlet.generated.h"

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogTextAsset, Log, All);

UENUM()
enum class ETextAssetCommandletMode
{
	ResaveText,
	ResaveBinary,
	RoundTrip,
	LoadBinary,
	LoadText,
	FindMismatchedSerializers,
	GenerateSchema
};

UCLASS(config=Editor)
class UTextAssetCommandlet
	: public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:

	struct FProcessingArgs
	{
		ETextAssetCommandletMode ProcessingMode = ETextAssetCommandletMode::ResaveText;
		int32 NumSaveIterations = 1;
		bool bIncludeEngineContent = false;
		bool bFilenameIsFilter = true;
		FString Filename;
		FString CSVFilename;
		FString OutputPath;
		bool bVerifyJson = true;
	};

	//~ Begin UCommandlet Interface

	virtual int32 Main(const FString& CmdLineParams) override;
	
	//~ End UCommandlet Interface
	static UNREALED_API bool DoTextAssetProcessing(const FString& InCommandLine);
	static UNREALED_API bool DoTextAssetProcessing(const FProcessingArgs& InArgs);
};