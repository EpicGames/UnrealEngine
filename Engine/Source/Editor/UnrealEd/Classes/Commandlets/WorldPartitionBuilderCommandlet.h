// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "Templates/SubclassOf.h"
#include "CollectionManagerTypes.h"
#include "PackageSourceControlHelper.h"
#include "WorldPartitionBuilderCommandlet.generated.h"

class UWorldPartitionBuilder;

UCLASS(Config=Engine)
class UWorldPartitionBuilderCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

private:
	TArray<FString> GatherMapsFromCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType) const;
	bool RunBuilder(TSubclassOf<UWorldPartitionBuilder> InBuilderClass, const FString& InWorldPackageName);

	bool OnFilesModified(const TArray<FString>& InModifiedFiles, const FString& InChangeDescription);
	bool AutoSubmitModifiedFiles() const;

private:
	bool bAutoSubmit = false;
	FString AutoSubmitTags;

	TMap<FString, TArray<FString>> AutoSubmitFiles;

	TArray<FString> AdditionalWorldPackagesToProcess;
};