// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectAnnotation.h"
#include "Commandlets/Commandlet.h"
#include "PackageSourceControlHelper.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartitionConvertCommandlet.generated.h"

UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldPartitionConvertCommandlet, Log, All);

class ULevelStreaming;
class UWorldPartition;
class ULevelStreaming;
class UDataLayerFactory;

USTRUCT()
struct FHLODLayerActorMapping
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TSoftClassPtr<AActor> ActorClass;

	UPROPERTY()
	FTopLevelAssetPath HLODLayer;
};

UCLASS(Config = Engine, MinimalAPI)
class UWorldPartitionConvertCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UCommandlet Interface
	UNREALED_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	static UNREALED_API const FString GetConversionSuffix(const bool bInOnlyMergeSubLevels);

private:
	UNREALED_API void GatherAndPrepareSubLevelsToConvert(ULevel* Level, TArray<ULevel*>& SubLevels);

protected:
	UNREALED_API virtual bool GetAdditionalLevelsToConvert(ULevel* Level, TArray<ULevel*>& SubLevels);
	UNREALED_API virtual bool PrepareStreamingLevelForConversion(ULevelStreaming* StreamingLevel);
	UNREALED_API virtual bool ShouldConvertStreamingLevel(ULevelStreaming* StreamingLevel);
	UNREALED_API virtual bool ShouldDeleteActor(AActor* Actor, bool bMainLevel) const;
	UNREALED_API virtual void PerformAdditionalWorldCleanup(UWorld* World) const;
	virtual void PerformAdditionalActorChanges(AActor* Actor) const {}
	UNREALED_API virtual void OutputConversionReport() const;
	UNREALED_API virtual void OnWorldLoaded(UWorld* World);
	virtual void ReadAdditionalTokensAndSwitches(const TArray<FString>& Tokens, const TArray<FString>& Switches) {}

	UNREALED_API UWorldPartition* CreateWorldPartition(class AWorldSettings* MainWorldSettings);
	UNREALED_API UWorld* LoadWorld(const FString& LevelToLoad);
	UNREALED_API ULevel* InitWorld(UWorld* World);

	UNREALED_API void ChangeObjectOuter(UObject* Object, UObject* NewOuter);
	UNREALED_API void FixupSoftObjectPaths(UPackage* OuterPackage);

	UNREALED_API bool DetachDependantLevelPackages(ULevel* Level);
	UNREALED_API void ClearHierarchicalLODSetup(ULevel* Level);
	UNREALED_API bool RenameWorldPackageWithSuffix(UWorld* World);

	UNREALED_API void SetupHLOD();
	UNREALED_API void SetupHLODLayerAssets();

	UNREALED_API void SetActorGuid(AActor* Actor, const FGuid& NewGuid);
	UNREALED_API void CreateWorldMiniMapTexture(UWorld* World);

	// Conversion report
	TSet<FString> MapsWithLevelScriptsBPs;
	TSet<FString> MapsWithMapBuildData;
	TSet<FString> ActorsWithChildActors;
	TSet<FString> GroupActors;
	TSet<FString> ActorsInGroupActors;
	TSet<FString> ActorsReferencesToActors;

	TMap<FString, FString> RemapSoftObjectPaths;

	FString LevelConfigFilename;
	TArray<UPackage*> PackagesToSave;
	TArray<UPackage*> PackagesToDelete;

	bool bOnlyMergeSubLevels;
	bool bDeleteSourceLevels;
	bool bGenerateIni;
	bool bReportOnly;
	bool bVerbose;
	bool bConversionSuffix;
	bool bDisableStreaming;
	FString ConversionSuffix;

	UPROPERTY(Config)
	TSubclassOf<UWorldPartitionEditorHash> EditorHashClass;

	UPROPERTY(Config)
	TSubclassOf<UWorldPartitionRuntimeHash> RuntimeHashClass;

	// Levels excluded from conversion.
	UPROPERTY(Config)
	TArray<FString> ExcludedLevels;

	UPROPERTY(Config)
	bool bConvertActorsNotReferencedByLevelScript;

	UPROPERTY(Config)
	FVector WorldOrigin;
	
	UPROPERTY(Config)
	FVector WorldExtent;

	UPROPERTY(Config)
	FTopLevelAssetPath DefaultHLODLayerAsset;

	UPROPERTY(Config)
	FString FoliageTypePath;

	UPROPERTY(Config)
	TArray<FHLODLayerActorMapping> HLODLayersForActorClasses;

	UPROPERTY(Config)
	uint32 LandscapeGridSize;

	UPROPERTY(Config)
	FString DataLayerAssetFolder;

	UPROPERTY()
	TObjectPtr<UDataLayerFactory> DataLayerFactory;
};
