// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"
#include "Commandlets/Commandlet.h"
#include "Commandlets/WorldPartitionCommandletHelpers.h"
#include "PackageSourceControlHelper.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/Deprecated/DeprecatedDataLayer.h"

#include "WorldPartitionDataLayerToAssetCommandLet.generated.h"

// COMMANDLET TO CONVERT DATALAYERS TO DATA LAYERS ASSET AND DATA LAYERS INSTANCES WITHIN A GIVEN WORLD.
// 
// All data layers will be converted to data layer instances and data layer assets.The commandlet will create the data layer assets at the specified destination folder.
// All actors will now be referencing data layer assets.
// 
// RUNNNING THE COMMANDLET:
// [Project] -run=DataLayerToAssetCommandlet [LevelName] -DestinationFolder=[Folder]
// 
// -DestinationFolder : Specifies the folder where the Data Layer Assets will be created
// 
// Optional Args :
// 
// -NoSave : Will run the commandlet, report information, but will not save.
// -IgnoreActorLoadingErrors:  Will ignore if an actor failed to load.By default the commandlet will abort upon those errors.
// -Verbose:  Adds extra logging information.
// 
// 
// If the Commandlet fails it will report an error which will need to be addressed.
// Once the commandlet finishes successfully, all data layers will be saved and actors will be updated.
//
// PROJECT SPECIFIC CONVERSION:
// Some projects might have added additional references to data layers. Those projects can subclass UDataLayerToAssetCommandlet and override 2 functions 
// virtual bool PerformAdditionalActorConversions(...) : Called on every actor when they are converted. 
// virtual bool PerformProjectSpecificConversions(...) : Called before wrapping up the commandlet.
//
// Both functions receive the CommandletContext in parameter. 
// UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::GetDataLayerConversionInfo(...) can be used to retrieve the conversion information for a specific data layer.
// the UDataLayerConversionInfo reference the UDeprecatedDataLayerInstance to convert, the newly created UDataLayerAsset and the newly created UDataLayerInstanceWithAsset.
//


UNREALED_API DECLARE_LOG_CATEGORY_EXTERN(LogDataLayerToAssetCommandlet, Log, All);

class UDataLayerAsset;
class UDataLayerInstance;
class UDataLayerInstanceWithAsset;
class UDataLayerFactory;
class UDeprecatedDataLayerInstance;
class UWorld;

UCLASS(Transient)
class UDataLayerConversionInfo : public UObject
{
	GENERATED_BODY()

	friend class UDataLayerToAssetCommandletContext;

public:
	bool IsConverting() const { return DataLayerToConvert != nullptr; }
	bool IsAPreviousConversion() const { return CurrentConvertingInfo != nullptr; }
	bool IsConverted() const { return DataLayerInstance != nullptr && PreviousConversionsInfo.IsEmpty(); }

	TArray<TWeakObjectPtr<UDataLayerConversionInfo>> const& GetPreviousConversions() const { return PreviousConversionsInfo; }
	const TWeakObjectPtr<UDataLayerConversionInfo>& GetCurrentConversion() const { return CurrentConvertingInfo; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void SetDataLayerToConvert(const UDeprecatedDataLayerInstance* InDataLayerToConvert);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void SetDataLayerInstance(UDataLayerInstanceWithAsset* InDataLayerInstance);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TObjectPtr<const UDeprecatedDataLayerInstance> DataLayerToConvert;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TObjectPtr<UDataLayerAsset> DataLayerAsset;

	UPROPERTY()
	TObjectPtr<UDataLayerInstanceWithAsset> DataLayerInstance;

private:
	UPROPERTY()
	TArray<TWeakObjectPtr<UDataLayerConversionInfo>> PreviousConversionsInfo;

	UPROPERTY()
	TWeakObjectPtr<UDataLayerConversionInfo> CurrentConvertingInfo;
};

UCLASS(Transient)
class UDataLayerToAssetCommandletContext : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	const TArray<TObjectPtr<UDataLayerConversionInfo>>& GetDataLayerConversionInfos() const { return DataLayerConversionInfo; }
	const TArray<TWeakObjectPtr<UDataLayerConversionInfo>>& GetConvertingDataLayerConversionInfo() const { return ConvertingDataLayerInfo; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDataLayerConversionInfo* GetDataLayerConversionInfo(const UDeprecatedDataLayerInstance* DataLayer) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UDataLayerConversionInfo* GetDataLayerConversionInfo(const UDataLayerAsset* DataLayerAsset) const;
	UDataLayerConversionInfo* GetDataLayerConversionInfo(const UDataLayerInstanceWithAsset* DataLayerInstance) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDataLayerConversionInfo* GetDataLayerConversionInfo(const FActorDataLayer& ActorDataLayer) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UDataLayerConversionInfo* StoreExistingDataLayer(FAssetData& AssetData);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDataLayerConversionInfo* StoreDataLayerAssetConversion(const UDeprecatedDataLayerInstance* DataLayerToConvert, UDataLayerAsset* NewDataLayerAsset);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UDataLayerConversionInfo* StoreDataLayerInstanceConversion(const UDataLayerAsset* DataLayerAsset, UDataLayerInstanceWithAsset* NewDataLayerInstance);
	
	bool SetPreviousConversions(UDataLayerConversionInfo* CurrentConversion, TArray<TWeakObjectPtr<UDataLayerConversionInfo>>&& PreviousConversions);

	bool FindDataLayerConversionInfos(FName DataLayerAssetName, TArray<TWeakObjectPtr<UDataLayerConversionInfo>>& OutConversionInfos) const;

	void LogConversionInfos() const;

private:
	UPROPERTY()
	TArray<TObjectPtr<UDataLayerConversionInfo>> DataLayerConversionInfo;

	UPROPERTY()
	TArray<TWeakObjectPtr<UDataLayerConversionInfo>> ConvertingDataLayerInfo;
};

UCLASS(Config = Engine, MinimalAPI)
class UDataLayerToAssetCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	enum EReturnCode
	{
		Success = 0,
		CommandletInitializationError,
		DataLayerConversionError,
		ActorDataLayerRemappingError,
		ProjectSpecificConversionError,
	};

	//~ Begin UCommandlet Interface
	UNREALED_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

protected:
	virtual bool PerformProjectSpecificConversions(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext) { return true; }
	virtual bool PerformAdditionalActorConversions(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, AActor* Actor) { return true; }

private:
	UNREALED_API bool InitializeFromCommandLine(TArray<FString>& Tokens, TArray<FString> const& Switches, TMap<FString, FString> const& Params);

	UNREALED_API bool BuildConversionInfos(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UNREALED_API bool CreateConversionFromDataLayer(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, const UDeprecatedDataLayerInstance* DataLayer, FPackageSourceControlHelper& PackageHelper);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UNREALED_API TObjectPtr<UDataLayerAsset> GetOrCreateDataLayerAssetForConversion(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FName DataLayerName);

	UNREALED_API bool ResolvePreviousConversionsToCurrent(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext);

	UNREALED_API bool CreateDataLayerInstances(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext);
	UNREALED_API bool RebuildDataLayerHierarchies(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext);

	UNREALED_API bool RemapActorDataLayersToAssets(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper);
	UNREALED_API uint32 RemapActorDataLayers(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, AActor* Actor);
	UNREALED_API uint32 RemapDataLayersAssetsFromPreviousConversions(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, AActor* Actor);
	
	UNREALED_API bool DeletePreviousConversionsData(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper);

	UNREALED_API bool CommitConversion(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper);

	UNREALED_API FString GetConversionFolder() const;
	UNREALED_API bool IsAssetInConversionFolder(const FSoftObjectPath& DataLayerAsset);

	UPROPERTY(Config)
	FString DestinationFolder;

	UPROPERTY(Transient)
	FString ConversionFolder;

	UPROPERTY(Config)
	bool bPerformSavePackages = true;

	UPROPERTY(Config)
	bool bIgnoreActorLoadingErrors = false;

	UPROPERTY()
	TObjectPtr<UDataLayerFactory> DataLayerFactory;

	UPROPERTY(Transient)
	TObjectPtr<UWorld> MainWorld;
};
