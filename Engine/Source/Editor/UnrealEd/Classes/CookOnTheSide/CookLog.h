// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCook, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogCookStats, Log, All);
// LogCookStatus: Logs should be hidden by default but enabled on the build machines so they can be forwarded to stdout
DECLARE_LOG_CATEGORY_EXTERN(LogCookStatus, Warning, All);
extern FName LogCookName;
