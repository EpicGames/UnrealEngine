// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/RingBuffer.h"
#include "Cooker/CookArtifact.h"
#include "CookOnTheSide/CookLog.h"
#include "CookPackageSplitter.h"
#include "HAL/PlatformMemory.h"
#include "INetworkFileSystemModule.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/PackageAccessTracking.h"
#include "Serialization/PackageWriter.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TickableEditorObject.h"
#include "UObject/ICookInfo.h"
#include "UObject/Object.h"
#include "UObject/ObjectHandleTracking.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include "CookOnTheFlyServer.generated.h"

class FAssetRegistryGenerator;
class FAsyncIODelete;
class FDiffModeCookServerUtils;
class FIncrementalValidatePackageWriter;
class FLayeredCookArtifactReader;
class FLooseFilesCookArtifactReader;
class FReferenceCollector;
class FSavePackageContext;
class IAssetRegistry;
class IPlugin;
class ITargetPlatform;
enum class EPackageWriterResult : uint8;
enum class ODSCRecompileCommand;
namespace UE::Cook { class ICookArtifactReader; }
namespace UE::Cook { void CalculateGlobalDependenciesHash(const ITargetPlatform* Platform, const UCookOnTheFlyServer& COTFS); }
namespace UE::LinkerLoad { enum class EImportBehavior : uint8; }
struct FBeginCookContext;
struct FCrashContextExtendedWriter;
struct FGenericMemoryStats;
struct FPropertyChangedEvent;
struct FResourceSizeEx;


enum class ECookInitializationFlags
{
	None =										0x00000000, // No flags
	//unused =									0x00000001, 
	Iterative UE_DEPRECATED(5.6, "Use LegacyIterative instead") = 0x00000002,
	LegacyIterative =							0x00000002, // use legacyiterative cooking (previous cooks will not be cleaned unless detected out of date, experimental)
	SkipEditorContent =							0x00000004, // do not cook any content in the content\editor directory
	Unversioned =								0x00000008, // save the cooked packages without a version number
	AutoTick =									0x00000010, // enable ticking (only works in the editor)
	AsyncSave =									0x00000020, // save packages async
	// unused =									0x00000040,
	IncludeServerMaps =							0x00000080, // should we include the server maps when cooking
	// unused = 								0x00000100,
	BuildDDCInBackground =						0x00000200, // build ddc content in background while the editor is running (only valid for modes which are in editor IsCookingInEditor())
	// unused =									0x00000400,
	OutputVerboseCookerWarnings =				0x00000800, // output additional cooker warnings about content issues
	EnablePartialGC =							0x00001000, // mark up with an object flag objects which are in packages which we are about to use or in the middle of using, this means we can gc more often but only gc stuff which we have finished with
	TestCook =									0x00002000, // test the cooker garbage collection process and cooking (cooker will never end just keep testing).
	//unused =									0x00004000,
	LogDebugInfo =								0x00008000, // enables additional debug log information
	IterateSharedBuild UE_DEPRECATED(5.6, "Use LegacyIterativeSharedBuild instead") = 0x00010000,
	LegacyIterativeSharedBuild =				0x00010000, // cook legacyiteratively from a build in the SharedIterativeBuild directory 
	IgnoreIniSettingsOutOfDate =				0x00020000, // when using legacyiterative, if the inisettings say the cook is out of date keep using the previously cooked build. Not used in incremental cooks.
	IgnoreScriptPackagesOutOfDate =				0x00040000, // for incremental cooking, ignore script package changes
	//unused =									0x00080000,
	CookEditorOptional =						0x00100000,	// enable producing the cooked output of optional editor packages that can be use in the editor when loading cooked data
};
ENUM_CLASS_FLAGS(ECookInitializationFlags);

enum class ECookByTheBookOptions
{
	None =								0x00000000, // no flags
	CookAll	=							0x00000001, // cook all maps and content in the content directory
	MapsOnly =							0x00000002, // cook only maps
	NoDevContent =						0x00000004, // don't include dev content
	ForceDisableCompressed =			0x00000010, // force compression to be disabled even if the cooker was initialized with it enabled
	ForceEnableCompressed =				0x00000020, // force compression to be on even if the cooker was initialized with it disabled
	ForceDisableSaveGlobalShaders =		0x00000040, // force global shaders to not be saved (used if cooking multiple times for the same platform and we know we are up todate)
	NoGameAlwaysCookPackages =			0x00000080, // don't include the packages specified by the game in the cook (this cook will probably be missing content unless you know what you are doing)
	NoAlwaysCookMaps =					0x00000100, // don't include always cook maps (this cook will probably be missing content unless you know what you are doing)
	NoDefaultMaps =						0x00000200, // don't include default cook maps (this cook will probably be missing content unless you know what you are doing)
	NoStartupPackages =					0x00000400, // Don't include packages that are loaded by engine startup (this cook will probably be missing content unless you know what you are doing)
	NoInputPackages =					0x00000800, // don't include slate content (this cook will probably be missing content unless you know what you are doing)
	SkipSoftReferences =				0x00001000, // Don't follow soft references when cooking. Usually not viable for a real cook and the results probably wont load properly, but can be useful for debugging
	SkipHardReferences =				0x00002000, // Don't follow hard references when cooking. Not viable for a real cook, only useful for debugging
	// Unused=							0x00004000,
	CookAgainstFixedBase =				0x00010000, // If cooking DLC, assume that the base content can not be modified. 
	DlcLoadMainAssetRegistry =			0x00020000, // If cooking DLC, populate the main game asset registry
	ZenStore =							0x00040000, // Store cooked data in Zen Store
	DlcReevaluateUncookedAssets =		0x00080000, // If cooking DLC, ignore assets in the base asset registry that were not cooked, so that this cook has an opportunity to cook the assets
	RunAssetValidation =				0x00100000, // Run asset validation (EditorValidatorSubsystem) on assets loaded during cook
	RunMapValidation =					0x00200000, // Run map validation (MapCheck) on maps loaded during cook
	ValidationErrorsAreFatal =			0x00400000, // Consider validation errors (from RunAssetValidation or RunMapValidation) as fatal (preventing the package from being cooked)
};
ENUM_CLASS_FLAGS(ECookByTheBookOptions);

enum class ECookListOptions
{
	None =								0x00000000,
	ShowRejected =						0x00000001,
};
ENUM_CLASS_FLAGS(ECookListOptions);

UENUM()
namespace ECookMode
{
	enum Type : int
	{
		/** Default mode, handles requests from network. */
		CookOnTheFly,
		/** Cook on the side. */
		CookOnTheFlyFromTheEditor,
		/** Precook all resources while in the editor. */
		CookByTheBookFromTheEditor,
		/** Cooking by the book (not in the editor). */
		CookByTheBook,
		/** Commandlet helper for a separate director process. Director might be in any of the other modes. */
		CookWorker,
	};
}
inline bool IsCookByTheBookMode(ECookMode::Type CookMode)
{
	return CookMode == ECookMode::CookByTheBookFromTheEditor || CookMode == ECookMode::CookByTheBook;
}
inline bool IsRealtimeMode(ECookMode::Type CookMode)
{
	return CookMode == ECookMode::CookByTheBookFromTheEditor || CookMode == ECookMode::CookOnTheFlyFromTheEditor;
}
inline bool IsCookingInEditor(ECookMode::Type CookMode)
{
	return CookMode == ECookMode::CookByTheBookFromTheEditor || CookMode == ECookMode::CookOnTheFlyFromTheEditor;
}
inline bool IsCookOnTheFlyMode(ECookMode::Type CookMode)
{
	return CookMode == ECookMode::CookOnTheFly || CookMode == ECookMode::CookOnTheFlyFromTheEditor;
}
inline bool IsCookWorkerMode(ECookMode::Type CookMode)
{
	return CookMode == ECookMode::CookWorker;
}

UENUM()
enum class ECookTickFlags : uint8
{
	None =									0x00000000, /* no flags */
	MarkupInUsePackages =					0x00000001, /** Markup packages for partial gc */
	HideProgressDisplay =					0x00000002, /** Hides the progress report */
};
ENUM_CLASS_FLAGS(ECookTickFlags);

namespace UE::Cook
{
/** MPCook Behavior set from config/commandline that decides where generatedpackages should be assigned. */
enum class EMPCookGeneratorSplit : uint8
{
	AnyWorker,
	AllOnSameWorker,
	SomeOnSameWorker,
	NoneOnSameWorker,
};
}

namespace UE::Cook
{
	class FAssetRegistryMPCollector;
	class FBuildDefinitions;
	class FCookDirector;
	class FCookGCDiagnosticContext;
	class FCookSandbox;
	class FCookWorkerClient;
	class FCookWorkerServer;
	class FDiagnostics;
	class FIncrementallyModifiedDiagnostics;
	class FGlobalCookArtifact;
	class FLogHandler;
	class FODSCClientData;
	class FPackagePreloader;
	class FPackageWriterMPCollector;
	class FRequestCluster;
	class FRequestQueue;
	class FSaveCookedPackageContext;
	class FShaderLibraryCookArtifact;
	class FSoftGCHistory;
	class FStallDetector;
	class FValidationCookArtifact;
	class FWorkerRequestsLocal;
	class FWorkerRequestsRemote;
	class ICookOnTheFlyRequestManager;
	class ICookOnTheFlyNetworkServer;
	class ILogHandler;
	class IWorkerRequests;
	enum class ECachedCookedPlatformDataEvent : uint8;
	enum class ECookPhase : uint8;
	enum class EPackageState : uint8;
	enum class EPollStatus : uint8;
	enum class EReachability : uint8;
	enum class EStateChangeReason : uint8;
	enum class ESuppressCookReason : uint8;
	enum class ESendFlags : uint8;
	enum class EUrgency : uint8;
	struct FBeginCookConfigSettings;
	struct FCachedObjectInOuter;
	struct FConstructPackageData;
	struct FCookByTheBookOptions;
	struct FCookOnTheFlyOptions;
	struct FCookerTimer;
	struct FCookGenerationInfo;
	struct FCookSavePackageContext;
	struct FDiscoveredPlatformSet;
	struct FGenerationHelper;
	struct FInitializeConfigSettings;
	struct FPackageData;
	struct FPackageDatas;
	struct FPackageTracker;
	struct FPendingCookedPlatformData;
	struct FPlatformManager;
	struct FScopeFindCookReferences;
	struct FTickStackData;
}
namespace UE::Cook::Private
{
	class FRegisteredCookPackageSplitter;
}

namespace UE::Cook
{

struct FStatHistoryInt
{
public:
	void Initialize(int64 InitialValue);
	void AddInstance(int64 CurrentValue);
	int64 GetMinimum() const { return Minimum; }
	int64 GetMaximum() const { return Maximum; }

private:
	int64 Minimum = 0;
	int64 Maximum = 0;
};

// tmap of the Config name, Section name, Key name, to the value
typedef TMap<FName, TMap<FName, TMap<FName, TArray<FString>>>> FIniSettingContainer;

}
UCLASS(MinimalAPI)
class UCookOnTheFlyServer : public UObject,
	public FTickableEditorObject,
	public FExec,
	public UE::Cook::ICookInfo,
	public UE::PackageWriter::Private::ICookerInterface
{
	GENERATED_BODY()

	UNREALED_API UCookOnTheFlyServer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	UNREALED_API UCookOnTheFlyServer(FVTableHelper& Helper); // Declare the FVTableHelper constructor manually so that we can forward-declare-only TUniquePtrs in the header without getting compile error in generated cpp

private:

	using FPollFunction = TUniqueFunction<void(UE::Cook::FTickStackData&)>;
	/**
	 * Wrapper around a function for cooker tasks that need to be ticked on a schedule.
	 * Includes data to support calling it on a schedule or triggering it manually.
	 */
	struct FPollable : public FRefCountedObject
	{
	public:
		enum EManualTrigger {};
		FPollable(const TCHAR* InDebugName, float InPeriodSeconds, float InPeriodIdleSeconds, FPollFunction&& InFunction);
		FPollable(const TCHAR* InDebugName, EManualTrigger, FPollFunction&& InFunction);

	public:
		/** Trigger the pollable to execute on the next PumpPollables */
		void Trigger(UCookOnTheFlyServer& COTFS);
		/**
		 * Run the pollable's callback function now. Note this means running it outside of its normal location which is not valid for all pollables.
		 * PumpPollables is not thread safe, so this must be called on the scheduler thread.
		 */
		void RunNow(UCookOnTheFlyServer& COTFS);

	public:
		const TCHAR* DebugName = TEXT("");
		FPollFunction PollFunction;
		/** Time when this should be next called, if the cooker is idle. (See also NextTimeSeconds). */
		double NextTimeIdleSeconds = 0.;
		float PeriodSeconds = 60.f;
		float PeriodIdleSeconds = 5.f;

	public: // To be called only by PumpPollables
		/** Run the pollable's callback function, from PumpPollables. Calculate the new time and store it in the output. */
		void RunDuringPump(UE::Cook::FTickStackData& StackData, double& OutNewCurrentTime, double& OutNextTimeSeconds);
		/** Update the position of this in the Pollables queue, called from Trigger or for a deferred Trigger. */
		void TriggerInternal(UCookOnTheFlyServer& COTFS);
		/** Update the position of this in the Pollables queue, called from RunNow or for a deferred RunNow. */
		void RunNowInternal(UCookOnTheFlyServer& COTFS, double LastTimeRun);
	};
	/** A key/value pair for storing Pollables in a priority queue, keyed by next call time. */
	struct FPollableQueueKey
	{
	public:
		FPollableQueueKey() = default;
		explicit FPollableQueueKey(FPollable* InPollable);
		explicit FPollableQueueKey(const TRefCountPtr<FPollable>& InPollable);
		explicit FPollableQueueKey(TRefCountPtr<FPollable>&& InPollable);
		bool operator<(const FPollableQueueKey& Other) const
		{
			return NextTimeSeconds < Other.NextTimeSeconds;
		}

	public:
		TRefCountPtr<FPollable> Pollable;
		/**
		 * Time when the pollable be next called, if the cooker is not idle. Stored here rather than
		 * on the FPollable to support fast access in the queue.
		 */
		double NextTimeSeconds = 0.;
	};

	//////////////////////////////////////////////////////////////////////////
	// Cook on the fly server interface adapter
	class FCookOnTheFlyServerInterface;
	TUniquePtr<FCookOnTheFlyServerInterface> CookOnTheFlyServerInterface;

	/** Current cook mode the cook on the fly server is running in */
	ECookMode::Type CurrentCookMode = ECookMode::CookOnTheFly;
	/** CookMode of the Cook Director, equal to CurrentCookMode if not a CookWorker, otherwise equal to Director's CurrentCookMode */
	ECookMode::Type DirectorCookMode = ECookMode::CookOnTheFly;
	/** Directory to output to instead of the default should be empty in the case of DLC cooking */
	FString OutputDirectoryOverride;

	TUniquePtr<UE::Cook::FCookByTheBookOptions> CookByTheBookOptions;
	TUniquePtr<UE::Cook::FPlatformManager> PlatformManager;

	//////////////////////////////////////////////////////////////////////////
	// Cook on the fly options

	TUniquePtr<UE::Cook::FCookOnTheFlyOptions> CookOnTheFlyOptions;
	/** Cook on the fly server uses the NetworkFileServer */
	TArray<class INetworkFileServer*> NetworkFileServers;
	FOnFileModifiedDelegate FileModifiedDelegate;
	TUniquePtr<UE::Cook::ICookOnTheFlyRequestManager> CookOnTheFlyRequestManager;
	TSharedPtr<UE::Cook::ICookOnTheFlyNetworkServer> CookOnTheFlyNetworkServer;

	//////////////////////////////////////////////////////////////////////////
	// General cook options

	/** Number of packages to load before performing a garbage collect. Set to 0 to never GC based on number of loaded packages */
	uint32 PackagesPerGC;
	/** Amount of time that is allowed to be idle before forcing a garbage collect. Set to 0 to never force GC due to idle time */
	double IdleTimeToGC;
	/** Amount of time to wait when save and load are busy waiting on async operations before trying them again. */
	float CookProgressRetryBusyPeriodSeconds = 0.f;

	// Memory Limits for when to Collect Garbage
	UE_DEPRECATED(5.6, "MemoryMaxUsedVirtual is deprecated and will be removed in future version.")
	uint64 MemoryMaxUsedVirtual;

	UE_DEPRECATED(5.6, "MemoryMaxUsedPhysical is deprecated and will be removed in future version.")
	uint64 MemoryMaxUsedPhysical;

	uint64 MemoryMinFreeVirtual;
	uint64 MemoryMinFreePhysical;
	FGenericPlatformMemoryStats::EMemoryPressureStatus MemoryTriggerGCAtPressureLevel;
	float MemoryExpectedFreedToSpreadRatio;
	/** Max number of packages to save before we partial gc */
	int32 MaxNumPackagesBeforePartialGC;
	/** Max number of concurrent shader jobs reducing this too low will increase cook time */
	int32 MaxConcurrentShaderJobs;
	/** Min number of free UObject indices before the cooker should partial gc */
	int32 MinFreeUObjectIndicesBeforeGC;
	double LastGCTime = 0.;
	double LastFullGCTime = 0.;
	double LastSoftGCTime = 0.;
	int64 SoftGCNextAvailablePhysicalTarget = -1;
	int32 SoftGCStartNumerator = 5;
	int32 SoftGCDenominator = 10;
	float SoftGCTimeFractionBudget = 0.05f;
	float SoftGCMinimumPeriodSeconds = 30.f;
	TUniquePtr<UE::Cook::FSoftGCHistory> SoftGCHistory;
	bool bUseSoftGC = false;
	bool bWarnedExceededMaxMemoryWithinGCCooldown = false;
	bool bGarbageCollectTypeSoft = false;
	bool bSaveGCDataForEvaluation = false;

	/**
	 * The maximum number of packages that should be preloaded at once. Once this is full, packages in LoadPrepare will
	 * remain unpreloaded in LoadPrepare until the existing preloaded packages exit {LoadPrepare,LoadReady} state.
	 */
	uint32 MaxPreloadAllocated;
	/**
	 * A knob to tune performance - How many packages should be present in the SaveQueue before we start processing the
	 * SaveQueue. If number is less, we will find other work to do and save packages only if all other work is done.
	 * This allows us to have enough population in the SaveQueue to get benefit from the asynchronous work done on
	 * packages in the SaveQueue.
	 */
	uint32 DesiredSaveQueueLength;
	/**
	 * A knob to tune performance - How many packages should be present in the LoadPrepare+LoadReady queues before we
	 * start processing the LoadQueue. If number is less, we will find other work to do, and load packages only if all
	 * other work is done.
	 * This allows us to have enough population in the LoadQueue to get benefit from the asynchronous work done
	 * on preloading packages.
	 */
	uint32 DesiredLoadQueueLength;
	/** A knob to tune performance - how many packages to pull off in each call to PumpRequests. */
	uint32 RequestBatchSize;
	/** A knob to tune performance - how many packages to load in each call to PumpLoads. */
	int32 LoadBatchSize;

	ECookInitializationFlags CookFlags = ECookInitializationFlags::None;
	TUniquePtr<UE::Cook::FCookSandbox> SandboxFile;
	FString SandboxFileOutputDirectory;
	TUniquePtr<FAsyncIODelete> AsyncIODelete; // Helper for deleting the old cook directory asynchronously
	bool bIsSavingPackage = false; // used to stop recursive mark package dirty functions
	/**
	 * Set to true during CookOnTheFly if a plugin is calling RequestPackage and we should therefore not make
	 * assumptions about when platforms are done cooking
	 */
	bool bCookOnTheFlyExternalRequests = false;

	/** max number of objects of a specific type which are allowed to async cache at once */
	TMap<FName, int32> MaxAsyncCacheForType;
	/** max number of objects of a specific type which are allowed to async cache at once */
	mutable TMap<FName, int32> CurrentAsyncCacheForType;

	/** List of additional plugin directories to remap into the sandbox as needed */
	TArray<TSharedRef<IPlugin> > PluginsToRemap;

	TMultiMap<UClass*, UE::Cook::Private::FRegisteredCookPackageSplitter*> RegisteredSplitDataClasses;

	//////////////////////////////////////////////////////////////////////////
	// precaching system
	//
	// this system precaches materials and textures before we have considered the object 
	// as requiring save so as to utilize the system when it's idle
	
	TArray<FWeakObjectPtr> CachedMaterialsToCacheArray;
	TArray<FWeakObjectPtr> CachedTexturesToCacheArray;
	int32 LastUpdateTick = 0;
	int32 MaxPrecacheShaderJobs = 0;
	void TickPrecacheObjectsForPlatforms(const float TimeSlice, const TArray<const ITargetPlatform*>& TargetPlatform);

	//////////////////////////////////////////////////////////////////////////

	int32 LastCookPendingCount = 0;
	int32 LastCookedPackagesCount = 0;
	double LastProgressDisplayTime = 0;
	double LastDiagnosticsDisplayTime = 0;

	/** Cached copy of asset registry */
	IAssetRegistry* AssetRegistry = nullptr;

	/** Map of platform name to scl.csv files we saved out. */
	TMap<FName, TArray<FString>> OutSCLCSVPaths;

	/** List of filenames that may be out of date in the asset registry */
	TSet<FName> ModifiedAssetFilenames;

	//////////////////////////////////////////////////////////////////////////
	// legacyiterative ini settings checking

	TArray<FString> ConfigSettingDenyList;

	void OnRequestClusterCompleted(const UE::Cook::FRequestCluster& RequestCluster);

	/**
	* OnTargetPlatformChangedSupportedFormats
	* called when target platform changes the return value of supports shader formats 
	* used to reset the cached cooked shaders
	*/
	void OnTargetPlatformChangedSupportedFormats(const ITargetPlatform* TargetPlatform);

	/* Initializing Platforms must be done on the tickloop thread; Platform data is read only on other threads */
	void AddCookOnTheFlyPlatformFromGameThread(ITargetPlatform* TargetPlatform);
	/* Start the session for the given platform in response to the first client connection. */
	void StartCookOnTheFlySessionFromGameThread(ITargetPlatform* TargetPlatform);

	/* Callback to recalculate all ITargetPlatform* pointers when they change due to modules reloading */
	void OnTargetPlatformsInvalidated();

	/* Update polled fields used by CookOnTheFly's network request handlers */
	void TickNetwork();

	/** Tick the cook for the current cook mode: request, load, save, finish. */
	void TickMainCookLoop(UE::Cook::FTickStackData& StackData);

	/** Execute operations that need to be done after each Scheduler task, such as checking for new external requests. */
	void TickCookStatus(UE::Cook::FTickStackData& StackData);
	void SetSaveBusy(bool bInSaveBusy, int32 NumAsyncWorkRetired);
	void SetLoadBusy(bool bInLoadBusy);
	enum class EIdleStatus
	{
		Active,
		Idle,
		Done
	};
	void SetIdleStatus(UE::Cook::FTickStackData& StackData, EIdleStatus InStatus);
	void UpdateDisplay(UE::Cook::FTickStackData& StackData, bool bForceDisplay);
	FString GetCookSettingsForMemoryLogText() const;
	enum class ECookAction
	{
		Done,			// The cook is complete; no requests remain in any non-idle state
		Request,		// Process the RequestQueue
		Load,			// Process the LoadQueue
		LoadLimited,	// Process the LoadQueue, stopping when loadqueuelength reaches the desired population level
		Save,			// Process the SaveQueue
		SaveLimited,	// Process the SaveQueue, stopping when savequeuelength reaches the desired population level
		Poll,			// Execute pollables which have exceeded their period
		PollIdle,		// Execute pollables which have exceeded their idle period
		KickBuildDependencies, // Find all packages that are build dependencies and request them for cook
		WaitForAsync,	// Sleep for a time slice while we wait for async tasks to complete
		YieldTick,		// Progress is blocked by an async result. Temporarily exit TickMainCookLoop.
	};
	/** Inspect all tasks the scheduler could do and return which one it should do. */
	ECookAction DecideNextCookAction(UE::Cook::FTickStackData& StackData);
	/** How many packages are assigned to the CookOnTheFlyServer for saving/loading? Returns 0 if the current process is not a CookDirector. */
	int32 NumMultiprocessLocalWorkerAssignments() const;
	/** Execute any existing external callbacks and push any existing external cook requests into new RequestClusters. */
	void PumpExternalRequests(const UE::Cook::FCookerTimer& CookerTimer);
	/** Send the PackageData back to request state to create a request cluster, if it has not yet been explored. */
	bool TryCreateRequestCluster(UE::Cook::FPackageData& PackageData);
	/** Inspect the next package in the RequestQueue and push it on to its next state. Report the number of PackageDatas that were pushed to load. */
	void PumpRequests(UE::Cook::FTickStackData& StackData, int32& OutNumPushed);
	/**
	 * Assign the requests found in PumpRequests; either pushing them to ReadyRequests if SingleProcess Cook or
	 * to CookWorkers if MultiProcess. Input Requests have been sorted by leaf to root load order.
	 */
	void AssignRequests(TArrayView<UE::Cook::FPackageData*> Requests, UE::Cook::FRequestQueue& RequestQueue,
		TMap<UE::Cook::FPackageData*, TArray<UE::Cook::FPackageData*>>&& RequestGraph);
	/**
	 * Multiprocess cook: Notify the CookDirector that a package it assigned to a worker was demoted
	 * due to e.g. cancelled cook and should be removed from the CookWorker.
	 */
	void NotifyRemovedFromWorker(UE::Cook::FPackageData& PackageData);

	/** Load packages in the LoadQueue until it's time to break. Report the number of loads that were pushed to save. */
	void PumpLoads(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength, int32& OutNumPushed, bool& bOutBusy);

	/**
	 * Load the given PackageData that was in the load queue and send it on to its next state.
	 * Report the number of PackageDatas that were pushed to save (0 or 1)
	 */
	void LoadPackageInQueue(UE::Cook::FPackageData& PackageData, uint32& ResultFlags, int32& OutNumPushed);
	/** Mark that the given PackageData failed to load and return it to idle. */
	void RejectPackageToLoad(UE::Cook::FPackageData& PackageData, const TCHAR* ReasonText,
		UE::Cook::ESuppressCookReason Reason);

	/**
	 * Try to save all packages in the SaveQueue until it's time to break.
	 * Report the number of requests that were completed (either skipped or successfully saved or failed to save)
	 */
	void PumpSaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength, int32& OutNumPushed,
		bool& bOutBusy, int32& OutNumAsyncWorkRetired);
	void PumpRuntimeSaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength, int32& OutNumPushed,
		bool& bOutBusy, int32& OutNumAsyncWorkRetired);
	void PumpBuildDependencySaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength, int32& OutNumPushed,
		bool& bOutBusy, int32& OutNumAsyncWorkRetired);

	/** Find all packages that are build dependencies and request them for cook. */
	void KickBuildDependencies(UE::Cook::FTickStackData& StackData);

	/**
	 * Send a fence to CookWorkers, and record its number for use by PumpPhaseTransitionFence. Noop if not
	 * in MPCook or not on Director.
	 */
	void InsertPhaseTransitionFence();
	/**
	 * Check whether the fence inserted by the last InsertPhaseTransitionFence has been passed if it exists.
	 * Return bOutCompleted=true only when the fence does not exist or has been passed.
	 * Noop if not MPCook or not on director; no fence is used and the function sets bOutComplete=true whenever called.
	 */
	void PumpPhaseTransitionFence(bool& bOutComplete);

	/**
	 * Inspect the given package and queue it for saving if necessary.
	 *
	 * @param PackageData			The PackageData to be considered for saving.
	 */
	void QueueDiscoveredPackage(UE::Cook::FPackageData& PackageData, UE::Cook::FInstigator&& Instigator, 
		UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms, UE::Cook::EUrgency Urgency,
		UE::Cook::FGenerationHelper* ParentGenerationHelper = nullptr);
	void QueueDiscoveredPackage(UE::Cook::FPackageData& PackageData, UE::Cook::FInstigator&& Instigator,
		UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms);
	void QueueDiscoveredPackageOnDirector(UE::Cook::FPackageData& PackageData, UE::Cook::FInstigator&& Instigator,
		UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms, UE::Cook::EUrgency Urgency);

	/** Called when a package is cancelled and returned to idle. Notifies CookDirector when on a CookWorker. */
	void DemoteToIdle(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags, UE::Cook::ESuppressCookReason Reason);
	/**
	 * Called when a package is sent back to requestcluster for a change in request data. Notifies CookDirector when
	 * on a CookWorker.
	 */
	void DemoteToRequest(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags, UE::Cook::ESuppressCookReason Reason);
	/** Called when a package completes its save and returns to idle. Notifies the CookDirector when on a CookWorker. */
	void PromoteToSaveComplete(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags);

	/**
	 * Remove all request data about the given platform from any data in the CookOnTheFlyServer.
	 * Called when a platform is removed from the list of session platforms e.g. because it has not been recently
	 * used by CookOnTheFly. Does not modify Cooked platforms.
	 */
	void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform, int32 RemovedIndex);
	/** Update structures that have data per platfrom when COTF adds a new SessionPlatform. */
	void OnBeforePlatformAddedToSession(const ITargetPlatform* TargetPlatform);
	void OnPlatformAddedToSession(const ITargetPlatform* TargetPlatform);

	void InitializePollables();
	void PumpPollables(UE::Cook::FTickStackData& StackData, bool bIsIdle);
	void PollFlushRenderingCommands();
	TRefCountPtr<FPollable> CreatePollableLLM();
	TRefCountPtr<FPollable> CreatePollableTriggerGC();
	void PollGarbageCollection(UE::Cook::FTickStackData& StackData);
	void PollQueuedCancel(UE::Cook::FTickStackData& StackData);
	void WaitForAsync(UE::Cook::FTickStackData& StackData);
	void TickRecompileShaderRequestsPrivate(UE::Cook::FTickStackData& StackData);

public:

	enum ECookOnTheSideResult
	{
		COSR_None					= 0x00000000,
		COSR_CookedMap				= 0x00000001,
		COSR_CookedPackage			= 0x00000002,
		COSR_ErrorLoadingPackage	= 0x00000004,
		COSR_RequiresGC				= 0x00000008,
		COSR_WaitingOnCache			= 0x00000010,
		COSR_MarkedUpKeepPackages	= 0x00000040,
		COSR_RequiresGC_OOM			= 0x00000080,
		COSR_RequiresGC_PackageCount= 0x00000100,
		COSR_RequiresGC_Periodic	= 0x00000200,
		COSR_YieldTick				= 0x00000400,
		COSR_RequiresGC_Soft		= 0x00000800,
		COSR_RequiresGC_IdleTimer   UE_DEPRECATED(5.6, "Use COSR_RequiresGC_Periodic") = 0x40000000,
		COSR_RequiresGC_Soft_OOM	UE_DEPRECATED(5.6, "Use COSR_RequiresGC_Soft or COSR_RequiresGC_Periodic") = 0x80000000,
	};

	struct FCookByTheBookStartupOptions
	{
	public:
		TArray<ITargetPlatform*> TargetPlatforms;
		TArray<FString> CookMaps;
		TArray<FString> CookDirectories;
		TArray<FString> NeverCookDirectories;
		TArray<FString> CookCultures;
		TArray<FString> IniMapSections;
		/** list of packages we should cook, used to specify specific packages to cook */
		TArray<FString> CookPackages;
		ECookByTheBookOptions CookOptions = ECookByTheBookOptions::None;
		FString DLCName;
		FString CreateReleaseVersion;
		FString BasedOnReleaseVersion;
		bool bGenerateStreamingInstallManifests = false;
		bool bGenerateDependenciesForMaps = false;
		/** this is a flag for dlc, will cause the cooker to error if the dlc references engine content */
		bool bErrorOnEngineContentUse = false;
		/** True if CookByTheBook is being run in cooklist mode and will not be loading/saving packages. */
		bool bCookList = false;
	};

	struct FCookOnTheFlyStartupOptions
	{
		static constexpr int32 AnyPort = 0;
		static constexpr int32 DefaultPort = -1;

		/** What port the network file server or the I/O store connection server should bind to */
		int32 Port = DefaultPort;
		/** Whether to save the cooked output to the Zen storage server. */
		bool bZenStore = false;
		/**
		 * Whether the network file server should use a platform-specific communication protocol instead of TCP
		 * (used when bZenStore == false)
		 */
		bool bPlatformProtocol = false;
		/** Target platforms */
		TArray<ITargetPlatform*> TargetPlatforms;
	};

	UNREALED_API virtual ~UCookOnTheFlyServer();

	// FTickableEditorObject interface used by cook on the side
	UNREALED_API TStatId GetStatId() const override;
	UNREALED_API void Tick(float DeltaTime) override;
	UNREALED_API bool IsTickable() const override;
	ECookMode::Type GetCookMode() const { return CurrentCookMode; }
	ECookInitializationFlags GetCookFlags() const { return CookFlags; }

	// ICookInfo interface
	UNREALED_API virtual UE::Cook::FInstigator GetInstigator(FName PackageName) const override;
	UNREALED_API virtual TArray<UE::Cook::FInstigator> GetInstigatorChain(FName PackageName) const override;
	UNREALED_API virtual UE::Cook::ECookType GetCookType() const override;
	UNREALED_API virtual UE::Cook::ECookingDLC GetCookingDLC() const override;
	UNREALED_API virtual FString GetDLCName() const override;
	UNREALED_API virtual FString GetBasedOnReleaseVersion() const override;
	UNREALED_API virtual FString GetCreateReleaseVersion() const override;
	UNREALED_API virtual UE::Cook::EProcessType GetProcessType() const override;
	UNREALED_API virtual UE::Cook::ECookValidationOptions GetCookValidationOptions() const override;
	UNREALED_API virtual bool IsIncremental() const override;
	UNREALED_API virtual TArray<const ITargetPlatform*> GetSessionPlatforms() const override;
	UNREALED_API virtual FString GetCookOutputFolder(const ITargetPlatform* TargetPlatform) const override;
	UNREALED_API virtual FString GetCookMetadataOutputFolder(const ITargetPlatform* TargetPlatform) const override;	
	UNREALED_API virtual const TSet<IPlugin*>* GetEnabledPlugins(const ITargetPlatform* TargetPlatform) const override;

	UNREALED_API virtual void RegisterCollector(UE::Cook::IMPCollector* Collector,
		UE::Cook::EProcessType ProcessType = UE::Cook::EProcessType::AllMPCook) override;
	UNREALED_API virtual void UnregisterCollector(UE::Cook::IMPCollector* Collector) override;
	UNREALED_API virtual void RegisterArtifact(UE::Cook::ICookArtifact* Artifact) override;
	UNREALED_API virtual void UnregisterArtifact(UE::Cook::ICookArtifact* Artifact) override;

	UNREALED_API virtual void GetCulturesToCook(TArray<FString>& OutCulturesToCook) const override;
	// End ICookInfo interface

	// UE::PackageWriter::Private::ICookerInterface
	UNREALED_API virtual EPackageWriterResult CookerBeginCacheForCookedPlatformData(
		UE::PackageWriter::Private::FBeginCacheForCookedPlatformDataInfo& Info) override;
	UNREALED_API virtual void RegisterDeterminismHelper(ICookedPackageWriter* PackageWriter, UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) override;
	UNREALED_API virtual bool IsDeterminismDebug() const override;
	UNREALED_API virtual void WriteFileOnCookDirector(const UE::PackageWriter::Private::FWriteFileData& FileData,
		FMD5& AccumulatedHash, const TRefCountPtr<FPackageHashes>& PackageHashes,
		IPackageWriter::EWriteOptions WriteOptions) override;
	UNREALED_API virtual UE::Cook::ICookSandbox* GetCookSandbox() override;
	// End UE::PackageWriter::Private::ICookerInterface

	/** Dumps cooking stats to the log. Run from the exec command "Cook stats". */
	UNREALED_API void DumpStats();

	/** Initialize *this so that either CookOnTheFly or CookByTheBook can be started and ticked */
	UNREALED_API void Initialize( ECookMode::Type DesiredCookMode, ECookInitializationFlags InCookInitializationFlags,
		const FString& OutputDirectoryOverride = FString() );

	/**
	 * Initialize cook on the fly server
	 *
	 * @param InCookOnTheFlyOptions Cook on the fly startup options
	 *
	 * @return true on success, false otherwise.
	 */
	UNREALED_API bool StartCookOnTheFly(FCookOnTheFlyStartupOptions InCookOnTheFlyOptions);

	/** Broadcast the fileserver's presence on the network */
	UNREALED_API bool BroadcastFileserverPresence( const FGuid &InstanceId );
	
	/** Shutdown *this from running in CookOnTheFly mode */
	UNREALED_API void ShutdownCookOnTheFly();

	/**
	* Start a cook by the book session
	* Cook on the fly can't run at the same time as cook by the book
	*/
	UNREALED_API void StartCookByTheBook( const FCookByTheBookStartupOptions& CookByTheBookStartupOptions );

	/** Queue a cook by the book cancel (can be called from any thread) */
	UNREALED_API void QueueCancelCookByTheBook();

	/** Cancel the currently running cook by the book (needs to be called from the game thread) */
	UNREALED_API void CancelCookByTheBook();

	/** Connect to the CookDirector host, Initialize this UCOTFS, and start the CookWorker session */
	UNREALED_API bool TryInitializeCookWorker();

	/** Log stats for the CookWorker; this is called before the connection to the director is terminated. */
	UNREALED_API void LogCookWorkerStats();

	/** Terminate the CookWorker session */
	UNREALED_API void ShutdownCookAsCookWorker();

	/**
	 * Is the local CookOnTheFlyServer in a cook session in any CookMode?
	 * Used to restrict operations when cooking and reduce cputime when not cooking.
	 */
	UNREALED_API bool IsInSession() const;
	/** Is the local CookOnTheFlyServer in a cook session in CookByTheBook Mode? */
	UNREALED_API bool IsCookByTheBookRunning() const;

	/** Execute class-specific special case cook postloads and reference discovery on a given package. */
	UNREALED_API void PostLoadPackageFixup(UE::Cook::FPackageData& PackageData, UPackage* Package);

	/** Execute validation on a loaded source package. */
	UNREALED_API EDataValidationResult ValidateSourcePackage(UE::Cook::FPackageData& PackageData, UPackage* Package);

	/** Tick CBTB until it finishes or needs to yield. Should only be called when in CookByTheBook Mode. */
	UNREALED_API uint32 TickCookByTheBook(const float TimeSlice, ECookTickFlags TickFlags = ECookTickFlags::None);
	/** Tick COTF until it finishes or needs to yield. Should only be called when in CookOnTheFly Mode. */
	UNREALED_API uint32 TickCookOnTheFly(const float TimeSlice, ECookTickFlags TickFlags = ECookTickFlags::None);
	/** Tick CookWorker until it finishes or needs to yield. Should only be called when in CookWorker Mode. */
	UNREALED_API uint32 TickCookWorker();
	/** Log a list of all of the transitively requested packages. */
	UNREALED_API void RunCookList(ECookListOptions CookListOptions);

	/** Clear all the previously cooked data all cook requests from now on will be considered recook requests */
	UNREALED_API void ClearAllCookedData();

	/** Demote PackageDatas in any queue back to Idle, and eliminate pending requests. Used when canceling a cook. */
	UNREALED_API void CancelAllQueues();

	/**
	 * Clear any cached cooked platform data for a platform and call ClearCachedCookedPlatformData on all UObjects.
	 * 
	 * @param TargetPlatform platform to clear all the cached data for
	 */
	UNREALED_API void ClearCachedCookedPlatformDataForPlatform(const ITargetPlatform* TargetPlatform);

	/**
	 * Clear all the previously cooked data for the platform passed in 
	 * 
	 * @param TargetPlatform the platform to clear the cooked packages for
	 */
	UNREALED_API void ClearPlatformCookedData(const ITargetPlatform* TargetPlatform);

	/**
	 * Clear platforms' explored flags for all PackageDatas and optionally clear the cookresult flags.
	 *
	 * @param TargetPlatforms List of pairs with targetplatforms to reset and bool bResetResults indicating whether
	 *        the platform should clear CookResults in addition to clearing IsExplored.
	 */
	UNREALED_API void ResetCook(TConstArrayView<TPair<const ITargetPlatform*, bool>> TargetPlatforms);

	/**
	 * Recompile any global shader changes 
	 * if any are detected then clear the cooked platform data so that they can be rebuilt
	 * 
	 * @return return true if shaders were recompiled
	 */
	UNREALED_API bool RecompileChangedShaders(const TArray<const ITargetPlatform*>& TargetPlatforms);

	/**
	 * Force stop whatever pending cook requests are going on and clear all the cooked data
	 * Note CookOnTheFly clients may not be able to recover from this if they are waiting on a cook request to complete
	 */
	UNREALED_API void StopAndClearCookedData();

	UNREALED_API void TickRequestManager();

	/**
	 * Return whether the tick needs to take any action for the current session. If not, the session is done.
	 * Used for external managers of the cooker to know when to tick it.
	 */
	UNREALED_API bool HasRemainingWork() const;
	UNREALED_API void WaitForRequests(int TimeoutMs);

	UNREALED_API uint32 NumConnections() const;

	/** Is the local CookOnTheFlyServer running in the editor? */
	UNREALED_API bool IsCookingInEditor() const;

	/** Is the local CookOnTheFlyServer initialized to run in real time mode (respects the timeslice), (e.g. in Editor)? */
	UNREALED_API bool IsRealtimeMode() const;

	/** Is the local CookOnTheFlyServer initialized to run CookByTheBook (find all required packages and cook them)? */
	UNREALED_API bool IsCookByTheBookMode() const;
	/**
	 * Is the CookDirector (local CookOnTheFlyServer for SPCook or MPDirector, or the remote MPDirector for a
	 * CookWorker) initialized to run CookByTheBook?
	 */
	UNREALED_API bool IsDirectorCookByTheBook() const;

	/**
	 * Is the local CookOnTheFlyServer in a mode that uses ShaderCodeLibraries rather than storing shaders in
	 * the Packages that use them?
	 */
	UNREALED_API bool IsUsingShaderCodeLibrary() const;

	/**
	 * Is the local CookOnTheFlyServer using ZenStore (cooked packages are stored in Zen's Cache storage rather
	 * than as loose files on disk)?
	 */
	UNREALED_API bool IsUsingZenStore() const;
	/** Is the databuild using IoStore (cooked packages will be stored in iostore containers rather than pak files)? */
	UNREALED_API bool IsUsingIoStore() const;

	/**
	 * Is the local CookOnTheFlyServer initialized to run CookOnTheFly (accept connections from game executables,
	 * cook what they ask for)?
	 */
	UNREALED_API bool IsCookOnTheFlyMode() const;
	/**
	 * Is the CookDirector (local CookOnTheFlyServer for SPCook or MPDirector, or the remote MPDirector for a
	 * CookWorker) initialized to run CookOnTheFly?
	 */
	UNREALED_API bool IsDirectorCookOnTheFly() const;

	/**
	 * Is the local CookOnTheFlyServer initialized to run as a CookWorker (connect to a Director cooker cooking
	 * CBTB or COTF, cook what it assigns to us)?
	 */
	UNREALED_API bool IsCookWorkerMode() const;

	UNREALED_API virtual void BeginDestroy() override;

	/** Returns the configured number of packages to process before GC */
	UNREALED_API uint32 GetPackagesPerGC() const;

	/** Returns the configured number of packages to process before partial GC */
	UNREALED_API uint32 GetPackagesPerPartialGC() const;

	/** Returns the configured amount of idle time before forcing a GC */
	UNREALED_API double GetIdleTimeToGC() const;

	UE_DEPRECATED(5.2, "UCookOnTheFlyServer now uses a more complicated private GC scheme; HasExceededMaxMemory is no longer used and returns false")
	bool HasExceededMaxMemory() { return false; }
	UNREALED_API void SetGarbageCollectType(uint32 ResultFlagsFromTick);
	UNREALED_API void ClearGarbageCollectType();

	UNREALED_API void OnCookerStartCollectGarbage(uint32& ResultFlagsFromTick);
	UNREALED_API void OnCookerEndCollectGarbage(uint32& ResultFlagsFromTick);
	UNREALED_API int32 NumLoadedPackages() const;
	UNREALED_API void EvaluateGarbageCollectionResults(bool bWasDueToOOM, bool bWasPartialGC, uint32 ResultFlags,
		int32 NumObjectsBeforeGC, const FPlatformMemoryStats& MemStatsBeforeGC, int32 NumPackagesBeforeGC,
		const FGenericMemoryStats& AllocatorStatsBeforeGC,
		int32 NumObjectsAfterGC, const FPlatformMemoryStats& MemStatsAfterGC, int32 NumPackagesAfterGC,
		const FGenericMemoryStats& AllocatorStatsAfterGC,
		float GCDurationSeconds);
	UNREALED_API bool NeedsDiagnosticSecondGC() const;

	bool IsStalled();

	/**
	 * RequestPackage to be cooked
	 *
	 * @param StandardFileName FileName of the package in standard format as returned by FPaths::MakeStandardFilename
	 * @param TargetPlatforms The TargetPlatforms we want this package cooked for
	 * @param bForceFrontOfQueue should package go to front of the cook queue (next to be processed) or the end
	 */
	UNREALED_API bool RequestPackage(const FName& StandardFileName, const TArrayView<const ITargetPlatform* const>& TargetPlatforms,
		const bool bForceFrontOfQueue);

	/**
	 * RequestPackage to be cooked
	 * This function can only be called while the cooker is in cook by the book mode
	 *
	 * @param StandardPackageFName name of the package in standard format as returned by FPaths::MakeStandardFilename
	 * @param bForceFrontOfQueue should package go to front of the cook queue (next to be processed) or the end
	 */
	UNREALED_API bool RequestPackage(const FName& StandardPackageFName, const bool bForceFrontOfQueue);


	/**
	* Callbacks from editor 
	*/

	UNREALED_API void OnObjectModified( UObject *ObjectMoving );
	UNREALED_API void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
	UNREALED_API void OnObjectUpdated( UObject *Object );
	UNREALED_API void OnObjectSaved( UObject *ObjectSaved, FObjectPreSaveContext SaveContext );

	/**
	* Marks a package as dirty for cook
	* causes package to be recooked on next request (and all dependent packages which are currently cooked)
	*/
	UNREALED_API void MarkPackageDirtyForCooker( UPackage *Package, bool bAllowInSession = false );

	/**
	 * Helper function for MarkPackageDirtyForCooker. Executes the MarkPackageDirtyForCooker operations that are
	 * only safe to execute from the scheduler's designated point for handling external requests.
	 */
	UNREALED_API void MarkPackageDirtyForCookerFromSchedulerThread(const FName& PackageName);

	/**
	 * MaybeMarkPackageAsAlreadyLoaded
	 * Mark the package as already loaded if we have already cooked the package for all requested target platforms
	 * this hints to the objects on load that we don't need to load all our bulk data
	 * 
	 * @param Package to mark as not requiring reload
	 */
	UNREALED_API void MaybeMarkPackageAsAlreadyLoaded(UPackage* Package);

	// Callbacks from UObject globals
	UNREALED_API void PreGarbageCollect();
	UNREALED_API void CookerAddReferencedObjects(FReferenceCollector& Ar);
	UNREALED_API void PostGarbageCollect();

	/** Calculate the ShaderLibrary codedir and metadatadir */
	UNREALED_API void GetShaderLibraryPaths(const ITargetPlatform* TargetPlatform, FString& OutShaderCodeDir,
		FString& OutMetaDataPath, bool bUseProjectDirForDLC=false);

	/** Print detailed stats from the cook. */
	UNREALED_API void PrintDetailedCookStats();

	/** Is the local CookOnTheFlyServer cooking a DLC plugin rather than a Project+Engine+EmbeddedPlugins? */
	UNREALED_API bool IsCookingDLC() const;

	/** Is the local CookOnTheFlyServer ONLY cooking a DLC plugin, ignoring all references outside the plugin path? */
	UNREALED_API bool IsCookingDLCPluginOnly() const;

	/** Return whether EDLCookInfo verification has NOT been rendered useless by settings such as CookFilter. */
	UNREALED_API bool ShouldVerifyEDLCookInfo() const;

protected:
	// FExec interface used in the editor
	UNREALED_API virtual bool Exec_Editor(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:

	UE::Cook::FInstigator GetInstigator(FName PackageName, UE::Cook::EReachability Reachability) const;
	bool PumpHasExceededMaxMemory(uint32& OutResultFlags);

	/**
	 * Is the local CookOnTheFlyServer initialized to run in CookOnTheFly AND using the legacy scheduling method
	 * that predates COTF2?
	 */
	bool IsUsingLegacyCookOnTheFlyScheduling() const;
	bool IsDebugRecordUnsolicited() const;

	/** Update accumulators of editor data and consume editor changes after last cook when starting cooks in the editor. */
	void BeginCookEditorSystems();
	void BeginCookPackageWriters(FBeginCookContext& BeginContext);
	void BeginCookDirector(FBeginCookContext& BeginContext);
	/**
	 * Initialization for systems that persist across CookSessions but that are not available during Initialize(which
	 * can occur during EngineStartup if !bDisableCookInEditor). We initialize these as late as possible: at the
	 * beginning of the first session, or for CookOnTheFly at the first cook request.
	 */
	void InitializeAtFirstSession();
	/** Initialize steps that are reexecuted at the beginning of every cook session. */
	void InitializeSession();

	//////////////////////////////////////////////////////////////////////////
	// cook by the book specific functions
	const FCookByTheBookStartupOptions& BlockOnPrebootCookGate(bool& bOutAbortCook,
		const FCookByTheBookStartupOptions& CookByTheBookStartupOptions,
		TOptional<FCookByTheBookStartupOptions>& ModifiedStartupOptions);
	/** Construct the on-stack data for StartCook functions, based on the arguments to StartCookByTheBook */
	FBeginCookContext CreateBeginCookByTheBookContext(const FCookByTheBookStartupOptions& StartupOptions);
	/** Set the PlatformManager's session platforms and finish filling out the BeginContext for StartCookByTheBook */
	void SelectSessionPlatforms(FBeginCookContext& BeginContext);

	/** Calculate the global dependencies hashes for every platform being cooked. This needs to be called after SelectSessionPlatforms.*/
	void CalculateGlobalDependenciesHashes();

	/** CollectFilesToCook and add them as external requests. */
	void GenerateInitialRequests(FBeginCookContext& BeginContext);
	/** If cooking DLC, initialize cooked PackageDatas for all of the packages cooked by the base release. */
	void RecordDLCPackagesFromBaseGame(FBeginCookContext& BeginContext);
	void RegisterCookByTheBookDelegates();
	void UnregisterCookByTheBookDelegates();
	/** Start the collection of EDL diagnostics for the cook, if applicable. */
	void BeginCookEDLCookInfo(FBeginCookContext& BeginContext);

	/** Collect all the files which need to be cooked for a cook by the book session */
	void CollectFilesToCook(TArray<FName>& FilesInPath, TMap<FName, UE::Cook::FInstigator>& Instigators,
		const TArray<FString>& CookMaps, const TArray<FString>& CookDirectories, 
		const TArray<FString>& IniMapSections, ECookByTheBookOptions FilesToCookFlags,
		const TArrayView<const ITargetPlatform* const>& TargetPlatforms,
		const TMap<FName, TArray<FName>>& GameDefaultObjects);

	/** Gets all game default objects for all platforms. */
	static void GetGameDefaultObjects(const TArray<ITargetPlatform*>& TargetPlatforms, TMap<FName,
		TArray<FName>>& GameDefaultObjectsOut);

	/*
	 * Collect filespackages that should not be cooked from ini settings and commandline.
	 * Does not include checking UAssetManager, which has to be queried later
	 * This function is const because it is not always called and should avoid side effects
	 */
	TArray<FName> GetNeverCookPackageNames(TArrayView<const FString> ExtraNeverCookDirectories
		= TArrayView<const FString>()) const;

	/** AddFileToCook add file to cook list */
	void AddFileToCook( TArray<FName>& InOutFilesToCook, TMap<FName, UE::Cook::FInstigator>& InOutInstigators,
		const FString &InFilename, const UE::Cook::FInstigator& Instigator) const;
	/** AddFileToCook add file to cook list */
	void AddFlexPathToCook(TArray<FName>& InOutFilesToCook, TMap<FName, UE::Cook::FInstigator>& InOutInstigators,
		const FString& InFlexPath, const UE::Cook::FInstigator& Instigator) const;

	void RegisterValidationCookArtifact();

	/** Return the name to use for the project's global shader library */
	FString GetProjectShaderLibraryName() const;

	void RegisterShaderLibraryIncrementalCookArtifact(FBeginCookContext& BeginContext);

	/** Invokes the necessary FShaderCodeLibrary functions to start cooking the shader code library. */
	void BeginCookStartShaderCodeLibrary(FBeginCookContext& BeginContext);
	/**
	 * Finishes async operations from ShaderCodeLibraryBeginCookStart, saves initial data to disk,
	 * and opens the library for the rest of the cook
	 */
	void BeginCookFinishShaderCodeLibrary(FBeginCookContext& BeginContext);
    
	/** Opens Global shader library. Global shaderlib is a special case - no chunks and not connected to assets. */
	void OpenGlobalShaderLibrary();

	/** Saves Global shader library. Global shaderlib is a special case - no chunks and not connected to assets. */
	void SaveAndCloseGlobalShaderLibrary();

	/** Invokes the necessary FShaderCodeLibrary functions to open a named code library. */
	void OpenShaderLibrary(FString const& Name);
    
	/**
	 * Finishes collection of data that should be in the named code library (e.g. load data from previous
	 * incremental cook)
	 */
	void FinishPopulateShaderLibrary(const ITargetPlatform* TargetPlatform, FString const& Name, bool bInitializeReferenceTracking);
	/**
	 * Prune the named ShaderLibrary of any data no longer referenced by assets in the current cook or
	 * IncrementalCook oplog.
	 */
	void PruneShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name);

	/** Invokes the necessary FShaderCodeLibrary functions to save and close a named code library. */
	void SaveShaderLibrary(const ITargetPlatform* TargetPlatform, FString const& Name);

	/** Called during CookByTheBookFinished. */
	void ShutdownShaderLibraryCookerAndCompilers(const FString& ProjectShaderLibraryName);

	/** Log shader stats during CookByTheBookFinished. */
	void DumpShaderTypeStats(const ITargetPlatform* TargetPlatform, const FString& PlatformNameString);

	/**
	 * Calls the ShaderPipelineCacheToolsCommandlet to build a upipelinecache file from the stable pipeline cache
	 * (.spc, in the past textual .stablepc.csv) file, if any
	 */
	void CreatePipelineCache(const ITargetPlatform* TargetPlatform, const FString& LibraryName);

	void RegisterShaderChunkDataGenerator();

	/** Called at the end of CookByTheBook to write aggregated data such as AssetRegistry and shaders. */
	void CookByTheBookFinished();
	void CookByTheBookFinishedInternal();
	bool IsForceNoFilterAssetsFromAssetRegistry();

	/** Clears session-lifetime data from COTFS and CookPackageDatas. */
	void ShutdownCookSession();

	/** Print some stats when finishing or cancelling CookByTheBook */
	void PrintFinishStats();

	/**
	 * Get all the packages which are listed in asset registry passed in.  
	 *
	 * @param AssetRegistryPath path of the assetregistry.bin file to read
	 * @param bVerifyPackagesExist whether to verify the packages exist on disk.
	 * @param bSkipUncookedPackages whether to skip over packages that are in the AR but were not cooked/saved
	 * @param OutPackageDatas out list of packagename/filenames for packages contained in the asset registry file
	 * @return true if successfully read false otherwise
	 */
	bool GetAllPackageFilenamesFromAssetRegistry( const FString& AssetRegistryPath, bool bVerifyPackagesExist,
		 bool bSkipUncookedPackages, TArray<UE::Cook::FConstructPackageData>& OutPackageDatas) const;

	/** Build a map of the package dependencies used by each Map Package. */
	TMap<FName, TSet<FName>> BuildMapDependencyGraph(const ITargetPlatform* TargetPlatform);

	/** Write a MapDependencyGraph to a metadata file in the sandbox for the given platform. */
	void WriteMapDependencyGraph(const ITargetPlatform* TargetPlatform, TMap<FName, TSet<FName>>& MapDependencyGraph);

	/** Generates the CachedEditorThumbnails.bin file */
	void GenerateCachedEditorThumbnails();

	void InitializeAllCulturesToCook(TConstArrayView<FString> CookCultures);
	void CompileDLCLocalization(FBeginCookContext& BeginContext);
	/** Find localization dependencies for all packages, used to add required localization files as soft references. */
	void GenerateLocalizationReferences();
	void RegisterLocalizationChunkDataGenerator();

	//////////////////////////////////////////////////////////////////////////
	// cook on the fly specific functions

	/** Construct the on-stack data for StartCook functions, based on the arguments to StartCookOnTheFly */
	FBeginCookContext CreateBeginCookOnTheFlyContext(const FCookOnTheFlyStartupOptions& Options);
	/** Construct the on-stack data for StartCook functions, based on the arguments when adding a COTF platform */
	FBeginCookContext CreateAddPlatformContext(ITargetPlatform* TargetPlatform);
	void GetCookOnTheFlyUnsolicitedFiles(const ITargetPlatform* TargetPlatform, const FString& PlatformName,
		TArray<FString>& UnsolicitedFiles, const FString& Filename, bool bIsCookable);

	//////////////////////////////////////////////////////////////////////////
	// CookWorker specific functions
	/** Start a CookWorker Session */
	void StartCookAsCookWorker();
	/** Construct the on-stack data for StartCook functions, based on the CookWorker's configuration */
	FBeginCookContext CreateCookWorkerContext();
	void CookAsCookWorkerFinished();
	/** Return the best packages to retract and give to other workers -the ones with least effort spent so far. */
	void GetPackagesToRetract(int32 NumToRetract, TArray<FName>& OutRetractionPackages);

	//////////////////////////////////////////////////////////////////////////
	// general functions

	/** Perform any special processing for freshly loaded packages */
	void ProcessUnsolicitedPackages(TArray<FName>* OutDiscoveredPackageNames = nullptr,
		TMap<FName, UE::Cook::FInstigator>* OutInstigators = nullptr);

	/**
	 * Loads a package and prepares it for cooking
	 *  this is the same as a normal load but also ensures that the sublevels are loaded if they are streaming sublevels
	 *
	 * @param BuildFilename long package name of the package to load 
	 * @param OutPackage UPackage of the package loaded, non-null on success, may be non-null on failure if
	 *        the UPackage existed but had another failure
	 * @param ReportingPackageData PackageData which caused the load, for e.g. generated packages
	 * @return Whether LoadPackage was completely successful and the package can be cooked
	 */
	bool LoadPackageForCooking(UE::Cook::FPackageData& PackageData, UPackage*& OutPackage,
		UE::Cook::FPackageData* ReportingPackageData = nullptr);

	/**
	 * Read information about the previous cook from disk and set whether the current cook is incremental based on
	 * previous cook, config, and commandline.
	 */
	void LoadBeginCookIncrementalFlags(FBeginCookContext& BeginContext);
	/** const because it is not always calledand should avoid sideeffects */
	void LoadBeginCookIncrementalFlagsLocal(FBeginCookContext& BeginContext);
	/** Initialize the sandbox for a new cook session */
	void BeginCookSandbox(FBeginCookContext& BeginContext);

	/** Set parameters that rely on config settings from startup and don't depend on StartCook* functions. */
	void LoadInitializeConfigSettings(const FString& InOutputDirectoryOverride);
	void SetInitializeConfigSettings(UE::Cook::FInitializeConfigSettings&& Settings);
	void ParseCookFilters();
	void ParseCookFilters(const TCHAR* Parameter, const TCHAR* Message, TSet<FName>& OutFilterClasses);

	/** Set parameters that rely on config and CookByTheBook settings. */
	void LoadBeginCookConfigSettings(FBeginCookContext& BeginContext);
	void SetBeginCookConfigSettings(FBeginCookContext& BeginContext, UE::Cook::FBeginCookConfigSettings&& Settings);
	void SetNeverCookPackageConfigSettings(FBeginCookContext& BeginContext, UE::Cook::FBeginCookConfigSettings& Settings);

	/** Finalize the package store */
	void FinalizePackageStore(const ITargetPlatform* TargetPlatform, const TSet<FName>& CookedPackageNames);
	/**
	 * Calculate which Plugins (and also /Game and /Engine) had packages referenced in the current session,
	 * this is important to know for DLC cooks.
	 */
	void CalculateReferencedPlugins(const ITargetPlatform* TargetPlatform,
		const TSet<FName>& CookedPackageNames, ICookedPackageWriter::FReferencedPluginsInfo& OutReferencedPluginsInfo);
	/** Empties SavePackageContexts and deletes the contents */
	void ClearPackageStoreContexts();

	/** Initialize shaders for the specified platforms when running cook on the fly. */
	void InitializeShadersForCookOnTheFly(const TArrayView<ITargetPlatform* const>& NewTargetPlatforms);

	/**
	 * Some content plugins does not support all target platforms.
	 * Build up a map of unsupported packages per platform that can be checked before saving.
	 * const because it is not always called and should avoid side effects
	 */
	void DiscoverPlatformSpecificNeverCookPackages(
		const TArrayView<const ITargetPlatform* const>& TargetPlatforms, const TArray<FString>& UBTPlatformStrings,
		UE::Cook::FBeginCookConfigSettings& Settings) const;

	/**
	 * GetDependentPackages
	 * get package dependencies according to the asset registry
	 * 
	 * @param Packages List of packages to use as the root set for dependency checking
	 * @param Found return value, all objects which package is dependent on
	 */
	void GetDependentPackages( const TSet<UPackage*>& Packages, TSet<FName>& Found);

	/**
	 * GetDependentPackages
	 * get package dependencies according to the asset registry
	 *
	 * @param Root set of packages to use when looking for dependencies
	 * @param FoundPackages list of packages which were found
	 */
	void GetDependentPackages(const TSet<FName>& RootPackages, TSet<FName>& FoundPackages);

	/**
	 * ContainsWorld
	 * use the asset registry to determine if a Package contains a UWorld or ULevel object
	 * 
	 * @param PackageName to return if it contains the a UWorld object or a ULevel
	 * @return true if the Package contains a UWorld or ULevel false otherwise
	 */
	bool ContainsMap(const FName& PackageName) const;


	/** 
	 * Returns true if this package contains a redirector, and fills in paths
	 *
	 * @param PackageName to return if it contains a redirector
	 * @param RedirectedPaths map of original to redirected object paths
	 * @return true if the Package contains a redirector false otherwise
	 */
	bool ContainsRedirector(const FName& PackageName, TMap<FSoftObjectPath, FSoftObjectPath>& RedirectedPaths) const;
	
	/**
	 * Prepares save by calling BeginCacheForCookedPlatformData on all UObjects in the package. 
	 * Also splits the package if an object of this package has its class registered to a corresponding 
	 * FRegisteredCookPackageSplitter and an instance of this splitter class returns true ShouldSplitPackage 
	 * for this UObject.
	 *
	 * @param PackageData the PackageData used to gather all uobjects from
	 * @param bIsPrecaching true if called for precaching
	 * @param OutDemotionRequestedReason Value is ESuppressCookReason::NotSuppressed if no suppression is requested,
	 *        or if returned PollStatus is not InComplete. It can be set to a different value, requesting suppression,
	 *        and returned PollStatus will be InComplete. If caller handles demotion, the caller should
	 *        demote the package with the given reason. It is valid to call PrepareSave again later if caller does not
	 *        handle demotion.
	 * @return false if time slice was reached, true if all objects have had BeginCacheForCookedPlatformData called
	 */
	UE::Cook::EPollStatus PrepareSave(UE::Cook::FPackageData& PackageData, UE::Cook::FCookerTimer& Timer,
		bool bPrecaching, UE::Cook::ESuppressCookReason& OutDemotionRequestedReason);
	UE::Cook::EPollStatus PrepareSaveInternal(UE::Cook::FPackageData& PackageData, UE::Cook::FCookerTimer& Timer,
		bool bPrecaching, UE::Cook::ESuppressCookReason& OutDemotionRequestedReason);
	/** Call BeginCacheForCookedPlatformData on all objects currently found in the PackageData. */
	UE::Cook::EPollStatus CallBeginCacheOnObjects(UE::Cook::FPackageData& PackageData, UPackage* Package,
		TArray<UE::Cook::FCachedObjectInOuter>& Objects, int32& NextIndex, UE::Cook::FCookerTimer& Timer);

	/**
	 * Frees all the memory used to call BeginCacheForCookedPlatformData on all the objects in PackageData.
	 * If the calls were incomplete because the PackageData's save was cancelled, handles canceling them and
	 * leaving any required CancelManagers in PendingCookedPlatformDatas
	 * 
	 * @param bCompletedSave If false, data that can not be efficiently recomputed will be preserved to try
	 *        the save again. If true, all data will be wiped.
	 * @param ReleaseSaveReason Why the save data is being released, allows specifying how much to tear down
	 */
	void ReleaseCookedPlatformData(UE::Cook::FPackageData& PackageData, UE::Cook::EStateChangeReason ReleaseSaveReason,
		UE::Cook::EPackageState NewState);

	/**
	 * Poll the PendingCookedPlatformDatas and release their resources when they are complete.
	 * This is done inside of PumpSaveQueue, but is also required after a cancelled cook so that references to
	 * the pending objects will be eventually dropped.
	 */
	void TickCancels();

	/**
	* GetCurrentIniVersionStrings gets the current ini version strings for compare against previous cook
	* 
	* @param IniVersionStrings return list of the important current ini version strings
	* @return false if function fails (should assume all platforms are out of date)
	*/
	bool GetCurrentIniVersionStrings( const ITargetPlatform* TargetPlatform,
		UE::Cook::FIniSettingContainer& IniVersionStrings ) const;

	/**
	* GetCookedIniVersionStrings gets the ini version strings used in previous cook for specified target platform
	* 
	* @param IniVersionStrings return list of the previous cooks ini version strings
	* @return false if function fails to find the ini version strings
	*/
	bool GetCookedIniVersionStrings( const ITargetPlatform* TargetPlatform,
		UE::Cook::FIniSettingContainer& IniVersionStrings, TMap<FString, FString>& AdditionalStrings ) const;

	/** Get the path to the CookSettings file for the given artifact in the output directory for the given platform. */
	FString GetCookSettingsFileName(const ITargetPlatform* TargetPlatform, const FString& ArtifactName) const;
	/** Load the CookSettings configfile for the given ArtifactName from output directory for the given platform. */
	FConfigFile LoadCookSettings(const ITargetPlatform* TargetPlatform, const FString& ArtifactName);
	/** Save the CookSettings configfile for the given ArtifactName to the output directory for the given platform. */
	void SaveCookSettings(const FConfigFile& ConfigFile, const ITargetPlatform* TargetPlatform, const FString& ArtifactName) const;
	/** Load the global CookSettings written at beginning of cook and rewrite them with CookInProgress flag removed. */
	void ClearCookInProgressFlagFromGlobalCookSettings(const ITargetPlatform* TargetPlatform) const;

	/**
	 * Convert a path to a full sandbox path. Is effected by the cooking dlc settings.
	 * This function should be used instead of calling the FSandbox Sandbox->ConvertToSandboxPath functions.
	 */
	FString ConvertToFullSandboxPath( const FString &FileName, bool bForWrite = false ) const;
	FString ConvertToFullSandboxPath( const FString &FileName, bool bForWrite, const FString& PlatformName ) const;

	/**
	 * GetSandboxAssetRegistryFilename
	 * 
	 * @return full path of the asset registry in the sandbox
	 */
	FString GetSandboxAssetRegistryFilename();

	FString GetCookedAssetRegistryFilename(const FString& PlatformName);
	FString GetCookMetadataOutputFolderInternal(const ITargetPlatform* TargetPlatform) const;

	// Return the filename to use for the cook metadata file, adjusted for DLC, sandbox, and platform.
	FString GetCookedCookMetadataFilename(const ITargetPlatform* TargetPlatform);
	void WriteCookMetadata(const ITargetPlatform* InTargetPlatform, uint64 InDevelopmentAssetRegistryHash);
	void WriteReferencedSet(const ITargetPlatform* InTargetPlatform, TArray<FName>&& CookedPackageNames);

	/* @return Full path of the CachedEditorThumbnails.bin file in the sandbox */
	FString GetSandboxCachedEditorThumbnailsFilename();

	/**
	 * Get the sandbox root directory for that platform. Is effected by the CookingDlc settings.
	 * This should be used instead of calling the Sandbox function.
	 */
	FString GetSandboxDirectory( const FString& PlatformName ) const;

	/* Create the delete-old-cooked-directory helper.*/
	FAsyncIODelete& GetAsyncIODelete();

	/** Is the local CookOnTheFlyServer cooking a Project+Engine+EmbeddedPlugin Patch based on a previous Release? */
	bool IsCookingAgainstFixedBase() const;

	/** Returns whether or not we should populate the Asset Registry using the main game content */
	bool ShouldPopulateFullAssetRegistry() const;

	/**
	 * GetBaseDirectoryForDLC
	 * 
	 * @return return the path to the DLC
	 */
	FString GetBaseDirectoryForDLC() const;

	FString GetContentDirectoryForDLC() const;

	FString GetMountedAssetPathForDLC() const;

	static FString GetMountedAssetPathForPlugin(const FString& InPluginName);

	FString GetMetadataDirectory() const;

	/**
	 * Is the local CookOnTheFlyServer cooking a Project+Engine+EmbeddedPlugin Release that can be used as a
	 * base for future DLC or Patches?
	 */
	bool IsCreatingReleaseVersion();

	/**
	 * Checks if important ini settings have changed since last cook for each target platform 
	 * 
	 * @param TargetPlatforms to check if out of date
	 * @param OutOfDateTargetPlatforms return list of out of date target platforms which should be cleaned
	 */
	bool IniSettingsOutOfDate( const ITargetPlatform* TargetPlatform ) const;

	/**
	 * Saves ini settings which are in the memory cache to the hard drive in ini files
	 *
	 * @param TargetPlatforms to save
	 */
	bool SaveCurrentIniSettings( const ITargetPlatform* TargetPlatform ) const;



	/** Does the local CookOnTheFlyServer have all flags in InCookFlags set to true? */
	bool IsCookFlagSet( const ECookInitializationFlags& InCookFlags ) const 
	{
		return (CookFlags & InCookFlags) != ECookInitializationFlags::None;
	}

	void RouteBeginCacheForCookedPlatformData(UE::Cook::FPackageData& PackageData, UObject* Obj,
		const ITargetPlatform* TargetPlatform, UE::Cook::ECachedCookedPlatformDataEvent* ExistingEvent);
	bool RouteIsCachedCookedPlatformDataLoaded(UE::Cook::FPackageData& PackageData, UObject* Obj,
		const ITargetPlatform* TargetPlatform, UE::Cook::ECachedCookedPlatformDataEvent* ExistingEvent);
	EPackageWriterResult SavePackageBeginCacheForCookedPlatformData(FName PackageName,
		const ITargetPlatform* TargetPlatform, TConstArrayView<UObject*> SaveableObjects, uint32 SaveFlags);

	/** Cook (save) a package and process the results */
	void SaveCookedPackage(UE::Cook::FSaveCookedPackageContext& Context);
	void CommitUncookedPackage(UE::Cook::FSaveCookedPackageContext& Context);
	/** Helper for package saves using ExternalActors: record ExternalActors for incremental builds. */
	void RecordExternalActorDependencies(TConstArrayView<FName> ExternalActorDependencies);

	/**
	 * Save the global shader map
	 *
	 * @param Platforms List of platforms to make global shader maps for
	 */
	void SaveGlobalShaderMapFiles(const TArrayView<const ITargetPlatform* const>& Platforms,
		ODSCRecompileCommand RecompileCommand);


	/** Create sandbox file in directory using current settings supplied */
	void CreateSandboxFile(FBeginCookContext& BeginContext);
	/** Gets the output directory respecting any command line overrides */
	FString GetOutputDirectoryOverride(FBeginCookContext& BeginContext) const;

	/**
	 * Populate cooked packages from the PackageWriter's previous manifest and assetregistry of cooked output.
	 * Delete any out of date packages from the PackageWriter's manifest.
	 */
	void PopulateCookedPackages(const TConstArrayView<const ITargetPlatform*> TargetPlatforms);

	/** Waits for the AssetRegistry to complete so that we know any missing assets are missing on disk */
	void BlockOnAssetRegistry(TConstArrayView<FString> CommandlinePackages);

	/** Construct or refresh-for-filechanges the platform-specific asset registry for the given platforms */
	void RefreshPlatformAssetRegistries(const TArrayView<const ITargetPlatform* const>& TargetPlatforms);

	/** Generates long package names for all files to be cooked */
	void GenerateLongPackageNames(TArray<FName>& FilesInPath, TMap<FName, UE::Cook::FInstigator>& Instigators);

	/** Generate the list of cook-time-created packages created by the generator package. */
	UE::Cook::EPollStatus QueueGeneratedPackages(UE::Cook::FGenerationHelper& GenerationHelper,
		UE::Cook::FPackageData& PackageData);
	/** Run additional steps in PrepareSave required when the package is a Generator or a GeneratedPackage. */
	UE::Cook::EPollStatus PrepareSaveGenerationPackage(UE::Cook::FGenerationHelper& GenerationHelper,
		UE::Cook::FPackageData& PackageData, UE::Cook::FCookerTimer& Timer, bool bPrecaching);
	/**
	 * Call BeginCacheForCookedPlatformData on objects the CookPackageSplitter plans to move
	 * into its main UPackage.
	 */
	UE::Cook::EPollStatus BeginCacheObjectsToMove(UE::Cook::FGenerationHelper& GenerationHelper,
		UE::Cook::FCookGenerationInfo& Info, UE::Cook::FCookerTimer& Timer,
		TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate);
	/** Call the CookPackageSplitter's PreSaveGeneratorPackage to create/move objects into its main UPackage. */
	UE::Cook::EPollStatus PreSaveGeneratorPackage(UE::Cook::FPackageData& PackageData,
		UE::Cook::FGenerationHelper& GenerationHelper, UE::Cook::FCookGenerationInfo& Info,
		TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate);
	/** Construct the list of generated packages that is required for some CookPackageSplitter interface calls. */
	bool TryConstructGeneratedPackagesForPopulate(UE::Cook::FPackageData& PackageData,
		UE::Cook::FGenerationHelper& GenerationHelper,
		TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate);
	/**
	 * Call BeginCacheForCookedPlatformData on any undeclared objects in the CookPackageSplitter's main UPackage
	 * after the move.
	 */
	UE::Cook::EPollStatus BeginCachePostMove(UE::Cook::FGenerationHelper& GenerationHelper,
		UE::Cook::FCookGenerationInfo& Info, UE::Cook::FCookerTimer& Timer);

	/** Try calling the CookPackageSplitter's PreSave hook on the package. */
	UE::Cook::EPollStatus TryPreSaveGeneratedPackage(UE::Cook::FGenerationHelper& GenerationHelper,
		UE::Cook::FCookGenerationInfo& GeneratedInfo);

	UE::Cook::ICookArtifactReader& FindOrCreateCookArtifactReader(const ITargetPlatform* TargetPlatform);
	const UE::Cook::ICookArtifactReader* FindCookArtifactReader(const ITargetPlatform* TargetPlatform) const;
	ICookedPackageWriter& FindOrCreatePackageWriter(const ITargetPlatform* TargetPlatform);
	const ICookedPackageWriter* FindPackageWriter(const ITargetPlatform* TargetPlatform) const;
	void FindOrCreateSaveContexts(TConstArrayView<const ITargetPlatform*> TargetPlatforms);
	UE::Cook::FCookSavePackageContext& FindOrCreateSaveContext(const ITargetPlatform* TargetPlatform);
	const UE::Cook::FCookSavePackageContext* FindSaveContext(const ITargetPlatform* TargetPlatform) const;
	/** Allocate a new FCookSavePackageContext and ICookedPackageWriter for the given platform. */
	UE::Cook::FCookSavePackageContext* CreateSaveContext(const ITargetPlatform* TargetPlatform);
	/**
	 * Delete files that exist for the package in the PackageWriter from a previous cook. Used when packages are
	 * detected as invalidated for incremental cook after cook startup (e.g. generated packages).
	 */
	void DeleteOutputForPackage(FName PackageName, const ITargetPlatform* TargetPlatform);

	/** Callback for analytics when a new UPackage is loaded. */
	void OnDiscoveredPackageDebug(FName PackageName, const UE::Cook::FInstigator& Instigator);
	/** Callback for analytics when a TObjectPtr is read. */
	void OnObjectHandleReadDebug(const TArrayView<const UObject*const>& ReadObjects);
	/** Send warnings/telemetry when a discovered or read package is found to be a hidden dependency. */
	void ReportHiddenDependency(FName Referencer, FName Dependency);
	void BroadcastCookStarted();
	void BroadcastCookFinished();

	/** Returns the current Phase of the cook; this is separate axis of behavior from CookMode. @see ECookPhase. */
	UE::Cook::ECookPhase GetCookPhase() const;
	/** Sets the current phase of the cook to the TargetPhase, and updates per-state data. */
	void SetCookPhase(UE::Cook::ECookPhase TargetPhase);

	/** Clear temporary data that is allocated during GarbageCollect for use by AddReferencedObjects and PostGC evaluation. */
	void ClearGCData();

	static UCookOnTheFlyServer* ActiveCOTFS;
	uint32		StatLoadedPackageCount = 0;
	uint32		StatSavedPackageCount = 0;
	int32		PackageDataFromBaseGameNum = 0;
	UE::Cook::FStatHistoryInt NumObjectsHistory;
	UE::Cook::FStatHistoryInt NumPackagesHistory;
	UE::Cook::FStatHistoryInt VirtualMemoryHistory;

	FCriticalSection HiddenDependenciesLock;
	/**
	 * AllowList or BlockList for reporting hidden dependencies, parsed from ini and commandline, only used if bHiddenDependenciesDebug.
	 * Read/Write only within HiddenDependenciesLock.
	 */
	TSet<FName> HiddenDependenciesClassPathFilterList;

	/** Registration handle for TObjectPtr's OnRead delegate. */
#if UE_WITH_OBJECT_HANDLE_TRACKING
	UE::CoreUObject::FObjectHandleTrackingCallbackId ObjectHandleReadHandle;
#endif // UE_WITH_OBJECT_HANDLE_TRACKING
	/** Classes (and all subclasses) that were listed as the only classes that should be cooked in the filter settings. */
	TSet<FName> CookFilterIncludedClasses;
	/** Classes (and all subclasses) that were listed as the only asset classes that should be cooked in the filter settings. */
	TSet<FName> CookFilterIncludedAssetClasses;

	ELogVerbosity::Type CookerIdleWarningSeverity = ELogVerbosity::Warning;
	ELogVerbosity::Type UnexpectedLoadWarningSeverity = ELogVerbosity::Warning;
	UE::Cook::EMPCookGeneratorSplit MPCookGeneratorSplit = UE::Cook::EMPCookGeneratorSplit::AnyWorker;
	/** True when PumpLoads has detected it is blocked on async work and CookOnTheFlyServer should do work elsewhere. */
	bool bLoadBusy = false;
	/** True when PumpSaves has detected it is blocked on async work and CookOnTheFlyServer should do work elsewhere. */
	bool bSaveBusy = false;
	/** We need to track whether the compiler has been inactive for a long time before issuing a warning about it. */
	bool bShaderCompilerWasActiveOnPreviousBusyReport = true;
	/**
	 * After cooking all requested packages in CookByTheBook we need to search for any build dependencies and request them
	 * for commit as build dependencies. This bool records whether we have done that for the current session.
	 */
	bool bKickedBuildDependencies = false;
	/**
	 * If preloading is enabled, we call TryPreload until it returns true before sending the package to LoadReady,
	 * otherwise we skip TryPreload and it goes immediately.
	 */
	bool bPreloadingEnabled = false;
	/**
	 * By default (aka CookIncremental system), we load/save TargetDomainKey hashes and tweak behavior throughout the
	 * cooker to support that. We decide whether to cook incrementally based on config/commandline options that specify
	 * forcerecook; if cooking incrementally we decide whether packages have been cooked based on an unchanged
	 * TargetDomainKey hash present in oplog.
	 * 
	 * This can be disabled and we use the old system instead - now called legacy iterative - which detects workspace
	 * domain package changes by a change in their recorded PackageSaveHash, and propagates changed packages out to
	 * AssetRegistry referencers. Legacy iterative is a simpler model that has more false positive and false negative
	 * incremental skips. We do not collect TargetDomainKey hashes when running in legacy iterative.
	 * 
	 * This field specifies which dependency handling mode we are in, and is orthogonal to whether we are cooking
	 * incrementally in the current mode; see PlatformManager.GetPlatformData().bFullCook for that property.
	 */
	bool bLegacyBuildDependencies = false;
	bool bLegacyBuildDependenciesDueToNoZenStore = false;
	/**
	 * Automated DependencyTracking is only used in CookIncremental; legacyiterative does not use its data.
	 * Additionally, it can be turned off to improve performance in CookIncremental, at the cost of not being able to
	 * cook incrementally afterwards.
	 */
	bool bDependencyTrackingEnabled = true;
	bool bCookIncrementalAllowAllClasses = false;
	bool bHiddenDependenciesDebug = false;
	bool bIncrementallyModifiedDiagnostics = false;
	bool bOnlyEditorOnlyDebug = false;
	bool bHiddenDependenciesClassPathFilterListIsAllowList = true;
	bool bFirstCookInThisProcessInitialized = false;
	bool bFirstCookInThisProcess = true;
	bool bImportBehaviorCallbackInstalled = false;
	/** True if a session is started for any mode. If started, then we have at least one TargetPlatform specified. */
	bool bSessionRunning = false;
	/** Whether we're using ZenStore for storage of cook results. If false, we are using LooseCookedPackageWriter. */
	bool bZenStore = false;
	/** Multithreaded synchronization of Pollables, accessible only inside PollablesLock. */
	bool bPollablesInTick = false;
	/** Config value that specifies whether the Skip-Only-Editor-Only feature is enabled. */
	bool bSkipOnlyEditorOnly = false;
	/** True if we're running cooklist; tweak output */
	bool bCookListMode = false;
	/**
	 * True if we want to randomize cook order, for robustness validation or to avoid encountering the same DDC
	 * build jobs when multiple machines are cooking at the same time.
	 */
	bool bRandomizeCookOrder = false;
	/** True if commandline arguments specified that we suppress the cook of packages based on filter criteria. */
	bool bCookFilter = false;
	/** True if experimental optimizations for fast startup should be used. */
	bool bCookFastStartup = false;
	/**
	 * Experimental feature to correctly invoke the BeginCacheForCookedPlatformData contracts for
	 * objects created by PreSave
	 */
	bool bCallIsCachedOnSaveCreatedObjects = false;
	/** Command-line parameter; if true then legacyiterative cooks will not be invalidated by ini changes. */
	bool bLegacyIterativeIgnoreIni = false;
	/** Command-line parameter; if true then legacyiterative cooks will not be invalidated by exe changes. */
	bool bLegacyIterativeIgnoreExe = false;
	/**
	 * Whether we should calculate the exe's hash for LegacyIiterative; might be true even if
	 * bLegacyIterativeIgnoreExe is true.
	 */
	bool bLegacyIterativeCalculateExe = true;
	/**
	 * When running as a shader server (-odsc) we avoid cooking packages; the cooker does not queue them and the game does not request them. 
	 * This mode is used to respond to the shader requests and shares that implementation with cookonthefly.
	 */
	bool bRunningAsShaderServer = false;
	/** Whether to skip saving packages that are cooked. When true, the cook will only load and process packages but not write them to disk */
	bool bSkipSave = false;
	/** Whether cooked packages should store extra data to debug indeterminism. */
	bool bDeterminismDebug = false;
	/**
	 * True (on CookWorkers and CookDirector) if and only if commandline requested CookingDLCBaseGamePlugin. Only used during DLCCooks,
	 * and exposed to system-specific code to e.g. do different validation based on whether the DLC will be welded into the base game or
	 * released separately.
	 */
	bool bCookingDLCBaseGamePlugin = false;
	/** Timers for tracking how long we have been busy, to manage retries and warnings of deadlock */
	double SaveBusyStartTimeSeconds = MAX_flt;
	double SaveBusyRetryTimeSeconds = MAX_flt;
	double SaveBusyWarnTimeSeconds = MAX_flt;
	double LoadBusyStartTimeSeconds = MAX_flt;
	double LoadBusyRetryTimeSeconds = MAX_flt;
	double LoadBusyWarnTimeSeconds = MAX_flt;
	/** Tracking for the ticking of tickable cook objects */
	double LastCookableObjectTickTime = 0.;

	/**
	 * Number of packages requested in CookByTheBook, set to 0 in other modes. Used to inform heuristics such as
	 * number of cookworkers.
	 */
	int32 InitialRequestCount = 0;

	// These structs are TUniquePtr rather than inline members so we can keep their headers private.
	// See class header comments for their purpose.
	TUniquePtr<UE::Cook::FPackageTracker> PackageTracker;
	TUniquePtr<UE::Cook::FPackageDatas> PackageDatas;
	TUniquePtr<UE::Cook::IWorkerRequests> WorkerRequests;
	TUniquePtr<UE::Cook::FBuildDefinitions> BuildDefinitions;
	TUniquePtr<UE::Cook::FCookDirector> CookDirector;
	TUniquePtr<UE::Cook::FCookWorkerClient> CookWorkerClient;
	TUniquePtr<FLayeredCookArtifactReader> AllContextArtifactReader;
	TSharedPtr<FLooseFilesCookArtifactReader> SharedLooseFilesCookArtifactReader;
	TUniquePtr<UE::Cook::FCookGCDiagnosticContext> GCDiagnosticContext;
	TUniquePtr<UE::Cook::ILogHandler> LogHandler;
	// These structs are TUniquePtr both for implementation hiding and because they are only sometimes needed.
	TUniquePtr<UE::Cook::FIncrementallyModifiedDiagnostics> IncrementallyModifiedDiagnostics;

	TArray<UE::Cook::FCookSavePackageContext*> SavePackageContexts;
	/**
	 * Objects that were collected during the single-threaded PreGarbageCollect callback and that should be reported
	 * as referenced in CookerAddReferencedObjects.
	 */
	TArray<TObjectPtr<UObject>> GCKeepObjects;
	/** Used during garbagecolletion: a flat array of all the elements in UPackage::SoftGCPackageToObjectList arrayviews. */
	TArray<UObject*> SoftGCPackageToObjectListBuffer;
	/** Packages that were expected to be freed by the last Soft GC and we expect not to load again. */
	TSet<FName> ExpectedFreedPackageNames;

	UE::Cook::FPackageData* SavingPackageData = nullptr;
	/** Helper struct for running cooking in diagnostic modes */
	TUniquePtr<FDiffModeCookServerUtils> DiffModeHelper;

	TRefCountPtr<UE::Cook::IMPCollector> ConfigCollector;

	/** Override for the platform-dependent base device profile used for cooking, parsed from the commandline (-DeviceProfile=) */
	FName OverrideDeviceProfileName;

	/** Override for Cook.CVarControl, parsed from the commandline (-CookCVarControl=) */
	int32 OverrideCookCVarControl = -1;

	TArray<TRefCountPtr<UE::Cook::ICookArtifact>> CookArtifacts;
	UE::Cook::FGlobalCookArtifact* GlobalArtifact = nullptr;
	UE::Cook::FValidationCookArtifact* ValidationArtifact = nullptr;

	/**
	 * Heap of Pollables to tick, ordered by NextTimeSeconds.
	 * Only accessible by PumpPollables when bPollablesInTick==true, can be accessed outside of PollablesLock.
	 * or elsewhere when bPollablesInTick==false, must be accessed inside PollablesLock.
	 */
	TArray<FPollableQueueKey> Pollables;
	/**
	 * List of pollables that were triggered during PumpPollables and need to be updated when the Pump is done.
	 * Accessible only inside PollablesLock.
	 */
	TArray<FPollableQueueKey> PollablesDeferredTriggers;
	/** Together with bPollablesInTick, provides a lock around Pollables. */
	FCriticalSection PollablesLock;
	TRefCountPtr<FPollable> RecompileRequestsPollable;
	TRefCountPtr<FPollable> QueuedCancelPollable;
	TRefCountPtr<FPollable> DirectorPollable;
	double PollNextTimeSeconds = 0.;
	double PollNextTimeIdleSeconds = 0.;
	double IdleStatusStartTime = 0.0;
	double IdleStatusLastReportTime = 0.0;
	uint32 CookedPackageCountSinceLastGC = 0;
	float WaitForAsyncSleepSeconds = 0.0f;
	float DisplayUpdatePeriodSeconds = 0.0f;
	int32 PhaseTransitionFence = -1;
	EIdleStatus IdleStatus = EIdleStatus::Done;
	TUniquePtr<UE::Cook::FODSCClientData> ODSCClientData;

	TUniquePtr<UE::Cook::FStallDetector> StallDetector;

	friend FAssetRegistryGenerator;
	friend FIncrementalValidatePackageWriter;
	friend UE::Cook::FAssetRegistryMPCollector;
	friend UE::Cook::FBeginCookConfigSettings;
	friend UE::Cook::FCookDirector;
	friend UE::Cook::FCookGCDiagnosticContext;
	friend UE::Cook::FCookGenerationInfo;
	friend UE::Cook::FCookWorkerClient;
	friend UE::Cook::FCookWorkerServer;
	friend UE::Cook::FDiagnostics;
	friend UE::Cook::FGenerationHelper;
	friend UE::Cook::FGlobalCookArtifact;
	friend UE::Cook::FInitializeConfigSettings;
	friend UE::Cook::FLogHandler;
	friend UE::Cook::FPackageData;
	friend UE::Cook::FPackageDatas;
	friend UE::Cook::FPackagePreloader;
	friend UE::Cook::FPackageTracker;
	friend UE::Cook::FPackageWriterMPCollector;
	friend UE::Cook::FPendingCookedPlatformData;
	friend UE::Cook::FPlatformManager;
	friend UE::Cook::FRequestCluster;
	friend UE::Cook::FSaveCookedPackageContext;
	friend UE::Cook::FScopeFindCookReferences;
	friend UE::Cook::FShaderLibraryCookArtifact;
	friend UE::Cook::FSoftGCHistory;
	friend UE::Cook::FWorkerRequestsLocal;
	friend UE::Cook::FWorkerRequestsRemote;

	friend void UE::Cook::CalculateGlobalDependenciesHash(const ITargetPlatform* Platform, const UCookOnTheFlyServer& COTFS);
};

namespace UE::Cook
{

/**
 * Public function for systems that track dependencies for the cooker and need to disable the tracking as early as
 * possible for performance when commandline or config settings disable CookIncremental dependency tracking. If this
 * function returns false, systems can turn off dependency tracking or reporting to the cooker.
 */
UNREALED_API bool IsDependencyTrackingEnabled();

}
