// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"
#include "AssetGuideline.generated.h"

// Helper struct for setting string console ini values.
USTRUCT()
struct FIniStringValue
{
	GENERATED_BODY()
		
	/** From .ini. Ex: /Script/Engine.RendererSettings */
	UPROPERTY(EditAnywhere, Category = General)
	FString Section;

	/** From .ini. Ex: r.GPUSkin.Support16BitBoneIndex */
	UPROPERTY(EditAnywhere, Category = General)
	FString Key;

	/** From .ini. Ex: True */
	UPROPERTY(EditAnywhere, Category = General)
	FString Value;

	/** From .ini, relative to {PROJECT}. Ex: /Config/DefaultEngine.ini */
	UPROPERTY(EditAnywhere, Category = General)
	FString Filename;

	/** 
	 * Ini branch that Filename belongs to. Ex: if Filename is "/Config/DefaultEngine.ini", Branch should be "Engine".
	 *
	 * If this isn't set, the system will attempt to detect it based on Filename.
	 */
	UPROPERTY(EditAnywhere, Category = General)
	FString Branch;
};

/**
* User data that can be attached to assets to check on load for guidlelines (plugins, project settings, etc).
*
* This class intentionally does not accept FText arguments. The project using your bundled asset would need to have
* your localization tables, and we currently do not support text table referencing.
*/
UCLASS(Blueprintable, MinimalAPI)
class UAssetGuideline : public UAssetUserData
{
	GENERATED_BODY()

public:

#if WITH_EDITORONLY_DATA
	/** Plugins to check for on load */
	UPROPERTY(EditAnywhere, Category = General)
	TArray<FString> Plugins;

	/** Project settings to check for on load. Look at your .ini's to populate this. */
	UPROPERTY(EditAnywhere, Category = General, Meta = (TitleProperty = "Key"))
	TArray<FIniStringValue> ProjectSettings;

	/** Name of this guideline, we will only check once per unique guideline name. */
	UPROPERTY(EditAnywhere, Category = General)
	FName GuidelineName;

private:

	TWeakPtr<class SNotificationItem> NotificationPtr;

	static UNREALED_API bool bAssetGuidelinesEnabled;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
public:
	/** Begin UObject interface */
	UNREALED_API virtual bool IsPostLoadThreadSafe() const override;
	UNREALED_API virtual void PostLoad() override;
	UNREALED_API virtual void BeginDestroy() override;
	/** End UObject interface */

	static void EnableAssetGuidelines(bool isEnabled) { bAssetGuidelinesEnabled = isEnabled; }
	static bool AreAssetGuidelinesEnabled() { return bAssetGuidelinesEnabled; }

private:
	[[nodiscard]] static bool TryDetectIniBranchFromFilename(const FString& Filename, FString& OutIniBranch);

	void EnableMissingGuidelines(TArray<FString> IncorrectPlugins, TArray<FIniStringValue> IncorrectProjectSettings);
	void DismissNotifications();
	void RemoveAssetGuideline();
#endif // WITH_EDITOR
};
