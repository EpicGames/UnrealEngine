// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"

class FText;
struct FAssetData;

/** Used in filtering allowed references between assets. Implement a subclass of this and return it in OnMakeAssetReferenceFilter */
class IAssetReferenceFilter
{
public:
	virtual ~IAssetReferenceFilter() = default;
	/** Filter function to pass/fail an asset. Called in some situations that are performance-sensitive so is expected to be fast. */
	virtual bool PassesFilter(const FAssetData& AssetData, FText* OutOptionalFailureReason = nullptr) const = 0;

	/** Return true if the given asset downgrades any illegal reference errors to warnings */
	virtual bool DoesAssetDowngradeReferenceErrorsToWarnings(const FAssetData& AssetData) const
	{
		return false;
	}

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnIsCrossPluginReferenceAllowed, const FAssetData& /*ReferencingAssetData*/, const FAssetData& /*ReferencedAssetData*/);

	/** Used to register a custom delegate implementation that returns if a plugin asset can reference another plugin asset. */
	static FOnIsCrossPluginReferenceAllowed& OnIsCrossPluginReferenceAllowed() { return OnIsCrossPluginReferenceAllowedDelegate; }
	
protected:
	UNREALED_API static bool IsCrossPluginReferenceAllowed(const FAssetData& ReferencingAssetData, const FAssetData& ReferencedAssetData);

private:
	UNREALED_API static FOnIsCrossPluginReferenceAllowed OnIsCrossPluginReferenceAllowedDelegate;
};
