// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Internationalization/Text.h"

namespace UE::DataValidation
{
	struct IFixer;
}

/** Used in fixing invalid references between assets. Implement a subclass of this and return it in OnMakeAssetReferenceFixer */
class IAssetReferenceFixer
{
public:
	virtual ~IAssetReferenceFixer() = default;

	/** Create a reference fixer for the given an asset, or null if no automatic fix-up can be performed for the given asset. */
	virtual TSharedPtr<UE::DataValidation::IFixer> CreateFixer(const FAssetData& AssetData) const = 0;

	/** Get the label to use as the fixer for the given asset. */
	FText GetFixerLabel(const FAssetData& AssetData) const
	{
		FText FixerLabel = GetFixerLabelImpl(AssetData);
		if (FixerLabel.IsEmpty())
		{
			FixerLabel = FText::FromName(AssetData.AssetName);
		}
		return FixerLabel;
	}

protected:
	virtual FText GetFixerLabelImpl(const FAssetData& AssetData) const
	{
		return FText();
	}
};
