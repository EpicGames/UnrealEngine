// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericApplication.h"
#include "Widgets/SWindow.h"
#include "TimerManager.h"
#include "UObject/UObjectAnnotation.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "Misc/CompilationResult.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PlayInEditorDataTypes.h"
#include "EditorSubsystem.h"
#include "Subsystems/SubsystemCollection.h"
#include "UnrealEngine.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniqueObj.h"
#include "Editor/AssetReferenceFilter.h"
#include "RHIShaderPlatform.h"
#include "EditorEngine.generated.h"

#define UE_API UNREALED_API

class APlayerStart;
class Error;
class FEditorViewportClient;
class FEditorWorldManager;
class FLevelEditorViewportClient;
class FMessageLog;
class FOutputLogErrorsToMessageLogProxy;
class FPoly;
class FSceneViewport;
class FSceneViewStateInterface;
class FViewport;
class IEngineLoop;
class ILauncherWorker;
class ILayers;
class IAssetReferenceFixer;
class IAssetViewport;
class ITargetPlatform;
class SViewport;
class UActorFactory;
class UAnimSequence;
class UAudioComponent;
class UBrushBuilder;
class UFoliageType;
class UFbxImportUI;
class UGameViewportClient;
class ULevelEditorPlaySettings;
class ULocalPlayer;
class UNetDriver;
class UPrimitiveComponent;
class USkeleton;
class USoundBase;
class USoundNode;
class UTextureRenderTarget2D;
class UTransactor;
class FTransactionObjectEvent;
enum ERotationGridMode : int;
struct FEdge;
struct FTransactionContext;
struct FEditorTransactionDeltaContext;
struct FTypedElementHandle;
struct FAnalyticsEventAttribute;
struct FAssetCompileData;
class UEditorWorldExtensionManager;
class ITargetDevice;
class ULevelEditorDragDropHandler;
class UTypedElementSelectionSet;
class IProjectExternalContentInterface;

//
// Things to set in mapSetBrush.
//
UENUM()
enum EMapSetBrushFlags : int
{
	/** Set brush color. */
	MSB_BrushColor	= 1,
	/** Set group. */
	MSB_Group		= 2,
	/** Set poly flags. */
	MSB_PolyFlags	= 4,
	/** Set CSG operation. */
	MSB_BrushType	= 8,
};

UENUM()
enum EPasteTo : int
{
	PT_OriginalLocation	= 0,
	PT_Here				= 1,
	PT_WorldOrigin		= 2
};

USTRUCT()
struct FCopySelectedInfo
{
	GENERATED_USTRUCT_BODY()

	/** Do not cache this info, it is only valid after a call to CanCopySelectedActorsToClipboard has been made, and becomes redundant
	when the current selection changes. Used to determine whether a copy can be performed based on the current selection state */
	FCopySelectedInfo()
		: bHasSelectedActors( false )
		, bAllActorsInSameLevel( true )
		, LevelAllActorsAreIn( NULL )
		, bHasSelectedSurfaces( false )
		, LevelWithSelectedSurface( NULL )
	{}


	/** Does the current selection contain actors */
	bool bHasSelectedActors;		

	/** If we have selected actors, are they within the same level */
	bool bAllActorsInSameLevel;

	/** If they are in the same level, what level is it */
	ULevel* LevelAllActorsAreIn;


	/** Does the current selection contain surfaces */
	bool bHasSelectedSurfaces;

	/** If we have selected surfaces, what level is it */
	ULevel* LevelWithSelectedSurface;


	/** Can a quick copy be formed based on the selection information */
	bool CanPerformQuickCopy() const
	{
		// If there are selected actors and BSP surfaces AND all selected actors 
		// and surfaces are in the same level, we can do a quick copy.
		bool bCanPerformQuickCopy = false;
		if( LevelAllActorsAreIn && LevelWithSelectedSurface )
		{
			bCanPerformQuickCopy = ( LevelWithSelectedSurface == LevelAllActorsAreIn );
		}
		// Else, if either we have only selected actors all in one level OR we have 
		// only selected surfaces all in one level, then we can perform a quick copy. 
		else
		{
			bCanPerformQuickCopy = (LevelWithSelectedSurface != NULL && !bHasSelectedActors) || (LevelAllActorsAreIn != NULL && !bHasSelectedSurfaces);
		}
		return bCanPerformQuickCopy;
	}
};

/** A cache of actor labels */
struct FCachedActorLabels
{
	/** Default constructor - does not populate the array */
	UNREALED_API FCachedActorLabels();

	/** Constructor that populates the set of actor names */
	UNREALED_API explicit FCachedActorLabels(UWorld* World, const TSet<AActor*>& IgnoredActors = TSet<AActor*>());

	/** Populate the set of actor names */
	UNREALED_API void Populate(UWorld* World, const TSet<AActor*>& IgnoredActors = TSet<AActor*>());

	/** Add a new label to this set */
	FORCEINLINE void Add(const FString& InLabel)
	{
		ActorLabels.Add(InLabel);
	}

	/** Remove a label from this set */
	FORCEINLINE void Remove(const FString& InLabel)
	{
		ActorLabels.Remove(InLabel);
	}

	/** Check if the specified label exists */
	FORCEINLINE bool Contains(const FString& InLabel) const
	{
		return ActorLabels.Contains(InLabel);
	}

private:
	TSet<FString> ActorLabels;
};

/**
 * Represents the current selection state of a level (its selected actors and components) from a given point in a time, in a way that can be safely restored later even if the level is reloaded
 */
USTRUCT()
struct FSelectionStateOfLevel
{
	GENERATED_BODY()

	/** Path names of all the selected actors */
	UPROPERTY()
	TArray<FString> SelectedActors;

	/** Path names of all the selected components */
	UPROPERTY()
	TArray<FString> SelectedComponents;
};

struct FPreviewPlatformInfo
{
	FPreviewPlatformInfo()
	:	PreviewFeatureLevel(ERHIFeatureLevel::Num)
	,	ShaderPlatform(EShaderPlatform::SP_NumPlatforms)
	,	PreviewPlatformName(NAME_None)
	,	PreviewShaderFormatName(NAME_None)
	,	bPreviewShaderPlatformActive(false)
	,	PreviewShaderPlatformName(NAME_None)
	,	PreviewShaderPlatformFriendlyName()
	,	ConstrainedAspectRatio(0.0f)
	,	SafeZones(FVector4f(0))
	{}

	FPreviewPlatformInfo(ERHIFeatureLevel::Type InFeatureLevel, 
		EShaderPlatform InShaderPlatform = EShaderPlatform::SP_NumPlatforms, 
		FName InPreviewPlatformName = NAME_None, 
		FName InPreviewShaderFormatName = NAME_None, 
		FName InDeviceProfileName = NAME_None, 
		bool InbPreviewShaderPlatformActive = false, 
		FName InShaderPlatformName = NAME_None, 
		FText InPreviewShaderPlatformFriendlyName = FText(), 
		float InConstrainedAspectRatio = 0.0f,
		FVector4f InSafeZones = FVector4f::Zero()
	)
	:	PreviewFeatureLevel(InFeatureLevel)
	,	ShaderPlatform(InShaderPlatform)
	,	PreviewPlatformName(InPreviewPlatformName)
	,	PreviewShaderFormatName(InPreviewShaderFormatName)
	,	DeviceProfileName(InDeviceProfileName)
	,	bPreviewShaderPlatformActive(InbPreviewShaderPlatformActive)
	,	PreviewShaderPlatformName(InShaderPlatformName)
	,	PreviewShaderPlatformFriendlyName(InPreviewShaderPlatformFriendlyName)
	,	ConstrainedAspectRatio(InConstrainedAspectRatio)
	,	SafeZones(InSafeZones)
	{
		InternalSetFriendlyName();
	}

	/** The feature level we should use when loading or creating a new world */
	ERHIFeatureLevel::Type PreviewFeatureLevel;

	/** The ShaderPlatform to be used when in preview */
	EShaderPlatform ShaderPlatform;

	/** The the platform to preview, or NAME_None if there is no preview platform */
	FName PreviewPlatformName;

	/** The shader Format to preview, or NAME_None if there is no shader preview format */
	FName PreviewShaderFormatName;

	/** The device profile to preview. */
	FName DeviceProfileName;

	/** Is shader platform preview currently active */
	bool bPreviewShaderPlatformActive;

	/** Preview Shader Platform Name */
	FName PreviewShaderPlatformName;

	/** Checks if two FPreviewPlatformInfos are for the same preview platform. Note, this does NOT compare the bPreviewShaderPlatformActive flag */
	bool Matches(const FPreviewPlatformInfo& Other) const
	{
		return PreviewFeatureLevel == Other.PreviewFeatureLevel && ShaderPlatform == Other.ShaderPlatform && PreviewPlatformName == Other.PreviewPlatformName && PreviewShaderFormatName == Other.PreviewShaderFormatName && DeviceProfileName == Other.DeviceProfileName && PreviewShaderPlatformName == Other.PreviewShaderPlatformName;
	}

	/** Return platform name like "Android", or NAME_None if none is set or the preview feature level is not active */
	FName GetEffectivePreviewPlatformName() const
	{
		return bPreviewShaderPlatformActive ? PreviewPlatformName : NAME_None;
	}

	/** Return the Constrained Aspect Ratio of the preview platform */
	float GetConstrainedAspectRatio()
	{
		return ConstrainedAspectRatio;
	}

	/** Return the Safe Zones of the preview platform */
	FVector4f GetSafeZones()
	{
		return SafeZones;
	}

	/** returns the preview feature level if active, or GMaxRHIFeatureLevel otherwise */
	UNREALED_API ERHIFeatureLevel::Type GetEffectivePreviewFeatureLevel() const;
	
	/** Returns the Friendly Name of the Preview*/
	FText GetFriendlyName() const { return PreviewShaderPlatformFriendlyName; }
	
	/** Returns the Shader Platform of the Preview*/
	UNREALED_API EShaderPlatform GetShaderPlatform() const;
private:
	/**Cache Preview ShaderPlatform Name*/
	FText PreviewShaderPlatformFriendlyName;
	
	/** Constrained Aspect Ratio of the preview platform */
	float ConstrainedAspectRatio;
	
	/** Safe Zones of the preview platform */
	FVector4f SafeZones;

	UNREALED_API void InternalSetFriendlyName();
};

enum class EAssetReferenceFilterProperties : uint8
{
	None = 0,
	EditorOnly = 1,
	Soft = 2
};
ENUM_CLASS_FLAGS(EAssetReferenceFilterProperties);

struct FAssetReferenceFilterReferencerInfo
{
	FAssetData Data;
	EAssetReferenceFilterProperties Properties;

	FAssetReferenceFilterReferencerInfo() : Properties(EAssetReferenceFilterProperties::None) {}
	FAssetReferenceFilterReferencerInfo(const FAssetData& InData) : Data(InData), Properties(EAssetReferenceFilterProperties::None) {}
	FAssetReferenceFilterReferencerInfo(const FAssetData& InData, EAssetReferenceFilterProperties InProperties) : Data(InData), Properties(InProperties) {}
	bool operator==(const FAssetReferenceFilterReferencerInfo& Other) const
	{
		return Data == Other.Data && Properties == Other.Properties;
	}

	bool operator!=(const FAssetReferenceFilterReferencerInfo& Other) const
	{
		return !(*this == Other);
	}
};

enum class EAssetReferenceFilterRole : uint8
{
	/** This filter has no specific role */
	None = 0,
	/** This filter is being used by illegal reference validation */
	Validation = 1<<0,
};
ENUM_CLASS_FLAGS(EAssetReferenceFilterRole);

/** Struct used in filtering allowed references between assets. Passes context about the referencers to game-level filters */
struct FAssetReferenceFilterContext
{
	UNREALED_API void AddReferencingAsset(const UObject* InReferencingAsset, FAssetData::ECreationFlags InCreationFlags = FAssetData::ECreationFlags::None, EAssetReferenceFilterProperties InProperties = EAssetReferenceFilterProperties::None);
	UNREALED_API void AddReferencingAsset(const FAssetData& InReferencingAsset, EAssetReferenceFilterProperties InProperties = EAssetReferenceFilterProperties::None);
	UNREALED_API void AddReferencingAssets(TConstArrayView<UObject*> InReferencingAssets, FAssetData::ECreationFlags InCreationFlags = FAssetData::ECreationFlags::None, EAssetReferenceFilterProperties InProperties = EAssetReferenceFilterProperties::None);
	UNREALED_API void AddReferencingAssets(const TArray<FAssetData>& InReferencingAssets, EAssetReferenceFilterProperties InProperties = EAssetReferenceFilterProperties::None);
	UNREALED_API void AddReferencingAssetsFromPropertyHandle(const TSharedPtr<class IPropertyHandle>& PropertyHandle, FAssetData::ECreationFlags InCreationFlags = FAssetData::ECreationFlags::None);

	const TArray<FAssetReferenceFilterReferencerInfo>& GetReferencingAssets() const { return ReferencingAssetInfo; }

	UE_DEPRECATED(5.5, "ReferencingAssets is deprecated. Use the AddReferencingAsset functions to add referencers and GetReferencingAssets to get them.")
	TArray<FAssetData> ReferencingAssets;

	void AddRole(const EAssetReferenceFilterRole InRole)
	{
		EnumAddFlags(Role, InRole);
	}

	void RemoveRole(const EAssetReferenceFilterRole InRole)
	{
		EnumRemoveFlags(Role, InRole);
	}

	bool HasRole(const EAssetReferenceFilterRole InRole) const
	{
		return EnumHasAnyFlags(Role, InRole);
	}

	EAssetReferenceFilterRole GetRole() const
	{
		return Role;
	}

private:
	TArray<FAssetReferenceFilterReferencerInfo> ReferencingAssetInfo;
	EAssetReferenceFilterRole Role = EAssetReferenceFilterRole::None;
};

/**
 * Engine that drives the Editor.
 * Separate from UGameEngine because it may have much different functionality than desired for an instance of a game itself.
 */

UCLASS(config=Engine, transient, MinimalAPI)
class UEditorEngine : public UEngine
{
public:
	GENERATED_BODY()

public:
	UNREALED_API UEditorEngine(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// Objects.
	UPROPERTY()
	TObjectPtr<class UModel> TempModel;

	UPROPERTY()
	TObjectPtr<class UModel> ConversionTempModel;

	UPROPERTY()
	TObjectPtr<class UTransactor> Trans;

	// Textures.
	UPROPERTY()
	TObjectPtr<class UTexture2D> Bad;

	// Font used by Canvas-based editors
	UPROPERTY()
	TObjectPtr<class UFont> EditorFont;

	// Audio
	UPROPERTY(transient)
	TObjectPtr<class USoundCue> PreviewSoundCue;

	UPROPERTY(transient)
	TObjectPtr<class UAudioComponent> PreviewAudioComponent;

	// Used in UnrealEd for showing materials
	UPROPERTY()
	TObjectPtr<class UStaticMesh> EditorCube;

	UPROPERTY()
	TObjectPtr<class UStaticMesh> EditorSphere;

	UPROPERTY()
	TObjectPtr<class UStaticMesh> EditorPlane;

	UPROPERTY()
	TObjectPtr<class UStaticMesh> EditorCylinder;

	// Toggles.
	UPROPERTY()
	uint32 bFastRebuild:1;

	UPROPERTY()
	uint32 IsImportingT3D:1;

	// Other variables.
	UPROPERTY()
	uint32 ClickFlags;

	UPROPERTY()
	TObjectPtr<class UPackage> ParentContext;

	UPROPERTY()
	FVector UnsnappedClickLocation;

	UPROPERTY()
	FVector ClickLocation;

	UPROPERTY()
	FPlane ClickPlane;

	UPROPERTY()
	FVector MouseMovement;

	// Setting for the detail mode to show in the editor viewports
	UPROPERTY()
	TEnumAsByte<enum EDetailMode> DetailMode;

	// Advanced.
	UPROPERTY(EditAnywhere, config, Category=Advanced)
	uint32 UseSizingBox:1;

	UPROPERTY(EditAnywhere, config, Category=Advanced)
	uint32 UseAxisIndicator:1;

	UPROPERTY(EditAnywhere, config, Category=Advanced)
	FString GameCommandLine;

	/** If true, show translucent marker polygons on the builder brush and volumes. */
	UE_DEPRECATED(5.6, "bShowBrushMarkerPoly has been moved to ULevelEditorViewportSettings::bShowBrushMarkerPoly. Setting can now be found under Editor Preferences->Level Editor->Viewport->Look And Feel")
	UPROPERTY(config, meta=(DeprecatedProperty, DeprecationMessage = "bShowBrushMarkerPoly can now be found under Editor Preferences->Level Editor->Viewport->Look And Feel"))
	uint32 bShowBrushMarkerPolys:1;

	/** If true, socket snapping is enabled in the main level viewports. */
	UPROPERTY(EditAnywhere, config, Category=Advanced)
	uint32 bEnableSocketSnapping:1;

	/** If true, same type views will be camera tied, and ortho views will use perspective view for LOD parenting */
	UE_DEPRECATED(5.6, "bEnableLODLocking is now handled by ULevelEditorViewportSettings::bUseLODViewLocking. Setting can now be found under Editor Preferences->Level Editor->Viewport->Look And Feel")
	UPROPERTY(config, meta=(DeprecatedProperty, DeprecationMessage = "bEnableLODLocking can now be found under Editor Preferences->Level Editor->Viewport->Look And Feel"))
	uint32 bEnableLODLocking:1;

	UPROPERTY(config)
	FString HeightMapExportClassName;

	/** Array of actor factories created at editor startup and used by context menu etc. */
	UPROPERTY()
	TArray<TObjectPtr<class UActorFactory>> ActorFactories;

	/** The name of the file currently being opened in the editor. "" if no file is being opened. */
	UPROPERTY()
	FString UserOpenedFile;

	///////////////////////////////
	// "Play From Here" properties
	
	/** Additional per-user/per-game options set in the .ini file. Should be in the form "?option1=X?option2?option3=Y"					*/
	UPROPERTY(EditAnywhere, config, Category=Advanced)
	FString InEditorGameURLOptions;

	/** A pointer to a UWorld that is the duplicated/saved-loaded to be played in with "Play From Here" 								*/
	UPROPERTY()
	TObjectPtr<class UWorld> PlayWorld;



	/** Has a request to toggle between PIE and SIE been made? */
	UPROPERTY()
	uint32 bIsToggleBetweenPIEandSIEQueued:1;

	/** Allows multiple PIE worlds under a single instance. If false, you can only do multiple UE processes for pie networking */
	UPROPERTY(globalconfig)
	uint32 bAllowMultiplePIEWorlds:1;

	/** True if there is a pending end play map queued */
	UPROPERTY()
	uint32 bRequestEndPlayMapQueued:1;

	/** True if we should ignore noting any changes to selection on undo/redo */
	UPROPERTY()
	uint32 bIgnoreSelectionChange:1;

	/** True if we should suspend notifying clients post undo/redo */
	UPROPERTY()
	uint32 bSuspendBroadcastPostUndoRedo:1;

	/** True if we should not display notifications about undo/redo */
	UPROPERTY()
	uint32 bSquelchTransactionNotification:1;

	/** True if we should force a selection change notification during an undo/redo */
	UPROPERTY()
	uint32 bNotifyUndoRedoSelectionChange:1;

	/** The PlayerStart class used when spawning the player at the current camera location. */
	UPROPERTY()
	TSubclassOf<class ANavigationObjectBase>  PlayFromHerePlayerStartClass;

	/** When Simulating In Editor, a pointer to the original (non-simulating) editor world */
	UPROPERTY()
	TObjectPtr<class UWorld> EditorWorld;

	/** When Simulating In Editor, an array of all actors that were selected when it began*/
	UPROPERTY()
	TArray<TWeakObjectPtr<class AActor> > ActorsThatWereSelected;

	/** Where did the person want to play? Where to play the game - -1 means in editor, 0 or more is an index into the GConsoleSupportContainer	*/
	UPROPERTY()
	int32 PlayWorldDestination;

	/** The current play world destination (I.E console).  -1 means no current play world destination, 0 or more is an index into the GConsoleSupportContainer	*/
	UPROPERTY()
	int32 CurrentPlayWorldDestination;

	/** Mobile preview settings for what orientation to default to */
	UPROPERTY(config)
	uint32 bMobilePreviewPortrait:1;

	/** Currently targeted device for mobile previewer. */
	UPROPERTY(config)
	int32 BuildPlayDevice;


	/** Maps world contexts to their slate data */
	TMap<FName, FSlatePlayInEditorInfo>	SlatePlayInEditorMap;

	/** Play world url string edited by a user. */
	UPROPERTY()
	FString UserEditedPlayWorldURL;

	/** Temporary render target that can be used by the editor. */
	UPROPERTY(transient)
	TObjectPtr<class UTextureRenderTarget2D> ScratchRenderTarget2048;

	UPROPERTY(transient)
	TObjectPtr<class UTextureRenderTarget2D> ScratchRenderTarget1024;

	UPROPERTY(transient)
	TObjectPtr<class UTextureRenderTarget2D> ScratchRenderTarget512;

	UPROPERTY(transient)
	TObjectPtr<class UTextureRenderTarget2D> ScratchRenderTarget256;

	/** A mesh component used to preview in editor without spawning a static mesh actor. */
	UPROPERTY(transient)
	TObjectPtr<class UStaticMeshComponent> PreviewMeshComp;

	/** The index of the mesh to use from the list of preview meshes. */
	UPROPERTY()
	int32 PreviewMeshIndex;

	/** When true, the preview mesh mode is activated. */
	UPROPERTY()
	uint32 bShowPreviewMesh:1;

	/** If "Camera Align" emitter handling uses a custom zoom or not */
	UPROPERTY(config)
	uint32 bCustomCameraAlignEmitter:1;

	/** The distance to place the camera from an emitter actor when custom zooming is enabled */
	UPROPERTY(config)
	float CustomCameraAlignEmitterDistance;

	/** If true, then draw sockets when socket snapping is enabled in 'g' mode */
	UPROPERTY(config)
	uint32 bDrawSocketsInGMode:1;

	/** If true, then draw particle debug helpers in editor viewports */
	UPROPERTY(transient)
	uint32 bDrawParticleHelpers:1;

	/** Brush builders that have been created in the editor */
	UPROPERTY(transient)
	TArray<TObjectPtr<class UBrushBuilder>> BrushBuilders;	

	UE_DEPRECATED(5.8, "This is replaced by the (protected) bCheckForMovementLockedItems.")
	bool bCheckForLockActors;

	UE_DEPRECATED(5.8, "This is replaced by the (private) bHasSelectedMovementLockedItems.")
	bool bHasLockedActors;

protected:
	/** Whether or not to recheck the current selection for movement locked items the next time HasSelectedMovementLockedItems is called */
	bool bCheckForMovementLockedItems = true;
private:
	/** Cached state of whether or not we have locked items in the selection*/
	bool bHasSelectedMovementLockedItems = false;

	// Don't use these handles directly. Prefer the getters in case the queries are moved
	//  out to a shared location.
	mutable UE::Editor::DataStorage::QueryHandle SelectedMovementLockedItemsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	mutable UE::Editor::DataStorage::QueryHandle SelectedItemsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	
	/** Reads FTypedElementWorldColumn for items that have FTypedElementSelectionColumn and FTypedElementLockTransformTag */
	UE::Editor::DataStorage::QueryHandle GetSelectedMovementLockedItemsQuery() const;
	
	/** Reads FTypedElementWorldColumn column for items that have FTypedElementSelectionColumn */
	UE::Editor::DataStorage::QueryHandle GetSelectedItemsQuery() const;

	void UnregisterTEDSQueries();

public:

	/** Whether or not to recheck the current actor selection for world settings actors the next time IsWorldSettingsSelected is called */
	bool bCheckForWorldSettingsActors;

	/** Cached state of whether or not we have a worldsettings actor in the selection */
	bool bIsWorldSettingsSelected;

	/** The feature level we should use when loading or creating a new world */
	ERHIFeatureLevel::Type DefaultWorldFeatureLevel;

	/** The feature level and platform we should use when loading or creating a new world */
	FPreviewPlatformInfo PreviewPlatform;

	/** A delegate that is called when the preview feature level changes. Primarily used to switch a viewport's feature level. */
	UE_DEPRECATED(5.8, "FPreviewFeatureLevelChanged is deprecated, use FPreviewShaderPlatformChanged instead")
	DECLARE_MULTICAST_DELEGATE_OneParam(FPreviewFeatureLevelChanged, ERHIFeatureLevel::Type);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPreviewFeatureLevelChanged PreviewFeatureLevelChanged;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** A delegate that is called when the preview platform changes. */
	UE_DEPRECATED(5.8, "FPreviewPlatformChanged is deprecated, use FPreviewShaderPlatformChanged instead")
	DECLARE_MULTICAST_DELEGATE(FPreviewPlatformChanged);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPreviewPlatformChanged PreviewPlatformChanged;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** A delegate that is called when the preview platform changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPreviewShaderPlatformChanged, EShaderPlatform);
	FPreviewShaderPlatformChanged PreviewShaderPlatformChanged;

	/** An array of delegates that can force disable throttling cpu usage if any of them return true. */
	DECLARE_DELEGATE_RetVal(bool, FShouldDisableCPUThrottling);
	TArray<FShouldDisableCPUThrottling> ShouldDisableCPUThrottlingDelegates;

	/** A delegate that is called when the bugitgo command is used. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPostBugItGoCalled, const FVector& Loc, const FRotator& Rot);
	FPostBugItGoCalled PostBugItGoCalled;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSoundBasePreview, TObjectPtr<USoundBase>)
	FOnSoundBasePreview OnSoundBasePreview;

	/** Whether or not the editor is currently compiling */
	bool bIsCompiling;

protected:
	/** Used to prevent reentrant calls to RebuildAlteredBSP(). */
	bool bIsRebuildingAlteredBSP;

private:

	/** Manager that holds all extensions paired with a world */
	UPROPERTY()
	TObjectPtr<UEditorWorldExtensionManager> EditorWorldExtensionsManager;

public:

	/** List of all viewport clients */
	const TArray<class FEditorViewportClient*>& GetAllViewportClients() { return AllViewportClients; }
	const TArray<class FEditorViewportClient*>& GetAllViewportClients() const { return AllViewportClients; }

	/** Called when the viewport clients list changed */
	DECLARE_EVENT(UEditorEngine, FViewportClientListChangedEvent);
	FViewportClientListChangedEvent& OnViewportClientListChanged() { return ViewportClientListChangedEvent; }

	/**
	 * Add a viewport client.
	 * @return Index to the new item
	 */
	UNREALED_API int32 AddViewportClients(FEditorViewportClient* ViewportClient);

	/** Remove a viewport client */
	UNREALED_API void RemoveViewportClients(FEditorViewportClient* ViewportClient);

	/** List of level editor viewport clients for level specific actions */
	const TArray<class FLevelEditorViewportClient*>& GetLevelViewportClients() { return LevelViewportClients; }
	const TArray<class FLevelEditorViewportClient*>& GetLevelViewportClients() const { return LevelViewportClients; }

	/**
	 * Add a viewport client.
	 * @return Index to the new item
	 */
	UNREALED_API int32 AddLevelViewportClients(FLevelEditorViewportClient* ViewportClient);

	/** Remove a level editor viewport client */
	UNREALED_API void RemoveLevelViewportClients(FLevelEditorViewportClient* ViewportClient);

	/** Called when the level editor viewport clients list changed */
	FViewportClientListChangedEvent& OnLevelViewportClientListChanged() { return LevelViewportClientListChangedEvent; }

	/** Annotation to track which PIE/SIE (PlayWorld) UObjects have counterparts in the EditorWorld **/
	class FUObjectAnnotationSparseBool ObjectsThatExistInEditorWorld;

	/** Called prior to a Blueprint compile */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FBlueprintPreCompileEvent, UBlueprint* );
	FBlueprintPreCompileEvent& OnBlueprintPreCompile() { return BlueprintPreCompileEvent; }

	/** Broadcasts that a Blueprint is about to be compiled */
	void BroadcastBlueprintPreCompile(UBlueprint* BlueprintToCompile) { BlueprintPreCompileEvent.Broadcast(BlueprintToCompile); }

	/** Called when a Blueprint compile is completed. */
	DECLARE_TS_MULTICAST_DELEGATE( FBlueprintCompiledEvent );
	FBlueprintCompiledEvent& OnBlueprintCompiled() { return BlueprintCompiledEvent; }

	/**	Broadcasts that a blueprint just finished compiling. THIS SHOULD NOT BE PUBLIC */
	void BroadcastBlueprintCompiled() { BlueprintCompiledEvent.Broadcast(); }

	/** Called by the blueprint compiler after a blueprint has been compiled and all instances replaced, but prior to garbage collection. */
	DECLARE_TS_MULTICAST_DELEGATE(FBlueprintReinstanced);
	FBlueprintReinstanced& OnBlueprintReinstanced() { return BlueprintReinstanced; }

	/**	Broadcasts that a blueprint just finished being reinstanced. THIS SHOULD NOT BE PUBLIC */
	void BroadcastBlueprintReinstanced() { BlueprintReinstanced.Broadcast(); }

	/** Called when a package with data-driven classes becomes loaded or unloaded */
	DECLARE_EVENT( UEditorEngine, FClassPackageLoadedOrUnloadedEvent );
	FClassPackageLoadedOrUnloadedEvent& OnClassPackageLoadedOrUnloaded() { return ClassPackageLoadedOrUnloadedEvent; }

	/**	Broadcasts that a class package was just loaded or unloaded. THIS SHOULD NOT BE PUBLIC */
	void BroadcastClassPackageLoadedOrUnloaded() { ClassPackageLoadedOrUnloadedEvent.Broadcast(); }

	/** Editor-only event triggered before an actor or component is moved, rotated or scaled by an editor system */
	DECLARE_EVENT_OneParam( UEditorEngine, FOnBeginTransformObject, UObject& );
	FOnBeginTransformObject& OnBeginObjectMovement() { return OnBeginObjectTransformEvent; }

	/** Editor-only event triggered after actor or component has moved, rotated or scaled by an editor system */
	DECLARE_EVENT_OneParam( UEditorEngine, FOnEndTransformObject, UObject& );
	FOnEndTransformObject& OnEndObjectMovement() { return OnEndObjectTransformEvent; }

	/** Editor-only event triggered after actors are moved, rotated or scaled by an editor system */
	DECLARE_EVENT_OneParam(UEditorEngine, FOnActorsMoved, TArray<AActor*>&);
	FOnActorsMoved& OnActorsMoved() { return OnActorsMovedEvent; }

	/** Editor-only event triggered before the camera viewed through the viewport is moved by an editor system */
	DECLARE_EVENT_OneParam( UEditorEngine, FOnBeginTransformCamera, UObject& );
	FOnBeginTransformCamera& OnBeginCameraMovement() { return OnBeginCameraTransformEvent; }

	/** Editor-only event triggered after the camera viewed through the viewport has been moved by an editor system */
	DECLARE_EVENT_OneParam( UEditorEngine, FOnEndTransformCamera, UObject& );
	FOnEndTransformCamera& OnEndCameraMovement() { return OnEndCameraTransformEvent; }

	/** Editor-only event triggered when a HLOD Actor is moved between clusters */
	DECLARE_EVENT_TwoParams(UEngine, FHLODActorMovedEvent, const AActor*, const AActor*);
	FHLODActorMovedEvent& OnHLODActorMoved() { return HLODActorMovedEvent; }

	/** Called by internal engine systems after a HLOD Actor is moved between clusters */
	void BroadcastHLODActorMoved(const AActor* InActor, const AActor* ParentActor) { HLODActorMovedEvent.Broadcast(InActor, ParentActor); }

	/** Editor-only event triggered when a HLOD Actor's mesh is build */
	DECLARE_EVENT_OneParam(UEngine, FHLODMeshBuildEvent, const class ALODActor*);
	FHLODMeshBuildEvent& OnHLODMeshBuild() { return HLODMeshBuildEvent; }

	/** Called by internal engine systems after a HLOD Actor's mesh is build */
	void BroadcastHLODMeshBuild(const class ALODActor* InActor) { HLODMeshBuildEvent.Broadcast(InActor); }

	/** Editor-only event triggered when a HLOD Actor is added to a cluster */
	DECLARE_EVENT_TwoParams(UEngine, FHLODActorAddedEvent, const AActor*, const AActor*);
	FHLODActorAddedEvent& OnHLODActorAdded() { return HLODActorAddedEvent; }

	/** Called by internal engine systems after a HLOD Actor is added to a cluster */
	void BroadcastHLODActorAdded(const AActor* InActor, const AActor* ParentActor) { HLODActorAddedEvent.Broadcast(InActor, ParentActor); }

	/** Editor-only event triggered when a HLOD Actor is marked dirty */
	DECLARE_EVENT(UEngine, FHLODTransitionScreenSizeChangedEvent);
	FHLODTransitionScreenSizeChangedEvent& OnHLODTransitionScreenSizeChanged() { return HLODTransitionScreenSizeChangedEvent; }

	/** Called by internal engine systems after a HLOD Actor is marked dirty */
	void BroadcastHLODTransitionScreenSizeChanged() { HLODTransitionScreenSizeChangedEvent.Broadcast(); }

	/** Editor-only event triggered when a HLOD level is added or removed */
	DECLARE_EVENT(UEngine, FHLODLevelsArrayChangedEvent);
	FHLODLevelsArrayChangedEvent& OnHLODLevelsArrayChanged() { return HLODLevelsArrayChangedEvent; }

	/** Called by internal engine systems after a HLOD Actor is marked dirty */
	void BroadcastHLODLevelsArrayChanged() { HLODLevelsArrayChangedEvent.Broadcast(); }

	DECLARE_EVENT_TwoParams(UEngine, FHLODActorRemovedFromClusterEvent, const AActor*, const AActor*);
	FHLODActorRemovedFromClusterEvent& OnHLODActorRemovedFromCluster() { return HLODActorRemovedFromClusterEvent; }
	   
	/** Called by internal engine systems after an Actor is removed from a cluster */
	void BroadcastHLODActorRemovedFromCluster(const AActor* InActor, const AActor* ParentActor) { HLODActorRemovedFromClusterEvent.Broadcast(InActor, ParentActor); }

	/** Called when the editor has been asked to perform an exec command on particle systems. */
	DECLARE_EVENT_OneParam(UEditorEngine, FExecParticleInvoked, const TCHAR*);
	FExecParticleInvoked& OnExecParticleInvoked() { return ExecParticleInvokedEvent; }

	/** Called to allow selection of unloaded actors */
	DECLARE_EVENT_OneParam(UEditorEngine, FSelectUnloadedActorsEvent, const TArray<FGuid>&);
	FSelectUnloadedActorsEvent& OnSelectUnloadedActorsEvent() { return SelectUnloadedActorsEvent; }

	void BroadcastSelectUnloadedActors(const TArray<FGuid>& ActorGuids) const { SelectUnloadedActorsEvent.Broadcast(ActorGuids); }

	/**
	 * Called before an actor or component is about to be translated, rotated, or scaled by the editor
	 *
	 * @param Object	The actor or component that will be moved
	 */
	void BroadcastBeginObjectMovement(UObject& Object) const { OnBeginObjectTransformEvent.Broadcast(Object); }

	/**
	 * Called when an actor or component has been translated, rotated, or scaled by the editor
	 *
	 * @param Object	The actor or component that moved
	 */
	void BroadcastEndObjectMovement(UObject& Object) const { OnEndObjectTransformEvent.Broadcast(Object); }

	/**
	 * Called when actors have been translated, rotated, or scaled by the editor
	 *
	 * @param Object	The actor or component that moved
	 */
	void BroadcastActorsMoved(TArray<AActor*>& Actors) const { OnActorsMovedEvent.Broadcast(Actors); }

	/**
	 * Called before the camera viewed through the viewport is moved by the editor
	 *
	 * @param Object	The camera that will be moved
	 */
	void BroadcastBeginCameraMovement(UObject& Object) const { OnBeginCameraTransformEvent.Broadcast(Object); }

	/**
	 * Called when the camera viewed through the viewport has been moved by the editor
	 *
	 * @param Object	The camera that moved
	 */
	void BroadcastEndCameraMovement(UObject& Object) const { OnEndCameraTransformEvent.Broadcast(Object); }

	/**	Broadcasts that an object has been reimported. THIS SHOULD NOT BE PUBLIC */
	UNREALED_API void BroadcastObjectReimported(UObject* InObject);

	/**
	 * Load the editor module that are loaded by default in the editor even when the target doesn't have an dependency on those 
	 * Note: This is useful for the commandlets that may depend on those modules.
	 */
	UNREALED_API void LoadDefaultEditorModules();

	//~ Begin UObject Interface.
	UNREALED_API virtual void FinishDestroy() override;
	UNREALED_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	static UNREALED_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	//~ End UObject Interface.

	//~ Begin UEngine Interface.
public:
	UNREALED_API virtual void Init(IEngineLoop* InEngineLoop) override;
	UNREALED_API virtual void PreExit() override;
	UNREALED_API virtual float GetMaxTickRate(float DeltaTime, bool bAllowFrameRateSmoothing = true) const override;
	UNREALED_API virtual void Tick(float DeltaSeconds, bool bIdleMode) override;
	UNREALED_API virtual bool ShouldDrawBrushWireframe(AActor* InActor) override;
	UNREALED_API virtual void NotifyToolsOfObjectReplacement(const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
	UNREALED_API virtual bool ShouldThrottleCPUUsage() const override;
	UNREALED_API virtual bool GetPropertyColorationMatch(class UObject* Object);
	UNREALED_API virtual bool WorldIsPIEInNewViewport(UWorld* InWorld) override;
	UNREALED_API virtual void FocusNextPIEWorld(UWorld* CurrentPieWorld, bool previous = false) override;
	UNREALED_API virtual void ResetPIEAudioSetting(UWorld *CurrentPieWorld) override;
	UNREALED_API virtual class UGameViewportClient* GetNextPIEViewport(UGameViewportClient* CurrentViewport) override;
	UNREALED_API virtual UWorld* CreatePIEWorldByDuplication(FWorldContext &WorldContext, UWorld* InWorld, FString &PlayWorldMapName) override;
	UNREALED_API virtual void PostCreatePIEWorld(UWorld* InWorld) override;
	virtual bool GetMapBuildCancelled() const override { return false; }
	virtual void SetMapBuildCancelled(bool InCancelled) override { /* Intentionally empty. */ }
	UNREALED_API virtual void HandleNetworkFailure(UWorld *World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString) override;
	virtual ERHIFeatureLevel::Type GetDefaultWorldFeatureLevel() const override { return DefaultWorldFeatureLevel; }
	UNREALED_API virtual bool GetPreviewPlatformName(FName& PlatformName) const override;

protected:
	UNREALED_API virtual void InitializeObjectReferences() override;
	UNREALED_API virtual void ProcessToggleFreezeCommand(UWorld* InWorld) override;
	UNREALED_API virtual void ProcessToggleFreezeStreamingCommand(UWorld* InWorld) override;
	UNREALED_API virtual void HandleBrowseToDefaultMapFailure(FWorldContext& Context, const FString& TextURL, const FString& Error) override;
private:
	UNREALED_API virtual void RemapGamepadControllerIdForPIE(class UGameViewportClient* GameViewport, int32 &ControllerId) override;
	UNREALED_API virtual TSharedPtr<SViewport> GetGameViewportWidget() const override;
	UNREALED_API virtual void TriggerStreamingDataRebuild() override;

	UNREALED_API virtual bool NetworkRemapPath(UNetConnection* Connection, FString& Str, bool bReading = true) override;
	UNREALED_API virtual bool NetworkRemapPath(UPendingNetGame* PendingNetGame, FString& Str, bool bReading = true) override;

	UNREALED_API virtual bool AreEditorAnalyticsEnabled() const override;
	UNREALED_API virtual void CreateStartupAnalyticsAttributes(TArray<FAnalyticsEventAttribute>& StartSessionAttributes) const override;
	UNREALED_API virtual void CheckAndHandleStaleWorldObjectReferences(FWorldContext* InWorldContext) override;

	/** Called during editor init and whenever the vanilla status might have changed, to set the flag on the base class */
	UNREALED_API void UpdateIsVanillaProduct();

	/** Called when hotreload adds a new class to create volume factories */
	UNREALED_API void CreateVolumeFactoriesForNewClasses(const TArray<UClass*>& NewClasses);
	//~ End UEngine Interface.
	
	//~ Begin FExec Interface
protected:
	UNREALED_API virtual bool Exec_Editor( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog ) override;
	//~ End FExec Interface

public:
	UNREALED_API bool	CommandIsDeprecated( const TCHAR* CommandStr, FOutputDevice& Ar );
	
	/**
	 * Exec command handlers
	 */
	UNREALED_API bool	HandleCallbackCommand( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleTestPropsCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleMapCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool	HandleSelectCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool	HandleDeleteCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool	HandleLightmassDebugCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassStatsCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleSwarmDistributionCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassImmediateImportCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassImmediateProcessCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassSortCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassDebugMaterialCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassPaddingCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassDebugPaddingCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleLightmassProfileCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleSelectNameCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld  );
	UNREALED_API bool	HandleDumpPublicCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleJumpToCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleBugItGoCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleBugItCommand(const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool	HandleTagSoundsCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandlecheckSoundsCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleFixupBadAnimNotifiersCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleSetDetailModeCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleSetDetailModeViewCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool	HandleCleanBSPMaterialCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld  );
	UNREALED_API bool	HandleAutoMergeStaticMeshCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleAddSelectedCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleToggleSocketGModeCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleListMapPackageDependenciesCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleRebuildVolumesCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool	HandleRemoveArchtypeFlagCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool	HandleStartMovieCaptureCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	UNREALED_API bool	HandleBuildMaterialTextureStreamingData( const TCHAR* Cmd, FOutputDevice& Ar );

	/**
	 * Initializes the Editor.
	 */
	UNREALED_API void InitEditor(IEngineLoop* InEngineLoop);

	/**
	 * Constructs a default cube builder brush, this function MUST be called at the AFTER UEditorEngine::Init in order to guarantee builder brush and other required subsystems exist.
	 *
	 * @param		InWorld			World in which to create the builder brush.
	 */
	UNREALED_API void InitBuilderBrush( UWorld* InWorld );

	/** Access user setting for audio mute. */
	UNREALED_API bool IsRealTimeAudioMuted() const;

	/** Set user setting for audio mute. */
	UNREALED_API void MuteRealTimeAudio(bool bMute);

	/** Access user setting for audio level. Fractional volume 0.0->1.0 */
	UNREALED_API float GetRealTimeAudioVolume() const;

	/** Set user setting for audio mute. Fractional volume 0.0->1.0 */
	UNREALED_API void SetRealTimeAudioVolume(float VolumeLevel);

	/**
	 * Updates a single viewport
	 * @param Viewport - the viewport that we're trying to draw
	 * @param bInAllowNonRealtimeViewportToDraw - whether or not to allow non-realtime viewports to update
	 * @param bLinkedOrthoMovement	True if orthographic viewport movement is linked
	 * @param bOutViewportDrawn If non-null, will be set to true if the viewport was drawn in this call.
	 * @return - Whether a NON-realtime viewport has updated in this call.  Used to help time-slice canvas redraws
	 */
	UNREALED_API bool UpdateSingleViewportClient(FEditorViewportClient* InViewportClient, const bool bInAllowNonRealtimeViewportToDraw, bool bLinkedOrthoMovement, bool* bOutViewportDrawn = nullptr );

	/** Used for generating status bar text */
	enum EMousePositionType
	{
		MP_None,
		MP_WorldspacePosition,
		MP_Translate,
		MP_Rotate,
		MP_Scale,
		MP_CameraSpeed,
		MP_NoChange
	};


	// Execute a command that is safe for rebuilds.
	UNREALED_API virtual bool SafeExec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog );
	// Process an incoming network message meant for the editor server
	UNREALED_API bool Exec_StaticMesh( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Brush( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Poly( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Obj( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Camera( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Transaction(const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Exec_Particle(const TCHAR* Str, FOutputDevice& Ar);

	/**
	 * Executes each line of text in a file sequentially, as if each were a separate command
	 *
	 * @param InWorld	World context
	 * @param InFilename The name of the file to load and execute
	 * @param Ar Output device
	 */
	UNREALED_API void ExecFile( UWorld* InWorld, const TCHAR* InFilename, FOutputDevice& Ar );

	//~ Begin Transaction Interfaces.
	UNREALED_API virtual int32 BeginTransaction(const TCHAR* TransactionContext, const FText& Description, UObject* PrimaryObject) override;
	UNREALED_API int32 BeginTransaction(const FText& Description);
	UNREALED_API virtual bool CanTransact() override;
	UNREALED_API virtual int32 EndTransaction() override;
	UNREALED_API virtual void CancelTransaction(int32 Index) override;
	UNREALED_API void ResetTransaction(const FText& Reason);
	UNREALED_API bool UndoTransaction(bool bCanRedo = true);
	UNREALED_API bool RedoTransaction();
	UNREALED_API bool IsTransactionActive() const;
	UNREALED_API FText GetTransactionName() const;
	UNREALED_API bool IsObjectInTransactionBuffer( const UObject* Object ) const;

	enum EMapRebuildType
	{
		/** Rebuild only the current level */
		MRT_Current				= 0,
		/** Rebuild all levels currently marked as visible */
		MRT_AllVisible			= 1,
		/** Rebuilt all levels currently marked as dirty for lighting */
		MRT_AllDirtyForLighting	= 2,
	};

	/**
	 * Rebuilds the map.
	 *
	 * @param RebuildMap	The map to be rebuilt
	 * @param RebuildType	Allows to filter which of the map's level should be rebuilt
	 */
	UNREALED_API void RebuildMap(UWorld* RebuildMap, EMapRebuildType RebuildType);
	
	/**
	 * Quickly rebuilds a single level (no bounds build, visibility testing or Bsp tree optimization).
	 *
	 * @param Level	The level to be rebuilt.
	 */
	UNREALED_API void RebuildLevel(ULevel& Level);
	
	/**
	 * Builds up a model from a set of brushes. Used by RebuildLevel.
	 *
	 * @param Model					The model to be rebuilt.
	 * @param bSelectedBrushesOnly	Use all brushes in the current level or just the selected ones?.
	 * @param bTreatMovableBrushesAsStatic	Treat moveable brushes as static?.
	 */
	UNREALED_API void RebuildModelFromBrushes(UModel* Model, bool bSelectedBrushesOnly, bool bTreatMovableBrushesAsStatic = false);

	/**
	 * Builds up a model from a given set of brushes. Used by BspConversionTool to build brushes before converting them
	 * to static meshes.
	 *
	 * @param BrushesToBuild	List of brushes to build.
	 * @param Model				Model into which to put the output.
	 */
	UNREALED_API void RebuildModelFromBrushes(TArray<ABrush*>& BrushesToBuild, UModel* Model);

	/**
	 * Rebuilds levels containing currently selected brushes and should be invoked after a brush has been modified
	 */
	UNREALED_API void RebuildAlteredBSP();

	/** Helper method for executing the de/intersect CSG operation */
	UNREALED_API void BSPIntersectionHelper(UWorld* InWorld, ECsgOper Operation);

	/**
	 * @return	A pointer to the named actor or NULL if not found.
	 */
	UNREALED_API AActor* SelectNamedActor(const TCHAR *TargetActorName);


	/**
	 * Moves an actor in front of a camera specified by the camera's origin and direction.
	 * The distance the actor is in front of the camera depends on the actors bounding cylinder
	 * or a default value if no bounding cylinder exists.
	 *
	 * @param InActor			The actor to move
	 * @param InCameraOrigin	The location of the camera in world space
	 * @param InCameraDirection	The normalized direction vector of the camera
	 */
	UNREALED_API void MoveActorInFrontOfCamera( AActor& InActor, const FVector& InCameraOrigin, const FVector& InCameraDirection );

	/**
	 * Moves all viewport cameras to the target actor.
	 * @param	Actor					Target actor.
	 * @param	bActiveViewportOnly		If true, move/reorient only the active viewport.
	 */
	UNREALED_API void MoveViewportCamerasToActor(AActor& Actor,  bool bActiveViewportOnly);

	/**
	* Moves all viewport cameras to focus on the provided array of actors.
	* @param	Actors					Target actors.

	* @param	bActiveViewportOnly		If true, move/reorient only the active viewport.
	*/
	UNREALED_API void MoveViewportCamerasToActor(const TArray<AActor*> &Actors, bool bActiveViewportOnly);

	/**
	* Moves all viewport cameras to focus on the provided array of actors.
	* @param	Actors					Target actors.
	* @param	Components				Target components (used of actors array is empty)
	* @param	bActiveViewportOnly		If true, move/reorient only the active viewport.
	*/
	UNREALED_API void MoveViewportCamerasToActor(const TArray<AActor*> &Actors, const TArray<UPrimitiveComponent*>& Components, bool bActiveViewportOnly);

	/**
	* Moves all viewport cameras to focus on the provided component.
	* @param	Component				Target component
	* @param	bActiveViewportOnly		If true, move/reorient only the active viewport.
	*/
	UNREALED_API void MoveViewportCamerasToComponent(const USceneComponent* Component, bool bActiveViewportOnly);

	/**
	* Moves all viewport cameras to focus on the provided set of elements.
	* @param	SelectionSet			Target elements
	* @param	bActiveViewportOnly		If true, move/reorient only the active viewport.
	*/
	UNREALED_API void MoveViewportCamerasToElement(const UTypedElementSelectionSet* SelectionSet, bool bActiveViewportOnly) const;

	/**
	 * Moves all viewport cameras to focus on the provided bounding box.
	 * @param	BoundingBox					Target box
	 * @param	bActiveViewportOnly			If true, move/reorient only the active viewport.
	 * @param	DrawDebubBoxTimeInSeconds	If greater than 0 a debug box is drawn representing the bounding box.It will be drawn for specified time.
	 */
	UNREALED_API void MoveViewportCamerasToBox(const FBox& BoundingBox, bool bActiveViewportOnly, float DrawDebugBoxTimeInSeconds = 0.f) const;

	/** 
	 * Snaps an element in a direction.  Optionally will align with the trace normal.
	 * @param InElementHandle	Element to move to the floor.
	 * @param InAlign			Whether or not to rotate the actor to align with the trace normal.
	 * @param InUseLineTrace	Whether or not to only trace with a line through the world.
	 * @param InUseBounds		Whether or not to base the line trace off of the bounds.
	 * @param InUsePivot		Whether or not to use the pivot position.
	 * @param InDestination		The destination element we want to move this actor to, unset assumes we just want to go towards the floor
	 * @return					Whether or not the actor was moved.
	 */
	UNREALED_API bool SnapElementTo(const FTypedElementHandle& InElementHandle, const bool InAlign, const bool InUseLineTrace, const bool InUseBounds, const bool InUsePivot, const FTypedElementHandle& InDestination = FTypedElementHandle(), TArrayView<const FTypedElementHandle> InElementsToIgnore = TArrayView<const FTypedElementHandle>());

	/**
	 * Snaps the view of the camera to that of the provided element.
	 *
	 * @param InElementHandle	The element the camera is going to be snapped to.
	 */
	UNREALED_API void SnapViewTo(const FTypedElementHandle& InElementHandle);

	/**
	 * Remove the roll, pitch and/or yaw from the perspective viewports' cameras.
	 *
	 * @param	Roll		If true, the camera roll is reset to zero.
	 * @param	Pitch		If true, the camera pitch is reset to zero.
	 * @param	Yaw			If true, the camera yaw is reset to zero.
	 */
	UNREALED_API void RemovePerspectiveViewRotation(bool Roll, bool Pitch, bool Yaw);

	//
	// Pivot handling.
	//

	virtual FVector GetPivotLocation() { return FVector::ZeroVector; }
	/** Sets the editor's pivot location, and optionally the pre-pivots of actors.
	 *
	 * @param	NewPivot				The new pivot location
	 * @param	bSnapPivotToGrid		If true, snap the new pivot location to the grid.
	 * @param	bIgnoreAxis				If true, leave the existing pivot unaffected for components of NewPivot that are 0.
	 * @param	bAssignPivot			If true, assign the given pivot to any valid actors that retain it (defaults to false)
	 */
	virtual void SetPivot(FVector NewPivot, bool bSnapPivotToGrid, bool bIgnoreAxis, bool bAssignPivot=false) {}
	virtual void ResetPivot() {}

	//
	// General functions.
	//

	/**
	 * Cleans up after major events like e.g. map changes.
	 *
	 * @param	ClearSelection	Whether to clear selection
	 * @param	Redraw			Whether to redraw viewports
	 * @param	TransReset		Human readable reason for resetting the transaction system
	 */
	UNREALED_API virtual void Cleanse( bool ClearSelection, bool Redraw, const FText& TransReset, bool bTransReset = true );
	virtual void FinishAllSnaps() { }

	/**
	 * Redraws all level editing viewport clients.
	 *
	 * @param	bInvalidateHitProxies		[opt] If true (the default), invalidates cached hit proxies too.
	 */
	virtual void RedrawLevelEditingViewports(bool bInvalidateHitProxies=true) {}
		
	/**
	 * Triggers a high res screen shot for current editor viewports.
	 *
	 */
	virtual void TakeHighResScreenShots(){}

	virtual void NoteSelectionChange(bool bNotify = true) {}

	/**
	 * Adds an actor to the world at the specified location.
	 *
	 * @param	InLevel			Level in which to add the actor
	 * @param	Class			A non-abstract, non-transient, placeable class.  Must be non-NULL.
	 * @param	Transform		The world-space transform to spawn the actor with.
	 * @param	bSilent			If true, suppress logging (optional, defaults to false).
	 * @param	ObjectFlags		The object flags to place on the spawned actor.
	 * @param	bSelectActor	Whether or not to select the spawned actor.
	 * @return					A pointer to the newly added actor, or NULL if add failed.
	 */
	UNREALED_API virtual AActor* AddActor(ULevel* InLevel, UClass* Class, const FTransform& Transform, bool bSilent = false, EObjectFlags ObjectFlags = RF_Transactional, bool bSelectActor = true);

	/**
	 * Adds actors to the world at the specified location using export text.
	 *
	 * @param	ExportText		A T3D representation of the actor to create.
	 * @param	bSilent			If true, suppress logging
	 * @param	ObjectFlags		The object flags to place on the spawned actor.
	 * @return					A pointer to the newly added actor, or NULL if add failed.
	 */
	UNREALED_API virtual TArray<AActor*> AddExportTextActors(const FString& ExportText, bool bSilent, EObjectFlags ObjectFlags = RF_Transactional);

	virtual void NoteActorMovement() { check(0); }
	UNREALED_API virtual UTransactor* CreateTrans();

	/** Plays an editor sound, loading the sound on demand if necessary (if the user has sounds enabled.)  The reference to the sound asset is not retained. */
	UNREALED_API void PlayEditorSound( const FString& SoundAssetName );

	/** Plays an editor sound (if the user has sounds enabled.) */
	UNREALED_API void PlayEditorSound( USoundBase* InSound );

	/**
	 * Returns true if currently able to play a sound and if the user has sounds enabled.
	 */
	UNREALED_API bool CanPlayEditorSound() const;


	/**
	 * Returns the preview audio component
	 */
	UNREALED_API UAudioComponent* GetPreviewAudioComponent();

	/**
	 * Returns an audio component linked to the current scene that it is safe to play a sound on
	 *
	 * @param	Sound		A sound to attach to the audio component
	 * @param	SoundNode	A sound node that is attached to the audio component when the sound is NULL
	 */
	UNREALED_API UAudioComponent* ResetPreviewAudioComponent(USoundBase* Sound = nullptr, USoundNode* SoundNode = nullptr);

	/**
	 * Plays a preview of a specified sound or node
	 *
	 * @param	Sound		A sound to attach to the audio component
	 * @param	SoundNode	A sound node that is attached to the audio component when the sound is NULL
	 */
	UNREALED_API UAudioComponent* PlayPreviewSound(USoundBase* Sound, USoundNode* SoundNode = nullptr);

	/**
	 * Clean up any world specific editor components so they can be GC correctly
	 */
	UNREALED_API void ClearPreviewComponents();

	
	/**
	 * Close all the edit windows for assets that are owned by the world being closed
	 */
	UNREALED_API void CloseEditedWorldAssets(UWorld* InWorld);

	/**
	 * Redraws all editor viewport clients.
	 *
	 * @param	bInvalidateHitProxies		[opt] If true (the default), invalidates cached hit proxies too.
	 */
	UNREALED_API void RedrawAllViewports(bool bInvalidateHitProxies=true);

	UE_DEPRECATED(5.3, "InvalidateChildViewports is deprecated")
	void InvalidateChildViewports(FSceneViewStateInterface* InParentView, bool bInvalidateHitProxies=true) {}

	/**
	 * Looks for an appropriate actor factory for the specified UClass.
	 *
	 * @param	InClass		The class to find the factory for.
	 * @return				A pointer to the factory to use.  NULL if no factories support this class.
	 */
	UNREALED_API UActorFactory* FindActorFactoryForActorClass( const UClass* InClass );

	/**
	 * Looks for an actor factory spawned from the specified class.
	 *
	 * @param	InClass		The factories class to find
	 * @return				A pointer to the factory to use.  NULL if no factories support this class.
	 */
	UNREALED_API UActorFactory* FindActorFactoryByClass( const UClass* InClass ) const;

	/**
	* Looks for an actor factory spawned from the specified class, for the specified UClass for an actor.
	*
	* @param	InFactoryClass		The factories class to find
	* @param	InActorClass		The class to find the factory for.
	* @return				A pointer to the factory to use.  NULL if no factories support this class.
	*/
	UNREALED_API UActorFactory* FindActorFactoryByClassForActorClass( const UClass* InFactoryClass, const UClass* InActorClass );

	/**
	 * Uses the supplied factory to create an actor at the clicked location and adds to level.
	 *
	 * @param	Factory					The factory to create the actor from.  Must be non-NULL.
	 * @param	ObjectFlags				[opt] The flags to apply to the actor when it is created
	 * @return							A pointer to the new actor, or NULL on fail.
	 */
	UNREALED_API AActor* UseActorFactoryOnCurrentSelection( UActorFactory* Factory, const FTransform* InActorTransform, EObjectFlags ObjectFlags = RF_Transactional );

	/**
	 * Uses the supplied factory to create an actor at the clicked location and adds to level.
	 *
	 * @param	Factory					The factory to create the actor from.  Must be non-NULL.
	 * @param	AssetData				The optional asset to base the actor construction on.
	 * @param	ActorLocation			[opt] If null, positions the actor at the mouse location, otherwise specified. Default is null.
	 * @param	bUseSurfaceOrientation	[opt] If true, align new actor's orientation to the underlying surface normal.  Default is false.
	 * @param	ObjectFlags				[opt] The flags to apply to the actor when it is created
	 * @return							A pointer to the new actor, or NULL on fail.
	 */
	UNREALED_API AActor* UseActorFactory( UActorFactory* Factory, const FAssetData& AssetData, const FTransform* ActorLocation, EObjectFlags ObjectFlags = RF_Transactional );

	/**
	 * Replaces the selected Actors with the same number of a different kind of Actor using the specified factory to spawn the new Actors
	 * note that only Location, Rotation, Drawscale, Drawscale3D, Tag, and Group are copied from the old Actors
	 * 
	 * @param Factory - the Factory to use to create Actors
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ReplaceSelectedActors instead.")
	UNREALED_API void ReplaceSelectedActors(UActorFactory* Factory, const FAssetData& AssetData, bool bCopySourceProperties = true);

	/**
	 * Replaces specified Actors with the same number of a different kind of Actor using the specified factory to spawn the new Actors
	 * note that only Location, Rotation, Drawscale, Drawscale3D, Tag, and Group are copied from the old Actors
	 * 
	 * @param Factory - the Factory to use to create Actors
	 * @param AssetData - the asset to feed the Factory
	 * @param ActorsToReplace - Actors to replace
	 * @param OutNewActors - Actors that were created
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ReplaceActors instead.")
	UNREALED_API void ReplaceActors(UActorFactory* Factory, const FAssetData& AssetData, const TArray<AActor*>& ActorsToReplace, TArray<AActor*>* OutNewActors = nullptr, bool bCopySourceProperties = true);

	/**
	 * Converts passed in brushes into a single static mesh actor. 
	 * Note: This replaces all the brushes with a single actor. This actor will not be attached to anything unless a single brush was converted.
	 *
	 * @param	InStaticMeshPackageName		The name to save the brushes to.
	 * @param	InBrushesToConvert			A list of brushes being converted.
	 *
	 * @return							Returns the newly created actor with the newly created static mesh.
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ConvertBrushesToStaticMesh instead.")
	UNREALED_API AActor* ConvertBrushesToStaticMesh(const FString& InStaticMeshPackageName, TArray<ABrush*>& InBrushesToConvert, const FVector& InPivotLocation);

	/**
	 * Converts passed in light actors into new actors of another type.
	 * Note: This replaces the old actor with the new actor.
	 * Most properties of the old actor that can be copied are copied to the new actor during this process.
	 * Properties that can be copied are ones found in a common superclass between the actor to convert and the new class.
	 * Common light component properties between the two classes are also copied
	 *
	 * @param	ConvertToClass	The light class we are going to convert to.
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ConvertLightActors instead.")
	UNREALED_API void ConvertLightActors( UClass* ConvertToClass );

	/**
	 * Converts passed in actors into new actors of the specified type.
	 * Note: This replaces the old actors with brand new actors while attempting to preserve as many properties as possible.
	 * Properties of the actors components are also attempted to be copied for any component names supplied in the third parameter.
	 * If a component name is specified, it is only copied if a component of the specified name exists in the source and destination actors,
	 * as well as in the class default object of the class of the source actor, and that all three of those components share a common base class.
	 * This approach is used instead of simply accepting component classes to copy because some actors could potentially have multiple of the same
	 * component type.
	 *
	 * @param	ActorsToConvert				Array of actors which should be converted to the new class type
	 * @param	ConvertToClass				Class to convert the provided actors to
	 * @param	ComponentsToConsider		Names of components to consider for property copying as well
	 * @param	bUseSpecialCases			If true, looks for classes that can be handled by hardcoded conversions
	 * @param	InStaticMeshPackageName		The name to save the brushes to.
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::DoConvertActors instead.")
	UNREALED_API void DoConvertActors( const TArray<AActor*>& ActorsToConvert, UClass* ConvertToClass, const TSet<FString>& ComponentsToConsider, bool bUseSpecialCases, const FString& InStaticMeshPackageName );

	/**
	 * Sets up for a potentially deferred ConvertActors call, based on if any brushes are being converted to a static mesh. If one (or more)
	 * are being converted, the user will need to put in a package before the process continues.
	 *
	 * @param	ActorsToConvert			Array of actors which should be converted to the new class type
	 * @param	ConvertToClass			Class to convert the provided actors to
	 * @param	ComponentsToConsider	Names of components to consider for property copying as well
	 * @param	bUseSpecialCases		If true, looks for classes that can be handled by hardcoded conversions
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ConvertActors instead.")
	UNREALED_API void ConvertActors( const TArray<AActor*>& ActorsToConvert, UClass* ConvertToClass, const TSet<FString>& ComponentsToConsider, bool bUseSpecialCases = false );

	/**
	 * Changes the state of preview mesh mode to on or off.
	 *
	 * @param	bState	Enables the preview mesh mode if true; Disables the preview mesh mode if false.
	 */
	UNREALED_API void SetPreviewMeshMode( bool bState );

	/**
	 * Updates the position of the preview mesh in the level.
	 */
	UNREALED_API void UpdatePreviewMesh();

	/**
	 * Changes the preview mesh to the next one.
	 */
	UNREALED_API void CyclePreviewMesh();

	/**
	 * Copy selected actors to the clipboard.
	 *
	 * @param	InWorld					World context
	 * @param	DestinationData			If != NULL, fill instead of clipboard data
	 */
	virtual void edactCopySelected(UWorld* InWorld, FString* DestinationData = nullptr) {}

	/**
	 * Paste selected actors from the clipboard.
	 *
	 * @param	InWorld				World context
	 * @param	bDuplicate			Is this a duplicate operation (as opposed to a real paste)?
	 * @param	bOffsetLocations	Should the actor locations be offset after they are created?
	 * @param	bWarnIfHidden		If true displays a warning if the destination level is hidden
	 * @param	SourceData			If != NULL, use instead of clipboard data
	 */
	virtual void edactPasteSelected(UWorld* InWorld, bool bDuplicate, bool bOffsetLocations, bool bWarnIfHidden, const FString* SourceData = nullptr) {}

	/**
	 * Duplicates selected actors.
	 *
	 * @param	InLevel				Level to place duplicate
	 * @param	bOffsetLocations	Should the actor locations be offset after they are created?
	 */
	virtual void edactDuplicateSelected( ULevel* InLevel, bool bOffsetLocations ) {}

	/**
	 * Deletes all selected actors
	 *
	 * @param	InWorld				World context
	 * @param	bVerifyDeletionCanHappen	[opt] If true (default), verify that deletion can be performed.
	 * @param	bWarnAboutReferences		[opt] If true (default), we prompt the user about referenced actors they are about to delete
	 * @param	bWarnAboutSoftReferences	[opt] If true (default), we prompt the user about soft references to actors they are about to delete
	 * @return								true unless the delete operation was aborted.
	 */
	virtual bool edactDeleteSelected(UWorld* InWorld, bool bVerifyDeletionCanHappen=true, bool bWarnAboutReferences = true, bool bWarnAboutSoftReferences = true) { return true; }

	/**
	 * Checks the state of the selected actors and notifies the user of any potentially unknown destructive actions which may occur as
	 * the result of deleting the selected actors.  In some cases, displays a prompt to the user to allow the user to choose whether to
	 * abort the deletion.
	 *
	 * @return								false to allow the selected actors to be deleted, true if the selected actors should not be deleted.
	 */
	virtual bool ShouldAbortActorDeletion() const { return false; }

	/**
	*
	* Rebuild the level's Bsp from the level's CSG brushes.
	*
	* @param InWorld	The world in which the rebuild the CSG brushes for 
	*/
	UNREALED_API virtual void csgRebuild( UWorld* InWorld );

	/**
	*
	* Find the Brush EdPoly corresponding to a given Bsp surface.
	*
	* @param InModel	Model to get poly from
	* @param iSurf		surface index
	* @param Poly		
	*
	* returns true if poly not available
	*/
	static UNREALED_API bool polyFindBrush(UModel* InModel, int32 iSurf, FPoly &Poly);

	/**
	 * Update a the brush EdPoly corresponding to a newly-changed
	 * poly to reflect its new properties.
	 *
	 * Doesn't do any transaction tracking.
	 */
	static UNREALED_API void polyUpdateBrush(UModel* Model, int32 iSurf, bool bUpdateTexCoords, bool bOnlyRefreshSurfaceMaterials);

	/**
	 * Populates a list with all polys that are linked to the specified poly.  The
	 * resulting list includes the original poly.
	 */
	UNREALED_API virtual void polyGetLinkedPolys( ABrush* InBrush, FPoly* InPoly, TArray<FPoly>* InPolyList );

	/**
	 * Takes a list of polygons and returns a list of the outside edges (edges which are not shared
	 * by other polys in the list).
	 */
	UNREALED_API virtual void polyGetOuterEdgeList( TArray<FPoly>* InPolyList, TArray<FEdge>* InEdgeList );

	/**
	 * Takes a list of polygons and creates a new list of polys which have no overlapping edges.  It splits
	 * edges as necessary to achieve this.
	 */
	UNREALED_API virtual void polySplitOverlappingEdges( TArray<FPoly>* InPolyList, TArray<FPoly>* InResult );

	/**
	 * Sets and clears all Bsp node flags.  Affects all nodes, even ones that don't
	 * really exist.
	 */
	UNREALED_API virtual void polySetAndClearPolyFlags( UModel* Model, uint32 SetBits, uint32 ClearBits, bool SelectedOnly, bool UpdateBrush );

	// Selection.
	virtual void SelectActor(AActor* Actor, bool bInSelected, bool bNotify, bool bSelectEvenIfHidden = false, bool bForceRefresh = false) {}
	virtual bool CanSelectActor(AActor* Actor, bool bInSelected, bool bSelectEvenIfHidden=false, bool bWarnIfLevelLocked=false) const { return true; }
	virtual void SelectGroup(class AGroupActor* InGroupActor, bool bForceSelection=false, bool bInSelected=true, bool bNotify=true) {}
	virtual void SelectComponent(class UActorComponent* Component, bool bInSelected, bool bNotify, bool bSelectEvenIfHidden = false) {}

	/**
	 * Selects or deselects a BSP surface in the persistent level's UModel.  Does nothing if GEdSelectionLock is true.
	 *
	 * @param	InModel					The model of the surface to select.
	 * @param	iSurf					The index of the surface in the persistent level's UModel to select/deselect.
	 * @param	bSelected				If true, select the surface; if false, deselect the surface.
	 * @param	bNoteSelectionChange	If true, call NoteSelectionChange().
	 */
	virtual void SelectBSPSurf(UModel* InModel, int32 iSurf, bool bSelected, bool bNoteSelectionChange) {}

	/**
	 * Deselect all actors.  Does nothing if GEdSelectionLock is true.
	 *
	 * @param	bNoteSelectionChange		If true, call NoteSelectionChange().
	 * @param	bDeselectBSPSurfs			If true, also deselect all BSP surfaces.
	 */
	virtual void SelectNone(bool bNoteSelectionChange, bool bDeselectBSPSurfs, bool WarnAboutManyActors=true) {}

	/**
	 * Deselect all surfaces.
	 */
	virtual void DeselectAllSurfaces() {}

	// Bsp Poly selection virtuals from EditorCsg.cpp.
	UNREALED_API virtual void polySelectAll ( UModel* Model );
	UNREALED_API virtual void polySelectMatchingGroups( UModel* Model );
	UNREALED_API virtual void polySelectMatchingItems( UModel* Model );
	UNREALED_API virtual void polySelectCoplanars( UWorld* InWorld, UModel* Model );
	UNREALED_API virtual void polySelectAdjacents( UWorld* InWorld, UModel* Model );
	UNREALED_API virtual void polySelectAdjacentWalls( UWorld* InWorld, UModel* Model );
	UNREALED_API virtual void polySelectAdjacentFloors( UWorld* InWorld, UModel* Model );
	UNREALED_API virtual void polySelectAdjacentSlants( UWorld* InWorld, UModel* Model );
	UNREALED_API virtual void polySelectMatchingBrush( UModel* Model );

	/**
	 * Selects surfaces whose material matches that of any selected surfaces.
	 *
	 * @param	bCurrentLevelOnly		If true, select
	 */
	UNREALED_API virtual void polySelectMatchingMaterial(UWorld* InWorld, bool bCurrentLevelOnly);

	/**
	 * Selects surfaces whose lightmap resolution matches that of any selected surfaces.
	 *
	 * @param	bCurrentLevelOnly		If true, select
	 */
	UNREALED_API virtual void polySelectMatchingResolution(UWorld* InWorld, bool bCurrentLevelOnly);

	UNREALED_API virtual void polySelectReverse( UModel* Model );
	UNREALED_API virtual void polyMemorizeSet( UModel* Model );
	UNREALED_API virtual void polyRememberSet( UModel* Model );
	UNREALED_API virtual void polyXorSet( UModel* Model );
	UNREALED_API virtual void polyUnionSet( UModel* Model );
	UNREALED_API virtual void polyIntersectSet( UModel* Model );
	UNREALED_API virtual void polySelectZone( UModel *Model );

	// Pan textures on selected polys.  Doesn't do transaction tracking.
	UNREALED_API virtual void polyTexPan( UModel* Model, int32 PanU, int32 PanV, int32 Absolute );

	// Scale textures on selected polys. Doesn't do transaction tracking.
	UNREALED_API virtual void polyTexScale( UModel* Model,float UU, float UV, float VU, float VV, bool Absolute );

	// Map brush selection virtuals from EditorCsg.cpp.
	UNREALED_API virtual void MapSelectOperation( UWorld* InWorld, EBrushType BrushType );
	UNREALED_API virtual void MapSelectFlags( UWorld* InWorld, uint32 Flags );

	// Put the first selected brush into the current Brush.
	UNREALED_API virtual void MapBrushGet(UWorld* InWorld);

	// Replace all selected brushes with the current Brush.
	UNREALED_API virtual void mapBrushPut();

	// Send all selected brushes in a level to the front of the hierarchy
	UNREALED_API virtual void mapSendToFirst(UWorld* InWorld);

	// Send all selected brushes in a level to the back of the hierarchy
	UNREALED_API virtual void mapSendToLast(UWorld* InWorld);

	/**
	 * Swaps position in the actor list for the first two selected actors in the current level
	 */
	UNREALED_API virtual void mapSendToSwap(UWorld* InWorld);
	UNREALED_API virtual void MapSetBrush( UWorld* InWorld, EMapSetBrushFlags PropertiesMask, uint16 BrushColor, FName Group, uint32 SetPolyFlags, uint32 ClearPolyFlags, uint32 BrushType, int32 DrawType );

	// Bsp virtuals from Bsp.cpp.
	UNREALED_API virtual void bspRepartition( UWorld* InWorld, int32 iNode );

	/** Convert a Bsp node to an EdPoly.  Returns number of vertices in Bsp node. */
	UNREALED_API virtual int32 bspNodeToFPoly( UModel* Model, int32 iNode, FPoly* EdPoly );

	/**
	 * Clean up all nodes after a CSG operation.  Resets temporary bit flags and unlinks
	 * empty leaves.  Removes zero-vertex nodes which have nonzero-vertex coplanars.
	 */
	UNREALED_API virtual void bspCleanup( UModel* Model );

	/** 
	 * Build EdPoly list from a model's Bsp. Not transactional.
	 * @param DestArray helps build bsp FPolys in non-main threads. It also allows to perform this action without GUndo 
	 *	interfering. Temporary results will be written to DestArray. Defaults to Model->Polys->Element
	 */
	UNREALED_API virtual void bspBuildFPolys( UModel* Model, bool SurfLinks, int32 iNode, TArray<FPoly>* DestArray = NULL );
	UNREALED_API virtual void bspMergeCoplanars( UModel* Model, bool RemapLinks, bool MergeDisparateTextures );
	/**
	 * Performs any CSG operation between the brush and the world.
	 *
	 * @param	Actor							The brush actor to apply.
	 * @param	Model							The model to apply the CSG operation to; typically the world's model.
	 * @param	PolyFlags						PolyFlags to set on brush's polys.
	 * @param	BrushType						The type of brush.
	 * @param	CSGOper							The CSG operation to perform.
	 * @param	bBuildBounds					If true, updates bounding volumes on Model for CSG_Add or CSG_Subtract operations.
	 * @param	bMergePolys						If true, coplanar polygons are merged for CSG_Intersect or CSG_Deintersect operations.
	 * @param	bReplaceNULLMaterialRefs		If true, replace NULL material references with a reference to the GB-selected material.
	 * @param	bShowProgressBar				If true, display progress bar for complex brushes
	 * @return									0 if nothing happened, 1 if the operation was error-free, or 1+N if N CSG errors occurred.
	 */
	UNREALED_API virtual int32 bspBrushCSG( ABrush* Actor, UModel* Model, uint32 PolyFlags, EBrushType BrushType, ECsgOper CSGOper, bool bBuildBounds, bool bMergePolys, bool bReplaceNULLMaterialRefs, bool bShowProgressBar=true );

	/**
	 * Optimize a level's Bsp, eliminating T-joints where possible, and building side
	 * links.  This does not always do a 100% perfect job, mainly due to imperfect 
	 * levels, however it should never fail or return incorrect results.
	 */
	UNREALED_API virtual void bspOptGeom( UModel* Model );
	
	/**
	 * Builds lighting information depending on passed in options.
	 *
	 * @param	Options		Options determining on what and how lighting is built
	 */
	UNREALED_API void BuildLighting(const class FLightingBuildOptions& Options);

	/** Updates the asynchronous static light building */
	UNREALED_API void UpdateBuildLighting();

	/** Checks to see if the asynchronous lighting build is running or not */
	UNREALED_API bool IsLightingBuildCurrentlyRunning() const;

	UNREALED_API bool IsLightingBuildCurrentlyExporting() const;

	/** Checks if asynchronous lighting is building, if so, it throws a warning notification and returns true */
	UNREALED_API bool WarnIfLightingBuildIsCurrentlyRunning();

	/**
	 * Open a Fbx file with the given name, and import each sequence with the supplied Skeleton.
	 * This is only possible if each track expected in the Skeleton is found in the target file. If not the case, a warning is given and import is aborted.
	 * If Skeleton is empty (ie. TrackBoneNames is empty) then we use this Fbx file to form the track names array.
	 *
	 * @param Skeleton	The skeleton that animation is import into
	 * @param Filename	The FBX filename
	 * @param bImportMorphTracks	true to import any morph curve data.
	 */
	static UNREALED_API UAnimSequence * ImportFbxAnimation( USkeleton* Skeleton, UObject* Outer, class UFbxAnimSequenceImportData* ImportData, const TCHAR* InFilename, const TCHAR* AnimName, bool bImportMorphTracks );
	/**
	 * Reimport animation using SourceFilePath and SourceFileStamp 
	 *
	 * @param Skeleton				The skeleton that animation is import into
	 * @param AnimSequence			The existing AnimSequence.
	 * @param ImportData			The import data of the existing AnimSequence
	 * @param InFilename			The FBX filename
	 * @param bOutImportAll			
	 * @param bFactoryShowOptions	When true, create a UI popup asking the user for the reimport options.
	 * @param ReimportUI			Optional parameter used to pass reimport options.
	 */
	static UNREALED_API bool ReimportFbxAnimation( USkeleton* Skeleton, UAnimSequence* AnimSequence, class UFbxAnimSequenceImportData* ImportData, const TCHAR* InFilename, bool& bOutImportAll, const bool bFactoryShowOptions, UFbxImportUI* ReimportUI = nullptr);


	// Object management.
	UNREALED_API virtual void RenameObject(UObject* Object,UObject* NewOuter,const TCHAR* NewName, ERenameFlags Flags=REN_None);

	// Level management.
	UNREALED_API void AnalyzeLevel(ULevel* Level,FOutputDevice& Ar);

	/**
	 * Updates all components in the current level's scene.
	 */
	UNREALED_API void EditorUpdateComponents();

	/** 
	 * Displays a modal message dialog 
	 * @param	InMessageCategory Category of the message box
	 * @param	InMessage	Type of the message box
	 * @param	InText		Message to display
	 * @param	InTitle		Title for the message box
	 * @return	Returns the result of the modal message box
	 */
	UNREALED_API EAppReturnType::Type OnModalMessageDialog(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessage, const FText& InText, const FText& InTitle);

	/** 
	 * Returns whether an object should replace an exisiting one or not 
	 * @param	Filename		Filename of the package
	 * @return	Returns whether the objects should replace the already existing ones.
	 */
	UNREALED_API bool OnShouldLoadOnTop(const FString& Filename);

	//@todo Slate Editor: Merge PlayMap and RequestSlatePlayMapSession
	/**
	 * Makes a request to start a play from editor session (in editor or on a remote platform)
	 * @param	StartLocation			If specified, this is the location to play from (via a PlayerStartPIE - Play From Here)
	 * @param	StartRotation			If specified, this is the rotation to start playing at
	 * @param	DestinationConsole		Where to play the game - -1 means in editor, 0 or more is an index into the GConsoleSupportContainer
	 * @param	InPlayInViewportIndex	Viewport index to play the game in, or -1 to spawn a standalone PIE window
	 * @param	bUseMobilePreview		True to enable mobile preview mode (PC platform only)
	 */
	UNREALED_API virtual void PlayMap( const FVector* StartLocation = NULL, const FRotator* StartRotation = NULL, int32 DestinationConsole = -1, int32 InPlayInViewportIndex = -1, bool bUseMobilePreview = false );



	/**
	 * Can the editor do cook by the book in the editor process space
	 */
	virtual bool CanCookByTheBookInEditor(const FString& PlatformName ) const { return false; }

	/**
	 * Can the editor act as a cook on the fly server
	 */
	virtual bool CanCookOnTheFlyInEditor(const FString& PlatformName) const { return false; }

	/**
	 * Start cook by the book in the editor process space
	 */
	virtual void StartCookByTheBookInEditor( const TArray<ITargetPlatform*> &TargetPlatforms, const TArray<FString> &CookMaps, const TArray<FString> &CookDirectories, const TArray<FString> &CookCultures, const TArray<FString> &IniMapSections ) { }

	/**
	 * Checks if the cook by the book is finished
	 */
	virtual bool IsCookByTheBookInEditorFinished() const { return true; }

	/**
	 * Cancels the current cook by the book in editor
	 */
	virtual void CancelCookByTheBookInEditor() { }

	/** Request a play session (Play in Editor, Play in New Process, Launcher) with the configured parameters. See FRequestPlaySessionParams for more details. */
	UNREALED_API void RequestPlaySession(const FRequestPlaySessionParams& InParams);

	/** Cancel request to start a play session */
	UNREALED_API void CancelRequestPlaySession();

	/** Pause or unpause all PIE worlds. Returns true if successful */
	UNREALED_API bool SetPIEWorldsPaused(bool Paused);

	/** Makes a request to start a play from a Slate editor session */
	void RequestToggleBetweenPIEandSIE() { bIsToggleBetweenPIEandSIEQueued = true; }

	/** Called when the debugger has paused the active PIE or SIE session */
	UNREALED_API void PlaySessionPaused();

	/** Called when the debugger has resumed the active PIE or SIE session */
	UNREALED_API void PlaySessionResumed();

	/** Called when the debugger has single-stepped the active PIE or SIE session */
	UNREALED_API void PlaySessionSingleStepped();

	/** Returns true if we are currently either PIE/SIE in the editor, false if we are not (even if we would start next tick). See IsPlaySessionInProgress() */
	bool IsPlayingSessionInEditor() const { return PlayInEditorSessionInfo.IsSet(); }
	/** Returns true if we are going to start PIE/SIE on the next tick, false if we are not (or if we are already in progress). See IsPlaySessionInProgress() */
	bool IsPlaySessionRequestQueued() const { return PlaySessionRequest.IsSet(); }
	/** Returns true if Playing in Editor, Simulating in Editor, or are either of these queued to start on the next tick. */
	bool IsPlaySessionInProgress() const { return IsPlayingSessionInEditor() || IsPlaySessionRequestQueued(); }
	
	/** Returns true if we are currently Simulating in Editor, false if we are not (even if we would next tick). See IsSimulateInEditorInProgress() */
	bool IsSimulatingInEditor() const { return PlayInEditorSessionInfo.IsSet() && PlayInEditorSessionInfo->OriginalRequestParams.WorldType == EPlaySessionWorldType::SimulateInEditor; }	
	/** Returns true if we are going to start Simulating in Editor on the next tick, false if we are not (or we are already simulating). See IsSimulateInEditorInProgress() */
	bool IsSimulateInEditorQueued() const { return PlaySessionRequest.IsSet() && PlaySessionRequest->WorldType == EPlaySessionWorldType::SimulateInEditor; }
	/** Returns true if we are currently Simulating in Editor, or are going to start Simulating in Editor on the next tick. */
	bool IsSimulateInEditorInProgress() const { return IsSimulatingInEditor() || IsSimulateInEditorQueued(); }

	/** Returns the currently set PlaySessionRequest (if any). Unset if there is no session queued or has already been started (see GetPlayInEditorSessionInfo()) */
	const TOptional<FRequestPlaySessionParams> GetPlaySessionRequest() const { return PlaySessionRequest; }

	/** Returns information about the currently active Play In Editor session. Unset if there is no active PIE session. */
	const TOptional<FPlayInEditorSessionInfo> GetPlayInEditorSessionInfo() const { return PlayInEditorSessionInfo; }
	
	/** Returns true if the user has queued a Play in Editor (in VR Mode) or one is currently running. */
	bool IsVRPreviewActive() const
	{
		return (PlaySessionRequest.IsSet() && PlaySessionRequest->SessionPreviewTypeOverride == EPlaySessionPreviewType::VRPreview) ||
			(PlayInEditorSessionInfo.IsSet() && PlayInEditorSessionInfo->OriginalRequestParams.SessionPreviewTypeOverride == EPlaySessionPreviewType::VRPreview);
	}

	/** Called when game client received input key */
	UNREALED_API bool ProcessDebuggerCommands(const FKey InKey, const FModifierKeysState ModifierKeyState, EInputEvent EventType);

	/** Kicks off a "Request Play Session" request if there is one set. This should not be called mid-transaction. */
	UNREALED_API void StartQueuedPlaySessionRequest();

	/**
	 * Request that the current PIE/SIE session should end.  
	 * This should be used to end the game if its not safe to directly call EndPlayMap in your stack frame
	 */
	UNREALED_API void RequestEndPlayMap();

	/**
	 * @return true if there is an end play map request queued 
	 */
	bool ShouldEndPlayMap() const { return bRequestEndPlayMapQueued; }

	/**
	 * Request to create a new PIE window and join the currently running PIE session.
	 * Deferred until the next tick.
	 */
	UNREALED_API void RequestLateJoin();

	/**
	 * Builds a URL for game spawned by the editor (not including map name!). 
	 * @param	MapName			The name of the map to put into the URL
	 * @param	bSpecatorMode	If true, the player starts in spectator mode
	 * @param   AdditionalURLOptions	Additional URL Options to append (e.g, ?OptionX?OptionY). This is in addition to InEditorGameURLOptions.
	 *
	 * @return	The URL for the game
	 */
	UNREALED_API virtual FString BuildPlayWorldURL(const TCHAR* MapName, bool bSpectatorMode = false, FString AdditionalURLOptions=FString());

	/**
	 * Kills the Play From Here session
	 */
	UNREALED_API virtual void EndPlayMap();

	/** 
	 * Destroy the current play session and perform miscellaneous cleanup
	 */
	UNREALED_API virtual void TeardownPlaySession(FWorldContext &PieWorldContext);

	/**
	 * Ends the current play on local pc session.
	 * @todo gmp: temp hack for Rocket demo
	 */
	UNREALED_API virtual void EndPlayOnLocalPc();

	/**
	 * Overrides the realtime state all viewports until RemoveViewportsRealtimeOverride is called.
	 * The state of this override is not preserved between editor sessions. 
	 *
	 * @param bShouldBeRealtime	If true, this viewport will be realtime, if false this viewport will not be realtime
	 * @param SystemDisplayName	This display name of whatever system is overriding realtime. This name is displayed to users in the viewport options menu
	 */
	UNREALED_API void SetViewportsRealtimeOverride(bool bShouldBeRealtime, FText SystemDisplayName);

	/**
	 * Removes the current realtime override.  If there was another realtime override set it will restore that override
	 */
	UNREALED_API void RemoveViewportsRealtimeOverride(FText SystemDisplayName);

	/**
	 * Checks to see if any viewport is set to update in realtime.
	 */
	UNREALED_API bool IsAnyViewportRealtime();

	/**
	 *	Returns pointer to a temporary render target.
	 *	If it has not already been created, does so here.
	 */
	UNREALED_API UTextureRenderTarget2D* GetScratchRenderTarget( const uint32 MinSize );

	/**
	 *  Returns the Editors timer manager instance.
	 */
	TSharedRef<class FTimerManager> GetTimerManager() { return TimerManager.ToSharedRef(); }

	/**
	 *  Returns true if the editors timer manager is valid (may not be during early startup);
	 */
	bool IsTimerManagerValid() { return TimerManager.IsValid(); }

	/**
	*  Returns the Editors world manager instance.
	*/
	UEditorWorldExtensionManager* GetEditorWorldExtensionsManager() { return EditorWorldExtensionsManager; }

	// Editor specific

	/**
	* Closes the main editor frame.
	*/
	virtual void CloseEditor() {}
	virtual void GetPackageList( TArray<UPackage*>* InPackages, UClass* InClass ) {}

	/**
	 * Returns the number of currently selected actors.
	 */
	UNREALED_API int32 GetSelectedActorCount() const;

	/**
	 * Returns the set of selected actors.
	 */
	UNREALED_API class USelection* GetSelectedActors() const;

	/** Returns True is a world info Actor is selected */
	UNREALED_API bool IsWorldSettingsSelected();

	/** Function to return unique list of the classes of the assets currently selected in content browser (loaded/not loaded) */
	UNREALED_API void GetContentBrowserSelectionClasses( TArray<UClass*>& Selection ) const;

	/** Function to return list of assets currently selected in the content browser */
	UNREALED_API void GetContentBrowserSelections(TArray<FAssetData>& Selections) const;

	/**
	 * Returns an FSelectionIterator that iterates over the set of selected actors.
	 */
	UNREALED_API class FSelectionIterator GetSelectedActorIterator() const;

	/**
	* Returns an FSelectionIterator that iterates over the set of selected components.
	*/
	UNREALED_API class FSelectionIterator GetSelectedComponentIterator() const;

	UNREALED_API class FSelectedEditableComponentIterator GetSelectedEditableComponentIterator() const;

	/**
	* Returns the number of currently selected components.
	*/
	UNREALED_API int32 GetSelectedComponentCount() const;

	/**
	* @return the set of selected components.
	*/
	UNREALED_API class USelection* GetSelectedComponents() const;

	/**
	 * @return the set of selected non-actor objects.
	 */
	UNREALED_API class USelection* GetSelectedObjects() const;

	/**
	 * @return the appropriate selection set for the specified object class.
	 */
	UNREALED_API class USelection* GetSelectedSet( const UClass* Class ) const;

	/**
	 * @param	RequiredParentClass The parent class this type must implement, or null to accept anything
	 * @return	The first selected class (either a UClass type itself, or the UClass generated by a blueprint), or null if there are no class or blueprint types selected
	 */
	UNREALED_API const UClass* GetFirstSelectedClass( const UClass* const RequiredParentClass ) const;

	/**
	 * Get the selection state of the current level (its actors and components) so that it might be restored later.
	 */
	UNREALED_API void GetSelectionStateOfLevel(FSelectionStateOfLevel& OutSelectionStateOfLevel) const;

	/**
	 * Restore the selection state of the current level (its actors and components) from a previous state.
	 */
	UNREALED_API void SetSelectionStateOfLevel(const FSelectionStateOfLevel& InSelectionStateOfLevel);

	/**
	 * Reset All Selection Sets (i.e. objects, actors, components)
	 * @note each set is independent and a selected actor might not be in the object selection set.
	 */
	UNREALED_API void ResetAllSelectionSets();

	/**
	 * Clears out the current map, if any, and creates a new blank map.
	 *
	 * @param   bIsPartitionedWorld	If true, new map is partitioned.
	 * 
	 * @return	Pointer to the map that was created.
	 */
	UNREALED_API UWorld* NewMap(bool bIsPartitionedWorld = false);

	/**
	 * Exports the current map to the specified filename.
	 *
	 * @param	InFilename					Filename to export the map to.
	 * @param	bExportSelectedActorsOnly	If true, export only the selected actors.
	 */
	UNREALED_API void ExportMap(UWorld* InWorld, const TCHAR* InFilename, bool bExportSelectedActorsOnly);

	/**
	 *	Returns list of all foliage types used in the world
	 * 
	 * @param	InWorld	 The target world.
	 * @return	List of all foliage types used in the world
	 */
	UNREALED_API TArray<UFoliageType*> GetFoliageTypesInWorld(UWorld* InWorld);

	/**
	 * Checks to see whether it's possible to perform a copy operation on the selected actors.
	 *
	 * @param InWorld		World to get the selected actors from
	 * @param OutCopySelected	Can be NULL, copies the internal results of the copy check to this struct
	 * @return			true if it's possible to do a copy operation based on the selection
	 */
	UNREALED_API bool CanCopySelectedActorsToClipboard( UWorld* InWorld, FCopySelectedInfo* OutCopySelected = NULL );

	/**
	 * Copies selected actors to the clipboard.  Supports copying actors from multiple levels.
	 * NOTE: Doesn't support copying prefab instance actors!
	 *
	 * @param InWorld				World to get the selected actors from
	 * @param bShouldCut			If true, deletes the selected actors after copying them to the clipboard
	 * @param bIsMove				If true, this cut is part of a move and the actors will be immediately pasted
	 * @param bWarnAboutReferences	Whether or not to show a modal warning about referenced actors that may no longer function after being moved
	 * @param DestinationData		Data that was copied to clipboard 
	 */
	UNREALED_API void CopySelectedActorsToClipboard( UWorld* InWorld, const bool bShouldCut, const bool bIsMove = false, bool bWarnAboutReferences = true, FString* DestinationData = nullptr);

	/**
	 * Checks to see whether it's possible to perform a paste operation.
	 *
	 * @param InWorld		World to paste into
	 * @return				true if it's possible to do a Paste operation
	 */
	UNREALED_API bool CanPasteSelectedActorsFromClipboard( UWorld* InWorld );

	/**
	 * Pastes selected actors from the clipboard.
	 * NOTE: Doesn't support pasting prefab instance actors!
	 *
	 * @param InWorld		World to get the selected actors from
	 * @param PasteTo		Where to paste the content too
	 */
	UNREALED_API void PasteSelectedActorsFromClipboard( UWorld* InWorld, const FText& TransDescription, const EPasteTo PasteTo );

	/**
	 * Sets property value and property chain to be used for property-based coloration.
	 *
	 * @param	PropertyValue		The property value to color.
	 * @param	Property			The property to color.
	 * @param	CommonBaseClass		The class of object to color.
	 * @param	PropertyChain		The chain of properties from member to lowest property.
	 * @param	bSelectActors		Whether to also select the actors that match the PropertyValue or not
	 */
	UNREALED_API virtual void SetPropertyColorationTarget(UWorld* InWorld, const FString& PropertyValue, class FProperty* Property, class UClass* CommonBaseClass, TSharedRef<FEditPropertyChain>* PropertyChain, bool bSelectActors = false);

	/**
	 * Checks if the active viewport mode is property coloration mode.
	 *
	 * @return	true if the active viewport mode is property coloration mode.
	 */
	UNREALED_API bool IsPropertyColorationModeActive();

	/**
	 * Warns the user of any hidden levels, and prompts them with a Yes/No dialog
	 * for whether they wish to continue with the operation.  No dialog is presented if all
	 * levels are visible.  The return value is true if no levels are hidden or
	 * the user selects "Yes", or false if the user selects "No".
	 *
	 * @param	InWorld					World context
	 * @param	bIncludePersistentLvl	If true, the persistent level will also be checked for visibility
	 * @return							false if the user selects "No", true otherwise.
	 */
	UNREALED_API bool WarnAboutHiddenLevels( UWorld* InWorld, bool bIncludePersistentLvl) const;

	UNREALED_API void ApplyDeltaToActor(AActor* InActor, bool bDelta, const FVector* InTranslation, const FRotator* InRotation, const FVector* InScaling, bool bAltDown=false, bool bShiftDown=false, bool bControlDown=false) const;

	UNREALED_API void ApplyDeltaToComponent(USceneComponent* InComponent, bool bDelta, const FVector* InTranslation, const FRotator* InRotation, const FVector* InScaling, const FVector& PivotLocation ) const;

	/**
	 * Disable actor/component modification during delta movement.
	 *
	 * @param bDisable True if modification should be disabled; false otherwise.
	 */
	void DisableDeltaModification(bool bDisable) { bDisableDeltaModification = bDisable; }

	/**
	 * Test whether actor/component modification during delta movement is currently enabled?
	 */
	bool IsDeltaModificationEnabled() const { return !bDisableDeltaModification; }

	/**
	 *	Game-specific function called by Map_Check BEFORE iterating over all actors.
	 *
	 *	@param	Str						The exec command parameters
	 *	@param	Ar						The output archive for logging (?)
	 *	@param	bCheckDeprecatedOnly	If true, only check for deprecated classes
	 */
	virtual bool Game_Map_Check(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar, bool bCheckDeprecatedOnly) { return true; }

	/**
	 *	Game-specific function called per-actor by Map_Check
	 *
	 *	@param	Str						The exec command parameters
	 *	@param	Ar						The output archive for logging (?)
	 *	@param	bCheckDeprecatedOnly	If true, only check for deprecated classes
	 */
	virtual bool Game_Map_Check_Actor(const TCHAR* Str, FOutputDevice& Ar, bool bCheckDeprecatedOnly, AActor* InActor) { return true; }

	/**
	 * Auto merge all staticmeshes that are able to be merged
	 */
	UNREALED_API void AutoMergeStaticMeshes();

	/**
	 *	Check the InCmdParams for "MAPINISECTION=<name of section>".
	 *	If found, fill in OutMapList with the proper map names.
	 *
	 *	@param	InCmdParams		The cmd line parameters for the application
	 *	@param	OutMapList		The list of maps from the ini section, empty if not found
	 */
	UNREALED_API void ParseMapSectionIni(const TCHAR* InCmdParams, TArray<FString>& OutMapList);

	/**
	 *	Load the list of maps from the given section of the Editor.ini file
	 *	Supports sections contains other sections - but DOES NOT HANDLE CIRCULAR REFERENCES!!!
	 *
	 *	@param	InSectionName		The name of the section to load
	 *	@param	OutMapList			The list of maps from that section
	 */
	UNREALED_API void LoadMapListFromIni(const FString& InSectionName, TArray<FString>& OutMapList);

	/**
	 *	Check whether the specified package file is a map
	 *
	 *	@param	PackageFilename		The name of the package to check
	 *	@param	OutErrorMsg			if an errors occurs, this is the localized reason
	 *	@return	true if the package is not a map file
	 */
	UNREALED_API bool PackageIsAMapFile( const TCHAR* PackageFilename, FText& OutNotMapReason );

	/**
	 * Synchronizes the content or generic browser's selected objects to the collection specified.
	 *
	 * @param	InObjectsToSync	the list of objects to sync to
	 */
	UNREALED_API void SyncBrowserToObjects( const TArray<UObject*>& InObjectsToSync, bool bFocusContentBrowser = true );
	UNREALED_API void SyncBrowserToObjects( const TArray<struct FAssetData>& InAssetsToSync, bool bFocusContentBrowser = true );

	/**
	 * Synchronizes the content or generic browser's selected object to the one specified.
	 *
	 * @param	InObjectToSync	the object to sync to
	 */
	UNREALED_API void SyncBrowserToObject( const UObject* InObjectToSync, bool bFocusContentBrowser = true );
	UNREALED_API void SyncBrowserToObject( const struct FAssetData& InAssetToSync, bool bFocusContentBrowser = true );

	/**
	 * Syncs the selected actors objects to the content browser
	 * 
	 * @param bAllowOverrideMetadata If true, allows an asset to define "BrowseToAssetOverride" in its metadata to sync to an asset other than itself
	 */
	UNREALED_API void SyncToContentBrowser(bool bAllowOverrideMetadata = true);

	/**
	 * Syncs the selected actors' levels to the content browser
	 */
	UNREALED_API void SyncActorLevelsToContentBrowser();

	/**
	 * Checks if the slected objects contain something to browse to
	 *
	 * @return true if any of the selected objects contains something that can be browsed to
	 */
	UNREALED_API bool CanSyncToContentBrowser();

	/**
	 * Checks if the selected objects have levels which can be browsed to
	 *
	 * @return true if any of the selected objects contains something that can be browsed to
	 */
	UNREALED_API bool CanSyncActorLevelsToContentBrowser();

	/**
	 * Toggles the movement lock on selected actors so they may or may not be moved with the transform widgets
	 */
	UE_DEPRECATED(5.8, "Use ToggleSelectedItemMovementLock instead.")
	UNREALED_API void ToggleSelectedActorMovementLock();

	/**
	 * @return true if there are selected locked actors
	 */
	UE_DEPRECATED(5.8, "Use HasSelectedMovementLockedItems instead.")
	UNREALED_API bool HasLockedActors();

	/**
	 * Toggles the movement lock on selected items via TEDS.
	 */
	UE_EXPERIMENTAL(5.8, "This API is subject to change.")
	UNREALED_API void ToggleSelectedItemMovementLock();

	/**
	 * @return true if any selected item is marked as movement-locked in TEDS.
	 */
	UE_EXPERIMENTAL(5.8, "This API is subject to change.")
	UNREALED_API bool HasSelectedMovementLockedItems();

	/**
	 * Opens the objects specialized editor (Same as double clicking on the item in the content browser)
	 *
	 * @param ObjectToEdit	The object to open the editor for 
	 */
	UNREALED_API void EditObject( UObject* ObjectToEdit );

	/**
	 * Selects the currently selected actor(s) levels in the level browser
	 *
	 * @param bDeselectOthers	Whether or not to deselect the current level selection first
	 */
	UNREALED_API void SelectLevelInLevelBrowser( bool bDeselectOthers );

	/**
	 * Deselects the currently selected actor(s) levels in the level browser
	 */
	UNREALED_API void DeselectLevelInLevelBrowser();

	/**
	 * Selects all actors with the same class as the current selection
	 *
	 * @param bArchetype	true to only select actors of the same class AND same Archetype
	 */
	UNREALED_API void SelectAllActorsWithClass( bool bArchetype );

	/**
	 * Finds all references to the currently selected actors, and reports results in a find message log
	 */
	UNREALED_API void FindSelectedActorsInLevelScript();

	/** See if any selected actors are referenced in level script */
	UNREALED_API bool AreAnySelectedActorsInLevelScript();

	/**
	 * Checks if a provided package is valid to be auto-added to a default changelist based on several
	 * specifications (and if the user has elected to enable the feature). Only brand-new packages will be
	 * allowed.
	 *
	 * @param	InPackage	Package to consider the validity of auto-adding to a default source control changelist
	 * @param	InFilename	Filename of the package after it will be saved (not necessarily the same as the filename of the first param in the event of a rename)
	 *
	 * @return	true if the provided package is valid to be auto-added to a default source control changelist
	 */
	UNREALED_API bool IsPackageValidForAutoAdding(UPackage* InPackage, const FString& InFilename);

	/**
	 * Checks if a provided package is valid to be saved. For startup packages, it asks user for confirmation.
	 *
	 * @param	InPackage	Package to consider the validity of auto-adding to a default source control changelist
	 * @param	InFilename	Filename of the package after it will be saved (not necessarily the same as the filename of the first param in the event of a rename)
	 * @param	Error		Error output
	 *
	 * @return	true if the provided package is valid to be saved
	 */
	UNREALED_API virtual bool IsPackageOKToSave(UPackage* InPackage, const FString& InFilename, FOutputDevice* Error);

	/** The editor wrapper for UPackage::SavePackage. Auto-adds files to source control when necessary */
	UNREALED_API bool SavePackage(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename, const FSavePackageArgs& SaveArgs);

	/** The editor wrapper for UPackage::Save. Auto-adds files to source control when necessary */
	UNREALED_API FSavePackageResultStruct Save(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
		const FSavePackageArgs& SaveArgs);

	/** An async editor wrapper for UPackage::SaveAsync. Auto-adds files to source control when necessary
 	 *
 	 *  @note It is very important that any optional arguments passed via SaveArgs (i.e. ArchiveCookData, SavePackageContext) 
 	 *        stay valid for the entire duration of the task.
	 */
	UNREALED_API UE::Tasks::TTask<FSavePackageResultStruct> SaveAsync(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
		const FSavePackageArgs& SaveArgs);

	UNREALED_API virtual bool InitializePhysicsSceneForSaveIfNecessary(UWorld* World, bool &bOutForceInitialized);
	UNREALED_API void CleanupPhysicsSceneThatWasInitializedForSave(UWorld* World, bool bForceInitialized);

	/** Invoked before a UWorld is saved to update editor systems */
	UNREALED_API virtual void OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext);

	/** Invoked after a UWorld is saved to update editor systems */
	UNREALED_API virtual void OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/**
	 * Adds provided package to a default changelist
	 *
	 * @param	PackageNames	Filenames of the packages after they will be saved (not necessarily the same as the filename of the first param in the event of a rename)
	 */
	UNREALED_API void AddPackagesToDefaultChangelist(TArray<FString>& InPackageNames);

	/**
	 * Add the given file(s) to the deferred add list processed by RunDeferredMarkForAddFiles.
	 */
	UNREALED_API void AddDeferredMarkForAddFiles(TArrayView<const FString> Files);

	/**
	 * Will run a batch mark for add source control operation for files recently saved if source control is enabled
	 * @param	bool	Unused param to for the function to be used as a delegate when source control dialog is closed.
	 */
	UNREALED_API void RunDeferredMarkForAddFiles(bool = false);

	/** 
	 * @return - returns the currently selected positional snap grid setting
	 */
	UNREALED_API float GetGridSize() const;

	/** 
	 * @return - if the grid size is part of the 1,2,4,8,16,.. list or not
	 */
	UNREALED_API bool IsGridSizePowerOfTwo() const;

	/** 
	 * Sets the selected positional snap grid setting
	 *
	 * @param InIndex - The index of the selected grid setting
	 */
	UNREALED_API void SetGridSize(int32 InIndex);

	/** 
	 * Increase the positional snap grid setting (will not increase passed the maximum value)
	 */
	UNREALED_API void GridSizeIncrement();
	
	/** 
	 * Decreased the positional snap grid setting (will not decrease passed the minimum value)
	 */
	UNREALED_API void GridSizeDecrement();

	/** 
	 * Toggle on/off the ability of the Editor Viewport to apply a Constrained Aspect Ratio in the preview.
	 */
	UNREALED_API void ToggleAllowConstrainedAspectRatioInPreview();
	
	/** 
	 * @return  returns if the Editor allows Constrained Aspect Ratio in the preview
	 */
	UNREALED_API bool GetAllowConstrainedAspectRatioInPreview();

	/**
	 * Toggle on/off the Editor Flip of the Constrained Aspect Ratio in the preview.
	 */
	UNREALED_API void ToggleFlipConstrainedAspectRatioInPreview();

	/**
	 * @return  returns if the Editor Flips the Constrained Aspect Ratio in the preview
	 */
	UNREALED_API bool GetFlipConstrainedAspectRatioInPreview();

	/** 
	 * Accesses the array of snap grid position options
	 */
	UNREALED_API const TArray<float>& GetCurrentPositionGridArray() const;
	
	/** 
	 * @return - returns the currently selected rotational snap grid setting
	 */
	UNREALED_API FRotator GetRotGridSize();

	/** 
	 * Sets the selected Rotational snap grid setting
	 *
	 * @param InIndex - The index of the selected grid setting
	 * @param InGridMode - Selects which set of grid settings to use
	 */
	UNREALED_API void SetRotGridSize(int32 InIndex, enum ERotationGridMode InGridMode);

	/** 
	 * Increase the rotational snap grid setting (will not increase passed the maximum value)
	 */
	UNREALED_API void RotGridSizeIncrement();
	
	/** 
	 * Decreased the rotational snap grid setting (will not decrease passed the minimum value)
	 */
	UNREALED_API void RotGridSizeDecrement();
	
	/** 
	 * Accesses the array of snap grid rotation options
	 */
	UNREALED_API const TArray<float>& GetCurrentRotationGridArray() const;
	
	UNREALED_API float GetScaleGridSize();
	UNREALED_API void SetScaleGridSize(int32 InIndex);

	UNREALED_API float GetGridInterval();

	/**
	 * Get the location offset applied by the grid based on the grid size and active viewport.
	 */
	UNREALED_API FVector GetGridLocationOffset(bool bUniformOffset) const;

	/**
	 * Access the array of grid interval options
	 */
	UNREALED_API const TArray<float>& GetCurrentIntervalGridArray() const;
	
	 /**
	  * Function to convert selected brushes into volumes of the provided class.
	  *
	  * @param	VolumeClass	Class of volume that selected brushes should be converted into
	  */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ConvertSelectedBrushesToVolumes instead.")
	UNREALED_API void ConvertSelectedBrushesToVolumes( UClass* VolumeClass );

	/**
	 * Called to convert actors of one class type to another
	 *
	 * @param FromClass The class converting from
	 * @param ToClass	The class converting to
	 */
	UE_DEPRECATED(5.5, "Please use UEditorActorSubsystem::ConvertActorsFromClass instead.")
	UNREALED_API void ConvertActorsFromClass( UClass* FromClass, UClass* ToClass );

	/**
	* Update any outstanding reflection captures
	*/
	UNREALED_API void BuildReflectionCaptures(UWorld* World = GWorld);

	/**
	 * Convenience method for adding a Slate modal window that is parented to the main frame (if it exists)
	 * This function does not return until the modal window is closed.
	 */
	UNREALED_API void EditorAddModalWindow( TSharedRef<SWindow> InModalWindow ) const;

	/**
	 * Finds a brush builder of the provided class.  If it doesnt exist one will be created
	 *
	 * @param BrushBuilderClass	The class to get the builder brush from
	 * @return The found or created brush builder
	 */
	UNREALED_API UBrushBuilder* FindBrushBuilder( UClass* BrushBuilderClass );

	/**
	 * Parents one actor to another
	 * Check the validity of this call with CanParentActors().
	 *
	 * @param ParentActor	Actor to parent the child to
	 * @param ChildActor	Actor being parented
	 * @param SocketName	An optional socket name to attach to in the parent (use NAME_None when there is no socket)
	 * @param Component		Actual Component included in the ParentActor which is a usually blueprint actor
	 */
	UNREALED_API void ParentActors( AActor* ParentActor, AActor* ChildActor, const FName SocketName, USceneComponent* Component=NULL );

	/**
	 * Detaches selected actors from there parents
	 *
	 * @return				true if a detachment occurred
	 */
	UNREALED_API bool DetachSelectedActors();	

	/**
	 * Checks the validity of parenting one actor to another
	 *
	 * @param ParentActor	Actor to parent the child to
	 * @param ChildActor	Actor being parented
	 * @param ReasonText	Optional text to receive a description of why the function returned false.
	 * @return				true if the parenting action is valid and will succeed, false if invalid.
	 */
	UNREALED_API bool CanParentActors( const AActor* ParentActor, const AActor* ChildActor, FText* ReasonText = NULL);

	/** 
	 * turns all navigable static geometry of ULevel into polygon soup stored in passed Level (ULevel::StaticNavigableGeometry)
	 */
	UNREALED_API virtual void RebuildStaticNavigableGeometry(ULevel* Level);
	/**
	 * Gets all assets which can be synced to in content browser for current selection
	 *
	 * @param Assets	Array to be filled with assets which can be browsed to
	 * @param bAllowOverrideMetadata If true, allows an asset to define "BrowseToAssetOverride" in its metadata to sync to an asset other than itself
	 */
	UNREALED_API void GetAssetsToSyncToContentBrowser(TArray<FAssetData>& Assets, bool bAllowBrowseToAssetOverride = true);

	/**
	 * Gets all levels which can be synced to in content browser for current selection
	 *
	 * @param Objects	Array to be filled with ULevel objects
	 */
	UNREALED_API void GetLevelsToSyncToContentBrowser(TArray<UObject*>& Objects);

	/**
	* Queries for a list of assets that are referenced by the current editor selection (actors, surfaces, etc.)
	*
	* @param	Objects								Array to be filled with asset objects referenced by the current editor selection
	* @param	bIgnoreOtherAssetsIfBPReferenced	If true, and a selected actor has a Blueprint asset, only that will be returned.
	*/
	UNREALED_API void GetReferencedAssetsForEditorSelection(TArray<UObject*>& Objects, const bool bIgnoreOtherAssetsIfBPReferenced = false);

	/**
	* Queries for a list of assets that are soft referenced by the current editor selection (actors, surfaces, etc.)
	*
	* @param	SoftObjects							Array to be filled with asset objects referenced soft by the current editor selection
	*/
	UNREALED_API void GetSoftReferencedAssetsForEditorSelection(TArray<FSoftObjectPath>& SoftObjects);

	/** Returns a filter to restruct what assets show up in asset pickers based on what the selection is used for (i.e. what will reference the assets) */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IAssetReferenceFilter>, FOnMakeAssetReferenceFilter, const FAssetReferenceFilterContext& /*Context*/);
	FOnMakeAssetReferenceFilter& OnMakeAssetReferenceFilter() { return OnMakeAssetReferenceFilterDelegate; }
	TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const FAssetReferenceFilterContext& Context) { return OnMakeAssetReferenceFilterDelegate.IsBound() ? OnMakeAssetReferenceFilterDelegate.Execute(Context) : nullptr; }

	/** Returns a fixer to resolve illegal references between assets */
	DECLARE_DELEGATE_RetVal(TSharedPtr<IAssetReferenceFixer>, FOnMakeAssetReferenceFixer);
	FOnMakeAssetReferenceFixer& OnMakeAssetReferenceFixer() { return OnMakeAssetReferenceFixerDelegate; }
	TSharedPtr<IAssetReferenceFixer> MakeAssetReferenceFixer() { return OnMakeAssetReferenceFixerDelegate.IsBound() ? OnMakeAssetReferenceFixerDelegate.Execute() : nullptr; }

	DECLARE_DELEGATE_RetVal(ULevelEditorDragDropHandler*, FOnCreateLevelEditorDragDropHandler);
	FOnCreateLevelEditorDragDropHandler& OnCreateLevelEditorDragDropHandler() { return OnCreateLevelEditorDragDropHandlerDelegate; }
	UNREALED_API ULevelEditorDragDropHandler* GetLevelEditorDragDropHandler() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilterCopiedActors, TArray<AActor*>&);
	UE_INTERNAL FOnFilterCopiedActors& OnFilterCopiedActors() { return OnFilterCopiedActorsDelegate; }
	UE_INTERNAL const FOnFilterCopiedActors& OnFilterCopiedActors() const { return OnFilterCopiedActorsDelegate; }
	
	/** 
	 * Gets the interface to manage project references to external content
	 * @note the returned pointer cannot be null
	 */
	UNREALED_API IProjectExternalContentInterface* GetProjectExternalContentInterface();

	/** Delegate to override the IProjectExternalContentInterface */
	DECLARE_DELEGATE_RetVal(IProjectExternalContentInterface*, FProjectExternalContentInterfaceGetter);
	FProjectExternalContentInterfaceGetter ProjectExternalContentInterfaceGetter;

private:
	/**  Helper struct to cache the world package dirty state and reset the state for any packages dirtied during the scope. */
	struct FWorldScopedPackageDirtyGuard;

	UPROPERTY()
	mutable TObjectPtr<ULevelEditorDragDropHandler> DragDropHandler;

	FOnMakeAssetReferenceFilter OnMakeAssetReferenceFilterDelegate;
	FOnMakeAssetReferenceFixer OnMakeAssetReferenceFixerDelegate;
	FOnCreateLevelEditorDragDropHandler OnCreateLevelEditorDragDropHandlerDelegate;
	FOnFilterCopiedActors OnFilterCopiedActorsDelegate;

public:

	/** Returns the WorldContext for the editor world. For now, there will always be exactly 1 of these in the editor. 
	 *
	 * @param	bEnsureIsGWorld		Temporarily allow callers to validate that this maps to GWorld to help track down any conversion bugs
	 */
	UNREALED_API FWorldContext &GetEditorWorldContext(bool bEnsureIsGWorld = false);

	/** 
	 * Returns the WorldContext for the PIE world, by default will get the first one which will be the server or simulate instance.
	 * You need to iterate the context list if you want all the pie world contexts.
	 */
	UNREALED_API FWorldContext* GetPIEWorldContext(int32 WorldPIEInstance = 0);

	/** 
	 * mostly done to check if PIE is being set up, go GWorld is going to change, and it's not really _the_G_World_
	 * NOTE: hope this goes away once PIE and regular game triggering are not that separate code paths
	 */
	virtual bool IsSettingUpPlayWorld() const override { return EditorWorld != NULL && PlayWorld == NULL; }

	/**
	 *	Retrieves the active viewport from the editor.
	 *
	 *	@return		The currently active viewport.
	 */
	UNREALED_API FViewport* GetActiveViewport();

	/**
	 *	Retrieves the PIE viewport from the editor.
	 *
	 *	@return		The PIE viewport. NULL if there is no PIE viewport active.
	 */
	UNREALED_API FViewport* GetPIEViewport();

	/** 
	 *	Checks for any player starts and returns the first one found.
	 *
	 *	@return		The first player start location found.
	 */
	UNREALED_API APlayerStart* CheckForPlayerStart();

	/** Closes the popup created for GenericTextEntryModal or GenericTextEntryModeless*/
	UNREALED_API void CloseEntryPopupWindow();

	/**
	 * Prompts the user to save the current map if necessary, then creates a new (blank) map.
	 */
	UNREALED_API void CreateNewMapForEditing(bool bPromptUserToSave = true, bool bIsPartitionedWorld = false);

	/**
	 * If a PIE world exists, give the user the option to terminate it.
	 *
	 * @return				true if a PIE session exists and the user refused to end it, false otherwise.
	 */
	UNREALED_API bool ShouldAbortBecauseOfPIEWorld();

	/**
	 * If an unsaved world exists that would be lost in a map transition, give the user the option to cancel a map load.
	 *
	 * @return				true if an unsaved world exists and the user refused to continue, false otherwise.
	 */
	UNREALED_API bool ShouldAbortBecauseOfUnsavedWorld();

	/**
	 * Gets the user-friendly, localized (if exists) name of a property
	 *
	 * @param	Property	the property we want to try to et the friendly name of	
	 * @param	OwnerStruct	if specified, uses this class's loc file instead of the property's owner class
	 *						useful for overriding the friendly name given a property inherited from a parent class.
	 *
	 * @return	the friendly name for the property.  localized first, then metadata, then the property's name.
	 */
	static UNREALED_API FString GetFriendlyName( const FProperty* Property, UStruct* OwnerStruct = NULL );

	/**
	 * Register a client tool to receive undo events 
	 * @param UndoClient	An object wanting to receive PostUndo/PostRedo events
	 */
	UNREALED_API void RegisterForUndo(class FEditorUndoClient* UndoClient );

	/**
	 * Unregister a client from receiving undo events 
	 * @param UndoClient	An object wanting to unsubscribe from PostUndo/PostRedo events
	 */
	UNREALED_API void UnregisterForUndo( class FEditorUndoClient* UndoEditor );

	/**
	 * Ensures the assets specified are loaded and adds them to the global selection set
	 * @param Assets		An array of assets to load and select
	 * @param TypeOfAsset	An optional class to restrict the types of assets loaded & selected
	 */
	UNREALED_API void LoadAndSelectAssets( TArray<FAssetData>& Assets, UClass* TypeOfAsset=NULL );

	/**
	 * @return True if percentage based scaling is enabled
	 */
	UNREALED_API bool UsePercentageBasedScaling() const;

	/**
	 * Returns the actor grouping utility class that performs all grouping related tasks
	 * This will create the class instance if it doesn't exist.
	 */
	UNREALED_API class UActorGroupingUtils* GetActorGroupingUtils();

	/**
	 * Query to tell if the editor is currently in a mode where it wants XR HMD
	 * tracking to be used (like in the VR editor or VR PIE preview).
	 */
	UNREALED_API bool IsHMDTrackingAllowed() const;

private:
	//
	// Map execs.
	//

	UNREALED_API bool Map_Select( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Map_Brush( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Map_Sendto( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Map_Rebuild(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Map_Load(const TCHAR* Str, FOutputDevice& Ar);
	UNREALED_API bool Map_Import( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );

	struct EMapCheckNotification
	{
		enum Type
		{
			DontDisplayResults,
			DisplayResults,
			NotifyOfResults,
		};
	};

	/**
	 * Checks map for common errors.
	 */
	UNREALED_API bool Map_Check(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar, bool bCheckDeprecatedOnly, EMapCheckNotification::Type Notification = EMapCheckNotification::DisplayResults, bool bClearLog = true);
	UNREALED_API bool Map_Scale( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Map_Setbrush( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );

	/**
	 * Attempts to load a preview static mesh from the array preview static meshes at the given index.
	 *
	 * @param	Index	The index of the name in the PlayerPreviewMeshNames array from the editor user settings.
	 * @return	true if a static mesh was loaded; false, otherwise.
	 */
	UNREALED_API bool LoadPreviewMesh( int32 Index );

	/** Called when a recompilation of a module is beginning */
	UNREALED_API void OnModuleCompileStarted(bool bIsAsyncCompile);

	/** Called when a recompilation of a module is ending */
	UNREALED_API void OnModuleCompileFinished(const FString& CompilationOutput, ECompilationResult::Type CompilationResult, bool bShowLog);

public:
	/** Creates a pie world from the default entry map, used by clients that connect to a PIE server */
	UNREALED_API UWorld* CreatePIEWorldFromEntry(FWorldContext &WorldContext, UWorld* InWorld, FString &PlayWorldMapName);

	/** Called before creating PIE instance(s). */
	UNREALED_API virtual FGameInstancePIEResult PreCreatePIEInstances(const bool bAnyBlueprintErrors, const bool bStartInSpectatorMode, const float PIEStartTime, const bool bSupportsOnlinePIE, int32& InNumOnlinePIEInstances);

	/** Called before creating a PIE server instance. */
	UNREALED_API virtual FGameInstancePIEResult PreCreatePIEServerInstance(const bool bAnyBlueprintErrors, const bool bStartInSpectatorMode, const float PIEStartTime, const bool bSupportsOnlinePIE, int32& InNumOnlinePIEInstances);

	/*
	 * Handler for when viewport close request is made. 
	 *
	 * @param InViewport the viewport being closed.
	 */
	UNREALED_API void OnViewportCloseRequested(FViewport* InViewport);

	/*
	 * Fills level viewport client transform used in simulation mode 
	 *
	 * @param OutViewTransform filled view transform used in SIE
	 * @return true if function succeeded, false if failed
	 */
	UNREALED_API bool GetSimulateInEditorViewTransform(FTransform& OutViewTransform) const;
	
	/*
	 * Spawns a PlayFromHere playerstart in the given world
	 * @param	World		The World to spawn in (for PIE this may not be GWorld)
	 * @param	PlayerStartPIE	A reference to the resulting PlayerStartPIE actor
	 * @param	StartLocation	The location to spawn the player in.
	 * @param	StartRotation	The rotation to spawn the player at.
	 *
	 * @return	true if spawn succeeded, false if failed
	 */
	UNREALED_API bool SpawnPlayFromHereStart(UWorld* World, AActor*& PlayerStartPIE, const FVector& StartLocation, const FRotator& StartRotation);
	
	/**
	 * Spawns a PlayFromHere playerstart in the given world if there is a location set to spawn in.
	 * @param	World		The World to spawn in (for PIE this may not be GWorld)
	 * @param	PlayerStartPIE	A reference to the resulting PlayerStartPIE actor
	 *
	 * @return	true if spawn succeeded, false if failed. Returns true even if no object was spawned (due to location not being set).
	 */
	UNREALED_API bool SpawnPlayFromHereStart(UWorld* World, AActor*& PlayerStartPIE);


private:
	/**
	 * Delegate definition for the execute function that follows
	 */
	DECLARE_DELEGATE_OneParam( FSelectCommand, UModel* );
	DECLARE_DELEGATE_TwoParams( FSelectInWorldCommand, UWorld*, UModel* );
	
	/**
	 * Utility method call a select command for each level model in the worlds level list.
	 *
	 * @param InWorld			World containing the levels
	 * @param InSelectCommand	Select command delegate
	 */
	UNREALED_API void ExecuteCommandForAllLevelModels( UWorld* InWorld, FSelectCommand InSelectCommand, const FText& TransDesription );
	UNREALED_API void ExecuteCommandForAllLevelModels( UWorld* InWorld, FSelectInWorldCommand InSelectCommand, const FText& TransDesription );
	
public:

	/**
	 * Utility method call ModifySelectedSurfs for each level model in the worlds level list.
	 *
	 * @param InWorld			World containing the levels
	 */
	 UNREALED_API void FlagModifyAllSelectedSurfacesInLevels( UWorld* InWorld );
	 
private:
	/** Checks for UWorld garbage collection leaks and reports any that are found */
	UNREALED_API void CheckForWorldGCLeaks( UWorld* NewWorld, UPackage* WorldPackage );

	/**
	 * This destroys the given world.
	 * It also does editor specific tasks (EG clears selections from that world etc)
	 *
	 * @param	Context		The world to destroy
	 * @param	CleanseText	Reason for the destruction
	 * @param	NewWorld	An optional new world to keep in memory after destroying the world referenced in the Context.
	 *						This world and it's sublevels will remain in memory.
	 */
	UNREALED_API void EditorDestroyWorld( FWorldContext & Context, const FText& CleanseText, UWorld* NewWorld = nullptr );

	UNREALED_API ULevel* CreateTransLevelMoveBuffer( UWorld* InWorld );

	/**	Broadcasts that an undo or redo has just occurred. */
	UNREALED_API void BroadcastPostUndoRedo(const FTransactionContext& UndoContext, bool bWasUndo);

	/** Helper function to show undo/redo notifications */
	UNREALED_API void ShowUndoRedoNotification(const FText& NotificationText, bool bSuccess);

	/** Internal helper functions */
	UNREALED_API virtual void PostUndo (bool bSuccess);

	/** Delegate callback: the world origin is going to be moved. */
	UNREALED_API void PreWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin);

	/** Delegate callback for when a streaming level is added to world. */
	UNREALED_API void OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);

	/** Delegate callback for when a streaming level is removed from world. */
	UNREALED_API void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	/** Delegate callback when doing a seamless transition to a new world */
	void OnSeamlessTravelTransition(UWorld* InWorld);

	/** Puts the currently loaded project file at the top of the recents list and trims and files that fell off the list */
	UNREALED_API void UpdateRecentlyLoadedProjectFiles();

	/** Updates the project file to auto load and initializes the bLoadTheMostRecentlyLoadedProjectAtStartup flag */
	UNREALED_API void UpdateAutoLoadProject();

	/** Handles user setting changes. */
	UNREALED_API void HandleSettingChanged( FName Name );

	/** Callback for handling undo and redo transactions before they happen. */
	UNREALED_API void HandleTransactorBeforeRedoUndo(const FTransactionContext& TransactionContext);

	/** Common code for finished undo and redo transactions. */
	UNREALED_API void HandleTransactorRedoUndo(const FTransactionContext& TransactionContext, bool Succeeded, bool WasUndo);

	/** Callback for finished redo transactions. */
	UNREALED_API void HandleTransactorRedo(const FTransactionContext& TransactionContext, bool Succeeded);

	/** Callback for finished undo transactions. */
	UNREALED_API void HandleTransactorUndo(const FTransactionContext& TransactionContext, bool Succeeded);

public:
	/** Callback for object changes during undo/redo. */
	UNREALED_API void HandleObjectTransacted(UObject* InObject, const class FTransactionObjectEvent& InTransactionObjectEvent);

private:

	TPimplPtr<FEditorTransactionDeltaContext> CurrentUndoRedoContext;

	/** List of all viewport clients */
	TArray<class FEditorViewportClient*> AllViewportClients;

	/** List of level editor viewport clients for level specific actions */
	TArray<class FLevelEditorViewportClient*> LevelViewportClients;

	/** Delegate broadcast when the viewport client list changed */
	FViewportClientListChangedEvent ViewportClientListChangedEvent;

	/** Delegate broadcast when the level editor viewport client list changed */
	FViewportClientListChangedEvent LevelViewportClientListChangedEvent;

	/** Delegate broadcast just before a blueprint is compiled */
	FBlueprintPreCompileEvent BlueprintPreCompileEvent;

	/** Delegate broadcast when blueprint is compiled */
	FBlueprintCompiledEvent BlueprintCompiledEvent;

	/** Delegate broadcast when blueprint is reinstanced */
	FBlueprintReinstanced BlueprintReinstanced;

	/** Delegate broadcast when a package has been loaded or unloaded */
	FClassPackageLoadedOrUnloadedEvent ClassPackageLoadedOrUnloadedEvent;

	/** Delegate broadcast when an actor or component is about to be moved, rotated, or scaled  */
	FOnBeginTransformObject OnBeginObjectTransformEvent;

	/** Delegate broadcast when an actor or component has been moved, rotated, or scaled */
	FOnEndTransformObject OnEndObjectTransformEvent;

	/** Delegate broadcast when aactors have been moved, rotated, or scaled */
	FOnActorsMoved OnActorsMovedEvent;

	/** Delegate broadcast when the camera viewed through the viewport is about to be moved */
	FOnBeginTransformCamera OnBeginCameraTransformEvent;

	/** Delegate broadcast when the camera viewed through the viewport has been moved */
	FOnEndTransformCamera OnEndCameraTransformEvent;
	
	/** Broadcasts after an HLOD actor has been moved between clusters */
	FHLODActorMovedEvent HLODActorMovedEvent;

	/** Broadcasts after an HLOD actor's mesh is build*/
	FHLODMeshBuildEvent HLODMeshBuildEvent;

	/** Broadcasts after an HLOD actor has added to a cluster */
	FHLODActorAddedEvent HLODActorAddedEvent;

	/** Broadcasts after a Draw distance value (World settings) is changed */
	FHLODTransitionScreenSizeChangedEvent HLODTransitionScreenSizeChangedEvent;

	/** Broadcasts after the HLOD levels array is changed */
	FHLODLevelsArrayChangedEvent HLODLevelsArrayChangedEvent;

	/** Broadcasts after an Actor is removed from a cluster */
	FHLODActorRemovedFromClusterEvent HLODActorRemovedFromClusterEvent;

	/** Broadcasts after an Exec event on particles has been invoked.*/
	FExecParticleInvoked ExecParticleInvokedEvent; 

	/** Broadcasts to allow selection of unloaded actors */
	FSelectUnloadedActorsEvent SelectUnloadedActorsEvent;

	/** Reference to owner of the current popup */
	TWeakPtr<class SWindow> PopupWindow;

	/** True if we should disable actor/component modification on delta movement */
	bool bDisableDeltaModification;

	/** True if we allow Constrained Aspect Ratio in the Preview */
	bool bAllowConstrainedAspectRatioInPreview;

	/** True if we Flip the Constrained Aspect Ratio in the Preview */
	bool bFlipConstrainedAspectRatioInPreview;

	/** List of editors who want to receive undo/redo events */
	TSet< class FEditorUndoClient* > UndoClients;

	/** A temporary copy used to add resilience during the undo/redo broadcast. Used to allow unregistering of clients while the broadcast is happening.
	 * DO NOT add to this set while a broadcast is happening. */
	TArray<class FEditorUndoClient*> InflightUndoClients;

	/** The notification item to use for undo/redo */
	TSharedPtr<class SNotificationItem> UndoRedoNotificationItem;

	/** The Timer manager for all timer delegates */
	TSharedPtr<class FTimerManager> TimerManager;

	/** Currently active function execution world switcher, will be null most of the time */
	FScopedConditionalWorldSwitcher* FunctionStackWorldSwitcher = nullptr;

	/** Stack entry where world switcher was created, and should be destroyed at */
	int32 FunctionStackWorldSwitcherTag = -1;

	/** Delegate handles for function execution */
	FDelegateHandle ScriptExecutionStartHandle, ScriptExecutionEndHandle;

	// This chunk is used for Play In New Process
public:
	/**
	 * Are we playing on a local PC session?
	 */
	bool IsPlayingOnLocalPCSession() const { return PlayOnLocalPCSessions.Num() > 0; }
	
	/** 
	 * Launch a new process with settings pulled from the specified parameters.
	 * @param InParams 				Shared parameters to launch with.
	 * @param InInstanceNum 		Which instance index is this? Used to do load settings per-instance if needed.
	 * @param NetMode				What net mode should this process launch under?
	 * @param bIsDedicatedServer 	Is this instance a dedicated server? Overrides NetMode.
	 */
	UNREALED_API virtual void LaunchNewProcess(const FRequestPlaySessionParams& InParams, const int32 InInstanceNum, EPlayNetMode NetMode, bool bIsDedicatedServer);

	// This chunk is used for Play In New Process
protected:
	struct FPlayOnPCInfo
	{
		FProcHandle ProcessHandle;
	};

	/** List of New Process sessions that are currently active, started by this Editor instance. */
	TArray<FPlayOnPCInfo> PlayOnLocalPCSessions;

	/** Start a Play in New Process session with the given parameters. Called by StartQueuedPlaySessionRequestImpl based on request settings. */
	UNREALED_API virtual void StartPlayInNewProcessSession(FRequestPlaySessionParams& InRequestParams);

	/** Does the actual late join process. Don't call this directly, use RequestLateJoin() which will defer it to the next frame. */
	UNREALED_API void AddPendingLateJoinClient();
	
	// This chunk is used for Play Via Launcher settings.
public:
	/**
	 * Are we playing via the Launcher?
	 */
	bool IsPlayingViaLauncher() const { return LauncherSessionInfo.IsSet(); }

	/**
	 * Cancel playing via the Launcher
	 */
	UNREALED_API void CancelPlayingViaLauncher();

	/**
	* Returns the last device that was used via the Launcher.
	*/
	UNREALED_API FString GetPlayOnTargetPlatformName() const;

	/** 
	* Cancel Play using Launcher on error 
	* 
	* if the physical device is not authorized to be launched to, we need to pop an error instead of trying to launch
	*/
	UNREALED_API void CancelPlayUsingLauncher();

protected:
	struct FLauncherCachedInfo
	{
		FString PlayUsingLauncherDeviceName;
		bool bPlayUsingLauncherHasCode;
		bool bPlayUsingLauncherBuild;
	};

	/** 
	* If set, there is a active attempt to use the Launcher. Launching is an async process so we store the request info for the duration.
	* of the launch. This is cleared on cancel (or on finish).
	*/
	TOptional<FLauncherCachedInfo> LauncherSessionInfo;

	/** 
	* The last platform we ran on (as selected by the drop down or via automation)
	* Stored outside of the Session Info as the UI needs access to this at all times.
	*/
	FString LastPlayUsingLauncherDeviceId;
	
	/** Start a Launcher session with the given parameters. Called by StartQueuedPlaySessionRequestImpl based on request settings. */
	UNREALED_API virtual void StartPlayUsingLauncherSession(FRequestPlaySessionParams& InRequestParams);

	/** This flag is used to skip UAT\UBT compilation on every launch if it was successfully compiled once. */
	bool bUATSuccessfullyCompiledOnce;

	// This chunk is for Play in Editor
public:
	/** @return true if the editor is able to launch PIE with online platform support */
	UNREALED_API bool SupportsOnlinePIE() const;

	/** @return true if there are active PIE instances logged into an online platform */
	bool IsPlayingWithOnlinePIE() const
	{
		return PlayInEditorSessionInfo.IsSet() && PlayInEditorSessionInfo->bUsingOnlinePlatform;
	}

protected:

	/** Specifies whether or not there has been a Play Session requested to be kicked off on the next Tick. */
	TOptional<FRequestPlaySessionParams> PlaySessionRequest;
	
	/** If set, transient information about the current Play in Editor session which persists until the session stops. */
	TOptional<FPlayInEditorSessionInfo> PlayInEditorSessionInfo;
	
	/** Used to prevent reentrant calls to EndPlayMap(). */
	bool bIsEndingPlay;

protected:
	DECLARE_DELEGATE(FPIEInstanceWindowSwitch);

	/** Sets the delegate for when the focused PIE window is changed */
	UNREALED_API void SetPIEInstanceWindowSwitchDelegate(FPIEInstanceWindowSwitch PIEInstanceWindowSwitchDelegate);

	/** Gets the scene viewport for a viewport client */
	UNREALED_API FSceneViewport* GetGameSceneViewport(UGameViewportClient* ViewportClient) const;

	/**
	 * Toggles PIE to SIE or vice-versa
	 */
	UNREALED_API void ToggleBetweenPIEandSIE(bool bNewSession = false);

	/**
	 * Hack to switch worlds for the PIE window before and after a slate event
	 *
	 * @param WorldID	The id of the world to switch to where -1 is unknown, 0 is editor, and 1 is PIE
	 * @param PIEInstance	When switching to a PIE instance, this is the specific client/server instance to use
	 * @return The ID of the world to restore later or -1 if no world to restore
	 */
	UNREALED_API int32 OnSwitchWorldForSlatePieWindow(int32 WorldID, int32 WorldPIEInstance);

	/**
	 * Called via a delegate to toggle between the editor and pie world
	 */
	UNREALED_API void OnSwitchWorldsForPIE(bool bSwitchToPieWorld, UWorld* OverrideWorld = nullptr);

	/**
	 * Called to switch to a specific PIE instance, where -1 means the editor world
	 */
	UNREALED_API void OnSwitchWorldsForPIEInstance(int32 WorldPIEInstance);

	/** Call to enable/disable callbacks for PIE world switching when PIE starts/stops */
	UNREALED_API void EnableWorldSwitchCallbacks(bool bEnable);

	/** Callback when script execution starts, might switch world */
	UNREALED_API void OnScriptExecutionStart(const struct FBlueprintContextTracker& ContextTracker, const UObject* ContextObject, const UFunction* ContextFunction);

	/** Callback when script execution starts, might switch world */
	UNREALED_API void OnScriptExecutionEnd(const struct FBlueprintContextTracker& ContextTracker);

	/**
	 * Gives focus to the server or first PIE client viewport
	 */
	UNREALED_API void GiveFocusToLastClientPIEViewport();

	/**
	 * Delegate called as each PIE instance login is complete, continues creating the PIE world for a single instance
	 *
	 * @param LocalUserNum local user id, for PIE is going to be 0 (there is no splitscreen)
	 * @param bWasSuccessful was the login successful
	 * @param ErrorString descriptive error when applicable
	 * @param DataStruct data required to continue PIE creation, set at login time
	 */
	UNREALED_API virtual void OnLoginPIEComplete(int32 LocalUserNum, bool bWasSuccessful, const FString& ErrorString, FPieLoginStruct DataStruct);

	/** Above function but called a frame later, to stop PIE login from happening from a network callback */
	UNREALED_API virtual void OnLoginPIEComplete_Deferred(int32 LocalUserNum, bool bWasSuccessful, FString ErrorString, FPieLoginStruct DataStruct);

	/** allow for game specific override to determine if login should be treated as successful for pass-through handling instead */
	virtual bool IsLoginPIESuccessful(int32 LocalUserNum, bool bWasSuccessful, const FString& ErrorString, const FPieLoginStruct& DataStruct) { return bWasSuccessful; }

	/** Called when all PIE instances have been successfully logged in */
	UNREALED_API virtual void OnAllPIEInstancesStarted();
	
	/** Backs up the current editor selection and then clears it. Optionally reselects the instances in the Play world. */
	UNREALED_API void TransferEditorSelectionToPlayInstances(const bool bInSelectInstances);
	
	/** Create a new GameInstance for PIE with the specified parameters. */
	UNREALED_API virtual UGameInstance* CreateInnerProcessPIEGameInstance(FRequestPlaySessionParams& InParams, const FGameInstancePIEParameters& InPIEParameters, int32 InPIEInstanceIndex);
	
	/** Creates a SPIEViewport for the specified instance. This is used for Play in New Window, but not for instances that live in the level editor viewport. */
	UNREALED_API TSharedRef<class SPIEViewport> GeneratePIEViewportWindow(const FRequestPlaySessionParams& InSessionParams, int32 InViewportIndex, const FWorldContext& InWorldContext, EPlayNetMode InNetMode, UGameViewportClient* InViewportClient, FSlatePlayInEditorInfo& InSlateInfo);

	/** Stores the position of the window for that instance index. Doesn't save it to the CDO until the play session ends. */
	UNREALED_API void StoreWindowSizeAndPositionForInstanceIndex(const int32 InInstanceIndex, const FIntPoint& InSize, const FIntPoint& InPosition);
	/** Returns the stored position at that index. */
	UNREALED_API void GetWindowSizeAndPositionForInstanceIndex(ULevelEditorPlaySettings& InEditorPlaySettings, const int32 InInstanceIndex, const FWorldContext& InWorldContext, FIntPoint& OutSize, FIntPoint& OutPosition);

	/** Start the queued Play Session Request. After this is called the queued play session request will be cleared. */
	UNREALED_API virtual void StartQueuedPlaySessionRequestImpl();
	
	/** Start a Play in Editor session with the given parameters. Called by StartQueuedPlaySessionRequestImpl based on request settings. */
	UNREALED_API virtual void StartPlayInEditorSession(FRequestPlaySessionParams& InRequestParams);

	/** 
	* Creates a new Play in Editor instance (which may be in a new process if not running under one process.
	* This reads the current session state to start the next instance needed.
	* @param	InRequestParams			- Mostly used for global settings. Probably shouldn't vary per instance.
	* @param	bInDedicatedInstance	- If true, this will create a dedicated server instance. Shouldn't be used if we're using multiple processes.
	* @param	InNetMode				- The net mode to launch this instance with. Not pulled from the RequestParams because some net modes (such as ListenServer)
										  need different net modes for subsequent instances.
	*/
	UNREALED_API virtual void CreateNewPlayInEditorInstance(FRequestPlaySessionParams &InRequestParams, const bool bInDedicatedInstance, const EPlayNetMode InNetMode);


	/** Asks the player to save dirty maps, if this fails it will return false. */
	UNREALED_API bool SaveMapsForPlaySession();
	
protected:

	FPIEInstanceWindowSwitch PIEInstanceWindowSwitchDelegate;

	/** Cached version of the view location at the point the PIE session was ended */
	FVector LastViewLocation;

	/** Cached version of the view location at the point the PIE session was ended */
	FRotator LastViewRotation;

	/** Are the lastview/rotation variables valid */
	bool bLastViewAndLocationValid;

	/** The output log -> message log redirector for use during PIE */
	TSharedPtr<class FOutputLogErrorsToMessageLogProxy> OutputLogErrorsToMessageLogProxyPtr;

private:
	/** List of files we are deferring adding to source control */
	TArray<FString> DeferredFilesToAddToSourceControl;

	/** Ongoing failure notification */
	TWeakPtr<class SNotificationItem> SourceControlFailureNotificationPtr;
	void OnDismissSourceControlFailureNotification();

protected:

	/**
	 * Invalidates all editor viewports and hit proxies, used when global changes like Undo/Redo may have invalidated state everywhere
	 */
	UNREALED_API void InvalidateAllViewportsAndHitProxies();

	/** Initialize Portal RPC. */
	UNREALED_API void InitializePortal();

	/** Destroy any online subsystems generated by PIE */
	UNREALED_API void CleanupPIEOnlineSessions(TArray<FName> OnlineIdentifiers);

	// launch on callbacks
	UNREALED_API void HandleStageStarted(const FString& InStage, TWeakPtr<SNotificationItem> NotificationItemPtr);
	UNREALED_API void HandleStageCompleted(const FString& InStage, double StageTime, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr);
	UNREALED_API void HandleLaunchCanceled(double TotalTime, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr);
	UNREALED_API void HandleLaunchCompleted(bool Succeeded, double TotalTime, int32 ErrorCode, bool bHasCode, TWeakPtr<SNotificationItem> NotificationItemPtr);

	// Handle requests from slate application to open assets.
	UNREALED_API bool HandleOpenAsset(UObject* Asset);

	// Handle requests from slate application to open debug objects.
	UNREALED_API bool HandleOpenDebugObject(UObject* DebugObject);

private:
	/** Handler for when any asset is loaded in the editor */
	UNREALED_API void OnAssetLoaded( UObject* Asset );

	/** Handler for when an asset is created (used to detect world duplication after PostLoad) */
	UNREALED_API void OnAssetCreated( UObject* Asset );

	/** Handler for when an asset finishes compiling (used to notify AssetRegistry to update tags after Load) */
	UNREALED_API void OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets);

	/** Handler for when the timecode provider changed, so that we can register the BlueprintCompiled delegate to it.*/
	UNREALED_API void RegisterTimecodeProviderCompiledDelegate();

	/** Handler for when the custom timestep changed, so that we can register the BlueprintCompiled delegate to it.*/
	UNREALED_API void RegisterCustomTimeStepCompiledDelegate();

	/** Handler used to reinitialize Timecode when their blueprint is compiled. */
	UNREALED_API void HandleTimecodeProviderCompiled(UBlueprint* InBlueprint);

	/** Handler used to reinitialize CustomTimeStep when their blueprint is compiled. */
	UNREALED_API void HandleCustomTimeStepCompiled(UBlueprint* InBlueprint);

	/** Handler for when a world is duplicated in the editor */
	UNREALED_API void InitializeNewlyCreatedInactiveWorld(UWorld* World);

	/** Gets the init values for worlds opened via Map_Load in the editor */
	UNREALED_API UWorld::InitializationValues GetEditorWorldInitializationValues() const;

	/** Logic that needs to run on change of effective shader platform (can happen via both SetPreviewPlatform and TogglePreviewPlatform). */
	UNREALED_API void OnEffectivePreviewShaderPlatformChange();

public:
	// Launcher Worker
	TSharedPtr<class ILauncherWorker> LauncherWorker;
	
	/** Load a map for automation testing */
	UNREALED_API void AutomationLoadMap(const FString& MapName, bool bForceReload, FString* OutError);

	/** This function should be called to notify the editor that new materials were added to our scene or some materials were modified */
	UNREALED_API void OnSceneMaterialsModified();

	/** Called when a scene material is added or modified. */
	DECLARE_EVENT(UEditorEngine, FSceneMaterialsModifiedEvent);
	FSceneMaterialsModifiedEvent& OnSceneMaterialsModifiedEvent() { return SceneMaterialsModifiedEvent; }

	/** Call this function to change the feature level and to override the material quality platform of the editor and PIE worlds */
	UNREALED_API void SetPreviewPlatform(const FPreviewPlatformInfo& NewPreviewPlatform, bool bSaveSettings, bool bCompileShaders = true);

	/** Toggle the preview platform */
	UNREALED_API void TogglePreviewPlatform();

	/** Return whether the preview platform is enabled for use via the user accessing Settings->Preview Rendering Level. */
	UNREALED_API bool IsPreviewPlatformEnabled() const;

	/** Return whether the preview platform enabled via Settings->Preview Rendering Level is currently active/displaying. */
	UNREALED_API bool IsPreviewPlatformActive() const;

	UE_DEPRECATED(5.8, "Use TogglePreviewPlatform instead")
	void ToggleFeatureLevelPreview() { TogglePreviewPlatform(); }
	UE_DEPRECATED(5.8, "Use IsPreviewPlatformEnabled instead")
	bool IsFeatureLevelPreviewEnabled() const { return IsPreviewPlatformEnabled(); }
	UE_DEPRECATED(5.8, "Use IsPreviewPlatformActive instead")
	bool IsFeatureLevelPreviewActive() const { return IsPreviewPlatformActive(); }

	/** Return the current active shader platform in the editor*/
	UNREALED_API EShaderPlatform GetActiveShaderPlatform() const;

	/** Return the current active shader platform friendly name in the editor*/
	UNREALED_API FText GetActiveShaderPlatformFriendlyName() const;

	/**
	 * Return the active feature level. This will be the chosen Settings->Preview Rendering Level if the Preview Mode button
	 * is highlighted or SM5 if the Preview Mode button has been clicked and isn't highlighted.
	 */
	UNREALED_API ERHIFeatureLevel::Type GetActiveFeatureLevelPreviewType() const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/** Return the delegate that is called when the preview feature level changes */
	UE_DEPRECATED(5.8, "OnPreviewFeatureLevelChanged is deprecated, use OnPreviewShaderPlatformChanged instead")
	FPreviewFeatureLevelChanged& OnPreviewFeatureLevelChanged() { return PreviewFeatureLevelChanged; }
	/** Return the delegate that is called when the preview platform changes */
	UE_DEPRECATED(5.8, "OnPreviewPlatformChanged is deprecated, use OnPreviewShaderPlatformChanged instead")
	FPreviewPlatformChanged& OnPreviewPlatformChanged() { return PreviewPlatformChanged; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Return the delegate that is called when the preview shader platform changes */
	FPreviewShaderPlatformChanged& OnPreviewShaderPlatformChanged() { return PreviewShaderPlatformChanged; }

	/** Return the delegate that is called when the bugitgo command is called */
	FPostBugItGoCalled& OnPostBugItGoCalled() { return PostBugItGoCalled; }

protected:

	/** Function pair used to save and restore the global Shader Platform */
	UE_DEPRECATED(5.7, "LoadEditorFeatureLevel is deprecated, please use LoadEditorShaderPlatformPreview")
	UNREALED_API void LoadEditorFeatureLevel();
	UE_DEPRECATED(5.7, "SaveEditorFeatureLevel is deprecated, please use LoadEditorShaderPlatformPreview")
	UNREALED_API void SaveEditorFeatureLevel();

	UNREALED_API void LoadEditorShaderPlatformPreview();
	UNREALED_API void SaveEditorShaderPlatformPreview();

	/** Utility function that can determine whether some input world is using materials who's shaders are emulated in the editor */
	UNREALED_API bool IsEditorShaderPlatformEmulated(UWorld* World);

	/** Utility function that checks if, for the current shader platform used by the editor, there's an available offline shader compiler */
	UNREALED_API bool IsOfflineShaderCompilerAvailable(UWorld* World);

protected:

	UPROPERTY(EditAnywhere, config, Category = Advanced, meta = (MetaClass = "/Script/UnrealEd.ActorGroupingUtils"))
	FSoftClassPath ActorGroupingUtilsClassName;

	UPROPERTY()
	TObjectPtr<class UActorGroupingUtils> ActorGroupingUtils;
private:
	FTimerHandle CleanupPIEOnlineSessionsTimerHandle;
	
	FDelegateHandle WorldAddExtraDeletionObjectsHandle;

	/** Delegate handle for game viewport close requests in PIE sessions. */
	FDelegateHandle ViewportCloseRequestedDelegateHandle;

	/** Delegate handle for when a TimecodeProvider blueprint is compiled. */
	FDelegateHandle TimecodeProviderCompiledDelegateHandle;

	/** Delegate handle for when a CustomTimeStep blueprint is compiled. */
	FDelegateHandle CustomTimeStepCompiledDelegateHandle;

	/** Minimized Windows during PIE */
	TArray<TWeakPtr<SWindow>> MinimizedWindowsDuringPIE;

	/** Delegate broadcast when a scene material is added or modified. */
	FSceneMaterialsModifiedEvent SceneMaterialsModifiedEvent;

public:
	/**
	 * Get a Subsystem of specified type
	 */
	UEditorSubsystem* GetEditorSubsystemBase(TSubclassOf<UEditorSubsystem> SubsystemClass) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		checkSlow(IsThisNotNull(this, "UEditorEngine::GetEditorSubsystemBase"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return EditorSubsystemCollection.GetSubsystem<UEditorSubsystem>(SubsystemClass);
	}

	/**
	 * Get a Subsystem of specified type
	 */
	template <typename TSubsystemClass>
	TSubsystemClass* GetEditorSubsystem() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		checkSlow(IsThisNotNull(this, "UEditorEngine::GetEditorSubsystem"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return EditorSubsystemCollection.GetSubsystem<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Get all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	TArray<TSubsystemClass*> GetEditorSubsystemArrayCopy() const
	{
		return EditorSubsystemCollection.GetSubsystemArrayCopy<TSubsystemClass>(TSubsystemClass::StaticClass());
	}

	/**
	 * Performs an operation on all Subsystems of specified type, this is only necessary for interfaces that can have multiple implementations instanced at a time.
	 */
	template <typename TSubsystemClass>
	void ForEachEditorSubsystem(TFunctionRef<void(TSubsystemClass*)> Operation) const
	{
		static_assert(TIsDerivedFrom<TSubsystemClass, UEngineSubsystem>::IsDerived, "TSubsystemClass must be derived from UEditorSubsystem");
		return EditorSubsystemCollection.ForEachSubsystem([Operation=MoveTemp(Operation)](UEditorSubsystem* Subsystem){
			Operation(CastChecked<TSubsystemClass>(Subsystem));
		}, TSubsystemClass::StaticClass());
	}

private:
	FObjectSubsystemCollection<UEditorSubsystem> EditorSubsystemCollection;

	void BeginEditorSave(struct FEditorSaveContext& Context, struct FSlowTask* SlowTask);
	void EndEditorSave(struct FEditorSaveContext& Context, struct FSlowTask* SlowTask);

	// DEPRECATED VARIABLES ONLY
public:
	/** True if we're Simulating In Editor, as opposed to Playing In Editor.  In this mode, simulation takes place right the level editing environment */
	// UE_DEPRECATED(4.25, "Use IsSimulateInEditorInProgress instead.")
	UPROPERTY()
	uint32 bIsSimulatingInEditor:1;
};

//////////////////////////////////////////////////////////////////////////
// FActorLabelUtilities

struct FActorLabelUtilities
{
public:
	/**
	 * Given a label, attempts to split this into its alpha/numeric parts.
	 *
	 * @param	InOutLabel	The label to start with, this will only be modified if it ends in a number.
	 * @param	OutIdx		The number which the string ends with, if any.
	 *
	 * @return	true if the label ends with a number.
	 */
	static UE_API bool SplitActorLabel(FString& InOutLabel, int32& OutIdx);

	/**
	 * Assigns a new label to an actor. If the name exists it will be appended with a number to make it unique. Actor labels are only available in development builds.
	 *
	 * @param	Actor					The actor to change the label of
	 * @param	NewActorLabel			The new label string to assign to the actor.  If empty, the actor will have a default label.
	 * @param	InExistingActorLabels	(optional) Pointer to a set of actor labels that are currently in use
	 */
	static UE_API void SetActorLabelUnique(AActor* Actor, const FString& NewActorLabel, const FCachedActorLabels* InExistingActorLabels = nullptr);

	/** 
	 * Does an explicit actor rename. In addition to changing the label this will also fix any soft references pointing to it 
	 * 
	 * @param	Actor					The actor to change the label of
	 * @param	NewActorLabel			The new label string to assign to the actor.  If empty, the actor will have a default label.
	 * @param	bMakeUnique				If true, it will call SetActorLabelUnique, if false it will use the exact label specified
	 */
	static UE_API void RenameExistingActor(AActor* Actor, const FString& NewActorLabel, bool bMakeUnique = false);

private:
	FActorLabelUtilities() {}
};

#undef UE_API
