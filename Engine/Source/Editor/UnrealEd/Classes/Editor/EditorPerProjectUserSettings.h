// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Settings/EditorViewportSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EditorPerProjectUserSettings.generated.h"

UENUM()
enum class EGlobalTabSwitchingBehavior : uint8
{
	/** Legacy behavior where last accessed asset defines order of switching */
	LastAccessedAsset,
	/** Same behavior as browser where window tab order defines order of switching */
	BrowserStyle
};

UCLASS(minimalapi, autoexpandcategories=(ViewportControls, ViewportLookAndFeel, LevelEditing, SourceControl, Content, Startup), hidecategories=(Object, Options, Grid, RotationGrid), config=EditorPerProjectUserSettings)
class UEditorPerProjectUserSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// =====================================================================
	// The following options are exposed in the Preferences Editor 

	/** If enabled, any newly opened UI menus, menu bars, and toolbars will show the developer hooks that would accept extensions */
	UPROPERTY(EditAnywhere, config, Category=DeveloperTools, meta = (DisplayName = "Display UI Extension Points", ConfigRestartRequired=true))
	uint32 bDisplayUIExtensionPoints:1;

	/** If enabled, tooltips linked to documentation will show the developer the link bound to that UI item */
	UPROPERTY(EditAnywhere, config, Category=DeveloperTools)
	uint32 bDisplayDocumentationLink:1;

	/** If enabled, tooltips on SGraphPaletteItems will show the associated action's string id */
	UPROPERTY(/*EditAnywhere - deprecated (moved into UBlueprintEditorSettings), */config/*, Category=DeveloperTools*/)
	uint32 bDisplayActionListItemRefIds:1;

	/** If enabled, behavior tree debugger will collect its data even when all behavior tree editor windows are closed */
	UPROPERTY(EditAnywhere, config, Category = AI)
	uint32 bAlwaysGatherBehaviorTreeDebuggerData : 1;

	/** If enabled, blackboard keys displayed in blackboard editor and key selector will be sorted in alphabetical order . */
	UPROPERTY(EditAnywhere, config, Category = AI)
	uint32 bDisplayBlackboardKeysInAlphabeticalOrder : 1;

	/** When enabled, Engine Version Number is displayed in the ProjectBadge */
	UPROPERTY(EditAnywhere, config, Category = DeveloperTools, meta = (DisplayName = "Display Engine Version Number in Project Badge", ConfigRestartRequired = true))
	bool bDisplayEngineVersionInBadge;

	/** When enabled, use SimplygonSwarm Module / server to create proxies */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Use Simplygon distributed proxy server"))
	bool bUseSimplygonSwarm;

	/** Server IP for the distributed Simplygon server */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Simplygon distributed proxy server IP", editcondition = "bUseSimplygonSwarm"))
	FString SimplygonServerIP;
	
	/** Enable swarm debugging features. Temp ssf files are not removed. Detailed message printing */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Enable Swarm Debugging ", editcondition = "bUseSimplygonSwarm"))
	bool bEnableSwarmDebugging;
	
	/** Time between JSON net requests for Simplygon Swarm */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Time in (MS). Delay between json request (Default 5000ms)", editcondition = "bUseSimplygonSwarm", ClampMin = "5000", ClampMax = "30000", UIMin = "5000", UIMax = "30000"))
	uint32 SimplygonSwarmDelay;

	/** Number of concurrent swarm jobs to execute. This is independent of the main job queue. */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Number of concurrent jobs to submit on simplygon grid server", editcondition = "bUseSimplygonSwarm", ClampMin = "4", ClampMax = "512", UIMin = "4", UIMax = "512"))
	uint32 SwarmNumOfConcurrentJobs;

	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Max upload size of simplygon swarm zip (MB). File larger than max size will be split into chunks.", editcondition = "bUseSimplygonSwarm", ClampMin = "10", ClampMax = "2000", UIMin = "10", UIMax = "2000"))
	uint32 SwarmMaxUploadChunkSizeInMB;
	
	/** Folder in which Simplygon Swarm will store intermediate texture and mesh data that is uploaded to the Swarm */
	UPROPERTY(EditAnywhere, config, Category = SimplygonSwarm, meta = (DisplayName = "Simplygon Swarm Intermediate Folder", ConfigRestartRequired = true, editcondition = "bUseSimplygonSwarm"))
	FString SwarmIntermediateFolder;

	/** If enabled, any newly added classes will be automatically compiled and trigger a hot-reload of the module they were added to */
	UPROPERTY(EditAnywhere, config, Category=HotReload, meta=(DisplayName="Automatically Compile Newly Added C++ Classes"))
	uint32 bAutomaticallyHotReloadNewClasses:1;

	/** If enabled, the compile message log window will open if there is a compiler error on Hot Reload */
	UPROPERTY(EditAnywhere, config, Category=HotReload)
	uint32 bShowCompilerLogOnCompileError : 1;

	/** If enabled, the fbx parser will keep the fbx namespaces, otherwise the namespace will be append to fbx node. */
	UPROPERTY(EditAnywhere, config, Category = Import)
	uint32 bKeepFbxNamespace : 1;

	/** If enabled, the fbx option dialog will show when user re-import a fbx */
	UPROPERTY(EditAnywhere, config, Category = Import)
	uint32 bShowImportDialogAtReimport : 1;

	/** Specify a project data source folder to store relative source file path to ease the re-import process*/
	UPROPERTY(EditAnywhere, config, Category = Import)
	FDirectoryPath DataSourceFolder;

	/** If enabled, export level with attachment hierarchy set */
	UPROPERTY(EditAnywhere, config, Category=Export)
	uint32 bKeepAttachHierarchy:1;

	/** If enabled, will compare an animation's sequence length and curves against the old data and inform the user if something changed */
	UPROPERTY(EditAnywhere, config, Category = Import)
	uint32 bAnimationReimportWarnings: 1;

	/** If enabled, you'll be asked to confirm if you really want to close the editor. */
	UPROPERTY(EditAnywhere, Config, Category = Behavior)
	uint32 bConfirmEditorClose: 1;

	/** Defines global tab switching behavior for the editor */
	UPROPERTY(EditAnywhere, Config, Category = Behavior)
	EGlobalTabSwitchingBehavior TabSwitchingBehavior = EGlobalTabSwitchingBehavior::BrowserStyle;

	/** Whether to show or hide tab switching dialog when tab command is executed */
	UPROPERTY(EditAnywhere, Config, Category = Behavior)
	bool bShowTabSwitchingDialog = true;

	/** Select to make Distributions use the curves, not the baked lookup tables. */
	UPROPERTY(config)
	uint32 bUseCurvesForDistributions:1; //(GDistributionType == 0)

	/** If enabled, failed ensures will be displayed in the message log */
	UPROPERTY(EditAnywhere, config, Category = Ensures)
	uint32 bShowEnsuresInMessageLog : 1;

	/** Controls the minimum value at which the property matrix editor will display a loading bar when pasting values */
	UPROPERTY(config)
	int32 PropertyMatrix_NumberOfPasteOperationsBeforeWarning;

	UPROPERTY(config)
	bool bSCSEditorShowGrid;

	UPROPERTY(config)
	bool bSCSEditorShowFloor;

	/** If enabled, the Editor will attempt to get the users attention whenever a UAT task (such as cooking or packaging) is completed */
	UPROPERTY(EditAnywhere, config, Category = UnrealAutomationTool)
	bool bGetAttentionOnUATCompletion;

	/** Always build UAT\UBT before launching the game. It will decrease iteration times if disabled */
	UPROPERTY(EditAnywhere, config, Category = UnrealAutomationTool)
	bool bAlwaysBuildUAT = true;

	/** Camera Speed settings for the SCS viewport */
	UPROPERTY(config)
	FEditorViewportCameraSpeedSettings SCSViewportCameraSpeedSettings;

	/** How fast the SCS viewport camera moves */
	UPROPERTY(config, meta=(UIMin = "1", UIMax = "8", ClampMin="1", ClampMax="8"))
	int32 SCSViewportCameraSpeed_DEPRECATED;

	// Color curve:
	//   Release->Attack happens instantly
	//   Attack holds for AttackHoldPeriod, then
	//   Decays from Attack to Sustain for DecayPeriod with DecayExponent, then
	//   Sustain holds for SustainHoldPeriod, then
	//   Releases from Sustain to Release for ReleasePeriod with ReleaseExponent
	//
	// The effective time at which an event occurs is it's most recent exec time plus a bonus based on the position in the execution trace


	// =====================================================================
	// The following options are NOT exposed in the preferences Editor
	// (usually because there is a different way to set them interactively!)

	/** Controls whether packages which are checked-out are automatically fully loaded at startup */
	UPROPERTY(config)
	uint32 bAutoloadCheckedOutPackages:1;

	/** If this is true, the user will not be asked to fully load a package before saving or before creating a new object */
	UPROPERTY(config)
	uint32 bSuppressFullyLoadPrompt:1;

	/** True if user should be allowed to select translucent objects in perspective viewports */
	UPROPERTY(config)
	uint32 bAllowSelectTranslucent:1;

	/** If this is true, all of an actors' components will be drawn when the actor or one of its component is selected */
	UPROPERTY(config)
	uint32 bShowSelectionSubcomponents:1;

	UPROPERTY()
	TObjectPtr<class UBlueprintPaletteFavorites> BlueprintFavorites;
	
public:
	// Per project user settings for which asset viewer profile should be used
	UPROPERTY(Config)
	int32 AssetViewerProfileIndex;

	/** The feature level we should use when loading or creating a new world */
	UPROPERTY(config)
	int32 PreviewFeatureLevel;

	/** The platform name to preview, or NAME_None if there is no preview platform */
	UPROPERTY(config)
	FName PreviewPlatformName;

	/** The shader format to preview, or NAME_None if there is no preview platform */
	UPROPERTY(config)
	FName PreviewShaderFormatName;

	/** The shader platform to preview, or NAME_None if there is no preview platform */
	UPROPERTY(config)
	FName PreviewShaderPlatformName;
	
	/** The shader platforms friendly Name, or Empty Text if there is no preview platform or no Frienly Name */
	UPROPERTY(config)
	FText PreviewShaderPlatformFriendlyName;

	/** Constrained Aspect Ratio of the preview platform */
	UPROPERTY(config)
	float PreviewConstrainedAspectRatio;

	/** Safe Zones of the preview platform */
	UPROPERTY(config)
	FVector4f PreviewSafeZones;

	/** Is shader platform preview currently active */
	UPROPERTY(config)
	bool bPreviewShaderPlatformActive;

	/** Deprecated: renamed to bPreviewShaderPlatformActive */
	UPROPERTY(config, meta=(DeprecatedProperty, DeprecationMessage="bPreviewFeatureLevelActive has been renamed to bPreviewShaderPlatformActive."))
	bool bPreviewFeatureLevelActive = false;

	UPROPERTY(config)
	bool bPreviewFeatureLevelWasDefault;

	UPROPERTY(config)
	FName PreviewDeviceProfileName;

public:

	/** Delegate for when a user setting has changed */
	DECLARE_EVENT_OneParam(UEditorPerProjectUserSettings, FUserSettingChangedEvent, FName /*PropertyName*/);
	FUserSettingChangedEvent& OnUserSettingChanged() { return UserSettingChangedEvent; }

	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) override;
#endif
	virtual void PostInitProperties() override;
	//~ End UObject Interface

private:

	FUserSettingChangedEvent UserSettingChangedEvent;
};

