// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/DeveloperSettings.h"

#include "EditorPerformanceSettings.generated.h"


// Mode for the computation of the screen percentage (r.ScreenPercentage.Mode).
UENUM()
enum class EEditorUserScreenPercentageModeOverride
{
	// Uses the project's default configured in the project settings.
	ProjectDefault,

	// Directly controls the screen percentage with the r.ScreenPercentage cvar
	Manual UMETA(DisplayName = "Manual"),

	// Automatic control the screen percentage based on the display resolution, r.ScreenPercentage.Auto.*
	BasedOnDisplayResolution UMETA(DisplayName = "Based on display resolution"),

	// Based on DPI scale.
	BasedOnDPIScale UMETA(DisplayName = "Based on operating system's DPI scale"),
};


UCLASS(minimalapi, config=EditorSettings, meta=(DisplayName = "Performance", ToolTip="Settings to tweak the performance of the editor"))
class UEditorPerformanceSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/** Whether to enable Editor Perf Tool Completely */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Enable Tool", ConfigRestartRequired = true))
	bool bEnableEditorPeformanceTool = true;

	/** Whether to enable Editor Perf Warnings */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Enable Notifications", ConfigRestartRequired = false, EditCondition = "bEnableEditorPeformanceTool"))
	bool bEnableNotifications = false;

	/** Specific measurements to notify */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (ConfigRestartRequired = false, EditCondition = "bEnableEditorPeformanceTool"))
	TArray<FName> NotifyList;

	/** Specific measurements to notify */
	UPROPERTY(config, meta = (DisplayName = "Notification List", ConfigRestartRequired = false))
	TArray<FName> NotificationList_DEPRECATED;

	/** Whether to enable Telemetry */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Enable Telemetry", ConfigRestartRequired = false, EditCondition = "bEnableEditorPeformanceTool"))
	bool bEnableTelemetry = true;

	/** Whether to create Insights Snapshots */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Enable Snapshots", ConfigRestartRequired = false, EditCondition = "bEnableEditorPeformanceTool"))
	bool bEnableSnapshots = false;

	/** Whether to Show Warnings Only */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Show Warnings Only", ConfigRestartRequired = false, EditCondition = "bEnableEditorPeformanceTool"))
	bool bShowWarningsOnly = false;

	/** Whether to enable Editor Perf Tool Experimental Features */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformanceTool, meta = (DisplayName = "Experimental Features", ConfigRestartRequired = true, EditCondition = "bEnableEditorPeformanceTool"))
	bool bEnableExperimentalFeatures = false;

	/** When enabled, the application frame rate, memory and Unreal object count will be displayed in the main editor UI */
	UPROPERTY(EditAnywhere, config, Category=EditorPerformance)
	uint32 bShowFrameRateAndMemory:1;

	/** Lowers CPU usage when the editor is in the background and not the active application */
	UPROPERTY(EditAnywhere, config, Category=EditorPerformance, meta=(DisplayName="Use Less CPU when in Background") )
	uint32 bThrottleCPUWhenNotForeground:1;

	/** When enabled, the editor throttles background processing (world ticking, viewport updates) during UI interactions to keep menus and panels responsive. Disable this to keep viewports and simulations running at full speed while interacting with the editor. */
	UPROPERTY(EditAnywhere, config, Category=EditorPerformance, meta=(DisplayName="Allow Editor (Slate) Throttling", ConsoleVariable="Slate.bAllowThrottling"))
	uint32 bAllowSlateThrottling:1;

	/** Disables realtime viewports by default when connected via a remote session */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformance, meta = (DisplayName = "Disable realtime viewports by default in Remote Sessions"))
	uint32 bDisableRealtimeViewportsInRemoteSessions : 1;

	/** When turned on, the editor will constantly monitor performance and adjust scalability settings for you when performance drops (disabled in debug) */
	UPROPERTY(EditAnywhere, config, Category=EditorPerformance)
	uint32 bMonitorEditorPerformance:1;

	/** When enabled, Shared Data Cache performance notifications may be displayed when not connected to a shared cache */
	UE_DEPRECATED(5.2, "No longer supported")
	uint32 bEnableSharedDDCPerformanceNotifications : 1;

	/** When enabled, a warning will appear in the viewport when your editors scalability settings are non-default and you may be viewing a low quality scene */
	UPROPERTY(EditAnywhere, config, Category = EditorPerformance, meta = (DisplayName = "Enable Scalability Warning Indicator"))
	uint32 bEnableScalabilityWarningIndicator : 1;
	
	/** Should VSync be enabled in editor? */
	UPROPERTY(EditAnywhere, config, Category=EditorPerformance, meta=(DisplayName="Enable VSync", ConsoleVariable="r.VSyncEditor"))
	uint32 bEnableVSync : 1;
	
	/** 
	 * By default the editor will adjust scene scaling (quality) for high DPI in order to ensure consistent performance with very large render targets.
	 * Enabling this will disable automatic adjusting and render at the full resolution of the viewport
	 */
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Displays viewports' at high DPI",
		ConsoleVariable="r.Editor.Viewport.HighDPI"))
	uint32 bDisplayHighDPIViewports : 1;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage settings with editor viewports' settings in PIE",
		ConsoleVariable="r.Editor.Viewport.OverridePIEScreenPercentage"))
	uint32 bOverridePIEScreenPercentage : 1;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage mode for realtime editor viewports using desktop renderer."))
	EEditorUserScreenPercentageModeOverride RealtimeScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage mode for realtime editor viewports using mobile renderer."))
	EEditorUserScreenPercentageModeOverride MobileScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage mode for VR editor viewports."))
	EEditorUserScreenPercentageModeOverride VRScreenPercentageMode;
	
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage mode for path-traced editor viewports."))
	EEditorUserScreenPercentageModeOverride PathTracerScreenPercentageMode;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's default screen percentage mode for non-realtime editor viewports."))
	EEditorUserScreenPercentageModeOverride NonRealtimeScreenPercentageMode;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's manual screen percentage"))
	uint32 bOverrideManualScreenPercentage : 1;

	/** Editor viewport screen percentage */
	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		EditCondition="bOverrideManualScreenPercentage",
		DisplayName="Editor viewport screen percentage."))
	float ManualScreenPercentage;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's minimum viewport rendering resolution"))
	uint32 bOverrideMinViewportRenderingResolution : 1;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		EditCondition="bOverrideMinViewportRenderingResolution",
		DisplayName="Minimum default rendering resolution to use for editor viewports."))
	int32 MinViewportRenderingResolution;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		DisplayName="Override project's maximum viewport rendering resolution"))
	uint32 bOverrideMaxViewportRenderingResolution : 1;

	UPROPERTY(EditAnywhere, config, Category=ViewportResolution, meta=(
		EditCondition="bOverrideMaxViewportRenderingResolution",
		DisplayName="Maximum default rendering resolution to use for editor viewports."))
	int32 MaxViewportRenderingResolution;

public:
	/** UObject interface */
	virtual void PostInitProperties() override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

};

