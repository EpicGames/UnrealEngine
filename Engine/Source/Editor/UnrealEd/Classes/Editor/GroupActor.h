// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// GroupActor: Collects a group of actors, allowing for management and universal transformation.
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "GroupActor.generated.h"

class FLevelEditorViewportClient;
class FViewport;
class UTypedElementSelectionSet;

UCLASS(hidedropdown,MinimalAPI, notplaceable, NotBlueprintable)
class AGroupActor : public AActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint32 bLocked:1;

	UPROPERTY()
	TArray<TObjectPtr<class AActor>> GroupActors;

	UPROPERTY()
	TArray<TObjectPtr<class AGroupActor>> SubGroups;

#endif // WITH_EDITORONLY_DATA


	//~ Begin UObject Interface
	virtual void PostLoad() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	virtual bool IsEditorOnly() const override { return true; }
	//~ End UObject Interface

	//~ Begin AActor Interface
	virtual void PostActorCreated() override;
	virtual void SetIsTemporarilyHiddenInEditor( bool bIsHidden ) override;
	virtual void GetActorBounds(bool bOnlyCollidingComponents, FVector& Origin, FVector& BoxExtent, bool bIncludeFromChildActors) const override;
#if WITH_EDITOR
	virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
#endif
	//~ End AActor Interface

	// @todo document
	virtual bool IsSelected() const;

	/**
	 * Enumerate all actors and subgroups for this group (includes this).
	 * @param InCallback		The function to call for each actor
	 */
	UNREALED_API void ForEachActorInGroup(TFunctionRef<void(AActor*, AGroupActor*)> InCallback);

	/**
	 * Enumerate all actors and subgroups for this group that should be moved alongside this actor (includes this).
	 * @param InSelectionSet	The selection set to use, or null to use the global selection set
	 * @param InCallback		The function to call for each movable actor
	 */
	UNREALED_API void ForEachMovableActorInGroup(const UTypedElementSelectionSet* InSelectionSet, TFunctionRef<void(AActor*, AGroupActor*)> InCallback);
	
	/**
	 * Apply given deltas to all actors and subgroups for this group.
	 * @param	InDrag			Delta Transition
	 * @param	InRot			Delta Rotation
	 * @param	InScale			Delta Scale
	 */
	UNREALED_API void GroupApplyDelta(const FVector& InDrag, const FRotator& InRot, const FVector& InScale );

	/**
	 * Draw brackets around all selected groups
	 * @param	PDI				FPrimitiveDrawInterface used to draw lines in active viewports
	 * @param	Viewport		The viewport to draw brackets in.
	 * @param	bMustBeSelected	Flag to only draw currently selected groups. Defaults to true.
	 */
	UNREALED_API static void DrawBracketsForGroups( class FPrimitiveDrawInterface* PDI, FViewport* Viewport, bool bMustBeSelected=true );

	/**
	 * Changes the given array to remove any existing subgroups
	 * @param	GroupArray	Array to remove subgroups from
	 */
	UNREALED_API static void RemoveSubGroupsFromArray(TArray<AGroupActor*>& GroupArray);
	
	/**
	 * Returns the highest found root for the given actor or null if one is not found. Qualifications of root can be specified via optional parameters.
	 * @param	InActor			Actor to find a group root for.
	 * @param	bMustBeLocked	Flag designating to only return the topmost locked group.
	 * @param	bMustBeSelected	Flag designating to only return the topmost selected group.
	 * @param	bMustBeUnlocked		Flag designating to only return the topmost unlocked group.
	 * @param	bMustBeUnselected	Flag designating to only return the topmost unselected group.
	 * @return	The topmost group actor for this actor. Returns null if none exists using the given conditions.
	 */
	UNREALED_API static AGroupActor* GetRootForActor(AActor* InActor, bool bMustBeLocked=false, bool bMustBeSelected=false, bool bMustBeUnlocked=false, bool bMustBeUnselected=false);

	/**
	 * Returns the direct parent for the actor or null if one is not found.
	 * @param	InActor	Actor to find a group parent for.
	 * @return	The direct parent for the given actor. Returns null if no group has this actor as a child.
	 */
	static AGroupActor* GetParentForActor(AActor* InActor);

	/**
	 * Query to find how many active groups are currently in the editor.
	 * @param	bSelected	Flag to only return currently selected groups (defaults to false).
	 * @param	bDeepSearch	Flag to do a deep search when checking group selections (defaults to true).
	 * @return	Number of active groups currently in the editor.
	 */
	UNREALED_API static const int32 NumActiveGroups( bool bSelected=false, bool bDeepSearch=true );

	/**
	 * Adds selected ungrouped actors to a selected group. Does nothing if more than one group is selected.
	 */
	static void AddSelectedActorsToSelectedGroup();

	/**
	 * Locks the lowest selected groups in the current selection.
	 */
	static void LockSelectedGroups();

	/**
	 * Unlocks the highest locked parent groups for actors in the current selection.
	 */
	static void UnlockSelectedGroups();
	
	/**
	 * Toggle group mode
	 */
	UNREALED_API static void ToggleGroupMode();

	/**
	 * Reselects any valid groups based on current editor selection
	 */
	static void SelectGroupsInSelection();
	
	/**
	 * Lock this group and all subgroups.
	 */
	void Lock();
	
	/**
	 * Unlock this group
	 */
	FORCEINLINE void Unlock()
	{
		bLocked = false;
	};
	
	/**
	 * @return	Group's locked state
	 */
	FORCEINLINE bool IsLocked() const
	{
		return bLocked;
	};

	/**
	 * @param	InActor	Actor to add to this group
	 */
	UNREALED_API void Add(AActor& InActor);
	
	/**
	 * Removes the given actor from this group. If the group has no actors after this transaction, the group itself is removed.
	 * @param	InActor	Actor to remove from this group
	 */
	UNREALED_API void Remove(AActor& InActor);

	/**
	 * @param InActor	Actor to search for
	 * @return True if the group contains the given actor.
	 */
	UNREALED_API bool Contains(AActor& InActor) const;

	/**
	 * @param bDeepSearch	Flag to check all subgroups as well. Defaults to true.
	 * @return True if the group contains any selected actors.
	 */
	bool HasSelectedActors(bool bDeepSearch=true) const;

	/**
	 * Detaches all children (actors and subgroups) from this group and then removes it.
	 */
	void ClearAndRemove();

	/**
	 * Sets this group's location to the center point based on current location of its children.
	 */
	UNREALED_API void CenterGroupLocation();
	
	/**
	 * @param	OutGroupActors	Array to fill with all actors for this group.
	 * @param	bRecurse		Flag to recurse and gather any actors in this group's subgroups.
	 */
	UNREALED_API void GetGroupActors(TArray<AActor*>& OutGroupActors, bool bRecurse=false) const;

	/**
	 * @param	OutSubGroups	Array to fill with all subgroups for this group.
	 * @param	bRecurse	Flag to recurse and gather any subgroups in this group's subgroups.
	 */
	void GetSubGroups(TArray<AGroupActor*>& OutSubGroups, bool bRecurse=false) const;

	/**
	 * @param	OutChildren	Array to fill with all children for this group.
	 * @param	bRecurse	Flag to recurse and gather any children in this group's subgroups.
	 */
	UNREALED_API void GetAllChildren(TArray<AActor*>& OutChildren, bool bRecurse=false) const;

	/**
	 * @return The number of actors in the group.
	 */
	int32 GetActorNum() const;

private:
	/**
	 * Helper function for Remove() and ClearAndRemove() - cleans up actors when grouping is no longer needed
	 */
	void PostRemove();

public:
	static UNREALED_API bool SelectedGroupNeedsFixup();
	static UNREALED_API void FixupGroupActor();
};

namespace GroupActorHelpers
{

UNREALED_API bool ActorHasParentInGroup(const AActor* Actor, const AGroupActor* GroupActor);
UNREALED_API bool ActorHasParentInSelection(const AActor* Actor, FTypedElementListConstRef SelectionSet);

} // namespace GroupActorHelpers
