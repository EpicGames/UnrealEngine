// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Misc/Timecode.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/ScriptInterface.h"
#include "UObject/SoftObjectPath.h"
#include "Blueprint/UserWidget.h"
#include "Engine/BlendableInterface.h"
#include "Engine/EngineTypes.h"

#include "Curves/RichCurve.h"
#include "UObject/PerPlatformProperties.h"

#include "PropertyEditorTestObject.generated.h"

class AActor;
class IAnimClassInterface;
class ITableRow;
class STableViewBase;
class SWidget;
class UMaterialInterface;
class UPrimitiveComponent;
class UStaticMesh;
class UStaticMeshComponent;
class UTexture;
template <typename ItemType> class SListView;

UENUM()
enum EPropertyEditorTestEnum : int
{	
	/** This comment should appear above enum 1 */
	PropertyEditorTest_Enum1 UMETA(Hidden),
	/** This comment should appear above enum 2 */
	PropertyEditorTest_Enum2,
	/** This comment should appear above enum 3 */
	PropertyEditorTest_Enum3 UMETA(Hidden),
	/** This comment should appear above enum 4 */
	PropertyEditorTest_Enum4,
	/** This comment should appear above enum 5 */
	PropertyEditorTest_Enum5 UMETA(Hidden),
	/** This comment should appear above enum 6 */
	PropertyEditorTest_Enum6,
	PropertyEditorTest_MAX,
};

UENUM(meta=(Bitflags))
enum class EPropertyEditorTestBitflags : uint8
{
	First,
	Second,
	Third,
	Hidden UMETA(Hidden, ToolTip="This value shouldn't be used or even visible in the editor")
};
ENUM_CLASS_FLAGS(EPropertyEditorTestBitflags)

UENUM()
enum ArrayLabelEnum : int
{
	ArrayIndex0,
	ArrayIndex1,
	ArrayIndex2,
	ArrayIndex3,
	ArrayIndex4,
	ArrayIndex5,
	ArrayIndex_MAX,
};

UENUM()
enum class EPropertyEditorTestEditColor : uint8
{
	Red,
	Orange,
	Yellow,
	Green,
	Blue,
	Indigo UMETA(Hidden),
	Violet,
	Pink,
	Magenta,
	Cyan
};

UENUM()
enum class EPropertyEditorTestUnderscores : uint8
{
	_One,
	_Two,
	_Three,
	NotUnderscore
};

UENUM()
enum class ETestEnumFlags : uint8
{
	None = 0,
	One = 1 << 0,
	Two = 1 << 1,
	Four = 1 << 2
};

USTRUCT()
struct FPropertyEditTestTextStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=TextStruct)
	FText NormalProperty;
};

USTRUCT()
struct FPropertyEditorTestSubStruct
{
	GENERATED_BODY()

	FPropertyEditorTestSubStruct()
		: FirstProperty( 7897789 )
		, SecondProperty( 342432432 )
		, CustomizedStructInsideUncustomizedStruct(ForceInitToZero)
	{
	}

	UPROPERTY(EditAnywhere, Category=PropertyEditorTestSubStruct)
	int32 FirstProperty;

	UPROPERTY(EditAnywhere, Category=PropertyEditorTestSubStruct)
	int32 SecondProperty;

	UPROPERTY(EditAnywhere, Category=PropertyEditorTestSubStruct)
	FLinearColor CustomizedStructInsideUncustomizedStruct;

	UPROPERTY(EditAnywhere, Category=PropertyEditorTestSubStruct)
	FSoftObjectPath CustomizedStructInsideUncustomizedStruct2;
};

/**
 * This structs properties should be pushed out to categories inside its parent category unless it is in an array
 */
USTRUCT()
struct FPropertyEditorTestBasicStruct
{
	GENERATED_BODY()

	FPropertyEditorTestBasicStruct()
		: IntPropertyInsideAStruct( 0 )
		, FloatPropertyInsideAStruct( 0.0f )
		, ObjectPropertyInsideAStruct( nullptr )
		, InnerStruct()
	{
	}

	UPROPERTY(EditAnywhere, Category=InnerStructCategoryWithPushedOutProps)
	int32 IntPropertyInsideAStruct;

	UPROPERTY(EditAnywhere, Category=InnerStructCategoryWithPushedOutProps)
	float FloatPropertyInsideAStruct;

	UPROPERTY(EditAnywhere, Category=InnerStructCategoryWithPushedOutProps)
	TObjectPtr<UObject> ObjectPropertyInsideAStruct;

	UPROPERTY(EditAnywhere, Category=InnerStructCategoryWithPushedOutProps)
	FPropertyEditorTestSubStruct InnerStruct;
}; 

USTRUCT()
struct FPropertyEditorTestEditCondition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default, meta=(InlineEditConditionToggle))
	bool InlineEditCondition = false;

	UPROPERTY(EditAnywhere, Category=Default, meta=(EditCondition="InlineEditCondition"))
	int32 HasInlineEditCondition = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	ETestEnumFlags Flags = ETestEnumFlags::None;

	UPROPERTY(EditAnywhere, Category=Default, meta=(EditCondition="Flags == ETestEnumFlags::One", EditConditionHides))
	int32 EnabledAndVisibleWhenOne = 0;
};

UCLASS(EditInlineNew, Abstract)
class UPropertyEditorTestInstancedObject : public UObject
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	int32 Number;
};

UCLASS()
class UFirstDerivedPropertyEditorTestObject : public UPropertyEditorTestInstancedObject
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Foo)
	FString String;
};

UCLASS()
class USecondDerivedPropertyEditorTestObject : public UPropertyEditorTestInstancedObject
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Bar)
	bool Bool;
};

USTRUCT()
struct FPropertyEditorTestInstancedStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Instanced, Category=Default)
	TObjectPtr<UPropertyEditorTestInstancedObject> Object { nullptr };
};

UCLASS(transient, BlueprintType, EditInlineNew)
class UPropertyEditorTestObject : public UObject
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	int8 Int8Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	int16 Int16Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	int32 Int32Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	int64 Int64Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	uint8 ByteProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	uint16 UnsignedInt16Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	uint32 UnsignedInt32Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	uint64 UnsignedInt64Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	float FloatProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	double DoubleProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FName NameProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	bool BoolProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FString StringProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FText TextProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties, meta=(AllowPreserveRatio))
	FIntPoint IntPointProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties, meta=(AllowPreserveRatio))
	FVector Vector3Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties, meta=(AllowPreserveRatio))
	FVector2D Vector2Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties, meta=(AllowPreserveRatio))
	FVector4 Vector4Property;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FRotator RotatorProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	TObjectPtr<UObject> ObjectProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FLinearColor LinearColorProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FColor ColorProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	TEnumAsByte<enum EPropertyEditorTestEnum> EnumByteProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	EPropertyEditorTestEditColor EnumProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	EPropertyEditorTestUnderscores EnumUnderscores;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FMatrix MatrixProperty;

	UPROPERTY(EditAnywhere, Category=BasicProperties)
	FTransform TransformProperty;

	UPROPERTY(EditAnywhere, Category=Units, meta=(ForceUnits="GB"))
	double GigabyteProperty;

	UPROPERTY(EditAnywhere, Category=Classes)
	TObjectPtr<UClass> ClassProperty;

	UPROPERTY(EditAnywhere, Category=Classes, meta=(AllowedClasses="/Script/Engine.Texture2D"))
	TObjectPtr<UClass> ClassPropertyWithAllowed;

	UPROPERTY(EditAnywhere, Category=Classes, meta=(DisallowedClasses="/Script/Engine.Texture2D"))
	TObjectPtr<UClass> ClassPropertyWithDisallowed;

	UPROPERTY(EditAnywhere, Category=Classes)
	TSubclassOf<UTexture> SubclassOfTexture;

	UPROPERTY(EditAnywhere, Category=Classes, meta=(AllowedClasses="/Script/Engine.Texture2D"))
	TSubclassOf<UTexture> SubclassOfWithAllowed;

	UPROPERTY(EditAnywhere, Category=Classes, meta=(DisallowedClasses="/Script/Engine.Texture2D"))
	TSubclassOf<UTexture> SubclassOfWithDisallowed;

	UPROPERTY(EditAnywhere, Category=Classes, meta=(AllowedClasses="/Script/Engine.StaticMesh,  /Script/Engine.SkeletalMesh	"))
	TSoftObjectPtr<UObject> AssetPointerWithAllowedAndWhitespace;

	// Integer
	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<int32> IntProperty32Array;

	// Byte
	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<uint8> BytePropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<float> FloatPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FName> NamePropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<bool> BoolPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FString> StringPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FText> TextPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FVector> Vector3PropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FVector2D> Vector2PropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FVector4> Vector4PropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FRotator> RotatorPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<TObjectPtr<UObject>> ObjectPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<TObjectPtr<AActor>> ActorPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FLinearColor> LinearColorPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FColor> ColorPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FTimecode> TimecodePropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<TEnumAsByte<EPropertyEditorTestEnum> > EnumPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FPropertyEditorTestBasicStruct> StructPropertyArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties, meta=(TitleProperty=IntPropertyInsideAStruct))
	TArray<FPropertyEditorTestBasicStruct> StructPropertyArrayWithTitle;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties, meta=(TitleProperty="{IntPropertyInsideAStruct} + {FloatPropertyInsideAStruct}"))
	TArray<FPropertyEditorTestBasicStruct> StructPropertyArrayWithFormattedTitle;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties, meta=(TitleProperty=ErrorProperty))
	TArray<FPropertyEditorTestBasicStruct> StructPropertyArrayWithTitleError;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties, meta=(TitleProperty="{ErrorProperty}"))
	TArray<FPropertyEditorTestBasicStruct> StructPropertyArrayWithFormattedTitleError;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	TArray<FPropertyEditorTestInstancedStruct> InstancedStructArray;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties, Instanced, meta=(TitleProperty="Number"))
	TArray<TObjectPtr<UPropertyEditorTestInstancedObject>> ObjectPropertyArrayWithTitle;

	UPROPERTY(EditAnywhere, Instanced, Category=ArraysOfProperties)
	TArray<TObjectPtr<UPropertyEditorTestInstancedObject>> InstancedUObjectArray;

	UPROPERTY(EditAnywhere, editfixedsize, Category=ArraysOfProperties)
	TArray<int32> FixedArrayOfInts;

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	int32 StaticArrayOfInts[5];

	UPROPERTY(EditAnywhere, Category=ArraysOfProperties)
	int32 StaticArrayOfIntsWithEnumLabels[ArrayIndex_MAX];

	UPROPERTY(VisibleAnywhere, Category = AdvancedProperties)
	FFloatRange FloatRange;

	// This is a float property tooltip that is overridden
	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(ClampMin="0.0", ClampMax="100.0", UIMin="0.0", UIMax="50.0", ToolTip="This is a custom tooltip that should be shown"))
	float FloatPropertyWithClampedRange;

	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(ClampMin="0", ClampMax="100", UIMin="0", UIMax="50" ))
	int32 IntPropertyWithClampedRange;

	UPROPERTY(VisibleAnywhere, Category=AdvancedProperties)
	int32 IntThatCannotBeChanged;

	UPROPERTY(VisibleAnywhere, Category=AdvancedProperties)
	FString StringThatCannotBeChanged;

	UPROPERTY(VisibleAnywhere, Category=AdvancedProperties)
	TObjectPtr<UPrimitiveComponent> ObjectThatCannotBeChanged;

	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(Bitmask, BitmaskEnum="/Script/UnrealEd.EPropertyEditorTestBitflags"))
	int32 EnumBitflags=0;

	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(PasswordField=true))
	FString StringPasswordProperty;

	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(PasswordField=true))
	FText TextPasswordProperty;

	UPROPERTY(EditAnywhere, Category=SingleStruct, meta=(ShowOnlyInnerProperties))
	FPropertyEditorTestBasicStruct ThisIsBrokenIfItsVisibleInADetailsView;

	UPROPERTY(EditAnywhere, Category=StructTests)
	FPropertyEditorTestBasicStruct StructWithMultipleInstances1;

	UPROPERTY(EditAnywhere, Category=StructTests, meta=(InlineEditConditionToggle))
	bool bEditConditionStructWithMultipleInstances2;

	UPROPERTY(EditAnywhere, Category=StructTests, meta=(EditCondition="bEditConditionStructWithMultipleInstances2"))
	FPropertyEditorTestBasicStruct StructWithMultipleInstances2;

	UPROPERTY(EditAnywhere, Category=StructTests)
	FRichCurve RichCurve;

	UPROPERTY(EditAnywhere, Category=Assets)
	FSoftObjectPath SoftObjectPath;

	UPROPERTY(EditAnywhere, Category=Assets)
	FPrimaryAssetId PrimaryAssetId;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(DisplayThumbnail=false))
	FPrimaryAssetId PrimaryAssetIdWithoutThumbnail;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(DisplayThumbnail="true"))
	FSoftObjectPath AssetReferenceCustomStructWithThumbnail;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.PointLight", ExactClass))
	FSoftObjectPath ExactlyPointLightActorReference;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.Light"))
	FSoftObjectPath LightActorReference;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.PointLight, /Script/Engine.SpotLight", ExactClass=true))
	FSoftObjectPath ExactPointOrSpotLightActorReference;

	// NOTE: intentionally misplaced space in AllowedClasses
	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.Light ,/Script/Engine.StaticMeshActor", DisplayThumbnail))
	FSoftObjectPath LightOrStaticMeshActorReference;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.Actor", DisallowedClasses="/Script/Engine.Light"))
	FSoftObjectPath NotLightActorReference;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(AllowedClasses="/Script/Engine.Material,/Script/Engine.Texture"))
	FSoftObjectPath MaterialOrTextureAssetReference;

	UPROPERTY(EditAnywhere, Category=Assets, meta=(MetaClass="/Script/Engine.Actor"))
	FSoftObjectPath ActorWithMetaClass;

	UPROPERTY(EditAnywhere, Category=Assets)
	FSoftObjectPath DisabledByCanEditChange;

	UPROPERTY(EditAnywhere, Category=Assets)
	FComponentReference ComponentReference;

	UPROPERTY(EditAnywhere, Category=StructTests, meta=(InlineEditConditionToggle))
	bool bEditCondition;

	UPROPERTY(EditAnywhere, Category=AdvancedProperties, meta=(editcondition="bEditCondition"))
	int32 SimplePropertyWithEditCondition;

	UPROPERTY(EditAnywhere, Category=StructTests, meta=(InlineEditConditionToggle))
	bool bEditConditionAssetReferenceCustomStructWithEditCondition;

	UPROPERTY(EditAnywhere, Category=StructTests, meta=(editcondition="bEditConditionAssetReferenceCustomStructWithEditCondition"))
	FSoftObjectPath AssetReferenceCustomStructWithEditCondition;

	UPROPERTY(EditAnywhere, Category=StructTests)
	TArray<FPropertyEditorTestBasicStruct> ArrayOfStructs;

	UPROPERTY(EditAnywhere, Category=StructTests)
	FPropertyEditTestTextStruct Struct;

	UPROPERTY(EditAnywhere, Category=EditInlineProps)
	TObjectPtr<UStaticMeshComponent> EditInlineNewStaticMeshComponent;

	UPROPERTY(EditAnywhere, Category=EditInlineProps)
	TArray<TObjectPtr<UStaticMeshComponent>> ArrayOfEditInlineNewSMCs;

	UPROPERTY(EditAnywhere, Category=AssetPropertyTests)
	TObjectPtr<UTexture> TextureProp;

	UPROPERTY(EditAnywhere, Category=AssetPropertyTests)
	TObjectPtr<UStaticMesh> StaticMeshProp;

	UPROPERTY(EditAnywhere, Category=AssetPropertyTests)
	TObjectPtr<UMaterialInterface> AnyMaterialInterface;

	UPROPERTY(EditAnywhere, Category = AssetPropertyTests, meta=(DisplayThumbnail=false))
	TObjectPtr<UMaterialInterface> MaterialNoThumbnail;

	UPROPERTY(EditAnywhere, Category=AssetPropertyTests)
	TObjectPtr<AActor> OnlyActorsAllowed;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<int32> Int32Set;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<float> FloatSet;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<FString> StringSet;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<TObjectPtr<UObject>> ObjectSet;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<TObjectPtr<AActor>> ActorSet;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<EPropertyEditorTestEditColor> EditColorSet;

	UPROPERTY(EditAnywhere, Category="TSet Tests")
	TSet<FName> NameSet;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<int32, FString> Int32ToStringMap;

	UPROPERTY(EditAnywhere, Category = "TMap Tests", meta=(MultiLine=true))
	TMap<FString, FText> StringToMultilineTextMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FString, FLinearColor> StringToColorMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<int32, FPropertyEditorTestBasicStruct> Int32ToStructMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FString, float> StringToFloatMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FString, TObjectPtr<UObject>> StringToObjectMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FString, TObjectPtr<AActor>> StringToActorMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<TObjectPtr<UObject>, int32> ObjectToInt32Map;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<TObjectPtr<UObject>, FLinearColor> ObjectToColorMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<int32, TEnumAsByte<EPropertyEditorTestEnum> > IntToEnumMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FName, FName> NameToNameMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FName, TObjectPtr<UObject>> NameToObjectMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FName, FPropertyEditorTestBasicStruct> NameToCustomMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<FName, FLinearColor> NameToColorMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<int, FPropertyEditorTestBasicStruct> IntToCustomMap;

	UPROPERTY(EditAnywhere, Category="TMap Tests")
	TMap<int, FPropertyEditorTestSubStruct> IntToSubStructMap;

	UPROPERTY(EditAnywhere, Category=TSetStructTests)
	TSet<FLinearColor> LinearColorSet;

	UPROPERTY(EditAnywhere, Category=TSetStructTests)
	TSet<FVector> VectorSet;

	UPROPERTY(EditAnywhere, Category=TMapStructKeyTests)
	TMap<FLinearColor, FString> LinearColorToStringMap;

	UPROPERTY(EditAnywhere, Category=TMapStructKeyTests)
	TMap<FVector, float> VectorToFloatMap;

	UPROPERTY(EditAnywhere, Category=TMapStructKeyTests)
	TMap<FLinearColor, FVector> LinearColorToVectorMap;

	UPROPERTY(EditAnywhere, Category=ScriptInterfaces)
	TScriptInterface<IBlendableInterface> BlendableInterface;

	UPROPERTY(EditAnywhere, Category=ScriptInterfaces)
	TScriptInterface<IAnimClassInterface> AnimClassInterface;

	// This is an IBlendableInterface that only allows for ULightPropagationVolumeBlendable objects
	UPROPERTY(EditAnywhere, Category=ScriptInterfaces, meta=(AllowedClasses="/Script/Engine.LightPropagationVolumeBlendable"))
	TScriptInterface<IBlendableInterface> LightPropagationVolumeBlendable;

	// Allows either an object that's derived from UTexture or IBlendableInterface, to ensure that Object Property handles know how to
	// filter for AllowedClasses correctly.
	UPROPERTY(EditAnywhere, Category=ObjectPropertyAllowedClasses, meta=(AllowedClasses="/Script/Engine.Texture,/Script/Engine.BlendableInterface"))
	TObjectPtr<UObject> TextureOrBlendableInterface;

	UPROPERTY(EditAnywhere, Category="Subcategory")
	bool bSubcategory;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Subcategory")
	bool bSubcategoryAdvanced;

	UPROPERTY(EditAnywhere, Category="Subcategory|Foo")
	bool bSubcategoryFooSimple;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Subcategory|Foo")
	bool bSubcategoryFooAdvanced;

	UPROPERTY(EditAnywhere, Category="Subcategory|Bar")
	bool bSubcategoryBarSimple;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Subcategory|Bar")
	bool bSubcategoryBarAdvanced;

	UPROPERTY(EditAnywhere, Category="Subcategory")
	bool bSubcategoryLast;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	bool bEnablesNext;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="bEnablesNext == true"))
	bool bEnabledByPrevious;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	EPropertyEditorTestEditColor EnumEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EnumEditCondition == EPropertyEditorTestEditColor::Blue"))
	bool bEnabledWhenBlue;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EnumEditCondition == EPropertyEditorTestEditColor::Pink"))
	bool bEnabledWhenPink;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	TEnumAsByte<EPropertyEditorTestEnum> EnumAsByteEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EnumAsByteEditCondition == EPropertyEditorTestEnum::PropertyEditorTest_Enum2"))
	bool bEnabledWhenEnumIs2;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EnumAsByteEditCondition == EPropertyEditorTestEnum::PropertyEditorTest_Enum4"))
	bool bEnabledWhenEnumIs4;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	int32 IntegerEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="IntegerEditCondition >= 5"))
	bool bEnabledWhenIntGreaterOrEqual5;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="IntegerEditCondition <= 10"))
	bool bEnabledWhenIntLessOrEqual10;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	float FloatEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="FloatEditCondition > 5"))
	bool bEnabledWhenFloatGreaterThan5;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="FloatEditCondition < 10"))
	bool bEnabledWhenFloatLessThan10;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	bool bEditConditionForArrays;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="bEditConditionForArrays"))
	TArray<TObjectPtr<UTexture2D>> ArrayWithEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="bEditConditionForArrays"))
	TArray<FPropertyEditorTestBasicStruct> ArrayOfStructsWithEditCondition;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	bool bEditConditionForFixedArray;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="bEditConditionForFixedArray"))
	FString FixedArrayWithEditCondition[5];

	UPROPERTY(EditAnywhere, Category=EditCondition)
	bool bEditConditionForDirectoryPath;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="bEditConditionForDirectoryPath"))
	FDirectoryPath DirectoryPath;

	UPROPERTY(EditAnywhere, Category=EditCondition)
	int64 EditConditionFlags;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EditConditionFlags & ETestEnumFlags::Two || EditConditionFlags & ETestEnumFlags::Four"))
	bool bEnabledWhenFlagsHasTwoOrFour;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition="EditConditionFlags & ETestEnumFlags::One == false"))
	bool bDisabledWhenFlagsIsOdd;

	UPROPERTY(EditAnywhere, Category=EditCondition, meta=(EditCondition=false))
	int32 AlwaysDisabled;

	UPROPERTY(EditAnywhere, Category="Category Inline Edit Condition", meta=(InlineCategoryProperty))
	bool bCategoryInlineEditCondition;

	UPROPERTY(EditAnywhere, Category="Category Inline Edit Condition", meta=(EditCondition="bCategoryInlineEditCondition"))
	float EnabledWhenCategoryChecked;

	UPROPERTY(EditAnywhere, Category=OnlyInlineProperty, meta=(InlineCategoryProperty))
	TEnumAsByte<EComponentMobility::Type> InlineProperty;

	UPROPERTY(EditAnywhere, Category=EditConditionHides, meta=(InlineCategoryProperty))
	TEnumAsByte<EComponentMobility::Type> PropertyThatHides;

	UPROPERTY(EditAnywhere, Category=EditConditionHides, meta=(EditConditionHides, EditCondition="PropertyThatHides == EComponentMobility::Static"))
	bool bVisibleWhenStatic;

	UPROPERTY(EditAnywhere, Category=EditConditionHides, meta=(EditConditionHides, EditCondition="PropertyThatHides == EComponentMobility::Stationary"))
	int32 VisibleWhenStationary;

	UPROPERTY(EditAnywhere, Category=DateTime)
	FDateTime DateTime;

	UPROPERTY(EditAnywhere, Category = DateTime)
	FTimespan Timespan;

	UPROPERTY(EditAnywhere, Category = AdvancedProperties)
	FGuid Guid;

	UPROPERTY(EditAnywhere, Category = AdvancedProperties)
	FPerPlatformFloat PerPlatformFloat;

	UPROPERTY(EditAnywhere, Category = AdvancedProperties)
	FPerPlatformInt PerPlatformInt;

	UPROPERTY()
	bool bInlineEditConditionWithoutMetaToggle;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(EditCondition="bInlineEditConditionWithoutMetaToggle"))
	float InlineEditConditionWithoutMeta;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(InlineEditConditionToggle))
	bool bInlineEditConditionWithMetaToggle;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(EditCondition="bInlineEditConditionWithMetaToggle"))
	float InlineEditConditionWithMeta;

	UPROPERTY(meta=(InlineEditConditionToggle))
	bool bInlineEditConditionNotEditable;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(EditCondition="bInlineEditConditionNotEditable"))
	float HasNonEditableInlineCondition;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions")
	bool bSharedEditCondition;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(EditCondition="bSharedEditCondition"))
	float UsesSharedEditCondition1;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions", meta=(EditCondition="bSharedEditCondition"))
	float UsesSharedEditCondition2;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions")
	FPropertyEditorTestEditCondition StructWithInlineCondition;

	UPROPERTY(EditAnywhere, Category="Inline Edit Conditions")
	TArray<FPropertyEditorTestEditCondition> ArrayOfStructsWithInlineCondition;

	UPROPERTY(EditAnywhere, Category = ArraysOfProperties)
	int32 NestedArrayOfInts[5];
};

UCLASS(HideCategories=(ShownByDerived))
class UHideCategoriesBase : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category=ShownByDerived)
	int32 HiddenInBase;
};

UCLASS(ShowCategories=(ShownByDerived))
class UShowCategoriesTest : public UHideCategoriesBase
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category=InDerived)
	int32 InDerived;
};

UCLASS(EditInlineNew, Blueprintable)
class UBlueprintPropertyTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category="Hidden")
	int32 ShouldBeHidden;

	UPROPERTY(EditAnywhere, Category="Visible")
	int32 ShouldBeVisible;

	UPROPERTY(EditAnywhere, Instanced, Category=Default)
	TObjectPtr<USoundBase> ObjectA;

	UPROPERTY(EditAnywhere, Instanced, Category=Default)
	TObjectPtr<USoundBase> ObjectB;
};
 
UCLASS(Blueprintable)
class UBlueprintPropertyContainerTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(Instanced, EditAnywhere, Category="Default", meta=(ShowOnlyInnerProperties))
	TArray<TObjectPtr<UBlueprintPropertyTestObject>> Array;
};

UCLASS(Abstract, BlueprintType, SparseClassDataType=TestSparseClassDataStorage)
class UTestSparseClassDataBase : public UObject
{
	GENERATED_BODY()
};

UCLASS(BlueprintType)
class UTestSparseClassData : public UTestSparseClassDataBase
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestSparseClassDataStorage
{
	GENERATED_BODY()

	UPROPERTY(EditDefaultsOnly, Category=Default)
	TMap<int, int> Map;
};

UCLASS()
class APropertyEditorTestActor : public AActor
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Instanced, Category = ArraysOfProperties)
	TArray<TObjectPtr<UPropertyEditorTestInstancedObject>> InstancedUObjectArray;

	UPROPERTY(EditAnywhere, Category=Default, meta=(GetOptions=GetOptionsFunc))
	FName GetOptionsValue; 

	UPROPERTY(EditDefaultsOnly, Category="Defaults Only")
	float DefaultsOnly;

	UPROPERTY(EditDefaultsOnly, Category="Defaults Only|Subcategory")
	float DefaultsOnlySubcategory;

	UPROPERTY(EditInstanceOnly, Category="Instance Only")
	float InstanceOnly;

	UPROPERTY(EditInstanceOnly, Category="Instance Only|Subcategory")
	float InstanceOnlySubcategory;

	UPROPERTY(EditAnywhere, Category="Map")
	TMap<int32, FText> MultiLineMap;

	UFUNCTION()
	TArray<FString> GetOptionsFunc() const;
};

class IDetailTreeNode;
class IPropertyRowGenerator;

UCLASS()
class UPropertyEditorRowGeneratorTest : public UObject
{
	GENERATED_BODY()

public:

	TSharedRef<SWidget> GenerateWidget();

private:
	void OnRowsRefreshed();
	TSharedRef<ITableRow> GenerateListRow(TSharedPtr<IDetailTreeNode> InItem, const TSharedRef<STableViewBase>& OwnerTable);

private:
	TArray<TSharedPtr<IDetailTreeNode>> DetailsNodes;
	TSharedPtr<SListView<TSharedPtr<IDetailTreeNode>>> ListView;
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
};
