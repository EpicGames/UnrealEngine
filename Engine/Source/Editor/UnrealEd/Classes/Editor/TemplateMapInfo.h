// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "TemplateMapInfo.generated.h"

/** Used by new level dialog. */
USTRUCT()
struct FTemplateMapInfo
{
	GENERATED_USTRUCT_BODY()

	/** The Texture2D associated with this map template */
	UPROPERTY()
	TSoftObjectPtr<class UTexture2D> ThumbnailTexture;

	/** The Texture associated with this map template */
	UPROPERTY(config, meta = (AllowedClasses = "/Script/Engine.Texture2D"))
	FSoftObjectPath Thumbnail;

	/** The object path to the template map */
	UPROPERTY(config, meta = (AllowedClasses = "/Script/Engine.World"))
	FSoftObjectPath Map;

	/** Optional display name override for this map template  */
	UPROPERTY(config)
	FText DisplayName;

	/* Optional category used for sorting */
	UPROPERTY(config)
	FString Category;

	FTemplateMapInfo()
	{
	}
};