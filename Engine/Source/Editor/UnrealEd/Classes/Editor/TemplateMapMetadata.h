// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "TemplateMapMetadata.generated.h"

UCLASS(deprecated)
class UDEPRECATED_TemplateMapMetadata : public UObject
{
	GENERATED_UCLASS_BODY()
};

