// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Transaction tracking system, manages the undo and redo buffer.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Editor/Transactor.h"
#include "TransBuffer.generated.h"

UCLASS(transient, MinimalAPI)
class UTransBuffer
	: public UTransactor
{
public:
    GENERATED_BODY()
	// Variables.
	/** The queue of transaction records */
	TArray<TSharedRef<FTransaction>> UndoBuffer;

	/** Number of transactions that have been undone, and are eligible to be redone */
	int32 UndoCount;

	/** Text describing the reason that the undo buffer is empty */
	FText ResetReason;

	/** Number of actions in the current transaction */
	int32 ActiveCount;

	/** The cached count of the number of object records each time a transaction is begun */
	TArray<int32> ActiveRecordCounts;

	/** Maximum number of bytes the transaction buffer is allowed to occupy */
	SIZE_T MaxMemory;

	/** Undo barrier stack */
	TArray<int32> UndoBarrierStack;

public:

	// Constructor.
	UNREALED_API void Initialize(SIZE_T InMaxMemory);

public:

	/**
	 * Validates the state of the transaction buffer.
	 */
	UNREALED_API void CheckState() const;
	static UNREALED_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

public:

	//~ Begin UObject Interface.

	UNREALED_API virtual void Serialize( FArchive& Ar ) override;
	UNREALED_API virtual void FinishDestroy() override;

	//~ End UObject Interface.

protected:
	
	UNREALED_API void OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstances);

	/** Implementation of the begin function. Used to create a specific transaction type */
	template<typename TTransaction>
	int32 BeginInternal( const TCHAR* SessionContext, const FText& Description )
	{
		int32 Result = INDEX_NONE;
		CheckState();
		if (ensure(!GIsTransacting))
		{
			Result = ActiveCount;
			if (ActiveCount++ == 0)
			{
				// Redo removal handling
				if (UndoCount > 0)
				{
					RemovedTransactions.Reserve(UndoCount);

					for (int32 i = UndoBuffer.Num() - UndoCount; i < UndoBuffer.Num(); ++i)
					{
						RemovedTransactions.Add(UndoBuffer[i]);
					}

					UndoBuffer.RemoveAt(UndoBuffer.Num() - UndoCount, UndoCount, EAllowShrinking::No);
				}
				// Over memory budget handling
 				else 
				{
					int32 TransactionsToRemove = 0;

					// Determine if any additional entries need to be removed due to memory, avoid n^2 by unrolling GetUndoSize()
					SIZE_T AccumulatedBufferDataSize = GetUndoSize();

					if (AccumulatedBufferDataSize >= MaxMemory)
					{
						// Then remove from the oldest one
						for (int32 i = 0; i < UndoBuffer.Num() && AccumulatedBufferDataSize >= MaxMemory; i++)
						{
							if (AccumulatedBufferDataSize >= MaxMemory)
							{
								AccumulatedBufferDataSize -= UndoBuffer[i]->DataSize();
								++TransactionsToRemove;
							}
						}
					}

					if (TransactionsToRemove > 0)
					{
						RemovedTransactions.Reserve(TransactionsToRemove);

						for (int32 i = 0; i < TransactionsToRemove; ++i)
						{
							RemovedTransactions.Add(UndoBuffer[i]);
						}

						UndoBuffer.RemoveAt(0, TransactionsToRemove, EAllowShrinking::No);
					}
				}

				// Cache the redo buffer in case the transaction is canceled so we can restore the state
				PreviousUndoCount = UndoCount;
				UndoCount = 0;

				// Begin a new transaction.
				UndoBuffer.Emplace(MakeShareable(new TTransaction(SessionContext, Description, 1)));
				GUndo = &UndoBuffer.Last().Get();			

				GUndo->BeginOperation();
				TransactionStateChangedDelegate.Broadcast(GUndo->GetContext(), ETransactionStateEventType::TransactionStarted);
				UndoBufferChangedDelegate.Broadcast();
			}
			const int32 PriorRecordsCount = (Result > 0 ? ActiveRecordCounts[Result - 1] : 0);
			ActiveRecordCounts.Add(UndoBuffer.Last()->GetRecordCount() - PriorRecordsCount);
			CheckState();
		}
		return Result;
	}

public:

	//~ Begin UTransactor Interface.

	UNREALED_API virtual int32 Begin( const TCHAR* SessionContext, const FText& Description ) override;
	UNREALED_API virtual int32 End() override;
	UNREALED_API virtual void Cancel( int32 StartIndex = 0 ) override;
	UNREALED_API virtual void Reset( const FText& Reason ) override;
	UNREALED_API virtual bool CanUndo( FText* Text=NULL ) override;
	UNREALED_API virtual bool CanRedo( FText* Text=NULL ) override;
	virtual int32 GetQueueLength( ) const override { return UndoBuffer.Num(); }
	UNREALED_API virtual int32 FindTransactionIndex(const FGuid& TransactionId) const override;
	UNREALED_API virtual const FTransaction* GetTransaction( int32 QueueIndex ) const override;
	UNREALED_API virtual FTransactionContext GetUndoContext( bool bCheckWhetherUndoPossible = true ) override;
	UNREALED_API virtual SIZE_T GetUndoSize() const override;
	virtual int32 GetUndoCount( ) const override { return UndoCount; }
	UNREALED_API virtual FTransactionContext GetRedoContext() override;
	UNREALED_API virtual void SetUndoBarrier() override;
	UNREALED_API virtual void RemoveUndoBarrier() override;
	UNREALED_API virtual void ClearUndoBarriers() override;
	UNREALED_API virtual int32 GetCurrentUndoBarrier() const override;
	UNREALED_API virtual bool Undo(bool bCanRedo = true) override;
	UNREALED_API virtual bool Redo() override;
	UNREALED_API virtual bool EnableObjectSerialization() override;
	UNREALED_API virtual bool DisableObjectSerialization() override;
	virtual bool IsObjectSerializationEnabled() override { return DisallowObjectSerialization == 0; }
	UNREALED_API virtual void SetPrimaryUndoObject( UObject* Object ) override;
	UNREALED_API virtual bool IsObjectInTransactionBuffer( const UObject* Object ) const override;
	UNREALED_API virtual bool IsObjectTransacting(const UObject* Object) const override;
	UNREALED_API virtual bool ContainsPieObjects() const override;
	virtual bool IsActive() override
	{
		return ActiveCount > 0;
	}

	//~ End UTransactor Interface.

public:

	/**
	 * Gets an event delegate that is executed when a transaction state changes.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT_TwoParams(UTransBuffer, FOnTransactorTransactionStateChanged, const FTransactionContext& /*TransactionContext*/, ETransactionStateEventType /*TransactionState*/)
	FOnTransactorTransactionStateChanged& OnTransactionStateChanged( )
	{
		return TransactionStateChangedDelegate;
	}

	/**
	 * Gets an event delegate that is executed when a redo operation is being attempted.
	 *
	 * @return The event delegate.
	 *
	 * @see OnUndo
	 */
	DECLARE_EVENT_OneParam(UTransBuffer, FOnTransactorBeforeRedoUndo, const FTransactionContext& /*TransactionContext*/)
	FOnTransactorBeforeRedoUndo& OnBeforeRedoUndo( )
	{
		return BeforeRedoUndoDelegate;
	}

	/**
	 * Gets an event delegate that is executed when a redo operation is being attempted.
	 *
	 * @return The event delegate.
	 *
	 * @see OnUndo
	 */
	DECLARE_EVENT_TwoParams(UTransBuffer, FOnTransactorRedo, const FTransactionContext& /*TransactionContext*/, bool /*Succeeded*/)
	FOnTransactorRedo& OnRedo( )
	{
		return RedoDelegate;
	}

	/**
	 * Gets an event delegate that is executed when a undo operation is being attempted.
	 *
	 * @return The event delegate.
	 *
	 * @see OnRedo
	 */
	DECLARE_EVENT_TwoParams(UTransBuffer, FOnTransactorUndo, const FTransactionContext& /*TransactionContext*/, bool /*Succeeded*/)
	FOnTransactorUndo& OnUndo( )
	{
		return UndoDelegate;
	}

	/**
	* Gets an event delegate that is executed when the undo buffer changed.
	*
	* @return The event delegate.
	*
	* @see OnRedo
	*/
	DECLARE_EVENT(UTransBuffer, FOnTransactorUndoBufferChanged)
	FOnTransactorUndoBufferChanged& OnUndoBufferChanged()
	{
		return UndoBufferChangedDelegate;
	}

private:

	/** Controls whether the transaction buffer is allowed to serialize object references */
	int32 DisallowObjectSerialization;

private:

	// Holds an event delegate that is executed when a transaction state changes.
	FOnTransactorTransactionStateChanged TransactionStateChangedDelegate;

	// Holds an event delegate that is executed before a redo or undo operation is attempted.
	FOnTransactorBeforeRedoUndo BeforeRedoUndoDelegate;

	// Holds an event delegate that is executed when a redo operation is being attempted.
	FOnTransactorRedo RedoDelegate;

	// Holds an event delegate that is executed when a undo operation is being attempted.
	FOnTransactorUndo UndoDelegate;

	// Holds an event delegate that is executed when the undo buffer changed.
	FOnTransactorUndoBufferChanged UndoBufferChangedDelegate;

	// Reference to the current transaction, nullptr when not transacting:
	FTransaction* CurrentTransaction;

	// Cached previous undo count while a transaction is being built in case we cancel it and want to restore the previous undo buffer
	int32 PreviousUndoCount;

	// The list of transactions that were removed when a transaction was begun in case it is canceled and we want to restore the original state
	TArray<TSharedRef<FTransaction>> RemovedTransactions;

};
