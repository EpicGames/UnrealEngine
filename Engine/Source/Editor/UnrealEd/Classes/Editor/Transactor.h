// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class for tracking transactions for undo/redo.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Misc/ITransaction.h"
#include "Misc/TransactionObjectEvent.h"
#include "Misc/TransactionCommon.h"
#include "Algo/Find.h"
#include <type_traits>
#include "Transactor.generated.h"

/*-----------------------------------------------------------------------------
	FTransaction.
-----------------------------------------------------------------------------*/

/**
 * A single transaction, representing a set of serialized, undo-able changes to a set of objects.
 *
 * warning: The undo buffer cannot be made persistent because of its dependence on offsets
 * of arrays from their owning UObjects.
 *
 * warning: UObject::Serialize implicitly assumes that class properties do not change
 * in between transaction resets.
 */
class FTransaction : public ITransaction
{
	friend class FTransactionObjectEvent;

protected:
	// Record of an object.
	class FObjectRecord
	{
	public:
		struct FSerializedObject : public UE::Transaction::FSerializedObject
		{
			enum class EPendingKillChange : uint8
			{
				None,
				DeadToAlive,
				AliveToDead
			};

			void SetObject(const UObject* InObject)
			{
				ObjectId.SetObject(InObject);
				ObjectFlags = InObject->GetFlags();
				ObjectAnnotation = InObject->FindOrCreateTransactionAnnotation();
			}

			void Reset()
			{
				UE::Transaction::FSerializedObject::Reset();
				ObjectId.Reset();
				ObjectFlags = RF_NoFlags;
				ObjectAnnotation.Reset();
				PendingKillChange = EPendingKillChange::None;
			}

			void Swap(FSerializedObject& Other)
			{
				UE::Transaction::FSerializedObject::Swap(Other);
				ObjectId.Swap(Other.ObjectId);
				Exchange(ObjectFlags, Other.ObjectFlags);
				Exchange(ObjectAnnotation, Other.ObjectAnnotation);
				Exchange(PendingKillChange, Other.PendingKillChange);
			}

			/** ID of the object when it was serialized */
			FTransactionObjectId ObjectId;

			/** The flags of the object when it was serialized */
			EObjectFlags ObjectFlags = RF_NoFlags;

			/** Annotation data for the object stored externally */
			TSharedPtr<ITransactionObjectAnnotation> ObjectAnnotation;

			/** Whether this transaction marked this actor as garbage, unmarked it, or didn't touch the garbage flag at all. */
			EPendingKillChange PendingKillChange = EPendingKillChange::None;
		};

		// Variables.
		/** The object to track */
		UE::Transaction::FPersistentObjectRef Object;
		/** Custom change to apply to this object to undo this record.  Executing the undo will return an object that can be used to redo the change. */
		TUniquePtr<FChange> CustomChange;
		/** Array: If an array object, reference to script array */
		FScriptArray*		Array = nullptr;
		/** Array: Offset into the array */
		int32				Index = 0;
		/** Array: How many items to record */
		int32				Count = 0;
		/** @todo document this
		 Array: Operation performed on array: 1 (add/insert), 0 (modify), -1 (remove)? */
		int32				Oper = 0;
		/** Array: Size of each item in the array */
		int32				ElementSize = 0;
		/** Array: Alignment of each item in the array */
		uint32				ElementAlignment = 0;
		/** Array: DefaultConstructor for each item in the array */
		STRUCT_DC			DefaultConstructor;
		/** Array: Serializer to use for each item in the array */
		STRUCT_AR			Serializer;
		/** Array: Destructor for each item in the array */
		STRUCT_DTOR			Destructor;
		/** True if object  has already been restored from data. False otherwise. */
		bool				bRestored = false;
		/** True if object has been finalized and generated diff data */
		bool				bFinalized = false;
		/** True if record should serialize data as binary blob (more compact). False to use tagged serialization (more robust) */
		bool				bWantsBinarySerialization = true;
		/** True if the serialized object state changed between when the transaction started and when the transaction ended (set once finalized) */
		bool				bHasSerializedObjectChanges = false;
		/** The serialized object data */
		FSerializedObject	SerializedObject;
		/** The serialized object data that will be used when the transaction is flipped */
		FSerializedObject	SerializedObjectFlip;
		/** The diffable object data (always null once finalized) */
		TUniquePtr<UE::Transaction::FDiffableObject> DiffableObject;
		/** The diffable object data when it was last snapshot (always null once finalized) */
		TUniquePtr<UE::Transaction::FDiffableObject> DiffableObjectSnapshot;
		/** Annotation data the last time the object was snapshot (always null once finalized) */
		TSharedPtr<ITransactionObjectAnnotation> ObjectAnnotationSnapshot;
		/** The combined list of properties that have been passed any Snapshot call for this object (always empty once finalized) */
		TArray<const FProperty*> AllPropertiesSnapshot;
		/** Delta change information between the diffable state of the object when the transaction started, and the diffable state of the object when the transaction ended */
		FTransactionObjectDeltaChange DeltaChange;

		// Constructors.
		FObjectRecord() = default;
		UNREALED_API FObjectRecord( FTransaction* Owner, UObject* InObject, TUniquePtr<FChange> InCustomChange, FScriptArray* InArray, int32 InIndex, int32 InCount, int32 InOper, int32 InElementSize, uint32 InElementAlignment, STRUCT_DC InDefaultConstructor, STRUCT_AR InSerializer, STRUCT_DTOR InDestructor );

	private:
		// Non-copyable
		FObjectRecord( const FObjectRecord& ) = delete;
		FObjectRecord& operator=( const FObjectRecord& ) = delete;

	public:
		UNREALED_API UE::Transaction::FDiffableObject GetDiffableObject(TArrayView<const FProperty*> PropertiesToSerialize = TArrayView<const FProperty*>(), UE::Transaction::DiffUtil::EGetDiffableObjectMode ObjectSerializationMode = UE::Transaction::DiffUtil::EGetDiffableObjectMode::SerializeObject) const;

		// Functions.
		UNREALED_API void SerializeContents( FArchive& Ar, int32 InOper ) const;
		UNREALED_API void SerializeObject( FArchive& Ar ) const;
		UNREALED_API void Restore( FTransaction* Owner );
		UNREALED_API void Save( FTransaction* Owner );
		UNREALED_API void Load( FTransaction* Owner );
		UNREALED_API void Finalize( FTransaction* Owner, UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache& ArchetypeCache, TSharedPtr<ITransactionObjectAnnotation>& OutFinalizedObjectAnnotation );
		UNREALED_API void Snapshot( FTransaction* Owner, UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache& ArchetypeCache, TArrayView<const FProperty*> Properties );

		/** Used by GC to collect referenced objects. */
		UNREALED_API void AddReferencedObjects( FReferenceCollector& Collector );

		/** @return True if this record contains a reference to a pie object */
		UNREALED_API bool ContainsPieObject() const;

		/** @return true if the record has a delta change or a custom change */
		UNREALED_API bool HasChanges() const;

		/** @return true if the record has a custom change and the change has expired */
		UNREALED_API bool HasExpired() const;

		/** Transfers data from an array. */
		class FReader : public UE::Transaction::FSerializedObjectDataReader
		{
		public:
			FReader(
				FTransaction* InOwner,
				const FSerializedObject& InSerializedObject,
				bool bWantBinarySerialization
				)
				: UE::Transaction::FSerializedObjectDataReader(InSerializedObject)
				, Owner(InOwner)
			{
				this->SetWantBinaryPropertySerialization(bWantBinarySerialization);
				this->SetIsTransacting(true);
			}

		private:
			void Preload( UObject* InObject ) override
			{
				if (Owner)
				{
					if (const FObjectRecords* ObjectRecords = Owner->ObjectRecordsMap.Find(UE::Transaction::FPersistentObjectRef(InObject)))
					{
						for (FObjectRecord* Record : ObjectRecords->Records)
						{
							checkSlow(Record->Object.Get() == InObject);
							if (!Record->CustomChange)
							{
								Record->Restore(Owner);
							}
						}
					}
				}
			}
			FTransaction* Owner;
		};

		/**
		 * Transfers data to an array.
		 */
		class FWriter : public UE::Transaction::FSerializedObjectDataWriter
		{
		public:
			FWriter(
				FSerializedObject& InSerializedObject,
				bool bWantBinarySerialization
				)
				: UE::Transaction::FSerializedObjectDataWriter(InSerializedObject)
			{
				this->SetWantBinaryPropertySerialization(bWantBinarySerialization);
				this->SetIsTransacting(true);
			}
		};
	};

	struct FObjectRecords
	{
		// clang fix for std::is_default_constructible_v 
		// returning false in inlined code of outer class
		FObjectRecords() {}

		friend FArchive& operator<<(FArchive& Ar, FObjectRecords& ObjectRecords)
		{
			Ar << ObjectRecords.SaveCount;

			if (Ar.IsLoading())
			{
				// Clear the map on load, as it will need to be rebuilt from the source array
				ObjectRecords.Records.Empty();
			}

			return Ar;
		}

		TArray<FObjectRecord*, TInlineAllocator<1>> Records;
		int32 SaveCount = 0;
	};

	struct FPackageRecord
	{
		// clang fix for std::is_default_constructible_v 
		// returning false in inlined code of outer class
		FPackageRecord() {}

		friend FArchive& operator<<(FArchive& Ar, FPackageRecord& PackageRecord)
		{
			Ar << PackageRecord.DirtyFenceCount;
			Ar << PackageRecord.bWasDirty;
			return Ar;
		}

		/** The fence count used to determine if the cached dirty state is still valid on undo/redo (ie, was the package saved or deleted since the undo was made) */
		int32 DirtyFenceCount = 0;

		/** The cached dirty state of this package, to be restored on undo/redo as long as the DirtyFenceCount still matches */
		bool bWasDirty = false;
	};

	// Transaction variables.
	/** List of object records in this transaction */
	TIndirectArray<FObjectRecord> Records;

	/** Map of object records (non-array), for optimized look-up and to prevent an object being serialized to a transaction more than once. */
	TMap<UE::Transaction::FPersistentObjectRef, FObjectRecords> ObjectRecordsMap;

	/** Map of package records for tracking and restoring the dirty state of packages on undo/redo. The package itself may also be stored in ObjectRecordsMap. */
	TMap<UE::Transaction::FPersistentObjectRef, FPackageRecord> PackageRecordMap;

	/** Unique identifier for this transaction, used to track it during its lifetime */
	FGuid		Id;

	/**
	 * Unique identifier for the active operation on this transaction (if any).
	 * This is set by a call to BeginOperation and cleared by a call to EndOperation.
	 * BeginOperation should be called when a transaction or undo/redo starts, and EndOperation should be called when a transaction is finalized or canceled or undo/redo ends.
	 */
	FGuid		OperationId;

	/** Description of the transaction. Can be used by UI */
	FText		Title;
	
	/** A text string describing the context for the transaction. Typically the name of the system causing the transaction*/
	FString		Context;

	/** The key object being edited in this transaction. For example the blueprint object. Can be NULL */
	UObject* PrimaryObject = nullptr;

	/** If true, on apply flip the direction of iteration over object records. */
	bool		bFlip = true;
	// @TODO now that Matinee is gone, it's worth investigating removing this

	/** Used to track direction to iterate over transaction's object records. Typically -1 for Undo, 1 for Redo */
	int32		Inc = -1;

	struct FChangedObjectValue
	{
		FChangedObjectValue() = default;

		FChangedObjectValue(const int32 InRecordIndex, const TSharedPtr<ITransactionObjectAnnotation>& InAnnotation, const TSharedPtr<ITransactionObjectAnnotation>& InAnnotationSnapshot = nullptr)
			: Annotation(InAnnotation)
			, AnnotationSnapshot(InAnnotationSnapshot)
			, RecordIndex(InRecordIndex)
		{
		}

		TSharedPtr<ITransactionObjectAnnotation> Annotation;
		TSharedPtr<ITransactionObjectAnnotation> AnnotationSnapshot;
		int32 RecordIndex = INDEX_NONE;
	};

	/** Objects that will be changed directly by the transaction, empty when not transacting */
	TMap<UObject*, FChangedObjectValue> ChangedObjects;

	/** Cached state of the diff the last time this transaction was applied to an object (via PostTransacted for non-snapshot transactions) to allow historic introspection of the diff regardless of the current state of the object */
	TMap<FName, TSharedPtr<FTransactionObjectEvent>> HistoricDiffMap;

public:
	// Constructor.
	FTransaction(  const TCHAR* InContext=nullptr, const FText& InTitle=FText(), bool bInFlip=false )
		:	Id( FGuid::NewGuid() )
		,	Title( InTitle )
		,	Context( InContext )
		,	bFlip(bInFlip)
	{
	}

	virtual ~FTransaction() = default;

private:
	// Non-copyable
	FTransaction( const FTransaction& ) = delete;
	FTransaction& operator=( const FTransaction& ) = delete;

	UNREALED_API void SavePackage(UPackage* Package);

public:
	
	// FTransactionBase interface.
	UNREALED_API virtual void SaveObject( UObject* Object ) override;
	UNREALED_API virtual void SaveArray( UObject* Object, FScriptArray* Array, int32 Index, int32 Count, int32 Oper, int32 ElementSize, uint32 ElementAlignment, STRUCT_DC DefaultConstructor, STRUCT_AR Serializer, STRUCT_DTOR Destructor ) override;
	UNREALED_API virtual void StoreUndo( UObject* Object, TUniquePtr<FChange> UndoChange ) override;
	UNREALED_API virtual void SetPrimaryObject(UObject* InObject) override;
	UNREALED_API virtual void SnapshotObject( UObject* InObject, TArrayView<const FProperty*> Properties ) override;
	UNREALED_API virtual bool ContainsObject(const UObject* Object) const override;

	/** BeginOperation should be called when a transaction or undo/redo starts */
	UNREALED_API virtual void BeginOperation() override;

	/** EndOperation should be called when a transaction is finalized or canceled or undo/redo ends */
	UNREALED_API virtual void EndOperation() override;

	/**
	 * Enacts the transaction.
	 */
	UNREALED_API virtual void Apply() override;

	/**
	 * Finalize the transaction (try and work out what's changed).
	 */
	UNREALED_API virtual void Finalize() override;

	/**
	 * Gets the full context for the transaction.
	 */
	virtual FTransactionContext GetContext() const override
	{
		return FTransactionContext(Id, OperationId, Title, *Context, PrimaryObject);
	}

	/** @return true if the transaction is transient. */
	UNREALED_API virtual bool IsTransient() const override;

	/** @return True if this record contains a reference to a pie object */
	UNREALED_API virtual bool ContainsPieObjects() const override;

	/** @return true if the transaction contains custom changes and all the changes have expired */
	UNREALED_API virtual bool HasExpired() const;


	/** Returns a unique string to serve as a type ID for the FTranscationBase-derived type. */
	virtual const TCHAR* GetTransactionType() const
	{
		return TEXT("FTransaction");
	}

	// FTransaction interface.
	UNREALED_API SIZE_T DataSize() const;

	/** Returns the unique identifier for this transaction, used to track it during its lifetime */
	FGuid GetId() const
	{
		return Id;
	}

	/** Returns the unique identifier for the active operation on this transaction (if any) */
	FGuid GetOperationId() const
	{
		return OperationId;
	}

	/** Returns the descriptive text for the transaction */
	FText GetTitle() const
	{
		return Title;
	}

	/** Returns the description of each contained Object Record */
	FText GetDescription() const
	{			
		FString ConcatenatedRecords;
		for (const FObjectRecord& Record : Records)
		{
			if (Record.CustomChange.IsValid())
			{
				if (ConcatenatedRecords.Len())
				{
					ConcatenatedRecords += TEXT("\n");
				}
				ConcatenatedRecords += Record.CustomChange->ToString();
			}
		}

		return ConcatenatedRecords.IsEmpty() 
			? GetTitle() 
			: FText::FromString(ConcatenatedRecords);
	}

	/** Serializes a reference to a transaction in a given archive. */
	friend FArchive& operator<<( FArchive& Ar, FTransaction& T )
	{
		Ar << T.Records << T.ObjectRecordsMap << T.PackageRecordMap << T.Id << T.Title << T.Context << T.PrimaryObject;

		if (Ar.IsLoading())
		{
			// Rebuild the LUT from the records array
			for (FObjectRecord& Record : T.Records)
			{
				if (!Record.Array)
				{
					// We should have already added a map entry for this object when loading T.ObjectRecordsMap, so check that we have (otherwise something is out-of-sync)
					FObjectRecords& ObjectRecords = T.ObjectRecordsMap.FindChecked(Record.Object);
					// Note: This is only safe to do because T.Records is a TIndirectArray
					ObjectRecords.Records.Add(&Record);
				}
			}
		}

		return Ar;
	}

	/** Serializes a reference to a transaction in a given archive. */
	friend FArchive& operator<<(FArchive& Ar, TSharedRef<FTransaction>& SharedT)
	{
		return Ar << SharedT.Get();
	}

	/** Used by GC to collect referenced objects. */
	UNREALED_API void AddReferencedObjects( FReferenceCollector& Collector );

	/**
	 * Get all the objects that are part of this transaction.
	 * @param	Objects		[out] Receives the object list.  Previous contents are cleared.
	 */
	UNREALED_API void GetTransactionObjects(TArray<UObject*>& Objects) const;
	UNREALED_API int32 GetRecordCount() const;

	const UObject* GetPrimaryObject() const { return PrimaryObject; }

	/** Checks if a specific object is in the transaction currently underway */
	UNREALED_API bool IsObjectTransacting(const UObject* Object) const;

	/**
	 * Outputs the contents of the ObjectMap to the specified output device.
	 */
	UNREALED_API void DumpObjectMap(FOutputDevice& Ar) const;

	/**
	 * Create a map that holds information about objects of a given transaction compared to the current state of the object.
	 */
	UNREALED_API FTransactionDiff GenerateDiff() const;

	/**
	 * Create a map that holds information about objects of a given transaction compared to their previous state the last time this transaction was applied to those objects (via PostTransacted).
	 */
	UNREALED_API FTransactionDiff GetHistoricDiff() const;

private:
	void PostTransactAndSaveHistoricDiff(UObject* Obj, FTransactionObjectEvent&& TransactionEvent);

	// Transaction friends.
	friend FArchive& operator<<( FArchive& Ar, FTransaction::FObjectRecord& R );

	friend class FObjectRecord;
	friend class FObjectRecord::FReader;
	friend class FObjectRecord::FWriter;
};

UCLASS(abstract, transient, MinimalAPI)
class UTransactor : public UObject
{
    GENERATED_UCLASS_BODY()
	/**
	 * Begins a new undo transaction.  An undo transaction is defined as all actions
	 * which take place when the user selects "undo" a single time.
	 * If there is already an active transaction in progress, increments that transaction's
	 * action counter instead of beginning a new transaction.
	 * 
	 * @param	SessionContext	the context for the undo session; typically the tool/editor that cause the undo operation
	 * @param	Description		the description for the undo session;  this is the text that 
	 *							will appear in the "Edit" menu next to the Undo item
	 *
	 * @return	Number of active actions when Begin() was called;  values greater than
	 *			0 indicate that there was already an existing undo transaction in progress.
	 */
	virtual int32 Begin( const TCHAR* SessionContext, const FText& Description ) PURE_VIRTUAL(UTransactor::Begin,return 0;);

	/**
	 * Attempts to close an undo transaction.  Only successful if the transaction's action
	 * counter is 1.
	 * 
	 * @return	Number of active actions when End() was called; a value of 1 indicates that the
	 *			transaction was successfully closed
	 */
	virtual int32 End() PURE_VIRTUAL(UTransactor::End,return 0;);

	/**
	 * Cancels the current transaction, no longer capture actions to be placed in the undo buffer.
	 *
	 * @param	StartIndex	the value of ActiveIndex when the transaction to be canceled was began. 
	 */
	virtual void Cancel( int32 StartIndex = 0 ) PURE_VIRTUAL(UTransactor::Cancel,);

	/**
	 * Resets the entire undo buffer;  deletes all undo transactions.
	 */
	virtual void Reset( const FText& Reason ) PURE_VIRTUAL(UTransactor::Reset,);

	/**
	 * Returns whether there are any active actions; i.e. whether actions are currently
	 * being captured into the undo buffer.
	 */
	virtual bool IsActive() PURE_VIRTUAL(UTransactor::IsActive,return false;);

	/**
	 * Determines whether the undo option should be selectable.
	 * 
	 * @param	Str		[out] the reason that undo is disabled
	 *
	 * @return	true if the "Undo" option should be selectable.
	 */
	virtual bool CanUndo( FText* Text=nullptr ) PURE_VIRTUAL(UTransactor::CanUndo,return false;);

	/**
	 * Determines whether the redo option should be selectable.
	 * 
	 * @param	Str		[out] the reason that redo is disabled
	 *
	 * @return	true if the "Redo" option should be selectable.
	 */
	virtual bool CanRedo( FText* Text=nullptr ) PURE_VIRTUAL(UTransactor::CanRedo,return false;);

	/**
	 * Gets the current length of the transaction queue.
	 *
	 * @return Queue length.
	 */
	virtual int32 GetQueueLength( ) const PURE_VIRTUAL(UTransactor::GetQueueLength,return 0;);

	/**
	 * Gets the transaction queue index from its TransactionId.
	 *
	 * @param TransactionId - The id of transaction in the queue.
	 *
	 * @return The queue index or INDEX_NONE if not found.
	 */
	virtual int32 FindTransactionIndex(const FGuid& TransactionId ) const PURE_VIRTUAL(UTransactor::FindTransactionIndex, return INDEX_NONE;);

	/**
	 * Gets the transaction at the specified queue index.
	 *
	 * @param QueueIndex - The index of the transaction in the queue.
	 *
	 * @return A read-only pointer to the transaction, or NULL if it does not exist.
	 */
	virtual const FTransaction* GetTransaction( int32 QueueIndex ) const PURE_VIRTUAL(UTransactor::GetTransaction,return nullptr;);

	/**
	 * Returns the description of the undo action that will be performed next.
	 * This is the text that is shown next to the "Undo" item in the menu.
	 * 
	 * @param	bCheckWhetherUndoPossible	Perform test whether undo is possible and return Error if not and option is set
	 *
	 * @return	text describing the next undo transaction
	 */
	virtual FTransactionContext GetUndoContext( bool bCheckWhetherUndoPossible = true ) PURE_VIRTUAL(UTransactor::GetUndoContext,return FTransactionContext(););

	/**
	 * Determines the amount of data currently stored by the transaction buffer.
	 *
	 * @return	number of bytes stored in the undo buffer
	 */
	virtual SIZE_T GetUndoSize() const PURE_VIRTUAL(UTransactor::GetUndoSize,return 0;);

	/**
	 * Gets the number of transactions that were undone and can be redone.
	 *
	 * @return Undo count.
	 */
	virtual int32 GetUndoCount( ) const PURE_VIRTUAL(UTransactor::GetUndoCount,return 0;);

	/**
	 * Returns the description of the redo action that will be performed next.
	 * This is the text that is shown next to the "Redo" item in the menu.
	 * 
	 * @return	text describing the next redo transaction
	 */
	virtual FTransactionContext GetRedoContext() PURE_VIRTUAL(UTransactor::GetRedoContext,return FTransactionContext(););

	/**
	 * Sets an undo barrier at the current point in the transaction buffer.
	 * Undoing beyond this point will not be allowed until the barrier is removed.
	 */
	virtual void SetUndoBarrier() PURE_VIRTUAL(UTransactor::SetUndoBarrier, );

	/**
	 * Removes the last set undo barrier from the transaction buffer.
	 */
	virtual void RemoveUndoBarrier() PURE_VIRTUAL(UTransactor::RemoveUndoBarrier, );

	/**
	 * Clears all undo barriers.
	 */
	virtual void ClearUndoBarriers() PURE_VIRTUAL(UTransactor::ClearUndoBarriers, );

	/**
	 * Gets the current undo barrier's position in transaction queue
	 */
	 virtual int32 GetCurrentUndoBarrier() const PURE_VIRTUAL(UTransactor::GetCurrentUndoBarrier(), { return INDEX_NONE; });

	/**
	 * Executes an undo transaction, undoing all actions contained by that transaction.
	 * 
	 * @param	bCanRedo	If false indicates that the undone transaction (and any transactions that came after it) cannot be redone
	 * 
	 * @return				true if the transaction was successfully undone
	 */
	virtual bool Undo(bool bCanRedo = true) PURE_VIRTUAL(UTransactor::Undo,return false;);

	/**
	 * Executes an redo transaction, redoing all actions contained by that transaction.
	 * 
	 * @return				true if the transaction was successfully redone
	 */
	virtual bool Redo() PURE_VIRTUAL(UTransactor::Redo,return false;);

	/**
	 * Enables the transaction buffer to serialize the set of objects it references.
	 *
	 * @return	true if the transaction buffer is able to serialize object references.
	 */
	virtual bool EnableObjectSerialization() { return false; }

	/**
	 * Disables the transaction buffer from serializing the set of objects it references.
	 *
	 * @return	true if the transaction buffer is able to serialize object references.
	 */
	virtual bool DisableObjectSerialization() { return false; }

	/**
	 * Wrapper for checking if the transaction buffer is allowed to serialize object references.
	 */
	virtual bool IsObjectSerializationEnabled() { return false; }

	/** 
	 * Set passed object as the primary context object for transactions
	 */
	virtual void SetPrimaryUndoObject( UObject* Object ) PURE_VIRTUAL(UTransactor::SetPrimaryUndoObject,);

	/** Checks if a specific object is referenced by the transaction buffer */
	virtual bool IsObjectInTransactionBuffer( const UObject* Object ) const { return false; }

	/** Checks if a specific object is in the transaction currently underway */
	virtual bool IsObjectTransacting(const UObject* Object) const PURE_VIRTUAL(UTransactor::IsObjectTransacting, return false;);

	/** @return True if this record contains a reference to a pie object */
	virtual bool ContainsPieObjects() const { return false; }
};
