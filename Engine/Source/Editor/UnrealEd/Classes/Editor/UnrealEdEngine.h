// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Misc/NotifyHook.h"
#include "Editor/EditorEngine.h"
#include "IPackageAutoSaver.h"
#include "ISourceControlProvider.h"
#include "ComponentVisualizer.h"
#include "ComponentVisualizerManager.h"
#include "TemplateMapInfo.h"
#include "UnrealEdEngine.generated.h"

class AGroupActor;
class FCanvas;
class FLevelEditorViewportClient;
class FPerformanceMonitor;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class IEngineLoop;
class ITargetPlatform;
class UPrimitiveComponent;
class UHierarchicalInstancedStaticMeshComponent;
class UTexture2D;
class UUnrealEdOptions;
class USelection;
class UTypedElementSelectionSet;
class FName;
typedef FName FEditorModeID;
struct FTypedElementSelectionOptions;

UENUM()
enum EPackageNotifyState : int
{
	/** Updating the source control state of the package */
	NS_Updating,
	/** The user has been prompted with the balloon taskbar message. */
	NS_BalloonPrompted,
	/** The user responded to the balloon task bar message and got the modal prompt to checkout dialog and responded to it. */
	NS_DialogPrompted,
	/** The package has been marked dirty and is pending a balloon prompt. */
	NS_PendingPrompt,
	/** The package has been marked dirty but cannot be checked out, and is pending a modal warning dialog. */
	NS_PendingWarning,
	NS_MAX,
};

/** Used during asset renaming/duplication to specify class-specific package/group targets. */
USTRUCT()
struct FClassMoveInfo
{
	GENERATED_USTRUCT_BODY()

	/** The type of asset this MoveInfo applies to. */
	UPROPERTY(config)
	FString ClassName;

	/** The target package info which assets of this type are moved/duplicated. */
	UPROPERTY(config)
	FString PackageName;

	/** The target group info which assets of this type are moved/duplicated. */
	UPROPERTY(config)
	FString GroupName;

	/** If true, this info is applied when moving/duplicating assets. */
	UPROPERTY(config)
	uint32 bActive:1;


	FClassMoveInfo()
		: bActive(false)
	{
	}

};

class FPerformanceMonitor;


UCLASS(config=Engine, transient, MinimalAPI)
class UUnrealEdEngine : public UEditorEngine, public FNotifyHook
{
public:
	GENERATED_BODY()
public:

	/** Global instance of the editor options class. */
	UPROPERTY()
	TObjectPtr<class UUnrealEdOptions> EditorOptionsInst;

	/**
	 * Manager responsible for configuring auto reimport
	 */
	UPROPERTY()
	TObjectPtr<class UAutoReimportManager> AutoReimportManager;

	/** A buffer for implementing material expression copy/paste. */
	UPROPERTY()
	TObjectPtr<class UMaterial> MaterialCopyPasteBuffer;

	/** A buffer for implementing sound cue nodes copy/paste. */
	UPROPERTY()
	TObjectPtr<class USoundCue> SoundCueCopyPasteBuffer;

	/** Global list of instanced animation compression algorithms. */
	UPROPERTY()
	TArray<TObjectPtr<class UAnimCompress>> AnimationCompressionAlgorithms;

	/** Array of packages to be fully loaded at Editor startup. */
	UPROPERTY(config)
	TArray<FString> PackagesToBeFullyLoadedAtStartup;

	/** Current target for LOD parenting operations (actors will use this as the replacement) */
	UPROPERTY()
	TObjectPtr<class AActor> CurrentLODParentActor;

	/** Whether the user needs to be prompted about a package being saved with an engine version newer than the current one or not */
	UPROPERTY()
	uint32 bNeedWarningForPkgEngineVer:1;

	/** Whether there is a pending package notification */
 	uint32 bShowPackageNotification:1;

	/** Array of sorted, localized editor sprite categories */
	UPROPERTY()
	TArray<FString> SortedSpriteCategories_DEPRECATED;

	UE_DEPRECATED(5.0, "This variable may no longer contain the correct template maps for the project and this property will no longer be publically accessible in the future. Use GetDefaultTemplateMapInfos to get the default list of template maps for a project")
	/** List of info for all known template maps */
	UPROPERTY(config)
	TArray<FTemplateMapInfo> TemplateMapInfos;

	/** Cooker server incase we want to cook on the side while editing... */
	UPROPERTY()
	TObjectPtr<class UCookOnTheFlyServer> CookServer;

	/** When deleting actors, these types should not generate warnings when references will be broken (this should only be types that don't affect gameplay) */
	UPROPERTY()
	TArray<TObjectPtr<UClass>> ClassesToIgnoreDeleteReferenceWarning;

	/** A list of packages dirtied this tick */
	TArray<TWeakObjectPtr<UPackage>> PackagesDirtiedThisTick;

	/** A mapping of packages to their checkout notify state.  This map only contains dirty packages.  Once packages become clean again, they are removed from the map.*/
	TMap<TWeakObjectPtr<UPackage>, uint8> PackageToNotifyState;

	/** Mapping of sprite category ids to their matching indices in the sorted sprite categories array */
	TMap<FName, int32>			SpriteIDToIndexMap;

	/** Map from component class to visualizer object to use */
	TMap< FName, TSharedPtr<class FComponentVisualizer> > ComponentVisualizerMap;

	/** Manages currently active visualizer and routes interactions to it */
	FComponentVisualizerManager	ComponentVisManager;

	//~ Begin UObject Interface.
	UNREALED_API ~UUnrealEdEngine();
	UNREALED_API virtual void FinishDestroy() override;
	UNREALED_API virtual void Serialize( FArchive& Ar ) override;
	//~ End UObject Interface.

	//~ Begin FNotify Interface.
	UNREALED_API virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;
	UNREALED_API virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;
	//~ End FNotify Interface.

	//~ Begin UEditorEngine Interface
	UNREALED_API virtual void SelectActor(AActor* Actor, bool InSelected, bool bNotify, bool bSelectEvenIfHidden = false, bool bForceRefresh = false) override;
	UNREALED_API virtual bool CanSelectActor(AActor* Actor, bool InSelected, bool bSelectEvenIfHidden=false, bool bWarnIfLevelLocked=false) const override;
	UNREALED_API virtual void SelectGroup(AGroupActor* InGroupActor, bool bForceSelection=false, bool bInSelected=true, bool bNotify=true) override;
	UNREALED_API virtual void SelectComponent(class UActorComponent* Component, bool bInSelected, bool bNotify, bool bSelectEvenIfHidden = false) override;
	UNREALED_API virtual void SelectBSPSurf(UModel* InModel, int32 iSurf, bool bSelected, bool bNoteSelectionChange) override;
	UNREALED_API virtual void SelectNone(bool bNoteSelectionChange, bool bDeselectBSPSurfs, bool WarnAboutManyActors=true) override;
	UNREALED_API virtual void DeselectAllSurfaces() override;
	UNREALED_API virtual void NoteSelectionChange(bool bNotify = true) override;
	UNREALED_API virtual void NoteActorMovement() override;
	UNREALED_API virtual void FinishAllSnaps() override;
	UNREALED_API virtual void Cleanse( bool ClearSelection, bool Redraw, const FText& Reason, bool bResetTrans ) override;
	UNREALED_API virtual bool GetMapBuildCancelled() const override;
	UNREALED_API virtual void SetMapBuildCancelled( bool InCancelled ) override;
	UNREALED_API virtual FVector GetPivotLocation() override;
	UNREALED_API virtual void SetPivot(FVector NewPivot, bool bSnapPivotToGrid, bool bIgnoreAxis, bool bAssignPivot=false) override;
	UNREALED_API virtual void ResetPivot() override;
	UNREALED_API virtual void RedrawLevelEditingViewports(bool bInvalidateHitProxies=true) override;
	UNREALED_API virtual void TakeHighResScreenShots() override;
	UNREALED_API virtual void GetPackageList( TArray<UPackage*>* InPackages, UClass* InClass ) override;
	UNREALED_API virtual bool ShouldAbortActorDeletion() const override final; // @note Final - Override ShouldAbortComponentDeletion or ShouldAbortActorDeletion (with parameters) instead.
	UNREALED_API virtual void CloseEditor() override;

	
	UNREALED_API virtual bool IsAutosaving(const EPackageAutoSaveType AutoSaveType = EPackageAutoSaveType::Transient) const override;
	//~ End UEditorEngine Interface 
	
	//~ Begin FExec Interface
#if UE_ALLOW_EXEC_COMMANDS
	UNREALED_API virtual bool Exec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog ) override;
#endif
	//~ End FExec Interface


	//~ Begin UEngine Interface.
	UNREALED_API virtual void Init(IEngineLoop* InEngineLoop) override;

	UNREALED_API virtual void PreExit() override;
	UNREALED_API virtual void Tick(float DeltaSeconds, bool bIdleMode) override;
	//~ End UEngine Interface.

	/** Builds a list of sprite categories for use in menus */
	static UNREALED_API void MakeSortedSpriteInfo(TArray<struct FSpriteCategoryInfo>& OutSortedSpriteInfo);

	/** called when a package has has its dirty state updated */
	UNREALED_API void OnPackageDirtyStateUpdated( UPackage* Pkg);
	/** called when a package's source control state is updated */
	UNREALED_API void OnSourceControlStateUpdated(const FSourceControlOperationRef& SourceControlOp, ECommandResult::Type ResultType, TArray<TWeakObjectPtr<UPackage>> Packages);
	/** called when a package is automatically checked out from source control */
	UNREALED_API void OnPackagesCheckedOut(const FSourceControlOperationRef& SourceControlOp, ECommandResult::Type ResultType, TArray<TWeakObjectPtr<UPackage>> Packages);
	/** caled by FCoreDelegate::PostGarbageCollect */
	UNREALED_API void OnPostGarbageCollect();
	/** called by color picker change event */
	UNREALED_API void OnColorPickerChanged();
	/** called by the viewport client before a windows message is processed */
	UNREALED_API void OnPreWindowsMessage(FViewport* Viewport, uint32 Message);
	/** called by the viewport client after a windows message is processed */
	UNREALED_API void OnPostWindowsMessage(FViewport* Viewport, uint32 Message);

	/** Register a function to draw extra information when a particular component is selected */
	UNREALED_API void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<class FComponentVisualizer> Visualizer);
	/** Unregister component visualizer function */
	UNREALED_API void UnregisterComponentVisualizer(FName ComponentClassName);
	/** Find a component visualizer for the given component class name */
	UNREALED_API TSharedPtr<class FComponentVisualizer> FindComponentVisualizer(FName ComponentClassName) const;

	/** Find a component visualizer for the given component class (checking parent classes too) */
	UNREALED_API TSharedPtr<class FComponentVisualizer> FindComponentVisualizer(UClass* ComponentClass) const;

	/** Draw component visualizers for components for selected actors */
	UNREALED_API void DrawComponentVisualizers(const FSceneView* View, FPrimitiveDrawInterface* PDI);
	/** Draw component visualizers HUD elements for components for selected actors */
	UNREALED_API void DrawComponentVisualizersHUD(const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas);

	/** Updates the property windows of selected actors */
	UNREALED_API void UpdateFloatingPropertyWindows(bool bForceRefresh=false, bool bNotifyActorSelectionChanged=true);

	/**
	*	Updates the property windows to show the data of the supplied ActorList
	*
	*	@param	ActorList	The list of actors to show the properties for
	*
	*/
	UNREALED_API void UpdateFloatingPropertyWindowsFromActorList(const TArray<AActor*>& ActorList, bool bForceRefresh=false);

	/**
	 * Called whenever the actor selection has changed to invalidate any cached state
	 */
	UNREALED_API void PostActorSelectionChanged();

	/**
	 * Set whether the pivot has been moved independently or not
	 */
	UNREALED_API void SetPivotMovedIndependently( bool bMovedIndependently );

	/**
	 * Return whether the pivot has been moved independently or not
	 */
	UNREALED_API bool IsPivotMovedIndependently() const;

	/**
	 * Called to reset the editor's pivot (widget) location using the currently selected objects.  Usually
	 * called when the selection changes.
	 * @param bOnChange Set to true when we know for a fact the selected object has changed
	 */
	UNREALED_API void UpdatePivotLocationForSelection( bool bOnChange = false );


	/**
	 * Replaces the specified actor with a new actor of the specified class.  The new actor
	 * will be selected if the current actor was selected.
	 *
	 * @param	CurrentActor			The actor to replace.
	 * @param	NewActorClass			The class for the new actor.
	 * @param	Archetype				The template to use for the new actor.
	 * @param	bNoteSelectionChange	If true, call NoteSelectionChange if the new actor was created successfully.
	 * @return							The new actor.
	 */
	UNREALED_API virtual AActor* ReplaceActor( AActor* CurrentActor, UClass* NewActorClass, UObject* Archetype, bool bNoteSelectionChange );


	/**
	 * @return Returns the global instance of the editor options class.
	 */
	UNREALED_API UUnrealEdOptions* GetUnrealEdOptions();

	/**
	 * Iterate over all levels of the world and create a list of world infos, then
	 * Iterate over selected actors and assemble a list of actors which can be deleted.
	 * @see CanDeleteComponent and CanDeleteActor.
	 *
	 * @param	InWorld					The world we want to examine
	 * @param	bStopAtFirst			Whether or not we should stop at the first deletable actor we encounter
	 * @param	bLogUndeletable			Should we log all the undeletable actors
	 * @param	OutDeletableActors		Can be NULL, provides a list of all the actors, from the selection, that are deletable
	 * @return							true if any of the selection can be deleted
	 */
	UNREALED_API bool CanDeleteSelectedActors( const UWorld* InWorld, const bool bStopAtFirst, const bool bLogUndeletable, TArray<AActor*>* OutDeletableActors = NULL ) const;

	// UnrealEdSrv stuff.
	UNREALED_API bool Exec_Edit( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Pivot( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Actor( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Element( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Mode( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool Exec_Group( const TCHAR* Str, FOutputDevice& Ar );


	// Editor actor virtuals from EditorActor.cpp.
	/**
	 * Select all actors and BSP models, except those which are hidden.
	 */
	UNREALED_API virtual void edactSelectAll( UWorld* InWorld );

	/**
	 * Invert the selection of all actors and BSP models.
	 */
	UNREALED_API virtual void edactSelectInvert( UWorld* InWorld );

	/**
	 * Select all children actors of the current selection.
	 *
	 * @param   bRecurseChildren	true to recurse through all descendants of the children
	 */
	UNREALED_API virtual void edactSelectAllChildren( bool bRecurseChildren );

	/**
	 * Select all actors in a particular class.
	 *
	 * @param	InWorld		World context
	 * @param	InClass		Class of actor to select
	 */
	UNREALED_API virtual void edactSelectOfClass( UWorld* InWorld, UClass* Class );

	/**
	 * Select all actors of a particular class and archetype.
	 *
	 * @param	InWorld		World context
	 * @param	InClass		Class of actor to select
	 * @param	InArchetype	Archetype of actor to select
	 */
	UNREALED_API virtual void edactSelectOfClassAndArchetype( UWorld* InWorld, const TSubclassOf<AActor> InClass, const UObject* InArchetype );

	/**
	 * Select all actors in a particular class and its subclasses.
	 *
	 * @param	InWorld		World context
	 */
	UNREALED_API virtual void edactSelectSubclassOf( UWorld* InWorld, UClass* Class );

	/**
	 * Select all actors in a level that are marked for deletion.
	 *
	 * @param	InWorld		World context
	 */
	UNREALED_API virtual void edactSelectDeleted( UWorld* InWorld );

	/**
	 * Select all actors that have the same static mesh assigned to them as the selected ones.
	 *
	 * @param bAllClasses		If true, also select non-AStaticMeshActor actors whose meshes match.
	 */
	UNREALED_API virtual void edactSelectMatchingStaticMesh(bool bAllClasses);

	/**
	 * Select all actors that have the same skeletal mesh assigned to them as the selected ones.
	 *
	 * @param bAllClasses		If true, also select non-ASkeletalMeshActor actors whose meshes match.
	 */
	UNREALED_API virtual void edactSelectMatchingSkeletalMesh(bool bAllClasses);

	/**
	 * Select all material actors that have the same material assigned to them as the selected ones.
	 */
	UNREALED_API virtual void edactSelectMatchingMaterial();

	/**
	 * Select all emitter actors that have the same particle system template assigned to them as the selected ones.
	 */
	UNREALED_API virtual void edactSelectMatchingEmitter();

	/**
	 * Select the relevant lights for all selected actors
	 *
	 * @param	InWorld					World context
	 */
	UNREALED_API virtual void edactSelectRelevantLights( UWorld* InWorld );

	/**
	 * Can the given component be deleted?
	 * 
	 * @param InComponent				Component to check
	 * @param OutReason					Optional value to fill with the reason the component cannot be deleted, if any
	 */
	UNREALED_API virtual bool CanDeleteComponent(const UActorComponent* InComponent, FText* OutReason = nullptr) const;

	/**
	 * Can the given actor be deleted?
	 *
	 * @param InActor					Actor to check
	 * @param OutReason					Optional value to fill with the reason the actor cannot be deleted, if any
	 */
	UNREALED_API virtual bool CanDeleteActor(const AActor* InActor, FText* OutReason = nullptr) const;

	/**
	 * Should the deletion of the given components be outright aborted?
	 *
	 * @param InComponentsToDelete		Components to check
	 * @param OutReason					Optional value to fill with the reason the component deletion was aborted, if any
	 */
	UNREALED_API virtual bool ShouldAbortComponentDeletion(const TArray<UActorComponent*>& InComponentsToDelete, FText* OutReason = nullptr) const;

	/**
	 * Should the deletion of the given actors be outright aborted?
	 *
	 * @param InActorsToDelete			Actors to check
	 * @param OutReason					Optional value to fill with the reason the actor deletion was aborted, if any
	 */
	UNREALED_API virtual bool ShouldAbortActorDeletion(const TArray<AActor*>& InActorsToDelete, FText* OutReason = nullptr) const;

	/**
	 * Delete the given components.
	 *
	 * @param	InComponentsToDelete		Array of components to delete
	 * @param	InSelectionSet				The selection set potentially containing to components that are being deleted
	 * @param	OutSuggestedNewSelection	Array to fill with suitable components to select post-delete
	 * @param	bVerifyDeletionCanHappen	If true (default), verify that deletion can be performed
	 * 
	 * @return								true unless the delete operation was aborted.
	 */
	UNREALED_API virtual bool DeleteComponents(const TArray<UActorComponent*>& InComponentsToDelete, UTypedElementSelectionSet* InSelectionSet, const bool bVerifyDeletionCanHappen = true);

	/**
	 * Deletes the given actors.
	 *
	 * @param	InActorsToDelete			Array of actors to delete
	 * @param	InWorld						World context
	 * @param	InSelectionSet				The selection set potentially containing to actors that are being deleted
	 * @param	bVerifyDeletionCanHappen	If true (default), verify that deletion can be performed
	 * @param	bWarnAboutReferences		If true (default), we prompt the user about referenced actors they are about to delete
	 * @param	bWarnAboutSoftReferences	If true (default), we prompt the user about soft references to actors they are about to delete
	 * 
	 * @return								true unless the delete operation was aborted.
	 */
	UNREALED_API virtual bool DeleteActors(const TArray<AActor*>& InActorsToDelete, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const bool bVerifyDeletionCanHappen = true, const bool bWarnAboutReferences = true, const bool bWarnAboutSoftReferences = true);

	/**
	 * Deletes all selected actors
	 * @note Final - Override DeleteComponents or DeleteActors instead
	 *
	 * @param	InWorld						World context
	 * @param	bVerifyDeletionCanHappen	[opt] If true (default), verify that deletion can be performed.
	 * @param	bWarnAboutReferences		[opt] If true (default), we prompt the user about referenced actors they are about to delete
	 * @param	bWarnAboutSoftReferences	[opt] If true (default), we prompt the user about soft references to actors they are about to delete
	 * @return								true unless the delete operation was aborted.
	 */
	UNREALED_API virtual bool edactDeleteSelected( UWorld* InWorld, bool bVerifyDeletionCanHappen=true, bool bWarnAboutReferences = true, bool bWarnAboutSoftReferences = true) override final;

	/**
	 * Copy selected actors to the clipboard.  Does not copy PrefabInstance actors or parts of Prefabs.
	 * @note Final - Override CopyComponents or CopyActors instead.
	 *
	 * @param	InWorld					World context
	 * @param	DestinationData			If != NULL, fill instead of clipboard data
	 */
	UNREALED_API virtual void edactCopySelected(UWorld* InWorld, FString* DestinationData = nullptr) override final;

	/**
	 * Copy the given components to the clipboard.
	 *
	 * @param	InComponentsToCopy		Array of components to copy
	 * @param	DestinationData			If != NULL, fill instead of clipboard data
	 */
	UNREALED_API virtual void CopyComponents(const TArray<UActorComponent*>& InComponentsToCopy, FString* DestinationData = nullptr) const;

	/**
	 * Copy the given actors to the clipboard.  Does not copy PrefabInstance actors or parts of Prefabs.
	 *
	 * @param	InActorsToCopy			Array of actors to copy
	 * @param	InWorld					World context
	 * @param	DestinationData			If != NULL, fill instead of clipboard data
	 */
	UNREALED_API virtual void CopyActors(const TArray<AActor*>& InActorsToCopy, UWorld* InWorld, FString* DestinationData = nullptr) const;

	/**
	 * Paste selected actors from the clipboard.
	 * @note Final - Override PasteComponents or PasteActors instead.
	 *
	 * @param	InWorld				World context
	 * @param	bDuplicate			Is this a duplicate operation (as opposed to a real paste)?
	 * @param	bOffsetLocations	Should the actor locations be offset after they are created?
	 * @param	bWarnIfHidden		If true displays a warning if the destination level is hidden
	 * @param	SourceData			If != NULL, use instead of clipboard data
	 */
	UNREALED_API virtual void edactPasteSelected(UWorld* InWorld, bool bDuplicate, bool bOffsetLocations, bool bWarnIfHidden, const FString* SourceData = nullptr) override final;

	/**
	 * Paste the components from the clipboard.
	 *
	 * @param	OutPastedComponents List of all the components that were pasted
	 * @param	TargetActor			The actor to attach the pasted components to
	 * @param	bWarnIfHidden		If true displays a warning if the destination level is hidden
	 * @param	SourceData			If != NULL, use instead of clipboard data
	 */
	UNREALED_API virtual void PasteComponents(TArray<UActorComponent*>& OutPastedComponents, AActor* TargetActor, const bool bWarnIfHidden, const FString* SourceData = nullptr);

	/**
	 * Paste the actors from the clipboard.
	 *
	 * @param	OutPastedActors		List of all the actors that were pasted
	 * @param	InWorld				World context
	 * @param	LocationOffset		Offset to apply to actor locations after they're created
	 * @param	bDuplicate			Is this a duplicate operation (as opposed to a real paste)?
	 * @param	bWarnIfHidden		If true displays a warning if the destination level is hidden
	 * @param	SourceData			If != NULL, use instead of clipboard data
	 */
	UNREALED_API virtual void PasteActors(TArray<AActor*>& OutPastedActors, UWorld* InWorld, const FVector& LocationOffset, bool bDuplicate, bool bWarnIfHidden, const FString* SourceData = nullptr);

	/**
	 * Duplicates selected actors.  Handles the case where you are trying to duplicate PrefabInstance actors.
	 * @note Final - Override DuplicateComponents or DuplicateActors instead.
	 *
	 * @param	InLevel				Level to place duplicate
	 * @param	bOffsetLocations	Should the actor locations be offset after they are created?
	 */
	UNREALED_API virtual void edactDuplicateSelected(ULevel* InLevel, bool bOffsetLocations) override final;

	/**
	 * Duplicate the given components.
	 * 
	 * @param	InComponentsToDuplicate		Array of components to duplicate
	 * @param	OutNewComponents			List of all the components that were duplicated
	 */
	UNREALED_API virtual void DuplicateComponents(const TArray<UActorComponent*>& InComponentsToDuplicate, TArray<UActorComponent*>& OutNewComponents);

	/**
	 * Duplicates the given actors.  Handles the case where you are trying to duplicate PrefabInstance actors.
	 *
	 * @param	InActorsToDuplicate	Array of actors to duplicate
	 * @param	OutNewActors		List of all the actors that were duplicated
	 * @param	InLevel				Level to place duplicate
	 * @param	LocationOffset		Offset to apply to actor locations after they're created
	 */
	UNREALED_API virtual void DuplicateActors(const TArray<AActor*>& InActorsToDuplicate, TArray<AActor*>& OutNewActors, ULevel* InLevel, const FVector& LocationOffset);

	/**
	 * Replace all selected brushes with the default brush.
	 *
	 * @param	InWorld					World context
	 */
	UNREALED_API virtual void edactReplaceSelectedBrush( UWorld* InWorld );

	/**
	 * Replace all selected non-brush actors with the specified class.
	 */
	UNREALED_API virtual void edactReplaceSelectedNonBrushWithClass(UClass* Class);

	/**
	 * Replace all actors of the specified source class with actors of the destination class.
	 *
	 * @param	InWorld		World context	 
	 * @param	SrcClass	The class of actors to replace.
	 * @param	DstClass	The class to replace with.
	 */
	UNREALED_API virtual void edactReplaceClassWithClass(UWorld* InWorld, UClass* SrcClass, UClass* DstClass);

	/**
	* Align the origin with the current grid.
	*/
	UNREALED_API virtual void edactAlignOrigin();

	/**
	 * Align all vertices with the current grid.
	 */
	UNREALED_API virtual void edactAlignVertices();

	/**
	 * Hide selected actors and BSP models by marking their bHiddenEdTemporary flags true. Will not
	 * modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactHideSelected( UWorld* InWorld );

	/**
	 * Hide unselected actors and BSP models by marking their bHiddenEdTemporary flags true. Will not
	 * modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactHideUnselected( UWorld* InWorld );

	/**
	 * Hide selected actors, their child hierarchy, and BSP models by marking their bHiddenEdTemporary flags true.
	 * Will not modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactHideSelectedHierarchy( UWorld* InWorld );

	/**
	 * Toggles visibility of selected actors. If any selected actors are visible, they will be made invisible.
	 * If all selected actors are invisible, the selection will be made visible. Will not modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactToggleSelectedVisibility( UWorld* InWorld );

	/**
	 * Toggles visibility of selected actors and their child hierarchy. If any selected actors are visible, they will be 
	 * made invisible. If all selected actors are invisible, the selection will be made visible. Will not modify/dirty
	 * actors/BSP.
	 */
	UNREALED_API virtual void edactToggleSelectedHierarchyVisibility( UWorld* InWorld );

	/**
	 * Attempt to unhide all actors and BSP models by setting their bHiddenEdTemporary flags to false if they
	 * are true. Note: Will not unhide actors/BSP hidden by higher priority visibility settings, such as bHiddenEdGroup,
	 * but also will not modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactUnHideAll( UWorld* InWorld );

	/**
	 * Mark all selected actors and BSP models to be hidden upon editor startup, by setting their bHiddenEd flag to
	 * true, if it is not already. This directly modifies/dirties the relevant actors/BSP.
	 */
	UNREALED_API virtual void edactHideSelectedStartup( UWorld* InWorld );

	/**
	 * Mark all selected actors, their child hierarchy, and BSP models to be hidden upon editor startup, by setting
	 * their bHiddenEd flag to true, if it is not already. This directly modifies/dirties the relevant actors/BSP.
	 */
	UNREALED_API virtual void edactHideSelectedHierarchyStartup( UWorld* InWorld );

	/**
	 * Mark all actors and BSP models to be shown upon editor startup, by setting their bHiddenEd flag to false, if it is
	 * not already. This directly modifies/dirties the relevant actors/BSP.
	 */
	UNREALED_API virtual void edactUnHideAllStartup( UWorld* InWorld );

	/**
	 * Mark all selected actors and BSP models to be shown upon editor startup, by setting their bHiddenEd flag to false, if it
	 * is not already. This directly modifies/dirties the relevant actors/BSP.
	 */
	UNREALED_API virtual void edactUnHideSelectedStartup( UWorld* InWorld );

	/**
	 * Mark all selected actors, their child hierarchy, and BSP models to be shown upon editor startup, by setting their bHiddenEd
	 * flag to false, if it is not already. This directly modifies/dirties the relevant actors/BSP.
	 */
	UNREALED_API virtual void edactUnHideSelectedHierarchyStartup( UWorld* InWorld );

	/**
	 * Show selected actors and BSP models by marking their bHiddenEdTemporary flags false. Will not modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactUnhideSelected( UWorld* InWorld );
	
	/**
	 * Show selected actors, their child hierarchy, and BSP models by marking their bHiddenEdTemporary flags false. Will
	 * not modify/dirty actors/BSP.
	 */
	UNREALED_API virtual void edactUnhideSelectedHierarchy( UWorld* InWorld );

	/** Will create a map of currently visible BSP surfaces. */
	UNREALED_API virtual void CreateBSPVisibilityMap( UWorld* InWorld, TMap<AActor*, TArray<int32>>& OutBSPMap, bool& bOutAllVisible );

	/** Go through a map of BSP and make only the requested objects visible. */
	UNREALED_API virtual void MakeBSPMapVisible(const TMap<AActor*, TArray<int32>>& InBSPMap, UWorld* InWorld );

	/** Returns the configuration of attachment that would result from calling AttachSelectedActors at this point in time */
	UNREALED_API AActor* GetDesiredAttachmentState(TArray<AActor*>& OutNewChildren);

	/** Uses the current selection state to attach actors together. Last selected Actor becomes the base. */
	UNREALED_API void AttachSelectedActors();


	
	/**
	 * Can the editor do cook by the book in the editor process space
	 */
	UNREALED_API virtual bool CanCookByTheBookInEditor(const FString& PlatformName) const override;

	/**
	 * Can the editor act as a cook on the fly server
	 */
	UNREALED_API virtual bool CanCookOnTheFlyInEditor(const FString& PlatformName) const override;

	/**
	 * Start cook by the book in the editor process space
	 */
	UNREALED_API virtual void StartCookByTheBookInEditor( const TArray<ITargetPlatform*> &TargetPlatforms, const TArray<FString> &CookMaps, const TArray<FString> &CookDirectories, const TArray<FString> &CookCultures, const TArray<FString> &IniMapSections ) override;

	/**
	 * Checks if the cook by the book is finished
	 */
	UNREALED_API virtual bool IsCookByTheBookInEditorFinished() const override;


	/**
	 * cancels the current cook by the book in editor
	 */
	UNREALED_API virtual void CancelCookByTheBookInEditor() override;



	// Hook replacements.
	UNREALED_API void ShowActorProperties();

	/**
	 * Checks to see if any worlds are dirty (that is, they need to be saved.)
	 *
	 * @param	InWorld	World to search for dirty worlds
	 * 
	 * @return true if any worlds are dirty
	 */
	UNREALED_API bool AnyWorldsAreDirty( UWorld* InWorld ) const;

	/**
	 * Checks to see if any content packages are dirty (that is, they need to be saved.)
	 *
	 * @return true if any content packages are dirty
	 */
	UNREALED_API bool AnyContentPackagesAreDirty() const;

	// Misc
	/**
	 * Attempts to prompt the user with a balloon notification to checkout modified packages from source control.
	 * Will defer prompting the user if they are interacting with something
	 */
	UNREALED_API void AttemptModifiedPackageNotification();

	/**
	 * Prompts the user with a modal checkout dialog to checkout packages from source control.
	 * This should only be called by the auto prompt to checkout package notification system.
	 * For a general checkout packages routine use FEditorFileUtils::PromptToCheckoutPackages
	 *
	 * @param bPromptAll	If true we prompt for all packages in the PackageToNotifyState map.  If false only prompt about ones we have never prompted about before.
	 */
	UNREALED_API void PromptToCheckoutModifiedPackages( bool bPromptAll = false );

	/**
	 * Displays a toast notification or warning when a package is dirtied, indicating that it needs checking out (or that it cannot be checked out)
	 */
	UNREALED_API void ShowPackageNotification();

	/**
	 * @return Returns the number of dirty packages that require checkout.
	 */
	UNREALED_API int32 GetNumDirtyPackagesThatNeedCheckout() const;

	/**
	 * Checks to see if there are any packages in the PackageToNotifyState map that are not checked out by the user
	 *
	 * @return True if packages need to be checked out.
	 */
	UNREALED_API bool DoDirtyPackagesNeedCheckout() const;

	/**
	 * Checks whether the specified map is a template map.
	 *
	 * @return true if the map is a template map, false otherwise.
	 */
	UNREALED_API bool IsTemplateMap( const FString& MapName ) const;

	UNREALED_API void RebuildTemplateMapData();

	/**
	 * Appends the specified template maps to the list of available templates.
	 */
	UNREALED_API void AppendTemplateMaps( const TArray<FTemplateMapInfo>& TemplateMapInfos );

	/**
	 * Returns true if the user is currently interacting with a viewport.
	 */
	UNREALED_API bool IsUserInteracting();

	UNREALED_API void SetCurrentClass( UClass* InClass );

	/**
	 * @return true if selection of translucent objects in perspective viewports is allowed
	 */
	UNREALED_API virtual bool AllowSelectTranslucent() const override;

	/**
	 * @return true if only editor-visible levels should be loaded in Play-In-Editor sessions
	 */
	UNREALED_API virtual bool OnlyLoadEditorVisibleLevelsInPIE() const override;

	/**
	 * @return true if level streaming should prefer to stream levels from disk instead of duplicating them from editor world
	 */
	UNREALED_API virtual bool PreferToStreamLevelsInPIE() const override;

	/**
	 * Duplicate the currently selected actors.
	 *
	 * This is a high level routine which may ultimately call edactDuplicateSelected
	 */
	UNREALED_API void DuplicateSelectedActors(UWorld* InWorld);

	/**
	 * If all selected actors belong to the same level, that level is made the current level.
	 */
	UNREALED_API void MakeSelectedActorsLevelCurrent();

	/** Returns the thumbnail manager and creates it if missing */
	UNREALED_API class UThumbnailManager* GetThumbnailManager();

	/**
	 * Returns whether saving the specified package is allowed
	 */
	UNREALED_API virtual bool CanSavePackage( UPackage* PackageToSave );

	/**
	 * Updates the volume actor visibility for all viewports based on the passed in volume class
	 *
	 * @param InVolumeActorClass	The type of volume actors to update.  If NULL is passed in all volume actor types are updated.
	 * @param InViewport			The viewport where actor visibility should apply.  Pass NULL for all editor viewports.
	 */
	UNREALED_API void UpdateVolumeActorVisibility( UClass* InVolumeActorClass = NULL , FLevelEditorViewportClient* InViewport = NULL);

	/**
	 * Identify any brushes whose sense is inverted and repair them
	 */
	UNREALED_API void FixAnyInvertedBrushes(UWorld* World);

	/**
	 * Get the index of the provided sprite category
	 *
	 * @param	InSpriteCategory	Sprite category to get the index of
	 *
	 * @return	Index of the provided sprite category, if possible; INDEX_NONE otherwise
	 */
	UNREALED_API virtual int32 GetSpriteCategoryIndex( const FName& InSpriteCategory ) override;
	
	/**
	 * Shows the LightingStaticMeshInfoWindow, creating it first if it hasn't been initialized.
	 */
	UNREALED_API void ShowLightingStaticMeshInfoWindow();
	
	/**
	 * Shows the SceneStatsWindow, creating it first if it hasn't been initialized.
	 */
	UNREALED_API void OpenSceneStatsWindow();

	/**
	 * Shows the TextureStatsWindow, creating it first if it hasn't been initialized.
	 */
	UNREALED_API void OpenTextureStatsWindow();

	/**
	* Puts all of the AVolume classes into the passed in array and sorts them by class name.
	*
	* @param	VolumeClasses		Array to populate with AVolume classes.
	*/
	static UNREALED_API void GetSortedVolumeClasses( TArray< UClass* >* VolumeClasses );

	/**
	 * Checks the destination level visibility and warns the user if they are trying to paste to a hidden level, offering the option to cancel the operation or unhide the level that is hidden
	 * 
	 * @param InWorld			World context
	 */
	UNREALED_API bool WarnIfDestinationLevelIsHidden( UWorld* InWorld ) const;

	/**
	 * Generate the package thumbails if they are needed. 
	 */
	UNREALED_API UPackage* GeneratePackageThumbnailsIfRequired( const TCHAR* Str, FOutputDevice& Ar, TArray<FString>& ThumbNamesToUnload );

	/** @return The package auto-saver instance used by the editor */
	IPackageAutoSaver& GetPackageAutoSaver() const
	{
		return *PackageAutoSaver;
	}

	/**
	 * Exec command handlers
	 */
	UNREALED_API bool HandleDumpModelGUIDCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleModalTestCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleDisallowExportCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleDumpBPClassesCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleFindOutdateInstancesCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleDumpSelectionCommand( const TCHAR* Str, FOutputDevice& Ar );
	UNREALED_API bool HandleBuildLightingCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool HandleBuildPathsCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld );
	UNREALED_API bool HandleRecreateLandscapeCollisionCommand(const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld);
	UNREALED_API bool HandleRemoveLandscapeXYOffsetsCommand(const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld);
	UNREALED_API bool HandleDisasmScriptCommand( const TCHAR* Str, FOutputDevice& Ar );	

	UNREALED_API bool IsComponentSelected(const UPrimitiveComponent* PrimComponent);

	/** Return if we have write permission under the mount point this packages lives in. */
	UNREALED_API bool HasMountWritePermissionForPackage(const FString& PackageName);

	/* Delegate to override TemplateMapInfos */
	DECLARE_DELEGATE_RetVal(const TArray<FTemplateMapInfo>&, FGetTemplateMapInfos);
	FGetTemplateMapInfos& OnGetTemplateMapInfos() { return GetTemplateMapInfosDelegate; }

	/** Delegate called when UUnrealEdEngine::PasteActors starts executing */
	DECLARE_MULTICAST_DELEGATE(FOnPasteActorsBegin);
	FOnPasteActorsBegin& OnPasteActorsBegin() { return OnPasteActorsBeginDelegate; }

	/** Delegate called when UUnrealEdEngine::PasteActors is done executing */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPasteActorsEnd, const TArray<AActor*>&)
	FOnPasteActorsEnd& OnPasteActorsEnd() { return OnPasteActorsEndDelegate; }

	/** Gets the canonical list of map templates that should be visible in new level picker. This function calls OnGetTemplateMapInfos to allow runtime override of the default maps */
	UNREALED_API const TArray<FTemplateMapInfo>& GetTemplateMapInfos() const;

	/** Gets the project default map templates without any runtime overrides */
	UNREALED_API const TArray<FTemplateMapInfo>& GetProjectDefaultMapTemplates() const;

protected:
	/** Called when the element selection set pointer set on the global editor selection changes */
	UNREALED_API void OnEditorElementSelectionPtrChanged(USelection* Selection, UTypedElementSelectionSet* OldSelectionSet, UTypedElementSelectionSet* NewSelectionSet);

	/** Called when the element selection set associated with the global editor selection changes */
	UNREALED_API void OnEditorElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet);

	/** Called when a HISM tree has finished building */
	UNREALED_API void OnHISMTreeBuilt(UHierarchicalInstancedStaticMeshComponent* Component, bool bWasAsyncBuild);

	/** The package auto-saver instance used by the editor */
	TUniquePtr<IPackageAutoSaver> PackageAutoSaver;

	/**
	 * The list of visualizers to draw when selection changes
	 */
	struct FComponentVisualizerForSelection
	{
		FCachedComponentVisualizer ComponentVisualizer;
		TOptional<TFunction<bool(void)>> IsEnabledDelegate;
	};

	TArray<FComponentVisualizerForSelection> VisualizersForSelection;

	/** Instance responsible for monitoring this editor's performance */
	FPerformanceMonitor* PerformanceMonitor;

	/** Whether the pivot has been moved independently */
	bool bPivotMovedIndependently;

	/** Weak Pointer to the file checkout notification toast. */
	TWeakPtr<SNotificationItem> CheckOutNotificationWeakPtr;

private:
	/** 
	 * Helper function to recursively add the children of a given actor to the OutActorArray. This is used for Hide/Show Selected functions to
	 * hide/show their child hierarchy. Set bExcludeSelectedChildren to true when calling this function in a loop of all selected actors, we
	 * want the helper function to not travel down pre-selected children so there won't be duplicate entries and unnecessary complexity.
	 */
	static void GetChildHierarchyOfSelectedActor(TArray<AActor*>& OutActorArray, AActor* ParentActor, const bool bExcludeSelectedChildren = false, const TOptional<bool>& bIsFindingHidden = TOptional<bool>());
	
	/**
	 * Internal function to hide selected Actors, allows bApplyToChildHierarchy to define whether we also want to hide the
	 * child hierarchies of the selected actors.
	 */
	void Internal_HideSelected(UWorld* InWorld, bool bApplyToChildHierarchy);
	
	/**
	 * Internal function to unhide selected Actors, allows bApplyToChildHierarchy to define whether we also want to unhide
	 * the child hierarchies of the selected actors.
	 */
	void Internal_UnhideSelected(UWorld* InWorld, bool bApplyToChildHierarchy);
	
	/**
	 * Internal function to toggle the visibility of selected Actors, allows bApplyToChildHierarchy to define whether we also
	 * want to toggle the child hierarchies of the selected actors.
	 */
	void Internal_ToggleSelectedVisibility(UWorld* InWorld, bool bApplyToChildHierarchy);
	
	/**
	 * Internal function to hide selected Actors on startup, allows bApplyToChildHierarchy to define whether we also want to
	 * hide the current child hierarchies of the selected actors.
	 */
	void Internal_HideSelectedStartup(UWorld* InWorld, bool bApplyToChildHierarchy);
	
	/**
	 * Internal function to unhide selected Actors on startup, allows bApplyToChildHierarchy to define whether we also want
	 * to unhide the current child hierarchies of the selected actors.
	 */
	void Internal_UnHideSelectedStartup(UWorld* InWorld, bool bApplyToChildHierarchy);
	
	/** Verify if we have write permission under the specified mount point. */
	UNREALED_API bool VerifyMountPointWritePermission(FName MountPoint);

	/** Delegate when a new mount point is added, used to test writing permission. */
	UNREALED_API void OnContentPathMounted(const FString& AssetPath, const FString& FileSystemPath);

	/** Delegate when a new mount point is removed, used to test writing permission. */
	UNREALED_API void OnContentPathDismounted(const FString& AssetPath, const FString& FileSystemPath);

	/** Map to track which mount point has write permissions. */
	TMap<FName, bool> MountPointCheckedForWritePermission;

	/** Weak Pointer to the write permission warning toast. */
	TWeakPtr<SNotificationItem> WritePermissionWarningNotificationWeakPtr;

	/* Delegate to override TemplateMapInfos */
	FGetTemplateMapInfos GetTemplateMapInfosDelegate;

	/** Delegate called when UUnrealEdEngine::PasteActors starts executing */
	FOnPasteActorsBegin OnPasteActorsBeginDelegate;

	/** Delegate called when UUnrealEdEngine::PasteActors is done executing */
	FOnPasteActorsEnd OnPasteActorsEndDelegate;

	/** Transient unsaved version of template map infos used by the editor. */
	TArray<FTemplateMapInfo> TemplateMapInfoCache;

	/** Proxy for a cotf server running in a separate process */
	class FExternalCookOnTheFlyServer* ExternalCookOnTheFlyServer = nullptr;

	/**
	* Internal helper function to count how many dirty packages require checkout.
	*
	* @param	bCheckIfAny		If true, checks instead if any packages need checkout.
	*
	* @return	Returns the number of dirty packages that require checkout. If bCheckIfAny is true, returns 1 if any packages will require checkout.
	*/
	UNREALED_API int32 InternalGetNumDirtyPackagesThatNeedCheckout(bool bCheckIfAny) const;

	/** Internal function to filter and add visualizers to a specific list */
	UNREALED_API void AddVisualizers(AActor* Actor, TArray<FCachedComponentVisualizer>& Visualizers, TFunctionRef<bool(const TSharedPtr<FComponentVisualizer>&)> Condition);

	/** Keep track of need to stop stall detector thread */
	bool bRequiresStallDetectorShutdown = false;
};
