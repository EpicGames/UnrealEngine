// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	This will hold all of our enums and types and such that we need to
 *	use in multiple files where the enum can't be mapped to a specific file.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UnrealEdTypes.generated.h"

// Structures for 'optional' Lightmass parameters...

/** Base LightmassParameterValue class */
USTRUCT()
struct FLightmassParameterValue
{
	GENERATED_USTRUCT_BODY()

	/** If true, override the given parameter with the given settings */
	UPROPERTY(EditAnywhere, Category=LightmassParameterValue)
	uint32 bOverride:1;


	FLightmassParameterValue()
		: bOverride(false)
	{
	}

};

/** Boolean parameter value */
USTRUCT()
struct FLightmassBooleanParameterValue : public FLightmassParameterValue
{
	GENERATED_USTRUCT_BODY()

	/** The boolean value to override the parent value with */
	UPROPERTY(EditAnywhere, Category=LightmassBooleanParameterValue)
	uint32 ParameterValue:1;


	FLightmassBooleanParameterValue()
		: ParameterValue(false)
	{
	}

};

/** Scalar parameter value */
USTRUCT()
struct FLightmassScalarParameterValue : public FLightmassParameterValue
{
	GENERATED_USTRUCT_BODY()

	/** The scalar value to override the parent value with */
	UPROPERTY(EditAnywhere, Category=LightmassScalarParameterValue)
	float ParameterValue;



		FLightmassScalarParameterValue(): ParameterValue(0)
		{
		}
		FLightmassScalarParameterValue(float InParameterValue): ParameterValue(InParameterValue)
		{
		}
	
};

/** Structure for 'parameterized' Lightmass settings */
USTRUCT()
struct FLightmassParameterizedMaterialSettings
{
	GENERATED_USTRUCT_BODY()

	/** If true, forces translucency to cast static shadows as if the material were masked. */
	UPROPERTY(EditAnywhere, Category=LightmassParameterizedMaterialSettings)
	struct FLightmassBooleanParameterValue CastShadowAsMasked;

	/** Scales the emissive contribution of this material to static lighting. */
	UPROPERTY(EditAnywhere, Category=LightmassParameterizedMaterialSettings)
	struct FLightmassScalarParameterValue EmissiveBoost;

	/** Scales the diffuse contribution of this material to static lighting. */
	UPROPERTY(EditAnywhere, Category=LightmassParameterizedMaterialSettings)
	struct FLightmassScalarParameterValue DiffuseBoost;

	/** 
	 * Scales the resolution that this material's attributes were exported at. 
	 * This is useful for increasing material resolution when details are needed.
	 */
	UPROPERTY(EditAnywhere, Category=LightmassParameterizedMaterialSettings)
	struct FLightmassScalarParameterValue ExportResolutionScale;

	FLightmassParameterizedMaterialSettings()
		: EmissiveBoost(1.0f)
		, DiffuseBoost(1.0f)
		, ExportResolutionScale(1.0f)
	{}
};


//
//	ELevelViewportType
//
UENUM()
enum ELevelViewportType : int
{
	/** Top */
	LVT_OrthoXY = 0,
	/** Right */
	LVT_OrthoXZ = 1,
	/** Back */
	LVT_OrthoYZ = 2,
	
	LVT_Perspective = 3,
	LVT_OrthoFreelook = 4,
	/** Bottom */
	LVT_OrthoNegativeXY = 5,
	/** Left */
	LVT_OrthoNegativeXZ = 6,
	/** Front */
	LVT_OrthoNegativeYZ = 7,

	/** Alias semantic meaning of viewport types for orthographic views */
	/** Top */
	LVT_OrthoTop = LVT_OrthoXY,
	/** Left */
	LVT_OrthoLeft = LVT_OrthoXZ,
	/** Front */
	LVT_OrthoFront = LVT_OrthoNegativeYZ,
	/** Back */
	LVT_OrthoBack = LVT_OrthoYZ,
	/** Bottom */
	LVT_OrthoBottom = LVT_OrthoNegativeXY,
	/** Right */
	LVT_OrthoRight = LVT_OrthoNegativeXZ,
	LVT_MAX = 8,

	LVT_None = 255,
};

UENUM()
enum EDestructiveAssetActions : int
{
	AssetDelete = 0,
	AssetRename = 1 << 0,
	AssetMove = 1 << 1,
	AssetPrivatize = 1 << 2
};

/** Simple Return struct for supplying success or failure with an optional error message */
struct FResultMessage
{
	bool bSuccess = false;
	FString ErrorMessage;
};

UCLASS(abstract, config=UnrealEd)
class UUnrealEdTypes : public UObject
{
	GENERATED_UCLASS_BODY()

};

