// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "ActorExporterT3D.generated.h"

class FExportObjectInnerContext;
class UActorComponent;

UCLASS(MinimalAPI)
class UActorExporterT3D : public UExporter
{
public:
	GENERATED_BODY()

public:
	UNREALED_API UActorExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	UNREALED_API virtual bool ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) override;
	virtual bool SupportsObject(UObject* Object) const override;
	//~ End UExporter Interface
};

UCLASS(MinimalAPI)
class UGroupActorExporterT3D : public UActorExporterT3D
{
	GENERATED_BODY()
public:
	UGroupActorExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};

UCLASS(MinimalAPI)
class UPhysicsVolumeExporterT3D : public UActorExporterT3D
{
	GENERATED_BODY()
public:
	UPhysicsVolumeExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};