// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Import data and options used when export an animation sequence
 */

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Curves/RealCurve.h"
#include "AnimSeqExportOption.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAnimSeqExportOption : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	/** If enabled, export the transforms from the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bExportTransforms = true;

	/** If enabled, export the morph targets from the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bExportMorphTargets = true;

	/** If enabled, export the attribute curves from the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bExportAttributeCurves = true;

	/** If enabled, export the material curves from the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bExportMaterialCurves = true;

	/** If enabled, we will skip any curves that have only zero values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bSkipCurvesWithZeroValue = true;

	/** If enabled we record in World Space otherwise we record from 0,0,0 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bRecordInWorldSpace = false;

	/** If true we evaluate all other skeletal mesh components under the same actor, this may be needed for example, to get physics to get baked*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);	
	bool bEvaluateAllSkeletalMeshComponents = true;

	/** If enabled set the retarget source asset from current skel mesh component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bSetRetargetSourceAsset = false;

	/** This defines how values between keys are calculated for transforms*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	EAnimInterpolationType Interpolation = EAnimInterpolationType::Linear;

	/** This defines how values between keys are calculated for curves*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	TEnumAsByte<ERichCurveInterpMode> CurveInterpolation = ERichCurveInterpMode::RCIM_Linear;

	/** Include only the animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	TArray<FString> IncludeAnimationNames;

	/** Exclude all animation bones/curves that match this list */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	TArray<FString> ExcludeAnimationNames;

	/** If enabled, curves in ExcludeAnimationNames that already exist in the target sequence will be removed at the start of recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bRemoveExcludedCurves = false;

	/** Number of Display Rate frames to evaluate before doing the export. It will evaluate after any Delay. This will use frames before the start frame. Use it if there is some post anim BP effects you want to run before export start time.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	FFrameNumber WarmUpFrames;

	/** Number of Display Rate frames to delay at the same frame before doing the export. It will evalaute first, then any warm up, then the export. Use it if there is some post anim BP effects you want to ran repeatedly at the start.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export);
	FFrameNumber DelayBeforeStart;

	/** Whether or not to transact the animation sequence data recording */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bTransactRecording = true;

	/** Set to true if sequence timecode should be baked into the sequence. Timecode rate will default to the project setting "Generate Default Timecode Frame Rate" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bBakeTimecode = false;

	/** Set to true if the timecode rate should be overridden with the specified value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (InlineEditConditionToggle), Category = Export)
	bool bTimecodeRateOverride = false;

	/** Overriding timecode rate to be used when baking. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (EditCondition = bTimecodeRateOverride), Category = Export)
	FFrameRate OverrideTimecodeRate;

	/** Whether or not to use custom time range */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bUseCustomTimeRange = false;

	/** Custom start frame in custom display rate*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (EditCondition = bUseCustomTimeRange),  Category = Export)
	FFrameNumber CustomStartFrame = 0;

	/** Custom end frame in custom display rate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, meta = (EditCondition = bUseCustomTimeRange),  Category = Export)
	FFrameNumber CustomEndFrame = 120;

	/** Custom display rate for use when specifying custom start and end frame, should be set from the movie scene/sequencer display rate */
	UPROPERTY(BlueprintReadWrite, AdvancedDisplay, meta = (EditCondition = bUseCustomTimeRange), Category = Export)
	FFrameRate CustomDisplayRate = FFrameRate(30,1);

	/** Whether or not to use custom frame rate when recording the anim sequence, if false will use Sequencers display rate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Export)
	bool bUseCustomFrameRate = false;

	/** Custom frame rate that the anim sequence will be recorded at*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = bUseCustomFrameRate), AdvancedDisplay, Category = Export)
	FFrameRate CustomFrameRate = FFrameRate(30, 1);
	
	UNREALED_API void ResetToDefault();
};

#undef UE_API
