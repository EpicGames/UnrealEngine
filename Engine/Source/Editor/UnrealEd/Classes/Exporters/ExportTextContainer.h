// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ExportTextContainer.generated.h"

UCLASS(MinimalAPI)
class UExportTextContainer : public UObject
{
	GENERATED_UCLASS_BODY()

	/** ExportText representation of one or more objects */
	UPROPERTY()
	FString ExportText;
};



