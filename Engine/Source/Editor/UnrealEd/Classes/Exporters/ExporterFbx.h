// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ExporterFBX
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "ExporterFbx.generated.h"

class UFbxExportOption;

UCLASS()
class UExporterFBX : public UExporter
{
public:
	GENERATED_BODY()

	class UFbxExportOption* GetAutomatedExportOptionsFbx();
};

