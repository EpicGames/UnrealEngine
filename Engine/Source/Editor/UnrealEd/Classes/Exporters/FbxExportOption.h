// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Import data and options used when importing a static mesh from fbx
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "FbxExportOption.generated.h"

 // Fbx export compatibility
UENUM(BlueprintType)
enum class EFbxExportCompatibility : uint8
{
	FBX_2011,
	FBX_2012,
	FBX_2013,
	FBX_2014,
	FBX_2016,
	FBX_2018,
	FBX_2019,
	FBX_2020,
};

// Bake options for animated properties of exported objects
UENUM(BlueprintType)
enum class EMovieSceneBakeType : uint8
{
	None = 0x000,

	BakeChannels = 0x001,
	BakeTransforms = 0x002,

	BakeAll = BakeChannels | BakeTransforms,
};

UENUM(BlueprintType)
enum class EFbxMaterialBakeMode : uint8
{
	/** Never bake material inputs. */
	Disabled,
	/** Only use a simple quad if a material input needs to be baked out. */
	Simple,
	/** Allow usage of the mesh data if a material input needs to be baked out with vertex data. */
	UseMeshData,
};

USTRUCT(Blueprintable)
struct FFbxMaterialBakeSize
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	FIntPoint Size = FIntPoint(1024, 1024);

	/** If enabled, bake size is based on the largest texture used in the material input's expression graph. If none found, bake size will fall back to the explicit dimensions. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "")
	bool bAutoDetect = true;
};

UCLASS(config = EditorPerProjectUserSettings, MinimalAPI, BlueprintType)
class UFbxExportOption : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	/** This will set the fbx sdk compatibility when exporting to fbx file. The default value is 2013 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Exporter)
	EFbxExportCompatibility FbxExportCompatibility;

	/** If enabled, save as ascii instead of binary */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Exporter)
	uint32 bASCII : 1;

	/** If enabled, export with X axis as the front axis instead of default -Y */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Exporter)
	uint32 bForceFrontXAxis : 1;

	/** If enabled, export vertex color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Mesh)
	uint32 VertexColor : 1;

	/** If enabled, export the level of detail */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Mesh)
	uint32 LevelOfDetail : 1;

	/** If enabled, export collision */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = StaticMesh, meta = (EditCondition = "!bExportSourceMesh"))
	uint32 Collision : 1;

	/*
	 * If enabled, export the highest LOD source data instead of the render data.
	 * Note:
	 *     - No LOD will be exported for static meshes. (Level Of Detail option will be ignored)
	 *     - No Collision will be exported for static meshes. (Collision option will be ignore)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = StaticMesh)
	uint32 bExportSourceMesh : 1;

	/** If enabled, export the morph targets */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = SkeletalMesh)
	uint32 bExportMorphTargets : 1;

	/** If enable, the preview mesh link to the exported animations will be also exported. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Animation)
	uint32 bExportPreviewMesh : 1;

	/** If enable, Map skeletal actor motion to the root bone of the skeleton. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Animation)
	uint32 MapSkeletalMotionToRoot : 1;

	/** If enabled, export sequencer animation in its local time, relative to its sequence. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, category = Animation)
	uint32 bExportLocalTime : 1;

	/** Bake settings for camera and light animation curves. Camera Scale not exported. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Animation)
	EMovieSceneBakeType BakeCameraAndLightAnimation;

	/** Bake settings for exported non-camera, non-light object animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Animation)
	EMovieSceneBakeType BakeActorAnimation;

	/** Bake mode determining if and how a material input is baked out to a texture. Baking is only used for non-trivial material inputs (i.e. not simple texture or constant expressions). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material)
	EFbxMaterialBakeMode BakeMaterialInputs;

	/** Default size of the baked out texture (containing the material input).*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = Material, Meta = (EditCondition = "BakeMaterialInputs != EFbxMaterialBakeMode::Disabled"))
	FFbxMaterialBakeSize DefaultMaterialBakeSize;

	/* Set all the FProperty to the CDO value */
	void ResetToDefault();

	/* Save the FProperty to a local ini to retrieve the value the next time we call function LoadOptions() */
	virtual void SaveOptions();
	
	/* Load the FProperty data from a local ini which the value was store by the function SaveOptions() */
	virtual void LoadOptions();
};
