// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LevelExporterFBX
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "ExporterFbx.h"
#include "LevelExporterFBX.generated.h"

UCLASS()
class ULevelExporterFBX : public UExporterFBX
{
public:
	GENERATED_BODY()

public:
	ULevelExporterFBX(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};



