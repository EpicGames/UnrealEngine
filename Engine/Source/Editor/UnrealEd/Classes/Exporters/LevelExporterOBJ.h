// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LevelExporterOBJ
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "LevelExporterOBJ.generated.h"

class FExportObjectInnerContext;

UCLASS()
class ULevelExporterOBJ : public UExporter
{
public:
	GENERATED_BODY()

public:
	ULevelExporterOBJ(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};



