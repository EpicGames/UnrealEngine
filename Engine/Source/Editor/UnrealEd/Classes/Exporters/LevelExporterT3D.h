// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LevelExporterT3D
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "LevelExporterT3D.generated.h"

class FExportObjectInnerContext;
class UActorComponent;

UCLASS()
class ULevelExporterT3D : public UExporter
{
public:
	GENERATED_BODY()

public:
	ULevelExporterT3D(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags=0 ) override;
	virtual void ExportPackageObject(FExportPackageParams& ExpPackageParams) override;
	virtual void ExportPackageInners(FExportPackageParams& ExpPackageParams) override;
	virtual void ExportComponentExtra(const FExportObjectInnerContext* Context, const TArray<UActorComponent*>& Components, FOutputDevice& Ar, uint32 PortFlags) override;
	//~ End UExporter Interface
};



