// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// URenderTargetExporterPNG+URenderTargetExporterEXR
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "RenderTargetExporterHDR.generated.h"


// note : URenderTargetExporterHDR deleted
//	export to HDR is discouraged, use PNG or EXR


UCLASS(MinimalAPI)
class URenderTargetExporterPNG : public UExporter
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UExporter Interface
	UNREALED_API virtual bool SupportsObject(UObject* Object) const override;
	UNREALED_API virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};



UCLASS(MinimalAPI)
class URenderTargetExporterEXR : public UExporter
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UExporter Interface
	UNREALED_API virtual bool SupportsObject(UObject* Object) const override;
	UNREALED_API virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};

