// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SkeletalMeshExporterFBX
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "ExporterFbx.h"
#include "SkeletalMeshExporterFBX.generated.h"

UCLASS()
class USkeletalMeshExporterFBX : public UExporterFBX
{
	GENERATED_UCLASS_BODY()


	//~ Begin UExporter Interface
	virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};



