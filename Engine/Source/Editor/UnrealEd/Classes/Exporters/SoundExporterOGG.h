// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "SoundExporterOGG.generated.h"

UCLASS()
class USoundExporterOGG : public UExporter
{
public:
	GENERATED_BODY()

public:
	USoundExporterOGG(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	virtual bool SupportsObject(UObject* Object) const override;
	//~ End UExporter Interface
};


