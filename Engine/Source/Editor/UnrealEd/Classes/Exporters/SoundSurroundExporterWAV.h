// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// SoundSurroundExporterWAV
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Exporters/Exporter.h"
#include "SoundSurroundExporterWAV.generated.h"

UCLASS()
class USoundSurroundExporterWAV : public UExporter
{
public:
	GENERATED_BODY()

public:
	USoundSurroundExporterWAV(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	virtual int32 GetFileCount( UObject* Object ) const override;
	virtual FString GetUniqueFilename( UObject* Object, const TCHAR* Filename, int32 FileIndex, int32 FileCount ) const override;
	virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	virtual bool SupportsObject(UObject* Object) const override;
	//~ End UExporter Interface
};



