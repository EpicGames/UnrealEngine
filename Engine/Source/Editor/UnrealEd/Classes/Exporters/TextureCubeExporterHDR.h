// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureCubeExporterHDR
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "TextureCubeExporterHDR.generated.h"

UCLASS(MinimalAPI)
class UTextureCubeExporterHDR : public UExporter
{
	GENERATED_UCLASS_BODY()

	//~ Begin UExporter Interface
	UNREALED_API virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0) override;
	//~ End UExporter Interface
};

UCLASS(MinimalAPI)
class URenderTargetCubeExporterHDR : public UTextureCubeExporterHDR
{
	GENERATED_UCLASS_BODY()
};





