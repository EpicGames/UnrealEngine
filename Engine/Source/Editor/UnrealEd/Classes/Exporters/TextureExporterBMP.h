// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// TextureExporterBMP
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "TextureExporterBMP.generated.h"

UCLASS()
class UTextureExporterBMP : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()
	
	virtual bool SupportsTexture(UTexture* Texture) const override;
};

UCLASS()
class UVirtualTextureBuilderExporterBMP : public UTextureExporterBMP
{
	GENERATED_UCLASS_BODY()
};
