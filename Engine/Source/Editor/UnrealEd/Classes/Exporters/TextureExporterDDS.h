// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// TextureExporterDDS
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "Engine/Texture.h"
#include "TextureExporterDDS.generated.h"

UCLASS()
class UTextureExporterDDS : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()

	virtual bool SupportsObject(UObject* Object) const override;

	// should not get called :
	virtual bool SupportsTexture(UTexture* Texture) const override { check(0); return true; };

	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0) override;
	
};


UCLASS()
class UVirtualTextureBuilderExporterDDS : public UTextureExporterDDS
{
	GENERATED_UCLASS_BODY()
};