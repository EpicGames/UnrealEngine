// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureExporterEXR
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "TextureExporterEXR.generated.h"

UCLASS()
class UTextureExporterEXR : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()
	
	virtual bool SupportsTexture(UTexture* Texture) const override;
};

UCLASS()
class UVirtualTextureBuilderExporterEXR : public UTextureExporterEXR
{
	GENERATED_UCLASS_BODY()
};

