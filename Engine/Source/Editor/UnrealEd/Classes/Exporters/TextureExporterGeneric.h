// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// TextureExporterGeneric
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Engine/Texture.h"
#include "TextureExporterGeneric.generated.h"

UCLASS()
class UTextureExporterGeneric : public UExporter
{
	GENERATED_UCLASS_BODY()

	virtual bool SupportsObject(UObject* Object) const override;
	virtual int32 GetFileCount(UObject* Object) const override;
	virtual FString GetUniqueFilename(UObject* Object, const TCHAR* Filename, int32 FileIndex, int32 FileCount) const override;
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0) override;
	
	// derived class exporter must implement :
	virtual bool SupportsTexture(UTexture* Texture) const { return false; };

protected:
	class UTexture* GetExportTexture(UObject* Object) const;
};

