// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureExporterHDR
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "TextureExporterHDR.generated.h"

UCLASS()
class UTextureExporterHDR : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()
	
	virtual bool SupportsTexture(UTexture* Texture) const override;
};

UCLASS()
class UVirtualTextureBuilderExporterHDR : public UTextureExporterHDR
{
	GENERATED_UCLASS_BODY()
};
