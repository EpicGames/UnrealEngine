// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureExporterPNG
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "TextureExporterPNG.generated.h"

UCLASS()
class UTextureExporterPNG : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()
	
	virtual bool SupportsTexture(UTexture* Texture) const override;
};

UCLASS()
class UVirtualTextureBuilderExporterPNG : public UTextureExporterPNG
{
	GENERATED_UCLASS_BODY()
};



// JPEG is not UTextureExporterGeneric
// JPEG exporter writes stored JPEG bits in TextureSource compressed payloads
UCLASS(MinimalAPI)
class UTextureExporterJPEG : public UExporter
{
	GENERATED_UCLASS_BODY()


	//~ Begin UExporter Interface
	UNREALED_API virtual bool SupportsObject(UObject* Object) const override;
	UNREALED_API virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};


UCLASS()
class UTextureExporterUEJPEG : public UExporter
{
	GENERATED_UCLASS_BODY()
	
	//~ Begin UExporter Interface
	UNREALED_API virtual bool SupportsObject(UObject* Object) const override;
	UNREALED_API virtual bool ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags=0 ) override;
	//~ End UExporter Interface
};




