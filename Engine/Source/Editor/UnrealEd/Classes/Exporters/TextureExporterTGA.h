// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureExporterTGA
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterGeneric.h"
#include "TextureExporterTGA.generated.h"

UCLASS(MinimalAPI)
class UTextureExporterTGA : public UTextureExporterGeneric
{
	GENERATED_UCLASS_BODY()
	
	UNREALED_API virtual bool SupportsTexture(UTexture* Texture) const override;
};
