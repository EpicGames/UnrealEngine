// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// VectorFieldExporter
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Exporters/Exporter.h"
#include "VectorFieldExporter.generated.h"

UCLASS(MinimalAPI)
class UVectorFieldExporter : public UExporter
{
	GENERATED_BODY()

	UVectorFieldExporter(const FObjectInitializer& ObjectInitializer = FObjectInitializer());

	//~ Begin UExporter Interface
	UNREALED_API virtual bool ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags = 0) override;
	//~ End UExporter Interface
};



