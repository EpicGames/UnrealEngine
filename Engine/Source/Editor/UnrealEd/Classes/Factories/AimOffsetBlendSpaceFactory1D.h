// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// UAimOffsetBlendSpaceFactory1D
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/BlendSpaceFactory1D.h"
#include "AimOffsetBlendSpaceFactory1D.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UAimOffsetBlendSpaceFactory1D : public UBlendSpaceFactory1D
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};



