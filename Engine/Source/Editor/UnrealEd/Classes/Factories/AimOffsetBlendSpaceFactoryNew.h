// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// UAimOffsetBlendSpaceFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/BlendSpaceFactoryNew.h"
#include "AimOffsetBlendSpaceFactoryNew.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UAimOffsetBlendSpaceFactoryNew : public UBlendSpaceFactoryNew
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};



