// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "TransformProviderFactory.h"
#include "AnimBankFactory.generated.h"

struct FAssetData;
class SWindow;

UCLASS(HideCategories=Object, BlueprintType, MinimalAPI)
class UAnimBankFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class USkeleton> TargetSkeleton;

	/** The preview mesh to use with this animation bank */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override { return true; } 
	//~ End UFactory Interface	

private:
	UNREALED_API void OnTargetSkeletonSelected(const FAssetData& SelectedAsset);

private:
	TSharedPtr<SWindow> PickerWindow;
};

UCLASS(MinimalAPI)
class UAnimBankDataFactory : public UTransformProviderDataFactory
{
	GENERATED_UCLASS_BODY()

	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return false; }
};