// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/Blueprint.h"
#include "Factories/Factory.h"
#include "AnimBlueprintFactory.generated.h"

UCLASS(HideCategories=Object,MinimalAPI)
class UAnimBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The type of blueprint that will be created
	UPROPERTY(EditAnywhere, Category=AnimBlueprintFactory)
	TEnumAsByte<EBlueprintType> BlueprintType;

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category=AnimBlueprintFactory, meta=(AllowAbstract = ""))
	TSubclassOf<class UAnimInstance> ParentClass;

	// The kind of skeleton that animation graphs compiled from the blueprint will animate
	UPROPERTY(EditAnywhere, Category=AnimBlueprintFactory)
	TObjectPtr<class USkeleton> TargetSkeleton;

	// The preview mesh to use with this animation blueprint
	UPROPERTY(EditAnywhere, Category=AnimBlueprintFactory)
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	// Whether the created blueprint should be a template with no target skeleton
	UPROPERTY(EditAnywhere, Category=AnimBlueprintFactory)
	bool bTemplate;
	
	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

UCLASS(HideCategories=Object, MinimalAPI)
class UAnimLayerInterfaceFactory : public UAnimBlueprintFactory
{
	GENERATED_BODY()

	UAnimLayerInterfaceFactory();

	// UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual FName GetNewAssetThumbnailOverride() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetToolTip() const override;
	virtual FString GetToolTipDocumentationExcerpt() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual bool ConfigureProperties() override;
};