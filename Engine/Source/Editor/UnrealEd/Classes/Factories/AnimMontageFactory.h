// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "AnimMontageFactory.generated.h"

struct FAssetData;
class SWindow;
class UAnimMontage;

UCLASS(HideCategories=Object,MinimalAPI)
class UAnimMontageFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Scripting")
	TObjectPtr<class USkeleton> TargetSkeleton;

	/* Used when creating a montage from an AnimSequence, becomes the only AnimSequence contained */
	UPROPERTY(BlueprintReadWrite, Category = "Scripting")
	TObjectPtr<class UAnimSequence> SourceAnimation;

	/** The preview mesh to use with this animation */
	UPROPERTY()
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	/** Ensure there is at least one section in the montage and that the first section starts at T=0.f */
	static UNREALED_API bool EnsureStartingSection(UAnimMontage* Montage);

private:
	void OnTargetSkeletonSelected(const FAssetData& SelectedAsset);

private:
	TSharedPtr<SWindow> PickerWindow;
};

