// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "AnimSequenceFactory.generated.h"

struct FAssetData;
class SWindow;

UCLASS(HideCategories=Object, BlueprintType, MinimalAPI)
class UAnimSequenceFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class USkeleton> TargetSkeleton;

	/** The preview mesh to use with this animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override { return false; } 
	//~ Begin UFactory Interface	

private:
	UNREALED_API void OnTargetSkeletonSelected(const FAssetData& SelectedAsset);

private:
	TSharedPtr<SWindow> PickerWindow;
};

