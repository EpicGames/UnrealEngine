// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "AnimStreamableFactory.generated.h"

struct FAssetData;
class SWindow;

UCLASS(HideCategories=Object,MinimalAPI)
class UAnimStreamableFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class USkeleton> TargetSkeleton;

	/* Used when creating a composite from an AnimSequence, becomes the only AnimSequence contained */
	UPROPERTY()
	TObjectPtr<class UAnimSequence> SourceAnimation;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

