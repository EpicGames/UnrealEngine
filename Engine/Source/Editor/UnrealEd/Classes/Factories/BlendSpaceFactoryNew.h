// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// BlendSpaceFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "BlendSpaceFactoryNew.generated.h"

struct FAssetData;
class SWindow;

UCLASS(hidecategories=Object, MinimalAPI)
class UBlendSpaceFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** Target skeleton for the created blendspace */
	UPROPERTY(BlueprintReadWrite, Category=BlendSpaceFactory)
	TObjectPtr<class USkeleton>	TargetSkeleton;

	/** The preview mesh to use for the created blendspace */
	UPROPERTY(BlueprintReadWrite, Category=BlendSpaceFactory)
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

private:
	void OnTargetSkeletonSelected(const FAssetData& SelectedAsset);

private:
	TSharedPtr<SWindow> PickerWindow;
};



