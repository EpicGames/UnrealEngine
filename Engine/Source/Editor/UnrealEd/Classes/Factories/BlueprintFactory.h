// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "Engine/Blueprint.h"
#include "BlueprintFactory.generated.h"

class FClassViewerInitializationOptions;

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UBlueprintFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The parent class of the created blueprint
	UPROPERTY(EditAnywhere, Category=BlueprintFactory, meta=(AllowAbstract = "", BlueprintBaseOnly = ""))
	TSubclassOf<class UObject> ParentClass;

	// The type of Blueprint to create in FactoryCreateNew
	UPROPERTY()
	TEnumAsByte<EBlueprintType> BlueprintType = EBlueprintType::BPTYPE_Normal;

	// Skips the class choosing dialog and uses the ParentClass as the blueprint base class
	UPROPERTY()
	bool bSkipClassPicker = false;

	// Delegate allows overriding the class viewer initialization options before displaying it
	DECLARE_DELEGATE_OneParam(FOnConfigureProperties, FClassViewerInitializationOptions*)
	FOnConfigureProperties OnConfigurePropertiesDelegate;

	// UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory Interface
};



