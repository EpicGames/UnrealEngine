// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/BlueprintFactory.h"
#include "BlueprintInterfaceFactory.generated.h"

UCLASS(hidecategories=Object, collapsecategories)
class UBlueprintInterfaceFactory : public UBlueprintFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual FText GetDisplayName() const override;
	virtual FName GetNewAssetThumbnailOverride() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetToolTip() const override;
	virtual FString GetToolTipDocumentationExcerpt() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual bool ConfigureProperties() override;
	virtual FString GetDefaultNewAssetName() const override;
	// End of UFactory interface
};



