// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Factory for importing table from text (CSV)
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/RichCurve.h"
#include "Factories/Factory.h"
#include "Factories/ImportSettings.h"
#include "EditorReimportHandler.h"
#include "CSVImportFactory.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCSVImportFactory, Log, All);

/** Enum to indicate what to import CSV as */
UENUM(BlueprintType)
enum class ECSVImportType : uint8
{
	/** Import as UDataTable */
	ECSV_DataTable,
	/** Import as UCurveTable */
	ECSV_CurveTable,
	/** Import as a UCurveFloat */
	ECSV_CurveFloat,
	/** Import as a UCurveVector */
	ECSV_CurveVector,
	/** Import as a UCurveLinearColor */
	ECSV_CurveLinearColor,
};

USTRUCT(BlueprintType)
struct FCSVImportSettings
{
	GENERATED_BODY()

	FCSVImportSettings();

	UPROPERTY(BlueprintReadWrite, Category="Misc")
	TObjectPtr<UScriptStruct> ImportRowStruct;

	UPROPERTY(BlueprintReadWrite, Category="Misc")
	ECSVImportType ImportType;

	UPROPERTY(BlueprintReadWrite, Category="Misc")
	TEnumAsByte<ERichCurveInterpMode> ImportCurveInterpMode;

	/** True to force IsAutomatedImport to return true during the import */
	bool bForceAutomatedImport = false;

	/** Despite its name, DataToImport can be JSON instead of CSV if this bool is set */
	bool bDataIsJson = false;
	FString DataToImport;
};

UCLASS(hidecategories=Object, MinimalAPI)
class UCSVImportFactory : public UFactory, public IImportSettingsParser
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UFactory Interface
	UNREALED_API virtual bool IsAutomatedImport() const override;
	UNREALED_API virtual FText GetDisplayName() const override;
	UNREALED_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
		const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UNREALED_API virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags,
		UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn,
		bool& bOutOperationCanceled) override;
	UNREALED_API virtual bool DoesSupportClass(UClass * Class) override;
	UNREALED_API virtual bool FactoryCanImport(const FString& Filename) override;
	UNREALED_API virtual	IImportSettingsParser* GetImportSettingsParser() override;
	UNREALED_API virtual void CleanUp() override;

	/* Reimport an object that was created based on a CSV */
	UNREALED_API EReimportResult::Type ReimportCSV(UObject* Obj);
	
	/**
	 * IImportSettings interface
	 */
	UNREALED_API virtual void ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson) override;

protected:
	UNREALED_API virtual TArray<FString> DoImportDataTable(const FCSVImportSettings& ImportSettings, class UDataTable* TargetDataTable);
	UNREALED_API virtual TArray<FString> DoImportCurveTable(const FCSVImportSettings& ImportSettings, class UCurveTable* TargetCurveTable);
	UNREALED_API virtual TArray<FString> DoImportCurve(const FCSVImportSettings& InImportSettings, class UCurveBase* TargetCurve);

private:
	/* Reimport object from the given path*/
	EReimportResult::Type Reimport(UObject* Obj, const FString& Path);

	bool bImportAll = false;

	// Function to Copy DataTable row to avoid dangling pointers
	TUniquePtr<uint8[]> CopyRow(const uint8* RowData, const UScriptStruct* RowStruct) const;

public:
	UPROPERTY(BlueprintReadWrite, Category="Automation")
	FCSVImportSettings AutomatedImportSettings;

	/** Temporary data table to use to display import options */
	UPROPERTY()
	TObjectPtr<UDataTable> DataTableImportOptions;
};

