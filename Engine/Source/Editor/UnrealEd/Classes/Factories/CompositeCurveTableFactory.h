// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CSVImportFactory.h"
#include "CompositeCurveTableFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class UCompositeCurveTableFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ End UFactory Interface

protected:
	UNREALED_API virtual UCurveTable* MakeNewCurveTable(UObject* InParent, FName Name, EObjectFlags Flags);
};

