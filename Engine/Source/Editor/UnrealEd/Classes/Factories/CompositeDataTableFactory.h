// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTableFactory.h"
#include "CompositeDataTableFactory.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class UCompositeDataTableFactory : public UDataTableFactory
{
	GENERATED_UCLASS_BODY()

protected:
	UNREALED_API virtual UDataTable* MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags) override;
};

