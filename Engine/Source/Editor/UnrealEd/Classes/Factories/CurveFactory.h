// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Curves/CurveBase.h"
#include "Factories/Factory.h"
#include "CurveFactory.generated.h"

/**
 * Factory that creates curve assets, prompting to pick the kind of curve at creation time
 */

UCLASS(MinimalAPI)
class UCurveFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** The type of blueprint that will be created */
	UPROPERTY(EditAnywhere, Category=CurveFactory)
	TSubclassOf<UCurveBase> CurveClass;

	// UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	// End of UFactory interface
};

/**
 * Factory that creates float curve assets
 */

UCLASS(MinimalAPI)
class UCurveFloatFactory : public UCurveFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return false; }
	// End of UFactory interface
};

/**
 * Factory that creates linear color curve assets
 */

UCLASS(MinimalAPI)
class UCurveLinearColorFactory : public UCurveFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return false; }
	// End of UFactory interface
};

/**
 * Factory that creates vector curve assets
 */

UCLASS(MinimalAPI)
class UCurveVectorFactory : public UCurveFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return false; }
	// End of UFactory interface
};
