// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Factory for importing curves from other data formats.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "CurveImportFactory.generated.h"

UCLASS(hidecategories=Object)
class UCurveImportFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateText( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn ) override;
	//~ Begin UFactory Interface
};



