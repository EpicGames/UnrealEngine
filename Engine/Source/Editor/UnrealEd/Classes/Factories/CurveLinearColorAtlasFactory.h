// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// UCurveLinearColorAtlasFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "CurveLinearColorAtlasFactory.generated.h"

UCLASS(hidecategories=(Object, Texture))
class UCurveLinearColorAtlasFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** Width of the texture atlas */
	UPROPERTY(meta=(ToolTip="Width of the texture atlas"))
	int32 Width;

	virtual FText GetDisplayName() const override;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};



