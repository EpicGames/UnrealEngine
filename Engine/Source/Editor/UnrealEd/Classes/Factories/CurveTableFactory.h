// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Curves/RealCurve.h"
#include "CurveTableFactory.generated.h"

class UCurveTable;

/* Creates a CurveTable with Rich Curves */

UCLASS(hidecategories=Object, MinimalAPI)
class UCurveTableFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

  public:
	// UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;


protected:
	UNREALED_API virtual UCurveTable* MakeNewCurveTable(UObject* InParent, FName Name, EObjectFlags Flags);

	ERichCurveInterpMode InterpMode;
};
