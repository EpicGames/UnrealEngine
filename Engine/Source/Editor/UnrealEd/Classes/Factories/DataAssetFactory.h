// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "DataAssetFactory.generated.h"

class UDataAsset;

UCLASS(hidecategories=Object, MinimalAPI)
class UDataAssetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DataAsset)
	TSubclassOf<UDataAsset> DataAssetClass;

	// UFactory interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	// End of UFactory interface
};



