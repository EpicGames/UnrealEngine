// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "DataTableFactory.generated.h"

class UDataTable;

UCLASS(hidecategories=Object, MinimalAPI)
class UDataTableFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "Data Table Factory")
	TObjectPtr<const class UScriptStruct> Struct;

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

protected:
	UNREALED_API virtual UDataTable* MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags);
};
