// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AssetCategoryPath.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Misc/SecureHash.h"

#include "Factory.generated.h"

class UAssetImportTask;
class FBulkData;

/**
 * Determines which workflows are available for a factory.
 * All factories *should* support the default workflow!
 */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EFactoryCreateWorkflow : uint8
{
	/**
	 * Synchronous asset creation (potentially with a modal dialog).
	 */
	Default = 1 << 0,

	/**
	 * Asynchronous asset creation (usually with a dialog).
	 * If supported, the UFactory::ConfigureProperties will not be called, but UFactory::ConfigurePropertiesAsync.
	 */
	Asynchronous = 1 << 1
};
ENUM_CLASS_FLAGS(EFactoryCreateWorkflow);

DECLARE_DELEGATE_OneParam(FOnFactoryConfigurePropertiesAsyncComplete, UFactory* /* this */);
DECLARE_DELEGATE_OneParam(FOnFactoryConfigurePropertiesAsyncCancelled, UFactory* /* this */);

/**
 * Base class for all factories
 * An object responsible for creating and importing new objects.
 * 
 */
UCLASS(abstract, MinimalAPI)
class UFactory : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * @return true if the factory can currently create a new object from scratch.
	 */
	virtual bool CanCreateNew() const
	{
		return bCreateNew;
	}

	/**
	 * Whether the specified file can be imported by this factory.
	 *
	 * @return true if the file is supported, false otherwise.
	 */
	UNREALED_API virtual bool FactoryCanImport(const FString& Filename);

	/**
	 * Whether the factory is checking for SlowTask::ShouldCancel()
	 * while importing and aborting the import when appropriate.
	 *
	 * @return true if factory import can be canceled.
	 */
	virtual bool CanImportBeCanceled() const
	{
		return false;
	}

	/**
	 * Whether the specified file can be imported by this factory. (Implemented in script)
	 *
	 * @return true if the file is supported, false otherwise.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Miscellaneous")
	UNREALED_API bool ScriptFactoryCanImport(const FString& Filename);

	/**
	 * Create a new object by importing it from a file name.
	 *
	 * The default implementation of this method will load the contents of the entire
	 * file into a byte buffer and call FactoryCreateBinary. User defined factories
	 * may override this behavior to process the provided file name on their own.
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Filename
	 * @param Parms
	 * @param Warn
	 * @param bOutOperationCanceled Will indicate whether the user canceled the import.
	 * @return The new object.
	 */
	UNREALED_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled);

	/**
	 * Create a new object by class.
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Context
	 * @param Warn
	 * @param CallingContext
	 * @return The new object.
	 */
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
	{
		return FactoryCreateNew(InClass, InParent, InName, Flags, Context, Warn);
	}

	/**
	 * Create a new object by class.
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Context
	 * @param Warn
	 * @return The new object.
	 */
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
	{
		return nullptr;
	}

	UNREALED_API virtual UObject* ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, bool& OutCanceled);

	/**
	 * Returns an array of all the additional objects created during the last imports, as some factories may produce more than one object.
	 * The internal array is cleared before each import and upon Cleanup().
	 */
	const TArray<UObject*>& GetAdditionalImportedObjects() const { return AdditionalImportedObjects; }

	/**
	 * Import object(s) using a task via script
	 *
	 * @param InTask
	 * @return True if script implements
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Miscellaneous")
	UNREALED_API bool ScriptFactoryCreateFile(UAssetImportTask* InTask);

	/** Returns true if this factory should be shown in the New Asset menu (by default calls CanCreateNew). */
	UNREALED_API virtual bool ShouldShowInNewMenu() const;

	/** Returns an optional override brush name for the new asset menu. If this is not specified, the (64x64) thumbnail for the supported class will be used. */
	UNREALED_API virtual FName GetNewAssetThumbnailOverride() const;

	/** Returns an optional override brush name for the new asset menu. If this is not specified, the (16x16) icon for the supported class will be used. */
	UNREALED_API virtual FName GetNewAssetIconOverride() const;

	/** Returns the name of the factory for menus */
	UNREALED_API virtual FText GetDisplayName() const;

	/** When shown in menus, this is the category containing this factory. Return type is a BitFlag mask using EAssetTypeCategories. */
	UNREALED_API virtual uint32 GetMenuCategories() const;

	/** Branch of sub-menus containing factory under each provided category. */
	UNREALED_API virtual const TArray<FText>& GetMenuCategorySubMenus() const;

	/** Get the asset category path using the AssetDefinition format, used mainly for the ContentBrowser add menu. */
	UNREALED_API virtual TArray<FAssetCategoryPath> GetAssetMenuPathsForCategory(FName InCategory) const;

	/** Returns the tooltip text description of this factory */
	UNREALED_API virtual FText GetToolTip() const;

	/** Returns the documentation page that should be use for the rich tool tip for this factory */
	UNREALED_API virtual FString GetToolTipDocumentationPage() const;

	/** Returns the documentation excerpt that should be use for the rich tool tip for this factory */
	UNREALED_API virtual FString GetToolTipDocumentationExcerpt() const;

	/**
	 * @return		The object class supported by this factory.
	 */
	UNREALED_API UClass* GetSupportedClass() const;

	/**
	 * @return true if it supports this class 
	 */
	UNREALED_API virtual bool DoesSupportClass(UClass* Class);

	/**
	 * Resolves SupportedClass for factories which support multiple classes.
	 * Such factories will have a nullptr SupportedClass member.
	 */
	UNREALED_API virtual UClass* ResolveSupportedClass();

	/** Opens a dialog to configure the factory properties. Return false if user opted out of configuring properties */
	virtual bool ConfigureProperties()
	{
		return true;
	}

	/**
	 * Called instead of ConfigureProperties() when an asset is being created asynchronously.
	 * It is not the responsibility of the caller of this function to stop the factory from being GC'd. It is the
	 * responsibility of the implementor.
	 * @param OnComplete  Called when the asynchronous process completes successfully.
	 * @param OnCancelled Called when the asynchronous process was cancelled.
	 * @return true if the async process was successfully started.
	 */
	virtual bool ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled)
	{
		OnCancelled.Execute(this);
		return false;
	}

	// @todo document
	UNREALED_API virtual bool ImportUntypedBulkDataFromText(const TCHAR*& Buffer, FBulkData& BulkData);

	/** Creates a list of file extensions supported by this factory */
	UNREALED_API virtual void GetSupportedFileExtensions(TArray<FString>& OutExtensions) const;

	/** Returns true if the provided file extension is supported */
	UNREALED_API virtual bool IsSupportedFileExtension(FStringView InExtension) const;

	/** Do clean up after importing is done. Will be called once for multi batch import. */
	virtual void CleanUp() { AdditionalImportedObjects.Empty(); }
	/**
	 * Creates an asset if it doesn't exist. If it does exist then it overwrites it if possible. If it can not overwrite then it will delete and replace. If it can not delete, it will return nullptr.
	 * 
	 * @param InClass The class of the asset to create
	 * @param InPackage The package to create this object within.
	 * @param Name The name to give the new asset. If no value (NAME_None) is specified, the asset will be given a unique name in the form of ClassName_#.
	 * @param InFlags The ObjectFlags to assign to the new asset.
	 * @param Template If specified, the property values from this object will be copied to the new object, and the new object's ObjectArchetype value will be set to this object.
	 *	               If nullptr, the class default object is used instead.
	 * @return A pointer to a new asset of the specified type or null if the creation failed.
	 */
	UNREALED_API UObject* CreateOrOverwriteAsset(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InTemplate = nullptr) const;

	/** Returns a new starting point name for newly created assets in the content browser */
	UNREALED_API virtual FString GetDefaultNewAssetName() const;

	/** @return the parser that is capable of parsing a json string of import settings for this factory */
	virtual class IImportSettingsParser* GetImportSettingsParser() {return nullptr;}

	/**
	 * Sets the automated import data being used with this factory
	 *
	 * @param Data	The automated import data or nullptr if it doesnt exist
	 */
	UNREALED_API void SetAutomatedAssetImportData(const class UAutomatedAssetImportData* Data);

	/**
	 * Sets the import task being used with this factory
	 *
	 * @param Task	The import task or nullptr if it does not exist
	 */
	UNREALED_API void SetAssetImportTask(class UAssetImportTask* Task);

	/**
	 * @return true if this factory is being used for automated import.  Dialogs and user input should be disabled if this method returns true
	 */
	UNREALED_API virtual bool IsAutomatedImport() const;

	/**
	 * @return the supported factory formats list.
	 */
	virtual TArray<FString> GetFormats() const { return Formats; }

	/**
	 * @return the supported factory workflows.
	 */
	virtual EFactoryCreateWorkflow GetSupportedWorkflows() const
	{
		return static_cast<EFactoryCreateWorkflow>(SupportedWorkflows);
	}
public:

	/**
	 * Pop up message to the user asking whether they wish to overwrite existing state or not.
	 *
	 * @param Message The message text.
	 **/
	UNREALED_API void DisplayOverwriteOptionsDialog(const FText& Message);

	/** Get the name of the file currently being imported. */
	static FString GetCurrentFilename()
	{
		return CurrentFilename;
	}

	/** Get the default import priority for factories. */
	static int32 GetDefaultImportPriority()
	{
		return DefaultImportPriority;
	}

	//@third party code BEGIN SIMPLYGON
	/** Get the Hash for the file being imported. Provides enormous speed impovements for large CAD file imports  */
	static FMD5Hash GetFileHash()
	{
		return FileHash;
	}
	//@third party code END SIMPLYGON

	/**
	 * Resets the saved state of this factory.
	 *
	 * The states are used to suppress messages during multiple object import. 
	 * It needs to be reset each time a new import is started
	 */
	UNREALED_API void ResetState();

	/** Helper function to sort an array of factories by their import priority - use as a predicate for Sort */
	static UNREALED_API bool SortFactoriesByPriority(const UFactory& A, const UFactory& B);

	// @todo document
	static UNREALED_API UObject* StaticImportObject(UClass* Class, UObject* InOuter, FName Name, EObjectFlags Flags, const TCHAR* Filename = TEXT(""), UObject* Context = nullptr, UFactory* Factory = nullptr, const TCHAR* Parms = nullptr, FFeedbackContext* Warn = GWarn, int32 MaxImportFileSize = 0xC100000);
	static UNREALED_API UObject* StaticImportObject(UClass* Class, UObject* InOuter, FName Name, EObjectFlags Flags, bool& bOutOperationCanceled, const TCHAR* Filename = TEXT(""), UObject* Context = nullptr, UFactory* Factory = nullptr, const TCHAR* Parms = nullptr, FFeedbackContext* Warn = GWarn, int32 MaxImportFileSize = 0xC100000);

public:

	//~ UObject interface

	static UNREALED_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

protected:

	/** The default value to return from CanCreateNew() */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	uint32 bCreateNew : 1;

protected:

	/**
	 * Create a new object by importing it from a text buffer.
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Context
	 * @param Type (must not be nullptr, i.e. TEXT("TGA"))
	 * @param Buffer
	 * @param BufferEnd
	 * @param Warn
	 * @return The new object.
	 */
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
	{
		return nullptr;
	}

	/**
	* Create a new object by importing it from a text buffer.
	*
	* @param InClass
	* @param InParent
	* @param InName
	* @param Flags
	* @param Context
	* @param Type (must not be nullptr, i.e. TEXT("TGA"))
	* @param Buffer
	* @param BufferEnd
	* @param Warn
	* @param bOutOperationCanceled Will indicate whether the user canceled the import.
	* @return The new object.
	*/
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
	{
		return FactoryCreateText(InClass, InParent, InName, Flags, Context, Type, Buffer, BufferEnd, Warn);
	}

	/**
	 * Create a new object by importing it from a binary buffer.
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Context
	 * @param Type (must not be nullptr, i.e. TEXT("TGA"))
	 * @param Buffer
	 * @param BufferEnd
	 * @param Warn
	 * @return The new object.
	 */
	virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn)
	{
		return nullptr;
	}
	
	/**
	 * Create a new object by importing it from a binary buffer (cancelable).
	 *
	 * @param InClass
	 * @param InParent
	 * @param InName
	 * @param Flags
	 * @param Context
	 * @param Type (must not be nullptr, i.e. TEXT("TGA"))
	 * @param Buffer
	 * @param BufferEnd
	 * @param Warn
	 * @param bOutOperationCanceled Will indicate whether the user canceled the import.
	 * @return The new object.
	 */
	virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
	{
		return FactoryCreateBinary(InClass, InParent, InName, Flags, Context, Type, Buffer, BufferEnd, Warn);
	}

	/** 
	 * Utility function that grabs the asset definition categories for a specific class and category
	 * Used mainly for factories not having the actual class for its supported class. 
	 */
	UNREALED_API TArray<FAssetCategoryPath> GetAssetMenuPaths(const UClass* InClass, FName InCategory) const;

public:

	/** The class manufactured by this factory. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TSubclassOf<UObject>  SupportedClass;

	/** Class of the context object used to help create the object. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TSubclassOf<UObject>  ContextClass;

	/** List of formats supported by the factory. Each entry is of the form "ext;Description" where ext is the file extension. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TArray<FString> Formats;

	/** true if the associated editor should be opened after creating a new object. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	uint32 bEditAfterNew:1;

	/** true if the factory imports objects from files. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	uint32 bEditorImport:1;

	/** true if the factory imports objects from text. */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	uint32 bText:1;

	/** Determines the order in which factories are tried when importing or reimporting an object.
	Factories with higher priority values will go first. Factories with negative priorities will be excluded. */
	UPROPERTY()
	int32 ImportPriority;

	/** Data for how to import files via the automated command line importing interface */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TObjectPtr<const class UAutomatedAssetImportData> AutomatedImportData;

	/** Task for importing file via script interfaces */
	UPROPERTY(BlueprintReadWrite, Category=Misc)
	TObjectPtr<class UAssetImportTask> AssetImportTask;

	/** Which workflows are supported. */
	UPROPERTY(BlueprintReadWrite, Category=Misc, meta = (Bitmask))
	uint8 SupportedWorkflows = static_cast<uint8>(EFactoryCreateWorkflow::Default);

protected:

	/** Name of the file currently being imported. */
	static UNREALED_API FString CurrentFilename;

	/** This is the import priority that all factories are given in the default constructor. */
	static UNREALED_API const int32 DefaultImportPriority;

	//@third party code BEGIN SIMPLYGON
	/** This is the HASH for the file being imported */
	static UNREALED_API FMD5Hash FileHash;
	//@third party code END SIMPLYGON

	/**
	 * For interactive object imports, this value indicates whether the user wants
	 * objects to be automatically overwritten (See EAppReturnType), or -1 if the
	  * user should be prompted.
	*/
	UPROPERTY()
	int32 OverwriteYesOrNoToAllState;

	TArray<UObject*> AdditionalImportedObjects;

};
