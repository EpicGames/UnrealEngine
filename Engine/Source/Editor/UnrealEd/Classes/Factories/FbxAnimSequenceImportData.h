// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/FbxAssetImportData.h"
#include "FbxAnimSequenceImportData.generated.h"

class UAnimSequence;
struct FPropertyChangedEvent;

/**
* I know these descriptions don't make sense, but the functions I use act a bit different depending on situation.
a) FbxAnimStack::GetLocalTimeSpan will return you the start and stop time for this stack (or take). It's simply two time values that can be set to anything regardless of animation curves. Generally speaking, applications set these to the start and stop time of the timeline.
b) As for FbxNode::GetAnimationInternval, this one will iterate through all properties recursively, and then for all animation curves it finds, for the animation layer index specified. So in other words, if one property has been animated, it will modify this result. This is completely different from GetLocalTimeSpan since it calculates the time span depending on the keys rather than just using the start and stop time that was saved in the file.
*/

/** Animation length type when importing */
UENUM(BlueprintType)
enum EFBXAnimationLengthImportType : int
{
	/** This option imports animation frames based on what is defined at the time of export */
	FBXALIT_ExportedTime			UMETA(DisplayName = "Exported Time"),
	/** Will import the range of frames that have animation. Can be useful if the exported range is longer than the actual animation in the FBX file */
	FBXALIT_AnimatedKey				UMETA(DisplayName = "Animated Time"),
	/** This will enable the Start Frame and End Frame properties for you to define the frames of animation to import */
	FBXALIT_SetRange				UMETA(DisplayName = "Set Range"),

	FBXALIT_MAX,
};

/**
* Import data and options used when importing any mesh from FBX
*/
UCLASS(BlueprintType, config = EditorPerProjectUserSettings, configdonotcheckdefaults, MinimalAPI)
class UFbxAnimSequenceImportData : public UFbxAssetImportData
{
	GENERATED_UCLASS_BODY()
	
	/** If checked, meshes nested in bone hierarchies will be imported instead of being converted to bones. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (ImportType = "Animation"))
	bool bImportMeshesInBoneHierarchy;
	
	/** Which animation range to import. The one defined at Exported, at Animated time or define a range manually */
	UPROPERTY(EditAnywhere, Category = ImportSettings, config, meta = (DisplayName = "Animation Length"))
	TEnumAsByte<enum EFBXAnimationLengthImportType> AnimationLength;

	/** Start frame when Set Range is used in Animation Length */
	UPROPERTY()
	int32	StartFrame_DEPRECATED;

	/** End frame when Set Range is used in Animation Length  */
	UPROPERTY()
	int32	EndFrame_DEPRECATED;

	/** Frame range used when Set Range is used in Animation Length */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta=(UIMin=0, ClampMin=0))
	FInt32Interval FrameImportRange;

	/** Enable this option to use default sample rate for the imported animation at 30 frames per second */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (ToolTip = "If enabled, samples all animation curves to 30 FPS"))
	bool bUseDefaultSampleRate;

	/** Use this option to specify a sample rate for the imported animation, a value of 0 use the best matching samplerate. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (EditCondition = "!bUseDefaultSampleRate", ToolTip = "Sample fbx animation data at the specified sample rate, 0 find automaticaly the best sample rate", ClampMin = 0, UIMin = 0, ClampMax = 48000, UIMax = 60))
	int32 CustomSampleRate;

	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (ToolTip = "If enabled, snaps the animation to the closest frame boundary using the import sampling rate"))
	bool bSnapToClosestFrameBoundary;

	/** Name of source animation that was imported, used to reimport correct animation from the FBX file */
	UPROPERTY()
	FString SourceAnimationName;

	/** If true, import node attributes as either Animation Curves or Animation Attributes */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (DisplayName = "Import Attributes as Curves or Animation Attributes"))
	bool bImportCustomAttribute;

	/** If true, all previous node attributes imported as Animation Curves will be deleted when doing a re-import. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (DisplayName = "Delete existing Animation Curves"))
	bool bDeleteExistingCustomAttributeCurves;

	/** If true, all previous node attributes imported as Animation Attributes will be deleted when doing a re-import. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (DisplayName = "Delete existing Animation Attributes"))
	bool bDeleteExistingNonCurveCustomAttributes;
	
	/** Import bone transform tracks. If false, this will discard any bone transform tracks. (useful for curves only animations)*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings)
	bool bImportBoneTracks;

	/** Set Material Curve Type for all custom attributes that exists */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (EditCondition = "bImportCustomAttribute", DisplayName="Set Material Curve Type"))
	bool bSetMaterialDriveParameterOnCustomAttribute;

	/** Whether to automatically add curve metadata to an animation's skeleton. If this is disabled, curve metadata will be added to skeletal meshes for morph targets, but no metadata entry will be created for general curves. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings)
	bool bAddCurveMetadataToSkeleton;

	/** Set Material Curve Type for the custom attribute with the following suffixes. This doesn't matter if Set Material Curve Type is true  */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (EditCondition = "bImportCustomAttribute", DisplayName = "Material Curve Suffixes"))
	TArray<FString> MaterialCurveSuffixes;

	/** When importing custom attribute as curve, remove redundant keys */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (EditCondition = "bImportCustomAttribute", DisplayName = "Remove Redundant Keys"))
	bool bRemoveRedundantKeys;

	/** If enabled, this will delete this type of asset from the FBX */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings)
	bool bDeleteExistingMorphTargetCurves;

	/** When importing custom attribute or morphtarget as curve, do not import if it doesn't have any value other than zero. This is to avoid adding extra curves to evaluate */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings, meta = (DisplayName = "Do not import curves with only 0 values"))
	bool bDoNotImportCurveWithZero;

	/** If enabled, this will import a curve within the animation */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = ImportSettings)
	bool bPreserveLocalTransform;

	/** Gets or creates fbx import data for the specified anim sequence */
	static UNREALED_API UFbxAnimSequenceImportData* GetImportDataForAnimSequence(UAnimSequence* AnimSequence, UFbxAnimSequenceImportData* TemplateForCreation);

	UNREALED_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	UNREALED_API virtual void Serialize(FArchive& Ar) override;

	UNREALED_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UNREALED_API void CopyAnimationValues(const UFbxAnimSequenceImportData* Other);
};
