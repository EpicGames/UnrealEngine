// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "FbxFactory.generated.h"

class IImportSettingsParser;

namespace UnFbx {
	class FFbxImporter;
}

UCLASS(hidecategories=Object, MinimalAPI)
class UFbxFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UFbxImportUI> ImportUI;

	/** Prevent garbage collection of original when overriding ImportUI property */
	UPROPERTY()
	TObjectPtr<class UFbxImportUI> OriginalImportUI;

	/**  Set import batch **/
	void EnableShowOption() { bShowOption = true; }


	//~ Begin UObject Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UFactory Interface
	UNREALED_API virtual void CleanUp() override;
	UNREALED_API virtual bool DoesSupportClass(UClass * Class) override;
	UNREALED_API virtual UClass* ResolveSupportedClass() override;
	UNREALED_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UNREALED_API virtual bool FactoryCanImport(const FString& Filename) override;
	UNREALED_API virtual bool CanImportBeCanceled() const override;
	UNREALED_API virtual IImportSettingsParser* GetImportSettingsParser() override;
	UNREALED_API virtual TArray<FString> GetFormats() const override;
	//~ End UFactory Interface
	
	static UNREALED_API TArray<FString> GetFbxFormats(const UFactory* Factory);

	/**
	 * Detect mesh type to import: Static Mesh or Skeletal Mesh.
	 * Only the first mesh will be detected.
	 *
	 * @param InFilename	FBX file name
	 * @return bool	return true if parse the file successfully
	 */
	UNREALED_API bool DetectImportType(const FString& InFilename);

	void SetDetectImportTypeOnImport(bool bDetectState) { bDetectImportTypeOnImport = bDetectState; }

private:

	/**
	 * Delete the newly created objects after an import cancelation.
	 */
	UNREALED_API void CancelObjectCreation(UnFbx::FFbxImporter* FbxImporter) const;

protected:
	// @todo document
	UNREALED_API UObject* RecursiveImportNode(UnFbx::FFbxImporter* FFbxImporter, void* Node, UObject* InParent, FName InName, EObjectFlags Flags, struct FScopedSlowTask& SlowTask, TArray<UObject*>& OutNewAssets);

	// @todo document
	UNREALED_API UObject* ImportANode(UnFbx::FFbxImporter* FbxImporter, TArray<void*> VoidNodes, UObject* InParent, FName InName, EObjectFlags Flags, struct FScopedSlowTask& SlowTask, UObject* InMesh = NULL, int LODIndex = 0);

	/**
	 * Used to display special task progression text if needs be (like import being cancelled).
	 * 
	 * @param TaskText	The current import task text we want to display.
	 * @return			The task text after taking into account the factory's current state.
	 */
	UNREALED_API FText GetImportTaskText(const FText& TaskText) const;

	bool bShowOption;
	bool bDetectImportTypeOnImport;

	/** true if the import operation was canceled. */
	bool bOperationCanceled;
};



