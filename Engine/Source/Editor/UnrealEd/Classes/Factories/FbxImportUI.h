// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Fbx Importer UI options.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Factories/ImportSettings.h"
#include "FbxImportUI.generated.h"

namespace UnFbx
{
	class FFbxImporter;
}

/** Import mesh type */
UENUM(BlueprintType)
enum EFBXImportType : int
{
	/** Select Static Mesh if you'd like to import static mesh. */
	FBXIT_StaticMesh UMETA(DisplayName="Static Mesh"),
	/** Select Skeletal Mesh if you'd like to import skeletal mesh. */
	FBXIT_SkeletalMesh UMETA(DisplayName="Skeletal Mesh"),
	/** Select Animation if you'd like to import only animation. */
	FBXIT_Animation UMETA(DisplayName="Animation"),

	FBXIT_MAX,
};

DECLARE_DELEGATE(FOnUpdateCompareFbx);
DECLARE_DELEGATE(FOnShowConflictDialog);

namespace ImportCompareHelper
{
	struct FMaterialData
	{
		FName MaterialSlotName;
		FName ImportedMaterialSlotName;
		int32 MaterialIndex;
	};

	struct FMaterialCompareData
	{
		TArray<FMaterialData> CurrentAsset;
		TArray<FMaterialData> ResultAsset;
		void Empty()
		{
			CurrentAsset.Empty();
			ResultAsset.Empty();
			bHasConflict = false;
		}
		bool HasConflict() { return bHasConflict; }
		bool bHasConflict;
	};

	struct FSkeletonTreeNode
	{
		FName JointName;
		TArray<FSkeletonTreeNode> Childrens;
		void Empty()
		{
			JointName = NAME_None;
			Childrens.Empty();
		}
	};

	enum class ECompareResult : int32
	{
		SCR_None = 0x00000000,
		SCR_SkeletonMissingBone = 0x00000001,
		SCR_SkeletonAddedBone = 0x00000002,
		SCR_SkeletonBadRoot = 0x00000004,
	};

	ENUM_CLASS_FLAGS(ECompareResult);

	struct FSkeletonCompareData
	{
		FSkeletonTreeNode CurrentAssetRoot;
		FSkeletonTreeNode ResultAssetRoot;
		void Empty()
		{
			CurrentAssetRoot.Empty();
			ResultAssetRoot.Empty();
			CompareResult = ECompareResult::SCR_None;
		}
		ECompareResult GetCompareResult() { return CompareResult; }
		ECompareResult CompareResult;
	};

}

UCLASS(BlueprintType, config=EditorPerProjectUserSettings, AutoExpandCategories=(FTransform), HideCategories=Object, MinimalAPI)
class UFbxImportUI : public UObject, public IImportSettingsParser
{
	GENERATED_UCLASS_BODY()

public:
	/** Whether or not the imported file is in OBJ format */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bIsObjImport;

	/** The original detected type of this import */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TEnumAsByte<enum EFBXImportType> OriginalImportType;

	/** Type of asset to import from the FBX file */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	TEnumAsByte<enum EFBXImportType> MeshTypeToImport;

	/** Use the string in "Name" field as full name of mesh. The option only works when the scene contains one mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category=Miscellaneous, meta=(OBJRestrict="true"))
	uint32 bOverrideFullName:1;

	/** Whether to import the incoming FBX as a skeletal object */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mesh, meta = (ImportType = "StaticMesh|SkeletalMesh", DisplayName="Skeletal Mesh"))
	bool bImportAsSkeletal;
	
	/** Whether to import the incoming FBX as a Subdivision Surface (could be made a combo box together with bImportAsSkeletal) (Experimental, Early work in progress) */
	/** Whether to import the mesh. Allows animation only import when importing a skeletal mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mesh, meta=(ImportType="SkeletalMesh"))
	bool bImportMesh;

	/** Skeleton to use for imported asset. When importing a mesh, leaving this as "None" will create a new skeleton. When importing an animation this MUST be specified to import the asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Mesh, meta=(ImportType="SkeletalMesh|Animation"))
	TObjectPtr<class USkeleton> Skeleton;

	/** If checked, create new PhysicsAsset if it doesn't have it */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh"))
	uint32 bCreatePhysicsAsset:1;

	/** If this is set, use this PhysicsAsset. It is possible bCreatePhysicsAsset == false, and PhysicsAsset == NULL. It is possible they do not like to create anything. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Mesh, meta=(ImportType="SkeletalMesh", editcondition="!bCreatePhysicsAsset"))
	TObjectPtr<class UPhysicsAsset> PhysicsAsset;

	/** If checked, the editor will automatically compute screen size values for the static mesh's LODs. If unchecked, the user can enter custom screen size values for each LOD. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", DisplayName = "Auto Compute LOD Screen Size"))
	uint32 bAutoComputeLodDistances : 1;
	/** Set a screen size value for LOD 0*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 0 Screen Size"))
	float LodDistance0;
	/** Set a screen size value for LOD 1*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 1 Screen Size"))
	float LodDistance1;
	/** Set a screen size value for LOD 2*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 2 Screen Size"))
	float LodDistance2;
	/** Set a screen size value for LOD 3*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 3 Screen Size"))
	float LodDistance3;
	/** Set a screen size value for LOD 4*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 4 Screen Size"))
	float LodDistance4;
	/** Set a screen size value for LOD 5*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 5 Screen Size"))
	float LodDistance5;
	/** Set a screen size value for LOD 6*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 6 Screen Size"))
	float LodDistance6;
	/** Set a screen size value for LOD 7*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0.0", DisplayName = "LOD 7 Screen Size"))
	float LodDistance7;

	/** Set the minimum LOD used for rendering. Setting the value to 0 will use the default value of LOD0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0", DisplayName = "Minimum LOD"))
	int32 MinimumLodNumber;

	/** Set the number of LODs for the editor to import. Setting the value to 0 imports the number of LODs found in the file (up to the maximum). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category = LODSettings, meta = (ImportType = "StaticMesh", UIMin = "0", DisplayName = "Number of LODs"))
	int32 LodNumber;

	/** True to import animations from the FBX File */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category=Animation, meta=(ImportType="SkeletalMesh|Animation|RigOnly"))
	uint32 bImportAnimations:1;

	/** Override for the name of the animation to import. By default, it will be the name of FBX **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Animation, meta=(editcondition="bImportAnimations", ImportType="SkeletalMesh|RigOnly"))
	FString OverrideAnimationName;

	/** Enables importing of 'rigid skeletalmesh' (unskinned, hierarchy-based animation) from this FBX file, no longer shown, used behind the scenes */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	uint32 bImportRigidMesh:1;

	/** If no existing materials are found, whether to automatically create Unreal materials for materials found in the FBX scene */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Material, meta=(ImportType="GeoOnly"))
	uint32 bImportMaterials:1;

	/** Whether or not we should import textures. This option is disabled when we are importing materials because textures are always imported in that case. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category=Material, meta = (ImportType = "GeoOnly", EditCondition="!bImportMaterials"))
	uint32 bImportTextures:1;

	/** If true, the imported material sections will automatically be reset to the imported data in case of a reimport conflict. */
	UPROPERTY(BlueprintReadWrite, Transient, Category = Material)
	uint32 bResetToFbxOnMaterialConflict:1;

	/** Import data used when importing static meshes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Instanced, Category = Mesh, meta=(ImportType = "StaticMesh"))
	TObjectPtr<class UFbxStaticMeshImportData> StaticMeshImportData;

	/** Import data used when importing skeletal meshes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Instanced, Category=Mesh, meta=(ImportType = "SkeletalMesh"))
	TObjectPtr<class UFbxSkeletalMeshImportData> SkeletalMeshImportData;

	/** Import data used when importing animations */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Instanced, Category=Animation, meta=(editcondition="bImportAnimations", ImportType = "Animation"))
	TObjectPtr<class UFbxAnimSequenceImportData> AnimSequenceImportData;

	/** Import data used when importing textures */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Transient, Instanced, Category=Material)
	TObjectPtr<class UFbxTextureImportData> TextureImportData;

	/** If true the automated import path should detect the import type.  If false the import type was specified by the user */
	UPROPERTY(BlueprintReadWrite, Category = Miscellaneous)
	bool bAutomatedImportShouldDetectType;

	UFUNCTION(BlueprintCallable, Category = Miscellaneous)
	void ResetToDefault();

	/** UObject Interface */
	virtual bool CanEditChange( const FProperty* InProperty ) const override;

	/** IImportSettings Interface */
	virtual void ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson) override;

	/** sets MeshTypeToImport */
	void SetMeshTypeToImport()
	{
		MeshTypeToImport = bImportAsSkeletal ? FBXIT_SkeletalMesh : FBXIT_StaticMesh;
	}

	/* We want to change the last dialog state but not the CDO, so we cannot call UObject::SaveConfig here.
	 * The problem is the class is use to store the import settings with the imported asset and the serialization
	 * is serializing the diff with the CDO. Data will be lost if the CDO change.
	*/
	static void SaveOptions(UObject* ObjectToSaveOptions);
	static void LoadOptions(UObject* ObjectToLoadOptions);

	/* Whether this UI is construct for a reimport */
	bool bIsReimport;
	
	/* When we are reimporting, we need the current object to preview skeleton and materials match issues. */
	UObject* ReimportMesh;

	ImportCompareHelper::FMaterialCompareData MaterialCompareData;
	ImportCompareHelper::FSkeletonCompareData SkeletonCompareData;

	void UpdateCompareData(UnFbx::FFbxImporter* FbxImporter);

	FOnUpdateCompareFbx OnUpdateCompareFbx;
	FOnShowConflictDialog OnShowMaterialConflictDialog;
	FOnShowConflictDialog OnShowSkeletonConflictDialog;

	bool bAllowContentTypeImport;
	
//////////////////////////////////////////////////////////////////////////
	// FBX file informations
	// Transient value that are set everytime we show the options dialog. These are information only and should be string.

	/* The fbx file version */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "Mesh|Animation", DisplayName = "File Version"))
	FString FileVersion;

	/* The file creator information */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "Mesh|Animation", DisplayName = "File Creator"))
	FString FileCreator;

	/* The file vendor information, software name and version that was use to create the file */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "Mesh|Animation", DisplayName = "File Creator Application"))
	FString FileCreatorApplication;

	/* The file units */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "Mesh|Animation", DisplayName = "File Units"))
	FString FileUnits;

	/* The file axis direction, up vector and hand */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "Mesh|Animation", DisplayName = "File Axis Direction"))
	FString FileAxisDirection;

	/* The fbx animation frame rate */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "SkeletalMesh|Animation", DisplayName = "File Frame Rate"))
	FString FileSampleRate;
	
	/* The fbx animation start frame */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "SkeletalMesh|Animation", DisplayName = "Animation Start Frame"))
	FString AnimStartFrame;
	
	/* The fbx animation end frame */
	UPROPERTY(VisibleAnywhere, Transient, Category = FbxFileInformation, meta = (ImportType = "SkeletalMesh|Animation", DisplayName = "Animation End Frame"))
	FString AnimEndFrame;

};


