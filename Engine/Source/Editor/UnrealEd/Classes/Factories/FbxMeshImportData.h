// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/FbxAssetImportData.h"
#include "FbxMeshImportData.generated.h"

UENUM(BlueprintType)
enum EFBXNormalImportMethod : int
{
	FBXNIM_ComputeNormals UMETA(DisplayName="Compute Normals"),
	FBXNIM_ImportNormals UMETA(DisplayName="Import Normals"),
	FBXNIM_ImportNormalsAndTangents UMETA(DisplayName="Import Normals and Tangents"),
	FBXNIM_MAX,
};

UENUM(BlueprintType)
namespace EFBXNormalGenerationMethod
{
	enum Type : int
	{
		/** Use the legacy built in method to generate normals (faster in some cases) */
		BuiltIn,
		/** Use MikkTSpace to generate normals and tangents */
		MikkTSpace,
	};
}

UENUM(BlueprintType)
namespace EVertexColorImportOption
{
	enum Type : int
	{
		/** Import the static mesh using the vertex colors from the FBX file. */
		Replace,
		/** Ignore vertex colors from the FBX file, and keep the existing mesh vertex colors. */
		Ignore,
		/** Override all vertex colors with the specified color. */
		Override
	};
}

/** Action to add nodes to the graph based on selected objects*/
USTRUCT(BlueprintType)
struct FImportMeshLodSectionsData
{
	GENERATED_USTRUCT_BODY();

	/*Every original imported fbx material name for every section*/
	UPROPERTY()
	TArray<FName> SectionOriginalMaterialName;
};


/**
 * Import data and options used when importing any mesh from FBX
 */
UCLASS(BlueprintType, config=EditorPerProjectUserSettings, configdonotcheckdefaults, abstract)
class UFbxMeshImportData : public UFbxAssetImportData
{
	GENERATED_UCLASS_BODY()

	/** If this option is true the node absolute transform (transform, offset and pivot) will be apply to the mesh vertices. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (ImportType = "StaticMesh"))
	bool bTransformVertexToAbsolute;

	/** - Experimental - If this option is true the inverse node rotation pivot will be apply to the mesh vertices. The pivot from the DCC will then be the origin of the mesh. Note: "TransformVertexToAbsolute" must be false.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (EditCondition = "!bTransformVertexToAbsolute", ImportType = "StaticMesh"))
	bool bBakePivotInVertex;

	/** Enables importing of mesh LODs from FBX LOD groups, if present in the FBX file */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(ScriptName="ImportMeshLods;ImportMeshLODs", OBJRestrict="true", ImportType="Mesh|GeoOnly", ToolTip="If enabled, creates LOD models for Unreal meshes from LODs in the import file; If not enabled, only the base mesh from the LOD group is imported"))
	uint32 bImportMeshLODs:1;

	/** Enabling this option will read the tangents(tangent,binormal,normal) from FBX file instead of generating them automatically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(ImportType="Mesh|GeoOnly"))
	TEnumAsByte<enum EFBXNormalImportMethod> NormalImportMethod;

	/** Use the MikkTSpace tangent space generator for generating normals and tangents on the mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category = Mesh, meta=(ImportType="Mesh|GeoOnly"))
	TEnumAsByte<enum EFBXNormalGenerationMethod::Type> NormalGenerationMethod;

	/** Enabling this option will use weighted normals algorithm (area and angle) when computing normals or tangents */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (OBJRestrict = "true", ImportType = "Mesh|GeoOnly"))
	uint32 bComputeWeightedNormals : 1;

	/* If checked, The material list will be reorder to the same order has the FBX file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category = Material, meta = (OBJRestrict = "true"))
	bool bReorderMaterialToFbxOrder;

	bool CanEditChange( const FProperty* InProperty ) const override;

	//////////////////////////////////////////////////////////////////////////
	//Original import section/material data
	UPROPERTY()
	TArray<FName> ImportMaterialOriginalNameData;
	
	UPROPERTY()
	TArray<FImportMeshLodSectionsData> ImportMeshLodData;
};
