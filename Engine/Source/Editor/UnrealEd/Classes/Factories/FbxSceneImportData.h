// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Factories/FbxSceneImportFactory.h"

#include "FbxSceneImportData.generated.h"

UENUM()
enum class EFbxSceneReimportStatusFlags : uint8
{
	None = 0x00,
	Added = 0x01,
	Removed = 0x02,
	Same = 0x04,
	FoundContentBrowserAsset = 0x08,
	ReimportAsset = 0x10,
};
ENUM_CLASS_FLAGS(EFbxSceneReimportStatusFlags);


UCLASS(MinimalAPI)
class UFbxSceneImportData : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
public:
	/* The path of the fbx file use for the last import */
	UPROPERTY(EditAnywhere, Category = ImportSettings)
	FString SourceFbxFile;

	/* The full name of the blueprint create at import */
	FString BluePrintFullName;

	/* Is the last reimport has import the hierarchy */
	bool bImportScene;

	/* Is the original import create a folder hierarchy */
	bool bCreateFolderHierarchy;

	/* Is the original import force front X axis */
	bool bForceFrontXAxis;

	/* Which type of hierarchy was create see */
	int32 HierarchyType;

	//The last import scene hierarchy data
	TSharedPtr<FFbxSceneInfo> SceneInfoSourceData;

	ImportOptionsNameMap NameOptionsMap;

	/** Convert this import information to JSON */
	UNREALED_API FString ToJson() const;

	/** Attempt to parse an asset import structure from the specified json string. */
	UNREALED_API void FromJson(FString InJsonString);

	UNREALED_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UNREALED_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

protected:
	/** Overridden serialize function to write out the underlying data as json */
	UNREALED_API virtual void Serialize(FArchive& Ar) override;
#endif
};



