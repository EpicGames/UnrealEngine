// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SceneImportFactory.h"
#include "FbxSceneImportFactory.generated.h"

class AActor;
class UFbxSceneImportData;
class USceneComponent;

#define INVALID_UNIQUE_ID 0xFFFFFFFFFFFFFFFF

class FFbxAttributeInfo : public TSharedFromThis<FFbxAttributeInfo>
{
public:
	FString Name;
	uint64 UniqueId;
	bool bImportAttribute;
	//Log the path where it was import so we can retrieve it when doing the re-import
	FString OriginalImportPath;
	FString OriginalFullImportName;

	bool bOriginalTypeChanged;

	bool bOverridePath;
	FString OverrideImportPath;
	FString OverrideFullImportName;

	//The name of the Options so reimport can show the options
	FString OptionName;

	//The node pivot user want to bake by default its the first node that reference the mesh
	uint64 PivotNodeUid;
	FString PivotNodeName; //Transient variable use for the ui to not display a uid

	//If there is more then one Pivots we have to present a list to the user so they can choose which node pivot they want to bake
	TMap<FVector, TArray<uint64>> NodeReferencePivots;

	FString GetImportPath()
	{
		if (bOverridePath)
		{
			return OverrideImportPath;
		}
		return OriginalImportPath;
	}

	FString GetFullImportName()
	{
		if (bOverridePath)
		{
			return OverrideFullImportName;
		}
		return OriginalFullImportName;
	}

	void SetOriginalImportPath(FString ImportPath)
	{
		OriginalImportPath = ImportPath;
		IsContentObjectUpToDate = false;
	}
	void SetOriginalFullImportName(FString FullImportName)
	{
		OriginalFullImportName = FullImportName;
		IsContentObjectUpToDate = false;
	}

	void SetOverridePath(bool OverridePath)
	{
		bOverridePath = OverridePath;
		IsContentObjectUpToDate = false;
	}

	FFbxAttributeInfo()
		: Name(TEXT(""))
		, UniqueId(INVALID_UNIQUE_ID)
		, bImportAttribute(true)
		, OriginalImportPath(TEXT(""))
		, OriginalFullImportName(TEXT(""))
		, bOriginalTypeChanged(false)
		, bOverridePath(false)
		, OverrideImportPath(TEXT(""))
		, OverrideFullImportName(TEXT(""))
		, OptionName(TEXT(""))
		, PivotNodeUid(INVALID_UNIQUE_ID)
		, PivotNodeName(TEXT("-"))
		, IsContentObjectUpToDate(false)
		, ContentPackage(nullptr)
		, ContentObject(nullptr)
	{}

	virtual ~FFbxAttributeInfo() = default;

	virtual UClass *GetType() = 0;

	UPackage *GetContentPackage();
	UObject *GetContentObject();
private:
	//Cache the existing object state
	bool IsContentObjectUpToDate;
	UPackage *ContentPackage;
	UObject *ContentObject;
};

class FFbxMeshInfo : public FFbxAttributeInfo
{
public:
	int32 FaceNum;
	int32 VertexNum;
	bool bTriangulated;
	int32 MaterialNum;
	bool bIsSkelMesh;
	FString SkeletonRoot;
	int32 SkeletonElemNum;
	FString LODGroup;
	int32 LODLevel;
	int32 MorphNum;
	bool IsLod;
	bool IsCollision;

	FFbxMeshInfo()
		: FaceNum(0)
		, VertexNum(0)
		, bTriangulated(false)
		, MaterialNum(0)
		, bIsSkelMesh(false)
		, SkeletonRoot(TEXT(""))
		, SkeletonElemNum(0)
		, LODGroup(TEXT(""))
		, LODLevel(0)
		, MorphNum(0)
		, IsLod(false)
		, IsCollision(false)
	{}

	virtual ~FFbxMeshInfo() = default;

	virtual UClass *GetType();
};

class FFbxTextureInfo : public FFbxAttributeInfo
{
public:
	FString TexturePath;

	FFbxTextureInfo()
		: TexturePath(TEXT(""))
	{}

	virtual UClass *GetType();
};

class FFbxMaterialInfo : public FFbxAttributeInfo
{
public:
	//This string is use to help match the material when doing a reimport
	FString HierarchyPath;

	//All the textures use by this material
	TArray<TSharedPtr<FFbxTextureInfo>> Textures;

	FFbxMaterialInfo()
		: HierarchyPath(TEXT(""))
	{}

	virtual UClass *GetType();
};

class FFbxCameraInfo : public TSharedFromThis<FFbxCameraInfo>
{
public:
	FString Name;
	uint64 UniqueId;

	bool ProjectionPerspective;
	float AspectWidth;
	float AspectHeight;
	float NearPlane;
	float FarPlane;
	float OrthoZoom;
	float FieldOfView;
	float FocalLength;
	float ApertureWidth;
	float ApertureHeight;

	FFbxCameraInfo()
		: Name(TEXT(""))
		, UniqueId(INVALID_UNIQUE_ID)
		, ProjectionPerspective(true)
		, AspectWidth(0.0f)
		, AspectHeight(0.0f)
		, NearPlane(0.0f)
		, FarPlane(0.0f)
		, OrthoZoom(0.0f)
		, FieldOfView(0.0f)
		, FocalLength(0.0f)
		, ApertureWidth(0.0f)
		, ApertureHeight(0.0f)
	{}
};

class FFbxLightInfo : public TSharedFromThis<FFbxLightInfo>
{
public:
	FString Name;
	uint64 UniqueId;
	int32 Type; //ePoint=0, eDirectional=1, eSpot=2, eArea=3, eVolume=4
	FColor Color; //RGB color no alpha
	float Intensity; //fbx default is 100
	int32 Decay; //eNone=0, eLinear=1, eQuadratic=2, eCubic=3
	bool CastLight;
	bool CastShadow;
	FColor ShadowColor; //RGB color no alpha

	float InnerAngle;
	float OuterAngle;
	float Fog;
	float DecayStart;
	bool EnableNearAttenuation;
	float NearAttenuationStart;
	float NearAttenuationEnd;
	bool EnableFarAttenuation;
	float FarAttenuationStart;
	float FarAttenuationEnd;

	//Notes:
	//Fbx use positive X to point light direction, we have to turn the component 90 degree in z object space

	FFbxLightInfo()
		: Name(TEXT(""))
		, UniqueId(INVALID_UNIQUE_ID)
		, Type(0)
		, Color(FColor::White)
		, Intensity(1.0f)
		, Decay(0)
		, CastLight(false)
		, CastShadow(false)
		, ShadowColor(FColor::Black)
		, InnerAngle(0.0f)
		, OuterAngle(0.0f)
		, Fog(0.0f)
		, DecayStart(0.0f)
		, EnableNearAttenuation(false)
		, NearAttenuationStart(0.0f)
		, NearAttenuationEnd(0.0f)
		, EnableFarAttenuation(false)
		, FarAttenuationStart(0.0f)
		, FarAttenuationEnd(0.0f)
	{}
};

//Node use to store the scene hierarchy transform will be relative to the parent
class FFbxNodeInfo : public TSharedFromThis<FFbxNodeInfo>
{
public:
	FString NodeName;
	uint64 UniqueId;
	FString NodeHierarchyPath;

	TSharedPtr<FFbxNodeInfo> ParentNodeInfo;
	
	TSharedPtr<FFbxAttributeInfo> AttributeInfo;
	uint64 AttributeUniqueId;
	FString AttributeType;

	FTransform Transform;
	FVector PivotRotation;
	FVector PivotScaling;
	bool bImportNode;

	TArray<TSharedPtr<FFbxNodeInfo>> Childrens;
	TArray<TSharedPtr<FFbxMaterialInfo>> Materials;

	FFbxNodeInfo()
		: NodeName(TEXT(""))
		, UniqueId(INVALID_UNIQUE_ID)
		, NodeHierarchyPath(TEXT(""))
		, ParentNodeInfo(NULL)
		, AttributeInfo(NULL)
		, AttributeUniqueId(INVALID_UNIQUE_ID)
		, AttributeType(TEXT(""))
		, Transform(FTransform::Identity)
		, bImportNode(true)
	{}
};

class FFbxSceneInfo : public TSharedFromThis<FFbxSceneInfo>
{
public:
	// data for static mesh
	int32 NonSkinnedMeshNum;
	//data for skeletal mesh
	int32 SkinnedMeshNum;
	// common data
	int32 TotalGeometryNum;
	int32 TotalMaterialNum;
	int32 TotalTextureNum;
	TArray<TSharedPtr<FFbxMeshInfo>> MeshInfo;
	TArray<TSharedPtr<FFbxNodeInfo>> HierarchyInfo;
	
	//Component attributes
	TMap<uint64, TSharedPtr<FFbxLightInfo>> LightInfo;
	TMap<uint64, TSharedPtr<FFbxCameraInfo>> CameraInfo;
	
	/* true if it has animation */
	bool bHasAnimation;
	double FrameRate;
	double TotalTime;

	FFbxSceneInfo()
		: NonSkinnedMeshNum(0)
		, SkinnedMeshNum(0)
		, TotalGeometryNum(0)
		, TotalMaterialNum(0)
		, TotalTextureNum(0)
		, bHasAnimation(false)
		, FrameRate(0.0)
		, TotalTime(0.0)
	{}

	//Function helper to find the LOD parent of a nodeinfo with a mesh attribute
	static TSharedPtr<FFbxNodeInfo> RecursiveFindLODParentNode(TSharedPtr<FFbxNodeInfo> NodeInfo)
	{
		if (!NodeInfo.IsValid())
			return nullptr;
		if (NodeInfo->ParentNodeInfo.IsValid() && NodeInfo->ParentNodeInfo->AttributeType.Compare(TEXT("eLODGroup")) == 0)
			return NodeInfo->ParentNodeInfo;

		return RecursiveFindLODParentNode(NodeInfo->ParentNodeInfo);
	}
};

namespace UnFbx
{
	struct FBXImportOptions;
}

typedef TMap<FString, UnFbx::FBXImportOptions*> ImportOptionsNameMap;
typedef ImportOptionsNameMap* ImportOptionsNameMapPtr;

UCLASS(BlueprintType, hidecategories=Object, MinimalAPI)
class UFbxSceneImportFactory : public USceneImportFactory
{
	GENERATED_UCLASS_BODY()

	/** UFactory Interface */
	UNREALED_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UNREALED_API virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	UNREALED_API virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UNREALED_API virtual bool FactoryCanImport(const FString& Filename) override;
	UNREALED_API virtual TArray<FString> GetFormats() const override;

	/** USceneImportFactory Interface */
	virtual bool ImportsAssets() const override { return true; }

	/** Import options UI detail when importing fbx scene */
	UPROPERTY(BlueprintReadWrite, Category = "Editor Scripting | Fbx Scene Import", Transient)
	TObjectPtr<class UFbxSceneImportOptions> SceneImportOptions;
	
	/** Import options UI detail when importing fbx scene static mesh*/
	UPROPERTY(BlueprintReadWrite, Category = "Editor Scripting | Fbx Scene Import", Transient)
	TObjectPtr<class UFbxSceneImportOptionsStaticMesh> SceneImportOptionsStaticMesh;
	
	/** Import options UI detail when importing fbx scene skeletal mesh*/
	UPROPERTY(BlueprintReadWrite, Category = "Editor Scripting | Fbx Scene Import", Transient)
	TObjectPtr<class UFbxSceneImportOptionsSkeletalMesh> SceneImportOptionsSkeletalMesh;
	
	/** Import data used when importing static meshes */
	UPROPERTY(Transient)
	TObjectPtr<class UFbxStaticMeshImportData> StaticMeshImportData;

	/** Import data used when importing skeletal meshes */
	UPROPERTY(Transient)
	TObjectPtr<class UFbxSkeletalMeshImportData> SkeletalMeshImportData;

	/** Import data used when importing animations */
	UPROPERTY(Transient)
	TObjectPtr<class UFbxAnimSequenceImportData> AnimSequenceImportData;

	/** Import data used when importing textures */
	UPROPERTY(Transient)
	TObjectPtr<class UFbxTextureImportData> TextureImportData;
	
	/* Default Options always have the same name "Default" */
	static UNREALED_API FString DefaultOptionName;

public:
	static UNREALED_API TSharedPtr<FFbxSceneInfo> ConvertSceneInfo(void* VoidFbxImporter, void* VoidFbxSceneInfo);
	static UNREALED_API void ExtractMaterialInfo(void* FbxImporterVoid, TSharedPtr<FFbxSceneInfo> SceneInfoPtr);

protected:
	/** Convert the scene and remake all the transform for the SceneInfo pass in parameter.
	 *  We need this because EvaluateGlobal and EvaluateLocal are dependent of the scene conversion.
	 */
	UNREALED_API void ChangeFrontAxis(void* VoidFbxImporter, void* VoidSceneInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr);

	/** Make sure GlobalImportSettings is pointing to the correct options */
	UNREALED_API void ApplyMeshInfoFbxOptions(TSharedPtr<FFbxMeshInfo> MeshInfo);

	/* Compute the path of every node and fill the result in the node. This data will be use by the reimport
	*  as a unique key for for the reimport status of the node hierarchy.
	*/
	static UNREALED_API void FillSceneHierarchyPath(TSharedPtr<FFbxSceneInfo> SceneInfo);

	/** Create a hierarchy of actor in the current level */
	UNREALED_API void CreateLevelActorHierarchy(TSharedPtr<FFbxSceneInfo> SceneInfoPtr);

	/** Create a hierarchy of actor in the current level */
	UNREALED_API AActor *CreateActorComponentsHierarchy(TSharedPtr<FFbxSceneInfo> SceneInfoPtr);

	/** Apply the LocalTransform to the SceneComponent and if PreMultiplyTransform is not null do a pre multiplication
	* SceneComponent: Must be a valid pointer
	* LocalTransform: Must be a valid pointer
	* PreMultiplyTransform: Can be nullptr
	*/
	UNREALED_API void ApplyTransformToComponent(USceneComponent *SceneComponent, FTransform *LocalTransform, FTransform *PreMultiplyTransform, FVector &PivotLocation, FVector &ParentPivotAccumulation);

	/** Import all skeletal mesh from the fbx scene */
	UNREALED_API void ImportAllSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, EObjectFlags Flags, int32& NodeIndex, int32& InterestingNodeCount , TSharedPtr<FFbxSceneInfo> SceneInfo);
	
	UNREALED_API UObject* ImportOneSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, TSharedPtr<FFbxSceneInfo> SceneInfo, EObjectFlags Flags, TArray<void*> &VoidNodeArray, int32 &TotalNumNodes);

	/** Import all static mesh from the fbx scene */
	UNREALED_API void ImportAllStaticMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, EObjectFlags Flags, int32& NodeIndex, int32& InterestingNodeCount, TSharedPtr<FFbxSceneInfo> SceneInfo);

	// @todo document
	UNREALED_API UObject* RecursiveImportNode(void* FFbxImporter, void* VoidNode, EObjectFlags Flags, int32& Index, int32 Total, TSharedPtr<FFbxSceneInfo> SceneInfo, FString PackagePath);

	// @todo document
	UNREALED_API UObject* ImportANode(void* VoidFbxImporter, TArray<void*> &VoidNode, EObjectFlags Flags, int32& NodeIndex, TSharedPtr<FFbxSceneInfo> SceneInfo, TSharedPtr<FFbxNodeInfo> &OutNodeInfo, FString PackagePath, int32 Total = 0, UObject* InMesh = NULL, int LODIndex = 0);

	/** Find the FFbxNodeInfo in the hierarchy. */
	UNREALED_API bool FindSceneNodeInfo(TSharedPtr<FFbxSceneInfo> SceneInfo, uint64 NodeInfoUniqueId, TSharedPtr<FFbxNodeInfo> &OutNodeInfo);

	/** Create a package for the specified node. Package will be the concatenation of UFbxSceneImportFactory::Path and Node->GetName(). */
	UNREALED_API UPackage *CreatePackageForNode(FString PackageName, FString &StaticMeshName);

	UNREALED_API bool SetStaticMeshComponentOverrideMaterial(class UStaticMeshComponent* StaticMeshComponent, TSharedPtr<FFbxNodeInfo> NodeInfo);

	/** The path of the asset to import */
	FString Path;

	/** Pointer on the fbx scene import data, we fill this object to be able to do re import of the scene */
	UPROPERTY()
	TObjectPtr<UFbxSceneImportData> ReimportData;
	
	/** Assets created by the factory*/
	TMap<TSharedPtr<FFbxAttributeInfo>, UObject*> AllNewAssets;

	/*Global Setting for non overriden Node*/
	UnFbx::FBXImportOptions* GlobalImportSettings;

	/*The Global Settings Reference*/
	UnFbx::FBXImportOptions* GlobalImportSettingsReference;
	
	/*The options dictionary*/
	ImportOptionsNameMap NameOptionsMap;

	/* Return the Options from the NameOptionMap Map. return nulptr if the options are not found*/
	UNREALED_API UnFbx::FBXImportOptions *GetOptionsFromName(FString OptionName);

	/** Is the import was cancel*/
	bool ImportWasCancel;
};



