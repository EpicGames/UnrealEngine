// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "FbxSceneImportOptions.generated.h"

UENUM(BlueprintType)
enum class EFBXSceneOptionsCreateHierarchyType : uint8
{
	FBXSOCHT_CreateLevelActors UMETA(DisplayName = "Create Level Actors", ToolTip = "Create an actor for every node in the fbx hierarchy. No reimport of the hierarchy."),
	FBXSOCHT_CreateActorComponents UMETA(DisplayName = "Create one Actor with Components", ToolTip = "Create one actor and a component for every node in the fbx hierarchy. No reimport of the hierarchy."),
	FBXSOCHT_CreateBlueprint UMETA(DisplayName = "Create one Blueprint asset", ToolTip = "Create one blueprint and a component for every node in the fbx hierarchy. Hierarchy can be reimport."),
	FBXSOCHT_MAX
};
//	Fbx_AddToBlueprint UMETA(DisplayName = "Add to an existing Blueprint asset"),



UCLASS(BlueprintType, config = EditorPerProjectUserSettings, HideCategories=Object, MinimalAPI)
class UFbxSceneImportOptions : public UObject
{
	GENERATED_UCLASS_BODY()
	
	//////////////////////////////////////////////////////////////////////////
	// Scene section, detail of a fbx node
	// TODO: Move the content folder selection here with a custom detail class
	// TODO: Allow the user to select an existing blueprint
	// TODO: Allow the user to specify a name for the root node

	/** TODO - The Content Path choose by the user */
	//UPROPERTY()
	//FString ContentPath;

	/** If checked, a folder's hierarchy will be created under the import asset path. All the created folders will match the actor hierarchy. In case there is more than one actor that links to an asset, the shared asset will be placed at the first actor's place. */
	UPROPERTY(EditAnywhere, config, category = ImportOptions)
	uint32 bCreateContentFolderHierarchy : 1;

	/** If checked, the mobility of all actors or components will be dynamic. If unchecked, they will be static. */
	UPROPERTY(EditAnywhere, config, category = ImportOptions)
	uint32 bImportAsDynamic : 1;

	/** Choose if you want to generate the scene hierarchy with normal level actors, one actor with multiple components, or one blueprint asset with multiple components. */
	UPROPERTY(EditAnywhere, config, category = ImportOptions)
	EFBXSceneOptionsCreateHierarchyType HierarchyType;

	/** Whether to force the front axis to be align with X instead of -Y. */
	UPROPERTY(EditAnywhere, config, category = ImportOptions)
	uint32 bForceFrontXAxis : 1;

	UPROPERTY()
	FVector ImportTranslation;

	UPROPERTY()
	FRotator ImportRotation;

	UPROPERTY()
	float ImportUniformScale;

	/** If this option is true the node absolute transform (transform, offset and pivot) will be apply to the mesh vertices. */
	UPROPERTY()
	bool bTransformVertexToAbsolute;

	/** - Experimental - If this option is true the inverse node pivot will be apply to the mesh vertices. The pivot from the DCC will then be the origin of the mesh. This option only work with static meshes.*/
	UPROPERTY(EditAnywhere, config, category = Meshes)
	bool bBakePivotInVertex;

	/** If enabled, creates LOD models for Unreal static meshes from LODs in the import file; If not enabled, only the base static mesh from the LOD group is imported. */
	UPROPERTY(EditAnywhere, config, category = Meshes)
	uint32 bImportStaticMeshLODs : 1;

	/** If enabled, creates LOD models for Unreal skeletal meshes from LODs in the import file; If not enabled, only the base skeletal mesh from the LOD group is imported. */
	UPROPERTY(EditAnywhere, config, category = Meshes)
	uint32 bImportSkeletalMeshLODs : 1;

	/** If enabled, this option will cause normal map Y (Green) values to be inverted when importing textures */
	UPROPERTY(EditAnywhere, config, category = Texture)
	uint32 bInvertNormalMaps : 1;
};



