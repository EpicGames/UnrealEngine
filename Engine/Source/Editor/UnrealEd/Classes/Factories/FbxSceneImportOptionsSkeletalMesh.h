// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "MeshBuild.h"
#include "FbxSceneImportOptionsSkeletalMesh.generated.h"

UCLASS(BlueprintType, config=EditorPerProjectUserSettings, HideCategories=Object, MinimalAPI)
class UFbxSceneImportOptionsSkeletalMesh : public UObject
{
	GENERATED_UCLASS_BODY()
	

	//////////////////////////////////////////////////////////////////////////
	// Skeletal Mesh section

	/** Enable this option to update Skeleton (of the mesh)'s reference pose. Mesh's reference pose is always updated.  */
	UPROPERTY(EditAnywhere, Category = SkeletalMesh, meta = (ToolTip = "If enabled, update the Skeleton (of the mesh being imported)'s reference pose."))
	uint32 bUpdateSkeletonReferencePose : 1;

	/** If checked, create new PhysicsAsset if it doesn't have it */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh)
	uint32 bCreatePhysicsAsset : 1;

	/** TODO support T0AsRefPose Enable this option to use frame 0 as reference pose */
	UPROPERTY()
	uint32 bUseT0AsRefPose : 1;

	/** If checked, triangles with non-matching smoothing groups will be physically split. */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh)
	uint32 bPreserveSmoothingGroups : 1;

	/** If checked, sections with matching materials are kept separate and will not get combined. */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh)
	uint32 bKeepSectionsSeparate : 1;

	/** If checked, meshes nested in bone hierarchies will be imported instead of being converted to bones. */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh)
	uint32 bImportMeshesInBoneHierarchy : 1;

	/** True to import morph target meshes from the FBX file */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh, meta = (ToolTip = "If enabled, creates Unreal morph objects for the imported meshes"))
	uint32 bImportMorphTargets : 1;

	/** True to import morph target meshes from the FBX file */
	UPROPERTY(EditAnywhere, config, Category = SkeletalMesh, meta = (ToolTip = "If enabled, import single-channel/weight/alpha vertex attributes"))
	uint32 bImportVertexAttributes : 1;

	/** Threshold to compare vertex position equality. */
	UPROPERTY(EditAnywhere, config, Category = "SkeletalMesh|Thresholds", meta = (NoSpinbox = "true", ClampMin = "0.0"))
	float ThresholdPosition;
	
	/** Threshold to compare normal, tangent or bi-normal equality. */
	UPROPERTY(EditAnywhere, config, Category = "SkeletalMesh|Thresholds", meta = (NoSpinbox = "true", ClampMin = "0.0", ClampMax = "1.0"))
	float ThresholdTangentNormal;
	
	/** Threshold to compare UV equality. */
	UPROPERTY(EditAnywhere, config, Category = "SkeletalMesh|Thresholds", meta = (NoSpinbox = "true", ClampMin = "0.0", ClampMax = "1.0"))
	float ThresholdUV;

	/** Threshold to compare vertex position equality when computing morph target deltas. */
	UPROPERTY(EditAnywhere, config, Category = "SkeletalMesh|Thresholds", meta = (NoSpinbox = "true", ClampMin = "0.0"))
	float MorphThresholdPosition;

	//////////////////////////////////////////////////////////////////////////
	// Animation section

	/** True to import animations from the FBX File */
	UPROPERTY(EditAnywhere, config, Category = Animation)
	uint32 bImportAnimations : 1;

	/** Type of asset to import from the FBX file */
	UPROPERTY(EditAnywhere, Category = Animation, config, meta = (DisplayName = "Animation Length"))
	TEnumAsByte<enum EFBXAnimationLengthImportType> AnimationLength;

	/** Frame range used when Set Range is used in Animation Length */
	UPROPERTY(EditAnywhere, Category = Animation, meta = (UIMin = 0, ClampMin = 0))
	FInt32Interval FrameImportRange;

	/** Enable this option to use default sample rate for the imported animation at 30 frames per second */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (ToolTip = "If enabled, samples all animation curves to 30 FPS"))
	bool bUseDefaultSampleRate;

	/** Use this option to specify a sample rate for the imported animation, a value of 0 use the best matching sample rate. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (EditCondition = "!bUseDefaultSampleRate", ToolTip = "Sample fbx animation data at the specified sample rate, 0 find automaticaly the best sample rate"))
	int32 CustomSampleRate;
	
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (ToolTip = "If enabled, snaps the animation to the closest frame boundary using the import sampling rate"))
	bool bSnapToClosestFrameBoundary;

	/** If true, import node attributes as either Animation Curves or Animation Attributes */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (DisplayName = "Import Attributes as Curves or Animation Attributes"))
	bool bImportCustomAttribute;

	/** If true, all previous node attributes imported as Animation Curves will be deleted when doing a re-import. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (DisplayName = "Delete existing Animation Curves"))
	bool bDeleteExistingCustomAttributeCurves;

	/** If true, all previous node attributes imported as Animation Attributes will be deleted when doing a re-import. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation, meta = (DisplayName = "Delete existing Animation Attributes"))
	bool bDeleteExistingNonCurveCustomAttributes;

	/** Type of asset to import from the FBX file */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation)
	bool bPreserveLocalTransform;

	/** Type of asset to import from the FBX file */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Animation)
	bool bDeleteExistingMorphTargetCurves;

	void FillSkeletalMeshInmportData(class UFbxSkeletalMeshImportData* SkeletalMeshImportData, class UFbxAnimSequenceImportData* AnimSequenceImportData, class UFbxSceneImportOptions* SceneImportOptions);
};



