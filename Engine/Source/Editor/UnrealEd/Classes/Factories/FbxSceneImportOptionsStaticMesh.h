// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Fbx Importer UI options.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "FbxSceneImportOptionsStaticMesh.generated.h"

UENUM(BlueprintType)
enum class EFbxSceneVertexColorImportOption : uint8
{
	/** Import the static mesh using the vertex colors from the FBX file. */
	Replace,
	/** Ignore vertex colors from the FBX file, and keep the existing mesh vertex colors. */
	Ignore,
	/** Override all vertex colors with the specified color. */
	Override
};

UENUM(BlueprintType)
enum class EFBXSceneNormalImportMethod : uint8
{
	FBXSceneNIM_ComputeNormals UMETA(DisplayName = "Compute Normals"),
	FBXSceneNIM_ImportNormals UMETA(DisplayName = "Import Normals"),
	FBXSceneNIM_ImportNormalsAndTangents UMETA(DisplayName = "Import Normals and Tangents"),
	FBXSceneNIM_MAX,
};

UENUM(BlueprintType)
enum class EFBXSceneNormalGenerationMethod : uint8
{
	/** Use the legacy built in method to generate normals (faster in some cases) */
	BuiltIn,
	/** Use MikkTSpace to generate normals and tangents */
	MikkTSpace,
};


UCLASS(BlueprintType, config = EditorPerProjectUserSettings, HideCategories=Object, MinimalAPI)
class UFbxSceneImportOptionsStaticMesh : public UObject
{
	GENERATED_UCLASS_BODY()
	
	//////////////////////////////////////////////////////////////////////////
	// Static Mesh section

	/** For static meshes, enabling this option will combine all meshes in the FBX into a single monolithic mesh in Unreal */
	UPROPERTY()
	FName StaticMeshLODGroup;

	/** If checked, collision will automatically be generated (ignored if custom collision is imported or used). */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh, DisplayName = "Generate Missing Collision")
	uint32 bAutoGenerateCollision : 1;

	/** Specify how vertex colors should be imported */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	EFbxSceneVertexColorImportOption VertexColorImportOption;

	/** Specify override color in the case that VertexColorImportOption is set to Override */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	FColor VertexOverrideColor;

	/** Disabling this option will keep degenerate triangles found.  In general you should leave this option on. */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	uint32 bRemoveDegenerates : 1;

	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	uint32 bBuildReversedIndexBuffer : 1;

	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	uint32 bGenerateLightmapUVs : 1;

	/** If checked, one convex hull per UCX_ prefixed collision mesh will be generated instead of decomposing into multiple hulls */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	uint32 bOneConvexHullPerUCX : 1;

	/** Enabling this option will read the tangents(tangent,binormal,normal) from FBX file instead of generating them automatically. */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	EFBXSceneNormalImportMethod NormalImportMethod;

	/** Use the MikkTSpace tangent space generator for generating normals and tangents on the mesh */
	UPROPERTY(EditAnywhere, config, Category = StaticMesh)
	EFBXSceneNormalGenerationMethod NormalGenerationMethod;

	void FillStaticMeshInmportData(class UFbxStaticMeshImportData* StaticMeshImportData, class UFbxSceneImportOptions* SceneImportOptions);
};



