// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/FbxMeshImportData.h"
#include "MeshBuild.h"
#include "FbxSkeletalMeshImportData.generated.h"

class USkeletalMesh;
class USkeleton;
class FSkeletalMeshImportData;
class FSkeletalMeshLODModel;

struct FExistingSkelMeshData;
struct FReferenceSkeleton;
struct FSkeletalMaterial;

namespace SkeletalMeshImportData
{
	struct FRawBoneInfluence;
	struct FVertex;
}

UENUM(BlueprintType)
enum EFBXImportContentType : int
{
	FBXICT_All UMETA(DisplayName = "Geometry and Skinning Weights.", ToolTip = "Import all fbx content: geometry, skinning and weights."),
	FBXICT_Geometry UMETA(DisplayName = "Geometry Only", ToolTip = "Import the skeletal mesh geometry only (will create a default skeleton, or map the geometry to the existing one). Morph and LOD can be imported with it."),
	FBXICT_SkinningWeights UMETA(DisplayName = "Skinning Weights Only", ToolTip = "Import the skeletal mesh skinning and weights only (no geometry will be imported). Morph and LOD will not be imported with this settings."),
	FBXICT_MAX,
};

/**
 * Import data and options used when importing a static mesh from fbx
 * Notes:
 * - Meta data ImportType i.e.       meta = (ImportType = "SkeletalMesh|GeoOnly")
 *     - SkeletalMesh : the property will be shown when importing skeletalmesh
 *     - GeoOnly: The property will be hide if we import skinning only
 *     - RigOnly: The property will be hide if we import geo only
 *     - RigAndGeo: The property will be show only if we import both skinning and geometry, it will be hiden otherwise
 */
UCLASS(BlueprintType, MinimalAPI)
class UFbxSkeletalMeshImportData : public UFbxMeshImportData
{
	GENERATED_UCLASS_BODY()
public:
	virtual void Serialize(FArchive& Ar) override;

	/** Filter the content we want to import from the incoming FBX skeletal mesh.*/
	UPROPERTY(EditAnywhere, Category = Mesh, meta = (ImportType = "SkeletalMesh", DisplayName = "Import Content Type", OBJRestrict = "true"))
	TEnumAsByte<enum EFBXImportContentType> ImportContentType;
	
	/** The value of the content type during the last import. This cannot be edited and is set only on successful import or re-import*/
	UPROPERTY()
	TEnumAsByte<enum EFBXImportContentType> LastImportContentType;

	/** Specify how vertex colors should be imported */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (OBJRestrict = "true", ImportType = "SkeletalMesh|GeoOnly"))
	TEnumAsByte<EVertexColorImportOption::Type> VertexColorImportOption;

	/** Specify override color in the case that VertexColorImportOption is set to Override */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (OBJRestrict = "true", ImportType = "SkeletalMesh|GeoOnly"))
	FColor VertexOverrideColor;

	/** Enable this option to update Skeleton (of the mesh)'s reference pose. Mesh's reference pose is always updated.  */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh|RigOnly", ToolTip="If enabled, update the Skeleton (of the mesh being imported)'s reference pose."))
	uint32 bUpdateSkeletonReferencePose:1;

	/** Enable this option to use frame 0 as reference pose */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category= Mesh, meta=(ImportType="SkeletalMesh|RigAndGeo", DisplayName="Use T0 As Ref Pose"))
	uint32 bUseT0AsRefPose:1;

	/** If checked, triangles with non-matching smoothing groups will be physically split. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh|GeoOnly"))
	uint32 bPreserveSmoothingGroups:1;

	/** If checked, sections with matching materials are kept separate and will not get combined. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh|GeoOnly"))
	uint32 bKeepSectionsSeparate:1;

	/** If checked, meshes nested in bone hierarchies will be imported instead of being converted to bones. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh"))
	uint32 bImportMeshesInBoneHierarchy:1;

	/** True to import morph target meshes from the FBX file */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh|GeoOnly", ToolTip="If enabled, creates Unreal morph objects for the imported meshes"))
	uint32 bImportMorphTargets:1;
	
	/** True to import per-vertex attributes from the FBX file */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category=Mesh, meta=(ImportType="SkeletalMesh|GeoOnly", ToolTip="If enabled, creates a named vertex attribute for each single-channel weight map of the imported mesh."))
	uint32 bImportVertexAttributes:1;

	/** Threshold to compare vertex position equality. */
	UPROPERTY(EditAnywhere, config, Category="Mesh", meta = (ImportType = "SkeletalMesh|GeoOnly", SubCategory = "Thresholds", NoSpinbox = "true", ClampMin = "0.0"))
	float ThresholdPosition;
	
	/** Threshold to compare normal, tangent or bi-normal equality. */
	UPROPERTY(EditAnywhere, config, Category="Mesh", meta = (ImportType = "SkeletalMesh|GeoOnly", SubCategory = "Thresholds", NoSpinbox = "true", ClampMin = "0.0", ClampMax = "1.0"))
	float ThresholdTangentNormal;
	
	/** Threshold to compare UV equality. */
	UPROPERTY(EditAnywhere, config, Category="Mesh", meta = (ImportType = "SkeletalMesh|GeoOnly", SubCategory = "Thresholds", NoSpinbox = "true", ClampMin = "0.0", ClampMax = "1.0"))
	float ThresholdUV;

	/** Threshold to compare vertex position equality when computing morph target deltas. */
	UPROPERTY(EditAnywhere, config, Category = "Mesh", meta = (editcondition = "bImportMorphTargets", ImportType = "SkeletalMesh|GeoOnly", SubCategory = "Thresholds", NoSpinbox = "true", ClampMin = "0.0", ClampMax = "1.0"))
	float MorphThresholdPosition;

	/** Gets or creates fbx import data for the specified skeletal mesh */
	static UFbxSkeletalMeshImportData* GetImportDataForSkeletalMesh(USkeletalMesh* SkeletalMesh, UFbxSkeletalMeshImportData* TemplateForCreation);

	bool CanEditChange( const FProperty* InProperty ) const override;

	bool GetImportContentFilename(FString& OutFilename, FString& OutFilenameLabel) const;

	/** This function add the last import content type to the asset registry which is use by the thumbnail overlay of the skeletal mesh */
	virtual void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) override;
};