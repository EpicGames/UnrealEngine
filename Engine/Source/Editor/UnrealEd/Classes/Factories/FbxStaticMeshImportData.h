// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Import data and options used when importing a static mesh from fbx
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/FbxMeshImportData.h"
#include "FbxStaticMeshImportData.generated.h"

class UStaticMesh;

UCLASS(BlueprintType, config=EditorPerProjectUserSettings, AutoExpandCategories=(Options), MinimalAPI)
class UFbxStaticMeshImportData : public UFbxMeshImportData
{
	GENERATED_UCLASS_BODY()

	/** The LODGroup to associate with this mesh when it is imported */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category=Mesh, meta=(ImportType="StaticMesh"))
	FName StaticMeshLODGroup;

	/** Specify how vertex colors should be imported */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(OBJRestrict="true", ImportType="StaticMesh"))
	TEnumAsByte<EVertexColorImportOption::Type> VertexColorImportOption;

	/** Specify override color in the case that VertexColorImportOption is set to Override */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(OBJRestrict="true", ImportType="StaticMesh"))
	FColor VertexOverrideColor;

	/** Disabling this option will keep degenerate triangles found.  In general you should leave this option on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (ImportType = "StaticMesh", ReimportRestrict = "true", EditCondition = "!bBuildNanite"))
	uint32 bRemoveDegenerates:1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (ImportType = "StaticMesh", ReimportRestrict = "true"))
	uint32 bBuildReversedIndexBuffer:1;

	/** For static meshes, enabling this option will allow you to use Nanite rendering at runtime. Can only be used with simple opaque materials. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Mesh, meta = (ToolTip = "If enabled, allows to render objects with Nanite", ReimportRestrict = "true", ImportType = "StaticMesh"))
	uint32 bBuildNanite : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, AdvancedDisplay, Category= Mesh, meta=(ImportType="StaticMesh", ReimportRestrict = "true"))
	uint32 bGenerateLightmapUVs:1;

	/** If checked, one convex hull per UCX_ prefixed collision mesh will be generated instead of decomposing into multiple hulls */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(OBJRestrict="true", ImportType="StaticMesh"))
	uint32 bOneConvexHullPerUCX:1;

	/** If checked, collision will automatically be generated (ignored if custom collision is imported or used). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Mesh, meta=(OBJRestrict="true", ImportType="StaticMesh", DisplayName="Generate Missing Collision"))
	uint32 bAutoGenerateCollision : 1;

	/** For static meshes, enabling this option will combine all meshes in the FBX into a single monolithic mesh in Unreal */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category = Mesh, meta = (ToolTip = "If enabled, combines all meshes into a single mesh", ImportType = "StaticMesh"))
	uint32 bCombineMeshes : 1;

	/**
	 * Scale to apply to the mesh when allocating the distance field volume texture.
	 * The default scale is 1, which is assuming that the mesh will be placed unscaled in the world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category= Mesh, meta=(ImportType="StaticMesh"))
	float DistanceFieldResolutionScale;

	/** Gets or creates fbx import data for the specified static mesh */
	static UFbxStaticMeshImportData* GetImportDataForStaticMesh(UStaticMesh* StaticMesh, UFbxStaticMeshImportData* TemplateForCreation);

	bool CanEditChange( const FProperty* InProperty ) const override;
};



