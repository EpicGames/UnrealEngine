// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "Factories/FbxAssetImportData.h"
#include "MaterialImportHelpers.h"
#include "FbxTextureImportData.generated.h"

/**
 * Import data and options used when importing any mesh from FBX
 */
UCLASS(BlueprintType, AutoExpandCategories=(Texture), MinimalAPI)
class UFbxTextureImportData : public UFbxAssetImportData
{
	GENERATED_UCLASS_BODY()

	/** If importing textures is enabled, this option will cause normal map Y (Green) values to be inverted */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, config, Category=ImportSettings, meta=(OBJRestrict="true"))
	uint32 bInvertNormalMaps:1;

	/** Specify where we should search for matching materials when importing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = ImportSettings, meta = (DisplayName="Search Location", OBJRestrict = "true", ImportType = "Mesh"))
	EMaterialSearchLocation MaterialSearchLocation;

	/** Base material to instance from when importing materials. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh", AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath BaseMaterialName;

	/** transient, ImportUI customize helper to store if we must show or not the BaseMaterialName property. */
	bool bUseBaseMaterial;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseColorName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseDiffuseTextureName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseNormalTextureName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseEmissiveColorName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseEmmisiveTextureName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseSpecularTextureName;

	UPROPERTY(BlueprintReadWrite, config, Category = Material, meta = (ImportType = "Mesh"))
	FString BaseOpacityTextureName;

	UNREALED_API bool CanEditChange( const FProperty* InProperty ) const override;
};
