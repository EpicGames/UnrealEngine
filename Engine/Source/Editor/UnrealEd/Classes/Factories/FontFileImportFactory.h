// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// FontFactory: Creates a Font Factory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "FontFileImportFactory.generated.h"

UENUM(BlueprintType)
enum class EBatchCreateFontAsset : uint8
{
	// Internal do not use
	Unknown,
	// Internal do not use
	Cancel,
	// Create a UFont to match the new UFontFace
	Yes,
	// Do not create a UFont to match the new UFontFace
	No,
	// Only create a new font if one doesn't already exist, this will help with automated/script imports
	CreateIfNoFontExists,
};

UCLASS(MinimalAPI)
class UFontFileImportFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateBinary(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

	//~ Begin FReimportHandler Interface
	UNREALED_API virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	UNREALED_API virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	UNREALED_API virtual EReimportResult::Type Reimport(UObject* Obj) override;
	UNREALED_API virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

	UPROPERTY(BlueprintReadWrite, Category = Misc)
	EBatchCreateFontAsset BatchCreateFontAsset;
};
