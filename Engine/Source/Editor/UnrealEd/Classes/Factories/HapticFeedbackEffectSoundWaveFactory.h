// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "HapticFeedbackEffectSoundWaveFactory.generated.h"

UCLASS(hidecategories=Object)
class UHapticFeedbackEffectSoundWaveFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	// Begin UFactory Interface	
};



