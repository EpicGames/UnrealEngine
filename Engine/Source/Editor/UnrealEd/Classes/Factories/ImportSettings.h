// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Base class for all factories to provide import settings
 * 
 * Supports filling the settings from a json file for automated importing
 */
class IImportSettingsParser
{
public:
	/**
	 * Parses settings data from a json object.  The format of the object is dependent on what the derived types expect
	 */
	virtual void ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson) = 0;
};



