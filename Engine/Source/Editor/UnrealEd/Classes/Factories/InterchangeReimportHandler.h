// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportFbxSkeletalMeshFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "InterchangeReimportHandler.generated.h"

/**
 * This FReimportHandler class is temporary until we remove the UFactory import/reimport code.
 * We use this ReimportHandler factory to make sure the set reimport is not touching the UInterchangeAssetImportData when reimporting an asset
 */
UCLASS(MinimalAPI, collapsecategories)
class UInterchangeReimportHandler : public UObject, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override
	{
		for (int32 SourceIndex = 0; SourceIndex < NewReimportPaths.Num(); ++SourceIndex)
		{
			SetReimportPaths(Obj, NewReimportPaths[SourceIndex], SourceIndex);
		}
	}
	virtual void SetReimportPaths(UObject* Obj, const FString& NewReimportPath, const int32 SourceIndex) override;

	virtual void SetReimportSourceIndex(UObject* Obj, const int32 SourceIndex) override;

	virtual EReimportResult::Type Reimport(UObject* Obj) override
	{
		return Reimport(Obj, INDEX_NONE);
	}
	virtual EReimportResult::Type Reimport(UObject* Obj, int32 SourceFileIndex);
	virtual int32 GetPriority() const override;

	virtual bool IsInterchangeFactory() const override
	{
		return true;
	}
	//~ End FReimportHandler Interface

};
