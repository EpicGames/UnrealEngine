// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// LevelFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "LevelFactory.generated.h"

UCLASS(MinimalAPI)
class ULevelFactory : public UFactory
{
	GENERATED_UCLASS_BODY()


	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateText( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn ) override;
	//~ End UFactory Interface

	UNREALED_API UObject* LevelFactoryCreateText(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn, TArray<AActor*>* OutActors);
};



