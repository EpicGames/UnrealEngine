// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "LightWeightInstanceFactory.generated.h"

class ADEPRECATED_LightWeightInstanceManager;

UCLASS(hidecategories=Object, MinimalAPI, Deprecated, meta = (DeprecationMessage ="Deprecated in 5.8. Consider using InstancedActors for similar functionality"))
class UDEPRECATED_LightWeightInstanceFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

protected:

	// The parent class of the created blueprint
	UPROPERTY()
	TSubclassOf<class UObject> ParentClass;
};
