// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// MaterialFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "MaterialFactoryNew.generated.h"

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UMaterialFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** An initial texture to place in the newly created material */
	UPROPERTY()
	TObjectPtr<class UTexture> InitialTexture;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ Begin UFactory Interface	
};
