// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "MaterialFunctionInstanceFactory.generated.h"

UCLASS(MinimalAPI, hidecategories=Object, collapsecategories)
class UMaterialFunctionInstanceFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UMaterialFunctionInterface> InitialParent;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

UCLASS(MinimalAPI, hidecategories = Object, collapsecategories)
class UMaterialFunctionMaterialLayerInstanceFactory : public UMaterialFunctionInstanceFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	FString GetDefaultNewAssetName() const override
	{
		return FString(TEXT("NewMaterialLayerInstance"));
	}
	//~ Begin UFactory Interface	
};


UCLASS(MinimalAPI, hidecategories = Object, collapsecategories)
class UMaterialFunctionMaterialLayerBlendInstanceFactory : public UMaterialFunctionInstanceFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	FString GetDefaultNewAssetName() const override
	{
		return FString(TEXT("NewMaterialLayerBlendInstance"));
	}
	//~ Begin UFactory Interface	
};