// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/TextProperty.h"
#include "MaterialImportHelpers.generated.h"

class UMaterialInterface;

UENUM(BlueprintType)
enum class EMaterialSearchLocation : uint8
{
	/** Search for matching material in local import folder only. */
	Local,
	/** Search for matching material recursively from parent folder. */
	UnderParent,
	/** Search for matching material recursively from root folder. */
	UnderRoot,
	/** Search for matching material in all assets folders. */
	AllAssets,
	/** Do not search for existing matching materials */
	DoNotSearch,
};

UCLASS(transient)
class UMaterialImportHelpers : public UObject
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Materials")
	static UNREALED_API UMaterialInterface* FindExistingMaterialFromSearchLocation(const FString& MaterialFullName, const FString& BasePackagePath, EMaterialSearchLocation SearchLocation, FText& OutError);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Materials")
	static UNREALED_API UMaterialInterface* FindExistingMaterial(const FString& BasePath, const FString& MaterialFullName, const bool bRecursivePaths, FText& OutError);
};
