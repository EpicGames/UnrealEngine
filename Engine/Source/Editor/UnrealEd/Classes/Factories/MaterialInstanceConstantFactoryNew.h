// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// MaterialInstanceConstantFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "MaterialInstanceConstantFactoryNew.generated.h"

UCLASS(hidecategories=Object, collapsecategories,MinimalAPI)
class UMaterialInstanceConstantFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UMaterialInterface> InitialParent;

	//~ Begin UFactory Interface
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	UNREALED_API virtual bool ShouldShowInNewMenu() const override;
	//~ Begin UFactory Interface	
};



