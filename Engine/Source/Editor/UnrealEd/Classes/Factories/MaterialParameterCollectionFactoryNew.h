// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// MaterialParameterCollectionFactoryNew.h
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "MaterialParameterCollectionFactoryNew.generated.h"

UCLASS(MinimalAPI, hidecategories=Object, collapsecategories)
class UMaterialParameterCollectionFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};



