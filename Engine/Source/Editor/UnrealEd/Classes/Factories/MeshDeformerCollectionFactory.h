// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "MeshDeformerCollectionFactory.generated.h"


UCLASS(hidecategories = Object, MinimalAPI)
class UMeshDeformerCollectionFactory : public UFactory
{
public:
	GENERATED_BODY()
	UMeshDeformerCollectionFactory();

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	// End of UFactory interface
};