// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Animation/MirrorDataTable.h"
#include "MirrorDataTableFactory.generated.h"

class UMirrorDataTable;

UCLASS(MinimalAPI)
class UMirrorTableFindReplaceExpressions : public UObject
{
	GENERATED_BODY()
	UMirrorTableFindReplaceExpressions(const FObjectInitializer& ObjectInitializer);
public:
	// Collection of animations for motion matching
	UPROPERTY(EditAnywhere, Category = TableCreation)
	TArray<FMirrorFindReplaceExpression> FindReplaceExpressions;
};

UCLASS(hidecategories=Object, MinimalAPI)
class UMirrorDataTableFactory : public UFactory
{
	GENERATED_BODY()
	UMirrorDataTableFactory(const FObjectInitializer& ObjectInitializer);

	UPROPERTY()
	TObjectPtr<const class UScriptStruct> Struct;

	UPROPERTY()
	TObjectPtr<class USkeleton> Skeleton;

	UPROPERTY()
	TObjectPtr<class UMirrorTableFindReplaceExpressions> MirrorFindReplaceExpressions;

public:
	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

		/** Returns the name of the factory for menus */
	UNREALED_API virtual FText GetDisplayName() const override;

	/** When shown in menus, this is the category containing this factory. Return type is a BitFlag mask using EAssetTypeCategories. */
	UNREALED_API virtual uint32 GetMenuCategories() const override;
protected:

	UNREALED_API void OnWindowUserActionDelegate(bool bCreate, USkeleton* InSkeleton, UMirrorTableFindReplaceExpressions* InFindReplaceExpressions, const UScriptStruct* InResultStruct);
	UNREALED_API virtual UMirrorDataTable* MakeNewMirrorDataTable(UObject* InParent, FName Name, EObjectFlags Flags);
};

