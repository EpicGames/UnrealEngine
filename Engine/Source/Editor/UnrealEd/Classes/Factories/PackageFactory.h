// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// PackageFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "PackageFactory.generated.h"

UCLASS()
class UPackageFactory : public UFactory
{
	GENERATED_UCLASS_BODY()


	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateText( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn ) override;
	//~ End UFactory Interface
};



