// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ParticleSystemFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "ParticleSystemFactoryNew.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UParticleSystemFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()


	//~ Begin UFactory Interface
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};



