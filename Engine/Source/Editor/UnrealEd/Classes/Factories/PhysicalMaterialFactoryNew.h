// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// PhysicalMaterialFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "PhysicalMaterialFactoryNew.generated.h"

class UPhysicalMaterial;

UCLASS(MinimalAPI, HideCategories=Object)
class UPhysicalMaterialFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = PhysicalMaterialFactory)
	TSubclassOf<UPhysicalMaterial> PhysicalMaterialClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
