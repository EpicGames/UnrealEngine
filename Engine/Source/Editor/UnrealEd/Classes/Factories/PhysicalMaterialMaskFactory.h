// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// PhysicalMaterialFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "PhysicalMaterialMaskFactory.generated.h"

class UPhysicalMaterialMask;

UCLASS(MinimalAPI, HideCategories=Object)
class UPhysicalMaterialMaskFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = PhysicalMaterialMaskFactory)
	TSubclassOf<UPhysicalMaterialMask> PhysicalMaterialMaskClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
};
