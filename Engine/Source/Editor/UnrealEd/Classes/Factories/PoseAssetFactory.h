// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "PoseAssetFactory.generated.h"

struct FAssetData;
class SWindow;
class SMultiLineEditableTextBox;

UCLASS(HideCategories=Object,MinimalAPI)
class UPoseAssetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The following Properties are BlueprintReadWrite to expose to Python / Blutilities

	/* Used when creating a composite from an AnimSequence, becomes the only AnimSequence contained */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "PoseAssetFactory")
	TObjectPtr<class UAnimSequence> SourceAnimation;

	/** Optional. If specified the poses will be named according to this array.
	If you don't specify, or you don't specify enough names for all poses, then default names will be generated.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "PoseAssetFactory")
	TArray<FString> PoseNames;

	// Only used for AnimationEditorUtils::ExecuteNewAnimAsset template. Do not use directly.

	UPROPERTY()
	TObjectPtr<class USkeleton> TargetSkeleton;

	UPROPERTY()
	TObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

private:
	void OnWindowUserActionDelegate(bool bCreate, UAnimSequence* InSequence, const TArray<FString>& InPoseNames);
};

