// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "Factories/Factory.h"
#include "PreviewMeshCollectionFactory.generated.h"

class USkeleton;

UCLASS(hidecategories = Object, MinimalAPI)
class UPreviewMeshCollectionFactory : public UFactory
{
public:
	GENERATED_UCLASS_BODY()

	// UFactory interface
	virtual bool ShouldShowInNewMenu() const override { return true; }
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	// End of UFactory interface

public:
	/** The current skeleton we are creating collections for */
	UPROPERTY()
	TWeakObjectPtr<USkeleton> CurrentSkeleton;
};

