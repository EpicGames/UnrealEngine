// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/CSVImportFactory.h"
#include "EditorReimportHandler.h"
#include "ReimportCurveTableFactory.generated.h"

UCLASS()
class UReimportCurveTableFactory : public UCSVImportFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
};



