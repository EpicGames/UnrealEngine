// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/CSVImportFactory.h"
#include "EditorReimportHandler.h"
#include "ReimportDataTableFactory.generated.h"

UCLASS(MinimalAPI)
class UReimportDataTableFactory : public UCSVImportFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	UNREALED_API virtual bool FactoryCanImport( const FString& Filename ) override;
	UNREALED_API virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	UNREALED_API virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	UNREALED_API virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	UNREALED_API virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface
};



