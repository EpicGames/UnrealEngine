// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportFbxAnimSequenceFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "Factories/FbxFactory.h"
#include "ReimportFbxAnimSequenceFactory.generated.h"

UCLASS(collapsecategories)
class UReimportFbxAnimSequenceFactory : public UFbxFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()


	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	virtual int32 GetPriority() const override;
	virtual void PostImportCleanUp() { CleanUp(); }
	//~ End FReimportHandler Interface

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual bool IsAutomatedImport() const override;
	//~ End UFactory Interface
};



