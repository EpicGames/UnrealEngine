// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportFbxSceneFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetRegistry/AssetData.h"
#include "EditorReimportHandler.h"
#include "Factories/FbxSceneImportFactory.h"
#include "Camera/CameraTypes.h"

#include "ReimportFbxSceneFactory.generated.h"

class UActorComponent;
class UBlueprint;
class USceneComponent;
class USimpleConstructionScript;

UCLASS(BlueprintType)
class UReimportFbxSceneFactory : public UFbxSceneImportFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface

public:
	/* Script helper to allow fbx scene reimport from scripted language
	 * @param Obj: type must be one of the following type
	 * UFbxSceneImportData
	 * UStaticMesh, USkeletalMesh, UAnimSequence: It must have been imported with the scene import so the AssetImportData point on a UFbxSceneImportData
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Fbx Scene Reimport")
	void ScriptReimportHelper(UObject* Obj);

private:
	EReimportResult::Type ReimportStaticMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo);
	EReimportResult::Type ReimportSkeletalMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo);
	EReimportResult::Type ImportStaticMesh(void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr);
	EReimportResult::Type ImportSkeletalMesh(void* VoidRootNodeToImport, void* VoidFbxImporter, TSharedPtr<FFbxMeshInfo> MeshInfo, TSharedPtr<FFbxSceneInfo> SceneInfoPtr);
	UBlueprint *UpdateOriginalBluePrint(FString &BluePrintFullName, void* VoidNodeStatusMapPtr, TSharedPtr<FFbxSceneInfo> SceneInfoPtr, TSharedPtr<FFbxSceneInfo> SceneInfoOriginalPtr, TArray<FAssetData> &AssetDataToDelete);
	
	//Remove all node from a blueprint SimpleConstructionScript
	void RemoveChildNodeRecursively(USimpleConstructionScript* SimpleConstructionScript, class USCS_Node* ScsNode);

	//Structure use to allow what can be re-import when reimporting a blueprint hierarchy
	//For now we reimport the transform, and some special light and camera data. All other property will be
	//serialize from the old blueprint value.
	//For staticmesh and skeletal mesh we trash the node and use the new one if the asset change (the mesh asset is not the same)
	struct FSpecializeComponentData
	{
		//Scene component stuff
		FTransform NodeTransform;

		//Save Light value
		FColor LightColor;
		float LightIntensity;
		bool CastShadow;
		FColor ShadowColor;
		float InnerAngle;
		float OuterAngle;
		float FarAttenuation;

		//Save Camera Value
		ECameraProjectionMode::Type ProjectionMode;
		float AspectRatio;
		float OrthoNearPlane;
		float OrthoFarPlane;
		float OrthoWidth;
	};
	void StoreImportedSpecializeComponentData(USceneComponent *SceneComponent, FSpecializeComponentData &SpecializeComponentData, UClass *CurrentNodeComponentClass);
	void RestoreImportedSpecializeComponentData(USceneComponent *SceneComponent, const FSpecializeComponentData &SpecializeComponentData, UClass *CurrentNodeComponentClass);
	//This function will serialize the old blueprint component value in the new blueprint re-imported component. But it will not override some data, see FSpecializeComponentData comment.
	void RecursivelySetComponentProperties(class USCS_Node* CurrentNode, const TArray<UActorComponent*>& ActorComponents, TArray<FString> ParentNames, bool IsDefaultSceneNode);

	FString FbxImportFileName;

	TArray<UObject*> AssetToSyncContentBrowser;
};



