// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportFbxSkeletalMeshFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "Factories/FbxFactory.h"
#include "ReimportFbxSkeletalMeshFactory.generated.h"

UCLASS(MinimalAPI, collapsecategories)
class UReimportFbxSkeletalMeshFactory : public UFbxFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()


	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override
	{
		return SetReimportPaths(Obj, NewReimportPaths[0], 0);
	}
	virtual void SetReimportPaths(UObject* Obj, const FString& NewReimportPath, const int32 SourceIndex) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override
	{
		return Reimport(Obj, INDEX_NONE);
	}
	virtual EReimportResult::Type Reimport( UObject* Obj, int32 SourceFileIndex );
	virtual int32 GetPriority() const override;
	virtual void PostImportCleanUp() { CleanUp(); }
	//~ End FReimportHandler Interface

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual bool IsAutomatedImport() const override;
	//~ End UFactory Interface
};
