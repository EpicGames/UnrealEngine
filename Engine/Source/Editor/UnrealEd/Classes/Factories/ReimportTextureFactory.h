// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// ReimportTextureFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "Factories/TextureFactory.h"
#include "ReimportTextureFactory.generated.h"

class UTexture2D;
class UTextureCube;

UCLASS(hidecategories=(LightMap, DitherMipMaps, LODGroup), collapsecategories)
class UReimportTextureFactory : public UTextureFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UTexture> pOriginalTex;


	//~ Begin FReimportHandler Interface
	virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

	//~ Begin UFactory Interface
	virtual bool IsAutomatedImport() const override;
	//~ End UFactory Interface

private:
	//~ Begin UTextureFactory Interface
	virtual UTexture2D* CreateTexture2D( UObject* InParent, FName Name, EObjectFlags Flags ) override;
	virtual UTextureCube* CreateTextureCube( UObject* InParent, FName Name, EObjectFlags Flags ) override;
	//~ End UTextureFactory Interface
};



