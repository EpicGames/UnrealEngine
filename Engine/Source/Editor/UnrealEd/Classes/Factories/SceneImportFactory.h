// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factory.h"
#include "SceneImportFactory.generated.h"

/*
 * Base class for all factories that import objects into a scene (e.g in a level)
 */
UCLASS(abstract, MinimalAPI)
class USceneImportFactory : public UFactory
{
	GENERATED_BODY()

public:
	/**
	* @returns whether or not this scene importer can also create assets and thus must ask the user where they want to place content
	*/
	virtual bool ImportsAssets() const
	{
		return false;
	}
};
