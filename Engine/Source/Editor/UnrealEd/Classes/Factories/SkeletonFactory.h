// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SkeletonFactory.generated.h"

struct FAssetData;
class SWindow;

UCLASS(HideCategories=Object,MinimalAPI)
class USkeletonFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// The skeletal mesh with which to initialize this skeleton.
	UPROPERTY()
	TObjectPtr<class USkeletalMesh> TargetSkeletalMesh;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

private:
	void OnTargetSkeletalMeshSelected(const FAssetData& SelectedAsset);

private:
	TSharedPtr<SWindow> PickerWindow;
};

