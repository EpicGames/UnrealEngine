// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Factories/Factory.h"
#include "SlateWidgetStyleAssetFactory.generated.h"

/** Factory for creating SlateStyles */
UCLASS(MinimalAPI, hidecategories=Object)
class USlateWidgetStyleAssetFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=StyleType)
	TSubclassOf< USlateWidgetStyleContainerBase > StyleType;

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};
