// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "EditorReimportHandler.h"
#include "MaterialInstanceConstantFactoryNew.h"

#include "SparseVolumeTextureMaterialFactory.generated.h"

class UMaterialInterface;
class USparseVolumeTexture;

UCLASS(MinimalAPI)
class USparseVolumeTextureMaterialFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<USparseVolumeTexture> InitialTexture;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

UCLASS(config=Engine, MinimalAPI)
class USparseVolumeTextureMaterialInstanceFactoryNew : public UMaterialInstanceConstantFactoryNew
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<USparseVolumeTexture> InitialTexture;

	UPROPERTY(config)
	TSoftObjectPtr<UMaterialInterface> DefaultSVTMaterial;

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

#endif // WITH_EDITOR
