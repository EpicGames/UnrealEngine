// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Factory for SubUVAnimation assets
*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "SubUVAnimationFactory.generated.h"

UCLASS(MinimalAPI)
class USubUVAnimationFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** An initial texture to use */
	UPROPERTY()
	TObjectPtr<class UTexture2D> InitialTexture;

	// UFactory interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const;
	// End of UFactory interface
};
