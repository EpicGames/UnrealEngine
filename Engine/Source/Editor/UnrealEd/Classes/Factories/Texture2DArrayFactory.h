// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Texture2DArrayFactory.generated.h"

class UTexture2D;

/** Factory for creating volume texture */
UCLASS(hidecategories=Object, MinimalAPI)
class UTexture2DArrayFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UTexture2D>> InitialTextures;

	//~ Begin UFactory Interface
	virtual bool CanCreateNew() const override;
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface

	bool CheckArrayTexturesCompatibility();
};



