// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// Texture2DFactoryNew
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Texture2dFactoryNew.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UTexture2DFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** width of new texture */
	UPROPERTY()
	int32		Width;

	/** height of new texture */
	UPROPERTY()
	int32		Height;

	virtual bool ShouldShowInNewMenu() const override;
	virtual UObject* FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn ) override;
};
