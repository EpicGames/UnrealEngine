// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays the texture cube array
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "TextureCubeArrayThumbnailRenderer.generated.h"

UCLASS()
class UTextureCubeArrayThumbnailRenderer : public UTextureThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Interface
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Interface
};
