// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Factories/Factory.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "ImportSettings.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TextureFactory.generated.h"

struct FImportImage
{
	TArray64<uint8> RawData;
	ETextureSourceFormat Format = TSF_Invalid;
	TextureCompressionSettings CompressionSettings = TC_Default;
	int32 NumMips = 0;
	int32 SizeX = 0;
	int32 SizeY = 0;
	bool SRGB = true;
	/** Which compression format (if any) that is applied to RawData */
	ETextureSourceCompressionFormat RawDataCompressionFormat = TSCF_None;

	void Init2DWithParams(int32 InSizeX, int32 InSizeY, ETextureSourceFormat InFormat, bool InSRGB);

	int64 GetMipSize(int32 InMipIndex) const;
	void* GetMipData(int32 InMipIndex);
};


class UTexture2D;
class UTextureCube;
class UTexture2DArray;

UENUM()
enum class ETextureSourceColorSpace
{
	/** Auto lets the texture factory figure out in what color space the source image is in. */
	Auto,
	Linear,
	SRGB
};

UCLASS(customconstructor, collapsecategories, hidecategories=Object, MinimalAPI)
class UTextureFactory : public UFactory, public IImportSettingsParser
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	uint32 NoCompression:1;

	/** If enabled, the texture's alpha channel will be discarded during compression */
	UPROPERTY(EditAnywhere, Category=Compression, meta=(ToolTip="If enabled, the texture's alpha channel will be discarded during compression"))
	uint32 NoAlpha:1;

	/** If enabled, compression is deferred until the texture is saved */
	UPROPERTY(EditAnywhere, Category=Compression, meta=(ToolTip="If enabled, compression is deferred until the texture is saved"))
	uint32 bDeferCompression:1;

	/** Compression settings for the texture */
	UPROPERTY(EditAnywhere, Category=Compression, meta=(ToolTip="Compression settings for the texture"))
	TEnumAsByte<enum TextureCompressionSettings> CompressionSettings;

	/** If enabled, a material will automatically be created for the texture */
	UPROPERTY(EditAnywhere, Category=TextureFactory, meta=(ToolTip="If enabled, a material will automatically be created for the texture"))
	uint32 bCreateMaterial:1;

	/** If enabled, link the texture to the created material's base color */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture to the created material's base color"))
	uint32 bRGBToBaseColor:1;

	/** If enabled, link the texture to the created material's emissive color */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture to the created material's emissive color"))
	uint32 bRGBToEmissive:1;

	/** If enabled, link the texture's alpha to the created material's specular color */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture's alpha to the created material's roughness"))
	uint32 bAlphaToRoughness:1;

	/** If enabled, link the texture's alpha to the created material's emissive color */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture's alpha to the created material's emissive color"))
	uint32 bAlphaToEmissive:1;

	/** If enabled, link the texture's alpha to the created material's opacity */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture's alpha to the created material's opacity"))
	uint32 bAlphaToOpacity:1;

	/** If enabled, link the texture's alpha to the created material's opacity mask */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, link the texture's alpha to the created material's opacity mask"))
	uint32 bAlphaToOpacityMask:1;

	/** If enabled, the created material will be two-sided */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="If enabled, the created material will be two-sided"))
	uint32 bTwoSided:1;

	/** The blend mode of the created material */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="The blend mode of the created material"))
	TEnumAsByte<enum EBlendMode> Blending;

	/** The shading model of the created material */
	UPROPERTY(EditAnywhere, Category=CreateMaterial, meta=(ToolTip="The shading model of the created material"))
	TEnumAsByte<enum EMaterialShadingModel> ShadingModel;

	/** The mip-map generation settings for the texture; Allows customization of the content of the mip-map chain */
	UPROPERTY(EditAnywhere, Category=TextureFactory, meta=(ToolTip="The mip-map generation settings for the texture; Allows customization of the content of the mip-map chain"))
	TEnumAsByte<enum TextureMipGenSettings> MipGenSettings;

	/** The group the texture belongs to */
	UPROPERTY(EditAnywhere, Category=LODGroup, meta=(ToolTip="The group the texture belongs to"))
	TEnumAsByte<enum TextureGroup> LODGroup;

	/** Whether mip RGBA should be scaled to preserve the number of pixels with Value >= AlphaCoverageThresholds */
	UPROPERTY(EditAnywhere, Category=PreserveAlphaCoverage, meta=(ToolTip="Whether mip RGBA should be scaled to preserve the number of pixels with Value >= AlphaCoverageThresholds"))
	bool bDoScaleMipsForAlphaCoverage = false;

	/** Whether to use newer & faster mip generation filter, same quality but produces slightly different results from previous implementation */
	UPROPERTY(EditAnywhere, Category=TextureFactory, meta=(ToolTip="Whether to use newer & faster mip generation filter"))
	bool bUseNewMipFilter = false;

	/** Channel values to compare to when preserving alpha coverage from a mask. */
	UPROPERTY(EditAnywhere, Category=PreserveAlphaCoverage, meta=(ToolTip="Channel values to compare to when preserving alpha coverage from a mask for mips"))
	FVector4 AlphaCoverageThresholds = FVector4(0,0,0,0.75f);

	/** If enabled, preserve the value of border pixels when creating mip-maps */
	UPROPERTY(EditAnywhere, Category=PreserveBorder, meta=(ToolTip="If enabled, preserve the value of border pixels when creating mip-maps"))
	uint32 bPreserveBorder:1;

	/** If enabled, the texture's green channel will be inverted. This is useful for some normal maps */
	UPROPERTY(EditAnywhere, Category=NormalMap, meta=(ToolTip="If enabled, the texture's green channel will be inverted. This is useful for some normal maps"))
	uint32 bFlipNormalMapGreenChannel:1;

	/** If enabled, we are using the existing settings for a texture that already existed. */
	UPROPERTY(Transient)
	uint32 bUsingExistingSettings:1;

	/** If enabled, we are using the texture content hash as the guid. */
	UPROPERTY(Transient)
	uint32 bUseHashAsGuid:1;

	/**
	 * The pattern to use to match UDIM files to indices. Defaults to match a filename that ends with either .1001 or _1001
	 * This 1st and 3rd (optional) capture groups are used as the texture name. The 2nd capture group is considered to be the UDIM index.
	 * ie: (Capture Group 1)(\d{4})( Capture Group 3)
	 */
	UPROPERTY(Transient)
	FString UdimRegexPattern;

	/** Mode for how to determine the color space of the source image. Auto will let the factory decide based on header metadata or bit depth. Linear or SRGB will force the color space on the resulting texture. */
	UPROPERTY(Transient)
	ETextureSourceColorSpace ColorSpaceMode;

	/* Store YesAll/NoAll responses: */
	UPROPERTY(Transient)
	TEnumAsByte<EAppReturnType::Type> HDRImportShouldBeLongLatCubeMap = EAppReturnType::Retry;

public:
	UNREALED_API UTextureFactory(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ Begin UObject Interface
	UNREALED_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UFactory Interface
	UNREALED_API virtual bool DoesSupportClass(UClass* Class) override;
	UNREALED_API virtual UObject* FactoryCreateBinary( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn ) override;
	UNREALED_API virtual bool FactoryCanImport(const FString& Filename) override;
	UNREALED_API virtual IImportSettingsParser* GetImportSettingsParser() override;

	//~ End UFactory Interface
	
	/** IImportSettingsParser interface */
	UNREALED_API virtual void ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson) override;


	/** Create a texture given the appropriate input parameters	*/
	UNREALED_API virtual UTexture2D* CreateTexture2D( UObject* InParent, FName Name, EObjectFlags Flags );
	UNREALED_API virtual UTextureCube* CreateTextureCube( UObject* InParent, FName Name, EObjectFlags Flags );
	UNREALED_API virtual UTexture2DArray* CreateTexture2DArray(UObject* InParent, FName Name, EObjectFlags Flags);
	/**
	 * Suppresses the dialog box that, when importing over an existing texture, asks if the users wishes to overwrite its settings.
	 * This is primarily for reimporting textures.
	 */
	static UNREALED_API void SuppressImportOverwriteDialog(bool bOverwriteExistingSettings = false);

	/**
	 *	Initializes the given texture from the TextureData text block supplied.
	 *	The TextureData text block is assumed to have been generated by the UTextureExporterT3D.
	 *
	 *	@param	InTexture	The texture to initialize
	 *	@param	Text		The texture data text generated by the TextureExporterT3D
	 *	@param	Warn		Where to send warnings/errors
	 *
	 *	@return	bool		true if successful, false if not
	 */
	UNREALED_API bool InitializeFromT3DTextureDataText(UTexture* InTexture, const FString& Text, FFeedbackContext* Warn);
	
	// @todo document
	UNREALED_API bool InitializeFromT3DTexture2DDataText(UTexture2D* InTexture2D, const TCHAR*& Buffer, FFeedbackContext* Warn);

	// @todo document
	UNREALED_API void FindCubeMapFace(const FString& ParsedText, const FString& FaceString, UTextureCube& TextureCube, UTexture2D*& TextureFace);

	// @todo document
	UNREALED_API bool InitializeFromT3DTextureCubeDataText(UTextureCube* InTextureCube, const TCHAR*& Buffer, FFeedbackContext* Warn);

protected:
	/** Keep track of if we are doing a reimport */
	bool bIsDoingAReimport = false;

private:
	/** This variable is static because in StaticImportObject() the type of the factory is not known. */
	static bool bSuppressImportOverwriteDialog;

	/** Force overwriting the existing texture without the dialog box */
	static bool bForceOverwriteExistingSettings;

	/**
	*	Tests if the given height and width specify a supported texture resolution to import; Can optionally check if the height/width are powers of two
	*
	*	@param	Width					The width of an imported texture whose validity should be checked
	*	@param	Height					The height of an imported texture whose validity should be checked
	*	@param	bAllowNonPowerOfTwo		Whether or not non-power-of-two textures are allowed
	*	@param	Warn					Where to send warnings/errors
	*
	*	@return	bool					true if the given height/width represent a supported texture resolution, false if not
	*/
	static bool IsImportResolutionValid(int64 Width, int64 Height, bool bAllowNonPowerOfTwo, FFeedbackContext* Warn);

	/** Flags to be used when calling ImportImage */
	enum class EImageImportFlags
	{
		/** No options selected */
		None						= 0,
		/** Allows textures to be imported with dimensions that are not to the power of two */
		AllowNonPowerOfTwo			= 1 << 0,
		/** Allows the return of texture data in it's original compressed format, if this occurs then FImportImage::RawDataCompressionFormat will contain the returned format. */
		AllowReturnOfCompressedData	= 1 << 1
	};
	FRIEND_ENUM_CLASS_FLAGS(EImageImportFlags);

	/** Import image file into generic image struct, may be easily copied to FTextureSource */
	bool ImportImage(const uint8* Buffer, int64 Length, FFeedbackContext* Warn, EImageImportFlags Flags, FImportImage& OutImage);

	/** used by CreateTexture() */
	UTexture* ImportTexture(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn);
	
	UTexture * ImportDDS(const uint8* Buffer,int64 Length,UObject* InParent,FName Name, EObjectFlags Flags,EImageImportFlags ImportFlags,FFeedbackContext* Warn);

	UTexture* ImportTextureUDIM(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const TCHAR* Type, const TMap<int32, FString>& UDIMIndexToFile, FFeedbackContext* Warn);

	/** Applies import settings directly to the texture after import */
	void ApplyAutoImportSettings(UTexture* Texture);
private:
	/** Texture settings from the automated importer that should be applied to the new texture */
	TSharedPtr<class FJsonObject> AutomatedImportSettings;
};

ENUM_CLASS_FLAGS(UTextureFactory::EImageImportFlags);

UCLASS()
class UUDIMTextureFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	* Make a UDIM virtual texture from a list of regular 2D textures
	* @param OutputPathName			Path name of the UDIM texture (e.g. /Game/MyTexture)
	* @param SourceTextures			List of regular 2D textures to be packed into the atlas
	* @param BlockCoords			Coordinates of the corresponding texture in the atlas
	* @param bKeepExistingSettings	Whether to keep existing settings if a texture with the same path name exists. Otherwise, settings will be copied from the first source texture
	* @param bCheckOutAndSave		Whether to check out and save the UDIM texture
	* @return UTexture2D*			Pointer to the UDIM texture or null if failed
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities", meta = (DisplayName = "Make UDIM Texture from Texture2Ds"))
	static UTexture2D* MakeUDIMVirtualTextureFromTexture2Ds(FString OutputPathName, const TArray<UTexture2D*>& SourceTextures, const TArray<FIntPoint>& BlockCoords, bool bKeepExistingSettings = false, bool bCheckOutAndSave = false);
};
