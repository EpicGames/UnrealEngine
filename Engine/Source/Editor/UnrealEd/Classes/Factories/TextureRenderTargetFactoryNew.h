// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureRenderTargetFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "TextureRenderTargetFactoryNew.generated.h"

UCLASS(MinimalAPI, hidecategories=(Object, Texture))
class UTextureRenderTargetFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** Width of the texture render target */
	UPROPERTY(meta=(ToolTip="Width of the texture render target"))
	int32 Width;

	/** Height of the texture render target */
	UPROPERTY(meta=(ToolTip="Height of the texture render target"))
	int32 Height;

	/** Pixel format of the texture render target */
	UPROPERTY(meta=(ToolTip="Pixel format of the texture render target"))
	uint8 Format;


	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};



