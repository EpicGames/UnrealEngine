// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TextureRenderTargetCubeFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "TextureRenderTargetVolumeFactoryNew.generated.h"

UCLASS(MinimalAPI, hidecategories=(Object, Texture))
class UTextureRenderTargetVolumeFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** width of new texture */
	UPROPERTY(meta=(ToolTip="Width of the texture render target"))
	int32 Width;

	/** height of new texture */
	UPROPERTY(meta=(ToolTip = "Height of the texture render target"))
	int32 Height;

	/** depth of new texture */
	UPROPERTY(meta=(ToolTip = "Depth of the texture render target"))
	int32 Depth;

	/** surface format of new texture */
	UPROPERTY(meta=(ToolTip="Pixel format of the texture render target"))
	uint8 Format;


	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
