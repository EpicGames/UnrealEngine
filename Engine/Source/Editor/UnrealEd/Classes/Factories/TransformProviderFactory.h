// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "TransformProviderFactory.generated.h"

class UTransformProviderData;

UCLASS(MinimalAPI)
class UTransformProviderDataFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** The type of transform provider data that will be created */
	UPROPERTY(EditAnywhere, Category=CurveFactory)
	TSubclassOf<UTransformProviderData> ProviderDataClass;

	virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
};