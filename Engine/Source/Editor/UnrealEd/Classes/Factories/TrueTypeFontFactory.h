// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TrueTypeFontFactory
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "Factories/TextureFactory.h"
#include "TrueTypeFontFactory.generated.h"

class UFont;
class UTexture2D;

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UTrueTypeFontFactory : public UTextureFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	/** Import options for this font */
	UPROPERTY(EditAnywhere, Instanced, Category = TrueTypeFontFactory, meta = (ToolTip = "Import options for the font"))
	TObjectPtr<class UFontImportOptions> ImportOptions;

	/** True when the font dialog was shown for this factory during the non-legacy creation process */
	UPROPERTY()
	bool bPropertiesConfigured;

	/** True if a font was selected during the non-legacy creation process */
	UPROPERTY()
	bool bFontSelected;

	//~ Begin UObject Interface
	UNREALED_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UFactory Interface
	UNREALED_API virtual bool ConfigureProperties() override;
	UNREALED_API virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override
	{
		// Don't show this factory in the content browser menu; it's invoked manually when changing the UFont cache type to "Offline" 
		return false;
	}
	//~ Begin UFactory Interface	

	//~ Begin FReimportHandler Interface
	UNREALED_API virtual bool CanReimport( UObject* Obj, TArray<FString>& OutFilenames ) override;
	UNREALED_API virtual void SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths ) override;
	UNREALED_API virtual EReimportResult::Type Reimport( UObject* Obj ) override;
	UNREALED_API virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

	/** Creates the import options structure for this font */
	UNREALED_API void SetupFontImportOptions();

#if PLATFORM_WINDOWS
	/**
	 * Win32 Platform Only: Creates a single font texture using the Windows GDI
	 *
	 * @param Font (In/Out) The font we're operating with
	 * @param dc The device context configured to paint this font
	 * @param RowHeight Height of a font row in pixels
	 * @param TextureNum The current texture index
	 *
	 * @return Returns the newly created texture, if successful, otherwise NULL
	 */
	UNREALED_API UTexture2D* CreateTextureFromDC( UFont* Font, Windows::HDC dc, int32 BitmapWidth, int32 BitmapHeight, int32 TextureNum );
#endif

#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX
	UNREALED_API void* LoadFontFace( void* FTLibrary, int32 Height, FFeedbackContext* Warn, void** OutFontData );
	UNREALED_API UTexture2D* CreateTextureFromBitmap( UFont* Font, uint8* BitmapData, int32 BitmapWidth, int32 BitmapHeight, int32 TextureNum );
	UNREALED_API bool CreateFontTexture( UFont* Font, FFeedbackContext* Warn, const int32 NumResolutions, const int32 CharsPerPage,
		const TMap<TCHAR,TCHAR>& InverseMap, const TArray< float >& ResHeights );

#if PLATFORM_WINDOWS
	UNREALED_API FString FindBitmapFontFile();
#endif

	/**
	 * Windows/Mac Platform Only: Imports a TrueType font
	 *
	 * @param Font (In/Out) The font object that we're importing into
	 * @param Warn Feedback context for displaying warnings and errors
	 * @param NumResolutions Number of resolution pages we should generate 
	 * @param ResHeights Font height for each resolution (one array entry per resolution)
	 *
	 * @return true if successful
	 */
	UNREALED_API bool ImportTrueTypeFont( UFont* Font, FFeedbackContext* Warn, const int32 NumResolutions, const TArray< float >& ResHeights );
#endif
};



