// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// UIMaterialFactoryNew
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "UIMaterialFactoryNew.generated.h"

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UUIMaterialFactoryNew : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ End UFactory Interface	
};
