// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorFieldStaticFactory: Factory for importing a 3D grid of vectors.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "VectorFieldStaticFactory.generated.h"

UCLASS()
class UVectorFieldStaticFactory : public UFactory
{
	GENERATED_UCLASS_BODY()


	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateBinary( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn ) override;
	virtual bool FactoryCanImport( const FString& Filename ) override;
	//~ End UFactory Interface

};



