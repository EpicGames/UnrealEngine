// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "VolumeTextureFactory.generated.h"

/** Factory for creating volume texture */
UCLASS(hidecategories=Object, MinimalAPI)
class UVolumeTextureFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** An initial texture to assign to the newly created volume texture */
	UPROPERTY()
	TObjectPtr<class UTexture2D> InitialTexture;

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface
};



