// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 *
 */

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Factories/Factory.h"
#include "WorldFactory.generated.h"

namespace ERHIFeatureLevel { enum Type : int; }

UCLASS(MinimalAPI)
class UWorldFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	TEnumAsByte<EWorldType::Type> WorldType;
	bool bInformEngineOfWorld;
	bool bCreateWorldPartition;
	bool bEnableWorldPartitionStreaming;
	ERHIFeatureLevel::Type FeatureLevel;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
	virtual FText GetToolTip() const override;
	virtual FString GetToolTipDocumentationPage() const override;
	virtual FString GetToolTipDocumentationExcerpt() const override;
	//~ Begin UFactory Interface	
};

