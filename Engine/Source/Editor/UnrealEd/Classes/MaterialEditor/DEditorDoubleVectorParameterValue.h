// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "DEditorDoubleVectorParameterValue.generated.h"

UCLASS(hidecategories = Object, collapsecategories, editinlinenew, MinimalAPI)
class UDEditorDoubleVectorParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = DEditorVectorParameterValue)
	FVector4d ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Double Vector Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::DoubleVector)
		{
			ParameterValue = Value.AsVector4d();
			return true;
		}
		return false;
	}
};

