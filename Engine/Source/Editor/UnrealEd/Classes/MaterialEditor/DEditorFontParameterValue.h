// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorFontParameterValue.generated.h"

USTRUCT()
struct FDFontParameters
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = DFontParameter)
	TObjectPtr<class UFont> FontValue = nullptr;

	UPROPERTY(EditAnywhere, Category = DFontParameter)
	int32 FontPage = 0;
};

UCLASS(hidecategories = Object, collapsecategories, MinimalAPI)
class UDEditorFontParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = DEditorFontParameterValue)
	struct FDFontParameters ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Font Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = FMaterialParameterValue(ParameterValue.FontValue, ParameterValue.FontPage);
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::Font)
		{
			ParameterValue.FontValue = Value.Font.Value;
			ParameterValue.FontPage = Value.Font.Page;
			return true;
		}
		return false;
	}
};

