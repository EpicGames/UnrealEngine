// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "Materials/MaterialLayersFunctions.h"
#include "DEditorMaterialLayersParameterValue.generated.h"

// FMaterialLayersFunctions are no longer treated as material parameters, so this should maybe be refactored at some point
UCLASS(hidecategories = Object, collapsecategories, MinimalAPI)
class UDEditorMaterialLayersParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorMaterialLayersParameterValue)
	struct FMaterialLayersFunctions ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Material Layers Parameter Values"); }
};
