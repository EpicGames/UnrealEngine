// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialEditor/DEditorParameterValue.h"
#include "DEditorParameterCollectionParameterValue.generated.h"

class UMaterialParameterCollection;

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorParameterCollectionParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorTextureCollectionParameterValue)
	TObjectPtr<UMaterialParameterCollection> ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Parameter Collection Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::ParameterCollection)
		{
			ParameterValue = Value.ParameterCollection;
			return true;
		}
		return false;
	}
};
