// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialExpressionParameter.h"

#include "DEditorParameterValue.generated.h"

UCLASS(hidecategories=Object, collapsecategories, editinlinenew, MinimalAPI)
class UDEditorParameterValue : public UObject
{
	GENERATED_UCLASS_BODY()

	static UNREALED_API UDEditorParameterValue* Create(UObject* Owner,
		EMaterialParameterType Type,
		const FMaterialParameterInfo& ParameterInfo,
		const FMaterialParameterMetadata& Meta);

	UPROPERTY(EditAnywhere, Category=DEditorParameterValue)
	uint32 bOverride:1;

	UPROPERTY(EditAnywhere, Category=DEditorParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY()
	FGuid ExpressionId;

	UPROPERTY()
	FString Description;

#if WITH_EDITORONLY_DATA
	/** Controls where this parameter is displayed in a material instance parameter list.  The lower the number the higher up in the parameter list. */
	UPROPERTY()
	int32 SortPriority = 32;
#endif

	virtual FName GetDefaultGroupName() const { return TEXT("None"); }
	virtual bool SetValue(const FMaterialParameterValue& Value) { return false; }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const
	{
		OutResult.Description = Description;
		OutResult.ExpressionGuid = ExpressionId;
		OutResult.SortPriority = SortPriority;
		OutResult.bOverride = bOverride;
		return false;
	}

	bool IsStaticParameter() const
	{
		EMaterialParameterType Type = GetParameterType();
		return Type == EMaterialParameterType::StaticSwitch || Type == EMaterialParameterType::StaticComponentMask;
	}

	EMaterialParameterType GetParameterType() const
	{
		FMaterialParameterMetadata Result;
		if (GetValue(Result))
		{
			return Result.Value.Type;
		}
		return EMaterialParameterType::None;
	}
};

