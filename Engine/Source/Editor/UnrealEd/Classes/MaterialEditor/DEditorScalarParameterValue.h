// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorScalarParameterValue.generated.h"

USTRUCT()
struct FScalarParameterAtlasData
{
	GENERATED_BODY()
public:
	UPROPERTY(Transient)
	bool bIsUsedAsAtlasPosition=false;

	UPROPERTY(Transient)
	TSoftObjectPtr<class UCurveLinearColor> Curve;

	UPROPERTY(Transient)
	TSoftObjectPtr<class UCurveLinearColorAtlas> Atlas;
};

UCLASS(hidecategories=Object, collapsecategories, editinlinenew, MinimalAPI)
class UDEditorScalarParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, Category=DEditorScalarParameterValue)
	float ParameterValue;

	UPROPERTY(Transient)
	FScalarParameterAtlasData AtlasData;

	UPROPERTY(Transient)
	TSoftObjectPtr<UObject> Enumeration;

	float SliderMin = 0.0f;
	float SliderMax = 0.0f;
	int32 EnumerationIndex = INDEX_NONE;
	bool bUseCustomPrimitiveData = false;

	virtual FName GetDefaultGroupName() const override { return TEXT("Scalar Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		OutResult.ScalarMin = SliderMin;
		OutResult.ScalarMax = SliderMax;
		OutResult.ScalarCurve = AtlasData.Curve;
		OutResult.ScalarAtlas = AtlasData.Atlas;
		OutResult.bUsedAsAtlasPosition = AtlasData.bIsUsedAsAtlasPosition;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::Scalar)
		{
			ParameterValue = Value.AsScalar();
			return true;
		}
		return false;
	}

	FString GetAtlasCurvePath() const { return AtlasData.Curve.ToString(); }
};

