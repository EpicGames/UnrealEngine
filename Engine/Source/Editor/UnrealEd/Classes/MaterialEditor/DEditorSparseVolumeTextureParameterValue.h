// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorSparseVolumeTextureParameterValue.generated.h"

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorSparseVolumeTextureParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorTextureParameterValue)
	TObjectPtr<class USparseVolumeTexture> ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Texture Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::SparseVolumeTexture)
		{
			ParameterValue = Value.SparseVolumeTexture;
			return true;
		}
		return false;
	}
};
