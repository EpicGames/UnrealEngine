// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorStaticComponentMaskParameterValue.generated.h"

USTRUCT()
struct FDComponentMaskParameter
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=DComponentMaskParameter)
	uint32 R:1;

	UPROPERTY(EditAnywhere, Category=DComponentMaskParameter)
	uint32 G:1;

	UPROPERTY(EditAnywhere, Category=DComponentMaskParameter)
	uint32 B:1;

	UPROPERTY(EditAnywhere, Category=DComponentMaskParameter)
	uint32 A:1;



		/** Constructor */
		FDComponentMaskParameter(bool InR, bool InG, bool InB, bool InA) :
			R(InR),
			G(InG),
			B(InB),
			A(InA)
		{
		};
		FDComponentMaskParameter() :
			R(false),
			G(false),
			B(false),
			A(false)
		{
		};
	
};

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorStaticComponentMaskParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorStaticComponentMaskParameterValue)
	struct FDComponentMaskParameter ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Static Component Mask Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = FMaterialParameterValue(ParameterValue.R, ParameterValue.G, ParameterValue.B, ParameterValue.A);
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::StaticComponentMask)
		{
			ParameterValue.R = Value.Bool[0];
			ParameterValue.G = Value.Bool[1];
			ParameterValue.B = Value.Bool[2];
			ParameterValue.A = Value.Bool[3];
			return true;
		}
		return false;
	}
};

