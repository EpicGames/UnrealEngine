// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorStaticSwitchParameterValue.generated.h"

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorStaticSwitchParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorStaticSwitchParameterValue)
	uint32 ParameterValue:1;

	virtual FName GetDefaultGroupName() const override { return TEXT("Static Switch Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = (bool)ParameterValue;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::StaticSwitch)
		{
			ParameterValue = Value.AsStaticSwitch();
			return true;
		}
		return false;
	}
};

