// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialEditor/DEditorParameterValue.h"
#include "DEditorTextureCollectionParameterValue.generated.h"

class UTextureCollection;

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorTextureCollectionParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorTextureCollectionParameterValue)
	TObjectPtr<UTextureCollection> ParameterValue;

	virtual FName GetDefaultGroupName() const override { return TEXT("Texture Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::TextureCollection)
		{
			ParameterValue = Value.TextureCollection;
			return true;
		}
		return false;
	}
};

