// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"

#include "DEditorTextureParameterValue.generated.h"

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UDEditorTextureParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorTextureParameterValue)
	TObjectPtr<class UTexture> ParameterValue;

	UPROPERTY(Transient)
	FParameterChannelNames ChannelNames;

	virtual FName GetDefaultGroupName() const override { return TEXT("Texture Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		OutResult.ChannelNames = ChannelNames;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::Texture)
		{
			ParameterValue = Value.Texture;
			return true;
		}
		return false;
	}
};

