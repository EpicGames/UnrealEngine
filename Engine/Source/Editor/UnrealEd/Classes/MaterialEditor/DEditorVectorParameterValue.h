// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "DEditorVectorParameterValue.generated.h"

UCLASS(hidecategories=Object, collapsecategories, editinlinenew, MinimalAPI)
class UDEditorVectorParameterValue : public UDEditorParameterValue
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=DEditorVectorParameterValue)
	FLinearColor ParameterValue;

	UPROPERTY(Transient)
	bool bIsUsedAsChannelMask;

	UPROPERTY(Transient)
	bool bUseCustomPrimitiveData;

	UPROPERTY(Transient)
	FParameterChannelNames ChannelNames;

	virtual FName GetDefaultGroupName() const override { return TEXT("Vector Parameter Values"); }

	virtual bool GetValue(FMaterialParameterMetadata& OutResult) const override
	{
		UDEditorParameterValue::GetValue(OutResult);
		OutResult.Value = ParameterValue;
		OutResult.bUsedAsChannelMask = bIsUsedAsChannelMask;
		OutResult.ChannelNames = ChannelNames;
		return true;
	}

	virtual bool SetValue(const FMaterialParameterValue& Value) override
	{
		if (Value.Type == EMaterialParameterType::Vector)
		{
			ParameterValue = Value.AsLinearColor();
			return true;
		}
		return false;
	}
};

