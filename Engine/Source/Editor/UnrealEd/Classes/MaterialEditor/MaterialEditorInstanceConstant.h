// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialEditorInstanceConstant.h: This class is used by the material instance editor to hold a set of inherited parameters which are then pushed to a material instance.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "StaticParameterSet.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/BlendableInterface.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "Materials/MaterialExpression.h"
#include "MaterialEditorInstanceConstant.generated.h"

class UDEditorParameterValue;
class UMaterial;
class UMaterialInstanceConstant;
struct FPropertyChangedEvent;

USTRUCT()
struct FEditorParameterGroup
{
	GENERATED_USTRUCT_BODY()

	FEditorParameterGroup()
		: GroupAssociation(GlobalParameter)
		, GroupSortPriority(0)
	{}

	UPROPERTY()
	FName GroupName;

	UPROPERTY()
	TEnumAsByte<EMaterialParameterAssociation> GroupAssociation= EMaterialParameterAssociation::LayerParameter;

	UPROPERTY(EditAnywhere, editfixedsize, Instanced, Category=EditorParameterGroup)
	TArray<TObjectPtr<class UDEditorParameterValue>> Parameters;

	UPROPERTY()
	int32 GroupSortPriority=0;
};

USTRUCT()
struct FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorParameterValue)
	uint32 bOverride:1;

	UPROPERTY(EditAnywhere, Category=EditorParameterValue)
	FMaterialParameterInfo ParameterInfo;

	UPROPERTY()
	FGuid ExpressionId;

	FEditorParameterValue()
		: bOverride(false)
	{
	}
};

USTRUCT()
struct FEditorVectorParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorVectorParameterValue)
	FLinearColor ParameterValue;

	FEditorVectorParameterValue()
		: ParameterValue(ForceInit)
	{
	}
};

USTRUCT()
struct FEditorScalarParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorScalarParameterValue)
	float ParameterValue;

	FEditorScalarParameterValue()
		: ParameterValue(0)
	{
	}
};

USTRUCT()
struct FEditorTextureParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorTextureParameterValue)
	TObjectPtr<class UTexture> ParameterValue;

	FEditorTextureParameterValue()
		: ParameterValue(NULL)
	{
	}
};

USTRUCT()
struct FEditorFontParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorFontParameterValue)
	TObjectPtr<class UFont> FontValue;

	UPROPERTY(EditAnywhere, Category=EditorFontParameterValue)
	int32 FontPage;

	FEditorFontParameterValue()
		: FontValue(NULL)
		, FontPage(0)
	{
	}
};

USTRUCT()
struct FEditorMaterialLayersParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorLayersParameterValue)
	TObjectPtr<class UMaterialFunctionInterface> FunctionValue;

	FEditorMaterialLayersParameterValue()
		: FunctionValue(NULL)
	{
	}
};

USTRUCT()
struct FEditorStaticSwitchParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorStaticSwitchParameterValue)
	uint32 ParameterValue:1;

	FEditorStaticSwitchParameterValue()
		: ParameterValue(false)
	{
	}

	/** Constructor */
	FEditorStaticSwitchParameterValue(const FStaticSwitchParameter& InParameter)
		: ParameterValue(InParameter.Value)
	{
		//initialize base class members
		bOverride = InParameter.bOverride;
		ParameterInfo = InParameter.ParameterInfo;
		ExpressionId = InParameter.ExpressionGUID;
	}
};

USTRUCT()
struct FComponentMaskParameter
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=ComponentMaskParameter)
	uint32 R:1;

	UPROPERTY(EditAnywhere, Category=ComponentMaskParameter)
	uint32 G:1;

	UPROPERTY(EditAnywhere, Category=ComponentMaskParameter)
	uint32 B:1;

	UPROPERTY(EditAnywhere, Category=ComponentMaskParameter)
	uint32 A:1;

	FComponentMaskParameter()
		: R(false)
		, G(false)
		, B(false)
		, A(false)
	{
	}

	/** Constructor */
	FComponentMaskParameter(bool InR, bool InG, bool InB, bool InA) :
		R(InR),
		G(InG),
		B(InB),
		A(InA)
	{
	}
};

USTRUCT()
struct FEditorStaticComponentMaskParameterValue : public FEditorParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=EditorStaticComponentMaskParameterValue)
	struct FComponentMaskParameter ParameterValue;

	/** Constructor */
	FEditorStaticComponentMaskParameterValue() {}
	FEditorStaticComponentMaskParameterValue(const FStaticComponentMaskParameter& InParameter)
		: ParameterValue(InParameter.R, InParameter.G, InParameter.B, InParameter.A)
	{
		//initialize base class members
		bOverride = InParameter.bOverride;
		ParameterInfo = InParameter.ParameterInfo;
		ExpressionId = InParameter.ExpressionGUID;
	}
};

USTRUCT()
struct FEditorUserSceneTextureOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Key;

	UPROPERTY(EditAnywhere, Category = EditorParameterValue)
	FName Value;
};

USTRUCT()
struct FMaterialEditorPostProcessOverrides
{
	GENERATED_USTRUCT_BODY()

	// Tracks if this is a material where post process overrides can be applied (MaterialDomain == MD_PostProcess, BlendableLocation != BL_ReplacingTonemapper)
	UPROPERTY()
	bool bIsOverrideable = false;

	UPROPERTY(EditAnywhere, Category = PostProcessOverrideValue)
	bool bOverrideBlendableLocation = false;

	UPROPERTY(EditAnywhere, Category = PostProcessOverrideValue)
	bool bOverrideBlendablePriority = false;

	UPROPERTY(EditAnywhere, Category = PostProcessOverrideValue, meta = (DisplayName = "Blendable Location"), meta = (InvalidEnumValues = "BL_ReplacingTonemapper"))
	TEnumAsByte<EBlendableLocation> BlendableLocationOverride { BL_SceneColorAfterTonemapping };

	UPROPERTY(EditAnywhere, Category = PostProcessOverrideValue, meta = (DisplayName = "Blendable Priority"))
	int32 BlendablePriorityOverride = 0;

	/** Overrides for user scene texture inputs */
	UPROPERTY(EditAnywhere, editfixedsize, Category = PostProcessOverrideValue)
	TArray<FEditorUserSceneTextureOverride> UserSceneTextureInputs;

	/** Override for user scene texture output */
	UPROPERTY(EditAnywhere, Category = PostProcessOverrideValue)
	FName UserSceneTextureOutput;
};

/** Common Interface for material parameter containers */
UCLASS( abstract )
class UMaterialEditorParameters: public UObject
{
	GENERATED_BODY()
public:
	/** 
	 * Get the source/preview material interface for the parameters
	 * @return source/preview material interface
	 */
	virtual TObjectPtr<UMaterialInterface> GetMaterialInterface() { return nullptr;};
	virtual TObjectPtr<UMaterialInterface> GetParentMaterialInterface() { return nullptr;};

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<class UMaterialInstanceConstant>> StoredLayerPreviews;

	UPROPERTY()
	TArray<TObjectPtr<class UMaterialInstanceConstant>> StoredBlendPreviews;

#endif
	
	/** 
	 * Regenerates the parameter arrays. 
	 */
	virtual	void RegenerateArrays() {}

#if WITH_EDITOR
	/** Sets back to zero the overrides for any parameters copied out of the layer stack */
	virtual void CleanParameterStack(int32 Index, EMaterialParameterAssociation MaterialType) { }

	/** Copies the overrides for any parameters copied out of the layer stack from the layer or blend */
	virtual void ResetOverrides(int32 Index, EMaterialParameterAssociation MaterialType) {}
	
	/** Copies the parameter array values back to the source instance. */
	virtual void CopyToSourceInstance(const bool bForceStaticPermutationUpdate = false) {}
	
#endif
};

UCLASS(hidecategories=Object, collapsecategories, MinimalAPI)
class UMaterialEditorInstanceConstant : public UMaterialEditorParameters
{
private:
	GENERATED_UCLASS_BODY()
	/** Physical material to use for this graphics material. Used for sounds, effects etc.*/
	UPROPERTY(EditAnywhere, Category=MaterialEditorInstanceConstant)
	TObjectPtr<class UPhysicalMaterial> PhysMaterial;

	// since the Parent may point across levels and the property editor needs to import this text, it must be marked lazy so it doesn't set itself to NULL in FindImportedObject
	UPROPERTY(EditAnywhere, Category=MaterialEditorInstanceConstant, meta=(DisplayThumbnail="true"))
	TObjectPtr<class UMaterialInterface> Parent;

	UPROPERTY(EditAnywhere, editfixedsize, Category=MaterialEditorInstanceConstant)
	TArray<struct FEditorParameterGroup> ParameterGroups;

	/** This is the refraction depth bias, larger values offset distortion to prevent closer objects from rendering into the distorted surface at acute viewing angles but increases the disconnect between surface and where the refraction starts. */
	UPROPERTY(EditAnywhere, Category=MaterialEditorInstanceConstant)
	float RefractionDepthBias;

	/** SubsurfaceProfile, for Screen Space Subsurface Scattering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Subsurface Profile"))
	TObjectPtr<class USubsurfaceProfile> SubsurfaceProfile;

	/** Defines if SubsurfaceProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant)
	uint32 bOverrideSubsurfaceProfile : 1;

	/** Specular profile */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Specular Profile"))
	TObjectPtr<class USpecularProfile> SpecularProfile;

	/** Defines if SpecularProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant)
	uint32 bOverrideSpecularProfile : 1;

	/** Toon profile */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Material, meta = (DisplayName = "Toon Profile"))
	TObjectPtr<class UToonProfile> ToonProfile;

	/** Defines if ToonProfile from this instance is used or it uses the parent one. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant)
	uint32 bOverrideToonProfile : 1;

	UPROPERTY()
	uint32 bOverrideBaseProperties_DEPRECATED : 1;

	UPROPERTY(transient, duplicatetransient)
	uint32 bIsFunctionPreviewMaterial : 1;

	UPROPERTY(transient, duplicatetransient)
	uint32 bIsFunctionInstanceDirty : 1;
	
	/** Defines if PhysMaterial from this instance is used or if the parent one is used. */
	UPROPERTY(EditAnywhere, Category=MaterialEditorInstanceConstant)
	uint32 bOverridePhysMaterial : 1;
	
	UPROPERTY(EditAnywhere, Category=MaterialOverrides)
	FMaterialInstanceBasePropertyOverrides BasePropertyOverrides;

	UPROPERTY()
	TObjectPtr<class UMaterialInstanceConstant> SourceInstance;

	UPROPERTY()
	TObjectPtr<class UMaterialFunctionInstance> SourceFunction;	

	UPROPERTY()
	TObjectPtr<class UMaterialInstanceConstant> PreviewInstance;

	UPROPERTY(transient, duplicatetransient)
	TArray<FMaterialParameterInfo> VisibleExpressions;

	/** The Lightmass override settings for this object. */
	UPROPERTY(EditAnywhere, Category=Lightmass)
	struct FLightmassParameterizedMaterialSettings LightmassSettings;

	/** Should we use old style typed arrays for unassigned parameters instead of a None group (new style)? */
	UPROPERTY(EditAnywhere, Category=MaterialEditorInstanceConstant)
	uint32 bUseOldStyleMICEditorGroups:1;

	/** When set we will use the override from NaniteOverrideMaterial. Otherwise we inherit any override on the parent. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant, meta = (InlineEditConditionToggle))
	uint32 bNaniteOverride : 1;

	/** An override material which will be used instead of this one when rendering with nanite. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant, meta = (editcondition = "bNaniteOverride"))
	TObjectPtr<UMaterialInterface> NaniteOverrideMaterial;

	/** Array of enumeration objects for use by scalar parameter enumeration indices. */
	UPROPERTY(EditAnywhere, Category = MaterialEditorInstanceConstant, AdvancedDisplay, meta = (DisplayThumbnail = "false", AllowedClasses = "/Script/CoreUObject.Enum, /Script/Engine.MaterialEnumerationProvider"))
	TArray<TSoftObjectPtr<UObject>> EnumerationObjects;

#if WITH_EDITOR
	/** Get enumeration object by index. If nothing is found at the index we also look at parent chain until we find something. Returns a null pointer if nothing is found. */
	UNREALED_API TSoftObjectPtr<UObject> GetEnumerationObject(int32 Index) const;
#endif

	/** Overrides specific to Post Process domain materials. */
	UPROPERTY(EditAnywhere, Category = PostProcessOverrides)
	FMaterialEditorPostProcessOverrides PostProcessOverrides;

	//~ Begin UObject Interface.
	UNREALED_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#if WITH_EDITOR
	UNREALED_API virtual void PostEditUndo() override;
#endif
	//~ End UObject Interface.

	/** Regenerates the parameter arrays. */
	UNREALED_API void RegenerateArrays() override;

#if WITH_EDITOR
	/** Sets back to zero the overrides for any parameters copied out of the layer stack */
	UNREALED_API void CleanParameterStack(int32 Index, EMaterialParameterAssociation MaterialType) override;

	/** Copies the overrides for any parameters copied out of the layer stack from the layer or blend */
	UNREALED_API void ResetOverrides(int32 Index, EMaterialParameterAssociation MaterialType) override;

	/** Arrays and clears parameters no longer valid (e.g. curve atlases). It has the potential effect of regenerating parameter arrays. */
	UNREALED_API void ClearInvalidParameterOverrides();
#endif

	/** Copies the parameter array values back to the source instance. */
	UNREALED_API void CopyToSourceInstance(const bool bForceStaticPermutationUpdate = false) override;

	UNREALED_API void ApplySourceFunctionChanges();

	/** 
	 * Sets the source instance for this object and regenerates arrays. 
	 *
	 * @param MaterialInterface		Instance to use as the source for this material editor instance.
	 */
	UNREALED_API void SetSourceInstance(UMaterialInstanceConstant* MaterialInterface);

	UNREALED_API void CopyBasePropertiesFromParent();

	UNREALED_API void SetSourceFunction(UMaterialFunctionInstance* MaterialFunction);

	/** 
	 * Update the source instance parent to match this
	 */
	UNREALED_API void UpdateSourceInstanceParent();

	/** 
	 *  Returns group for parameter. Creates one if needed. 
	 *
	 * @param ParameterGroup		Name to be looked for.
	 */
	UNREALED_API FEditorParameterGroup & GetParameterGroup(FName& ParameterGroup);
	/** 
	 *  Creates/adds value to group retrieved from parent material . 
	 *
	 * @param ParameterValue		Current data to be grouped
	 * @param GroupName				Name of the group
	 */
	UNREALED_API void AssignParameterToGroup(UDEditorParameterValue* ParameterValue, const FName& GroupName);

	TObjectPtr<UMaterialInterface> GetMaterialInterface() override;
	TObjectPtr<UMaterialInterface> GetParentMaterialInterface() override;

	static UNREALED_API FName GlobalGroupPrefix;

	TWeakPtr<class IDetailsView> DetailsView;


	/** Whether or not we should show only overridden properties*/
	bool bShowOnlyOverrides;
};



