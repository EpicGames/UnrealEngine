// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "MaterialEditorMeshComponent.generated.h"

UCLASS(MinimalAPI)
class UMaterialEditorMeshComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

	// USceneComponent Interface
	UNREALED_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
};

