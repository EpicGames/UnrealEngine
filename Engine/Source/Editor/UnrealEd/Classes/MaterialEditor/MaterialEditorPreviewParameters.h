// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialEditorInstanceConstant.h: This class is used by the material instance editor to hold a set of inherited parameters which are then pushed to a material instance.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "StaticParameterSet.h"
#include "Editor/UnrealEdTypes.h"
#include "Materials/MaterialInstanceBasePropertyOverrides.h"
#include "MaterialEditorInstanceConstant.h"
#include "MaterialEditorPreviewParameters.generated.h"

class UDEditorParameterValue;
class UMaterial;
class UMaterialInterface;
class UMaterialInstanceConstant;
struct FPropertyChangedEvent;

UCLASS(hidecategories = Object, collapsecategories, MinimalAPI)
class UMaterialEditorPreviewParameters : public UMaterialEditorParameters
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, editfixedsize, Category = UMaterialEditorPreviewParameters)
	TArray<struct FEditorParameterGroup> ParameterGroups;

	UPROPERTY()
	TObjectPtr<class UMaterial> PreviewMaterial;

	UPROPERTY()
	TObjectPtr<class UMaterialFunction> OriginalFunction;

	UPROPERTY()
	TObjectPtr<class UMaterial> OriginalMaterial;


	//~ Begin UObject Interface.
	UNREALED_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#if WITH_EDITOR
	UNREALED_API virtual void PostEditUndo() override;
#endif
	//~ End UObject Interface.

	/** Regenerates the parameter arrays. */
	UNREALED_API void RegenerateArrays() override;
	
	TObjectPtr<UMaterialInterface> GetMaterialInterface() override;

protected:
	/** Copies the parameter array values back to the source instance. */
	UNREALED_API void CopyToSourceInstance(const bool bForceStaticPermutationUpdate = false) override;

	UNREALED_API void ApplySourceFunctionChanges();

	/**
	*  Creates/adds value to group retrieved from parent material .
	*
	* @param ParameterValue		Current data to be grouped
	* @param GroupName			Name of the group
	*/
	UNREALED_API void AssignParameterToGroup(UDEditorParameterValue* ParameterValue, const FName& GroupName);

	static UNREALED_API FName GlobalGroupPrefix;
};
