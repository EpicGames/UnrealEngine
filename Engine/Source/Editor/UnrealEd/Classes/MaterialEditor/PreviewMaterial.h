// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/Material.h"

#include "PreviewMaterial.generated.h"

UCLASS(MinimalAPI)
class UPreviewMaterial : public UMaterial
{
	GENERATED_UCLASS_BODY()

	//~ Begin UMaterial Interface.
	UNREALED_API virtual FMaterialResource* AllocateResource() override;
	virtual bool IsAsset()  const override  { return false; }
	virtual uint32 GetUsageFlags() const override;
	virtual bool GetUsageByFlag(EMaterialUsage Usage) const override;
	//~ End UMaterial Interface.
};

