// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/UniquePtr.h"
#include "EdGraph/EdGraph.h"
#include "Materials/Material.h"
#include "RenderUtils.h"
#include "MaterialGraph.generated.h"

class UMaterialExpressionComment;
class UMaterialExpressionComposite;

DECLARE_DELEGATE_RetVal( bool, FRealtimeStateGetter );
DECLARE_DELEGATE( FSetMaterialDirty );
DECLARE_DELEGATE_OneParam( FToggleExpressionCollapsed, UMaterialExpression* );

/**
 * A human-readable name - material expression input pair.
 */
struct FMaterialInputInfo
{
	/** Constructor */
	FMaterialInputInfo()
	{
	}

	/** Constructor */
	FMaterialInputInfo(const FText& InName, EMaterialProperty InProperty, const FText& InToolTip)
		:	Name( InName )
		,	Property( InProperty )
		,	ToolTip( InToolTip )
	{
	}

	FExpressionInput& GetExpressionInput(UMaterial* Material) const
	{
		check(Material);

		auto Ret = Material->GetExpressionInputForProperty(Property);

		// if this happens UMaterialGraph::RebuildGraph() registered something it shouldn't
		check(Ret);

		return *Ret;
	}

	bool IsVisiblePin(const UMaterial* Material, bool bIgnoreMaterialAttributes = false) const
	{
		if (!Material->IsPropertySupported(Property) && !bIgnoreMaterialAttributes)
		{
			return false;
		}

		const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
		const bool bSubstrateHiddenMaterialConversion = Substrate::IsHiddenMaterialAssetConversionEnabled();
		const bool bFrontMaterialConnected = Material->HasSubstrateFrontMaterialConnected();
		const bool bShowAsSubstrate = bSubstrateEnabled && (!bSubstrateHiddenMaterialConversion || bFrontMaterialConnected);

		if (!bSubstrateEnabled && Property == MP_FrontMaterial)
		{
			return false;
		}

		if (Material->bUseMaterialAttributes && !bIgnoreMaterialAttributes)
		{
			// On the root node both MaterialAttributes and FrontMaterial are visible when Substrate is enabled. 
			// Otherwise only MaterialAttributes is visible.
			return Property == MP_MaterialAttributes || (Property == MP_FrontMaterial && Substrate::IsSubstrateEnabled());
		}
		else if( Material->IsUIMaterial() )
		{
			if (bSubstrateEnabled && bSubstrateHiddenMaterialConversion && Property == MP_FrontMaterial)
			{
				return true; // If hidden conversion, always enable FrontMaterial in case we want to plug a Substrate material in there.
			}
			if (bShowAsSubstrate)
			{
				if (Property == MP_FrontMaterial || Property == MP_Opacity || Property == MP_OpacityMask || Property == MP_WorldPositionOffset)
				{
					return true;
				}
			}
			else
			{
				if (Property == MP_EmissiveColor || Property == MP_Opacity || Property == MP_OpacityMask || Property == MP_WorldPositionOffset)
				{
					return true;
				}
			}

			if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
			{
				return (Property - MP_CustomizedUVs0) < Material->NumCustomizedUVs;
			}

			return false;

		}
		else
		{
			if (Property == MP_MaterialAttributes)
			{
				return false;
			}

			if (Property >= MP_CustomizedUVs0 && Property <= MP_CustomizedUVs7)
			{
				return (Property - MP_CustomizedUVs0) < Material->NumCustomizedUVs;
			}

			if (bShowAsSubstrate)
			{
				// The only properties visible when Substrate is enable and FrontMaterial pin is plugged.
				return Property == MP_Opacity
					|| Property == MP_OpacityMask
					|| Property == MP_WorldPositionOffset
					|| Property == MP_AmbientOcclusion
					|| Property == MP_Refraction
					|| Property == MP_PixelDepthOffset
					|| Property == MP_FrontMaterial
					|| Property == MP_SurfaceThickness
					|| Property == MP_Displacement;
			}


			return true;
		}
	}

	const FText& GetName() const { return Name; }

	EMaterialProperty GetProperty() const { return Property; }

	const FText& GetToolTip() const { return ToolTip; }

private:
	/** Name of the input shown to user */
	FText Name;
	/** Type of the input */
	EMaterialProperty Property;
	/** The tool-tip describing this input's purpose */
	FText ToolTip;

};

UCLASS(Optional, MinimalAPI)
class UMaterialGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

	/** Material this Graph represents */
	UPROPERTY()
	TObjectPtr<class UMaterial>				Material;

	/** Material Function this Graph represents (NULL for Materials) */
	UPROPERTY()
	TObjectPtr<class UMaterialFunction>		MaterialFunction;

	/** Root node representing Material inputs (NULL for Material Functions) */
	UPROPERTY()
	TObjectPtr<class UMaterialGraphNode_Root>	RootNode;

	/** Expression this subgraph represents (NULL if not subgraph, Material [Function] still populated) */
	UPROPERTY()
	TObjectPtr<UMaterialExpression>			SubgraphExpression;

	/** List of Material Inputs (not set up for Material Functions) */
	TArray<FMaterialInputInfo> MaterialInputs;

	/** Checks if Material Editor is in realtime mode, so we update SGraphNodes every frame */
	FRealtimeStateGetter RealtimeDelegate;

	/** Marks the Material Editor as dirty so that user prompted to apply change */
	FSetMaterialDirty MaterialDirtyDelegate;

	/** Toggles the bCollapsed flag of a material expression and updates material editor */
	FToggleExpressionCollapsed ToggleCollapsedDelegate;

	/** The name of the material that we are editing */
	UPROPERTY()
	FString	OriginalMaterialFullName;

	//~ Begin UEdGraph interface
	UNREALED_API virtual void NotifyGraphChanged() override;
protected:
	UNREALED_API virtual void NotifyGraphChanged(const FEdGraphEditAction& Action) override;
	//~ End UEdGraph interface

public:
	UNREALED_API UMaterialGraph();
	UNREALED_API UMaterialGraph(FVTableHelper& Helper);
	UNREALED_API virtual ~UMaterialGraph();

	/**
	 * Completely rebuild the graph from the material, removing all old nodes
	 */
	UNREALED_API void RebuildGraph();

	/**
	 * Add an Expression to the Graph
	 *
	 * @param	Expression	Expression to add
	 *
	 * @return	UMaterialGraphNode*	Newly created Graph node to represent expression
	 */
	UNREALED_API class UMaterialGraphNode*			AddExpression(UMaterialExpression* Expression, bool bUserInvoked);

	/**
	 * Add a Comment to the Graph
	 *
	 * @param	Comment			Comment to add
	 * @param	bIsUserInvoked	Whether the comment has just been created by the user via the UI
	 *
	 * @return	UMaterialGraphNode_Comment*	Newly created Graph node to represent comment
	 */
	UNREALED_API class UMaterialGraphNode_Comment*	AddComment(UMaterialExpressionComment* Comment, bool bIsUserInvoked = false);

	/**
	 * Add a Subgraph to the Graph
	 *
	 * @param	InSubgraphExpression, expression that will represent the new subgraph in this graph.
	 *
	 * @return	UMaterialGraph* Newly created subgraph
	 */
	UNREALED_API UMaterialGraph* AddSubGraph(UMaterialExpression* InSubgraphExpression);

	/** Link all of the Graph nodes using the Material's connections */
	UNREALED_API void LinkGraphNodesFromMaterial();

	/** Link the Material using the Graph node's connections */
	UNREALED_API void LinkMaterialExpressionsFromGraph();

	/**
	 * Check whether a material input should be marked as active
	 *
	 * @param	GraphPin	Pin representing the material input
	 */
	UNREALED_API bool IsInputActive(class UEdGraphPin* GraphPin) const;

	/** Returns the input index associated with the given property */
	UNREALED_API int32 GetInputIndexForProperty(EMaterialProperty Property) const;

	/**
	 * Get a list of nodes representing expressions that are not used in the Material
	 *
	 * @param	UnusedNodes	Array to contain nodes representing unused expressions
	 */
	UNREALED_API void GetUnusedExpressions(TArray<class UEdGraphNode*>& UnusedNodes) const;

	/**
	 * Get a list of expressions not reachable from any material output. Does not require an
	 * open editor and can be called headlessly via UMaterialEditingLibrary.
	 *
	 * @param	Material	The Material to query.
	 * @param	OutUnused	Array to contain unused expressions.
	 */
	UNREALED_API static void GetUnusedMaterialExpressions(UMaterial* Material, TArray<UMaterialExpression*>& OutUnused);

	UNREALED_API void UpdatePinTypes();

	inline UObject* GetMaterialOrFunction() const { return MaterialFunction ? (UObject*)MaterialFunction : Material; }

private:
	/**
	 * Remove all Nodes from the graph
	 */
	void RemoveAllNodes();

	void RebuildGraphInternal(const TMap<UMaterialExpression*, TArray<UMaterialExpression*>>& SubgraphExpressionMap, const TMap<UMaterialExpression*, TArray<UMaterialExpressionComment*>>& SubgraphCommentMap);

	/**
	 * Gets a valid output index, matching mask values if necessary
	 *
	 * @param	Input	Input we are finding an output index for
	 */
	int32 GetValidOutputIndex(FExpressionInput* Input) const;
};
