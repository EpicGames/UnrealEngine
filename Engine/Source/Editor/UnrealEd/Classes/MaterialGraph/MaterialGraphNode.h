// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraphNode.generated.h"

class IMaterialEditor;
class UEdGraphPin;

UCLASS(MinimalAPI)
class UMaterialGraphNode : public UMaterialGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	/** Material Expression this node is representing */
	UPROPERTY()
	TObjectPtr<class UMaterialExpression> MaterialExpression;

	/** Set to true when Expression Preview compiles, so we can update SGraphNode */
	bool bPreviewNeedsUpdate;

	/** Set to true if this expression causes an error in the material */
	bool bIsErrorExpression;

	/** Set to true if this expression is currently being previewed */
	bool bIsPreviewExpression;

	/** Checks if Material Editor is in realtime mode, so we update SGraphNode every frame */
	FRealtimeStateGetter RealtimeDelegate;

	/** Marks the Material Editor as dirty so that user prompted to apply change */
	FSetMaterialDirty MaterialDirtyDelegate;

	/** Called when the preview material attached to this graph node needs to be updated */
	FSimpleDelegate InvalidatePreviewMaterialDelegate;

public:
	/** Fix up the node's owner after being copied */
	UNREALED_API virtual void PostCopyNode();

	/** Get the expression preview for this node */
	UNREALED_API FMaterialRenderProxy* GetExpressionPreview();

	/** Recreate this node's pins and relink the Graph from the Material */
	UNREALED_API void RecreateAndLinkNode();

	/** Get the Material value type of an output pin */
	//virtual uint32 GetOutputType(const UEdGraphPin* OutputPin) const override;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void PrepareForCopying() override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void PostPlacedNewNode() override;
	virtual void NodeConnectionListChanged() override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void OnUpdateCommentText( const FString& NewComment ) override;
	virtual void OnCommentBubbleToggled( bool bInCommentBubbleVisible ) override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual bool CanUserDeleteNode() const override;
	virtual EMaterialValueType GetPinMaterialValueType(const UEdGraphPin* Pin) const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual bool CanJumpToDefinition() const override;
	virtual void JumpToDefinition() const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UMaterialGraphNode_Base Interface
	virtual UObject* GetMaterialNodeOwner() const override { return MaterialExpression; }
	virtual void CreateInputPins() override;
	virtual void CreateOutputPins() override;
	//~ End UMaterialGraphNode_Base Interface

	/** Will return the shorten pin name to use based on long pin name */
	static FName UNREALED_API GetShortenPinName(const FName PinName);

	virtual void PropagatePropertyChange() override;

	UNREALED_API void ToggleNodePreview();

private:
	/** Make sure the MaterialExpression is owned by the Material */
	void ResetMaterialExpressionOwner();

	/** Get the parameter name from the Material Expression */
	FString GetParameterName() const;

	/** Sets the Material Expression's parameter name */
	void SetParameterName(const FString& NewName);

	/** Should expression use the bool pin colour for its title */
	static bool UsesBoolColour(UMaterialExpression* Expression);

	/** Should expression use the float pin colour for its title */
	static bool UsesFloatColour(UMaterialExpression* Expression);

	/** Should expression use the vector pin colour for its title */
	static bool UsesVectorColour(UMaterialExpression* Expression);

	/** Should expression use the object pin colour for its title */
	static bool UsesObjectColour(UMaterialExpression* Expression);

	/** Should expression use the event node colour for its title */
	static bool UsesEventColour(UMaterialExpression* Expression);
};
