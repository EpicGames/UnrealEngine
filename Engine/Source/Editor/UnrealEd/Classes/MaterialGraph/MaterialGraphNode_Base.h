// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "MaterialValueType.h"
#include "MaterialGraphNode_Base.generated.h"

class UEdGraphPin;
class UEdGraphSchema;

UCLASS(MinimalAPI, Optional)
class UMaterialGraphNode_Base : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** Create all of the input pins required */
	virtual void CreateInputPins() {};
	/** Create all of the output pins required */
	virtual void CreateOutputPins() {};
	/** Is this the undeletable root node */
	virtual bool IsRootNode() const {return false;}
	/** Gets the object that owns this node, typically either a UMaterial, UMaterialFunction, UMaterialExpression */
	virtual UObject* GetMaterialNodeOwner() const { return nullptr; }
	/** Returns the SourceIndex associated with a particular FMaterialConnectionKey::InputIndex */
	virtual int32 GetSourceIndexForInputIndex(int32 InputIndex) const;
	/** Get a single Input Pin via its index */
	UNREALED_API class UEdGraphPin* GetInputPin(int32 InputIndex) const;
	/** Get a single Output Pin via its index */
	UNREALED_API class UEdGraphPin* GetOutputPin(int32 OutputIndex) const;
	/** Gets the exec input pin */
	UNREALED_API class UEdGraphPin* GetExecInputPin() const;
	/** Get a single exec Output Pin via its index */
	UNREALED_API class UEdGraphPin* GetExecOutputPin(int32 OutputIndex) const;
	/** Replace a given node with this one, changing all pin links */
	UNREALED_API void ReplaceNode(UMaterialGraphNode_Base* OldNode);

	/** Get the Material value type of an input pin */
	EMaterialValueType GetInputValueType(const UEdGraphPin* InputPin) const;

	/** Get the Material value type of an output pin */
	UNREALED_API EMaterialValueType GetOutputValueType(const UEdGraphPin* OutputPin) const;
	
	uint32 GetOutputType(const UEdGraphPin* OutputPin)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return static_cast<uint32>(GetOutputValueType(OutputPin));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Handles inserting the node between the FromPin and what the FromPin was original connected to
	 *
	 * @param FromPin			The pin this node is being spawned from
	 * @param NewLinkPin		The new pin the FromPin will connect to
	 * @param OutNodeList		Any nodes that are modified will get added to this list for notification purposes
	 */
	void InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList);

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual void ReconstructNode() override;
	virtual void RemovePinAt(const int32 PinIndex, const EEdGraphPinDirection PinDirection) override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual FString GetDocumentationLink() const override;
	virtual void PostPasteNode() override;
	//~ End UEdGraphNode Interface.

protected:
	void ModifyAndCopyPersistentPinData(UEdGraphPin& TargetPin, const UEdGraphPin& SourcePin) const;

	virtual EMaterialValueType GetPinMaterialValueType(const UEdGraphPin* Pin) const;

	void EmptyPins();

	/** Return the first input pin matching the name */
	class UEdGraphPin* GetInputPin(const FName& PinName) const;

	/** Used when changing one property may affect the preview of subsequent properties */
	virtual void PropagatePropertyChange() {}
};
