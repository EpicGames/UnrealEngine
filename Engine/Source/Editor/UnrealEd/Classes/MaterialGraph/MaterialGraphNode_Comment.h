// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialGraph/MaterialGraph.h"
#include "EdGraphNode_Comment.h"

#include "MaterialGraphNode_Comment.generated.h"

UCLASS(MinimalAPI, Optional)
class UMaterialGraphNode_Comment : public UEdGraphNode_Comment
{
	GENERATED_UCLASS_BODY()

	/** Material Comment that this node represents */
	UPROPERTY()
	TObjectPtr<class UMaterialExpressionComment> MaterialExpressionComment;

	/** Marks the Material Editor as dirty so that user prompted to apply change */
	FSetMaterialDirty MaterialDirtyDelegate;

	/** Fix up the node's owner after being copied */
	UNREALED_API void PostCopyNode();

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void PrepareForCopying() override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual void PostPlacedNewNode() override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void ResizeNode(const FVector2f& NewSize) override;
	virtual int32 GetFontSize() const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface.

private:
	/** Make sure the MaterialExpressionComment is owned by the Material */
	void ResetMaterialExpressionOwner();
};
