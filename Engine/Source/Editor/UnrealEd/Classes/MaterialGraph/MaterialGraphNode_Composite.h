// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"

#include "MaterialGraphNode_Composite.generated.h"

UCLASS(MinimalAPI)
class UMaterialGraphNode_Composite : public UMaterialGraphNode
{
	GENERATED_UCLASS_BODY()

	/** The graph that this composite node is representing */
	UPROPERTY()
	TObjectPtr<UMaterialGraph> BoundGraph;

	//~ Begin UObject Interface
	virtual void PostEditUndo() override; 
	//~ End UObject Interface

	//~ Begin UMaterialGraphNode Interface
	UNREALED_API virtual void PostCopyNode() override;
	//~ End UMaterialGraphNode Interface

	//~ Begin UEdGraphNode Interface.
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void DestroyNode() override; 
	virtual void PrepareForCopying() override;
	virtual void PostPasteNode() override; 
	virtual void OnRenameNode(const FString& NewName) override;
	virtual void ReconstructNode() override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override { return TArray<UEdGraph*>( { BoundGraph } ); }
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface.

private:

	/** Fixes up the input and output pin bases when needed, useful after PostEditUndo which changes which graph these nodes point to */
	void FixupInputAndOutputPinBases(); 
};
