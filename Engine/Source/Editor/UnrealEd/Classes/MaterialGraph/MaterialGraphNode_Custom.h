// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Styling/SlateTypes.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SBoxPanel.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraph/MaterialGraphNode.h"

#include "MaterialGraphNode_Custom.generated.h"

class UEdGraphPin;

UCLASS(MinimalAPI)
class UMaterialGraphNode_Custom : public UMaterialGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface.

	UNREALED_API FText GetHlslText() const;
	UNREALED_API void OnCustomHlslTextCommitted(const FText& InText, ETextCommit::Type InType);
	UNREALED_API void OnCodeViewChanged(const ECheckBoxState NewCheckedState);
	UNREALED_API ECheckBoxState IsCodeViewChecked() const;

private:
	void ChangeProperty(FString PropertyName, bool bUpdatePreview = true);
};
