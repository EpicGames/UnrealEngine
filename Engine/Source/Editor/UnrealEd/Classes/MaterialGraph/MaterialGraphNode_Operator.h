// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"

#include "MaterialGraphNode_Operator.generated.h"

class UEdGraphPin;

UCLASS(MinimalAPI)
class UMaterialGraphNode_Operator : public UMaterialGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode Interface.
};