// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialGraph/MaterialGraph.h"
#include "MaterialGraph/MaterialGraphNode.h"

#include "MaterialGraphNode_PinBase.generated.h"

UCLASS(MinimalAPI)
class UMaterialGraphNode_PinBase : public UMaterialGraphNode
{
	GENERATED_BODY()

	//~ Begin UMaterialGraphNode Interface
	UNREALED_API virtual void PostCopyNode() override;
	//~ End UMaterialGraphNode Interface

	//~ Begin UEdGraphNode Interface.
	virtual void PrepareForCopying() override;
	virtual bool CanDuplicateNode() const override { return false; }
	//~ End UEdGraphNode Interface.
};
