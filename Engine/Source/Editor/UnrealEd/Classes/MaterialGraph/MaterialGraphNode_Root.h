// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialGraph/MaterialGraphNode_Base.h"
#include "MaterialGraphNode_Root.generated.h"

class UEdGraphPin;

UCLASS(MinimalAPI)
class UMaterialGraphNode_Root : public UMaterialGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	/** Material whose inputs this root node represents */
	UPROPERTY()
	TObjectPtr<class UMaterial> Material;

	void UpdateInputUseConstant(UEdGraphPin* Pin, bool bUseConstant);

	//~ Begin UEdGraphNode Interface.
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool CanDuplicateNode() const override { return false; }
	virtual void PostPlacedNewNode() override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface.

	//~ Begin UMaterialGraphNode_Base Interface
	virtual UObject* GetMaterialNodeOwner() const override;
	virtual int32 GetSourceIndexForInputIndex(int32 InputIndex) const override;
	virtual EMaterialValueType GetPinMaterialValueType(const UEdGraphPin* Pin) const override;
	virtual void CreateInputPins() override;
	virtual bool IsRootNode() const override {return true;}
	//~ End UMaterialGraphNode_Base Interface
};
