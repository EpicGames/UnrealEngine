// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"
#include "MaterialValueType.h"
#include "MaterialGraphSchema.generated.h"

class UEdGraph;
class UMaterialExpression;
struct FToolMenuContext;

namespace UE::Shader
{
	enum class EValueType : uint8;
}

/** Action to add an expression node to the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Class of expression we want to create */
	UPROPERTY()
	TObjectPtr<class UClass> MaterialExpressionClass;

	/** Delegate to be run immediately after expression of class 'MaterialExpressionClass' is created */
	TDelegate<void(UMaterialExpression*)> PostCreationDelegate;

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FMaterialGraphSchemaAction_NewNode"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FMaterialGraphSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, MaterialExpressionClass(nullptr)
	{}

	FMaterialGraphSchemaAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
		, MaterialExpressionClass(nullptr)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	/**
	 * Sets the type of a Function input based on an EMaterialValueType value.
	 *
	 * @param	FunctionInput		The function input to set.
	 * @param	MaterialValueType	Value type we want input to accept.
	 */
	UE_DEPRECATED(5.8, "Use SetFunctionInputType(..., EMaterialValueType) overload instead")
	UNREALED_API void SetFunctionInputType(class UMaterialExpressionFunctionInput* FunctionInput, uint32 MaterialValueType) const;
	UNREALED_API void SetFunctionInputType(class UMaterialExpressionFunctionInput* FunctionInput, EMaterialValueType MaterialValueType) const;
};

/** Action to add a Material Function call to the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_NewFunctionCall : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Path to function that we want to call */
	UPROPERTY()
	FString FunctionPath;

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FMaterialGraphSchemaAction_NewFunctionCall"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FMaterialGraphSchemaAction_NewFunctionCall() 
		: FEdGraphSchemaAction()
	{}

	FMaterialGraphSchemaAction_NewFunctionCall(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a composite node to the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_NewComposite : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FMaterialGraphSchemaAction_NewComposite"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FMaterialGraphSchemaAction_NewComposite()
		: FEdGraphSchemaAction()
	{}

	FMaterialGraphSchemaAction_NewComposite(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	static UNREALED_API UEdGraphNode* SpawnNode(class UEdGraph* ParentGraph, const UE::Slate::FDeprecateVector2DParameter& Location);
};

/** Action to add a comment node to the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_NewComment : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FMaterialGraphSchemaAction_NewComment"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FMaterialGraphSchemaAction_NewComment() 
		: FEdGraphSchemaAction()
	{}

	FMaterialGraphSchemaAction_NewComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to add a local variable usage to the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_NewNamedRerouteUsage : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Declaration that we want to add an usage of
	UPROPERTY()
	TObjectPtr<class UMaterialExpressionNamedRerouteDeclaration> Declaration = nullptr;

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FMaterialGraphSchemaAction_NewNamedRerouteUsage"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FMaterialGraphSchemaAction_NewNamedRerouteUsage() 
		: FEdGraphSchemaAction()
	{}

	FMaterialGraphSchemaAction_NewNamedRerouteUsage(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

/** Action to paste clipboard contents into the graph */
USTRUCT()
struct FMaterialGraphSchemaAction_Paste : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FMaterialGraphSchemaAction_Paste"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FMaterialGraphSchemaAction_Paste() 
		: FEdGraphSchemaAction()
	{}

	FMaterialGraphSchemaAction_Paste(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UNREALED_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};

UCLASS(MinimalAPI)
class UMaterialGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	// Allowable PinType.PinCategory values
	UNREALED_API static const FName PC_Mask;
	UNREALED_API static const FName PC_Required;
	UNREALED_API static const FName PC_Optional;
	UNREALED_API static const FName PC_MaterialInput;
	UNREALED_API static const FName PC_Exec;

	// Common PinType.PinSubCategory values
	UNREALED_API static const FName PSC_Red;
	UNREALED_API static const FName PSC_Green;
	UNREALED_API static const FName PSC_Blue;
	UNREALED_API static const FName PSC_Alpha;
	UNREALED_API static const FName PSC_RGBA;
	UNREALED_API static const FName PSC_RGB;
	UNREALED_API static const FName PSC_RG;
	UNREALED_API static const FName PSC_Int;
	UNREALED_API static const FName PSC_Byte;
	UNREALED_API static const FName PSC_Bool;
	UNREALED_API static const FName PSC_Float;
	UNREALED_API static const FName PSC_Vector4;

	UNREALED_API static const FName PN_Execute; // Category=PC_Exec, singleton, input

	// Color of certain pins/connections
	UNREALED_API static const FLinearColor ActivePinColor;
	UNREALED_API static const FLinearColor InactivePinColor;
	UNREALED_API static const FLinearColor AlphaPinColor;

	/** Gathers Named Reroute Declaration actions from the given graph into the action list */
	UNREALED_API static void GetNamedRerouteActionsForGraph(FGraphActionListBuilderBase& OutAllActions, const UEdGraph* Graph);

	/**
	 * Connect a pin to one of the Material Function's outputs
	 *
	 * @param	InGraphPin	Pin we are connecting
	 * @param	InFuncPin	Desired output pin
	 */
	void OnConnectToFunctionOutput(UEdGraphPin* InGraphPin, UEdGraphPin* InFuncPin);

	/**
	 * Connect a pin to one of the Material's inputs
	 *
	 * @param	InGraphPin	Pin we are connecting
	 * @param	ConnIndex	Index of the Material input to connect to
	 */
	void OnConnectToMaterial(UEdGraphPin* InGraphPin, int32 ConnIndex);

	UE_DEPRECATED(5.8, "Should call version that provides a UMaterial or UMaterialFunction")
	UNREALED_API void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName, bool bMaterialFunction) const;
	UNREALED_API void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName, const UObject* MaterialOrFunction) const;

	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	/** Check whether the types of pins are compatible */
	bool ArePinsCompatible_Internal(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin, FText& ResponseMessage) const;

	/** update material when the default value of a material node's pin has changed */
	UNREALED_API void UpdateMaterialOnDefaultValueChanged(const UEdGraph* Graph) const;

	/** Mark the material as dirty (because of a change that shouldn't trigger recompile or preview update) */ 
	UNREALED_API void MarkMaterialDirty(const UEdGraph* Graph) const;

	/** Update the detail view */
	UNREALED_API void UpdateDetailView(const UEdGraph* Graph) const;

	/** Gets the type of this pin (must be part of a UMaterialGraphNode_Base) */
	UE_DEPRECATED(5.8, "Use GetMaterialIOValueType instead")
	UNREALED_API static uint32 GetMaterialValueType(const UEdGraphPin* MaterialPin);
	UNREALED_API static EMaterialValueType GetMaterialIOValueType(const UEdGraphPin* MaterialPin);

	//~ Begin UEdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual FLinearColor GetPinColor(const UEdGraphPin* InPin) const override;

	/** Get the color for a connection based on its resolved shader value type (from material insights) */
	UNREALED_API static FLinearColor GetColorForConnectionType(UE::Shader::EValueType ConnectionType);
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual bool CanEncapuslateNode(UEdGraphNode const& TestNode) const override;
	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	virtual int32 GetCurrentVisualizationCacheID() const override;
	virtual void ForceVisualizationCacheClear() const override;
	virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* NodeToDelete) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const;
	virtual void GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;
#if WITH_EDITORONLY_DATA
	virtual float GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup weight calculation was always performed on the first action, provide that if scoring a FGraphActionListBuilderBase::ActionGroup, or just provide the FEdGraphSchemaAction")
	virtual float GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FGraphSchemaSearchWeightModifiers GetSearchWeightModifiers() const override;
#endif // WITH_EDITORONLY_DATA	
	//~ End UEdGraphSchema Interface

private:
	/** Adds actions for all Material Functions */
	void GetMaterialFunctionActions(FGraphActionMenuBuilder& ActionMenuBuilder) const;
	/** Adds action for creating a composite */
	void GetCompositeAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph = NULL) const;
	/** Adds action for creating a comment */
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph = NULL) const;
	/** Adds actions for local variables */
	void GetNamedRerouteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const;
	/**
	 * Checks whether a Material Function has any connections that are compatible with a type/direction
	 *
	 * @param	FunctionAssetData	Asset Data for function to test against (may need to be fully loaded).
	 * @param	TestType			Material Value Type we are testing.
	 * @param	TestDirection		Pin Direction we are testing.
	*/
	bool HasCompatibleConnection(const FAssetData& FunctionAssetData, EMaterialValueType TestType, EEdGraphPinDirection TestDirection) const;

	/** Will promote selected pin to a parameter of the pin type */
	void OnPromoteToParameter(const FToolMenuContext& InMenuContext);

	/** Used to know if we can promote selected pin to a parameter of the pin type */
	bool OnCanPromoteToParameter(const FToolMenuContext& InMenuContext);

	/** Will  return the UClass to create from the Pin Type */
	UClass* GetOnPromoteToParameterClass(const UEdGraphPin* TargetPin) const;

private:
	// ID for checking dirty status of node titles against, increases whenever 
	static int32 CurrentCacheRefreshID;
};
