// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Shader/ShaderTypes.h"
#include "MaterialShaderValueTypeObject.generated.h"

/**
 * Lightweight flyweight UObject that wraps a UE::Shader::EValueType.
 * One singleton instance per EValueType value, stored in PinType.PinSubCategoryObject
 * to carry shader value type information without modifying PinCategory or PinSubCategory.
 * Used by the new material translator to drive pin and wire coloring.
 *
 * Owned by UMaterialShaderValueTypeSubsystem, whose lifetime is tied to GEditor.
 */
UCLASS(MinimalAPI, Transient)
class UMaterialShaderValueTypeObject : public UObject
{
	GENERATED_BODY()

public:
	UE::Shader::EValueType ValueType = UE::Shader::EValueType::Void;

	/** Get or create the flyweight singleton for a given EValueType. Returns nullptr for EValueType::Void or if GEditor is not available. */
	UNREALED_API static UMaterialShaderValueTypeObject* Get(UE::Shader::EValueType InType);
};
