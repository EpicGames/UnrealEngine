// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * A configuration class used by the UAnimationBlueprint Editor to save editor
 * settings across sessions.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AnimationBlueprintEditorOptions.generated.h"

UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UAnimationBlueprintEditorOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** If true, fade nodes which are not connected to the selected nodes */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bHideUnrelatedNodes:1;

};
