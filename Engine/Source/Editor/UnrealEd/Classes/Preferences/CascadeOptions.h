// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CascadeOptions
//
// A configuration class that holds information for the setup of Cascade.
// Supplied so that the editor 'remembers' the last setup the user had.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CascadeOptions.generated.h"

UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UCascadeOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowModuleDump:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor BackgroundColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bUseSubMenus:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bUseSpaceBarReset:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bUseSpaceBarResetInLevel:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor Empty_Background;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor Emitter_Background;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor Emitter_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor Emitter_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_General_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_General_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_TypeData_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_TypeData_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Beam_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Beam_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Trail_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Trail_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Spawn_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Spawn_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Light_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Light_Selected;

	UPROPERTY(EditAnywhere, config, Category = Options)
	FColor ModuleColor_SubUV_Unselected;

	UPROPERTY(EditAnywhere, config, Category = Options)
	FColor ModuleColor_SubUV_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Required_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Required_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Event_Unselected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor ModuleColor_Event_Selected;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowGrid:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor GridColor_Hi;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FColor GridColor_Low;

	UPROPERTY(EditAnywhere, config, Category=Options)
	float GridPerspectiveSize;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowParticleCounts:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowParticleEvents:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowParticleTimes:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowParticleDistance:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowParticleMemory:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	float ParticleMemoryUpdateTime;

	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bShowFloor:1;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FString FloorMesh;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FVector FloorPosition;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FRotator FloorRotation;

	UPROPERTY(EditAnywhere, config, Category=Options)
	float FloorScale;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FVector FloorScale3D;

	UPROPERTY(EditAnywhere, config, Category=Options)
	int32 ShowPPFlags;

	/** If true, use the 'slimline' module drawing method in cascade. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bUseSlimCascadeDraw:1;

	/** The height to use for the 'slimline' module drawing method in cascade. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	int32 SlimCascadeDrawHeight;

	/** If true, center the module name and buttons in the module box. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bCenterCascadeModuleText:1;

	/** The number of units the mouse must move before considering the module as dragged. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	int32 Cascade_MouseMoveThreshold;

	/** The radius of the motion mode */
	UPROPERTY(EditAnywhere, config, Category=Options)
	float MotionModeRadius;

};

