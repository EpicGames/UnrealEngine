// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// CurveEdOptions
//
// A configuration class that holds information for the setup of the CurveEd.
// Supplied so that the editor 'remembers' the last setup the user had.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "CurveEdOptions.generated.h"

UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UCurveEdOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, config, Category=Options)
	float MinViewRange;

	UPROPERTY(EditAnywhere, config, Category=Options)
	float MaxViewRange;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor BackgroundColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor LabelColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor SelectedLabelColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor GridColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor GridTextColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor LabelBlockBkgColor;

	UPROPERTY(EditAnywhere, config, Category=Options)
	FLinearColor SelectedKeyColor;

};

