// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *	Lightmass Options Object
 *	Property window wrapper for various Lightmass settings
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"

#include "LightmassOptionsObject.generated.h"

UCLASS(hidecategories=Object, editinlinenew)
class ULightmassOptionsObject : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Debug)
	struct FLightmassDebugOptions DebugSettings;

	UPROPERTY(EditAnywhere, Category=Swarm)
	struct FSwarmDebugOptions SwarmSettings;

};

