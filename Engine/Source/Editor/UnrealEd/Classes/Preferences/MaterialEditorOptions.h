// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "MaterialEditorOptions.generated.h"

/**
 * A configuration class used by the UMaterial Editor to save editor
 * settings across sessions.
 */
UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UMaterialEditorOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** If true, don't render connectors that are not connected to anything. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bHideUnusedConnectorsSetting:1;

	/** If true, the 3D material preview viewport updates in realtime. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bRealtimeMaterialViewport:1;

	/** If true, the linked object viewport updates in realtime. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bRealtimeExpressionViewport:1;

	/** If true, always refresh the material preview. */
	UPROPERTY(EditAnywhere, config, Category = Options)
	uint32 bLivePreviewUpdate : 1;
	
	/** If true, fade nodes which are not connected to the selected nodes */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bHideUnrelatedNodes:1;

	/** If true, always refresh all expression previews. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bAlwaysRefreshAllPreviews:1;

	/** If false, use expression categorized menus. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	uint32 bUseUnsortedMenus:1;

	/** The users favorite material expressions. */
	UPROPERTY(EditAnywhere, config, Category=Options)
	TArray<FString> FavoriteExpressions;
	
	/** Preview mesh used in material editor viewport */
	UPROPERTY(EditAnywhere, config, Category=Options, meta=(AllowedClasses="/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh", ExactClass="true"))
	FSoftObjectPath PreviewMesh;
	
	/** Active material slots on the preview mesh, by default slot 0 is active */
	UPROPERTY(EditAnywhere, config, Category=Options)
	TSet<uint32> PreviewMeshMaterialSlots = {0};

public:
	UNREALED_API static UMaterialEditorOptions* Get();

	/** Loads and returns the preview mesh or default preview mesh if none */
	UNREALED_API UObject* GetPreviewMesh() const;

	/** Changes the preview mesh */
	UNREALED_API void SetPreviewMesh(UObject* InPreviewMesh);
};

