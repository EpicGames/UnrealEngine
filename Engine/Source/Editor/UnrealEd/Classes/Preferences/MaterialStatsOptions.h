// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* A configuration class used by the FMaterialStats to save settings across sessions.
*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "RHIShaderPlatform.h"
#include "SceneTypes.h"
#include "MaterialStatsOptions.generated.h"

UENUM()
enum class EMaterialStatsDerivedMIOption : uint8
{
	Ignore = 0,
	CompileOnly,
	ShowStats,
	InvalidOrMax,
};

UCLASS(hidecategories = Object, config = EditorPerProjectUserSettings, MinimalAPI)
class UMaterialStatsOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, config, Category=Options)
	int32 bPlatformUsed[SP_NumPlatforms];

	UPROPERTY(EditAnywhere, config, Category = Options)
	int32 bMaterialQualityUsed[EMaterialQualityLevel::Num];

	UPROPERTY(EditAnywhere, config, Category=Options)
	EMaterialStatsDerivedMIOption MaterialStatsDerivedMIOption;
};
