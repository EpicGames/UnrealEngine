// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// PersonaOptions
//
// A configuration class that holds information for the setup of the Persona.
// Supplied so that the editor 'remembers' the last setup the user had.
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "Misc/NamePermissionList.h"
#include "Settings/EditorViewportSettings.h"
#include "PersonaOptions.generated.h"

enum class EFrameNumberDisplayFormats : uint8;

/** Persisted camera follow mode */
UENUM()
enum class EAnimationViewportCameraFollowMode : uint8
{
	/** Standard camera controls */
	None,

	/** Follow the bounds of the mesh */
	Bounds,

	/** Follow a bone or socket */
	Bone,

	/** Follow the root bone while keeping the mesh vertically centered */
	Root,

	/** Use transform of bone to build viewport transform */
	BoneAsCamera,
};

/** Persistent per-viewport options */
USTRUCT()
struct FViewportConfigOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	TEnumAsByte<EViewModeIndex> ViewModeIndex;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	float ViewFOV;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FEditorViewportCameraSpeedSettings CameraSpeedSettings;

	UPROPERTY(config)
	int32 CameraSpeedSetting_DEPRECATED;

	UPROPERTY(config)
	float CameraSpeedScalar_DEPRECATED;

	/** Persisted camera follow mode for a viewport */
	UPROPERTY(config)
	EAnimationViewportCameraFollowMode CameraFollowMode;

	UPROPERTY(config)
	FName CameraFollowBoneName;

	FViewportConfigOptions()
		: ViewModeIndex(EViewModeIndex::VMI_Lit)
		, ViewFOV(53.43f)
		, CameraSpeedSetting_DEPRECATED(4)
		, CameraSpeedScalar_DEPRECATED(1.0f)
		, CameraFollowMode(EAnimationViewportCameraFollowMode::None)
	{}

	void SetToDefault();
};

/** Options that should be unique per asset editor (like skeletal mesh or anim sequence editors) */
USTRUCT()
struct FAssetEditorOptions
{
	GENERATED_BODY()

	FAssetEditorOptions()
	{
		SetViewportConfigsToDefault();
	}

	FAssetEditorOptions(const FName& InContext)
		: Context(InContext)
	{
		SetViewportConfigsToDefault();
	}

	/** the name of the asset editor properties apply to */
	UPROPERTY(config)
	FName Context;

	/** Per-viewport configuration */
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FViewportConfigOptions ViewportConfigs[4];

	bool operator==(const FAssetEditorOptions& InOptions) const
	{
		return InOptions.Context == Context;
	}

	void SetViewportConfigsToDefault();
};

UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UPersonaOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Whether or not the floor should be aligned to the Skeletal Mesh's bounds by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Preview Scene")
	uint32 bAutoAlignFloorToMesh : 1;

	/** Whether or not the Animation Editor opens in an additional tab when double clicking an animation asset or if it reuses an already existing Animation Editor tab.
	  * You can also keep this disabled and hold shift pressed while double clicking the asset to open the asset inside its own tab.
	  */
	UPROPERTY(EditAnywhere, config, Category = "Assets")
	uint32 bAlwaysOpenAnimationAssetsInNewTab : 1;

	/** Whether or not the grid should be visible by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	uint32 bShowGrid:1;

	/** Whether or not the XYZ axis at the origin should be highlighted on the grid by default */
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	uint32 bHighlightOrigin:1;

	/** Whether or not audio should be muted by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Audio")
	uint32 bMuteAudio:1;

	/** Currently Stats can have None, Basic and Detailed. Please refer to EDisplayInfoMode. */
	UPROPERTY(EditAnywhere, config, Category = "Viewport", meta=(ClampMin ="0", ClampMax = "3", UIMin = "0", UIMax = "3"))
	int32 ShowMeshStats;
	
	/** Index used to determine which ViewMode should be used by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	uint32 DefaultLocalAxesSelection;

	/** Index used to determine which Bone Draw Mode should be used by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	uint32 DefaultBoneDrawSelection;

	/** If true then the bones are displayed with multiple semi-random colors */
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	bool bShowBoneColors = false;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor DefaultBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor SelectedBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor AffectedBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor DisabledBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor ParentOfSelectedBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	FLinearColor VirtualBoneColor;

	UPROPERTY(EditAnywhere, config, Category = "Composites and Montages")
	FLinearColor SectionTimingNodeColor;

	UPROPERTY(EditAnywhere, config, Category = "Composites and Montages")
	FLinearColor NotifyTimingNodeColor;

	UPROPERTY(EditAnywhere, config, Category = "Composites and Montages")
	FLinearColor BranchingPointTimingNodeColor;

	/** Pause the preview animation if playing when moving the camera and resume when finished */
	UPROPERTY(EditAnywhere, config, Category = "Viewport")
	bool bPauseAnimationOnCameraMove;

	/** Whether to use a socket editor that is created in-line inside the skeleton tree, or whether to use the separate details panel */
	UPROPERTY(EditAnywhere, config, Category = "Skeleton Tree")
	bool bUseInlineSocketEditor;

	/** Whether to keep the hierarchy or flatten it when searching for bones, sockets etc. */
	UPROPERTY(EditAnywhere, config, Category = "Skeleton Tree")
	bool bFlattenSkeletonHierarchyWhenFiltering;

	/** Whether to hide parent items when filtering or to display them grayed out */
	UPROPERTY(EditAnywhere, config, Category = "Skeleton Tree")
	bool bHideParentsWhenFiltering;

	/** Whether to append each bone's index to its name when shown in the skeleton tree */
	UPROPERTY(EditAnywhere, config, Category = "Skeleton Tree")
	bool bShowBoneIndexes = false;

	/** Whether to focus and expand an item's tree recursively based on selection */
	UPROPERTY(EditAnywhere, config, Category = "Skeleton Tree")
	bool bExpandTreeOnSelection;

	UPROPERTY(EditAnywhere, config, Category = "Preview Scene|AdditionalMesh")
	bool bAllowPreviewMeshCollectionsToSelectFromDifferentSkeletons;

	UPROPERTY(EditAnywhere, config, Category = "Preview Scene|AdditionalMesh")
	bool bAllowPreviewMeshCollectionsToUseCustomAnimBP;

	/** Whether or not Skeletal Mesh Section selection should be enabled by default for the Animation Editor(s)*/
	UPROPERTY(EditAnywhere, config, Category = "Mesh")
	bool bAllowMeshSectionSelection;

	/** The number of folder filters to allow at any one time in the animation tool's asset browser */
	UPROPERTY(EditAnywhere, config, Category = "Asset Browser", meta=(ClampMin ="1", ClampMax = "10", UIMin = "1", UIMax = "10"))
	uint32 NumFolderFiltersInAssetBrowser;

	/** Options that should be unique per asset editor (like skeletal mesh or anim sequence editors) */
	UPROPERTY(config)
	TArray<FAssetEditorOptions> AssetEditorOptions;

	/** Snap value used to determine scrub resolution of the curve timeline */
	UPROPERTY(config)
	float CurveEditorSnapInterval;

	/** Snap value used to determine scrub resolution of the anim timeline */
	UPROPERTY(config)
	int32 TimelineScrubSnapValue;

	/** Display format for the anim timeline */
	UPROPERTY(config)
	EFrameNumberDisplayFormats TimelineDisplayFormat;

	/** Whether to display percentage in the anim timeline */
	UPROPERTY(config)
	bool bTimelineDisplayPercentage;

	/** Whether to display secondary format (times/frames) in the anim timeline */
	UPROPERTY(config)
	bool bTimelineDisplayFormatSecondary;

	/** Whether to display keys in the timeline's curve tracks */
	UPROPERTY(config)
	bool bTimelineDisplayCurveKeys;

	/** Whether to snap to various things */
	UPROPERTY(config)
	TArray<FName> TimelineEnabledSnaps;

	/** Whether to allow animation assets that are incompatible with the current skeleton/skeletal mesh to be selected. */
	UPROPERTY(EditAnywhere, config, Category = "Assets")
	bool bAllowIncompatibleSkeletonSelection;

	/** Whether to use tree view for animation curves*/
	UPROPERTY(EditAnywhere, config, Category = "Timeline")
	bool bUseTreeViewForAnimationCurves = false;

	/** Delimiters to split animation curve names for grouping*/
	UPROPERTY(EditAnywhere, config, Category = "Timeline")
	FString AnimationCurveGroupingDelimiters = TEXT("._/|\\");
public:
	UNREALED_API void SetShowGrid( bool bInShowGrid );
	UNREALED_API void SetHighlightOrigin( bool bInHighlightOrigin );
	UNREALED_API void SetAutoAlignFloorToMesh(bool bInAutoAlignFloorToMesh);
	UNREALED_API void SetMuteAudio( bool bInMuteAudio );
	UNREALED_API void SetViewModeIndex( FName InContext, EViewModeIndex InViewModeIndex, int32 InViewportIndex );
	UNREALED_API void SetViewFOV( FName InContext, float InViewFOV, int32 InViewportIndex );
	UNREALED_API void SetCameraSpeedSettings(FName InContext, const FEditorViewportCameraSpeedSettings& SpeedSettings, int32 InViewportIndex);
	UE_DEPRECATED(5.7, "Use SetCameraSpeedSettings instead")
	UNREALED_API void SetCameraSpeed(FName InContext, int32 InCameraSpeed, int32 InViewportIndex);
	UE_DEPRECATED(5.7, "Use SetCameraSpeedSettings instead")
	UNREALED_API void SetCameraSpeedScalar(FName InContext, float InCameraSpeedScalar, int32 InViewportIndex);
	UNREALED_API void SetViewCameraFollow( FName InContext, EAnimationViewportCameraFollowMode InCameraFollowMode, FName InCameraFollowBoneName, int32 InViewportIndex );
	UNREALED_API void SetDefaultLocalAxesSelection( uint32 InDefaultLocalAxesSelection );
	UNREALED_API void SetDefaultBoneDrawSelection(uint32 InDefaultBoneAxesSelection);
	UNREALED_API void SetShowMeshStats( int32 InShowMeshStats );
	UNREALED_API void SetSectionTimingNodeColor(const FLinearColor& InColor);
	UNREALED_API void SetNotifyTimingNodeColor(const FLinearColor& InColor);
	UNREALED_API void SetBranchingPointTimingNodeColor(const FLinearColor& InColor);
	UNREALED_API FAssetEditorOptions& GetAssetEditorOptions(const FName& InContext);
	UNREALED_API bool GetAllowIncompatibleSkeletonSelection() const;
	UNREALED_API void SetAllowIncompatibleSkeletonSelection(bool bState);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateSettingsMulticaster, const UPersonaOptions*, EPropertyChangeType::Type);
	FOnUpdateSettingsMulticaster OnSettingsChange;

	FDelegateHandle RegisterOnUpdateSettings(const FOnUpdateSettingsMulticaster::FDelegate& Delegate)
	{
		return OnSettingsChange.Add(Delegate);
	}

	void UnregisterOnUpdateSettings(FDelegateHandle Object)
	{
		OnSettingsChange.Remove(Object);
	}

	FNamePermissionList& GetAllowedAnimationEditorTracks() { return AllowedAnimationEditorTracks; }

	bool bExposeClothingSceneElementMenu = true;

	bool bExposeNotifiesUICommands = true;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsClassAllowed, const UClass* /*InClass*/);

	/** Sets a delegate that allows external code to restrict which features can be used within the Persona editor by filtering which classes are allowed. */
	void SetOnIsClassAllowed(const FOnIsClassAllowed& InOnIsClassAllowed)
	{
		OnIsClassAllowedDelegate = InOnIsClassAllowed;
	}

	/** Returns whether or not the supplied class can be used in the current editor context. */
	bool IsAllowedClass(const UClass* InClass) const
	{
		return OnIsClassAllowedDelegate.IsBound() == false || OnIsClassAllowedDelegate.Execute(InClass);
	}

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsPropertyAllowed, const FProperty* /*InProperty*/);

	/** Sets a delegate that allows external code to restrict which features can be used within the Persona editor by filtering which properties are allowed. */
	void SetOnIsPropertyAllowed(const FOnIsPropertyAllowed& InOnIsPropertyAllowed)
	{
		OnIsPropertyAllowedDelegate = InOnIsPropertyAllowed;
	}

	/** Returns whether or not the supplied properties can be used in the current editor context. */
	bool IsAllowedProperty(const FProperty* InProperty) const
	{
		return OnIsPropertyAllowedDelegate.IsBound() == false || OnIsPropertyAllowedDelegate.Execute(InProperty);
	}

protected:
	// UObject interface
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	FNamePermissionList AllowedAnimationEditorTracks;

	FOnIsClassAllowed OnIsClassAllowedDelegate;

	FOnIsPropertyAllowed OnIsPropertyAllowedDelegate;
};
