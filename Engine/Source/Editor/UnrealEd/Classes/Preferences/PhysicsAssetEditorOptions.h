// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsAssetEditorOptions.generated.h"

UENUM()
enum class EPhysicsAssetEditorCollisionViewMode : uint8
{
	Solid,
	Wireframe,
	SolidWireframe,
	None
};

UENUM()
enum class EPhysicsAssetEditorCenterOfMassViewMode : uint8
{
	All,
	Selected,
	None
};

UENUM()
enum class EPhysicsAssetEditorMeshViewMode : uint8
{
	Solid,
	Wireframe,
	None
};

UENUM()
enum class EPhysicsAssetEditorConstraintViewMode : uint8
{
	None,
	AllPositions,
	AllLimits
};

UCLASS(hidecategories=Object, config=EditorPerProjectUserSettings, MinimalAPI)
class UPhysicsAssetEditorOptions : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Lets you manually control the physics/animation */
	UPROPERTY(EditAnywhere, transient, Category=Anim, meta=(UIMin=0, UIMax=1, ClampMin=0, ClampMax=1))
	float PhysicsBlend;

	/** Lets you manually control the physics/animation */
	UPROPERTY(EditAnywhere, transient, Category = Anim)
	bool bUpdateJointsFromAnimation;

	/** Determines whether simulation of root body updates component transform */
	UPROPERTY(EditAnywhere, transient, Category = Anim)
	TEnumAsByte<EPhysicsTransformUpdateMode::Type> PhysicsUpdateMode;

	/** Time between poking ragdoll and starting to blend back. */
	UPROPERTY(EditAnywhere, config, Category=Anim, meta = (ClampMin = 0))
	float PokePauseTime;

	/** Time taken to blend from physics to animation. */
	UPROPERTY(EditAnywhere, config, Category=Anim, meta = (ClampMin = 0))
	float PokeBlendTime;
	
	/** Scale factor for the gravity used in the simulation */
	UPROPERTY(EditAnywhere, config, Category=Simulation, meta=(UIMin=0, UIMax=100, ClampMin=-10000, ClampMax=10000, EditCondition = "!bUseGravityOverride"))
	float GravScale;

	/** Gravity override used in the simulation */
	UPROPERTY(EditAnywhere, config, Category = Simulation, meta = (UIMin = 0, UIMax = 100, ClampMin = -100000, ClampMax = 100000, EditCondition = "bUseGravityOverride"))
	float GravityOverrideZ;

	/* Toggle gravity override vs gravity scale */
	UPROPERTY(EditAnywhere, config, Category = Simulation, meta = (InlineEditConditionToggle))
	bool bUseGravityOverride;

	/** Max FPS for simulation in PhysicsAssetEditor. This is helpful for targeting the same FPS as your game. -1 means disabled*/
	UPROPERTY(EditAnywhere, config, Category = Simulation)
	int32 MaxFPS;

	/** Linear damping of mouse spring forces */
	UPROPERTY(EditAnywhere, config, Category=MouseSpring, meta = (ClampMin = 0))
	float HandleLinearDamping;

	/** Linear stiffness of mouse spring forces */
	UPROPERTY(EditAnywhere, config, Category=MouseSpring, meta = (ClampMin = 0))
	float HandleLinearStiffness;

	/** Angular damping of mouse spring forces */
	UPROPERTY(EditAnywhere, config, Category=MouseSpring, meta = (ClampMin = 0))
	float HandleAngularDamping;

	/** Angular stiffness of mouse spring forces */
	UPROPERTY(EditAnywhere, config, Category=MouseSpring, meta = (ClampMin = 0))
	float HandleAngularStiffness;

	/** How quickly we interpolate the physics target transform for mouse spring forces */
	UPROPERTY(EditAnywhere, config, Category=MouseSpring, meta = (ClampMin = 0))
	float InterpolationSpeed;

	/** Strength of the impulse used when poking with left mouse button */
	UPROPERTY(EditAnywhere, config, Category=Poking)
	float PokeStrength;

	/** Raycast distance when poking or grabbing */
	UPROPERTY(EditAnywhere, config, Category = Poking, meta = (ClampMin = 0))
	float InteractionDistance;

	/** Whether to draw overlapping physics bodies in a different color in the view port. */
	UPROPERTY(config)
	uint32 bHighlightOverlapingBodies : 1;

	/** Whether to draw constraints as points */
	UPROPERTY(config)
	uint32 bShowConstraintsAsPoints:1;

	/** Whether to highlight limits that have been violated */
	UPROPERTY(config)
	uint32 bDrawViolatedLimits:1;

	/** Whether to draw CoM positions / mass values for kinematic bodies as well as other body types */
	UPROPERTY(config)
	uint32 bHideCenterOfMassForKinematicBodies:1;

	/** Whether to only render selected constraints */
	UPROPERTY(config)
	uint32 bRenderOnlySelectedConstraints:1;

	/* Toggle collisions with floor in the simulation */
	UPROPERTY(config)
	uint32 bSimulationFloorCollisionEnabled:1;

	/** Controls the size of Center of Mass markers drawn in Physics Asset Editor */
	UPROPERTY(config)
	float COMRenderSize;

	/** Controls how large constraints are drawn in Physics Asset Editor */
	UPROPERTY(config)
	float ConstraintDrawSize;

	/** View mode for meshes in edit mode */
	UPROPERTY(config)
	EPhysicsAssetEditorMeshViewMode MeshViewMode;

	/** View mode for collision in edit mode */
	UPROPERTY(config)
	EPhysicsAssetEditorCollisionViewMode CollisionViewMode;

	/** View mode for Center of Mass in edit mode */
	UPROPERTY(config)
	EPhysicsAssetEditorCenterOfMassViewMode CenterOfMassViewMode;

	/** View mode for constraints in edit mode */
	UPROPERTY(config)
	EPhysicsAssetEditorConstraintViewMode ConstraintViewMode;

	/** View mode for meshes in simulation mode */
	UPROPERTY(config)
	EPhysicsAssetEditorMeshViewMode SimulationMeshViewMode;

	/** View mode for collision in simulation mode */
	UPROPERTY(config)
	EPhysicsAssetEditorCollisionViewMode SimulationCollisionViewMode;

	/** View mode for Center of Mass in simulation mode */
	UPROPERTY(config)
	EPhysicsAssetEditorCenterOfMassViewMode SimulationCenterOfMassViewMode;
	
	/** View mode for constraints in simulation mode */
	UPROPERTY(config)
	EPhysicsAssetEditorConstraintViewMode SimulationConstraintViewMode;

	/** Opacity of 'solid' rendering */
	UPROPERTY(config, meta = (ClampMin = 0, ClampMax = 1))
	float CollisionOpacity;

	/** When set, turns opacity of solid rendering for unselected bodies to zero */
	UPROPERTY(config)
	bool bSolidRenderingForSelectedOnly;

	/** When set, disables rendering for simulated bodies */
	UPROPERTY(config)
	bool bHideSimulatedBodies;

	/** When set, disables rendering for kinematic bodies */
	UPROPERTY(config)
	bool bHideKinematicBodies;

	/** When set, disables rendering for body mass text displayed next to Center of Mass indicator */
	UPROPERTY(config)
	bool bHideBodyMass;

	/** When set, constraints will be automatically created when physics bodies are created through the context menu in the skeleton tree. */
	UPROPERTY(config)
	bool bCreateConstraintsWhenCreatingBodiesFromSkeletonTree;

	/** When set, cloth will reset each time simulation is toggled */
	UPROPERTY(EditAnywhere, config, Category=Clothing)
	bool bResetClothWhenSimulating;

	// The following are for enabling/disabling controls at runtime.
	// Some controls use the new "UToolMenu" menus. These are enabled/disabled via json permissions.
	// Other controls use the legacy FMenuBuilder. These use the following properties to enable/disable.
	UPROPERTY()
	bool bExposeLegacyMenuSimulationControls = true;

	UPROPERTY()
	bool bExposeLegacyMenuConstraintControls = true;
};
