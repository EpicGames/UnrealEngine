// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This class handles hotkey binding management for the editor.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "UnrealEdKeyBindings.generated.h"

/** An editor hotkey binding to a parameterless exec. */
USTRUCT()
struct FEditorKeyBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint32 bCtrlDown:1;

	UPROPERTY()
	uint32 bAltDown:1;

	UPROPERTY()
	uint32 bShiftDown:1;

	UPROPERTY()
	FKey Key;

	UPROPERTY()
	FName CommandName;

	FEditorKeyBinding()
		: bCtrlDown(false)
		, bAltDown(false)
		, bShiftDown(false)
	{}
};

UCLASS(Config=EditorKeyBindings)
class UUnrealEdKeyBindings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Array of keybindings */
	UPROPERTY(config)
	TArray<struct FEditorKeyBinding> KeyBindings;

};

