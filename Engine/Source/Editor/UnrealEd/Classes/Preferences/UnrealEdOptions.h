// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This class stores options global to the entire editor.
 *
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "UnrealEdOptions.generated.h"

/** A category to store a list of commands. */
USTRUCT()
struct FEditorCommandCategory
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Parent;

	UPROPERTY()
	FName Name;

};

/** A parameterless exec command that can be bound to hotkeys and menu items in the editor. */
USTRUCT()
struct FEditorCommand
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Parent;

	UPROPERTY()
	FName CommandName;

	UPROPERTY()
	FString ExecCommand;

	UPROPERTY()
	FString Description;

};

/** Default Classes for the Class Picker Dialog*/
USTRUCT()
struct FClassPickerDefaults
{
	GENERATED_USTRUCT_BODY()

	FClassPickerDefaults()
	{}

	FClassPickerDefaults(const FString& InClassName, const FString& InAssetClass)
		: ClassName(InClassName)
		, AssetClass(InAssetClass)
	{}

	FClassPickerDefaults(FString&& InClassName, FString&& InAssetClass)
		: ClassName(MoveTemp(InClassName))
		, AssetClass(MoveTemp(InAssetClass))
	{}

	/** The name of the class to select */
	UPROPERTY()
	FString ClassName;

	/** The name of the asset type being created */
	UPROPERTY()
	FString AssetClass;

	/** Gets the localized name text for the class */
	FText GetName() const;

	/** Gets the localized descriptive text for the class */
	FText GetDescription() const;
};

UCLASS(Config=Editor, MinimalAPI)
class UUnrealEdOptions : public UObject
{
	GENERATED_BODY()
public:

	/** Categories of commands. */
	UPROPERTY(config)
	TArray<struct FEditorCommandCategory> EditorCategories;

	/** Commands that can be bound to in the editor. */
	UPROPERTY(config)
	TArray<struct FEditorCommand> EditorCommands;

	/** Pointer to the key bindings object that actually stores key bindings for the editor. */
	UPROPERTY()
	TObjectPtr<class UUnrealEdKeyBindings> EditorKeyBindings;

	/** If true, the list of classes in the class picker dialog will be expanded */
	UPROPERTY(config)
	bool bExpandClassPickerClassList;

	/** The array of default objects in the blueprint class dialog **/
	UPROPERTY(config)
	TArray<FClassPickerDefaults> NewAssetDefaultClasses;

	/* Delegate to override NewAssetDefaultClasses */
	DECLARE_DELEGATE_RetVal(const TArray<FClassPickerDefaults>&, FGetNewAssetDefaultClasses);
	FGetNewAssetDefaultClasses& OnGetNewAssetDefaultClasses() { return GetNewAssetDefaultClassesDelegate; }

	/** Get default objects in the blueprint class dialog */
	const TArray<FClassPickerDefaults>& GetNewAssetDefaultClasses() const
	{
		return GetNewAssetDefaultClassesDelegate.IsBound() ? GetNewAssetDefaultClassesDelegate.Execute() : NewAssetDefaultClasses;
	}

	/* Delegate to disallow C++ creation/editing from the editor */
	DECLARE_DELEGATE_RetVal(bool, FIsCPPAllowed);
	FIsCPPAllowed& OnIsCPPAllowed() { return IsCPPAllowedDelegate; }

	/** Returns whether C++ creation/editing from the editor is allowed */
	bool IsCPPAllowed() { return IsCPPAllowedDelegate.IsBound() ? IsCPPAllowedDelegate.Execute() : true; }

	/** Mapping of command name's to array index. */
	TMap<FName, int32>	CommandMap;

	//~ Begin UObject Interface
	UNREALED_API virtual void PostInitProperties() override;
	//~ End UObject Interface

	/**
	 * Generates a mapping from commnands to their parent sets for quick lookup.
	 */
	UNREALED_API void GenerateCommandMap();

	/**
	 * Attempts to locate a exec command bound to a hotkey.
	 *
	 * @param Key			Key name
	 * @param bAltDown		Whether or not ALT is pressed.
	 * @param bCtrlDown		Whether or not CONTROL is pressed.
	 * @param bShiftDown	Whether or not SHIFT is pressed.
	 * @param EditorSet		Set of bindings to search in.
	 */
	UNREALED_API FString GetExecCommand(FKey Key, bool bAltDown, bool bCtrlDown, bool bShiftDown, FName EditorSet);

private:

	/* Delegate to override NewAssetDefaultClasses */
	FGetNewAssetDefaultClasses GetNewAssetDefaultClassesDelegate;

	/** Delegate to disallow C++ creation/editing from the editor */
	FIsCPPAllowed IsCPPAllowedDelegate;
};
