// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "AnimBlueprintSettings.generated.h"

/**
 * Implements Editor settings for animation blueprints
 */
UCLASS(config = EditorPerProjectUserSettings, MinimalAPI)
class UAnimBlueprintSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether to allow using animation blueprints */
	UPROPERTY()
	bool bAllowAnimBlueprints = true;

	/** Whether to allow event graphs to be created/displayed in animation blueprints */
	UPROPERTY()
	bool bAllowEventGraphs = true;

	/** Whether to allow macros to be created/displayed in animation blueprints */
	UPROPERTY()
	bool bAllowMacros = true;

	/** Whether to allow delegates to be created/displayed in animation blueprints */
	UPROPERTY()
	bool bAllowDelegates = true;
	
	/** Whether to allow pose watches to be created/displayed in animation blueprints */
	UPROPERTY()
	bool bAllowPoseWatches = true;

	/** Whether to allow restrict which base function overrides can created/displayed in animation blueprints */
	UPROPERTY()
	bool bRestrictBaseFunctionOverrides = false;
	
	/**
	* Whether to allow input events to be created/displayed in animation blueprints.
	* 
	* You used to be able to place input event nodes in anim graphs and they would just not work, sometimes causing an ensure.
	* 
	* If needed, you can enable this legacy behavior by setting this property to true (allowing input events to be placed in a graph).
	* 
	* Default value is false.
	*/
	UPROPERTY(Config)
	bool bSupportInputEventsForBackwardsCompatibility = false;

	/** Whether or not anim blueprint should perform allow-list validation */
	UPROPERTY()
	bool bPerformValidation = false;

	/** The set of allowed base functions if restricted */
	UPROPERTY()
	TArray<FName> BaseFunctionOverrideAllowList;
};
