// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ClassViewerSettings.h: Declares the UClassViewerSettings class.
=============================================================================*/

#pragma once


#include "ClassViewerSettings.generated.h"


/** The developer folder view modes used in SClassViewer */
UENUM()
enum class EClassViewerDeveloperType : uint8
{
	/** Display no developer folders*/
	CVDT_None,
	/** Allow the current user's developer folder to be displayed. */
	CVDT_CurrentUser,
	/** Allow all users' developer folders to be displayed.*/
	CVDT_All,
	/** Max developer type*/
	CVDT_Max
};

/**
 * Implements the Class Viewer's loading and saving settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UClassViewerSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(config)
	TArray<FString> AllowedClasses;

	/** Whether to display internal use classes. */
	UPROPERTY(config)
	bool DisplayInternalClasses;


	/** The developer folder view modes used in SClassViewer */
	UPROPERTY(config)
	EClassViewerDeveloperType DeveloperFolderType;

public:
	
	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UClassViewerSettings, FSettingChangedEvent, FName /*PropertyName*/);
	static FSettingChangedEvent& OnSettingChanged() { return SettingChangedEvent; }

protected:

	// UObject overrides
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UNREALED_API virtual void PostInitProperties() override;
	UNREALED_API virtual void PostLoad() override;

	UNREALED_API void FixupShortNames();

	// Holds an event delegate that is executed when a setting has changed.
	static UNREALED_API FSettingChangedEvent SettingChangedEvent;
};
