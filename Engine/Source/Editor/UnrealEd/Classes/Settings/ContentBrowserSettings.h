// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ContentBrowserSettings.h: Declares the UContentBrowserSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ContentBrowserSettings.generated.h"

/**
 * Implements content browser settings.  These are global not per-project
 */
UCLASS(config=EditorSettings, MinimalAPI)
class UContentBrowserSettings : public UObject
{
	GENERATED_BODY()

public:

	/** The number of objects to load at once in the Content Browser before displaying a warning about loading many assets */
	UPROPERTY(EditAnywhere, config, Category=ContentBrowser, meta=(DisplayName = "Assets to Load at Once Before Warning", ClampMin = "1"))
	int32 NumObjectsToLoadBeforeWarning;

	/** Whether to render thumbnails for loaded assets in real-time in the Content Browser */
	UPROPERTY(config)
	bool RealTimeThumbnails;

	/** Whether to display folders in the asset view of the content browser. Note that this implies 'Show Only Assets in Selected Folders'. */
	UPROPERTY(config)
	bool DisplayFolders;

	/** Whether to empty display folders in the asset view of the content browser. */
	UPROPERTY(config)
	bool DisplayEmptyFolders;

	/** Whether to filter recursively when a filter is applied in the asset view of the content browser. */
	UPROPERTY(config)
	bool FilterRecursively = true;

	/** Whether to group root folders under a common folder in the path view */
	UPROPERTY(config)
	bool bShowAllFolder = true;

	/** Whether to organize folders in the content browser */
	UPROPERTY(config)
	bool bOrganizeFolders = true;

	/** Whether to append 'Content' text to displayed folder names */
	UPROPERTY(EditAnywhere, config, Category = ContentBrowser)
	bool bDisplayContentFolderSuffix = false;

	/** Whether display friendly name as plugin folder names */
	UPROPERTY(EditAnywhere, config, Category = ContentBrowser)
	bool bDisplayFriendlyNameForPluginFolders = true;

	/** The number of objects to keep in the Content Browser Recently Opened filter */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "The filter now always keeps track of your last 30 recent assets"))
	int32 NumObjectsInRecentList_DEPRECATED;

	/** Enables the rendering of Material Instance thumbnail previews */
	UPROPERTY(EditAnywhere, config, Category = ContentBrowser)
	bool bEnableRealtimeMaterialInstanceThumbnails = true;
	
	/** Display hint text of keys pressed in content browser in bottom right of the asset view, used to find item by name in a folder */
	UPROPERTY(Config, EditAnywhere, Category = ContentBrowser)
	bool bDisplayQuickJumpText = false;

	/** Displays collections in the Collections window that were excluded by project settings */
	UPROPERTY(Config, EditAnywhere, Category=Collections, meta = (DisplayName="Display excluded collections in Content Browser"))
	bool bDisplayExcludedCollections = false;

public:

	/** Sets whether we are allowed to display the engine folder or not, optional flag for setting override instead */
	void SetDisplayEngineFolder( bool bInDisplayEngineFolder )
	{
		DisplayEngineFolder = bInDisplayEngineFolder;
	}

	/** Gets whether we are allowed to display the engine folder or not, optional flag ignoring the override */
	bool GetDisplayEngineFolder() const
	{
		return DisplayEngineFolder;
	}

	/** Sets whether we are allowed to display the developers folder or not, optional flag for setting override instead */
	void SetDisplayDevelopersFolder( bool bInDisplayDevelopersFolder )
	{
		DisplayDevelopersFolder = bInDisplayDevelopersFolder;
	}

	/** Gets whether we are allowed to display the developers folder or not, optional flag ignoring the override */
	bool GetDisplayDevelopersFolder() const
	{
		return DisplayDevelopersFolder;
	}

	/** Sets whether we are allowed to display the L10N folder (contains localized assets) or not */
	void SetDisplayL10NFolder(bool bInDisplayL10NFolder)
	{
		DisplayL10NFolder = bInDisplayL10NFolder;
	}

	/** Gets whether we are allowed to display the L10N folder (contains localized assets) or not */
	bool GetDisplayL10NFolder() const
	{
		return DisplayL10NFolder;
	}

	/** Sets whether we are allowed to display the plugin folders or not */
	void SetDisplayPluginFolders( bool bInDisplayPluginFolders )
	{
		DisplayPluginFolders = bInDisplayPluginFolders;
	}

	/** Gets whether we are allowed to display the plugin folders or not */
	bool GetDisplayPluginFolders() const
	{
		return DisplayPluginFolders;
	}

	/** Sets whether we are allowed to display favorite folders or not */
	void SetDisplayFavorites(bool bInDisplayFavorites)
	{
		DisplayFavorites = bInDisplayFavorites;
	}

	/** Gets whether we are allowed to display the favorite folders or not*/
	bool GetDisplayFavorites() const
	{
		return DisplayFavorites;
	}

	/** Sets whether we should dock the collections view under the paths view */
	UE_DEPRECATED(5.6, "Collection docking is deprecated. This function has to effect.")
	void SetDockCollections(bool bInDockCollections)
	{
	}

	/** Gets whether we should dock the collections view under the paths view */
	UE_DEPRECATED(5.6, "Collection docking is deprecated.")
	bool GetDockCollections() const
	{
		return true;
	}

	/** Sets whether we are allowed to display C++ folders or not */
	void SetDisplayCppFolders(bool bDisplay)
	{
		DisplayCppFolders = bDisplay;
	}

	/** Gets whether we are allowed to display the C++ folders or not*/
	bool GetDisplayCppFolders() const
	{
		return DisplayCppFolders;
	}

	/** Sets whether text searches should also search in asset class names */
	void SetIncludeClassNames(bool bInclude)
	{
		IncludeClassNames = bInclude;
	}

	/** Gets whether text searches should also search in asset class names */
	bool GetIncludeClassNames() const
	{
		return IncludeClassNames;
	}

	/** Sets whether text searches should also search asset paths (instead of asset name only) */
	void SetIncludeAssetPaths(bool bInclude)
	{
		IncludeAssetPaths = bInclude;
	}

	/** Gets whether text searches should also search asset paths (instead of asset name only) */
	bool GetIncludeAssetPaths() const
	{
		return IncludeAssetPaths;
	}

	/** Sets whether text searches should also search for collection names */
	void SetIncludeCollectionNames(bool bInclude)
	{
		IncludeCollectionNames = bInclude;
	}

	/** Gets whether text searches should also search for collection names */
	bool GetIncludeCollectionNames() const
	{
		return IncludeCollectionNames;
	}

	/** Sets whether the tooltip should be expanded by default */
	void SetAlwaysExpandTooltips(bool bIsExpanded)
	{
		AlwaysExpandTooltips = bIsExpanded;
	}

	/** Gets whether the tooltip should be expanded by default */
	bool GetAlwaysExpandTooltips() const
	{
		return AlwaysExpandTooltips;
	}

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UContentBrowserSettings, FSettingChangedEvent, FName /*PropertyName*/);
	static FSettingChangedEvent& OnSettingChanged( ) { return SettingChangedEvent; }

protected:

	// UObject overrides

	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;

private:

	/** Whether to display the engine folder in the assets view of the content browser. */
	UPROPERTY(config)
	bool DisplayEngineFolder;

	/** Whether to display the developers folder in the path view of the content browser */
	UPROPERTY(config)
	bool DisplayDevelopersFolder;

	UPROPERTY(config)
	bool DisplayL10NFolder;

	/** List of plugin folders to display in the content browser. */
	UPROPERTY(config)
	bool DisplayPluginFolders;

	UPROPERTY(config)
	bool DisplayFavorites;

	UPROPERTY(config)
	bool DisplayCppFolders;

	UPROPERTY(config)
	bool IncludeClassNames;

	UPROPERTY(config)
	bool IncludeAssetPaths;

	UPROPERTY(config)
	bool IncludeCollectionNames;

	/** Whether to display the full tooltip on Asset and Folder or the small version with the Ctrl + Alt */
	UPROPERTY(config)
	bool AlwaysExpandTooltips;

	// Holds an event delegate that is executed when a setting has changed.
	static UNREALED_API FSettingChangedEvent SettingChangedEvent;
};
