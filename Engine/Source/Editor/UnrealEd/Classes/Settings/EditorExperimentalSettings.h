// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "InterchangePipelineBase.h"
#include "EditorExperimentalSettings.generated.h"

/**
 * Implements Editor settings for experimental features.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UEditorExperimentalSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Enable async texture compilation to improve PIE and map load time performance when compilation is required */
	UPROPERTY(EditAnywhere, config, Category = Performance, meta = (DisplayName = "Enable async texture compilation and loading"))
	bool bEnableAsyncTextureCompilation;

	/** Enable async static mesh compilation to improve import and map load time performance when compilation is required */
	UPROPERTY(EditAnywhere, config, Category = Performance, meta = (DisplayName = "Enable async static mesh compilation and loading"))
	bool bEnableAsyncStaticMeshCompilation;

	/** Enable async skeletal mesh compilation to improve import and map load time performance when compilation is required */
	UE_DEPRECATED(5.1, "Deprecated & replaced by bEnableAsyncSkinnedAssetCompilation.")
	UPROPERTY(/*EditAnywhere - deprecated & replaced by bEnableAsyncSkinnedAssetCompilation, */config/*, Category = Performance, meta = (DisplayName = "Enable async skeletal mesh compilation and loading")*/)
	bool bEnableAsyncSkeletalMeshCompilation;

	/** Enable async skinned asset compilation to improve import and map load time performance when compilation is required */
	UPROPERTY(EditAnywhere, config, Category = Performance, meta = (DisplayName = "Enable async skinned asset compilation and loading"))
	bool bEnableAsyncSkinnedAssetCompilation;

	/** Enable async sound compilation to improve import and map load time performance when compilation is required */
	UPROPERTY(EditAnywhere, config, Category = Performance, meta = (DisplayName = "Enable async sound compilation and loading"))
	bool bEnableAsyncSoundWaveCompilation;

	/** Enable async groom binding compilation to improve import and map load time performance when compilation is required */
	UPROPERTY(EditAnywhere, config, Category = Performance, meta = (DisplayName = "Enable async groom binding compilation and loading"))
	bool bEnableAsyncGroomBindingCompilation;

	/** Allows the editor to run on HDR monitors on Windows 10 */
	UPROPERTY(EditAnywhere, config, Category = HDR, meta = (ConfigRestartRequired = true, DisplayName = "Enable Editor Support for HDR Monitors"))
	bool bHDREditor;

	/** The brightness of the slate UI on HDR monitors */
	UPROPERTY(EditAnywhere, config, Category = HDR, meta = (ClampMin = "100.0", ClampMax = "300.0", UIMin = "100.0", UIMax = "300.0"))
	float HDREditorNITLevel;

	/** Allows usage of the procedural foliage system */
	UPROPERTY(EditAnywhere, config, Category = Foliage, meta = (DisplayName = "Procedural Foliage"))
	bool bProceduralFoliage;

	/** Allows usage of the Translation Picker */
	UPROPERTY(EditAnywhere, config, Category = Tools, meta = (DisplayName = "Translation Picker"))
	bool bEnableTranslationPicker;

	/** Specify which console-specific nomenclature to use for gamepad label text */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta=(DisplayName="Console for Gamepad Labels"))
	TEnumAsByte<EConsoleForGamepadLabels::Type> ConsoleForGamepadLabels;

	/** Allows for customization of toolbars and menus throughout the editor */
	UPROPERTY(config)
	bool bToolbarCustomization;

	/** Break on Exceptions allows you to trap Access Nones and other exceptional events in Blueprints. */
	UPROPERTY(EditAnywhere, config, Category=Blueprints, meta=(DisplayName="Blueprint Break on Exceptions"))
	bool bBreakOnExceptions;

	/** Should arrows indicating data/execution flow be drawn halfway along wires? */
	UPROPERTY(/*EditAnywhere - deprecated (moved into UBlueprintEditorSettings), */config/*, Category=Blueprints, meta=(DisplayName="Draw midpoint arrows in Blueprints")*/)
	bool bDrawMidpointArrowsInBlueprints;

	/** Allows ChunkIDs to be assigned to assets to via the content browser context menu. */
	UPROPERTY(EditAnywhere,config,Category=UserInterface,meta=(DisplayName="Allow ChunkID Assignments"))
	bool bContextMenuChunkAssignments;

	/**
	 * Cooking in the editor is deprecated in 5.7; it has never been updated to support WorldPartition and we are
	 * removing support for it entirely in a future release. bDisableCookInEditor should be set to true.
	 */
	UPROPERTY(EditAnywhere, config, Category = Cooking, meta = (DisplayName = "Disable Cook In The Editor feature (cooks from launch on will be run in a separate process if disabled)", ConfigRestartRequired=true))
	bool bDisableCookInEditor;

	UPROPERTY(EditAnywhere, config, Category = Cooking, meta = (DisplayName = "Use shared cooked builds in launch on", ConfigRestartRequired = true))
	bool bSharedCookedBuilds;

	/** Enable late joining in PIE */
	UPROPERTY(EditAnywhere, config, Category = PIE, meta = (DisplayName = "Allow late joining"))
	bool bAllowLateJoinInPIE;

	/** Enable multithreaded lightmap encoding (decreases time taken to encode lightmaps) */
	UPROPERTY(EditAnywhere, config, Category = LightingBuilds, meta = (DisplayName = "Enable Multithreaded lightmap encoding"))
	bool bEnableMultithreadedLightmapEncoding;

	/** Enable multithreaded shadow map encoding (decreases time taken to encode shadow maps) */
	UPROPERTY(EditAnywhere, config, Category = LightingBuilds, meta = (DisplayName = "Enable Multithreaded shadowmap encoding"))
	bool bEnableMultithreadedShadowmapEncoding;
	
	/** Whether to use OpenCL to accelerate convex hull decomposition (uses GPU to decrease time taken to decompose meshes, currently only available on Mac OS X) */
	UPROPERTY(EditAnywhere, config, Category = Tools, meta = (DisplayName = "Use OpenCL for convex hull decomposition"))
	bool bUseOpenCLForConvexHullDecomp;

	/** Allows editing of potentially unsafe properties during PIE. Advanced use only - use with caution. */
	UPROPERTY(EditAnywhere, config, Category = Tools, meta = (DisplayName = "Allow editing of potentially unsafe properties."))
	bool bAllowPotentiallyUnsafePropertyEditing;

	/** Enable experimental bulk facial animation importer (found in Developer Tools menu, requires editor restart) */
	UPROPERTY(EditAnywhere, config, Category = Tools, meta = (ConfigRestartRequired = true))
	bool bFacialAnimationImporter;

	/** Shows the option to import a custom project launcher profile in the project packaging 'custom builds' section */
	UPROPERTY(EditAnywhere, config, Category = Tools, meta = (DisplayName = "Enable tools in Packaging Settings -> Additional builds for this project"))
	bool bProjectCustomBuildTools;

	/** Enables Extensions for the new Project Launcher */
	UPROPERTY(EditAnywhere, config, Category = Tools)
	bool bEnableProjectLauncherExtensions;

	/** Enables in-editor support for text asset formats */
	UPROPERTY(EditAnywhere, config, Category = Core)
	bool bTextAssetFormatSupport;

	/** Enables in-editor support for rehydrating virtualized assets */
	UPROPERTY(EditAnywhere, config, Category = Core)
	bool bVirtualizedAssetRehydration;

	/** When creating new Material Layers and Material Layer Blends, set up example graphs. */
	UPROPERTY(EditAnywhere, config, Category = Materials)
	bool bExampleLayersAndBlends;

	/** Allows creation of assets with paths longer than 260 characters. Note that this also requires the Windows 10 Anniversary Update (1607), and support for long paths to be enabled through the group policy editor. */
	UPROPERTY(EditAnywhere, config, Category = "Content Browser", meta = (DisplayName = "Enable support for long paths (> 260 characters)"))
	bool bEnableLongPathsSupport;

	UPROPERTY(EditAnywhere, config, Category = WorldPartition)
	bool bEnableWorldPartitionActorFilters;

	UPROPERTY(EditAnywhere, config, Category = WorldPartition)
	bool bEnableWorldPartitionExternalDataLayers;

	UPROPERTY(EditAnywhere, config, Category = WorldPartition)
	bool bEnableStandaloneHLOD;

	UPROPERTY(EditAnywhere, config, Category = LevelInstance)
	bool bEnableLevelInstancePropertyOverrides;

	/** Allows editing of landscapes inside an editing level instance. */
	UPROPERTY(EditAnywhere, config, Category = LevelInstance)
	bool bEnableLevelInstanceLandscapeEdit;

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UEditorExperimentalSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged( )
	{
		return SettingChangedEvent;
	}

protected:

	// UObject overrides
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UNREALED_API virtual void PostInitProperties() override;

private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;
};
