// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "EditorLoadingSavingSettings.generated.h"

UENUM()
namespace ELoadLevelAtStartup
{
	enum Type : int
	{
		None,
		ProjectDefault,
		LastOpened
	};
}


UENUM()
enum class EAutoSaveMethod : uint8
{
	/** Autosave to a backup location and offer to restore after an editor crash */
	BackupAndRestore,
	/** Autosave in-place, overwriting your existing content after backing up the current file (BETA) */
	BackupAndOverwrite,
};

UENUM()
enum class ERestoreOpenAssetTabsMethod : uint8
{
	/** Always prompt the user if they want to restore previously opened asset tabs on launch */
	AlwaysPrompt = 0,
	
	/** Always automatically restore any previously opened asset tabs on launch */
	AlwaysRestore = 1,

	/** Never restore previously opened asset tabs on launch */
	NeverRestore = 2
};

/** A filter used by the auto reimport manager to explicitly include/exclude files matching the specified wildcard */
USTRUCT()
struct FAutoReimportWildcard
{
	GENERATED_USTRUCT_BODY()

	/** The wildcard filter as a string. Files that match this wildcard will be included/excluded according to the bInclude member */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport)
	FString Wildcard;
	
	/** When true, files that match this wildcard will be included (if it doesn't fail any other filters), when false, matches will be excluded from the reimporter */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport)
	bool bInclude = false;
};


/** Auto reimport settings for a specific directory */
USTRUCT()
struct FAutoReimportDirectoryConfig
{
	GENERATED_USTRUCT_BODY()

	/** The source directory to monitor. Either an absolute directory on the file system, or a virtual mounted path */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport, meta=(ToolTip="Path to a virtual package path (eg /Game/ or /MyPlugin/), or absolute paths on disk where your source content files reside."))
	FString SourceDirectory;

	/** Where SourceDirectory points to an ordinary file system path, MountPoint specifies the virtual mounted location to import new files to. */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport, meta=(ToolTip="(Optional) Specify a virtual mout point (e.g. /Game/) to map this directory to on disk. Doing so allows auto-creation of assets when a source content file is created in this folder (see below)."))
	FString MountPoint;

	/** A set of wildcard filters to apply to this directory */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport, meta=(DisplayName="Include/Exclude Wildcards", ToolTip="(Optional) Specify a set of wildcards to include or exclude files from this auto-reimporter."))
	TArray<FAutoReimportWildcard> Wildcards;

	struct FParseContext
	{
		TArray<TPair<FString, FString>> MountedPaths;
		bool bEnableLogging;
		UNREALED_API FParseContext(bool bInEnableLogging = true);
	};

	/** Parse and validate the specified source directory / mount point combination */
	UNREALED_API static bool ParseSourceDirectoryAndMountPoint(FString& SourceDirectory, FString& MountPoint, const FParseContext& InContext = FParseContext());
};


/**
 * Implements the Level Editor's loading and saving settings.
 */
UCLASS(config=EditorPerProjectUserSettings, autoexpandcategories=(AutoSave, AutoReimport, Blueprints), MinimalAPI)
class UEditorLoadingSavingSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Whether to load a default example map at startup  */
	UPROPERTY(EditAnywhere, config, Category=Startup)
	TEnumAsByte<ELoadLevelAtStartup::Type> LoadLevelAtStartup;

	/** Force project compilation at startup */
	UPROPERTY(EditAnywhere, config, Category=Startup)
	uint32 bForceCompilationAtStartup:1;

	/** Whether to restore previously open assets at startup after a clean shutdown */
	UPROPERTY(EditAnywhere, config, Category=Startup)
	ERestoreOpenAssetTabsMethod RestoreOpenAssetTabsOnRestart = ERestoreOpenAssetTabsMethod::AlwaysPrompt;

#if WITH_EDITOR
	/** Whether to restore previously open assets at startup */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use RestoreOpenAssetTabsOnRestart instead"))
	uint32 bRestoreOpenAssetTabsOnRestart_DEPRECATED:1;
#endif

private:

	UPROPERTY(config)
	bool bEnableSourceControlCompatabilityCheck_DEPRECATED;

public:

	/**Automatically reimports textures when a change to source content is detected */
	UPROPERTY(EditAnywhere, config, Category=AutoReimport, meta=(DisplayName="Monitor Content Directories", ToolTip="When enabled, changes to source content files inside the content directories will automatically be reflected in the content browser.\nNote that source content files must reside in one of the monitored directories to be eligible for auto-reimport.\nAdvanced setup options are available below."))
	bool bMonitorContentDirectories;

	UPROPERTY(config)
	TArray<FString> AutoReimportDirectories_DEPRECATED;

	/** Directories being monitored for Auto Reimport */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay,Category=AutoReimport, meta=(DisplayName="Directories to Monitor", ToolTip="Lists every directory to monitor for content changes. Can be virtual package paths (eg /Game/ or /MyPlugin/), or absolute paths on disk.\nPaths should point to the locations of the source content files (e.g. *.fbx, *.png) you want to be eligible for auto-reimport."))
	TArray<FAutoReimportDirectoryConfig> AutoReimportDirectorySettings;

	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=AutoReimport, meta=(ClampMin=0, ClampMax=60, Units=Seconds, DisplayName="Import Threshold Time", ToolTip="Specifies an amount of time to wait before a specific file change is considered for auto reimport"))
	float AutoReimportThreshold;
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=AutoReimport, meta=(DisplayName="Auto Create Assets", ToolTip="When enabled, newly added source content files will be automatically imported into new assets."))
	bool bAutoCreateAssets;
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=AutoReimport, meta=(DisplayName="Auto Delete Assets", ToolTip="When enabled, deleting a source content file will automatically prompt the deletion of any related assets."))
	bool bAutoDeleteAssets;
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=AutoReimport, meta=(DisplayName="Detect Changes On Startup", ToolTip="When enabled, changes to monitored directories since UE was closed will be detected on restart.\n(Not recommended when working in collaboration with others using revision control)."))
	bool bDetectChangesOnStartup;
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=AutoReimport, meta=(DisplayName="Prompt Before Action", ToolTip="Whether to prompt the user to import detected changes."))
	bool bPromptBeforeAutoImporting;

	/** Internal setting to control whether we should ask the user whether we should automatically delete source files when their assets are deleted */
	UPROPERTY(config)
	bool bDeleteSourceFilesWithAssets;

private:

	/** Deprecated properties - we automatically monitor all source file types that are supported by in memory factories now */
	UPROPERTY(config)
	bool bAutoReimportTextures_DEPRECATED;
	UPROPERTY(config)
	bool bAutoReimportCSV_DEPRECATED;

public:

	/** Whether to mark blueprints dirty if they are automatically migrated during loads */
	UPROPERTY(EditAnywhere, config, Category=Blueprints, meta=(DisplayName="Dirty Migrated Blueprints"))
	bool bDirtyMigratedBlueprints;

public:

	/** Whether to automatically save after a time interval */
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Enable AutoSave"))
	uint32 bAutoSaveEnable:1;

	/** Whether to automatically save maps during an autosave */
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Save Maps"))
	uint32 bAutoSaveMaps:1;

	/** Whether to automatically save content packages during an autosave */
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Save Content"))
	uint32 bAutoSaveContent:1;

	/** What method should be used when performing an autosave? */
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Save Method"), AdvancedDisplay)
	EAutoSaveMethod AutoSaveMethod = EAutoSaveMethod::BackupAndRestore;

	/** The time interval after which to auto save */
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Frequency in Minutes", ClampMin = "1"))
	int32 AutoSaveTimeMinutes;

	/** The minimum number of seconds to wait after the last user interactions (with the editor) before auto-save can trigger */
	UPROPERTY(EditAnywhere, Config, Category = AutoSave, meta = (DisplayName = "Interaction Delay in Seconds", ClampMin = "15"))
	int32 AutoSaveInteractionDelayInSeconds;

	/** The number of seconds warning before an autosave*/
	UPROPERTY(EditAnywhere, config, Category=AutoSave, meta=(DisplayName="Warning in seconds", ClampMin = "0", UIMin = "0", UIMax = "20"))
	int32 AutoSaveWarningInSeconds;

	/** How many auto save files to keep around*/
	UPROPERTY(EditAnywhere, config, Category = AutoSave, meta = (DisplayName = "Maximum number of AutoSaves", ClampMin = "1", UIMin = "1", UIMax = "100"))
	int32 AutoSaveMaxBackups = 10;

public:

	/** Whether to automatically checkout on asset modification */
	UPROPERTY(EditAnywhere, config, Category=SourceControl)
	uint32 bAutomaticallyCheckoutOnAssetModification:1;

	/** Whether to automatically prompt for SCC checkout on asset modification */
	UPROPERTY(EditAnywhere, config, Category=SourceControl)
	uint32 bPromptForCheckoutOnAssetModification:1;

	/** Auto add files to source control */
	UPROPERTY(EditAnywhere, config, Category=SourceControl, meta=(DisplayName="Add New Files when Modified"))
	uint32 bSCCAutoAddNewFiles:1;

	/** Use global source control login settings, rather than per-project. Changing this will require you to login again */
	UPROPERTY(EditAnywhere, config, Category=SourceControl, meta=(DisplayName="Use Global Settings"))
	uint32 bSCCUseGlobalSettings:1;

	/** Specifies the file path to the tool to be used for diffing text files */
	UPROPERTY(EditAnywhere, config, Category=SourceControl, meta=(DisplayName="Tool for diffing text"))
	FFilePath TextDiffToolPath;

public:

	// @todo thomass: proper settings support for source control module
	UNREALED_API void SccHackInitialize( );

	UNREALED_API bool GetAutomaticallyCheckoutOnAssetModification() const;

	UNREALED_API void SetAutomaticallyCheckoutOnAssetModificationOverride(bool InValue);

	UNREALED_API void ResetAutomaticallyCheckoutOnAssetModificationOverride();

public:

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UEditorLoadingSavingSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged( ) { return SettingChangedEvent; }

	// UObject overrides

	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;
	UNREALED_API virtual void PostInitProperties() override;

private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;

	// Holds the potentially overridden value of bAutomaticallyCheckoutOnAssetModification at runtime only.
	TOptional<bool> bAutomaticallyCheckoutOnAssetModificationOverride;
};
