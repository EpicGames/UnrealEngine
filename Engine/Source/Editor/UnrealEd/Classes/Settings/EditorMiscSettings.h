// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EditorMiscSettings.generated.h"


/**
 * Implements the miscellaneous Editor settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UEditorMiscSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

};
