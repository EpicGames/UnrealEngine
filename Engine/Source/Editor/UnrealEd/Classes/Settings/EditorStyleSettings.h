// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Styling/SlateBrush.h"
#include "Rendering/RenderingCommon.h"
#include "EditorStyleSettings.generated.h"

UENUM(BlueprintType)
enum class EAssetEditorOpenLocation : uint8
{
	/** Attempts to dock asset editors into either a new window, or the main window if they were docked there. */
	Default,
	/** Docks tabs into new windows. */
	NewWindow,
	/** Docks tabs into the main window. */
	MainWindow,
	/** Docks tabs into the content browser's window. */
	ContentBrowser,
	/** Docks tabs into the last window that was docked into, or a new window if there is no last docked window. */
	LastDockedWindowOrNewWindow,
	/** Docks tabs into the last window that was docked into, or the main window if there is no last docked window. */
	LastDockedWindowOrMainWindow,
	/** Docks tabs into the last window that was docked into, or the content browser window if there is no last docked window. */
	LastDockedWindowOrContentBrowser
};

/**
 * Implements the Editor style settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UEditorStyleSettings : public UObject
{
public:

	GENERATED_UCLASS_BODY()

	UNREALED_API void Init();
public:
	/**
	 * Enables high dpi support in the editor which will adjust the scale of elements in the UI to account for high DPI monitors
	 * The editor must be restarted for changes to take effect.
	 */
	UPROPERTY(EditAnywhere, Category=UserInterface, meta = (ConfigRestartRequired = true, DisplayName="Enable High DPI Support"))
	bool bEnableHighDPIAwareness;
	
	/** 
	 * Scales the entire editor interface up or down. 
	 */
	UPROPERTY(EditAnywhere, Config, Category=UserInterface, meta=(ClampMin=0.5, ClampMax=3.0))
	float ApplicationScale = 1.0f;

	/**
	 * Whether to enable the Editor UI Layout configuration tools for the user.
	 * If disabled, the "Save Layout As" and "Remove Layout" menus will be removed, as well as the "Import Layout..." sub-menu.
	 */
	UPROPERTY(EditAnywhere, config, Category = UserInterface)
	bool bEnableUserEditorLayoutManagement;

	/** Applies a color vision deficiency filter to the entire editor */
	UPROPERTY(EditAnywhere, config, Category = "Accessibility")
	EColorVisionDeficiency ColorVisionDeficiencyPreviewType;

	UPROPERTY(EditAnywhere, config, Category = "Accessibility", meta=(ClampMin=0, ClampMax=10))
	int32 ColorVisionDeficiencySeverity;

	/** Shifts the color spectrum to the visible range based on the current ColorVisionDeficiencyPreviewType */
	UPROPERTY(EditAnywhere, config, Category = "Accessibility")
	bool bColorVisionDeficiencyCorrection;

	/** If you're correcting the color deficiency, you can use this to visualize what the correction looks like with the deficiency. */
	UPROPERTY(EditAnywhere, config, Category = "Accessibility")
	bool bColorVisionDeficiencyCorrectionPreviewWithDeficiency;

	/** The color used for the X axis */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ConfigRestartRequired = true))
	FLinearColor XAxisColor;

	/** The color used for the Y axis */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ConfigRestartRequired = true))
	FLinearColor YAxisColor;

	/** The color used for the Z axis */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ConfigRestartRequired = true))
	FLinearColor ZAxisColor;

	/** The color used to represent selection */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta=(DisplayName="Viewport Selection Color"))
	FLinearColor SelectionColor;

	/** Additional colors used for selections with extra meaning */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta=(DisplayName="Additional Viewport Selection Colors"))
	FLinearColor AdditionalSelectionColors[6];

	/** Size (in pixels) of the dithering pattern applied to occluded primitives. */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ClampMin = 0.25, UIMin = 0.25))
	float OccludedDitheringPatternSize = 2.0f;

	/** The strength of the dithering applied to occluded primitives. */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ClampMin = 0.0, UIMin = 0.0, ClampMax = 1.0, UIMax = 1.0, DisplayName = "Occluded Pattern Opacity"))
	float OccludedDithering = 0.8f;

	/** The brightness of occluded primitives. */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (ClampMin = 0.0, UIMin = 0.0, ClampMax = 1.0, UIMax = 1.0, DisplayName = "Occluded Primitive Brightness"))
	float OccludedBrightness = 0.7f;

	/** The color used for overlay tools inside of the viewport, like the measure tool */
	UPROPERTY(EditAnywhere, config, Category = UserInterface)
	FLinearColor ViewportToolOverlayColor;

	UPROPERTY(config)
	bool bEnableEditorWindowBackgroundColor;

	/** The color used to tint the editor window backgrounds */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta=(EditCondition="bEnableEditorWindowBackgroundColor"))
	FLinearColor EditorWindowBackgroundColor;

	/** Whether to use small toolbar icons without labels or not. */
	UPROPERTY(EditAnywhere, config, Category=UserInterface)
	uint32 bUseSmallToolBarIcons:1;

	/** Menus longer than this threshold show their search field by default. Use 0 to always show, or a high number to always hide. When a searchable menu is open but the field is hidden, you can still start a search by typing. */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta = (UIMin="0", UIMax="100"))
	uint32 MenuSearchFieldVisibilityThreshold = 10;

	/** If true the material editor and blueprint editor will show a grid on it's background. */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Use Grids In The Material And Blueprint Editor"))
	uint32 bUseGrid : 1;

	/** Whether grid lines should be anti-aliased */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Grid Anti-Alias"))
	uint32 bAntiAliasGrid : 1;

	/** The color used to represent regular grid lines */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Grid Regular Color"))
	FLinearColor RegularColor;

	/** The color used to represent ruler lines in the grid */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Grid Ruler Color"))
	FLinearColor RuleColor;

	/** The color used to represent the center lines in the grid */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Grid Center Color"))
	FLinearColor CenterColor;

	/** The custom grid snap size to use  */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (ClampMin = "1.0", ClampMax = "100.0", UIMin = "1.0", UIMax = "100.0"))
	uint32 GridSnapSize;

	/** Optional brush used for graph backgrounds */
	UPROPERTY(EditAnywhere, config, Category = Graphs, meta = (DisplayName = "Background Brush"))
	FSlateBrush GraphBackgroundBrush;

	/** When enabled, the C++ names for properties and functions will be displayed in a format that is easier to read */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, meta=(DisplayName="Show Friendly Variable Names"))
	uint32 bShowFriendlyNames:1;

	/** When enabled, the underlying Names for Components inherited from C++ will be shown alongside their UProperty Variable name */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (DisplayName = "Show Underlying Names For Native Components"))
	uint32 bShowNativeComponentNames:1;

	/** When enabled, the Editor Preferences and Project Settings menu items in the main menu will be expanded with sub-menus for each settings section. */
	UPROPERTY(EditAnywhere, config, Category=UserInterface, AdvancedDisplay, meta = (ConfigRestartRequired = true))
	uint32 bExpandConfigurationMenus:1;

	/** When enabled, the project subsection of the File menu will be shown. */
	UPROPERTY(config)
	uint32 bShowProjectMenus : 1;

	/** When enabled, the Launch menu items will be shown. */
	UPROPERTY(config)
	uint32 bShowLaunchMenus : 1;

	/** When enabled, the Advanced Details will always auto expand. */
	UPROPERTY(config)
	uint32 bShowAllAdvancedDetails : 1;

	/** When Playing or Simulating, shows all properties (even non-visible and non-editable properties), if the object belongs to a simulating world.  This is useful for debugging. */
	UPROPERTY(config)
	uint32 bShowHiddenPropertiesWhilePlaying : 1;

	/** New asset editor tabs will open at the specified location. */
	UPROPERTY(EditAnywhere, config, Category=UserInterface)
	EAssetEditorOpenLocation AssetEditorOpenLocation;

	/** Should editor tabs be colorized according to the asset type */
	UPROPERTY(EditAnywhere, config, Category=UserInterface)
	uint32 bEnableColorizedEditorTabs : 1;

	UPROPERTY(config)
	FGuid CurrentAppliedTheme; 

	/** If true MiddleEllipsis will be used by certain editor widget (ex: Tab Title/Outliner Actor Names), otherwise the default policy will be used instead */
	UPROPERTY(EditAnywhere, config, Category=Text, meta = (ConfigRestartRequired = true))
	bool bEnableMiddleEllipsis = true;

	/** Scale the size of the global asset picker */
	UPROPERTY(EditAnywhere, config, Category = UserInterface, meta = (DisplayName = "Scale Asset Picker Widget Size", ClampMin = "0.75", ClampMax = "3.0", UIMin = "0.75", UIMax = "3.0"))
	float AssetPickerSizeScale = 1.0f;

public:

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UEditorStyleSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged( ) { return SettingChangedEvent; }

	/** @return A subdued version of the users selection color (for use with inactive selection)*/
	UNREALED_API FLinearColor GetSubduedSelectionColor() const;
	
	UNREALED_API bool OnImportBegin(const FString& ImportFromPath);
	UNREALED_API bool OnExportBegin(const FString& ExportToPath);

protected:

	// UObject overrides
	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;

private:
	void OnPropertyChanged(FName InPropertyName);

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;
};
