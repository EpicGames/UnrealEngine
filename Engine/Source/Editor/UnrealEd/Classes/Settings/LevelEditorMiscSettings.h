// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "LevelEditorMiscSettings.generated.h"

class ULevelStreaming;

/**
 * Configure miscellaneous settings for the Level Editor.
 */
UCLASS(config=EditorPerProjectUserSettings, meta=( DisplayName="Miscellaneous" ), MinimalAPI)
class ULevelEditorMiscSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	/** If checked lighting will be automatically applied to the level after a static lighting build is complete.
	 * Otherwise you will be prompted to apply it
	 */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=(DisplayName="Apply Lighting Automatically"))
	uint32 bAutoApplyLightingEnable:1;

	/** If true, BSP will auto-update */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Update BSP Automatically" ))
	uint32 bBSPAutoUpdate:1;

	/** If true, the pivot offset for BSP will be automatically moved to stay centered on its vertices */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Move BSP Pivot Offset Automatically" ))
	uint32 bAutoMoveBSPPivotOffset:1;

	/** If true, Navigation will auto-update */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Update Navigation Automatically" ))
	uint32 bNavigationAutoUpdate:1;

	/** If enabled, the copy properties option will be enabled by default when using the replace actors context menu. */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Copy Properties on Replace" ))
	uint32 bReplaceCopyPropertiesDefault:1;
	
	/** If enabled, replacing actors allows copying properties when related by the native parent of the replaced class. */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Copy Properties when Replacing with Different Blueprint" ))
	uint32 bReplaceCopiesPropertiesWhenRelatedByNativeParent:1;

	/** If enabled, replacing actors will respect the scale of the original actor.  Otherwise, the replaced actors will have a scale of 1.0 */
	UPROPERTY(EditAnywhere, config, Category=Editing, meta=( DisplayName = "Preserve Actor Scale on Replace" ))
	uint32 bReplaceRespectsScale:1;

	/** If enabled, will avoid relabeling actors in UUnrealEdEngine::edactPasteSelected */
	UPROPERTY(EditAnywhere, config, Category = Editing, meta = (DisplayName = "Avoid Actor Relabel on Paste Selected"))
	uint32 bAvoidRelabelOnPasteSelected:1;

	/** If enabled, will force checking references during a delete. Note: disabling this is for advanced usage only and not recommended.*/
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category = Editing, meta = (DisplayName = "Check References on Delete"))
	uint32 bCheckReferencesOnDelete:1;

public:
	/** If checked audio playing in the editor will continue to play even if the editor is in the background */
	UPROPERTY(EditAnywhere, config, Category=Sound)
	uint32 bAllowBackgroundAudio:1;

	/** If checked, will create a new audio engine instance for the play-in-editor window. Otherwise, will re-use the audio device used in content browser. */
	UPROPERTY(EditAnywhere, config, Category = Sound)
	uint32 bCreateNewAudioDeviceForPlayInEditor: 1;

	/** If true audio will be enabled in the editor. Does not affect PIE **/
	UPROPERTY(config)
	uint32 bEnableRealTimeAudio:1;

	/** Global volume setting for the editor */
	UPROPERTY(config)
	float EditorVolumeLevel;

	/** Enables audio feedback for certain operations in Unreal Editor, such as entering and exiting Play mode */
	UPROPERTY(EditAnywhere, config, Category=Sound)
	uint32 bEnableEditorSounds:1;

public:

	/** The default level streaming class to use when adding new streaming levels */
	UPROPERTY(EditAnywhere, config, Category=Levels)
	TSubclassOf<ULevelStreaming> DefaultLevelStreamingClass;

	UPROPERTY(EditAnywhere, config, Category = Levels)
	bool bPromptWhenAddingToLevelBeforeCheckout;

	UPROPERTY(EditAnywhere, config, Category = Levels)
	bool bPromptWhenAddingToLevelOutsideBounds;

	UPROPERTY(EditAnywhere, config, Category = Levels, Meta = (ClampMin = "0.0", ClampMax = "100.0"))
	float PercentageThresholdForPrompt;

	UPROPERTY(EditAnywhere, config, Category = Levels)
	FVector MinimumBoundsForCheckingSize;

public:

	/** The save directory for newly created screenshots */
	UPROPERTY(EditAnywhere, config, Category = Screenshots)
	FDirectoryPath EditorScreenshotSaveDirectory;

protected:

	// UObject overrides

	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;
};
