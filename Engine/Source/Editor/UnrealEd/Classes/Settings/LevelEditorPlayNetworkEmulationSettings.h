// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboBox.h"
#include "LevelEditorPlayNetworkEmulationSettings.generated.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class SWidget;
struct FPacketSimulationSettings;

USTRUCT()
struct FNetworkEmulationPacketSettings
{
	GENERATED_USTRUCT_BODY()

	// Minimum latency to add to packets
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Minimum Latency", ClampMin = "0", ClampMax = "5000"))
	int32 MinLatency = 0;

	// Maximum latency to add to packets. We use a random value between the minimum and maximum (when 0 = always the minimum value)
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Maximum Latency", ClampMin = "0", ClampMax = "5000"))
	int32 MaxLatency = 0;

	// Ratio of packets to randomly drop (0 = none, 100 = all)
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (ClampMin = "0", ClampMax = "100"))
	int32 PacketLossPercentage = 0;
};

UENUM()
enum class NetworkEmulationTarget
{
	Server UMETA(DisplayName = "Server Only"),
	Client UMETA(DisplayName = "Clients Only"),
	Any UMETA(DisplayName = "Everyone"),
};

USTRUCT()
struct FLevelEditorPlayNetworkEmulationSettings
{
	GENERATED_USTRUCT_BODY()

	// When true will apply the emulation settings when launching the game
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Enable Network Emulation"))
	bool bIsNetworkEmulationEnabled = false;

	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Emulation Target"))
	NetworkEmulationTarget EmulationTarget = NetworkEmulationTarget::Server;

	// The profile name of the settings currently applied
	UPROPERTY(EditAnywhere, Category = "Network Settings")
	FString CurrentProfile;

	// Settings that add latency and packet loss to all outgoing packets
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Outgoing traffic"))
	FNetworkEmulationPacketSettings OutPackets;

	// Settings that add latency and packet loss to all incoming packets
	UPROPERTY(EditAnywhere, Category = "Network Settings", meta = (DisplayName = "Incoming traffic"))
	FNetworkEmulationPacketSettings InPackets;

	bool IsCustomProfile() const;

	UNREALED_API bool IsEmulationEnabledForTarget(NetworkEmulationTarget CurrentTarget) const;

	/** Convert the PIE settings into NetDriver settings */
	UNREALED_API void ConvertToNetDriverSettings(FPacketSimulationSettings& OutNetDriverSettings) const;

	/** Convert the PIE settings into CmdLine settings*/
	UNREALED_API FString BuildPacketSettingsForCmdLine() const;

	/** Convert the PIE settings into URL settings*/
	UNREALED_API FString BuildPacketSettingsForURL() const;

	void OnPostInitProperties();
	void OnPostEditChange(struct FPropertyChangedEvent& PropertyChangedEvent);
};


/** Details customization for FEditorPlayNetworkEmulationSettings */
class FLevelEditorPlayNetworkEmulationSettingsDetail : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	FText GetSelectedNetworkEmulationProfile() const;

	// Combo button handlers
	TSharedRef<SWidget> OnGetProfilesMenuContent() const;
	void OnEmulationProfileChanged(int32 Index) const;

	bool HandleAreSettingsEnabled() const;

	/*
	 * Checks to see if separate server was launched, the number of player clients
	 *	is greater than one, or if the current net play mode is not standalone
	 */
	bool IsNetworkEmulationAvailable() const;

private:

	// Property Handles
	TSharedPtr<IPropertyHandle>	NetworkEmulationSettingsHandle;
	TSharedPtr<IPropertyHandle>	IsNetworkEmulationEnabledHandle;
	TSharedPtr<IPropertyHandle> CurrentProfileHandle;
	TSharedPtr<IPropertyHandle> OutPacketsHandle;
	TSharedPtr<IPropertyHandle> InPacketsHandle;
};
