// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/App.h"
#include "Settings/LevelEditorPlayNetworkEmulationSettings.h"
#include "ToolMenuContext.h"
#include "GenericPlatform/InputDeviceMappingPolicy.h"
#include "LevelEditorPlaySettings.generated.h"

class SWindow;
class UToolMenu;

/**
 * Enumerates label anchor modes.
 */
UENUM()
enum ELabelAnchorMode : int
{
	LabelAnchorMode_TopLeft UMETA(DisplayName="Top Left"),
	LabelAnchorMode_TopCenter UMETA(DisplayName="Top Center"),
	LabelAnchorMode_TopRight UMETA(DisplayName="Top Right"),
	LabelAnchorMode_CenterLeft UMETA(DisplayName="Center Left"),
	LabelAnchorMode_Centered UMETA(DisplayName="Centered"),
	LabelAnchorMode_CenterRight UMETA(DisplayName="Center Right"),
	LabelAnchorMode_BottomLeft UMETA(DisplayName="Bottom Left"),
	LabelAnchorMode_BottomCenter UMETA(DisplayName="Bottom Center"),
	LabelAnchorMode_BottomRight UMETA(DisplayName="Bottom Right")
};


UENUM()
enum ELaunchModeType : int
{
	/** Runs the map on a specified device. */
	LaunchMode_OnDevice,
};


UENUM()
enum EPlayModeLocations : int
{
	/** Spawns the player at the current camera location. */
	PlayLocation_CurrentCameraLocation,

	/** Spawns the player from the default player start. */
	PlayLocation_DefaultPlayerStart,
};


UENUM()
enum EPlayModeType : int
{
	/** Runs from within the editor. */
	PlayMode_InViewPort = 0,

	/** Runs in a new window. */
	PlayMode_InEditorFloating,

	/** Runs a mobile preview in a new process. */
	PlayMode_InMobilePreview,

	/** Runs a mobile preview targeted to a particular device in a new process. */
	PlayMode_InTargetedMobilePreview UE_DEPRECATED(5.8, "PlayMode_InTargetedMobilePreview is no longer supported, please use PlayMode_InMobilePreview instead"),

	/** Runs a vulkan preview in a new process. */
	PlayMode_InVulkanPreview UE_DEPRECATED(5.7, "PlayMode_InVulkanPreview is no longer supported, please use PlayMode_InMobilePreview instead"),

	/** Runs in a new process. */
	PlayMode_InNewProcess,

	/** Runs in VR. */
	PlayMode_InVR,

	/** Simulates in viewport without possessing the player. */
	PlayMode_Simulate,

	/** Runs the last launched device (from Platforms menu) */
	PlayMode_QuickLaunch,

	/** The number of different Play Modes. */
	PlayMode_Count,
};


UENUM()
enum EPlayNetMode : int
{
	/** A standalone game will be started. This will not create a dedicated server, nor automatically connect to one. A server can be launched by enabling bLaunchSeparateServer if you need to test offline -> server connection flow for your game. */
	PIE_Standalone UMETA(DisplayName="Play Standalone"),
	/** The editor will act as both a Server and a Client. Additional instances may be opened beyond that depending on the number of clients. */
	PIE_ListenServer UMETA(DisplayName="Play As Listen Server"),
	/** The editor will act as a Client. A server will be started for you behind the scenes to connect to. */
	PIE_Client UMETA(DisplayName="Play As Client"),
};


/**
 * Determines whether to build the executable when launching on device. Note the equivalence between these settings and EProjectPackagingBuild.
 */
UENUM()
enum EPlayOnBuildMode : int
{
	/** Always build. */
	PlayOnBuild_Always UMETA(DisplayName="Always"),

	/** Never build. */
	PlayOnBuild_Never UMETA(DisplayName="Never"),

	/** Build based on project type. */
	PlayOnBuild_Default UMETA(DisplayName="If project has code, or running a locally built editor"),

	/** Build if we're using a locally built (ie. non-promoted) editor. */
	PlayOnBuild_IfEditorBuiltLocally UMETA(DisplayName="If running a locally built editor"),
};

/* Configuration to use when launching on device. */
UENUM()
enum EPlayOnLaunchConfiguration : int
{
	/** Launch on device with the same build configuration as the editor. */
	LaunchConfig_Default UMETA(DisplayName = "Same as Editor"),
	/** Launch on device with a Debug build configuration. */
	LaunchConfig_Debug UMETA(DisplayName = "Debug"),
	/** Launch on device with a Development build configuration. */
	LaunchConfig_Development UMETA(DisplayName = "Development"),
	/** Launch on device with a Test build configuration. */
	LaunchConfig_Test UMETA(DisplayName = "Test"),
	/** Launch on device with a Shipping build configuration. */
	LaunchConfig_Shipping UMETA(DisplayName = "Shipping"),
};

/* Whether to content should be stored in pak files when launching on device. */
UENUM()
enum class EPlayOnPakFileMode : uint8
{
	/** Do not pack files. */
	NoPak UMETA(DisplayName="Use loose files"),
	
	/** Pack files with UnrealPak. */
	PakNoCompress UMETA(DisplayName="Use pak files without compression"),
	
	/** Compress and pack files with UnrealPak. */
	PakCompress UMETA(DisplayName="Use compressed pak files"),
};

/**
 * Holds information about a screen resolution to be used for playing.
 */
USTRUCT()
struct FPlayScreenResolution
{
	GENERATED_USTRUCT_BODY()

public:

	/** The description text for this screen resolution. */
	UPROPERTY(config)
	/*FText*/FString Description;

	/** The screen resolution's width (in pixels). */
	UPROPERTY(config)
	int32 Width = 1920;

	/** The screen resolution's height (in pixels). */
	UPROPERTY(config)
	int32 Height = 1080;

	/** The screen resolution's aspect ratio (as a string). */
	UPROPERTY(config)
	FString AspectRatio;

	/** Whether or not this device supports both landscape and portrait modes */
	UPROPERTY(config)
	bool bCanSwapAspectRatio = true;
	
	/** The name of the device profile this links to */
	UPROPERTY(config)
	FString ProfileName;

	UPROPERTY(transient)
	float ScaleFactor = 1.0f;

	UPROPERTY(transient)
	int32 LogicalHeight = 1080;
	
	UPROPERTY(transient)
	int32 LogicalWidth = 1920;

	void PostInitProperties();
};

UCLASS(MinimalAPI)
class UCommonResolutionMenuContext
	: public UToolMenuContextBase
{
	GENERATED_BODY()

public:
	DECLARE_DELEGATE_RetVal_OneParam(FUIAction, FGetUIActionFromLevelPlaySettings, const FPlayScreenResolution&);
	FGetUIActionFromLevelPlaySettings GetUIActionFromLevelPlaySettings;
};

/**
 * Implements the Editor's play settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class ULevelEditorPlaySettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** The PlayerStart class used when spawning the player at the current camera location. */
	UPROPERTY(globalconfig)
	FString PlayFromHerePlayerStartClassName;

public:

	/** Should Play-in-Editor automatically give mouse control to the game on PIE start (default = false). Note that this does not affect VR, which will always take focus */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta=(ToolTip="Give the game mouse control when PIE starts or require a click in the viewport first"))
	bool GameGetsMouseControl;

	/** While using the game viewport, it sends mouse movement and clicks as touch events, instead of as mouse events. */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor)
	bool UseMouseForTouch;

protected:
	
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta = (InlineEditConditionToggle))
	bool bSimulateDeviceMappingPolicy = false;
	
	/**
	 * If enabled, then this simulated device mapping policy will be used instead of the one set in the Project Settings.
	 * Simulating this device mapping policy may be useful to test different controller connection and login flows
	 * in the editor which would otherwise require you to deploy to a console to test.
	 */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta=(EditCondition=bSimulateDeviceMappingPolicy))
	EInputDeviceMappingPolicy SimulatedDeviceMappingPolicy = EInputDeviceMappingPolicy::UseManagedPlatformLogin;

	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (ConfigRestartRequired = true))
	FString PreferredInputAPIOverride;

public:

	bool IsSimulateDeviceMappingPolicy() const { return bSimulateDeviceMappingPolicy; }

	EInputDeviceMappingPolicy GetSimulatedDeviceMappingPolicy() const { return SimulatedDeviceMappingPolicy; }

	TMulticastDelegate<void()> SimulatedDeviceMappingPolicyPostEditChange;
	
	/** Whether to show a label for mouse control gestures in the PIE view. */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor)
	bool ShowMouseControlLabel;

	/** Location on screen to anchor the mouse control label when in PIE mode. */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor)
	TEnumAsByte<ELabelAnchorMode> MouseControlLabelPosition;

	/** Should Play-in-Viewport respect HMD orientations (default = false) */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta=(ToolTip="Whether or not HMD orientation should be used when playing in viewport"))
	bool ViewportGetsHMDControl;

	/** Should we minimize the editor when VR PIE is clicked (default = true) */
	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (ToolTip = "Whether or not the editor is minimized on VR PIE"))
	bool ShouldMinimizeEditorOnVRPIE;

	/** Should we minimize the editor when non-VR PIE is clicked (default = false) */
	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (ToolTip = "Whether or not the editor is minimized on non-VR PIE"))
	bool bShouldMinimizeEditorOnNonVRPIE;

	/** Whether we should emulate stereo (helps checking VR rendering issues). */
	UPROPERTY(config, EditAnywhere, Category = PlayInStandaloneGame)
	bool bEmulateStereo;

	/** Whether to automatically recompile blueprints on PIE */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta=(ToolTip="Automatically recompile blueprints used by the current level when initiating a Play In Editor session"))
	bool AutoRecompileBlueprints;

	/** Whether to play sounds during PIE */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor, meta=(ToolTip="Whether to play sounds when in a Play In Editor session"))
	bool EnableGameSound;

	/** Whether to automatically solo audio in first PIE client. */
	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (ToolTip="Whether to automatically solo audio in first PIE client", EditCondition="EnableGameSound", DisplayName = "Solo Audio in First PIE Client"))
	bool SoloAudioInFirstPIEClient;
	
	/** Whether to enable the editor's keyboard shortcuts in PIE. */
	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (ToolTip="Enable editor keyboard shortcuts when PIE is active", DisplayName = "Enable editor keyboard shortcuts"))
	bool EnableEditorShortcutsInPIE;

	/** Whether to play a sound when entering and exiting PIE */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = PlayInEditor, meta = (DisplayName = "Enable PIE Enter and Exit Sounds"))
	bool EnablePIEEnterAndExitSounds;

	/** Which quality level to use when playing in editor */
	UPROPERTY(config, EditAnywhere, Category=PlayInEditor)
	int32 PlayInEditorSoundQualityLevel;

	/** Whether to use a non-realtime audio device during PIE */
	UPROPERTY(config)
	bool bUseNonRealtimeAudioDevice;

	/** True if Play In Editor should only load currently-visible levels in PIE. */
	UPROPERTY(config)
	uint32 bOnlyLoadVisibleLevelsInPIE:1;

	UPROPERTY(config, EditAnywhere, Category = PlayInEditor, meta = (DisplayName="Stream Sub-Levels during Play in Editor", ToolTip="Prefer to stream sub-levels from the disk instead of duplicating editor sub-levels"))
	uint32 bPreferToStreamLevelsInPIE:1;

	/** Should warnings and errors in the Output Log during "Play in Editor" be promoted to the message log? */
	UPROPERTY(EditAnywhere, config, Category = PlayInEditor)
	bool bPromoteOutputLogWarningsDuringPIE;

public:
	/** The width of the new view port window in pixels (0 = use the desktop's screen resolution). */
	UPROPERTY(config, EditAnywhere, Category=GameViewportSettings, meta=(ClampMin=0))
	int32 NewWindowWidth;

	/** The height of the new view port window in pixels (0 = use the desktop's screen resolution). */
	UPROPERTY(config, EditAnywhere, Category=GameViewportSettings, meta=(ClampMin=0))
	int32 NewWindowHeight;

	/** The position of the new view port window on the screen in pixels. */
	UPROPERTY(config, EditAnywhere, Category = GameViewportSettings)
	FIntPoint NewWindowPosition;

	/** Whether the new window should be centered on the screen. */
	UPROPERTY(config, EditAnywhere, Category = GameViewportSettings)
	uint32 CenterNewWindow:1;

public:

	/** Whether to always have the PIE window on top of the parent windows. */
	UPROPERTY(config, EditAnywhere, Category = PlayInNewWindow, meta = (ToolTip="Always have the PIE window on top of the parent windows."))
	uint32 PIEAlwaysOnTop:1;

public:

	/** Whether sound should be disabled when playing standalone games. */
	UPROPERTY(config, EditAnywhere, Category=PlayInStandaloneGame)
	uint32 DisableStandaloneSound:1;

	/** Extra parameters to be include as part of the command line for the standalone game. */
	UPROPERTY(config, EditAnywhere, Category=PlayInStandaloneGame)
	FString AdditionalLaunchParameters;

public:

	/** Whether to build the game before launching on device. */
	UPROPERTY(config, EditAnywhere, Category = PlayOnDevice)
	TEnumAsByte<EPlayOnBuildMode> BuildGameBeforeLaunch;

	/* Which build configuration to use when launching on device. */
	UPROPERTY(config, EditAnywhere, Category = PlayOnDevice)
	TEnumAsByte<EPlayOnLaunchConfiguration> LaunchConfiguration;

	// Whether to content should be stored in pak files when launching on device. */
	UPROPERTY(config, EditAnywhere, Category = PlayOnDevice)
	EPlayOnPakFileMode PackFilesForLaunch;

	/** Whether to automatically recompile dirty Blueprints before launching */
	UPROPERTY(config, EditAnywhere, Category=PlayOnDevice)
	bool bAutoCompileBlueprintsOnLaunch;
	
	/**
	* This is a rarely used option that will launch a separate server (possibly hidden in-process depending on RunUnderOneProcess) 
	* even if the net mode does not require a server (such as Standalone). If the net mode requires a server (such as Client) a 
	* server will be launched for you (regardless of this setting). This allows you to test offline -> server workflows by connecting
	* ("open 127.0.0.1:<ServerPort>") from the offline game.
	*/
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options")
	bool bLaunchSeparateServer;

private:

	/** NetMode to use for Play In Editor. */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options")
	TEnumAsByte<EPlayNetMode> PlayNetMode;

	/** Spawn multiple player windows in a single instance of UE. This will load much faster, but has potential to have more issues.  */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options")
	bool RunUnderOneProcess;

	/** The number of client windows to open. The first one to open will respect the Play In Editor "Modes" option (PIE, PINW), additional clients respect the RunUnderOneProcess setting. */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options|Client", meta=(ClampMin = "1", ClampMax = "10", UIMin = "1", UIMax = "10"))
	int32 PlayNumberOfClients;

	/** In multiplayer PIE which client will be the 'primary'. (default = 0, the first client)*/
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Client", meta = (ClampMin = "-1", UIMin = "-1", UIMax = "9", ToolTip = "In multiplayer PIE which client will be the 'primary'. Considered most important and given a larger client window, access to unique hardware like a VirtualReality HMD, etc. Intended to help test issues that affect the second, etc client.  0 is the first client. If the setting is >= than the number of clients the last will be primary. -1 will result in no primary.  Note that this is an index only of PIE instance windows, in netmode 'Play as Client' pie instance zero is a windowless dedicated server, so setting 0 here would make the fist pie window the primary which would be PIEInstance 1, rather than 0 as in other netmodes.", DisplayName = "Primary PIE Client Index"))
	int PrimaryPIEClientIndex = 0;

	/** What port used by the server for simple networking */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Server", meta=(ClampMin="1", UIMin="1", ClampMax="65535", EditCondition = "PlayNetMode != EPlayNetMode::PIE_Standalone || bLaunchSeparateServer"))
	uint16 ServerPort;

	/** Width to use when spawning additional windows. */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options|Client", meta=(ClampMin=0))
	int32 ClientWindowWidth;

	/**
	 * When running multiple player windows in a single process, this option determines how the game pad input gets routed.
	 *
	 * If unchecked (default) then the 1st game pad is attached to the 1st window, 2nd to the 2nd window, and so on.
	 *
	 * If it is checked, the 1st game pad goes the 2nd window. The 1st window can then be controlled by keyboard/mouse, which is convenient if two people are testing on the same computer.
	 */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options|Client", meta=(EditCondition = "RunUnderOneProcess"))
	bool RouteGamepadToSecondWindow;

	/** 
	* If checked, a separate audio device is created for every player. 
	
	* If unchecked, a separate audio device is created for only the first two players and uses the main audio device for more than 2 players.
	*
	* Enabling this will allow rendering accurate audio from every player's perspective but will use more CPU. Keep this disabled on lower-perf machines.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Client", meta=(EditCondition = "EnableGameSound && RunUnderOneProcess"))
	bool CreateAudioDeviceForEveryPlayer;

	/** Height to use when spawning additional windows. */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options|Client", meta=(ClampMin = 0))
	int32 ClientWindowHeight;

	/** Override the map launched by the dedicated server (currently only used when in PIE_StandaloneWithServer net mode) */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Server", meta=(EditCondition = "PlayNetMode != EPlayNetMode::PIE_Standalone || bLaunchSeparateServer"))
	FString ServerMapNameOverride;

	/** Additional options that will be passed to the server as URL parameters, in the format ?bIsLanMatch=1?listen - any additional command line switches should be passed in the Additional Server Launch Parameters field below. */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options|Server", meta=(EditCondition = "PlayNetMode != EPlayNetMode::PIE_Standalone || bLaunchSeparateServer"))
	FString AdditionalServerGameOptions;

	/** Controls the default value of the show flag ServerDrawDebug */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options")
	bool bShowServerDebugDrawingByDefault;

	/** How strongly debug drawing originating from the server will be biased towards the tint color */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options", meta=(ClampMin=0, ClampMax=1, UIMin=0, UIMax=1))
	float ServerDebugDrawingColorTintStrength;

	/** Debug drawing originating from the server will be biased towards this color */
	UPROPERTY(config, EditAnywhere, Category="Multiplayer Options")
	FLinearColor ServerDebugDrawingColorTint;

	/** 
	* When True each PIE process is launched with "-HMDSimulator" argument.  
	* The usefullness of this will vary by XR platform.  
	* The PIE instances may get special -HMDSimulator behavior from an XR plugin, they may successfully make connections to the HMD hardware, their attempt to connect to hardware may be rejected by the runtime.
	*/
	UE_DEPRECATED(5.5, "Deprecated. Use bHMDForPrimaryProcessOnly instead")
	UPROPERTY(config)
	bool bOneHeadsetEachProcess_DEPRECATED;

	/** 
	* When running multiple player windows in multiple processes, this option determines how HMD(s)/Simulator(s) is/are attached to processes.
	* 
	* If checked (default), there is only one HMD/Simulator and it is only attached to the primary process.
	* 
	* If unchecked, there could be multiple HMDs/Simulators if HMD/XRSimulator supports it, and each process could have its own attached HMD or XRSimulator, or all processes could share the same HMD by switching the HMD to the active process dynamically.
	*/
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options", meta = (DisplayName = "HMD For Primary Process Only", EditCondition = "!RunUnderOneProcess"))
	bool bHMDForPrimaryProcessOnly;

private:
	UNREALED_API void PushDebugDrawingSettings();
	
	UNREALED_API void ClampPIEClientValues();

public:
	bool ShowServerDebugDrawingByDefault() const
	{
		return bShowServerDebugDrawingByDefault;
	}

	/** Additional options that will be passed to the server as arguments, for example -debug. Only works with separate process servers. */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Server")
	FString AdditionalServerLaunchParameters;

	/** If > 0, Tick dedicated server at a fixed frame rate. Does not impact Listen Server (use ClientFixedFPS setting). This is the target frame rate, e.g, "20" for 20fps, which will result in 1/20 second tick steps. */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Server", meta=(EditCondition = "PlayNetMode == EPlayNetMode::PIE_Client && RunUnderOneProcess"))
	int32 ServerFixedFPS;

	/** If > 0, Tick clients at a fixed frame rate. Each client instance will map to an element in the list, wrapping around if num clients exceeds size of list. Includes Listen Server. This is the target frame rate, e.g, "20" for 20fps, which will result in 1/20 second tick steps. */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options|Client", meta=(EditCondition = "PlayNetMode != EPlayNetMode::PIE_Standalone && RunUnderOneProcess"))
	TArray<int32> ClientFixedFPS;

public:

	/** Should network emulation settings be applied or not */
	bool IsNetworkEmulationEnabled() const
	{
		return NetworkEmulationSettings.bIsNetworkEmulationEnabled;
	}

	/**
	 * Customizable settings allowing to emulate latency and packetloss for game network transmissions
	 */
	UPROPERTY(config, EditAnywhere, Category = "Multiplayer Options" )
	FLevelEditorPlayNetworkEmulationSettings NetworkEmulationSettings;

public:

	// Accessors for fetching the values of multiplayer options, and returning whether the option is valid at this time
	void SetPlayNetMode( const EPlayNetMode InPlayNetMode ) { PlayNetMode = InPlayNetMode; }
	bool IsPlayNetModeActive() const { return true; }
	bool GetPlayNetMode( EPlayNetMode &OutPlayNetMode ) const { OutPlayNetMode = PlayNetMode; return IsPlayNetModeActive(); }
	EVisibility GetPlayNetModeVisibility() const { return (RunUnderOneProcess ? EVisibility::Hidden : EVisibility::Visible); }

	void SetRunUnderOneProcess( const bool InRunUnderOneProcess ) { RunUnderOneProcess = InRunUnderOneProcess; }
	bool IsRunUnderOneProcessActive() const { return true; }
	bool GetRunUnderOneProcess( bool &OutRunUnderOneProcess ) const { OutRunUnderOneProcess = RunUnderOneProcess; return IsRunUnderOneProcessActive(); }
	
	void SetPlayNumberOfClients( const int32 InPlayNumberOfClients )
	{
		PlayNumberOfClients = InPlayNumberOfClients;
		ClampPIEClientValues();
	}
	bool IsPlayNumberOfClientsActive() const { return true; }
	bool GetPlayNumberOfClients( int32 &OutPlayNumberOfClients ) const { OutPlayNumberOfClients = PlayNumberOfClients; return IsPlayNumberOfClientsActive(); }

	int GetPrimaryPIEClientIndex() const { return PrimaryPIEClientIndex; }

	void SetServerPort(const uint16 InServerPort) { ServerPort = InServerPort; }
	bool IsServerPortActive() const { return (PlayNetMode != PIE_Standalone) || RunUnderOneProcess; }
	bool GetServerPort(uint16 &OutServerPort) const { OutServerPort = ServerPort; return IsServerPortActive(); }
	
	bool IsRouteGamepadToSecondWindowActive() const { return PlayNumberOfClients > 1; }
	bool GetRouteGamepadToSecondWindow( bool &OutRouteGamepadToSecondWindow ) const { OutRouteGamepadToSecondWindow = RouteGamepadToSecondWindow; return IsRouteGamepadToSecondWindowActive(); }
	EVisibility GetRouteGamepadToSecondWindowVisibility() const { return (RunUnderOneProcess ? EVisibility::Visible : EVisibility::Hidden); }

	EVisibility GetNetworkEmulationVisibility() const { return (PlayNumberOfClients > 1) ? EVisibility::Visible : EVisibility::Hidden; }

	bool IsServerMapNameOverrideActive() const { return false /*(PlayNetMode == PIE_StandaloneWithServer)*/; }
	bool GetServerMapNameOverride( FString& OutStandaloneServerMapName ) const { OutStandaloneServerMapName = ServerMapNameOverride; return IsServerMapNameOverrideActive(); }
	EVisibility GetServerMapNameOverrideVisibility() const { return /*(PlayNetMode == PIE_StandaloneWithServer ? */EVisibility::Visible /*: EVisibility::Hidden)*/; }

	bool IsAdditionalServerGameOptionsActive() const { return (PlayNetMode != PIE_Standalone) || RunUnderOneProcess; }
	bool GetAdditionalServerGameOptions( FString &OutAdditionalServerGameOptions ) const { OutAdditionalServerGameOptions = AdditionalServerGameOptions; return IsAdditionalServerGameOptionsActive(); }

	void SetClientWindowSize( const FIntPoint InClientWindowSize ) { ClientWindowWidth = InClientWindowSize.X; ClientWindowHeight = InClientWindowSize.Y; }
	bool IsClientWindowSizeActive() const { return PlayNumberOfClients > 1; }
	bool GetClientWindowSize( FIntPoint &OutClientWindowSize ) const { OutClientWindowSize = FIntPoint(ClientWindowWidth, ClientWindowHeight); return IsClientWindowSizeActive(); }
	EVisibility GetClientWindowSizeVisibility() const { return (RunUnderOneProcess ? EVisibility::Hidden : EVisibility::Visible); }
	bool IsCreateAudioDeviceForEveryPlayer() const { return CreateAudioDeviceForEveryPlayer; }

	EBuildConfiguration GetLaunchBuildConfiguration() const
	{
		switch (LaunchConfiguration)
		{
		case LaunchConfig_Debug:
			return EBuildConfiguration::Debug;
		case LaunchConfig_Development:
			return EBuildConfiguration::Development;
		case LaunchConfig_Test:
			return EBuildConfiguration::Test;
		case LaunchConfig_Shipping:
			return EBuildConfiguration::Shipping;
		default:
			return FApp::GetBuildConfiguration();
		}
	}

	UE_DEPRECATED(5.5, "Deprecated. Use IsHMDForPrimaryProcessOnly function instead")
	bool IsOneHeadsetEachProcess() const
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bOneHeadsetEachProcess_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool IsHMDForPrimaryProcessOnly() const { return bHMDForPrimaryProcessOnly; }

public:

	/** The last known screen size for the first instance window (in pixels). */
	UPROPERTY(config)
	FIntPoint LastSize;

	/** The last known screen positions of multiple instance windows (in pixels). */
	UPROPERTY(config)
	TArray<FIntPoint> MultipleInstancePositions;

public:

	/** The name of the last platform that the user ran a play session on. */
	UPROPERTY(config)
	FString LastExecutedLaunchDevice;

	/** The name of the last device that the user ran a play session on. */
	UPROPERTY(config)
	FString LastExecutedLaunchName;

	/** The last type of play-on session the user ran. */
	UPROPERTY(config)
	TEnumAsByte<ELaunchModeType> LastExecutedLaunchModeType;

	/** The last type of play location the user ran. */
	UPROPERTY(config)
	TEnumAsByte<EPlayModeLocations> LastExecutedPlayModeLocation;

	/** The last type of play session the user ran. */
	UPROPERTY(config)
	TEnumAsByte<EPlayModeType> LastExecutedPlayModeType;

	/** The name of the last device that the user ran a play session on. */
	UPROPERTY(config)
	FString LastExecutedPIEPreviewDevice;
public:

	/** Collection of common screen resolutions on mobile phones. */
	UPROPERTY(config)
	TArray<FPlayScreenResolution> LaptopScreenResolutions;

	/** Collection of common screen resolutions on desktop monitors. */
	UPROPERTY(config)
	TArray<FPlayScreenResolution> MonitorScreenResolutions;

	/** Collection of common screen resolutions on mobile phones. */
	UPROPERTY(config)
	TArray<FPlayScreenResolution> PhoneScreenResolutions;

	/** Collection of common screen resolutions on tablet devices. */
	UPROPERTY(config)
	TArray<FPlayScreenResolution> TabletScreenResolutions;

	/** Collection of common screen resolutions on television screens. */
	UPROPERTY(config)
	TArray<FPlayScreenResolution> TelevisionScreenResolutions;

	UPROPERTY(config, VisibleAnywhere, Category = GameViewportSettings)
	FString DeviceToEmulate;

	UPROPERTY(config)
	FMargin PIESafeZoneOverride;

	UPROPERTY(config)
	TArray<FVector2D> CustomUnsafeZoneStarts;

	UPROPERTY(config)
	TArray<FVector2D> CustomUnsafeZoneDimensions;

	// UObject interface
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UNREALED_API virtual void PostInitProperties() override;
	UNREALED_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	// End of UObject interface

#if WITH_EDITOR
	// Recalculates and broadcasts safe zone size changes based on device to emulate and r.DebugSafeZone.TitleRatio values.
	UNREALED_API void UpdateCustomSafeZones();
#endif

	UNREALED_API FMargin CalculateCustomUnsafeZones(TArray<FVector2D>& CustomSafeZoneStarts, TArray<FVector2D>& CustomSafeZoneDimensions, FString& DeviceType, FVector2D PreviewSize);
	UNREALED_API FMargin FlipCustomUnsafeZones(TArray<FVector2D>& CustomSafeZoneStarts, TArray<FVector2D>& CustomSafeZoneDimensions, FString& DeviceType, FVector2D PreviewSize);
	UNREALED_API void RescaleForMobilePreview(const class UDeviceProfile* DeviceProfile, int32 &PreviewWidth, int32 &PreviewHeight, float &ScaleFactor);

	/**
     * Creates a widget for the resolution picker.
     *
     * @return The widget.
     */
	UNREALED_API void RegisterCommonResolutionsMenu();
	static UNREALED_API FName GetCommonResolutionsMenuName();

protected:
	/**
	 * Adds a section to the screen resolution menu.
	 *
	 * @param MenuBuilder The menu builder to add the section to.
	 * @param Resolutions The collection of screen resolutions to add.
	 * @param SectionName The name of the section to add.
	 */
	static UNREALED_API void AddScreenResolutionSection( UToolMenu* InToolMenu, const TArray<FPlayScreenResolution>* Resolutions, const FString SectionName );
};
