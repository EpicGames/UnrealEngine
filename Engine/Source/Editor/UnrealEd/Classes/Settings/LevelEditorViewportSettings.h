// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Settings/EditorViewportSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#include "Viewports.h"
#include "Editor/UnrealEdTypes.h"
#include "LevelEditorViewportSettings.generated.h"

/**
 * Enumerates modes for the viewport's rotation grid.
 */
UENUM()
enum ERotationGridMode : int
{
	/** Using Divisions of 360 degrees (e.g 360/2. 360/3, 360/4, ... ). */
	GridMode_DivisionsOf360,

	/** Uses the user defined grid values. */
	GridMode_Common,
};


/**
 * Enumerates camera control types for the W, A, S and D keys.
 */
UENUM()
enum EWASDType : int
{
	WASD_Always  UMETA(DisplayName="Use WASD for Camera Controls"),
	WASD_RMBOnly UMETA(DisplayName="Use WASD only when a Mouse Button is Pressed"),
	WASD_Never   UMETA(DisplayName="Never use WASD for Camera Controls"),
	WASD_MAX,
};

/**
 * Is Ctrl key required for editing landscape/foliage?
 */
UENUM()
enum class ELandscapeFoliageEditorControlType : uint8
{
	IgnoreCtrl    UMETA(DisplayName = "Ignore Ctrl key (allow but don't require Ctrl held)"),
	RequireCtrl   UMETA(DisplayName = "Require Ctrl held for tools"),
	RequireNoCtrl UMETA(DisplayName = "Require Ctrl is not held"),
};

/**
 * Units used by measuring tool
 */
UENUM()
enum EMeasuringToolUnits : int
{
	MeasureUnits_Centimeters UMETA(DisplayName="Centimeters"),
	MeasureUnits_Meters      UMETA(DisplayName="Meters"),
	MeasureUnits_Kilometers  UMETA(DisplayName="Kilometers")
};

/**
 * Scroll gesture direction
 */
UENUM()
enum class EScrollGestureDirection : uint8
{
	UseSystemSetting	UMETA(DisplayName = "Use system setting"),
	Standard			UMETA(DisplayName = "Standard"),
	Natural				UMETA(DisplayName = "Natural"),
};

UENUM()
enum class EMaterialKind : uint8
{
	Unknown,
	Base,
	Normal,
	Specular,
	Emissive,
};

UENUM()
enum class EMarqueeSelectionMode : uint8
{
	/** Selects anything the marquee touches. */
	Crossing,
	/** Selects only items the marquee surrounds. */
	Window,
	/** When dragging to the left, select anything the marquee touches. To the right, only select items the marquee surrounds. */
	CrossLeft,
	/** When dragging to the right, select anything the marquee touches. To the left, only select items the marquee surrounds. */
	CrossRight
};

/**
 * Implements the Level Editor's per-instance view port settings.
 */
USTRUCT()
struct FLevelEditorViewportInstanceSettings
{
	GENERATED_USTRUCT_BODY()

	FLevelEditorViewportInstanceSettings()
		: ViewportType(LVT_Perspective)
		, PerspViewModeIndex(VMI_Lit)
		, OrthoViewModeIndex(VMI_Unlit)
		, EditorShowFlagsString()
		, GameShowFlagsString()
		, BufferVisualizationMode()
		, NaniteVisualizationMode()
		, LumenVisualizationMode()
		, MegaLightsVisualizationMode()
		, SubstrateVisualizationMode()
		, GroomVisualizationMode()
		, VirtualShadowMapVisualizationMode()
		, VirtualTextureVisualizationMode()
		, RayTracingDebugVisualizationMode()
		, GPUSkinCacheVisualizationMode()
		, ColorChannelMask(INDEX_NONE)
		, bInvertAlphaChannelMask(true)
		, bDrawAlphaBlendedCheckerboard(false)
		, ExposureSettings()
		, FOVAngle(EditorViewportDefs::DefaultPerspectiveFOVAngle)
		, FarViewPlane(0)
		, bIsRealtime(false)
		, bShowFPS_DEPRECATED(false)
		// Show 'lighting needs to be rebuilt' message by default, avoids confusion when artists think lighting is built until they PIE
		, bShowOnScreenStats(true)
		, bShowFullToolbar(true)
		, bAllowCinematicControl(true)
	{ }

	/** The viewport type */
	UPROPERTY(config)
	TEnumAsByte<ELevelViewportType> ViewportType;

	/* View mode to set when this viewport is of type LVT_Perspective. */
	UPROPERTY(config)
	TEnumAsByte<EViewModeIndex> PerspViewModeIndex;

	/* View mode to set when this viewport is not of type LVT_Perspective. */
	UPROPERTY(config)
	TEnumAsByte<EViewModeIndex> OrthoViewModeIndex;

	/**
	 * A set of flags that determines visibility for various scene elements (FEngineShowFlags), converted to string form.
	 * These have to be saved as strings since FEngineShowFlags is too complex for UHT to parse correctly.
	 */
	UPROPERTY(config)
	FString EditorShowFlagsString;

	/**
	 * A set of flags that determines visibility for various scene elements (FEngineShowFlags), converted to string form.
	 * These have to be saved as strings since FEngineShowFlags is too complex for UHT to parse correctly.
	 */
	UPROPERTY(config)
	FString GameShowFlagsString;

	/** The buffer visualization mode for the viewport. */
	UPROPERTY(config)
	FName BufferVisualizationMode;

	/** The Nanite visualization mode for the viewport. */
	UPROPERTY(config)
	FName NaniteVisualizationMode;

	/** The Lumen visualization mode for the viewport. */
	UPROPERTY(config)
	FName LumenVisualizationMode;

	/** The MegaLights visualization mode for the viewport. */
	UPROPERTY(config)
	FName MegaLightsVisualizationMode;

	/** The Substrate visualization mode for the viewport. */
	UPROPERTY(config)
	FName SubstrateVisualizationMode;

	/** The Groom visualization mode for the viewport. */
	UPROPERTY(config)
	FName GroomVisualizationMode;

	/** The virtual shadow map visualization mode for the viewport. */
	UPROPERTY(config)
	FName VirtualShadowMapVisualizationMode;

	/** The virtual texture visualization mode for the viewport. */
	UPROPERTY(config)
	FName VirtualTextureVisualizationMode;

	/** The buffer visualization mode for the viewport. */
	UPROPERTY(config)
	FName RayTracingDebugVisualizationMode;

	/** The GPU Skin Cache visualization mode for the viewport. */
	UPROPERTY(config)
	FName GPUSkinCacheVisualizationMode;

	/** The color channel to display for this viewport */
	UPROPERTY(config)
	int32 ColorChannelMask;

	/** Whether to invert the alpha channel when displaying it */
	UPROPERTY(config)
	bool bInvertAlphaChannelMask;

	/** Whether to display an alpha-blended checkerboard beneath the viewport to help visualize alpha in the viewport */
	UPROPERTY(config)
	bool bDrawAlphaBlendedCheckerboard;
	
	/** Setting to allow designers to override the automatic expose. */
	UPROPERTY(config)
	FExposureSettings ExposureSettings;

	/* Field of view angle for the viewport. */
	UPROPERTY(config)
	float FOVAngle;

	/* Position of the var plane in the editor viewport */
	UPROPERTY(config)
	float FarViewPlane;

	/* Whether this viewport is updating in real-time. */
	UPROPERTY(config)
	bool bIsRealtime;

	/* Whether the FPS counter should be shown. */
	UPROPERTY(config)
	bool bShowFPS_DEPRECATED;

	/* Whether viewport statistics should be shown. */
	UPROPERTY(config)
	bool bShowOnScreenStats;

	/* Whether viewport statistics should be enabled by default. */
	UPROPERTY(config)
	TArray<FString> EnabledStats;

	/** When enabled, the full viewport toolbar will be shown. When disabled, a compact toolbar is used. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel)
	bool bShowFullToolbar;

	/** Whether or not this viewport is allowed to be possessed by cinematic/scrubbing tools. */
	UPROPERTY(config)
	bool bAllowCinematicControl;
};


/**
 * Implements a key -> value pair for the per-instance view port settings
 */
USTRUCT()
struct FLevelEditorViewportInstanceSettingsKeyValuePair
{
	GENERATED_USTRUCT_BODY()

	/*  Name identifying this config. */
	UPROPERTY(config)
	FString ConfigName;

	/* Settings for this config. */
	UPROPERTY(config)
	FLevelEditorViewportInstanceSettings ConfigSettings;
};


/**
 * Settings that control the behavior of the "snap to surface" feature
 */
USTRUCT()
struct FSnapToSurfaceSettings
{
	GENERATED_USTRUCT_BODY()

	FSnapToSurfaceSettings()
		: bEnabled(false)
		, SnapOffsetExtent(0.f)
		, bSnapRotation(true)
	{ }

	/** Whether snapping to surfaces in the world is enabled */
	UPROPERTY(config)
	bool bEnabled;

	/** The amount of offset to apply when snapping to surfaces */
	UPROPERTY(config)
	float SnapOffsetExtent;

	/** Whether objects should match the rotation of the surfaces they snap to */
	UPROPERTY(config)
	bool bSnapRotation;
};

USTRUCT()
struct FLevelEditorViewporEditorViews
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FLevelViewportInfo> LevelViewportsInfo;
};

/**
 * Implements the Level Editor's view port settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class ULevelEditorViewportSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Enable the use of flight camera controls under various circumstances. */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	TEnumAsByte<EWASDType> FlightCameraControlType;

	/** Enable the use of the experimental navigation in the flight camera controls. */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	bool FlightCameraControlExperimentalNavigation;

	/** Choose the control scheme for landscape tools (ignored for pen input) */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	ELandscapeFoliageEditorControlType LandscapeEditorControlType;

	/** Choose the control scheme for foliage tools */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	ELandscapeFoliageEditorControlType FoliageEditorControlType;

	/** If true, moves the canvas and shows the mouse.  If false, uses original camera movement. */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName = "Grab and Drag to Move Orthographic Cameras"), AdvancedDisplay)
	uint32 bPanMovesCanvas:1;

	/** If checked, in orthographic view ports zooming will center on the mouse position.  If unchecked, the zoom is around the center of the viewport. */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName = "Orthographic Zoom to Cursor Position"))
	uint32 bCenterZoomAroundCursor:1;

	/** The closest possible distance allowed when viewing through an orthographic camera */
	UPROPERTY(EditAnywhere, config, Category = Controls, meta=(UIMin = "1.0", ClampMin = "1.0"))
	float MinimumOrthographicZoom;

	/** Allow translate/rotate widget */
	UPROPERTY(config, meta=( DisplayName = "Enable Combined Translate/Rotate Widget" ))
	uint32 bAllowTranslateRotateZWidget:1;

	/** Allow arcball rotation with rotate widget */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, meta = (DisplayName = "Enable Arcball Rotate"))
	uint32 bAllowArcballRotate : 1;

	/** Allow screen rotation with rotate widget */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, meta = (DisplayName = "Enable Screen Rotate"))
	uint32 bAllowScreenRotate : 1;

	/** If true, Clicking a BSP selects the brush and ctrl+shift+click selects the surface. If false, vice versa */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=( DisplayName = "Clicking BSP Enables Brush" ), AdvancedDisplay)
	uint32 bClickBSPSelectsBrush:1;

	/** If true, viewport will show actor editor context (current level, current data layer(s), current folder) */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, meta=(AdvancedDisplay))
	uint32 bShowActorEditorContext : 1;

	/** If true, show translucent marker polygons on the builder brush and volumes. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel)
	uint32 bShowBrushMarkerPolys:1;

	/** If true, the Edit widget of a transform will display the axis */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=( DisplayName = "Enable Axis drawing for transform Edit Widget" ))
	uint32 bAllowEditWidgetAxisDisplay:1;

	/** The camera speed settings for level viewports. */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	FEditorViewportCameraSpeedSettings CameraSpeedSettings;

	/** How fast the perspective camera moves when flying through the world. */
	UPROPERTY(config, meta=(UIMin = "1", UIMax = "8", ClampMin="1", ClampMax="8"))
	int32 CameraSpeed_DEPRECATED;

	/** Scalar applied to perspective camera movement to increase movement range. */
	UPROPERTY(config, meta = (ClampMin = "1", UIMax = "128"))
	float CameraSpeedScalar_DEPRECATED;

	/** How fast the perspective camera moves through the world when using mouse scroll. */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(UIMin = "1", UIMax = "8", ClampMin="1", ClampMax="8"))
	int32 MouseScrollCameraSpeed;

	/** The sensitivity of mouse movement when rotating the camera. */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName="Mouse Sensitivity", ClampMin="0.01",ClampMax="1.0") )
	float MouseSensitivty;

	/** Camera movement notification toggle to switch back to the behavior that caused camera notifications to be sent during gizmo movement. */
	UPROPERTY(EditAnywhere, config, Category = Controls)
	uint32 bUseLegacyCameraMovementNotifications : 1;
	
	/** Whether or not to invert mouse on the y axis in free look mode */
	UPROPERTY(EditAnywhere, config, Category = Controls, meta = (DisplayName = "Invert Mouse Look Y Axis"))
	bool bInvertMouseLookYAxis;

	/** Whether or not to invert mouse on y axis in orbit mode */
	UPROPERTY(EditAnywhere, config, Category = Controls, meta = (DisplayName = "Invert Orbit Y Axis"))
	bool bInvertOrbitYAxis;

	/** Whether or not to invert the direction of middle mouse panning in viewports */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	bool bInvertMiddleMousePan;

	/** If enabled, vertical movements of the mouse while panning with the middle mouse button will move the camera vertically in world space. Otherwise, vertical mouse movements will be camera-relative. */
	UPROPERTY(EditAnywhere, config, Category=Controls)
	bool bWorldSpaceVerticalPan;

	/** Whether or not to invert the direction of right mouse dolly on the Y axis in orbit mode */
	UPROPERTY(EditAnywhere, config, Category = Controls)
	bool bInvertRightMouseDollyYAxis;

	/** Whether to use mouse position as direct widget position. */
	UPROPERTY(EditAnywhere, config, Category=Controls, AdvancedDisplay)
	uint32 bUseAbsoluteTranslation:1;

	/** If enabled, the viewport will stream in levels automatically when the camera is moved. */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName = "Stream in Levels Automatically when Camera is Moved"), AdvancedDisplay)
	bool bLevelStreamingVolumePrevis;

	/** When checked, orbit the camera by using the L or U keys when unchecked, Alt and Left Mouse Drag will orbit around the look at point */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName="Use UE3 Orbit Controls"), AdvancedDisplay)
	bool bUseUE3OrbitControls;

	/** Direction of the scroll gesture for 3D viewports */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName="Scroll gesture direction for 3D viewports"))
	EScrollGestureDirection ScrollGestureDirectionFor3DViewports;

	/** Direction of the scroll gesture for orthographic viewports */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName="Scroll gesture direction for orthographic viewports"))
	EScrollGestureDirection ScrollGestureDirectionForOrthoViewports;

	/** Enables joystick-based camera movement in 3D level editing viewports */
	UPROPERTY(EditAnywhere, config, Category=Controls, meta=(DisplayName="Level Editor Joystick Controls" ) )
	bool bLevelEditorJoystickControls;

	/** If enabled, scale the perspective camera speed based on the distance between the camera and its look-at position */
	UPROPERTY(EditAnywhere, config, Category = Controls, meta = (DisplayName = "Use distance-scaled camera speed"), AdvancedDisplay)
	bool bUseDistanceScaledCameraSpeed;

	/** If enabled, the camera will orbit around the current selection in the viewport */
	UPROPERTY(EditAnywhere, config, Category = Controls, meta = (DisplayName = "Orbit camera around selection"), AdvancedDisplay)
	bool bOrbitCameraAroundSelection;

public:

	/** If disabled, any and all snapping is disabled */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping)
	bool bSnappingActive = true;

	/** If enabled will use power of 2 grid settings (e.g, 1,2,4,8,16,...,1024) instead of decimal grid sizes */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping, meta=(DisplayName = "Use Power of Two Snap Size", EditCondition="bSnappingActive"))
	bool bUsePowerOf2SnapSize;

	/** Decimal grid sizes (for translation snapping and grid rendering) */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> DecimalGridSizes;

	/** The number of lines between each major line interval for decimal grids */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> DecimalGridIntervals;	

	/** Power of 2 grid sizes (for translation snapping and grid rendering) */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> Pow2GridSizes;

	/** The number of lines between each major line interval for pow2 grids */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> Pow2GridIntervals;

	/** User defined grid intervals for rotations */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> CommonRotGridSizes;

	/** Preset grid intervals for rotations */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> DivisionsOf360RotGridSizes;

	/** Grid sizes for scaling */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=GridSnapping, meta=(EditCondition="bSnappingActive"))
	TArray<float> ScalingGridSizes;

	/** If enabled, actor positions will snap to the grid. */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping, meta=(DisplayName = "Enable Grid Snapping", EditCondition="bSnappingActive"))
	uint32 GridEnabled:1;
	
	/** If enabled, actor rotations will snap to the grid. */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping, meta=(DisplayName = "Enable Rotation Snapping", EditCondition="bSnappingActive"))
	uint32 RotGridEnabled:1;

	/** If enabled, actor sizes will snap to the grid. */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping, meta=(DisplayName = "Enable Scale Snapping", EditCondition="bSnappingActive"))
	uint32 SnapScaleEnabled:1;

	/** If enabled, actors will snap to surfaces in the viewport when dragged around */
	UPROPERTY(config)
	FSnapToSurfaceSettings SnapToSurface;

private:

	/** If enabled, use the old-style multiplicative/percentage scaling method instead of the new additive/fraction method */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping)
	uint32 bUsePercentageBasedScaling:1;

public:

	/** If enabled, new Actors that you drag into the viewport snap to the active 2D layer. */
	UPROPERTY(EditAnywhere, config, Category=GridSnapping, meta=(DisplayName = "Enable 2D Layer Snapping", EditCondition="bSnappingActive"))
	uint32 bEnableLayerSnap:1;

	/** The index of the snap plane to use when 2D Layer Snapping is enabled, from the array of Snap Layers set for the project. */
	UPROPERTY(config)
	int32 ActiveSnapLayerIndex;

	/** If true actor snap will be enabled in the editor **/
	UPROPERTY(config, Category=GridSnapping, VisibleDefaultsOnly,AdvancedDisplay, meta=(EditCondition="bSnappingActive"))
	uint32 bEnableActorSnap:1;

	/** Global actor snap scale for the editor */
	UPROPERTY(config, Category=GridSnapping, VisibleDefaultsOnly,AdvancedDisplay, meta=(EditCondition="bSnappingActive"))
	float ActorSnapScale;

	/** Global actor snap distance setting for the editor */
	UPROPERTY(config)
	float ActorSnapDistance;

	UPROPERTY(config)
	bool bSnapVertices;
 
	UPROPERTY(config)
	float SnapDistance;

	UPROPERTY(config)
	int32 CurrentPosGridSize;

	UPROPERTY(config)
	int32 CurrentRotGridSize;

	UPROPERTY(config)
	int32 CurrentScalingGridSize;

	UPROPERTY(config)
	bool PreserveNonUniformScale;

	/** Controls which array of rotation grid values we are using */
	UPROPERTY(config)
	TEnumAsByte<ERotationGridMode> CurrentRotGridMode;

public:

	/** How to constrain perspective view port FOV */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel)
	TEnumAsByte<enum EAspectRatioAxisConstraint> AspectRatioAxisConstraint;

	/** Enables real-time hover feedback when mousing over objects in editor view ports */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Highlight Objects Under Mouse Cursor"))
	uint32 bEnableViewportHoverFeedback:1;

	/** If enabled, selected objects will be highlighted with brackets in all modes rather than a special highlight color. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, AdvancedDisplay, meta=(DisplayName = "Highlight Selected Objects with Brackets"))
	uint32 bHighlightWithBrackets:1;

	/** If checked all orthographic view ports are linked to the same position and move together. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Link Orthographic Viewport Movement"))
	uint32 bUseLinkedOrthographicViewports:1;

	// bUseLODViewLocking is added to keep around a legacy command from FLevelEditorCommands, as we are removing the Level Editor Settings menu.
	// LOD View Locking is possibly presenting some bugs.
	/** If true, same type views will be camera tied, and ortho views will use perspective view for LOD parenting */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Enable LOD View Locking"))
	uint32 bUseLODViewLocking:1;

	/** True if viewport box selection requires objects to be fully encompassed by the selection box to be selected */
	UPROPERTY(config)
	uint32 bStrictBoxSelection:1;
	
	/** Sets how marquee selection should behave. */
	UPROPERTY(EditAnywhere, Config, Category = Behavior)
	EMarqueeSelectionMode MarqueeSelectionMode = EMarqueeSelectionMode::Crossing;

	/** True if viewport box selection also selects occluded objects, false if only objects with visible pixels are selected */
	UPROPERTY(config)
	uint32 bTransparentBoxSelection:1;

	/** Whether to show selection outlines for selected Actors */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Use Selection Outline"))
	uint32 bUseSelectionOutline:1;

	/** Sets the intensity of the overlay displayed when an object is selected */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Selection Highlight Intensity" ,ClampMin = "0", UIMin = "0", UIMax = "1"))
	float SelectionHighlightIntensity;

	/** Sets the intensity of the overlay displayed when an object is selected */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, AdvancedDisplay, meta=(DisplayName = "BSP Surface Highlight Intensity" ,ClampMin = "0", UIMin = "0", UIMax = "1"))
	float BSPSelectionHighlightIntensity;

	/** Enables the editor perspective camera to be dropped at the last PlayInViewport cam position */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(DisplayName = "Use Camera Location from Play-In-Viewport"))
	uint32 bEnableViewportCameraToUpdateFromPIV:1;

	/** When enabled, selecting a camera actor will display a live 'picture in picture' preview from the camera's perspective within the current editor view port.  This can be used to easily tweak camera positioning, post-processing and other settings without having to possess the camera itself.  This feature may reduce application performance when enabled. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel)
	uint32 bPreviewSelectedCameras:1;

	/** Affects the size of 'picture in picture' previews if they are enabled */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, meta=(ClampMin = "1", UIMin = "1", UIMax = "10"))
	float CameraPreviewSize;

	/** Distance from the camera to place actors which are dropped on nothing in the view port. */
	UPROPERTY(EditAnywhere, config, Category=LookAndFeel, AdvancedDisplay, meta=(DisplayName = "Background Drop Distance"))
	float BackgroundDropDistance;

	/** A list of meshes that can be used as preview mesh in the editor view port by holding down the backslash key */
	UPROPERTY(EditAnywhere, config, Category=Preview, meta=(AllowedClasses = "/Script/Engine.StaticMesh"))
	TArray<FSoftObjectPath> PreviewMeshes;

	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category=LookAndFeel, meta=(ClampMin = "0.01", UIMin = "0.01", UIMax = "5"))
	float BillboardScale;

	/** The size adjustment to apply to the translate/rotate/scale widgets (in Unreal units). */
	UE_DEPRECATED(5.6, "TransformWidgetSizeAdjustment. See UTransformGizmoEditorSettings::TransformGizmoSize")
	UPROPERTY(config)
	int32 TransformWidgetSizeAdjustment_DEPRECATED;

	/** When enabled, engine stats that are enabled in level viewports are preserved between editor sessions */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category = LookAndFeel)
	uint32 bSaveEngineStats : 1;

	/** Specify the units used by the measuring tool */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel)
	TEnumAsByte<EMeasuringToolUnits> MeasuringToolUnits;

	/** The size adjustment to apply to selected spline points (in screen space units). */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, AdvancedDisplay, meta = (ClampMin = "-5.00", ClampMax = "20.00"))
	float SelectedSplinePointSizeAdjustment = 0.0f;

	/** The size adjustment to apply to spline line thickness which increases the spline's hit tolerance. */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, AdvancedDisplay, meta = (ClampMin = "0.00"))
	float SplineLineThicknessAdjustment = 0.0f;

	/** The size adjustment to apply to spline tangent handle (in screen space units). */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, AdvancedDisplay, meta = (ClampMin = "-5.00", ClampMax = "20.00"))
	float SplineTangentHandleSizeAdjustment = 0.0f;

	/** The scale to apply to spline tangent lengths */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel, AdvancedDisplay, meta = (ClampMin = "0.00"))
	float SplineTangentScale = 0.5f;

	UPROPERTY(config)
	FVector2D LastInViewportMenuLocation;

	/** When dropping a texture in the viewport, create an instance of this material instead of creating a new material. Populate MaterialParamsForDroppedTextures to specify the parameter names. */
	UPROPERTY(EditAnywhere, config, Category = Behavior)
	TSoftObjectPtr<class UMaterialInterface> MaterialForDroppedTextures;

	/** When dropping a texture in the viewport, determines which material parameter to assign for each found texture type. Only relevant if MaterialForDroppedTextures is assigned. */
	UPROPERTY(EditAnywhere, config, Category = Behavior)
	TMap<EMaterialKind, FName> MaterialParamsForDroppedTextures;

	/** Store the last camera settings for all the viewport of each world loaded in the editor. */
	UPROPERTY(config)
	TMap<TSoftObjectPtr<UWorld>, FLevelEditorViewporEditorViews> EditorViews;

	/** Property coloration color for matching objects */
	UPROPERTY(EditAnywhere, config, Category = LookAndFeel)
	FColor PropertyColorationColorForMatchingObjects = FColor::Red;

	/** When enabled, the channel mask menu will show up in the editor toolbar next to the View Modes button */
	UPROPERTY(config)
	uint32 bShowChannelsInToolbar:1;

private:

	// Per-instance viewport settings.
	UPROPERTY(config)
	TArray<FLevelEditorViewportInstanceSettingsKeyValuePair> PerInstanceSettings;

public:

	/**
	 * @return The instance settings for the given viewport; null if no settings were found for this viewport
	 */
	const FLevelEditorViewportInstanceSettings* GetViewportInstanceSettings( const FString& InConfigName ) const
	{
		for(auto It = PerInstanceSettings.CreateConstIterator(); It; ++It)
		{
			const FLevelEditorViewportInstanceSettingsKeyValuePair& ConfigData = *It;
			if(ConfigData.ConfigName == InConfigName)
			{
				return &ConfigData.ConfigSettings;
			}
		}

		return nullptr;
	}

	/**
	 * Set the instance settings for the given viewport
	 */
	void SetViewportInstanceSettings( const FString& InConfigName, const FLevelEditorViewportInstanceSettings& InConfigSettings )
	{
		check(!InConfigName.IsEmpty());

		bool bWasFound = false;
		for(auto It = PerInstanceSettings.CreateIterator(); It; ++It)
		{
			FLevelEditorViewportInstanceSettingsKeyValuePair& ConfigData = *It;
			if(ConfigData.ConfigName == InConfigName)
			{
				ConfigData.ConfigSettings = InConfigSettings;
				bWasFound = true;
				break;
			}
		}

		if(!bWasFound)
		{
			FLevelEditorViewportInstanceSettingsKeyValuePair ConfigData;
			ConfigData.ConfigName = InConfigName;
			ConfigData.ConfigSettings = InConfigSettings;
			PerInstanceSettings.Add(ConfigData);
		}

		PostEditChange();
	}

	/**
	 * Checks whether percentage based scaling should be used for view ports.
	 *
	 * @return true if percentage based scaling is enabled, false otherwise.
	 */
	bool UsePercentageBasedScaling( ) const
	{
		return bUsePercentageBasedScaling;
	}

public:

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(ULevelEditorViewportSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged( ) { return SettingChangedEvent; }

protected:

	// UObject overrides

	UNREALED_API virtual void PostInitProperties() override;
	UNREALED_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;

private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;
};
