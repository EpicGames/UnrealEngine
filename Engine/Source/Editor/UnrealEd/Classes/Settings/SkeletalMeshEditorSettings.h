// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletalMeshEditorDesttings.h: Declares the USkeletalMeshEditorDesttings class.
=============================================================================*/

#pragma once


#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SkeletalMeshEditorSettings.generated.h"


/**
 * Implements the settings for the skeletal mesh editor.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class USkeletalMeshEditorSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Floor Color"))
	FColor AnimPreviewFloorColor;

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Sky Color"))
	FColor AnimPreviewSkyColor;

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Sky Brightness"))
	float AnimPreviewSkyBrightness;

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Light Brightness"))
	float AnimPreviewLightBrightness;

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Lighting Direction"))
	FRotator AnimPreviewLightingDirection;

	UPROPERTY(EditAnywhere, config, Category=AnimationPreview, meta=(DisplayName="Viewport Directional Color"))
	FColor AnimPreviewDirectionalColor;
};
