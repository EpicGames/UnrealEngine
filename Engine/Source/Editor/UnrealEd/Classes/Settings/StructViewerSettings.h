// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructViewerSettings.generated.h"

/** The developer folder view modes used in SStructViewer */
UENUM()
enum class EStructViewerDeveloperType : uint8
{
	/** Display no developer folders*/
	SVDT_None,
	/** Allow the current user's developer folder to be displayed. */
	SVDT_CurrentUser,
	/** Allow all users' developer folders to be displayed.*/
	SVDT_All,
	/** Max developer type*/
	SVDT_Max
};

/**
 * Implements the Struct Viewer's loading and saving settings.
 */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UStructViewerSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether to display internal use structs. */
	UPROPERTY(config)
	bool DisplayInternalStructs;

	/** The developer folder view modes used in SStructViewer */
	UPROPERTY(config)
	EStructViewerDeveloperType DeveloperFolderType;

public:
	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UStructViewerSettings, FSettingChangedEvent, FName /*PropertyName*/);
	static FSettingChangedEvent& OnSettingChanged() { return SettingChangedEvent; }

protected:
	//~ UObject overrides
	UNREALED_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// Holds an event delegate that is executed when a setting has changed.
	static UNREALED_API FSettingChangedEvent SettingChangedEvent;
};
