// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TexAligner
// Base class for all texture aligners.
//
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "TexAligner.generated.h"

class FBspSurfIdx;
class FPoly;
class UModel;

/** Alignment types. */
UENUM()
enum ETexAlign : int
{
	/** When passed to functions it means "use whatever the aligners default is". */
	TEXALIGN_None,
	/** No special alignment (just derive from UV vectors). */
	TEXALIGN_Default,
	/** Aligns to best U and V axis based on polys normal. */
	TEXALIGN_Box,
	/** Maps the poly to the axis it is closest to laying parallel to. */
	TEXALIGN_Planar,
	/** Fits texture to a polygon. */
	TEXALIGN_Fit,
	/** Special version of TEXALIGN_Planar. */
	TEXALIGN_PlanarAuto,
	/** Special version of TEXALIGN_Planar. */
	TEXALIGN_PlanarWall,
	/** Special version of TEXALIGN_Planar. */
	TEXALIGN_PlanarFloor,
	TEXALIGN_MAX,
};

UCLASS()
class UTexAligner : public UObject
{
	GENERATED_UCLASS_BODY()

	/** The default alignment this aligner represents. */
	UPROPERTY()
	TEnumAsByte<enum ETexAlign> DefTexAlign;

	UPROPERTY()
	uint8 TAxis;

	UPROPERTY()
	float UTile;

	UPROPERTY()
	float VTile;

	/** Description for the editor to display. */
	UPROPERTY()
	FString Desc;


	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	// @todo document
	UNREALED_API void Align( UWorld* InWorld, ETexAlign InTexAlignType );

	// @todo document
	void Align( UWorld* InWorld, ETexAlign InTexAlignType, UModel* InModel );
	
	/** Aligns a specific BSP surface */
	virtual void AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal );
};

