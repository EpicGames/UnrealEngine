// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TexAlignerBox
// Aligns to the best U and V axis according to the polys normal.
//
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "TexAligner/TexAligner.h"
#include "TexAlignerBox.generated.h"

class FBspSurfIdx;
class FPoly;
class UModel;

UCLASS(hidecategories=Object)
class UTexAlignerBox : public UTexAligner
{
	GENERATED_UCLASS_BODY()


	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UTexAligner Interface
	virtual void AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal ) override;
	//~ End UTexAligner Interface
};

