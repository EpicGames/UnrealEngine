// Copyright Epic Games, Inc. All Rights Reserved.

//~=============================================================================
// TexAlignerPlanar
// Aligns according to which axis the poly is most facing.
// 
//~=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "TexAligner/TexAligner.h"
#include "TexAlignerPlanar.generated.h"

class FBspSurfIdx;
class FPoly;
class UModel;

UCLASS(hidecategories=Object)
class UTexAlignerPlanar : public UTexAligner
{
	GENERATED_UCLASS_BODY()


	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	//~ Begin UTexAligner Interface
	virtual void AlignSurf( ETexAlign InTexAlignType, UModel* InModel, FBspSurfIdx* InSurfIdx, FPoly* InPoly, FVector* InNormal ) override;
	//~ End UTexAligner Interface
};

