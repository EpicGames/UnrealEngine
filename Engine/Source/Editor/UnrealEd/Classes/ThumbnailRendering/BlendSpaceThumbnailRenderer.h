// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given blend space
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "BlendSpaceThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor, MinimalAPI)
class UBlendSpaceThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()


	// Begin UThumbnailRenderer Object
	UNREALED_API virtual bool CanVisualizeAsset(UObject* Object) override;
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	UNREALED_API virtual void BeginDestroy() override;

private:
	class FBlendSpaceThumbnailScene* ThumbnailScene;
};

