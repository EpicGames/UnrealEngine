// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given static mesh
 */

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "ThumbnailHelpers.h"
#include "BlueprintThumbnailRenderer.generated.h"

class UBlueprint;
class FCanvas;
class FRenderTarget;

UCLASS(config=Editor,MinimalAPI)
class UBlueprintThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	UNREALED_API virtual bool CanVisualizeAsset(UObject* Object) override;
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	UNREALED_API virtual void BeginDestroy() override;
	// End UObject implementation

	/** Notifies the thumbnail scene to refresh components for the specified blueprint */
	void BlueprintChanged(UBlueprint* Blueprint);

private:
	void OnBlueprintUnloaded(UBlueprint* Blueprint);

private:
	TOptional<FBlueprintThumbnailScene> ThumbnailScene;
};
