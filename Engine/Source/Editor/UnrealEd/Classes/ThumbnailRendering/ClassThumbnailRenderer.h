// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "ThumbnailHelpers.h"
#include "ClassThumbnailRenderer.generated.h"

class UBlueprintGeneratedClass;
class FCanvas;
class FRenderTarget;

UCLASS(config=Editor,MinimalAPI)
class UClassThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	UNREALED_API virtual bool CanVisualizeAsset(UObject* Object) override;
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject Implementation
	UNREALED_API virtual void BeginDestroy() override;
	// End UObject Implementation

private:
	void OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BPGC);

private:
	TOptional<FClassThumbnailScene> ThumbnailScene;
};
