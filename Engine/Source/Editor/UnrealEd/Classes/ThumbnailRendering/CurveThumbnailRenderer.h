// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given linear color curve
 */

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "CurveThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor, MinimalAPI)
class UCurveFloatThumbnailRenderer : public UThumbnailRenderer
{
	GENERATED_UCLASS_BODY()
public:

	// Begin UThumbnailRenderer Object
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	// End UThumbnailRenderer Object
};

UCLASS(config=Editor, MinimalAPI)
class UCurveVector3ThumbnailRenderer : public UThumbnailRenderer
{
	GENERATED_UCLASS_BODY()
public:

	// Begin UThumbnailRenderer Object
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	// End UThumbnailRenderer Object
};

UCLASS(config=Editor, MinimalAPI)
class UCurveLinearColorThumbnailRenderer : public UThumbnailRenderer
{
	GENERATED_UCLASS_BODY()
public:

	// Begin UThumbnailRenderer Object
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	// End UThumbnailRenderer Object
};

