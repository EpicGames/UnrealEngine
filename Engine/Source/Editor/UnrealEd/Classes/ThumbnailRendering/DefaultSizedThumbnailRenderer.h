// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer holds some commonly shared properties
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "DefaultSizedThumbnailRenderer.generated.h"

UCLASS(abstract, config=Editor, MinimalAPI)
class UDefaultSizedThumbnailRenderer : public UThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	/**
	 * The default width of this thumbnail
	 */
	UPROPERTY(config)
	int32 DefaultSizeX;

	/**
	 * The default height of this thumbnail
	 */
	UPROPERTY(config)
	int32 DefaultSizeY;


	// Begin UThumbnailRenderer Object
	UNREALED_API virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	// End UThumbnailRenderer Object
};

