// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given level
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "LevelThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class FSceneView;
class FSceneViewFamily;
class ULevel;

UCLASS(config=Editor)
class ULevelThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()


	// Begin UThumbnailRenderer Object
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

private:
	/** Allocates then adds an FSceneView to the ViewFamily. */
	[[nodiscard]] FSceneView* CreateView(ULevel* Level, FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const; 
};

