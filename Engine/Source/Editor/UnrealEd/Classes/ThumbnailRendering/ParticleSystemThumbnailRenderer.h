// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given particle system
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "ParticleSystemThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor)
class UParticleSystemThumbnailRenderer : public UTextureThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UTexture2D> NoImage;

	UPROPERTY()
	TObjectPtr<class UTexture2D> OutOfDate;


	// Begin UThumbnailRenderer Object
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	UNREALED_API virtual void BeginDestroy() override;

private:
	class FParticleSystemThumbnailScene* ThumbnailScene;
};



