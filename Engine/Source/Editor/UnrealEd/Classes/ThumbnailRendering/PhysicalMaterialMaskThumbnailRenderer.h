// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays the mask texture for the object in question
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "PhysicalMaterialMaskThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(MinimalAPI)
class UPhysicalMaterialMaskThumbnailRenderer : public UTextureThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	UNREALED_API virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object
};

