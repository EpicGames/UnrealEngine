// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "PhysicsAssetThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor, MinimalAPI)
class UPhysicsAssetThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	virtual bool CanVisualizeAsset(UObject* Object) override;

	// Begin UThumbnailRenderer Object
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject implementation
	UNREALED_API virtual void BeginDestroy() override;

private:
	class FPhysicsAssetThumbnailScene* ThumbnailScene;
};

