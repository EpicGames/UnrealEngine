// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Thumbnail information for assets that need a scene
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "SceneThumbnailInfo.generated.h"


UCLASS(MinimalAPI)
class USceneThumbnailInfo : public UThumbnailInfo
{
	GENERATED_UCLASS_BODY()

	/** UObject interface */
	virtual void Serialize(FArchive& Ar) override;

	/** The pitch of the orbit camera around the asset */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	float OrbitPitch;

	/** The yaw of the orbit camera around the asset */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	float OrbitYaw;

	/** The offset from the bounds sphere distance from the asset */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	float OrbitZoom;

public:
	UNREALED_API virtual void ResetToDefault();
	UNREALED_API virtual bool DiffersFromDefault() const;

};
