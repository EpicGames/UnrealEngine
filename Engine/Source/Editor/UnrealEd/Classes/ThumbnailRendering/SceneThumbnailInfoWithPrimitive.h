// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Thumbnail information for assets that need a scene and a primitive
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/ThumbnailManager.h"

#include "SceneThumbnailInfoWithPrimitive.generated.h"


UCLASS(MinimalAPI)
class USceneThumbnailInfoWithPrimitive : public USceneThumbnailInfo
{
	GENERATED_UCLASS_BODY()

	/** The type of primitive used in this thumbnail */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	TEnumAsByte<EThumbnailPrimType> PrimitiveType;
	
	TOptional<TEnumAsByte<EThumbnailPrimType>> DefaultPrimitiveType;

	/** The custom mesh used when the primitive type is TPT_None */
	UPROPERTY(EditAnywhere, Category=Thumbnail, meta = (AllowedClasses = "/Script/Engine.StaticMesh,/Script/Engine.SkeletalMesh", ExactClass = "true"))
	FSoftObjectPath PreviewMesh;

	UPROPERTY()
	bool bUserModifiedShape;
	
public:
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

	UNREALED_API virtual void ResetToDefault();
	UNREALED_API virtual bool DiffersFromDefault() const;
};
