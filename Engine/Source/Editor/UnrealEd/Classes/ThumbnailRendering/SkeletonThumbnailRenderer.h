// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given skeletal mesh
 */

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailHelpers.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "SkeletonThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class FPackageReloadedEvent;

enum class EPackageReloadPhase : uint8;

UCLASS(config=Editor, MinimalAPI)
class USkeletonThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()


	// Begin UThumbnailRenderer Object
	UNREALED_API virtual bool CanVisualizeAsset(UObject* Object) override;
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	UNREALED_API virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object

	// UObject implementation
	UNREALED_API virtual void BeginDestroy() override;

	virtual void AddAdditionalPreviewSceneContent(UObject* Object, UWorld* PreviewWorld) {}

protected:
	TObjectInstanceThumbnailScene<FSkeletalMeshThumbnailScene, 128> ThumbnailSceneCache;

private:
	void HandlePackageReloaded(EPackageReloadPhase Phase, FPackageReloadedEvent* Event);

	FDelegateHandle PackageReloadedHandle;
};

