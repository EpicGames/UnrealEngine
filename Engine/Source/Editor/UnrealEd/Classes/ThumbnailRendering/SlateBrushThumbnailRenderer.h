// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given static mesh
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "SlateBrushThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class UTexture2D;
struct FSlateBrush;

UCLASS(config=Editor,MinimalAPI)
class USlateBrushThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

public:

	// Begin UThumbnailRenderer Object
	UNREALED_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object

	void CreateThumbnailAsImage(uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FSlateBrush& Brush);
	void CreateTextureThumbnailOnCanvas(int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FSlateBrush& Brush, FCanvas* Canvas, UTexture2D* Texture);

private:
	bool bIsLastFrequencyRealTime;
};

