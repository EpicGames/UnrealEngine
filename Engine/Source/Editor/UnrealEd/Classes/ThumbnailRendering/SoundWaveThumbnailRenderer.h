// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer generates a render of a waveform
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "SoundWaveThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS()
class USoundWaveThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual bool CanVisualizeAsset(UObject* Object);
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object
};

