// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays the texture for the object in question
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "SpecularProfileRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS()
class USpecularProfileRenderer : public UTextureThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object
};

