// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This class contains a list of thumbnail rendering entries, which can be
 * configured from Editor.ini. The idea is for thumbnail rendering to be
 * extensible without having to modify Epic code.
 */

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "ThumbnailManager.generated.h"

class FViewport;
class FAssetThumbnailPool;

/**
 * Types of primitives for drawing thumbnails of resources.
 */
UENUM()
enum EThumbnailPrimType : int
{
	TPT_None,
	TPT_Sphere,
	TPT_Cube,
	TPT_Plane,
	TPT_Cylinder,
	TPT_ShaderBall,
	TPT_MAX,
};

/**
 * Holds the settings for a class that needs a thumbnail renderer. Each entry
 * maps to a corresponding class and holds the information needed
 * to render the thumbnail, including which object to render via and its
 * border color.
 */
USTRUCT()
struct FThumbnailRenderingInfo
{
	GENERATED_USTRUCT_BODY()

	/**
	 * The name of the class that this thumbnail is for (so we can lazy bind)
	 */
	UPROPERTY()
	FString ClassNeedingThumbnailName;

	/**
	 * This is the class that this entry is for, i.e. the class that
	 * will be rendered in the thumbnail views
	 */
	UPROPERTY()
	TSubclassOf<class UObject> ClassNeedingThumbnail;

	/**
	 * The name of the class to load when rendering this thumbnail
	 * NOTE: This is a string to avoid any dependencies of compilation
	 */
	UPROPERTY()
	FString RendererClassName;

	/**
	 * The instance of the renderer class
	 */
	UPROPERTY()
	TObjectPtr<class UThumbnailRenderer> Renderer;

	/**
	 * Should we get the default object from the specified object before passing it to the renderer?  (only used for blueprints)
	 */
	UPROPERTY()
	bool bUseClassDefaultObject = false;

public:
	FThumbnailRenderingInfo()
		: ClassNeedingThumbnail(NULL)
		, Renderer(NULL)
	{
	}

};

DECLARE_EVENT_OneParam(UThumbnailManager, FOnThumbnailDirtied, const FSoftObjectPath&);

UCLASS(config=Editor)
class UThumbnailManager : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	/**
	 * The render info to share across all object types when the object doesn't
	 * support rendering of thumbnails
	 */
	UPROPERTY()
	struct FThumbnailRenderingInfo NotSupported;

	// All these meshes/materials/textures are preloaded via default properties
	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorCube;

	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorSphere;

	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorCylinder;

	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorPlane;
	
	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorShaderBall;

	UPROPERTY(Transient)
	TObjectPtr<class UStaticMesh> EditorSkySphere;

	UPROPERTY(Transient)
	TObjectPtr<class UMaterial> FloorPlaneMaterial;

	UPROPERTY(Transient)
	TObjectPtr<class UTextureCube> AmbientCubemap;

	UPROPERTY(Transient)
	TObjectPtr<class UTexture2D> CheckerboardTexture;

public:
	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	/**
	 * Fixes up any classes that need to be loaded in the thumbnail types
	 */
	void Initialize(void);

	/**
	 * Returns the entry for the specified object
	 *
	 * @param Object the object to find thumbnail rendering info for
	 *
	 * @return A pointer to the rendering info if valid, otherwise NULL
	 */
	UNREALED_API FThumbnailRenderingInfo* GetRenderingInfo(UObject* Object);

	/**
	 * Registers a custom thumbnail renderer for a specific class
	 *
	 * @param Class	The class the custom thumbnail renderer is for
	 * @param RendererClass The rendering class to use
	 */
	UNREALED_API virtual void RegisterCustomRenderer(UClass* Class, TSubclassOf<UThumbnailRenderer> RendererClass);

	/**
	 * Unregisters a custom thumbnail renderer for a specific class
	 *
	 * @param Class	The class with the custom thumbnail renderer to remove
	 */
	UNREALED_API virtual void UnregisterCustomRenderer(UClass* Class);

	/** 
	 * Returns the thumbnail pool which should be used for most thumbnails in the editor.
	 */
	TSharedPtr<FAssetThumbnailPool> GetSharedThumbnailPool() const { return SharedThumbnailPool; }

	/**
	 * Event that is being broadcasted when a thumbnail gets dirtied.
	 * Parameter is the object soft object path associated with the thumbnail.
	 */
	FOnThumbnailDirtied& GetOnThumbnailDirtied() { return OnThumbnailDirtied; }

protected:
	/**
	 * The array of thumbnail rendering information entries. Each type that supports
	 * thumbnail rendering has an entry in here.
	 */
	UPROPERTY(config)
	TArray<struct FThumbnailRenderingInfo> RenderableThumbnailTypes;

	/**
	 * This holds a map of object type to render info entries
	 */
	TMap<UClass*, FThumbnailRenderingInfo*> RenderInfoMap;


	/**
	 * Holds the name of the thumbnail manager singleton class to instantiate
	 */
	UPROPERTY(config)
	FString ThumbnailManagerClassName;

	/**
	 * Manager responsible for configuring and rendering thumbnails
	 */
	static class UThumbnailManager* ThumbnailManagerSingleton;

	/** Thumbnail pool used for most thumbnails in the editor */
	TSharedPtr<FAssetThumbnailPool> SharedThumbnailPool;

	/**
	 * Determines whether the initialization function is needed or not
	 */
	bool bIsInitialized;

	// The following members are present for performance optimizations

	/**
	 * Whether to update the map or not (GC usually causes this)
	 */
	bool bMapNeedsUpdate;

	FOnThumbnailDirtied OnThumbnailDirtied;

public:
	/** Returns the thumbnail manager and creates it if missing */
	UNREALED_API static UThumbnailManager& Get();

	/** Returns the thumbnail manager if it exists */
	UNREALED_API static UThumbnailManager* TryGet();

	/** Writes out a png of what is currently in the specified viewport, scaled appropriately */
	UNREALED_API static bool CaptureProjectThumbnail(FViewport* Viewport, const FString& OutputFilename, bool bUseSCCIfPossible);

protected:
	/**
	 * Fixes up any classes that need to be loaded in the thumbnail types per-map type
	 */
	void InitializeRenderTypeArray(TArray<struct FThumbnailRenderingInfo>& ThumbnailRendererTypes);

	virtual void BeginDestroy() override;

private:
	/** Initialize the checkerboard texture for texture thumbnails */
	void SetupCheckerboardTexture();

	/** Handler for when an object is edited. Used to dirty its thumbnail. */
	void OnObjectPropertyChanged(UObject* Asset, FPropertyChangedEvent& PropertyChangedEvent);

	/** Handler for when an actor is moved in a level. Used to update world asset thumbnails. */
	void OnActorPostEditMove(AActor* Actor);

	/** Handler to dirty cached thumbnails in packages to make sure they are re-rendered later */
	void DirtyThumbnailForObject(UObject* ObjectBeingModified);
};


