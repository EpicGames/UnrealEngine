// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

/** Refresh frequency for thumbnail rendering, listed from most to least CPU demanding / frequent */
enum class EThumbnailRenderFrequency : uint8
{
	/** Always render when requested, used for assets needing live animated thumbnails like materials */
	Realtime,
	/** Render whenever a property has changed on request */
	OnPropertyChange,
	/** Render only on asset save */
	OnAssetSave,
	/** Render thumbnail only once */
	Once,
};

/**
 * This is an abstract base class that is used to define the interface that
 * UnrealEd will use when rendering a given object's thumbnail. The editor
 * only calls the virtual rendering function.
 */
UCLASS(abstract, MinimalAPI)
class UThumbnailRenderer : public UObject
{
	GENERATED_UCLASS_BODY()


public:
	/**
	 * Returns true if the renderer is capable of producing a thumbnail for the specified asset.
	 *
	 * @param Object the asset to attempt to render
	 */
	virtual bool CanVisualizeAsset(UObject* Object) { return true; }

	/**
	 * Calculates the size the thumbnail would be at the specified zoom level
	 *
	 * @param Object the object the thumbnail is of
	 * @param Zoom the current multiplier of size
	 * @param OutWidth the var that gets the width of the thumbnail
	 * @param OutHeight the var that gets the height
	 */
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const PURE_VIRTUAL(UThumbnailRenderer::GetThumbnailSize,);

	/**
	 * Draws a thumbnail for the object that was specified.
	 *
	 * @param Object the object to draw the thumbnail for
	 * @param X the X coordinate to start drawing at
	 * @param Y the Y coordinate to start drawing at
	 * @param Width the width of the thumbnail to draw
	 * @param Height the height of the thumbnail to draw
	 * @param Viewport the viewport being drawn in
	 * @param Canvas the render interface to draw with
	 * @param bAdditionalViewFamily whether this draw should write over the render target (true) or clear it before (false)
	 */
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* Viewport, FCanvas* Canvas, bool bAdditionalViewFamily) PURE_VIRTUAL(UThumbnailRenderer::Draw, );

	/**
	 * Override this method to control the render frequency for thumbnails in your editor
	 * Generally speaking you should use Realtime if you want an animated thumbnail, but if not it is expensive to use (per-frame draws)
	 * If not using realtime, the next best default is OnAssetSave, which updates the thumbnail only save.
	 * Remaining types are not suggested unless you have specific asset thumbnail update needs.
	 * 
	 * @TODO: The current default of Realtime maintains legacy behavior, but is overally agressive / for frequent than needed for most.
	 *
	 * @return ThumbnailRenderer-specific frequency for thumbnail updates
	 */
	virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const { return EThumbnailRenderFrequency::Realtime; }
	 
protected:
	/** Renders the thumbnail's view family. */
	UNREALED_API static void RenderViewFamily(FCanvas* Canvas, class FSceneViewFamily* ViewFamily, class FSceneView* View);

	UNREALED_API static struct FGameTime GetTime();
};

