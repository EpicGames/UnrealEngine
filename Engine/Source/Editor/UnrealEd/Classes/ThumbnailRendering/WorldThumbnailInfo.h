// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Thumbnail information for world assets
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Camera/CameraTypes.h"
#include "WorldThumbnailInfo.generated.h"

UENUM()
namespace EOrthoThumbnailDirection
{
	enum Type : int
	{
		Top,
		Bottom,
		Left,
		Right,
		Front,
		Back
	};
}

UCLASS(MinimalAPI)
class UWorldThumbnailInfo : public USceneThumbnailInfo
{
	GENERATED_UCLASS_BODY()

	/** The type of projection to use */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	TEnumAsByte<ECameraProjectionMode::Type> CameraMode;

	/** The direction of the camera when in Orthographic CameraMode */
	UPROPERTY(EditAnywhere, Category=Thumbnail)
	TEnumAsByte<EOrthoThumbnailDirection::Type> OrthoDirection;
};
