// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given world
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "WorldThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class FSceneViewFamily;

UCLASS(config=Editor)
class UWorldThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

private:
	/** Allocates then adds an FSceneView to the ViewFamily. */
	[[nodiscard]] FSceneView* CreateView(UWorld* World, FSceneViewFamily* ViewFamily, int32 X, int32 Y, uint32 SizeX, uint32 SizeY) const;

private:
	/** Offset used to orient all worlds to show a more vertical camera, if necessary. Individual thumbnail infos can provide additional offset. */
	UPROPERTY(config)
	float GlobalOrbitPitchOffset;

	/** Offset used to orient all worlds to face the camera in degrees when using a perspective camera. Individual thumbnail infos can provide additional offset. */
	UPROPERTY(config)
	float GlobalOrbitYawOffset;

	/** If true, all world thumbnails will be rendered unlit. This is useful in games that have shared lighting in a common map */
	UPROPERTY(config)
	bool bUseUnlitScene;

	/** If false, all world thumbnails rendering will be disabled */
	UPROPERTY(config)
	bool bAllowWorldThumbnails;
};

