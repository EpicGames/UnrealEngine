// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/SoftObjectPtr.h"
#include "EdGraph/EdGraphPin.h"
#include "UObject/StructOnScope.h"
#include "EditorUndoClient.h"
#include "Kismet2/StructureEditorUtils.h"
#include "StructUtils/UserDefinedStructEditorUtils.h"
#include "UserDefinedStructEditorData.generated.h"

#define UE_API UNREALED_API

class ITransactionObjectAnnotation;

struct FStructVariableMetaData
{
	static UE_API const FName ClampMin;
	static UE_API const FName ClampMax;
	static UE_API const FName UIMin;
	static UE_API const FName UIMax;

private:
	// This struct should never be instantiated
	UE_API FStructVariableMetaData();
};

USTRUCT()
struct FStructVariableDescription
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName VarName;

	UPROPERTY()
	FGuid VarGuid;

	UPROPERTY()
	FString FriendlyName;

	UPROPERTY()
	FString DefaultValue;

	// TYPE DATA
	UPROPERTY()
	FName Category;

	UPROPERTY()
	FName SubCategory;

	UPROPERTY()
	TSoftObjectPtr<UObject> SubCategoryObject;

	UPROPERTY()
	FEdGraphTerminalType PinValueType;

	UPROPERTY()
	EPinContainerType ContainerType;

	// UE_DEPRECATED(4.17)
	UPROPERTY()
	uint8 bIsArray_DEPRECATED:1;

	// UE_DEPRECATED(4.17)
	UPROPERTY()
	uint8 bIsSet_DEPRECATED:1;

	// UE_DEPRECATED(4.17)
	UPROPERTY()
	uint8 bIsMap_DEPRECATED:1;

	UPROPERTY(Transient)
	uint8 bInvalidMember:1;

	UPROPERTY()
	uint8 bDontEditOnInstance:1;

	UPROPERTY()
	uint8 bEnableSaveGame : 1;

	UPROPERTY()
	uint8 bEnableMultiLineText:1;

	UPROPERTY()
	uint8 bEnable3dWidget:1;

	// CurrentDefaultValue stores the actual default value, after the DefaultValue was changed, and before the struct was recompiled
	UPROPERTY(NonTransactional)
	FString CurrentDefaultValue;

	UPROPERTY()
	FString ToolTip;

	UPROPERTY()
	TMap<FName, FString> MetaData;

	UNREALED_API bool SetPinType(const struct FEdGraphPinType& VarType);

	UNREALED_API FEdGraphPinType ToPinType() const;

	/** Return true if it generates the same property. */
	UNREALED_API bool HasSameType(const FStructVariableDescription& Other) const;

	// UE_DEPRECATED(4.17)
	void PostSerialize(const FArchive& Ar);

	FStructVariableDescription()
		: ContainerType(EPinContainerType::None)
		, bIsArray_DEPRECATED(false)
		, bIsSet_DEPRECATED(false)
		, bIsMap_DEPRECATED(false)
		, bInvalidMember(false)
		, bDontEditOnInstance(false)
		, bEnableSaveGame(false)
		, bEnableMultiLineText(false)
		, bEnable3dWidget(false)
	{ }
};

template<>
struct TStructOpsTypeTraits< FStructVariableDescription > : public TStructOpsTypeTraitsBase2< FStructVariableDescription >
{
	enum 
	{
		WithPostSerialize = true,
	};
};

UCLASS(MinimalAPI)
class UUserDefinedStructEditorData : public UUserDefinedStructEditorDataBase, public FEditorUndoClient
{
	GENERATED_UCLASS_BODY()

private:
	// the property is used to generate a unique name id for member variable
	UPROPERTY(NonTransactional) 
	uint32 UniqueNameId;

public:
	UPROPERTY()
	TArray<FStructVariableDescription> VariablesDescriptions;

	UPROPERTY()
	FString ToolTip;

	UPROPERTY()
	TMap<FName, FString> MetaData;

public:
	// UObject interface.
	UNREALED_API virtual TSharedPtr<ITransactionObjectAnnotation> FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const override;
	UNREALED_API virtual void Serialize(FArchive& Ar) override;
	UNREALED_API virtual void PostEditUndo() override;
	UNREALED_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation) override;
	UNREALED_API virtual void PostLoad() override;
	UNREALED_API virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph) override;
	UNREALED_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	// End of UObject interface.

	// FEditorUndoClient interface
	UNREALED_API virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient interface.


	UNREALED_API uint32 GenerateUniqueNameIdForMemberVariable();
	UNREALED_API UUserDefinedStruct* GetOwnerStruct() const;

	UNREALED_API const uint8* GetDefaultInstance() const;
	UNREALED_API virtual void RecreateDefaultInstance(FString* OutLog = nullptr) override;
	UNREALED_API virtual void ReinitializeDefaultInstance(FString* OutLog = nullptr) override;
	UNREALED_API void CleanDefaultInstance();
	UNREALED_API void RefreshValuesFromDefaultInstance();
	
	UNREALED_API virtual FString GetTooltip() const override;
	UNREALED_API virtual FProperty* FindProperty(const UUserDefinedStruct* Struct, FName Name) const override;
	UNREALED_API virtual FString GetFriendlyNameForProperty(const UUserDefinedStruct* Struct, const FProperty* Property) const override;

private:
	// Track the structure change that PostEditUndo undid to pass to FUserDefinedStructureCompilerUtils::CompileStruct
	FStructureEditorUtils::EStructureEditorChangeInfo CachedStructureChange;

	// Utility function for both PostEditUndo to route through
	UNREALED_API void ConsolidatedPostEditUndo(FStructureEditorUtils::EStructureEditorChangeInfo ActiveChange);

	//Whether the MetaData was loaded from the package and saved in UUserDefinedStructEditorData.
	UPROPERTY()
	bool bMetaDataConverted = false;

};

#undef UE_API
