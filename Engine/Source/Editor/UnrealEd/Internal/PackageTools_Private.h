// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::UnrealEd
{
namespace PackageTools_Private
{
DECLARE_MULTICAST_DELEGATE_OneParam(FOnUnloadPackagesDelegate, const TSet<UPackage*> /*PackagesToUnload*/);

/// Broadcast before a given set of packages are unloaded.
UNREALED_API extern FOnUnloadPackagesDelegate OnUnloadPackagesDelegate;

}	 // namespace PackageTools
}	 // namespace UE::UnrealEd
