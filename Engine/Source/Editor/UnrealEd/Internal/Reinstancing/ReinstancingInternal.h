// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Flags to copy during reinstantiation of UObjects */
#define RF_Reinst					((EObjectFlags)(RF_Public | RF_ArchetypeObject | RF_Transactional | \
	RF_Transient | RF_TextExportTransient | RF_InheritableComponentTemplate | RF_Standalone | RF_HasDynamicImports)) 
