// Copyright Epic Games, Inc. All Rights Reserved.

#include "AboutScreen.h"
#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "UnrealEdMisc.h"
#include "IDocumentation.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "AboutScreen"

void SAboutScreen::Construct(const FArguments& InArgs)
{
#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 4428)	// universal-character-name encountered in source
#endif
	AboutLines.Add(MakeShareable(new FLineDefinition(LOCTEXT("Copyright1", "Copyright Epic Games, Inc. All rights reserved.  Epic, Epic Games, Unreal, and their respective logos are trademarks or registered trademarks of Epic Games, Inc. in the United States of America and elsewhere."), 9, FLinearColor(1.f, 1.f, 1.f), FMargin(0.f, 2.f) )));

#ifdef _MSC_VER
#pragma warning(pop)
#endif

	FText Version = FText::Format( LOCTEXT("VersionLabel", "Version: {0}"), FText::FromString( FEngineVersion::Current().ToString( ) ) );
	FText Title = FText::FromString(FApp::GetName());
	
	FString OSLabel, OSSubLabel;
	FPlatformMisc::GetOSVersions(OSLabel, OSSubLabel);
	FText OSName = FText::FromString(OSLabel);
#if PLATFORM_CPU_ARM_FAMILY
	FText ArchName = FText::FromString("arm64");
#else
	FText ArchName = FText::FromString("x86_64");
#endif
	
	FText Platform = FText::Format( LOCTEXT("PlatformArchLabel", "Platform: {0} ({1})"), OSName, ArchName);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(16.f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("AboutScreen.Background")))
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 16.f, 0.f, 0.f)
			[

				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				[

					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.Padding(0.f, 4.f)
					[
						SNew(STextBlock)
						.ColorAndOpacity(FStyleColors::ForegroundHover)
						.Font(FAppStyle::Get().GetFontStyle("AboutScreen.TitleFont"))
						.Text( Title )
					]

					+SVerticalBox::Slot()
					.Padding(0.f, 4.f)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.ColorAndOpacity(FStyleColors::ForegroundHover)
						.Text( Version )
					]
					+SVerticalBox::Slot()
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.ColorAndOpacity(FStyleColors::ForegroundHover)
						.Text(Platform)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.0f, 0.0f, 8.f, 0.0f)
				[
					SAssignNew(UEButton, SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked(this, &SAboutScreen::OnUEButtonClicked)
					.ContentPadding(0.f)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("AboutScreen.UnrealLogo"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.f)
				[
					SAssignNew(EpicGamesButton, SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked(this, &SAboutScreen::OnEpicGamesButtonClicked)
					.ContentPadding(0.f)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("AboutScreen.EpicGamesLogo"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]


			+SVerticalBox::Slot()
			.Padding(FMargin(0.f, 16.f))
			.AutoHeight()
			[
				SNew(SListView<TSharedRef<FLineDefinition>>)
				.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
				.ListItemsSource(&AboutLines)
				.OnGenerateRow(this, &SAboutScreen::MakeAboutTextItemWidget)
				.SelectionMode( ESelectionMode::None )
			] 

			+SVerticalBox::Slot()
			.Padding(FMargin(0.f, 16.f, 0.0f, 0.0f))
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("Close", "Close"))
					.OnClicked(this, &SAboutScreen::OnClose)
				]
			]
		]
	];
}

TSharedRef<ITableRow> SAboutScreen::MakeAboutTextItemWidget(TSharedRef<FLineDefinition> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	if( Item->Text.IsEmpty() )
	{
		return
			SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
			.Style( &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row") )
			.Padding(6.0f)
			[
				SNew(SSpacer)
			];
	}
	else 
	{
		return
			SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
			.Style( &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row") )
			.Padding( Item->Margin )
			[
				SNew(STextBlock)
				.LineHeightPercentage(1.3f)
				.AutoWrapText(true)
				.ColorAndOpacity( Item->TextColor )
				.Font( FCoreStyle::GetDefaultFontStyle("Regular", Item->FontSize) )
				.Text( Item->Text )
			];
	}
}

FReply SAboutScreen::OnUEButtonClicked()
{
	IDocumentation::Get()->OpenHome(FDocumentationSourceInfo(TEXT("logo_docs")));
	return FReply::Handled();
}

FReply SAboutScreen::OnEpicGamesButtonClicked()
{
	FString EpicGamesURL;
	if(FUnrealEdMisc::Get().GetURL( TEXT("EpicGamesURL"), EpicGamesURL ))
	{
		FPlatformProcess::LaunchURL( *EpicGamesURL, NULL, NULL );
	}
	return FReply::Handled();
}

FReply SAboutScreen::OnClose()
{
	TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() ).ToSharedRef();
	FSlateApplication::Get().RequestDestroyWindow( ParentWindow );
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
