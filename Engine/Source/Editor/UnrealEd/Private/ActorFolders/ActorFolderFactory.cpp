// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderFactory.h"

#include "ActorFolders/TedsActorFolderUtils.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "EngineUtils.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFolderFactory)

void UTedsActorFolderFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::ActorFolders;

	TableHandle Table = InDataStorage.RegisterTable(
		UE::Editor::DataStorage::TTypedElementColumnTypeList<
		FFolderTag, FFolderCompatibilityColumn, FTypedElementLabelColumn, FTypedElementLabelHashColumn, FSlateColorColumn, FExpandedInUITag,
		FVisibleInEditorColumn, FFolderVisibilityDirtyTag>(),
	GetActorFolderTableName());

	// Actor folders are registered with TEDS Compat for proper undo/redo support, old FFolders are handled manually in UWorldFolders
	if (ICompatibilityProvider* DataStorageCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		DataStorageCompat->RegisterTypeTableAssociation(UActorFolder::StaticClass(), Table);
	}
}

void UTedsActorFolderFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	using namespace UE::Editor::DataStorage;
	Super::PreRegister(InDataStorage);

	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));
	if (TedsFoldersCvar && TedsFoldersCvar->GetBool())
	{
		FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UTedsActorFolderFactory::PostWorldInitialized);
		FEditorDelegates::EndPIE.AddUObject( this, &UTedsActorFolderFactory::OnEndPIE );
	}
}

void UTedsActorFolderFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	
	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));
	if (TedsFoldersCvar && TedsFoldersCvar->GetBool())
	{
		// For old FFolders (not actor folders), there is actually no separate folder for PIE according to UWorldFolders. The Outliner simply queries
		// an actor's parent folder and adds a row to the UI representing it. To match that behavior for now but make some improvements, we keep that same
		// mechanism of actors in PIE registering their parent folders recursively but also add them to TEDS (but NOT to UWorldFolders so there's no risk 
		// of breaking old workflows)
	
		InDataStorage.RegisterQuery(
			Select(
				TEXT("Add PIE actor's owning folder to TEDS"),
				FObserver::OnAdd<FPieObjectTag>().SetExecutionMode(EExecutionMode::GameThread),
				[&InDataStorage](IQueryContext& Context, RowHandle RowHandle, const FTypedElementUObjectColumn& ActorColumn, const FTypedElementWorldColumn& WorldColumn)
				{
					if (AActor* Actor = Cast<AActor>(ActorColumn.Object))
					{
						FFolder Folder = Actor->GetFolder();
					
						if(!Folder.IsNone())
						{
							// PIE folders are only registered through this observer and nowhere else, so we need to recurse till we reach the root
							constexpr bool bRecursivelyAddParents = true;
							ActorFolders::RegisterFolderInTeds(Context, &InDataStorage, Folder, WorldColumn.World, bRecursivelyAddParents);
						}
					}
				})
			.Where()
				.All<FTypedElementActorTag>()
				.None<FHideRowFromUITag>() // We only want folders for actors visible in the UI
			.Compile());
	}
}

void UTedsActorFolderFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));
	if (TedsFoldersCvar && TedsFoldersCvar->GetBool())
	{
		FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);
	}
	
	Super::PreShutdown(InDataStorage);
}

void UTedsActorFolderFactory::PostWorldInitialized(UWorld* World, const UWorld::InitializationValues InitializationValues)
{
	// With old FFolders (non actor-folders), the legacy behavior was to only create the folders lazily when they were requested by the Outliner UI.
	// This means if the Outliner is closed no folders are added to TEDS. Actor folders on the other hand go through ULevel::FixupActorFolders which
	// implicitly loads them from disk into memory.
	
	// To work around this we explicitly load folders when the world is initialized when the TEDS integration is enabled
	if (GEditor && World)
	{
		FActorFolders::Get().InitializeForWorld(*World);
	}
}

void UTedsActorFolderFactory::OnEndPIE(const bool)
{
	using namespace UE::Editor::DataStorage;
	
	// One ending PIE, we can simply remove all the PIE folders at once instead of needing to do it per actor to mirror how they were added
	if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		DataStorage->RemoveAllRowsWithColumns({FPieObjectTag::StaticStruct(), FFolderCompatibilityColumn::StaticStruct()});
	}
}
