// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "ActorFolderFactory.generated.h"

namespace UE::Editor::DataStorage
{
	struct IQueryContext;
}

class IEditorDataStorageProvider;
class UWorld;
struct FFolder;
class UActorFolder;

/*
 * Factory that manages the registration of folders in TEDS and contains compatibility queries/delegates for feature parity with how legacy FFolders
 * interact with legacy editor systems until we can use TEDS as the source of truth
 */
UCLASS()
class UTedsActorFolderFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UTedsActorFolderFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void PostWorldInitialized(UWorld* World, const UWorld::InitializationValues InitializationValues);
	void OnEndPIE( const bool bIsSimulating );
};
