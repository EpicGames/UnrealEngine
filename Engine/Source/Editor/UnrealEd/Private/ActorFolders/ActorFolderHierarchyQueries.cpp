// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderHierarchyQueries.h"

#include "ActorFolders/TedsActorFolderUtils.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"

namespace UE::Editor::DataStorage::ActorFolders::Private
{
	// Helper to get the mapping domain + key for the  first viable parent for a folder in the editor
	bool GetEditorParentForFolder(const FFolder& Folder, FName& OutParentMappingDomain, FMapKey& OutParentMappingKey)
	{
		FFolder ParentFolder = Folder.GetParent();
		
		// Parent Folder
		if (!ParentFolder.IsNone())
		{
			OutParentMappingDomain = GetFolderMappingDomain(ParentFolder);
			OutParentMappingKey = GetFolderIndex(ParentFolder);
			return true;
		}
		
		UObject* RootObj = ParentFolder.GetRootObjectPtr();
		// Parent Level Instance
		if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(RootObj))
		{
			AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
			OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
			OutParentMappingKey = FMapKey(OwningActor);
			return true;

		}
		// Parent Level Using Actor Folders
		if (ULevel* OwningLevel = Cast<ULevel>(RootObj))
		{
			OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
			OutParentMappingKey = FMapKey(OwningLevel);
		 	return true;
		}
		return false;
	}
}

void UTedsActorFolderHierarchyFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));
	
	if (TedsFoldersCvar && TedsFoldersCvar->GetBool())
	{
		RegisterFolderHierarchyQueries(DataStorage);
	}
}

void UTedsActorFolderHierarchyFactory::RegisterFolderHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	FHierarchyHandle EditorFolderHierarchyHandle = DataStorage.FindHierarchyByName(TEXT("EditorObjectHierarchy"));

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add parent column to folder"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FFolderCompatibilityColumn& FolderColumn)
			{
				FMapKey IdKey;
				FName MappingDomain;
			
				if (ActorFolders::Private::GetEditorParentForFolder(FolderColumn.Folder, MappingDomain, IdKey))
				{
					const RowHandle ParentRow = Context.LookupMappedRow(MappingDomain, IdKey);
			
					if (Context.IsRowAvailable(ParentRow))
					{
						Context.SetParentRow(Row, ParentRow);
						Context.AddColumns<FFolderVisibilityDirtyTag>(ParentRow);
					}
					else
					{
						Context.SetUnresolvedParent(Row, IdKey, MappingDomain);
					}
				}
			})
		.AccessesHierarchy(TEXT("EditorObjectHierarchy"))
		.Where()
			.All<FTypedElementSyncFromWorldTag, FFolderTag>()
			.None(DataStorage.GetChildTagType(EditorFolderHierarchyHandle))
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync folder's parent from world to storage"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle TargetRow, const FFolderCompatibilityColumn& FolderColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				FMapKey IdKey;
				FName MappingDomain;
			
				if (ActorFolders::Private::GetEditorParentForFolder(FolderColumn.Folder, MappingDomain, IdKey))
				{
					RowHandle NewParentRow = Context.LookupMappedRow(MappingDomain, IdKey);
					RowHandle CurrentParentRow = Context.GetParentRow(TargetRow);
					
					if (CurrentParentRow != NewParentRow)
					{
						if (Context.IsRowAvailable(NewParentRow))
						{
							Context.SetParentRow(TargetRow, NewParentRow);
							Context.AddColumns<FFolderVisibilityDirtyTag>({ CurrentParentRow, NewParentRow });
						}
						else
						{
							Context.SetUnresolvedParent(TargetRow, IdKey, MappingDomain);
						}
					}
					return;
				}
					
				Context.SetParentRow(TargetRow, InvalidRowHandle);
			})
		.AccessesHierarchy(TEXT("EditorObjectHierarchy"))
		.Where(TColumn<FTypedElementSyncFromWorldTag>() && TColumn<FFolderTag>() && TColumn(DataStorage.GetChildTagType(EditorFolderHierarchyHandle)))
		.Compile());
}