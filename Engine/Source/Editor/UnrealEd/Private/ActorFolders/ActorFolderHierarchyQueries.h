// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "ActorFolderHierarchyQueries.generated.h"

/*
 * Factory that contains queries that are related to hierarchies for folders
 */
UCLASS()
class UTedsActorFolderHierarchyFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UTedsActorFolderHierarchyFactory() override = default;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

protected:
	void RegisterWorldFolderHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterFolderHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
};


