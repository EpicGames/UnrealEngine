// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolderSyncQueries.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "ActorFolderFactory.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"

void UTedsActorFolderSyncFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	Super::PreRegister(InDataStorage);

	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));
	if (TedsFoldersCvar && TedsFoldersCvar->GetBool() && GEngine)
	{
		DataStorage = &InDataStorage;
		GEngine->OnLevelActorFolderChanged().AddUObject(this, &UTedsActorFolderSyncFactory::OnLevelActorFolderChanged);
	}
}

void UTedsActorFolderSyncFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	const IConsoleVariable* TedsFoldersCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.Folders"));

	if (TedsFoldersCvar  && TedsFoldersCvar->GetBool())
	{
		RegisterLabelQueries(InDataStorage);
		RegisterTypeInfoQueries(InDataStorage);
		RegisterVisibilityQueries(InDataStorage);
		RegisterMiscQueries(InDataStorage);
	}
}

void UTedsActorFolderSyncFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	if (GEngine)
	{
		GEngine->OnLevelActorFolderChanged().RemoveAll(this);
	}
	DataStorage = nullptr;

	Super::PreShutdown(InDataStorage);
}

void UTedsActorFolderSyncFactory::OnLevelActorFolderChanged(const AActor* Actor, FName OldPath)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::ActorFolders;

	if (!DataStorage || !Actor)
	{
		return;
	}

	const FFolder OldFolder(Actor->GetFolderRootObject(), OldPath);
	if (const RowHandle OldFolderRow = LookupMappedRow(DataStorage, OldFolder); DataStorage->IsRowAvailable(OldFolderRow))
	{
		DataStorage->AddColumns<FFolderVisibilityDirtyTag>(OldFolderRow);
	}
}


void UTedsActorFolderSyncFactory::RegisterLabelQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// We have to defer the rename because for legacy FFolders it deletes and re-creates the folder which re-registers them in TEDS. 
	// Which cannot be done while we are in a query callback.

	struct FRenameFolderCommand
	{
		void operator()() const
		{
			FEditorFolderUtils::RenameFolder(Folder,
				FText::FromString(NewLabel), World.Get());
			
		}
		FFolder Folder;
		FString NewLabel;
		TWeakObjectPtr<UWorld> World;
	};
	
	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync folder name to label column"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext& Context, RowHandle RowHandle, const FTypedElementLabelColumn& LabelColumn,
				const FTypedElementLabelHashColumn& LabelHashColumn, const FFolderCompatibilityColumn& FolderColumn,
				const FTypedElementWorldColumn& WorldColumn)
			{
				FString FolderLabel = FolderColumn.Folder.GetLeafName().ToString();
				uint64 FolderLabelHash = CityHash64(reinterpret_cast<const char*>(*FolderLabel), FolderLabel.Len() * sizeof(**FolderLabel));

				if (LabelHashColumn.LabelHash != FolderLabelHash)
				{
					FRenameFolderCommand Command{.Folder = FolderColumn.Folder, .NewLabel = LabelColumn.Label, .World = WorldColumn.World};
					Context.PushCommand(Command);
				}
			}
		)
		.Where(TColumn<FFolderTag>() && TColumn<FTypedElementSyncBackToWorldTag>())
		.Compile()
	);

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync label column to folder name"),
			FProcessor(EQueryTickPhase::PostPhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FFolderCompatibilityColumn& FolderColumn,
				FTypedElementLabelColumn& LabelColumn, FTypedElementLabelHashColumn& LabelHashColumn)
			{
				FString FolderLabel = FolderColumn.Folder.GetLeafName().ToString();
				uint64 FolderLabelHash = CityHash64(reinterpret_cast<const char*>(*FolderLabel), FolderLabel.Len() * sizeof(**FolderLabel));

				if (LabelHashColumn.LabelHash != FolderLabelHash)
				{
					LabelColumn.Label = FolderLabel;
					LabelHashColumn.LabelHash = FolderLabelHash;
				}
			}
		)
		.Where(TColumn<FFolderTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
		);
}

void UTedsActorFolderSyncFactory::RegisterTypeInfoQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	const FText FolderTypeName = NSLOCTEXT("TedsActorFolderSyncFactory", "FolderTypeName", "Folder");
	
	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync folder type name to type info display override column"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this, FolderTypeName](IQueryContext& Context, RowHandle RowHandle)
			{
				Context.AddColumn(RowHandle, FTypedElementTypeInfoDisplayOverrideColumn{ .TypeDisplayName = FolderTypeName });
			}
		)
		.Where()
			.All<FFolderCompatibilityColumn, FTypedElementSyncFromWorldTag>()
			.None<FTypedElementTypeInfoDisplayOverrideColumn>()
		.Compile()
	);
}

void UTedsActorFolderSyncFactory::RegisterVisibilityQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	FolderQuery = InDataStorage.RegisterQuery(
		Select()
		.Where(TColumn<FFolderTag>())
		.Compile());

	const FName EditorObjectHierarchyName("EditorObjectHierarchy");
	const FHierarchyHandle Hierarchy = InDataStorage.FindHierarchyByName(EditorObjectHierarchyName);

	struct FSyncFolderVisibilityCommand
	{
		void operator()() const
		{
			if (Storage)
			{
				bool bVisible = false;
				Storage->IterateChildren(HierarchyHandle, Row,
				[&](const ICoreProvider& SubStorage, RowHandle Child) -> bool
					{
						if (const FVisibleInEditorColumn* VisibleColumn = SubStorage.GetColumn<FVisibleInEditorColumn>(Child))
						{
							if (VisibleColumn->bIsVisibleInEditor)
							{
								bVisible = true;
							}
						}
						// Continue till we find a visible child or there are no more children to evaluate
						return !bVisible;
					});
			
				if (FVisibleInEditorColumn* VisibilityColumn = Storage->GetColumn<FVisibleInEditorColumn>(Row))
				{
					if (VisibilityColumn->bIsVisibleInEditor != bVisible)
					{
						VisibilityColumn->bIsVisibleInEditor = bVisible;
						Storage->AddColumns<FVisibilityChangedTag>(Row);
					}
				}
				Storage->RemoveColumns<FFolderVisibilityDirtyTag>(Row);
			}
		}
		RowHandle Row;
		FHierarchyHandle HierarchyHandle;
		ICoreProvider* Storage = nullptr;
	};

	// When an object's visibility changes, dirty its parent folder so the walk recomputes it.
	InDataStorage.RegisterQuery(
		Select(
			TEXT("Add sync tag to folder parent hierarchy on visibility changed"),
			FObserver::OnAdd<FVisibilityChangedTag>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, const RowHandle Row)
			{
				const RowHandle ParentRow = Context.GetParentRow(Row);
				Context.RunSubquery(0, ParentRow, CreateSubqueryCallbackBinding([](ISubqueryContext& SubContext, RowHandle TargetRow)
				{
					SubContext.AddColumns<FFolderVisibilityDirtyTag>(TargetRow);
				}));
			}
		)
		.AccessesHierarchy(EditorObjectHierarchyName)
		.DependsOn()
			.SubQuery(FolderQuery)
		.Compile());

	if (const UScriptStruct* ParentChangedColumn = InDataStorage.GetParentChangedColumnType(Hierarchy))
	{
		// When a row's parent changes, dirty its new parent folder (if it has one) so visibility is recomputed. Folders do
		// not get this tag since they are reconstructed, so this addition is added in the Folder Hierarchy syncs directly
		InDataStorage.RegisterQuery(
			Select(
				TEXT("Add sync tag to folder parent on hierarchy changed"),
				FProcessor(EQueryTickPhase::DuringPhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, const RowHandle Row)
				{
					const RowHandle ParentRow = Context.GetParentRow(Row);
					Context.RunSubquery(0, ParentRow, CreateSubqueryCallbackBinding([](ISubqueryContext& SubContext, RowHandle TargetRow)
					{
						SubContext.AddColumns<FFolderVisibilityDirtyTag>(TargetRow);
					}));
				}
			)
			.AccessesHierarchy(EditorObjectHierarchyName)
			.Where(TColumn(ParentChangedColumn))
			.DependsOn()
				.SubQuery(FolderQuery)
			.Compile());
	}

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Recompute folder visibility"),
			FProcessor(EQueryTickPhase::FrameEnd, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[&InDataStorage, Hierarchy](IQueryContext& Context, RowHandle Row)
			{
				Context.PushCommand(FSyncFolderVisibilityCommand
				{
					.Row = Row,
					.HierarchyHandle = Hierarchy,
					.Storage = &InDataStorage
				});
			})
		.AccessesHierarchy(EditorObjectHierarchyName)
		.Where(TColumn<FFolderTag>() && TColumn<FFolderVisibilityDirtyTag>())
		.DependsOn()
			.SubQuery(FolderQuery)
		.Compile()
	);
}

void UTedsActorFolderSyncFactory::RegisterMiscQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// For actor folders, the FFolder data can change without the row getting unregistered and re-registered in TEDS (unlike legacy FFolders)
	// This query makes sure the FFolder in TEDS has the correct data
	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync FFolder from ActorFolder"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this](RowHandle RowHandle, const FTypedElementUObjectColumn& ObjectColumn, FFolderCompatibilityColumn& FolderColumn)
			{
				if (UActorFolder* ActorFolder = Cast<UActorFolder>(ObjectColumn.Object.Get(true)))
				{
					FolderColumn.Folder = ActorFolder->GetFolder();
				}
			}
		)
		.Where(TColumn<FFolderTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
	);
	
	InDataStorage.RegisterQuery(
		Select(
			TEXT("Add level column to folder row"),
			FProcessor(EQueryTickPhase::DuringPhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext& Context, RowHandle RowHandle, FRootObjectColumn& RootObjectColumn)
			{
				UObject* RootObj = FFolder::GetRootObjectPtr(RootObjectColumn.RootObject);
					
				if (ULevel* OwningLevel = Cast<ULevel>(RootObj))
				{
					Context.AddColumn(RowHandle, FTypedElementLevelColumn{.Level = OwningLevel});
				}
				else
				{
					Context.RemoveColumns<FTypedElementLevelColumn>(RowHandle);
				}
			}
		)
		.Where(TColumn<FFolderTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
	);

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync FFolder root object to FRootObjectColumn"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext&, RowHandle, const FFolderCompatibilityColumn& FolderColumn, FRootObjectColumn& RootObjectColumn)
			{
				RootObjectColumn.RootObject = FolderColumn.Folder.GetRootObject();
			}
		)
		.Where(TColumn<FFolderTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
	);
}