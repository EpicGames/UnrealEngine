// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "ActorFolderSyncQueries.generated.h"

class AActor;

/*
 * Factory that contains queries that are used to sync general data for actor folders to and from TEDS
 */
UCLASS()
class UTedsActorFolderSyncFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UTedsActorFolderSyncFactory() override = default;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterLabelQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	void RegisterTypeInfoQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	void RegisterVisibilityQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterMiscQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	void OnLevelActorFolderChanged(const AActor* Actor, FName OldPath);

	UE::Editor::DataStorage::QueryHandle FolderQuery = UE::Editor::DataStorage::InvalidQueryHandle;

	UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
};


