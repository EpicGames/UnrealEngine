// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "DataStorage/Features.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "Editor/EditorEngine.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"

namespace UE::Editor::ActorFolders::Tests
{
	using namespace UE::Editor::DataStorage;
	
	BEGIN_DEFINE_SPEC(TedsActorFolderTestFixture, "Editor.ActorFolders.Teds", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	
	ICoreProvider* TedsInterface = nullptr;
	UWorld* World = nullptr;
	
	TSet<FFolder> Folders;
	
	FFolder CreateNewFolder(FFolder ParentFolder)
	{
		const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderName(*World, ParentFolder);
		
		// Keep a record of all the folders we create so we can delete them later
		Folders.Add(NewFolderName);
		
		if (FActorFolders::Get().CreateFolder(*World, NewFolderName))
		{
			return NewFolderName;
		}
		
		return FFolder::GetInvalidFolder();
	}
	
	void DeleteFolder(FFolder InFolder)
	{
		// Not doing this on purpose as tests call undo/redo which might bring the folder back without adding them back to the Set
		// It's safer to just always cleanup folders even if it was already deleted
		//Folders.Remove(InFolder);
		
		FActorFolders::Get().DeleteFolder(*World, InFolder);
	}
	
	void CleanupFolders()
	{
		for (const FFolder& Folder : Folders)
		{
			FActorFolders::Get().DeleteFolder(*World, Folder);
		}
	}
	
	END_DEFINE_SPEC(TedsActorFolderTestFixture)
    
    void TedsActorFolderTestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTrue("", TedsInterface != nullptr);
			
			World = GEditor->GetEditorWorldContext().World();
		});
		
		Describe("Verify Immediate Teds State", [this]
		{
			It(TEXT("Create Folder"), EAsyncExecution::TaskGraphMainTick, [this]()
			{
				FFolder Folder = CreateNewFolder(FFolder::GetInvalidFolder());
				TestTrue(TEXT("Expected folder to be created successfully"), Folder.IsValid());
				
				RowHandle FolderRow = UE::Editor::DataStorage::ActorFolders::LookupMappedRow(TedsInterface, Folder);
				TestTrue(TEXT("Expected folder row to be valid in TEDS"), TedsInterface->IsRowAvailable(FolderRow));
			});
				
			It(TEXT("Delete Folder"), EAsyncExecution::TaskGraphMainTick, [this]()
			{
				FFolder Folder = CreateNewFolder(FFolder::GetInvalidFolder());
				TestTrue(TEXT("Expected folder to be created successfully"), Folder.IsValid());
				
				DeleteFolder(Folder);
				
				RowHandle FolderRow = UE::Editor::DataStorage::ActorFolders::LookupMappedRow(TedsInterface, Folder);
				TestFalse(TEXT("Expected folder row to be invalid in TEDS after deletion"), TedsInterface->IsRowAvailable(FolderRow));
			});	
				
			It(TEXT("Undo Folder Creation"), EAsyncExecution::TaskGraphMainTick, [this]()
			{
				FFolder Folder = CreateNewFolder(FFolder::GetInvalidFolder());
				TestTrue(TEXT("Expected folder to be created successfully"), Folder.IsValid());
			
				GEditor->UndoTransaction();
			
				RowHandle FolderRow = UE::Editor::DataStorage::ActorFolders::LookupMappedRow(TedsInterface, Folder);
				TestFalse(TEXT("Expected folder row to be invalid in TEDS after undo-ing creation"), TedsInterface->IsRowAvailable(FolderRow));
			});
				
			It(TEXT("Undo Folder Deletion"), EAsyncExecution::TaskGraphMainTick, [this]()
			{
				FFolder Folder = CreateNewFolder(FFolder::GetInvalidFolder());
				TestTrue(TEXT("Expected folder to be created successfully"), Folder.IsValid());
					
				DeleteFolder(Folder);
				GEditor->UndoTransaction();
					
				RowHandle FolderRow = UE::Editor::DataStorage::ActorFolders::LookupMappedRow(TedsInterface, Folder);
				TestTrue(TEXT("Expected folder row to be valid in TEDS after undo-ing deletion"), TedsInterface->IsRowAvailable(FolderRow));
			});
		});
		
		AfterEach([this]()
		{
			CleanupFolders();
			TedsInterface = nullptr;
		});
	}
}

#endif