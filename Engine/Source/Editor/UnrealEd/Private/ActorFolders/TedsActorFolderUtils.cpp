// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolders/TedsActorFolderUtils.h"

#include "ActorFolder.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "DataStorage/Features.h"
#include "Editor.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/World.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Selection.h"

namespace UE::Editor::DataStorage::ActorFolders
{
	namespace Private
	{
		static const FName TableName("Editor_ActorFolderTable");

		// Kept private on purpose because actor folders and non-actor folders are mapped under different domains and keys and we want the user to go through
		// LookupMappedRow instead of needing to know which one a folder is
		static const FName MappingDomain("ActorFolders");
		
		// Internal helper to get the parent for an actor, since the FolderToIgnore argument is only used internally and not something we want to expose
		// on the public function
		bool GetEditorParentForActor_Internal(const AActor* InActor, FName& OutParentMappingDomain, FMapKey& OutParentMappingKey, const FFolder& FolderToIgnore)
		{
			// For the editor hierarchy, the order to check is Parent Actor -> Folder -> Level Instance -> Owning Level -> Owning World
			if (const AActor* ParentActor = InActor->GetSceneOutlinerParent())
			{
				// Currently the editor hierarchy for actors mirrors 
				if (ParentActor->IsListedInSceneOutliner())
				{
					OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
					OutParentMappingKey = FMapKey(ParentActor);
					return true;
				}
			}
		
			FFolder Folder = InActor->GetFolder();
			
			// If this folder is meant to be ignored, get the next parent folder instead
			if (!Folder.IsNone() && Folder == FolderToIgnore)
			{
				Folder = Folder.GetParent();
			}
			
			if (!Folder.IsNone())
			{
				OutParentMappingDomain = GetFolderMappingDomain(Folder);
				OutParentMappingKey = GetFolderIndex(Folder);
				return true;
			}
		
			// Check level instance through the folder's root object
			if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(Folder.GetRootObjectPtr()))
			{
				AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
				OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
				OutParentMappingKey = FMapKey(OwningActor);
				return true;
			}

			if (const ULevel* ParentLevel = InActor->GetLevel())
			{
				OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
				OutParentMappingKey = FMapKey(ParentLevel);
				return true;
			}

			if (const UWorld* ParentWorld = InActor->GetWorld())
			{
				OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
				OutParentMappingKey = FMapKey(ParentWorld);
				return true;
			}
		
			return false;
		}
		
		void FixupChildActorsOnFolderDelete(ICoreProvider* Storage, const FFolder& Folder)
		{
			RowHandle FolderRow = LookupMappedRow(Storage, Folder);
			FHierarchyHandle FolderHierarchy = Storage->FindHierarchyByName("EditorObjectHierarchy");
			FRowHandleArray ChildActors;
			
			// Grab all actors in this folder
			Storage->IterateChildren(FolderHierarchy, FolderRow, [&ChildActors](const ICoreProvider& Context, RowHandle ChildRow)
				{
					if (Context.HasColumns<FTypedElementActorTag>(ChildRow))
					{
						ChildActors.Add(ChildRow);
					}
					return true;
				});

			for (RowHandle ChildRow : ChildActors.GetRows())
			{
				if (const FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(ChildRow))
				{
					if (AActor* Actor = Cast<AActor>(ObjectColumn->Object))
					{
						FName ParentMappingDomain;
						FMapKey ParentIdKey;
							
						// We want to update the hierarchy of this actor as it would be AFTER this folder is deleted, so we set 
						// FolderToIgnore == Folder
						if (GetEditorParentForActor_Internal(Actor, ParentMappingDomain, ParentIdKey, Folder))
						{
							RowHandle NewParentRow = Storage->LookupMappedRow(ParentMappingDomain, ParentIdKey);
								
							if (Storage->IsRowAvailable(NewParentRow))
							{
								Storage->SetParentRow(FolderHierarchy, ChildRow, NewParentRow);
							}
						}
						
						// This is a workaround for lack of proper undo/redo support. We'll set an unresolved parent for the actor with the mapping
						// index of the folder we are deleting. This makes it so when if the folder comes back because of undo/redo the actor 
						// is moved back to it.
						// If an undo/redo never happens the actor will simply continue to exist under the next viable parent we just set before this
						// until another sync event calls SetParent() or SetUnresolvedParent() to overwrite this
						Storage->SetUnresolvedParent(FolderHierarchy, ChildRow, GetFolderIndex(Folder), GetFolderMappingDomain(Folder));
					}
				}
			}
		}
		
		RowHandle RegisterFolder_Internal(ICoreProvider* Storage, const FFolder& Folder)
		{
			// Register actor folders with TEDS compat since they are UObjects
			if (Folder.GetActorFolder())
			{
				ICompatibilityProvider* StorageCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
				return StorageCompat->AddCompatibleObjectExplicit(Folder.GetActorFolder());
			}
			
			// Regular folders are registered and kept track of manually
			else
			{
				TableHandle FolderTable = Storage->FindTable(GetActorFolderTableName());
				checkf(Storage->IsTableValid(FolderTable), TEXT("Cannot register folders in TEDS if the folder table is not registered."));

				RowHandle Row = Storage->AddRow(FolderTable);
				Storage->MapRow(GetFolderMappingDomain(Folder), GetFolderIndex(Folder), Row);
				return Row;
			}

		}
		
		void UnregisterFolder_Internal(ICoreProvider* Storage, const FFolder& Folder)
		{
			// Sync the parent folder (if it has one) before removing this row so it can update after losing the child folder
			const FFolder ParentFolder = Folder.GetParent();
			if (!ParentFolder.IsNone())
			{
				const RowHandle ParentRow = LookupMappedRow(Storage, ParentFolder);
				if (Storage->IsRowAvailable(ParentRow) && Storage->HasColumns<FFolderTag>(ParentRow))
				{
					Storage->AddColumn<FFolderVisibilityDirtyTag>(ParentRow);
				}
			}
			
			if (Folder.GetActorFolder())
			{
				// Child actors are not dirtied when an actor folder is deleted, so we need to manually fix up their hierarchy in TEDS and
				// handle the actor folder coming back because of undo/redo
				FixupChildActorsOnFolderDelete(Storage, Folder);
				
				ICompatibilityProvider* StorageCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
				StorageCompat->RemoveCompatibleObjectExplicit(Folder.GetActorFolder());
				
			}
			else
			{
				Storage->RemoveRow(LookupMappedRow(Storage, Folder));
			}
		}
	}
	
	FName GetActorFolderTableName()
	{
		return Private::TableName;
	}

	void SetFolderColumns(ICoreProvider* Storage, RowHandle Row, UWorld* World, const FFolder& Folder)
	{
		Storage->AddColumn(Row, FTypedElementWorldColumn{.World = TWeakObjectPtr<UWorld>(World)});
		
		// This is also done through a processor for all rows with a world, but PIE folders have a specific lifecycle in TEDS that prevents
		// this from being added the correct time, so we do it explicitly here as well
		if (World->IsPlayInEditor())
		{
			Storage->AddColumn<FPieObjectTag>(Row);
		}
		
		Storage->AddColumn(Row, FFolderCompatibilityColumn{.Folder = Folder});
		Storage->AddColumn(Row, FRootObjectColumn{.RootObject = Folder.GetRootObject()});

		if (UActorFolder* ActorFolder = Folder.GetActorFolder())
		{
			Storage->AddColumn(Row, FTypedElementUObjectColumn{.Object = ActorFolder});
			Storage->AddColumn(Row, FTypedElementClassTypeInfoColumn{ .TypeInfo = UActorFolder::StaticClass() });
		}

		// Add the sync tag so other columns are synced through processors correctly
		Storage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
	}
	
	RowHandle RegisterFolderInTeds(ICoreProvider* Storage, const FFolder& Folder, UWorld* World, bool bRecursivelyAddParents)
	{
		// Walk up the parent chain and register parent folders if requested. The hierarchy is setup by a processor so we don't have to do it
		// here
		if (bRecursivelyAddParents)
		{
			if (FFolder ParentFolder = Folder.GetParent(); !ParentFolder.IsNone())
			{
				RegisterFolderInTeds(Storage, ParentFolder, World, bRecursivelyAddParents);
			}
		}
		
		RowHandle ExistingRow = LookupMappedRow(Storage, Folder);

		if (Storage->IsRowAvailable(ExistingRow))
		{
			return ExistingRow;
		}
		
		RowHandle FolderRow = Private::RegisterFolder_Internal(Storage, Folder);
		
		if (Storage->IsRowAvailable(FolderRow))
		{
			SetFolderColumns(Storage, FolderRow, World, Folder);
			return FolderRow;
		}

		return InvalidRowHandle;
	}

	void RegisterFolderInTeds(IQueryContext& Context, ICoreProvider* Storage, const FFolder& Folder, TWeakObjectPtr<UWorld> World, bool bRecursivelyAddParents)
	{
		struct FRegisterFolderCommand
		{
			void operator()() const
			{
				if (UWorld* World = WorldPtr.Get())
				{
					RegisterFolderInTeds(Storage, Folder, World, bRecursivelyAddParents);
				}
			}

			ICoreProvider* Storage;
			FFolder Folder;
			TWeakObjectPtr<UWorld> WorldPtr;
			bool bRecursivelyAddParents;
		};

		FRegisterFolderCommand Command
		{
			.Storage = Storage,
			.Folder = Folder,
			.WorldPtr = World,
			.bRecursivelyAddParents = bRecursivelyAddParents
		};
		Context.PushCommand(Command);
	}

	void UnregisterFolderFromTeds(ICoreProvider* Storage, const FFolder& Folder)
	{
		Private::UnregisterFolder_Internal(Storage, Folder);
	}
	
	void UnregisterFolderFromTeds(IQueryContext& Context, ICoreProvider* Storage, const FFolder& Folder)
	{
		struct FUnregisterFolderCommand
		{
			void operator()() const
			{
				UnregisterFolderFromTeds(Storage, Folder);
			}

			ICoreProvider* Storage;
			FFolder Folder;
		};

		FUnregisterFolderCommand Command
		{
			.Storage = Storage,
			.Folder = Folder,
		};
		Context.PushCommand(Command);
	}

	RowHandle LookupMappedRow(const ICoreProvider* Storage, const FFolder& Folder)
	{
		return Storage->LookupMappedRow(GetFolderMappingDomain(Folder), GetFolderIndex(Folder));
	}

	RowHandle LookupMappedRow(const IQueryContext& Context, const FFolder& Folder)
	{
		return Context.LookupMappedRow(GetFolderMappingDomain(Folder), GetFolderIndex(Folder));
	}

	RowHandle RemapFolder(ICoreProvider* Storage, const FFolder& InOldFolder, const FFolder& InNewFolder, UWorld* World)
	{
		if (!InNewFolder.GetActorFolder())
		{
			// Remap this row to the new folder information
            FMapKey OldIndex = GetFolderIndex(InOldFolder);
            FMapKey NewIndex = GetFolderIndex(InNewFolder);
    
            // Actor folders are mapped by the UObject pointer which doesn't really change, so there is no need to remap them if the index didn't change
            // OR if InOldFolder is not actually mapped to any row (i.e it's a stale FFolder that simply needs updating in FFolderCompatibilityColumn)
            if (OldIndex != NewIndex && Storage->IsRowAvailable(Storage->LookupMappedRow(GetFolderMappingDomain(InOldFolder), OldIndex)))
            {
            	Storage->RemapRow(GetFolderMappingDomain(InNewFolder), OldIndex, NewIndex);
            }
		}
		
		RowHandle NewFolderRow = LookupMappedRow(Storage, InNewFolder);
		
		// We want to set the default columns again even if the index didn't change, e.g an actor folder is mapped by the UObject ptr but the FFolder
		// associated with it has changed and FFolderCompatibilityColumn needs updating
		SetFolderColumns(Storage, NewFolderRow, World, InNewFolder);
		
		return NewFolderRow;
	}

	void SetUnresolvedFolderParent(IQueryContext& Context, RowHandle ChildRow, const FFolder& ParentFolder)
	{
		FMapKey ParentIndex = GetFolderIndex(ParentFolder);
		Context.SetUnresolvedParent(ChildRow, ParentIndex, GetFolderMappingDomain(ParentFolder));
	}
	
	FName GetFolderMappingDomain(const FFolder& Folder)
	{
		// Actor folders are registered with TEDS Compat
		if (Folder.GetActorFolder())
		{
			return ICompatibilityProvider::ObjectMappingDomain;
		}

		return Private::MappingDomain;
	}

	FMapKey GetFolderIndex(const FFolder& Folder)
	{
		// Actor folders are mapped by the UObject itself because it is stable across operations that create redirectors
		if (UActorFolder* ActorFolder = Folder.GetActorFolder())
		{
			return FMapKey(ActorFolder);
		}
		else
		{
			// We also use the root object as a part of the hash since two folders in different worlds (e.g Editor and PIE) can have the same hash
			return FMapKey(static_cast<uint64>(HashCombine(GetTypeHash(Folder.GetRootObject()), GetTypeHash(Folder))));
		}
	}
	
	void FixupActorPostDuplicate(AActor* Actor)
	{
		using namespace UE::Editor::DataStorage;
	
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FFolder ParentFolder = Actor->GetFolder();
			
			RowHandle FolderRow = LookupMappedRow(Storage, ParentFolder);

			// If the parent folder has the duplicate event column, it was also part of the duplicate operation and we want to actor to be
			// contained by the duplicate folder instead of the original folder, so follow the redirector.
			
			// Pre-Duplicate:
			// Folder
			//  - Actor
			
			// Post Duplicate:
			// Folder
			//  - Actor
			// Folder1
			//  - Actor1
			if (FFolderDuplicateEventColumn* Column = Storage->GetColumn<FFolderDuplicateEventColumn>(FolderRow))
			{
				// Make sure actor has the same root object before updating path
				if (Actor->GetFolderRootObject() == Column->DuplicatedFolder.GetRootObject())
				{
					Actor->SetFolderPath_Recursively(Column->DuplicatedFolder.GetPath());
				}
			}
		}
	}

	bool DeselectFolderIfSelected(ICoreProvider* Storage, const FFolder& InFolder)
	{
		if (!GEditor || !Storage)
		{
			return false;
		}

		USelection* SelectedActors = GEditor->GetSelectedActors();
		if (!SelectedActors)
		{
			return false;
		}

		UTypedElementSelectionSet* SelectionSet = SelectedActors->GetElementSelectionSet();
		if (!SelectionSet)
		{
			return false;
		}

		const RowHandle Row = LookupMappedRow(Storage, InFolder);
		if (!Storage->IsRowAvailable(Row))
		{
			return false;
		}

		const FTypedElementHandle Handle = UE::Editor::ActorFolders::AcquireTypedElementHandle(Row, /*bAllowCreate*/false);
		if (!Handle || !SelectionSet->IsElementSelected(Handle, FTypedElementIsSelectedOptions()))
		{
			return false;
		}

		const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
			.SetAllowHidden(true)
			.SetAllowGroups(false)
			.SetWarnIfLocked(false)
			.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);
		SelectionSet->DeselectElement(Handle, SelectionOptions);
		return true;
	}

	void SelectFolder(ICoreProvider* Storage, const FFolder& InFolder)
	{
		if (!GEditor || !Storage)
		{
			return;
		}

		USelection* SelectedActors = GEditor->GetSelectedActors();
		if (!SelectedActors)
		{
			return;
		}

		UTypedElementSelectionSet* SelectionSet = SelectedActors->GetElementSelectionSet();
		if (!SelectionSet)
		{
			return;
		}

		const RowHandle Row = LookupMappedRow(Storage, InFolder);
		if (!Storage->IsRowAvailable(Row))
		{
			return;
		}

		if (FTypedElementHandle Handle = UE::Editor::ActorFolders::AcquireTypedElementHandle(Row, /*bAllowCreate*/true))
		{
			const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
				.SetAllowHidden(true)
				.SetAllowGroups(false)
				.SetWarnIfLocked(false)
				.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);
			SelectionSet->SelectElement(Handle, SelectionOptions);
		}
	}
	
	bool GetEditorParentForActor(const AActor* InActor, FName& OutParentMappingDomain, FMapKey& OutParentMappingKey)
	{
		return Private::GetEditorParentForActor_Internal(InActor, OutParentMappingDomain, OutParentMappingKey, FFolder());
	}
}
