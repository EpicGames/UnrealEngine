// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolders/TypedElements/ActorFolderTypedElementFactory.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Misc/CoreDelegates.h"

void UTedsActorFolderTypedElementFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UTedsActorFolderTypedElementFactory::OnEnginePreExit);
}

void UTedsActorFolderTypedElementFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Generated TypedElement handle for folder"),
			FObserver::OnAdd<FFolderCompatibilityColumn>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle RowHandle)
			{
				constexpr bool bAllowCreate = true;
				FTypedElementHandle Handle = UE::Editor::ActorFolders::AcquireTypedElementHandle(RowHandle, bAllowCreate);
					
				Context.AddColumn(RowHandle, Compatibility::FTypedElementColumn
					{
						.Handle = MoveTemp(Handle)
					});
			})
		.Compile());
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Destroy TypedElement handle for folder"),
			FObserver::OnRemove<FFolderCompatibilityColumn>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle RowHandle)
			{
				UE::Editor::ActorFolders::DestroyTypedElementHandle(RowHandle);
					
				// Not necessary since this implies the folder row is being removed, but still good to do just in case that assumption changes
				Context.RemoveColumns<Compatibility::FTypedElementColumn>(RowHandle);
			})
		.Compile());
	
	// This event column is added to folders when they are duplicated or pasted and then removed at the end of the frame
	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove folder duplicate event column"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate)),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FFolderDuplicateEventColumn>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			}
		)
		.Where()
			.All<FFolderDuplicateEventColumn>()
		.Compile());
}

void UTedsActorFolderTypedElementFactory::OnEnginePreExit()
{
	using namespace UE::Editor::DataStorage;
	
	// We currently use the lifecycle of folder rows in TEDS to control the lifecycle of their TEv1 handles. But the destruction of these rows
	// is not reliable since it depends on the destruction order of ICoreProvider vs UWorldFolders (both UObjects) which can lead to cases where
	// the editor is shutdown without TEDS removing those rows, which leads to the typed element handles leaking.
	// To work around this, we explicitly remove the rows before engine exit
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		Storage->RemoveAllRowsWithColumns({FFolderCompatibilityColumn::StaticStruct()});
	}
}
