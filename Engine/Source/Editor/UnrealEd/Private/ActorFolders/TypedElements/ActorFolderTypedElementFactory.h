// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "ActorFolderTypedElementFactory.generated.h"

namespace UE::Editor::DataStorage
{
	struct IQueryContext;
}

class IEditorDataStorageProvider;
class UWorld;
struct FFolder;
class UActorFolder;

/*
 * Factory that manages the registration of folders into the typed elements system
 */
UCLASS()
class UTedsActorFolderTypedElementFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UTedsActorFolderTypedElementFactory() override = default;

	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	
protected:
	void OnEnginePreExit();
};
