// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolders/TypedElements/ActorFolderTypedElementInterfaces.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"
#include "EditorActorFolders.h"
#include "EditorFolderUtils.h"
#include "Engine/Level.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/World.h"
#include "Folder.h"
#include "ScopedTransaction.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "GameFramework/Actor.h"

static bool bEnableNewInterface = false;

// Cvar to control enabling of Typed Elements copy/paste/duplicate/delete for folders
static FAutoConsoleVariableRef CVarEnableInterface(
	TEXT("TEDS.Feature.Folders.TypedElements"),
	bEnableNewInterface,
	TEXT("Test folder TEv1 interface"));

namespace UE::Editor::ActorFolders::TypedElements::Private
{
	// Helper struct to contain both the FFolder and RowHandle for a folder during operations
	struct FFolderData
	{
		FFolder Folder;
		DataStorage::RowHandle FolderRow;
	};
	
	void GetFolderDataFromHandles(DataStorage::ICoreProvider* Storage, TArrayView<const FTypedElementHandle> InElementHandles, TArray<FFolderData>& OutFolderData)
	{
		for (const FTypedElementHandle& Handle : InElementHandles)
		{
			DataStorage::RowHandle FolderRow = GetFolderRow(Handle);
		
			if (FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(FolderRow))
			{
				OutFolderData.Add(FFolderData{.Folder = FolderColumn->Folder, .FolderRow = FolderRow});
			}
		}
	}
	
	// Internal function to handle duplicate and paste of folders since they follow the same logic. 
	// Mostly mirrors how the logic worked using legacy delegates in SSceneOutliner.cpp
	void DuplicateFolders_Internal(DataStorage::ICoreProvider& Storage, TArray<FFolderData> FoldersToDuplicate, UWorld& InWorld, TArray<FTypedElementHandle>& OutDuplicatedFolders)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::ActorFolders;

		using namespace UE::Editor::ActorFolders;
		
		// Mirrors FActorMode::GetPasteTargetRootObject
		FFolder::FRootObject FoldersEditRootObject 
				= FFolder::GetOptionalFolderRootObject(InWorld.GetCurrentLevel()).Get(FFolder::GetWorldRootFolder(&InWorld).GetRootObject());

		// Sort in ascending order so parents appear before children
		FoldersToDuplicate.Sort([](const FFolderData& FolderA, const FFolderData& FolderB)
		{
			return FolderA.Folder.GetPath().LexicalLess(FolderB.Folder.GetPath());
		});
			
		// Prepare CacheFolderMap which maps old to new/duplicate folder names
		TMap<FName, FName> CacheFolderMap;
			
		for (const FFolderData& Folder : FoldersToDuplicate)
		{
			FName FolderPath = Folder.Folder.GetPath();
			FName ParentPath = FEditorFolderUtils::GetParentPath(FolderPath);
			FName LeafName = FEditorFolderUtils::GetLeafName(FolderPath);
			if (LeafName != TEXT(""))
			{
				if (FName* NewParentPath = CacheFolderMap.Find(ParentPath))
				{
					ParentPath = *NewParentPath;
				}
				FFolder NewFolderPath = FActorFolders::Get().GetFolderName(InWorld, FFolder(FoldersEditRootObject, ParentPath), LeafName);
				CacheFolderMap.Add(FolderPath, NewFolderPath.GetPath());
			}
		}
			
		const FScopedTransaction Transaction(NSLOCTEXT("ActorFolderTypedElements", "DuplicateFolders", "Duplicate Folders"));
			
		for (const FFolderData& Folder : FoldersToDuplicate)
		{
			if (FName* NewFolderName = CacheFolderMap.Find(Folder.Folder.GetPath()))
			{
				FFolder NewFolder(FoldersEditRootObject, *NewFolderName);
				// When using Actor Folder, duplicated actors might already have created the actor folder (when destination rootobject is different)
				if (FActorFolders::Get().CreateFolder(InWorld, NewFolder))
				{
					OutDuplicatedFolders.Add(AcquireTypedElementHandle(LookupMappedRow(&Storage, NewFolder), true));
					
					// If a folder and a containing actor are both duplicated/pasted, we want the duplicated actor to end up in the duplicated folder
					// and not the original folder. To handle this case we add an event column on the original folder for 1 frame that can be used
					// to fixup the actor after the operation
					Storage.AddColumn(Folder.FolderRow, FFolderDuplicateEventColumn{.DuplicatedFolder = NewFolder});
				}
			}
		}
	}
}

/*
 * UActorFolderTypedElementBridgeInterface
 */

FTedsRowHandle UActorFolderTypedElementBridgeInterface::GetRowHandle(const FTypedElementHandle& InElementHandle) const
{
	return FTedsRowHandle(UE::Editor::ActorFolders::GetFolderRow(InElementHandle));
}

/*
 * UActorFolderElementWorldInterface
 */

ULevel* UActorFolderElementWorldInterface::GetOwnerLevel(const FTypedElementHandle& InElementHandle)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		RowHandle FolderRow = GetFolderRow(InElementHandle);
		
		if (FTypedElementLevelColumn* LevelColumn = Storage->GetColumn<FTypedElementLevelColumn>(FolderRow))
		{
			return LevelColumn->Level.Get();
		}
	}
	
	return nullptr;
}

UWorld* UActorFolderElementWorldInterface::GetOwnerWorld(const FTypedElementHandle& InElementHandle)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		RowHandle FolderRow = GetFolderRow(InElementHandle);
		
		if (FTypedElementWorldColumn* WorldColumn = Storage->GetColumn<FTypedElementWorldColumn>(FolderRow))
		{
			return WorldColumn->World.Get();
		}
	}
	
	return nullptr;
}

bool UActorFolderElementWorldInterface::CanScaleElement(const FTypedElementHandle& InElementHandle)
{
	return false; // default is true so we override to set to false
}

bool UActorFolderElementWorldInterface::FindSuitableTransformAtPoint(const FTypedElementHandle& InElementHandle,
	const FTransform& InPotentialTransform, FTransform& OutSuitableTransform)
{
	return false; // default is true so we override to set to false
}

bool UActorFolderElementWorldInterface::CanDeleteElement(const FTypedElementHandle& InElementHandle)
{
	return CVarEnableInterface->GetBool();
}

bool UActorFolderElementWorldInterface::DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld,
	UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	// Mostly mirrors the logic using legacy delegates in SSceneOutliner.cpp

	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders;
	using namespace UE::Editor::ActorFolders::TypedElements::Private;
	
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		// Grab the FFolder from the row handles - we don't handle deletion by directly removing the TEDS row just yet
		TArray<FFolderData> FoldersToDelete;
		GetFolderDataFromHandles(Storage, InElementHandles, FoldersToDelete);
		
		// Sort in descending order so children will be deleted before parents to reduce the amount of updates needed
		FoldersToDelete.Sort([](const FFolderData& FolderA, const FFolderData& FolderB)
		{
			return FolderB.Folder.GetPath().LexicalLess(FolderA.Folder.GetPath());
		});
		
		FHierarchyHandle FolderHierarchy = Storage->FindHierarchyByName("EditorObjectHierarchy");
		
		struct FMatchFolder
		{
			FMatchFolder(const FFolder& InFolder)
				: Folder(InFolder) {}

			const FFolder Folder;

			bool operator()(const FFolderData& Entry)
			{
				return Folder == Entry.Folder;
			}
		};
		
		for (const FFolderData& Folder : FoldersToDelete)
		{
			// Find lowest parent not being deleted, for reparenting children of current folder
			FFolder NewParentPath = Folder.Folder.GetParent();
			while (!NewParentPath.IsNone() && FoldersToDelete.FindByPredicate(FMatchFolder(NewParentPath)))
			{
				NewParentPath = NewParentPath.GetParent();
			}
			
			// For non-actor folders, we also need to iterate all immediate child actors and folders and update them.
			// Actor folders resolve hierarchies on demand so they don't need this update
			TArray<RowHandle> Children;
			
			Storage->IterateChildren(FolderHierarchy, Folder.FolderRow, [&Children](const ICoreProvider& Context, RowHandle Child)
				{
					Children.Add(Child);
					return true;
				});
			
			for (RowHandle ChildRow : Children)
			{
				// Update child actors
				if (Storage->HasColumns<FTypedElementActorTag>(ChildRow))
				{
					FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(ChildRow);
					FTypedElementLevelColumn* LevelColumn = Storage->GetColumn<FTypedElementLevelColumn>(ChildRow);
					if (ObjectColumn && LevelColumn && LevelColumn->Level.IsValid())
					{
						// Ignore actors in actor folders
						if (!LevelColumn->Level->IsUsingActorFolders())
						{
							if (AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
							{
								Actor->SetFolderPath_Recursively(NewParentPath.GetPath());
							}
						}
					}
				}
				// Update child folders
				else if (FFolderCompatibilityColumn* FolderCompatibilityColumn = Storage->GetColumn<FFolderCompatibilityColumn>(ChildRow))
				{
					// Ignore actor folders
					if (!Storage->HasColumns<FTypedElementUObjectColumn>(ChildRow))
					{
						const FFolder NewPath = FActorFolders::Get().GetFolderName(*InWorld, NewParentPath, FolderCompatibilityColumn->Folder.GetLeafName());
						FActorFolders::Get().RenameFolderInWorld(*InWorld, FolderCompatibilityColumn->Folder, NewPath);
					}
				}
			}
			
			// Actually delete the folder now
			FActorFolders::Get().DeleteFolder(*InWorld, Folder.Folder);
		}
		
		// Make sure to deselect to prevent stale items in the selection set
		if (InSelectionSet)
		{
			InSelectionSet->DeselectElements(InElementHandles, FTypedElementSelectionOptions());
		}
		
		return true;
	}
	
	return false;
}

bool UActorFolderElementWorldInterface::CanDuplicateElement(const FTypedElementHandle& InElementHandle)
{
	return CVarEnableInterface->GetBool();
}

void UActorFolderElementWorldInterface::DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld,
	const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders;
	using namespace UE::Editor::ActorFolders::TypedElements::Private;
	
	if (InWorld)
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			// Grab the FFolder from the row handles - we don't handle duplication by just removing the TEDS row yet
			TArray<FFolderData> FoldersToDuplicate;
			GetFolderDataFromHandles(Storage, InElementHandles, FoldersToDuplicate);
		
			DuplicateFolders_Internal(*Storage, FoldersToDuplicate, *InWorld, OutNewElements);
		}
	}
}

bool UActorFolderElementWorldInterface::CanCopyElement(const FTypedElementHandle& InElementHandle)
{
	return CVarEnableInterface->GetBool();
}

void UActorFolderElementWorldInterface::CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders;
	using namespace UE::Editor::ActorFolders::TypedElements::Private;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		// Grab the FFolder from the row handles - we don't handle duplication by just removing the TEDS row yet
		TArray<FFolderData> FoldersToDuplicate;
		GetFolderDataFromHandles(Storage, InElementHandles, FoldersToDuplicate);
		
		// We don't really need the Begin FolderList anymore, but we'll keep it around in case someone has old folder data copied and saved/shared
		// somewhere
		Out.Logf(TEXT("Begin FolderList\n"));
	
		for (const FFolderData& FolderData : FoldersToDuplicate)
		{
			Out.Logf(TEXT("\tFolder=%s"), *FolderData.Folder.GetPath().ToString());
			
			// We also need to save out the root object now so that we can reconstruct the FFolder to add the duplicate event column
			if (UObject* RootObject = FolderData.Folder.GetRootObjectPtr())
			{
				Out.Logf(TEXT(" RootObject=%s"), *RootObject->GetPathName());
			}
			Out.Logf(TEXT("\n"));
			
		}
		
		Out.Logf(TEXT("End FolderList\n"));
	}
}

TSharedPtr<FWorldElementPasteImporter> UActorFolderElementWorldInterface::GetPasteImporter()
{
	return MakeShared<FActorFolderElementPasteImporter>();
}

void FActorFolderElementPasteImporter::Import(FContext& Context)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::ActorFolders::TypedElements::Private;
	
	if (Context.World)
	{
		ImportedFolders.Empty();
	
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FStringView InText = Context.Text;
	
			int32 Index = InText.Find(TEXT("Begin FolderList"));
		
			TArray<FFolderData> FoldersToDuplicate;
	
			if (Index != INDEX_NONE)
			{
				FStringView TmpStr = InText.RightChop(Index);
				const TCHAR* Buffer = TmpStr.GetData();

				FString StrLine;
				while (FParse::Line(&Buffer, StrLine))
				{
					const TCHAR* Str = *StrLine;				
					FString FolderName;

					if (FParse::Command(&Str, TEXT("Begin")) && FParse::Command(&Str, TEXT("FolderList")))
					{
						continue;
					}
					else if (FParse::Command(&Str, TEXT("End")) && FParse::Command(&Str, TEXT("FolderList")))
					{
						break;
					}
					else if (FParse::Value(Str, TEXT("Folder="), FolderName))
					{
						// Need the root object to construct the FFolder and lookup the row in TEDS to add the duplicate event column
						FFolder::FRootObject RootObjectKey = FFolder::GetInvalidRootObject();
						FString RootObjectPath;
						if (FParse::Value(Str, TEXT("RootObject="), RootObjectPath))
						{
							if (UObject* RootObject = FindObject<UObject>(nullptr, RootObjectPath))
							{
								RootObjectKey = RootObject;
							}
						}
				
						FFolder Folder(RootObjectKey, FName(FolderName));
						FoldersToDuplicate.Add(FFolderData{.Folder = Folder, .FolderRow = ActorFolders::LookupMappedRow(Storage, Folder)});
					}
				}
			}
		
			DuplicateFolders_Internal(*Storage, FoldersToDuplicate, *Context.World, ImportedFolders);
		}
	}
	
}

TArray<FTypedElementHandle> FActorFolderElementPasteImporter::GetImportedElements()
{
	return ImportedFolders;
}