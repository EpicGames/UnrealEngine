// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Interfaces/TedsTypedElementBridgeInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Folder.h"
#include "UObject/Object.h"

#include "ActorFolderTypedElementInterfaces.generated.h"

/**
 * Interface that manages selection for folders
 */
UCLASS(MinimalAPI)
class UActorFolderElementSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()
};

/**
 * Interface that bridges between TEDS row handles <-> TEv1 element handles for folders
 */
UCLASS(MinimalAPI)
class UActorFolderTypedElementBridgeInterface : public UObject, public ITedsTypedElementBridgeInterface
{
	GENERATED_BODY()
public:
	virtual FTedsRowHandle GetRowHandle(const FTypedElementHandle& InElementHandle) const override;
};

/**
 * Handles a variety of operations on folders in the world, including copying, deleting, etc.
 */
UCLASS(MinimalAPI)
class UActorFolderElementWorldInterface : public UObject, public ITypedElementWorldInterface
{
	GENERATED_BODY()

public:
	virtual ULevel* GetOwnerLevel(const FTypedElementHandle& InElementHandle) override;
	virtual UWorld* GetOwnerWorld(const FTypedElementHandle& InElementHandle) override;
	virtual bool CanScaleElement(const FTypedElementHandle& InElementHandle) override;
	virtual bool FindSuitableTransformAtPoint(const FTypedElementHandle& InElementHandle, const FTransform& InPotentialTransform, FTransform& OutSuitableTransform) override;
	virtual bool CanDeleteElement(const FTypedElementHandle& InElementHandle) override;
	virtual bool DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions) override;
	virtual bool CanDuplicateElement(const FTypedElementHandle& InElementHandle) override;
	virtual void DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements) override;
	virtual bool CanCopyElement(const FTypedElementHandle& InElementHandle) override;
	virtual void CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out) override;
	virtual TSharedPtr<FWorldElementPasteImporter> GetPasteImporter() override;
};

/**
 * Paste importer for folders
 */
struct FActorFolderElementPasteImporter : public FWorldElementPasteImporter
{
public:
	UNREALED_API virtual void Import(FContext& Context) override;
	
	UNREALED_API virtual TArray<FTypedElementHandle> GetImportedElements() override;
private:
	TArray<FTypedElementHandle> ImportedFolders;
};