// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFolders/TypedElements/ActorFolderTypedElementSupport.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementOwnerStore.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::ActorFolders
{
	UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FActorFolderElementData);

	// The global store that we'll use
	TTypedElementOwnerStore<FActorFolderElementData, DataStorage::RowHandle> GActorFolderElementOwnerStore;
	
	TTypedElementOwner<FActorFolderElementData> CreateActorFolderElement(DataStorage::RowHandle FolderRow)
	{
		// First we create the element like we do in CreateTypedElement in EngineElementsLibrary.cpp
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

		TTypedElementOwner<FActorFolderElementData> TypedElement;
		if (ensureMsgf(Registry, TEXT("Typed element was requested for before the registry was available.")))
		{
			TypedElement = Registry->CreateElement<FActorFolderElementData>(NAME_ActorFolder);
		}

		// Update the data inside the element (corresponds to UEngineElementsLibrary::CreateActorElement)
		if (TypedElement)
		{
			TypedElement.GetDataChecked().FolderRow = FolderRow;
		}

		return TypedElement;
	}
	
	FTypedElementHandle AcquireTypedElementHandle(DataStorage::RowHandle FolderRow, const bool bAllowCreate)
	{
		using namespace DataStorage;
		
		if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (Storage->IsRowAvailable(FolderRow))
			{
				TTypedElementOwnerScopedAccess<FActorFolderElementData> EditorElement = bAllowCreate
					? GActorFolderElementOwnerStore.FindOrRegisterElementOwner(FolderRow, 
						[FolderRow]() { return CreateActorFolderElement(FolderRow); })
					: GActorFolderElementOwnerStore.FindElementOwner(FolderRow);
		
				if (EditorElement)
				{
					return EditorElement->AcquireHandle();
				}
			}
		}
		
		return FTypedElementHandle();
	}
	
	void DestroyTypedElementHandle(DataStorage::RowHandle FolderRow)
	{
		TTypedElementOwner<FActorFolderElementData> EditorElement = GActorFolderElementOwnerStore.UnregisterElementOwner(FolderRow);
		if (!EditorElement)
		{
			return;
		}

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		if (!Registry)
		{
			return;
		}

		Registry->DestroyElement(EditorElement);
	}
	
	DataStorage::RowHandle GetFolderRow(const FTypedElementHandle& InElementHandle)
    {
		if (const FActorFolderElementData* Element = InElementHandle.GetData<FActorFolderElementData>())
		{
			return Element->FolderRow;
		}
		
		return DataStorage::InvalidRowHandle;
	}

	FFolder GetFolder(const FTypedElementHandle& InElementHandle)
	{
		using namespace DataStorage;
		
		if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			RowHandle FolderRow = GetFolderRow(InElementHandle);
			
			if (const FFolderCompatibilityColumn* FolderCompatibilityColumn = Storage->GetColumn<FFolderCompatibilityColumn>(FolderRow))
			{
				return FolderCompatibilityColumn->Folder;
			}
		}
		
		return FFolder::GetInvalidFolder();
	}
}
