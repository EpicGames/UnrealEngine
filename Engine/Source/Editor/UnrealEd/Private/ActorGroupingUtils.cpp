// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorGroupingUtils.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Editor/GroupActor.h"
#include "ScopedTransaction.h"
#include "Misc/MessageDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorGroupingUtils)

bool UActorGroupingUtils::bGroupingActive = true;
TMap<FName, UActorGroupingUtils::FCanGroupActors> UActorGroupingUtils::CanGroupActorsDelegates;

void UActorGroupingUtils::SetGroupingActive(bool bInGroupingActive)
{
	bGroupingActive = bInGroupingActive;
}

UActorGroupingUtils* UActorGroupingUtils::Get()
{
	// @todo ActorGrouping This should be moved off of GEditor
	return GEditor->GetActorGroupingUtils();
}

AGroupActor* UActorGroupingUtils::GroupSelected()
{
	if (IsGroupingActive() && CanGroupSelectedActors())
	{
		TArray<AActor*> ActorsToAdd;
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			ActorsToAdd.Add(CastChecked<AActor>(*It));
		}

		if (ActorsToAdd.Num() > 0)
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Group_Selected", "Group Selected"));

			AGroupActor* GroupActor = GroupActors(ActorsToAdd);
			if (GroupActor)
			{
				GEditor->SelectActor(GroupActor, /*bIsSelected*/true, /*bNotify*/false);
			}
			return GroupActor;
		}
	}

	return nullptr;
}

AGroupActor* UActorGroupingUtils::GroupActors(const TArray<AActor*>& ActorsToGroup)
{
	if(IsGroupingActive() && CanGroupActors(ActorsToGroup))
	{
		ULevel* ActorLevel = nullptr;
		TArray<AActor*> FinalActorList;

		bool bActorsInSameLevel = true;
		for (AActor* Actor : ActorsToGroup)
		{
			if (!IsValid(Actor))
			{
				continue;
			}
			
			if (!ActorLevel)
			{
				ActorLevel = Actor->GetLevel();
			}
			else if (ActorLevel != Actor->GetLevel())
			{
				bActorsInSameLevel = false;
				break;
			}

			if (Actor->IsA(AActor::StaticClass()) && !Actor->IsA(AGroupActor::StaticClass()))
			{
				// Add each selected actor to our new group
				// Adding an actor will remove it from any existing groups.
				FinalActorList.Add(Actor);
			}
		}

		if (bActorsInSameLevel)
		{
			if (FinalActorList.Num() > 1)
			{
				check(ActorLevel);
				// Store off the current level and make the level that contain the actors to group as the current level
				UWorld* World = ActorLevel->OwningWorld;
				check(World);
				{
					const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Group_Regroup", "Regroup Ctrl+G"));

					FActorSpawnParameters SpawnInfo;
					SpawnInfo.OverrideLevel = ActorLevel;
					AGroupActor* SpawnedGroupActor = World->SpawnActor<AGroupActor>(SpawnInfo);

					bool bActorsInSameFolder = true;
					FName FolderPath;

					for (AActor* FinalActor : FinalActorList)
					{
						SpawnedGroupActor->Add(*FinalActor);

						if (bActorsInSameFolder)
						{
							if (FolderPath.IsNone())
							{
								FolderPath = FinalActor->GetFolderPath();
							}
							else if (FolderPath != FinalActor->GetFolderPath())
							{
								bActorsInSameFolder = false;
								FolderPath = FName();
							}
						}
					}

					SpawnedGroupActor->SetFolderPath(FolderPath);
					SpawnedGroupActor->CenterGroupLocation();
					SpawnedGroupActor->SetActorRotation(FinalActorList.Last()->GetActorRotation());
					SpawnedGroupActor->Lock();

					return SpawnedGroupActor;
				}
			}
		}
		else
		{
			const FText NotificationErrorText = NSLOCTEXT("UnrealEd", "Group_CantCreateGroupMultipleLevels", "Can't group the selected actors because they are in different levels.");
			FNotificationInfo Info(NotificationErrorText);
			Info.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	return nullptr;
}

void UActorGroupingUtils::UngroupSelected()
{
	if (IsGroupingActive())
	{
		TArray<AActor*> ActorsToUngroup;

		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = CastChecked<AActor>(*It);
			ActorsToUngroup.Add(Actor);
		}

		if (ActorsToUngroup.Num())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Ungroup_Selected", "Ungroup Selected"));

			UngroupActors(ActorsToUngroup);
		}
	}
}

void UActorGroupingUtils::UngroupActors(const TArray<AActor*>& ActorsToUngroup)
{
	if (IsGroupingActive())
	{
		TArray<AGroupActor*> OutermostGroupActors;

		for (AActor* Actor : ActorsToUngroup)
		{
			// Get the outermost locked group
			AGroupActor* OutermostGroup = AGroupActor::GetRootForActor(Actor, true);
			if (OutermostGroup == NULL)
			{
				// Failed to find locked root group, try to find the immediate parent
				OutermostGroup = AGroupActor::GetParentForActor(Actor);
			}

			if (OutermostGroup)
			{
				OutermostGroupActors.AddUnique(OutermostGroup);
			}
		}

		if (OutermostGroupActors.Num())
		{
			const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Group_Disband", "Disband Group"));
			for (int32 GroupIndex = 0; GroupIndex < OutermostGroupActors.Num(); ++GroupIndex)
			{
				AGroupActor* GroupActor = OutermostGroupActors[GroupIndex];
				GroupActor->ClearAndRemove();
			}
		}
	}
}

void UActorGroupingUtils::LockSelectedGroups()
{
	if (IsGroupingActive())
	{
		AGroupActor::LockSelectedGroups();
	}
}

void UActorGroupingUtils::UnlockSelectedGroups()
{
	if (IsGroupingActive())
	{
		AGroupActor::UnlockSelectedGroups();
	}
}

void UActorGroupingUtils::AddSelectedToGroup()
{
	if (IsGroupingActive() && CanGroupSelectedActors())
	{
		AGroupActor::AddSelectedActorsToSelectedGroup();
	}
}


void UActorGroupingUtils::RemoveSelectedFromGroup()
{
	if (IsGroupingActive())
	{
		TArray<AActor*> ActorsToRemove;
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = static_cast<AActor*>(*It);
			checkSlow(Actor->IsA(AActor::StaticClass()));

			// See if an entire group is being removed
			AGroupActor* GroupActor = Cast<AGroupActor>(Actor);
			if (GroupActor == NULL)
			{
				// See if the actor selected belongs to a locked group, if so remove the group in lieu of the actor
				GroupActor = AGroupActor::GetParentForActor(Actor);
				if (GroupActor && !GroupActor->IsLocked())
				{
					GroupActor = NULL;
				}
			}

			if (GroupActor)
			{
				// If the GroupActor has no parent, do nothing, otherwise just add the group for removal
				if (AGroupActor::GetParentForActor(GroupActor))
				{
					ActorsToRemove.AddUnique(GroupActor);
				}
			}
			else
			{
				ActorsToRemove.AddUnique(Actor);
			}
		}

		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "Group_Remove", "Remove from Group"));
		for (int32 ActorIndex = 0; ActorIndex < ActorsToRemove.Num(); ++ActorIndex)
		{
			AActor* Actor = ActorsToRemove[ActorIndex];
			AGroupActor* ActorGroup = AGroupActor::GetParentForActor(Actor);

			if (ActorGroup)
			{
				AGroupActor* ActorGroupParent = AGroupActor::GetParentForActor(ActorGroup);
				if (ActorGroupParent)
				{
					ActorGroupParent->Add(*Actor);
					ActorGroupParent->CenterGroupLocation();
				}
				else
				{
					ActorGroup->Remove(*Actor);
					ActorGroup->CenterGroupLocation();
				}
			}
		}
		// Do a re-selection of each actor, to maintain group selection rules
		GEditor->SelectNone(true, true);
		for (int32 ActorIndex = 0; ActorIndex < ActorsToRemove.Num(); ++ActorIndex)
		{
			GEditor->SelectActor(ActorsToRemove[ActorIndex], true, false);
		}
	}

}

void UActorGroupingUtils::AddCanGroupActorsDelegate(const FName& Owner, const FCanGroupActors& InGroupActorsFilter)
{
	CanGroupActorsDelegates.Emplace(Owner, InGroupActorsFilter);
}

void UActorGroupingUtils::RemoveCanGroupActorsDelegate(const FName& Owner)
{
	CanGroupActorsDelegates.Remove(Owner);
}

bool UActorGroupingUtils::CanGroupActors(const TArray<AActor*>& ActorsToGroup) const
{
	for(const TPair<FName, FCanGroupActors>& Filter : CanGroupActorsDelegates)
	{
		if(!Filter.Value.Execute(ActorsToGroup))
		{
			return false;
		}
	}
	
	return true;
}

bool UActorGroupingUtils::CanGroupSelectedActors() const
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	ActorSelection->GetSelectedObjects(Actors);
	
	for(const TPair<FName, FCanGroupActors>& Filter : CanGroupActorsDelegates)
	{
		if(!Filter.Value.Execute(Actors))
		{
			return false;
		}
	}

	return true;
}
