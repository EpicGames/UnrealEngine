// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics/AnalyticsPrivacySettings.h"
#include "UObject/UnrealType.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnalyticsPrivacySettings)

#define LOCTEXT_NAMESPACE "AnalyticsPrivacySettings"

UAnalyticsPrivacySettings::UAnalyticsPrivacySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bSendUsageData(true)
{
}

void UAnalyticsPrivacySettings::GetToggleCategoryAndPropertyNames(FName& OutCategory, FName& OutProperty) const
{
	OutCategory = FName("Options");
	OutProperty = FName("bSendUsageData");
};

FText UAnalyticsPrivacySettings::GetFalseStateLabel() const
{
	return LOCTEXT("FalseStateLabel", "Don't Send");
};

FText UAnalyticsPrivacySettings::GetFalseStateTooltip() const
{
	return LOCTEXT("FalseStateTooltip", "Don't send Editor usage data to Epic Games.");
};

FText UAnalyticsPrivacySettings::GetFalseStateDescription() const
{
	return LOCTEXT("FalseStateDescription", "By opting out you have chosen to not send Editor usage data to Epic Games. Please consider opting in to help improve Unreal Engine. Epic Games will never sell or trade individual usage data to / with third party organizations. If you enable this feature, we will collect information about how you use the editor, when you use the editor, the type of projects you are creating, how you interact with the various editor components and we would perform occasional checks on the type of hardware/OS you are using.");
};

FText UAnalyticsPrivacySettings::GetTrueStateLabel() const
{
	return LOCTEXT("TrueStateLabel", "Send Usage Data");
};

FText UAnalyticsPrivacySettings::GetTrueStateTooltip() const
{
	return LOCTEXT("TrueStateTooltip", "Send your Editor usage data to Epic Games.");
};

FText UAnalyticsPrivacySettings::GetTrueStateDescription() const
{
	return LOCTEXT("TrueStateDescription", "By opting in you have chosen to send Editor usage data to Epic Games. Thank you for helping to improve Unreal Engine. Epic Games will never sell or trade individual usage data to / with third party organizations. We will collect information about how you use the editor, when you use the editor, the type of projects you are creating, how you interact with the various editor components and we perform occasional checks on the type of hardware/OS you are using.");
};

FString UAnalyticsPrivacySettings::GetAdditionalInfoUrl() const
{
	return FString(TEXT("http://epicgames.com/privacynotice"));
};

FText UAnalyticsPrivacySettings::GetAdditionalInfoUrlLabel() const
{
	return LOCTEXT("HyperlinkLabel", "Epic Games Privacy Policy");
};

#if WITH_EDITOR
void UAnalyticsPrivacySettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnalyticsPrivacySettings, bSendUsageData))
	{
		OnSendFullUsageDataChanged();

		FCrashOverrideParameters Params;
		Params.bSetCrashReportClientMessageText = false;
		Params.bSetGameNameSuffix = false;
		Params.SendUsageData = bSendUsageData;

		FCoreDelegates::CrashOverrideParamsChanged.Broadcast(Params);
	}
}

void UAnalyticsPrivacySettings::OnSendFullUsageDataChanged()
{
	if (bSendUsageData)
	{
		// Attempt to initialize analytics and send opt-in event
		if (!FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::Initialize();
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Privacy.EndUserOptIn"));
			}
		}
	}
	else
	{
		// Send opt-out event and shutdown analytics
		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Privacy.EndUserOptOut"));
			const bool bIsEngineShutdown = false;
			FEngineAnalytics::Shutdown(bIsEngineShutdown);
		}
	}
}

#endif

#undef LOCTEXT_NAMESPACE
