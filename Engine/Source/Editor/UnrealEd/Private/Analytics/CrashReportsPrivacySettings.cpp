// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analytics/CrashReportsPrivacySettings.h"
#include "UObject/UnrealType.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CrashReportsPrivacySettings)

#define LOCTEXT_NAMESPACE "CrashReportsPrivacySettings"

UCrashReportsPrivacySettings::UCrashReportsPrivacySettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bSendUnattendedBugReports(false)
{
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/UnrealEd.CrashReportsPrivacySettings"), TEXT("bSendUnattendedBugReports"), bSendUnattendedBugReports, GEditorSettingsIni);
	}
}

void UCrashReportsPrivacySettings::GetToggleCategoryAndPropertyNames(FName& OutCategory, FName& OutProperty) const
{
	OutCategory = FName("Options");
	OutProperty = FName("bSendUnattendedBugReports");
};

FText UCrashReportsPrivacySettings::GetFalseStateLabel() const
{
	return LOCTEXT("FalseStateLabel", "Don't Send");
};

FText UCrashReportsPrivacySettings::GetFalseStateTooltip() const
{
	return LOCTEXT("FalseStateTooltip", "Don't send unattended bug reports to Epic Games.");
};

FText UCrashReportsPrivacySettings::GetFalseStateDescription() const
{
	return LOCTEXT("FalseStateDescription", "By opting out you have chosen to not send Editor non-critical bug reports by default to Epic Games. Please consider opting in to help improve Unreal Engine. Please note that while this setting is disabled, we may ask you to share information about critical bugs, but sharing such information is optional. You can find out more at our Privacy Policy.");
};

FText UCrashReportsPrivacySettings::GetTrueStateLabel() const
{
	return LOCTEXT("TrueStateLabel", "Send Unattended Bug Reports");
};

FText UCrashReportsPrivacySettings::GetTrueStateTooltip() const
{
	return LOCTEXT("TrueStateTooltip", "Automatically send bug reports that don't require user attention to Epic Games .");
};

FText UCrashReportsPrivacySettings::GetTrueStateDescription() const
{
	return LOCTEXT("TrueStateDescription", "By opting in you have chosen to send Editor non-critical bug reports by default to Epic Games to help us improve Unreal Engine. Please note that if you turn off this setting, we may ask you to share information about critical bugs, but sharing such information is optional. You can find out more at our Privacy Policy.");
};

FString UCrashReportsPrivacySettings::GetAdditionalInfoUrl() const
{
	return FString(TEXT("http://epicgames.com/privacynotice"));
};

FText UCrashReportsPrivacySettings::GetAdditionalInfoUrlLabel() const
{
	return LOCTEXT("HyperlinkLabel", "Epic Games Privacy Policy");
};

#if WITH_EDITOR
void UCrashReportsPrivacySettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCrashReportsPrivacySettings, bSendUnattendedBugReports))
	{
		FCrashOverrideParameters Params;
		Params.bSetCrashReportClientMessageText = false;
		Params.bSetGameNameSuffix = false;
		Params.SendUnattendedBugReports = bSendUnattendedBugReports;

		FCoreDelegates::CrashOverrideParamsChanged.Broadcast(Params);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
