// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAnalytics.h"
#include "GeneralProjectSettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"

#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"

#define LOCTEXT_NAMESPACE "EditorAnalytics"


void FEditorAnalytics::ReportEvent(FString EventName, FString PlatformName, bool bHasCode)
{
	TArray<FAnalyticsEventAttribute> Empty;
	ReportEvent(EventName, PlatformName, bHasCode, Empty);
}

void FEditorAnalytics::ReportEvent(FString EventName, FString PlatformName, bool bHasCode, const TArray<FAnalyticsEventAttribute>& ExtraParams)
{
	if( FEngineAnalytics::IsAvailable() )
	{
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ProjectID"), ProjectSettings.ProjectID.ToString()));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Platform"), PlatformName));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ProjectType"), bHasCode ? TEXT("C++ Code") : TEXT("Content Only")));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("VanillaEditor"), (GEngine && GEngine->IsVanillaProduct()) ? TEXT("Yes") : TEXT("No")));
		ParamArray.Append(ExtraParams);

		FEngineAnalytics::GetProvider().RecordEvent( EventName, ParamArray );
	}
}

void FEditorAnalytics::ReportEvent(FString EventName, FString PlatformName, bool bHasCode, int32 ErrorCode, TArray<FAnalyticsEventAttribute>& ExtraParams)
{
	if( FEngineAnalytics::IsAvailable() )
	{
		const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
		TArray<FAnalyticsEventAttribute> ParamArray;
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ProjectID"), ProjectSettings.ProjectID.ToString()));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("Platform"), PlatformName));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ProjectType"), bHasCode ? TEXT("C++ Code") : TEXT("Content Only")));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("VanillaEditor"), (GEngine && GEngine->IsVanillaProduct()) ? TEXT("Yes") : TEXT("No")));
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ErrorCode"), ErrorCode));
		const FString ErrorMessage = TranslateErrorCode(ErrorCode);
		ParamArray.Add(FAnalyticsEventAttribute(TEXT("ErrorName"), ErrorMessage));
		ParamArray.Append(ExtraParams);

		FEngineAnalytics::GetProvider().RecordEvent( EventName, ParamArray );
	}
}

FString FEditorAnalytics::TranslateErrorCode(int32 ErrorCode)
{
	switch (ErrorCode)
	{
	case EAnalyticsErrorCodes::UATNotFound:
		return TEXT("UAT Not Found");
	case EAnalyticsErrorCodes::Unknown:
		return TEXT("Unknown Error");
	case EAnalyticsErrorCodes::Arguments:
		return TEXT("Invalid Arguments");
	case EAnalyticsErrorCodes::UnknownCommand:
		return TEXT("Unknown Command");
	case EAnalyticsErrorCodes::SDKNotFound:
		return TEXT("SDK Not Found");
	case EAnalyticsErrorCodes::ProvisionNotFound:
		return TEXT("Provision Not Found");
	case EAnalyticsErrorCodes::CertificateNotFound:
		return TEXT("Certificate Not Found");
	case EAnalyticsErrorCodes::ManifestNotFound:
		return TEXT("Manifest Not Found");
	case EAnalyticsErrorCodes::KeyNotFound:
		return TEXT("Key Not Found in Manifest");
	case EAnalyticsErrorCodes::ProvisionExpired:
		return TEXT("Provision Has Expired");
	case EAnalyticsErrorCodes::CertificateExpired:
		return TEXT("Certificate Has Expired");
	case EAnalyticsErrorCodes::CertificateProvisionMismatch:
		return TEXT("Certificate Doesn't Match Provision");
	case EAnalyticsErrorCodes::LauncherFailed:
		return TEXT("LauncherWorker Failed to Launch");
	case EAnalyticsErrorCodes::UATLaunchFailure:
		return TEXT("UAT Failed to Launch");
	case EAnalyticsErrorCodes::UnknownCookFailure:
		return TEXT("Unknown Cook Failure");
	case EAnalyticsErrorCodes::UnknownDeployFailure:
		return TEXT("Unknown Deploy Failure");
	case EAnalyticsErrorCodes::UnknownBuildFailure:
		return TEXT("Unknown Build Failure");
	case EAnalyticsErrorCodes::UnknownPackageFailure:
		return TEXT("Unknown Package Failure");
	case EAnalyticsErrorCodes::UnknownLaunchFailure:
		return TEXT("Unknown Launch Failure");
	case EAnalyticsErrorCodes::StageMissingFile:
		return TEXT("Could not find file for staging");
	case EAnalyticsErrorCodes::FailedToCreateIPA:
		return TEXT("Failed to Create IPA");
	case EAnalyticsErrorCodes::FailedToCodeSign:
		return TEXT("Failed to Code Sign");
	case EAnalyticsErrorCodes::DeviceBackupFailed:
		return TEXT("Failed to backup device");
	case EAnalyticsErrorCodes::AppUninstallFailed:
		return TEXT("Failed to Uninstall app");
	case EAnalyticsErrorCodes::AppInstallFailed:
		return TEXT("Failed to Install app");
	case EAnalyticsErrorCodes::AppNotFound:
		return TEXT("App package file not found for Install");
	case EAnalyticsErrorCodes::StubNotSignedCorrectly:
		return TEXT("Stub not signed correctly.");
	case EAnalyticsErrorCodes::IPAMissingInfoPList:
		return TEXT("Failed to find Info.plist in IPA");
	case EAnalyticsErrorCodes::DeleteFile:
		return TEXT("Could not delete file");
	case EAnalyticsErrorCodes::DeleteDirectory:
		return TEXT("Could not delete directory");
	case EAnalyticsErrorCodes::CreateDirectory:
		return TEXT("Could not create directory");
	case EAnalyticsErrorCodes::CopyFile:
		return TEXT("Could not copy file");
	case EAnalyticsErrorCodes::OnlyOneObbFileSupported:
		return TEXT("Android packaging supports only exactly one obb/pak file");
	case EAnalyticsErrorCodes::FailureGettingPackageInfo:
		return TEXT("Failed to get package info from APK file");
	case EAnalyticsErrorCodes::OnlyOneTargetConfigurationSupported:
		return TEXT("Android is only able to package a single target configuration");
	case EAnalyticsErrorCodes::ObbNotFound:
		return TEXT("OBB/PAK file not found");
	case EAnalyticsErrorCodes::AndroidBuildToolsPathNotFound:
		return TEXT("Android build-tools directory not found");
	case EAnalyticsErrorCodes::NoApkSuitableForArchitecture:
		return TEXT("No APK suitable for architecture found");
	case EAnalyticsErrorCodes::FailedToDeleteStagingDirectory :
		return TEXT("Failed to delete staging directory.  This could be because something is currently using the staging directory (ps4/xbox/etc)");
	case EAnalyticsErrorCodes::MissingExecutable:
		return LOCTEXT("UATErrorMissingExecutable", "Missing UnrealGame binary.\nYou may have to build the UE project with your IDE. Alternatively, build using UnrealBuildTool with the commandline:\nUnrealGame <Platform> <Configuration>").ToString();
	case EAnalyticsErrorCodes::FilesInstallFailed:
		return TEXT("Failed to deploy files to device.  Check to make sure your device is connected.");
	case EAnalyticsErrorCodes::DeviceNotSetupForDevelopment:
		return TEXT("Failed to launch on device.  Make sure your device is currently unlocked and has been enabled for development by using a mobile provision including your device id.");
	case EAnalyticsErrorCodes::DeviceOSNewerThanSDK:
		return TEXT("Failed to launch on device.  Make sure your install of Xcode matches or is newer than the OS on your device.");
	case EAnalyticsErrorCodes::RemoteCertificatesNotFound:
		return TEXT("Failed to sign executable.  Make sure your developer certificates have been installed in the System Keychain on the remote Mac.");
	case EAnalyticsErrorCodes::SymbolizedSONotFound:
		return TEXT("Symbolized .so file not found");
	case EAnalyticsErrorCodes::AndroidOBBError:
		return TEXT("Failed to create valid OBB.  OBB may have exceeded 2 GiB limit; check log for details.");
	}
	return TEXT("Unknown Error");
}

bool FEditorAnalytics::ShouldElevateMessageThroughDialog(const int32 ErrorCode)
{
	return (EAnalyticsErrorCodes::Type)ErrorCode == EAnalyticsErrorCodes::MissingExecutable;
}

void FEditorAnalytics::ReportBuildRequirementsFailure(FString EventName, FString PlatformName, bool bHasCode, int32 Requirements)
{
	TArray<FAnalyticsEventAttribute> ParamArray;
	ParamArray.Add(FAnalyticsEventAttribute(TEXT("Time"), 0.0));
	if (Requirements & ETargetPlatformReadyStatus::SDKNotFound)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::SDKNotFound, ParamArray);
	}
	if (Requirements & ETargetPlatformReadyStatus::LicenseNotAccepted)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::LicenseNotAccepted, ParamArray);
	}
	if (Requirements & ETargetPlatformReadyStatus::ProvisionNotFound)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::ProvisionNotFound, ParamArray);
	}
	if (Requirements & ETargetPlatformReadyStatus::SigningKeyNotFound)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::CertificateNotFound, ParamArray);
	}
	if (Requirements & ETargetPlatformReadyStatus::CodeUnsupported)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::CodeUnsupported, ParamArray);
	}
	if (Requirements & ETargetPlatformReadyStatus::PluginsUnsupported)
	{
		ReportEvent(EventName, PlatformName, bHasCode, EAnalyticsErrorCodes::PluginsUnsupported, ParamArray);
	}
}

#undef LOCTEXT_NAMESPACE
