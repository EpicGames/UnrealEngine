// Copyright Epic Games, Inc. All Rights Reserved.
#include "Exporters/AnimSeqExportOption.h"

#include "UObject/Object.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSeqExportOption)

TAutoConsoleVariable<bool> CVarSetRetargetSourceAsset(TEXT("AnimationExport.SetRetargetSourceAsset"), false, TEXT("When true we set the Retarget Source Asset on the anim sequence when recording, if false we do not, by default false."));


UAnimSeqExportOption::UAnimSeqExportOption(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ResetToDefault();
}

void UAnimSeqExportOption::ResetToDefault()
{
	bExportTransforms = true;
	bExportMorphTargets = true;
	bExportAttributeCurves = true;
	bExportMaterialCurves = true;
	bSkipCurvesWithZeroValue = true;
	bRemoveExcludedCurves = false;
	bRecordInWorldSpace = false;
	bSetRetargetSourceAsset = CVarSetRetargetSourceAsset.GetValueOnGameThread();
	Interpolation = EAnimInterpolationType::Linear;
	CurveInterpolation = ERichCurveInterpMode::RCIM_Linear;
	bEvaluateAllSkeletalMeshComponents = true;
	WarmUpFrames = 0;
	DelayBeforeStart = 0;
	bTransactRecording = true;
	bBakeTimecode = false;
	bTimecodeRateOverride = false;
	OverrideTimecodeRate = FFrameRate(30, 1);
	bUseCustomTimeRange = false;
	CustomStartFrame = 0;
	CustomEndFrame = 120;
	CustomDisplayRate = FFrameRate(30, 1);
	bUseCustomFrameRate = false;
	CustomFrameRate = FFrameRate(30, 1);
}