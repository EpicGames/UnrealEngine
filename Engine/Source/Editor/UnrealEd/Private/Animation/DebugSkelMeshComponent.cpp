// Copyright Epic Games, Inc. All Rights Reserved.


#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/MirrorDataTable.h"
#include "BonePose.h"
#include "Materials/Material.h"
#include "Animation/AnimMontage.h"
#include "Engine/Engine.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "EngineGlobals.h"
#include "GameFramework/WorldSettings.h"
#include "SceneView.h"
#include "SkeletalRenderPublic.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimComposite.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"

#include "ClothingAsset.h"
#include "ClothingSimulationInstance.h"  // UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
#include "Utils/ClothingMeshUtils.h"
#include "DynamicMeshBuilder.h"
#include "SkeletalDebugRendering.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Preferences/PersonaOptions.h"

//////////////////////////////////////////////////////////////////////////
// UDebugSkelMeshComponent

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugSkelMeshComponent)

UDebugSkelMeshComponent::UDebugSkelMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bDrawMesh = true;
	PreviewInstance = nullptr;
	bDisplayRawAnimation = false;
	bDisplayNonRetargetedPose = false;

	bMeshSocketsVisible = true;
	bSkeletonSocketsVisible = true;

	bShowNotificationVisualizations = true;
	VisualizeRootMotionMode = EVisualizeRootMotionMode::Trajectory;
	bShowAssetUserDataVisualizations = true;

	TurnTableSpeedScaling = 1.f;
	TurnTableMode = EPersonaTurnTableMode::Stopped;

	RootMotionReferenceTransform = FTransform::Identity;

	bPauseClothingSimulationWithAnim = false;
	bPerformSingleClothingTick = false;

	bTrackAttachedInstanceLOD = false;

	WireframeMeshOverlayColor = FLinearColor(0.4f, 0.8f, 0.66f);
	
	CachedClothBounds = FBoxSphereBounds(ForceInit);

	RequestedProcessRootMotionMode = EProcessRootMotionMode::LoopAndReset;
	ProcessRootMotionMode = EProcessRootMotionMode::Ignore;
	ConsumeRootMotionPreviousPlaybackTime = 0.f;
}

FLinearColor UDebugSkelMeshComponent::GetBoneColor(int32 InBoneIndex) const
{
	// this returns the normal unmodified color of the bone, calling code must account
	// for any editor specific states that might affect the final bone color (like selection)
	
	// skeleton greyed out
	if (SkeletonDrawMode == ESkeletonDrawMode::GreyedOut)
	{
		return GetDefault<UPersonaOptions>()->DisabledBoneColor;
	}

	// using default color for all bones
	if (!GetDefault<UPersonaOptions>()->bShowBoneColors)
	{
		return GetDefault<UPersonaOptions>()->DefaultBoneColor;
	}
	
	// uses deterministic, semi-random desaturated color unique to the bone index
	return SkeletalDebugRendering::GetSemiRandomColorForBone(InBoneIndex);
}

void UDebugSkelMeshComponent::SetDebugForcedLOD(int32 InNewForcedLOD)
{
	SetForcedLOD(InNewForcedLOD);

#if WITH_EDITOR
	if (OnDebugForceLODChangedDelegate.IsBound())
	{
		OnDebugForceLODChangedDelegate.Execute();
	}
#endif
}

FBoxSphereBounds UDebugSkelMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// Override bounds with pre-skinned bounds if asking for them
	if (IsUsingPreSkinnedBounds())
	{
		FBoxSphereBounds PreSkinnedLocalBounds;
		GetPreSkinnedLocalBounds(PreSkinnedLocalBounds);
		return PreSkinnedLocalBounds;
	}

	FBoxSphereBounds Result = Super::CalcBounds(LocalToWorld);

	if (!IsUsingInGameBounds())
	{
		// extend bounds by required bones (respecting current LOD) but without root bone
		if (GetNumComponentSpaceTransforms())
		{
			FBox BoundingBox(ForceInit);
			const int32 NumRequiredBones = RequiredBones.Num();
			for (int32 BoneIndex = 1; BoneIndex < NumRequiredBones; ++BoneIndex)
			{
				FBoneIndexType RequiredBoneIndex = RequiredBones[BoneIndex];
				BoundingBox += GetBoneMatrix((int32)RequiredBoneIndex).GetOrigin();
			}

			if (BoundingBox.IsValid)
			{
				Result = Result + FBoxSphereBounds(BoundingBox);
			}			
		}

		if (GetSkeletalMeshAsset() && !GetSkeletalMeshAsset()->IsCompiling())
		{
			Result = Result + GetSkeletalMeshAsset()->GetBounds();
		}
	}

	if (!FMath::IsNearlyZero(CachedClothBounds.SphereRadius))
	{
		Result = Result + CachedClothBounds;
	}	

	return Result;
}

FBoxSphereBounds UDebugSkelMeshComponent::CalcGameBounds(const FTransform& LocalToWorld) const
{
	return Super::CalcBounds(LocalToWorld);
}

bool UDebugSkelMeshComponent::IsUsingInGameBounds() const
{
	return bIsUsingInGameBounds;
}

void UDebugSkelMeshComponent::UseInGameBounds(bool bUseInGameBounds)
{
	bIsUsingInGameBounds = bUseInGameBounds;
}

bool UDebugSkelMeshComponent::IsUsingPreSkinnedBounds() const
{
	return bIsUsingPreSkinnedBounds;
}

void UDebugSkelMeshComponent::UsePreSkinnedBounds(bool bUsePreSkinnedBounds)
{
	bIsUsingPreSkinnedBounds = bUsePreSkinnedBounds;
}

bool UDebugSkelMeshComponent::CheckIfBoundsAreCorrrect()
{
	if (GetPhysicsAsset())
	{
		bool bWasUsingInGameBounds = IsUsingInGameBounds();
		FTransform TempTransform = FTransform::Identity;
		UseInGameBounds(true);
		FBoxSphereBounds InGameBounds = CalcBounds(TempTransform);
		UseInGameBounds(false);
		FBoxSphereBounds PreviewBounds = CalcBounds(TempTransform);
		UseInGameBounds(bWasUsingInGameBounds);
		// calculate again to have bounds as requested
		CalcBounds(TempTransform);
		// if in-game bounds are of almost same size as preview bounds or bigger, it seems to be fine
		if (! InGameBounds.GetSphere().IsInside(PreviewBounds.GetSphere(), PreviewBounds.GetSphere().W * 0.1f) && // for spheres: A.IsInside(B) checks if A is inside of B
			! PreviewBounds.GetBox().IsInside(InGameBounds.GetBox().ExpandBy(PreviewBounds.GetSphere().W * 0.1f))) // for boxes: A.IsInside(B) checks if B is inside of A
		{
			return true;
		}
	}
	return false;
}

double WrapInRange(double StartVal, double MinVal, double MaxVal)
{
	double Size = MaxVal - MinVal;
	double EndVal = StartVal;
	while (EndVal < MinVal)
	{
		EndVal += Size;
	}

	while (EndVal > MaxVal)
	{
		EndVal -= Size;
	}
	return EndVal;
}

void UDebugSkelMeshComponent::ConsumeRootMotion(const FVector& FloorMin, const FVector& FloorMax)
{
	// Note: this method is called from FAnimationEditorPreviewScene() after world tick,
	// so care must be taken how the transforms are adjusted. Other features,
	// such as physic simulation or turn table rotation may change the transform. 
	if (PreviewInstance == nullptr)
	{
		ConsumeRootMotionPreviousAdditiveAnimType = AAT_MAX;
		return;
	}

	if (PreviewInstance->GetAnimSequence() != nullptr && ConsumeRootMotionPreviousAdditiveAnimType != PreviewInstance->GetAnimSequence()->GetAdditiveAnimType())
	{
		ConsumeRootMotionPreviousAdditiveAnimType = PreviewInstance->GetAnimSequence()->GetAdditiveAnimType();
		ResetRootTransform();
	}

	// Force ProcessRootMotionMode to Ignore if the current asset/animation blueprint is not using root motion. 
	if (ProcessRootMotionMode != EProcessRootMotionMode::Ignore && DoesCurrentAssetHaveRootMotion() == false)
	{
		SetProcessRootMotionModeInternal(EProcessRootMotionMode::Ignore);
	}

	// If our requested mode became available, use it.
	if (ProcessRootMotionMode != RequestedProcessRootMotionMode && CanUseProcessRootMotionMode(RequestedProcessRootMotionMode))
	{
		SetProcessRootMotionModeInternal(RequestedProcessRootMotionMode);
	}

	//Extract root motion regardless of where we use it so that we don't hit problems with it building up in the instance
	FRootMotionMovementParams ExtractedRootMotion = ConsumeRootMotion_Internal(1.0f);
	if (PreviewInstance->GetMirrorDataTable())
	{
		const FTransform MirroredTransform = UE::Anim::MirrorTransform(ExtractedRootMotion.GetRootMotionTransform(), *PreviewInstance->GetMirrorDataTable());
		ExtractedRootMotion.Set(MirroredTransform);
	}

	const float CurrentTime = PreviewInstance->GetCurrentTime();
	const float PreviousTime = ConsumeRootMotionPreviousPlaybackTime;
	ConsumeRootMotionPreviousPlaybackTime = CurrentTime;

	// Apply the root motion, including resetting the actors location in case the process root motion mode requires it.
	// If ShouldBlendPhysicsBones() is true, do not alter the transform, because the readback from the physics bodies into
	// the bone transforms will already have been done during the tick based on the component transform.
	// If we change the component transform here, then the bone transforms will not match the physics simulation.
	if (!ShouldBlendPhysicsBones())
	{
		if (ProcessRootMotionMode != EProcessRootMotionMode::Ignore)
		{
			if (PreviewInstance->IsPlaying())
			{
				// Figure out if the animation has looped, and the start end position of the root motion extraction.
				float SectionStartPosition = 0.0f;
				bool bLooped = false;

				// We have to deal with montage explicitly because we can have multiple sections and we want to reset the position when the section loops
				// and depending on the composition, CurrentTime < PreviousTime (or CurrentTime > PreviousTime when playing in reverse) is not enough
				if (const UAnimMontage* Montage = Cast<UAnimMontage>(PreviewInstance->CurrentAsset))
				{
					const int32 PreviewStartSectionIdx = Montage->CompositeSections.IsValidIndex(PreviewInstance->MontagePreviewStartSectionIdx) ? PreviewInstance->MontagePreviewStartSectionIdx : Montage->GetSectionIndexFromPosition(CurrentTime);
					const int32 FirstSectionIdx = PreviewInstance->MontagePreview_FindFirstSectionAsInMontage(PreviewStartSectionIdx);
					const int32 LastSectionIdx = PreviewInstance->MontagePreview_FindLastSection(FirstSectionIdx);

					// If FirstSection == LastSection we are previewing a single section
					// In this case to know if we have looped we just need to check if CurrentTime < PreviousTime (or the oposite if we are playing the montage in reverse)
					if (FirstSectionIdx == LastSectionIdx)
					{
						bLooped = PreviewInstance->IsReverse() ? (CurrentTime > PreviousTime) : (CurrentTime < PreviousTime);
					}
					// Otherwise, we are previewing a montage with multiple section. In this case we check if section at CurrentTime is the FirstSection and the section at PreviewTime is the LastSection (or the opposite if we are playing the montage in reverse)
					else
					{
						const int32 SectionIndexPrevTime = Montage->GetSectionIndexFromPosition(PreviousTime);
						const int32 SectionIndexCurrentTime = Montage->GetSectionIndexFromPosition(CurrentTime);
						bLooped = PreviewInstance->IsReverse() ? (SectionIndexPrevTime == FirstSectionIdx && SectionIndexCurrentTime == LastSectionIdx) : (SectionIndexPrevTime == LastSectionIdx && SectionIndexCurrentTime == FirstSectionIdx);
					}

					// If we have looped...
					if (bLooped)
					{
						float StartTime = 0.0f, EndTime = 0.0f;
						Montage->GetSectionStartAndEndTime(LastSectionIdx, StartTime, EndTime);
						SectionStartPosition = StartTime;
					}
				}
				else // CurrentAsset is not a Montage
				{
					bLooped = PreviewInstance->IsReverse() ? (CurrentTime > PreviousTime) : (CurrentTime < PreviousTime);
				}
				
				// Loop Mode: Preview mesh will consume root motion continually
				if (ProcessRootMotionMode == EProcessRootMotionMode::Loop)
				{
					FTransform RootMotionTransform = GetRelativeTransform();
					const FTransform RootMotionDelta = ExtractedRootMotion.GetRootMotionTransform();
					RootMotionTransform = RootMotionDelta * RootMotionTransform;

					//Handle moving component so that it stays within the editor floor
					FVector Trans = RootMotionTransform.GetLocation();
					Trans.X = WrapInRange(Trans.X, FloorMin.X, FloorMax.X);
					Trans.Y = WrapInRange(Trans.Y, FloorMin.Y, FloorMax.Y);
					const FVector WrapDelta = Trans - RootMotionTransform.GetTranslation();
					RootMotionTransform.SetTranslation(Trans);

					if (!WrapDelta.IsNearlyZero())
					{
						SetRelativeTransform(RootMotionTransform);
					}
					else
					{
						AddLocalTransform(RootMotionDelta);
					}
					
					// Looping resets the root motion reference transform.
					if (bLooped)
					{
						RootMotionReferenceTransform = RootMotionTransform;
					}
				}
				// Loop and Reset Mode: Preview mesh will consume root motion resetting the position back to the origin every time the animation loops
				else if (ProcessRootMotionMode == EProcessRootMotionMode::LoopAndReset)
				{
					if (bLooped)
					{
						const FTransform InitialTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, SectionStartPosition);
						const FTransform RootMotionDelta = UE::Anim::ExtractRootMotionFromAnimationAsset(PreviewInstance->CurrentAsset, PreviewInstance->GetMirrorDataTable(), SectionStartPosition, CurrentTime);
						const FTransform RootMotionTransform = RootMotionDelta * InitialTransform;

						// Reference transform is always relative to the beginning of the animation sequence. 
						RootMotionReferenceTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, 0.0f);
						
						SetRelativeTransform(RootMotionTransform);
					}
					else
					{
						const FTransform RootMotionDelta = ExtractedRootMotion.GetRootMotionTransform();
						AddLocalTransform(RootMotionDelta);
					}
				}
			}
			else // Not Playing. When not playing user can still scrub the time line but animation is not ticking so we have to extract and apply root motion manually
			{
				const FTransform RootMotionDelta = UE::Anim::ExtractRootMotionFromAnimationAsset(PreviewInstance->CurrentAsset, PreviewInstance->GetMirrorDataTable(), PreviousTime, CurrentTime);
				AddLocalTransform(RootMotionDelta);
			}
		}
	}
}

bool UDebugSkelMeshComponent::IsProcessingRootMotion() const 
{ 
	return GetProcessRootMotionMode() != EProcessRootMotionMode::Ignore;
}

EProcessRootMotionMode UDebugSkelMeshComponent::GetProcessRootMotionMode() const
{
	return ProcessRootMotionMode;
}

EProcessRootMotionMode UDebugSkelMeshComponent::GetRequestedProcessRootMotionMode() const
{
	return RequestedProcessRootMotionMode;
}

bool UDebugSkelMeshComponent::DoesCurrentAssetHaveRootMotion() const
{
	if (PreviewInstance)
	{
		// Allow root motion if current asset is sequence
		if(const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(PreviewInstance->GetCurrentAsset()))
		{
			return AnimSequenceBase->HasRootMotion();
		}
		
		// Allow root motion if current blend-space references any sequences with root motion
		if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(PreviewInstance->GetCurrentAsset()))
		{
			bool bIsRootMotionUsedInBlendSpace = false;
			
			BlendSpace->ForEachImmutableSample([&bIsRootMotionUsedInBlendSpace](const FBlendSample & Sample)
			{
				const TObjectPtr<UAnimSequence> Sequence = Sample.Animation;
				
				if (IsValid(Sequence) && Sequence->HasRootMotion())
				{
					bIsRootMotionUsedInBlendSpace = true;
				}
			});
			
			if (bIsRootMotionUsedInBlendSpace)
			{
				return true;
			}
		}

		// Allow previewing an animation blueprint with root motion
		if (!PreviewInstance->GetCurrentAsset())
		{
			if (PreviewInstance->RootMotionMode == ERootMotionMode::RootMotionFromEverything || PreviewInstance->RootMotionMode == ERootMotionMode::RootMotionFromMontagesOnly)
			{
				return true;
			}
		}
	}

	return false;
}

bool UDebugSkelMeshComponent::CanUseProcessRootMotionMode(EProcessRootMotionMode Mode) const
{
	if (PreviewInstance == nullptr)
	{
		return false;
	}
	
	// Disable Loop modes if the current asset or animation blueprint doesn't have root motion
	if (Mode != EProcessRootMotionMode::Ignore)
	{
		if (!DoesCurrentAssetHaveRootMotion())
		{
			return false;
		}
	}

	// Disable Loop and Reset mode for blend spaces and animation blueprints
	if (Mode == EProcessRootMotionMode::LoopAndReset)
	{
		if (Cast<UBlendSpace>(PreviewInstance->GetCurrentAsset()) || !PreviewInstance->GetCurrentAsset())
		{
			return false;
		}
	}
	
	return true;
}

void UDebugSkelMeshComponent::SetProcessRootMotionMode(EProcessRootMotionMode Mode)
{
	RequestedProcessRootMotionMode = Mode;
	
	if (CanUseProcessRootMotionMode(Mode))
	{
		SetProcessRootMotionModeInternal(Mode);
	}
}

void UDebugSkelMeshComponent::SetProcessRootMotionModeInternal(EProcessRootMotionMode Mode)
{
	ProcessRootMotionMode = Mode;
	ResetRootTransform();
}

void UDebugSkelMeshComponent::ResetRootTransform()
{
	FTransform RootMotionTransform = FTransform::Identity;

	if (DoesCurrentAssetHaveRootMotion())
	{
		if (ProcessRootMotionMode == EProcessRootMotionMode::LoopAndReset || ProcessRootMotionMode == EProcessRootMotionMode::Loop)
		{
			// Reset transform
			const float CurrentTime = PreviewInstance->GetCurrentTime();
			ConsumeRootMotionPreviousPlaybackTime = CurrentTime; // Important to cycle this otherwise root motion will be double applied
			float SectionStartPosition = 0.0f;
			if (const UAnimMontage* Montage = Cast<UAnimMontage>(PreviewInstance->CurrentAsset))
			{
				const int32 PreviewStartSectionIdx = Montage->CompositeSections.IsValidIndex(PreviewInstance->MontagePreviewStartSectionIdx) ? PreviewInstance->MontagePreviewStartSectionIdx : Montage->GetSectionIndexFromPosition(CurrentTime);
				const int32 FirstSectionIdx = PreviewInstance->MontagePreview_FindFirstSectionAsInMontage(PreviewStartSectionIdx);
				const int32 LastSectionIdx = PreviewInstance->MontagePreview_FindLastSection(FirstSectionIdx);
				float StartTime = 0.0f, EndTime = 0.0f;
				Montage->GetSectionStartAndEndTime(LastSectionIdx, StartTime, EndTime);
				SectionStartPosition = StartTime;
			}
		
			const FTransform InitialTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, SectionStartPosition);
			const FTransform RootMotionDelta = UE::Anim::ExtractRootMotionFromAnimationAsset(PreviewInstance->CurrentAsset, PreviewInstance->GetMirrorDataTable(), SectionStartPosition, CurrentTime);
			RootMotionTransform = RootMotionDelta * InitialTransform;
			
			// Reference transform is always relative to the beginning of the animation sequence. 
			RootMotionReferenceTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, 0.0f);
		}
		else if (ProcessRootMotionMode == EProcessRootMotionMode::Ignore)
		{
			RootMotionTransform = FTransform::Identity;
			RootMotionReferenceTransform = FTransform::Identity;
		}
	}
	else
	{
		// No root motion in animation, reset transform. 
		RootMotionTransform = FTransform::Identity;
		RootMotionReferenceTransform = FTransform::Identity;
	}

	SetRelativeTransform(RootMotionTransform);

	// Discard any runtime-extracted root motion still pending on the anim instances. RootMotionTransform
	// already represents the analytical position for the current playback time, so a pending delta would
	// be added on top of it by the next AddLocalTransform in ConsumeRootMotion, double-counting the span.
	ConsumeRootMotion_Internal(1.0f);
}

bool UDebugSkelMeshComponent::IsTrackingAttachedLOD() const
{
	return bTrackAttachedInstanceLOD;
}

FPrimitiveSceneProxy* UDebugSkelMeshComponent::CreateSceneProxy()
{
	FPrimitiveSceneProxy* Result = nullptr;
	ERHIFeatureLevel::Type SceneFeatureLevel = GetWorld()->GetFeatureLevel();
	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshAsset() ? GetSkeletalMeshAsset()->GetResourceForRendering() : nullptr;

	// only create a scene proxy for rendering if
	// properly initialized
	if(SkelMeshRenderData &&
		SkelMeshRenderData->LODRenderData.IsValidIndex(GetPredictedLODLevel()) &&
		!bHideSkin &&
		MeshObject)
	{
		if (MeshObject->IsNaniteMesh())
		{
			Result = ::new FDebugSkelMeshSceneProxyNanite(*MeshObject->GetNaniteMaterials(), this, SkelMeshRenderData, WireframeMeshOverlayColor);
		}
		else
		{
			Result = ::new FDebugSkelMeshSceneProxy(this, SkelMeshRenderData, WireframeMeshOverlayColor);
		}
	}

	return Result;
}

bool UDebugSkelMeshComponent::ShouldRenderSelected() const
{
	return bDisplayBound || bDisplayVertexColors;
}

bool UDebugSkelMeshComponent::IsPreviewOn() const
{
	return (PreviewInstance != nullptr) && (PreviewInstance == AnimScriptInstance);
}

FString UDebugSkelMeshComponent::GetPreviewText() const
{
#define LOCTEXT_NAMESPACE "SkelMeshComponent"

	if (IsPreviewOn())
	{
		UAnimationAsset* CurrentAsset = PreviewInstance->GetCurrentAsset();
		if (USkeletalMeshComponent* SkeletalMeshComponent = PreviewInstance->GetDebugSkeletalMeshComponent())
		{
			FText Label = SkeletalMeshComponent->GetOwner() ? FText::FromString(SkeletalMeshComponent->GetOwner()->GetActorLabel()) : LOCTEXT("NoActor", "None");
			return FText::Format(LOCTEXT("ExternalComponent", "External Instance on {0}"), Label).ToString();
		}
		else if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(CurrentAsset))
		{
			return FText::Format( LOCTEXT("BlendSpace", "Blend Space {0}"), FText::FromString(BlendSpace->GetName()) ).ToString();
		}
		else if (UAnimMontage* Montage = Cast<UAnimMontage>(CurrentAsset))
		{
			return FText::Format( LOCTEXT("Montage", "Montage {0}"), FText::FromString(Montage->GetName()) ).ToString();
		}
		else if(UAnimComposite* Composite = Cast<UAnimComposite>(CurrentAsset))
		{
			return FText::Format(LOCTEXT("Composite", "Composite {0}"), FText::FromString(Composite->GetName())).ToString();
		}
		else if (UAnimSequence* Sequence = Cast<UAnimSequence>(CurrentAsset))
		{
			return FText::Format( LOCTEXT("Animation", "Animation {0}"), FText::FromString(Sequence->GetName()) ).ToString();
		}
	}

	return LOCTEXT("ReferencePose", "Reference Pose").ToString();

#undef LOCTEXT_NAMESPACE
}

TObjectPtr<UAnimPreviewInstance> UDebugSkelMeshComponent::CreatePreviewInstance()
{
	return NewObject<UAnimPreviewInstance>(this);
}

void UDebugSkelMeshComponent::InitAnim(bool bForceReinit)
{
	if (PreviewInstance != nullptr && AnimScriptInstance == PreviewInstance && bForceReinit)
	{
		// Reset current animation data
		AnimationData.PopulateFrom(PreviewInstance);
		AnimationData.Initialize(PreviewInstance);
	}

	Super::InitAnim(bForceReinit);

	if(GetSkeletalMeshAsset() != nullptr)
	{
		// if PreviewInstance is nullptr, create here once
		if (PreviewInstance == nullptr)
		{
			PreviewInstance = CreatePreviewInstance();
			check(PreviewInstance);

			//Set transactional flag in order to restore slider position when undo operation is performed
			PreviewInstance->SetFlags(RF_Transactional);
		}

		// if anim script instance is null because it's not playing a blueprint, set to PreviewInstnace by default
		// that way if user would like to modify bones or do extra stuff, it will work
		if (AnimScriptInstance == nullptr)
		{
			AnimScriptInstance = PreviewInstance;
			AnimScriptInstance->InitializeAnimation();
		}
		else
		{
			// Make sure we initialize the preview instance here, as we want the required bones to be up to date
			// even if we arent using the instance right now.
			PreviewInstance->InitializeAnimation();
		}

		if(PostProcessAnimInstance)
		{
			// Add the same settings as the preview instance in this case.
			PostProcessAnimInstance->RootMotionMode = ERootMotionMode::RootMotionFromEverything;
			PostProcessAnimInstance->bUseMultiThreadedAnimationUpdate = false;
		}
	}
}

void UDebugSkelMeshComponent::OnClearAnimScriptInstance()
{
	// call to super not strictly necessary (since it is empty)
	Super::OnClearAnimScriptInstance();
	
	SavedAnimScriptInstance = nullptr;
}

void UDebugSkelMeshComponent::SetSkeletalMesh(USkeletalMesh* InSkelMesh, bool bReinitPose)
{
	Super::SetSkeletalMesh(InSkelMesh, bReinitPose);

	// Clear any transitive references to the skeleton if we are clearing the skeletal mesh
	if(GetSkinnedAsset() == nullptr)
	{
		if(AnimScriptInstance)
		{
			AnimScriptInstance->CurrentSkeleton = nullptr;
		}

		if(PreviewInstance)
		{
			PreviewInstance->CurrentSkeleton = nullptr;
		}

		if(SavedAnimScriptInstance)
		{
			SavedAnimScriptInstance->CurrentSkeleton = nullptr;
		}
	}
}

void UDebugSkelMeshComponent::PostInitProperties()
{
	EAnimationMode::Type OriginalMode = AnimationMode;

	// potentially reverts the mode to "AnimationSingleNode" which is not compatible with animation editors
	Super::PostInitProperties();

	if (OriginalMode == EAnimationMode::AnimationBlueprint)
	{
		// in cases where AnimationBlueprint mode is not supported, revert to custom mode to prevent
		// the base USkeletalMeshComponent from overriding the anim instance used by this component
		AnimationMode = EAnimationMode::AnimationCustomMode;
	}
}

void UDebugSkelMeshComponent::EnablePreview(bool bEnable, UAnimationAsset* PreviewAsset)
{
	if (PreviewInstance)
	{
		if (bEnable)
		{
			// back up current AnimInstance if not currently previewing anything
			if (!IsPreviewOn())
			{
				SavedAnimScriptInstance = AnimScriptInstance;
			}

			// When this code path gets hit during undo/redo calls and we are previewing the same asset as before 
			// we want to restore playback position and play state after SetAnimationAsset to avoid transacting actions reseting the state of anim editors
			const bool bRestoreState = GIsTransacting && IsPreviewOn() && PreviewInstance->GetCurrentAsset() == PreviewAsset;
			const bool bWasPlaying = PreviewInstance->IsPlaying();
			const float PrevTime = PreviewInstance->GetCurrentTime();
			
			AnimScriptInstance = PreviewInstance;
		    // restore previous state
		    bDisableClothSimulation = bPrevDisableClothSimulation;
    
			const bool bWasLooping = PreviewInstance->IsLooping();
			PreviewInstance->SetAnimationAsset(PreviewAsset, bWasLooping);
			
			if (bRestoreState)
			{
				PreviewInstance->SetPosition(PrevTime, false);
				PreviewInstance->SetPlaying(bWasPlaying);
			}

			// Reset to previous animation asset's root motion playback time to prevent this from influencing the new animation asset previewing during root motion consumption.
			ConsumeRootMotionPreviousPlaybackTime = 0.0f;
			
			// Update requested process root motion mode, the new asset might support requested processing mode.
			// Note: This might further reset the transform.
			SetProcessRootMotionModeInternal(CanUseProcessRootMotionMode(RequestedProcessRootMotionMode) ? RequestedProcessRootMotionMode : EProcessRootMotionMode::Ignore);

			// clear trajectory visualization when the new asset has no root motion
			if (!DoesCurrentAssetHaveRootMotion())
			{
				VisualizeRootMotionMode = EVisualizeRootMotionMode::None;
			}
		}
		else if (IsPreviewOn())
		{
			if (PreviewInstance->GetCurrentAsset() == PreviewAsset || PreviewAsset == nullptr)
			{
				// now recover to saved AnimScriptInstance;
				AnimScriptInstance = SavedAnimScriptInstance;
				SavedAnimScriptInstance = nullptr;
				PreviewInstance->SetAnimationAsset(nullptr);
			}
		}

		ClothTeleportMode = EClothingTeleportMode::TeleportAndReset;
	}
}

bool UDebugSkelMeshComponent::ShouldCPUSkin() const
{
	return 	GetCPUSkinningEnabled() || bDrawBoneInfluences || bDrawNormals || bDrawTangents || bDrawBinormals || bDrawMorphTargetVerts;
}

bool UDebugSkelMeshComponent::ShouldNaniteSkin()
{
	return !IsDisplayingNaniteFallback() && USkinnedMeshComponent::ShouldNaniteSkin();
}

void UDebugSkelMeshComponent::PostInitMeshObject(FSkeletalMeshObject* InMeshObject)
{
	Super::PostInitMeshObject( InMeshObject );

	if (InMeshObject)
	{
		if(bDrawBoneInfluences)
		{
			InMeshObject->EnableOverlayRendering(true, &BonesOfInterest, nullptr);
		}
		else if (bDrawMorphTargetVerts)
		{
			InMeshObject->EnableOverlayRendering(true, nullptr, &ToRawPtrTArrayUnsafe(MorphTargetOfInterests));
		}
	}
}

void UDebugSkelMeshComponent::OnMirrorDataTableChanged()
{
	if (!DoesCurrentAssetHaveRootMotion())
	{
		return;
	}

	if (ProcessRootMotionMode == EProcessRootMotionMode::LoopAndReset)
	{
		// Reset transform
		const float CurrentTime = PreviewInstance->GetCurrentTime();
		float SectionStartPosition = 0.0f;
		if (const UAnimMontage* Montage = Cast<UAnimMontage>(PreviewInstance->CurrentAsset))
		{
			const int32 PreviewStartSectionIdx = Montage->CompositeSections.IsValidIndex(PreviewInstance->MontagePreviewStartSectionIdx) ? PreviewInstance->MontagePreviewStartSectionIdx : Montage->GetSectionIndexFromPosition(CurrentTime);
			const int32 FirstSectionIdx = PreviewInstance->MontagePreview_FindFirstSectionAsInMontage(PreviewStartSectionIdx);
			const int32 LastSectionIdx = PreviewInstance->MontagePreview_FindLastSection(FirstSectionIdx);
			float StartTime = 0.0f, EndTime = 0.0f;
			Montage->GetSectionStartAndEndTime(LastSectionIdx, StartTime, EndTime);
			SectionStartPosition = StartTime;
		}

		const FTransform InitialTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, SectionStartPosition);
		const FTransform RootMotionDelta = UE::Anim::ExtractRootMotionFromAnimationAsset(PreviewInstance->CurrentAsset, PreviewInstance->GetMirrorDataTable(), SectionStartPosition, CurrentTime);
		const FTransform RootMotionTransform = RootMotionDelta * InitialTransform;

		// Reference transform is always relative to the beginning of the animation sequence. 
		RootMotionReferenceTransform = UE::Anim::ExtractRootTransformFromAnimationAsset(PreviewInstance->CurrentAsset, 0.0f);

		SetRelativeTransform(RootMotionTransform);
	}
}

void UDebugSkelMeshComponent::SetShowBoneWeight(bool bNewShowBoneWeight)
{
	// Check we are actually changing it!
	if(bNewShowBoneWeight == bDrawBoneInfluences)
	{
		return;
	}

	if (bDrawMorphTargetVerts)
	{
		SetShowMorphTargetVerts(false);
	}

	// if turning on this mode
	EnableOverlayMaterial(bNewShowBoneWeight);

	bDrawBoneInfluences = bNewShowBoneWeight;
}

void UDebugSkelMeshComponent::EnableOverlayMaterial(bool bEnable)
{
	if (bEnable)
	{
		SkelMaterials.Empty();
		int32 NumMaterials = GetNumMaterials();
		for (int32 i = 0; i < NumMaterials; i++)
		{
			// Back up old material
			SkelMaterials.Add(GetMaterial(i));
			// Set special bone weight material
			SetMaterial(i, GEngine->BoneWeightMaterial);
		}
	}
	// if turning it off
	else
	{
		int32 NumMaterials = GetNumMaterials();
		check(NumMaterials == SkelMaterials.Num());
		for (int32 i = 0; i < NumMaterials; i++)
		{
			// restore original material
			SetMaterial(i, SkelMaterials[i]);
		}
	}
}

bool UDebugSkelMeshComponent::ShouldRunClothTick() const
{
	const bool bBaseShouldTick = Super::ShouldRunClothTick();
	const bool bBaseCouldTick = CanSimulateClothing();

	// If we could tick, but our simulation is suspended - only tick if we've attempted to step the animation
	if(bBaseCouldTick && bClothingSimulationSuspended && bPerformSingleClothingTick)
	{
		return true;
	}

	return bBaseShouldTick;
}

void UDebugSkelMeshComponent::SendRenderDynamicData_Concurrent()
{
	Super::SendRenderDynamicData_Concurrent();
	
	if (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}

	if (SceneProxy)
	{
		FDebugSkelMeshDynamicData* NewDynamicData = new FDebugSkelMeshDynamicData(this);
		
		const bool bNaniteProxy = MeshObject && MeshObject->IsNaniteMesh();
		FDebugSkelMeshSceneProxyCommon* TargetProxy;
		if (bNaniteProxy)
		{
			TargetProxy = static_cast<FDebugSkelMeshSceneProxyNanite*>(SceneProxy);
		}
		else
		{
			TargetProxy = static_cast<FDebugSkelMeshSceneProxy*>(SceneProxy);
		}
		
		ENQUEUE_RENDER_COMMAND(DebugSkelMeshObjectUpdateDataCommand)(
			[NewDynamicData, TargetProxy, bNaniteProxy](FRHICommandListImmediate& RHICommandList)
		{
			if (TargetProxy->DynamicData)
			{
				delete TargetProxy->DynamicData;
			}

			TargetProxy->DynamicData = NewDynamicData;
		}
		);
	} //-V773
}

void UDebugSkelMeshComponent::SetShowMorphTargetVerts(bool bNewShowMorphTargetVerts)
{
	// Check we are actually changing it!
	if (bNewShowMorphTargetVerts == bDrawMorphTargetVerts)
	{
		return;
	}

	if (bDrawBoneInfluences)
	{
		SetShowBoneWeight(false);
	}

	// if turning on this mode
	EnableOverlayMaterial(bNewShowMorphTargetVerts);

	bDrawMorphTargetVerts = bNewShowMorphTargetVerts;
}

void UDebugSkelMeshComponent::GenSpaceBases(TArray<FTransform>& OutSpaceBases)
{
	TArray<FTransform> TempBoneSpaceTransforms;
	TempBoneSpaceTransforms.AddUninitialized(OutSpaceBases.Num());
	FVector TempRootBoneTranslation;
	FBlendedHeapCurve TempCurve;
	UE::Anim::FMeshAttributeContainer TempAtttributes;
	DoInstancePreEvaluation();
	PerformAnimationEvaluation(GetSkeletalMeshAsset(), AnimScriptInstance, OutSpaceBases, TempBoneSpaceTransforms, TempRootBoneTranslation, TempCurve, TempAtttributes);
	DoInstancePostEvaluation();
	DoInstanceFinalizeAnimation(true);
}

void UDebugSkelMeshComponent::RefreshBoneTransforms(FActorComponentTickFunction* TickFunction)
{
	// Run regular update first so we get RequiredBones up to date.
	Super::RefreshBoneTransforms(nullptr); // Pass nullptr so we force non threaded work

	// none of these code works if we don't have anim instance, so no reason to check it for every if
	if (AnimScriptInstance && AnimScriptInstance->GetRequiredBones().IsValid())
	{
		const bool bIsPreviewInstance = (PreviewInstance && PreviewInstance == AnimScriptInstance);	
		FBoneContainer& BoneContainer = AnimScriptInstance->GetRequiredBones();

		BakedAnimationPoses.Reset();
		if(bDisplayBakedAnimation && bIsPreviewInstance)
		{
			if(UAnimSequence* Sequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset()))
			{
				BakedAnimationPoses.AddUninitialized(BoneContainer.GetNumBones());
				bool bSavedUseSourceData = BoneContainer.ShouldUseSourceData();
				BoneContainer.SetUseRAWData(true);
				BoneContainer.SetUseSourceData(false);
				PreviewInstance->EnableControllers(false);
				GenSpaceBases(BakedAnimationPoses);
				BoneContainer.SetUseRAWData(false);
				BoneContainer.SetUseSourceData(bSavedUseSourceData);
				PreviewInstance->EnableControllers(true);
			}
		}

		SourceAnimationPoses.Reset();
		if(bDisplaySourceAnimation && bIsPreviewInstance)
		{
			if(UAnimSequence* Sequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset()))
			{
				SourceAnimationPoses.AddUninitialized(BoneContainer.GetNumBones());
				bool bSavedUseSourceData = BoneContainer.ShouldUseSourceData();
				BoneContainer.SetUseSourceData(true);
				PreviewInstance->EnableControllers(false);
				GenSpaceBases(SourceAnimationPoses);
				BoneContainer.SetUseSourceData(bSavedUseSourceData);
				PreviewInstance->EnableControllers(true);
			}
		}

		UncompressedSpaceBases.Reset();
		if ( bDisplayRawAnimation )
		{
			UncompressedSpaceBases.AddUninitialized(BoneContainer.GetNumBones());

			const bool bUseSource = BoneContainer.ShouldUseSourceData();
			const bool bUseRaw = BoneContainer.ShouldUseRawData();

			BoneContainer.SetUseSourceData(false);
			BoneContainer.SetUseRAWData(true);
			PreviewInstance->EnableControllers(false);

			GenSpaceBases(UncompressedSpaceBases);
			
			BoneContainer.SetUseRAWData(bUseRaw);
			BoneContainer.SetUseSourceData(bUseSource);
			PreviewInstance->EnableControllers(true);
		}

		// Non retargeted pose.
		NonRetargetedSpaceBases.Reset();
		if( bDisplayNonRetargetedPose )
		{
			NonRetargetedSpaceBases.AddUninitialized(BoneContainer.GetNumBones());
			BoneContainer.SetDisableRetargeting(true);
			GenSpaceBases(NonRetargetedSpaceBases);
			BoneContainer.SetDisableRetargeting(false);
		}

		// Only works in PreviewInstance, and not for anim blueprint. This is intended.
		AdditiveBasePoses.Reset();
		if( bDisplayAdditiveBasePose && bIsPreviewInstance )
		{
			if (UAnimSequence* Sequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset())) 
			{ 
				if (Sequence->IsValidAdditive()) 
				{ 
					FCSPose<FCompactPose> CSAdditiveBasePose;
					{
						FCompactPose AdditiveBasePose;
						FBlendedCurve AdditiveCurve;
						UE::Anim::FStackAttributeContainer AdditiveAttributes;
						AdditiveCurve.InitFrom(BoneContainer);
						AdditiveBasePose.SetBoneContainer(&BoneContainer);
						
						FAnimationPoseData AnimationPoseData(AdditiveBasePose, AdditiveCurve, AdditiveAttributes);
						Sequence->GetAdditiveBasePose(AnimationPoseData, FAnimExtractContext(static_cast<double>(PreviewInstance->GetCurrentTime())));
						CSAdditiveBasePose.InitPose(AnimationPoseData.GetPose());
					}

					const int32 NumSkeletonBones = BoneContainer.GetNumBones();

					AdditiveBasePoses.AddUninitialized(NumSkeletonBones);

					for (int32 i = 0; i < AdditiveBasePoses.Num(); ++i)
					{
						FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(i));

						// AdditiveBasePoses has one entry for every bone in the asset ref skeleton - if we're on a LOD
						// we need to check this is actually valid for the current pose.
						if(CSAdditiveBasePose.GetPose().IsValidIndex(CompactIndex))
						{
							AdditiveBasePoses[i] = CSAdditiveBasePose.GetComponentSpaceTransform(CompactIndex);
						}
						else
						{
							AdditiveBasePoses[i] = FTransform::Identity;
						}
					}
				}
			}
		}
	}
}

#if WITH_EDITOR
void UDebugSkelMeshComponent::ReportAnimNotifyError(const FText& Error, UObject* InSourceNotify)
{
	for (FAnimNotifyErrors& Errors : AnimNotifyErrors)
	{
		if (Errors.SourceNotify == InSourceNotify)
		{
			Errors.Errors.Add(Error.ToString());
			return;
		}
	}

	int32 i = AnimNotifyErrors.Num();
	AnimNotifyErrors.Add(FAnimNotifyErrors(InSourceNotify));
	AnimNotifyErrors[i].Errors.Add(Error.ToString());
}

void UDebugSkelMeshComponent::ClearAnimNotifyErrors(UObject* InSourceNotify)
{
	for (FAnimNotifyErrors& Errors : AnimNotifyErrors)
	{
		if (Errors.SourceNotify == InSourceNotify)
		{
			Errors.Errors.Empty();
		}
	}
}

FDelegateHandle UDebugSkelMeshComponent::RegisterExtendedViewportTextDelegate(const FGetExtendedViewportText& InDelegate)
{
	ExtendedViewportTextDelegates.Add(InDelegate);
	return ExtendedViewportTextDelegates.Last().GetHandle();
}

void UDebugSkelMeshComponent::UnregisterExtendedViewportTextDelegate(const FDelegateHandle& InDelegateHandle)
{
	ExtendedViewportTextDelegates.RemoveAll([&InDelegateHandle](const FGetExtendedViewportText& InDelegate)
	{
		return InDelegate.GetHandle() == InDelegateHandle;
	});
}

FDelegateHandle UDebugSkelMeshComponent::RegisterOnDebugForceLODChangedDelegate(const FOnDebugForceLODChanged& InDelegate)
{
	OnDebugForceLODChangedDelegate = InDelegate;
	return OnDebugForceLODChangedDelegate.GetHandle();
}

void UDebugSkelMeshComponent::UnregisterOnDebugForceLODChangedDelegate()
{
	checkf(OnDebugForceLODChangedDelegate.IsBound(), TEXT("OnDebugForceLODChangedDelegate is not registered"));
	OnDebugForceLODChangedDelegate.Unbind();
}

#endif

void UDebugSkelMeshComponent::ToggleClothSectionsVisibility(bool bShowOnlyClothSections)
{
	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
	if (SkelMeshRenderData)
	{
		for (int32 LODIndex = 0; LODIndex < SkelMeshRenderData->LODRenderData.Num(); LODIndex++)
		{
			FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];

			for (int32 SecIdx = 0; SecIdx < LODData.RenderSections.Num(); SecIdx++)
			{
				FSkelMeshRenderSection& Section = LODData.RenderSections[SecIdx];

				if(Section.HasClothingData())
				{
					ShowMaterialSection(Section.MaterialIndex, SecIdx, bShowOnlyClothSections, LODIndex);
				}
				else
				{
					ShowMaterialSection(Section.MaterialIndex, SecIdx, !bShowOnlyClothSections, LODIndex);
				}
			}
		}
	}
}

void UDebugSkelMeshComponent::RestoreClothSectionsVisibility()
{
	if (!GetSkeletalMeshAsset())
	{
		return;
	}

	for(int32 LODIndex = 0; LODIndex < GetNumLODs(); LODIndex++)
	{
		ShowAllMaterialSections(LODIndex);
	}
}

void UDebugSkelMeshComponent::SetMeshSectionVisibilityForCloth(FGuid InClothGuid, bool bVisibility)
{
	if(!InClothGuid.IsValid())
	{
		// Nothing to toggle.
		return;
	}

	FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
	if(SkelMeshRenderData)
	{
		for(int32 LODIndex = 0; LODIndex < SkelMeshRenderData->LODRenderData.Num(); LODIndex++)
		{
			FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];

			for(int32 SecIdx = 0; SecIdx < LODData.RenderSections.Num(); SecIdx++)
			{
				FSkelMeshRenderSection& Section = LODData.RenderSections[SecIdx];

				// disables cloth section and also corresponding original section for matching cloth asset
				if(Section.HasClothingData() && Section.ClothingData.AssetGuid == InClothGuid)
				{
					ShowMaterialSection(Section.MaterialIndex, SecIdx, bVisibility, LODIndex);
				}
			}
		}
	}
}

void UDebugSkelMeshComponent::ResetMeshSectionVisibility()
{
	for (int32 LODIndex = 0; LODIndex < GetNumLODs(); LODIndex++)
	{
		ShowAllMaterialSections(LODIndex);
	}
}

void UDebugSkelMeshComponent::CheckClothTeleport()
{
	// do nothing to avoid clothing reset while modifying properties
	// modifying values can cause frame delay and clothes will be reset by a large delta time (low fps)
	// doesn't need cloth teleport while previewing
}

void UDebugSkelMeshComponent::SetTurnTableMode(EPersonaTurnTableMode::Type NewMode)
{
	if (TurnTableMode == NewMode)
	{
		return;
	}
	TurnTableMode = NewMode;
	
	if (TurnTableMode == EPersonaTurnTableMode::Stopped)
	{
		// Use SetProcessRootMotionModeInternal with current value to reset the root motion preview state.
		SetProcessRootMotionModeInternal(ProcessRootMotionMode);
	}
}

void UDebugSkelMeshComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	//Do not tick a skeletalmesh component if the skeletalmesh is compiling
	if (GetSkeletalMeshAsset() && GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}

	if (TurnTableMode == EPersonaTurnTableMode::Playing)
	{
		FRotator Rotation = GetRelativeTransform().Rotator();
		// Take into account time dilation, so it doesn't affect turn table turn rate.
		float CurrentTimeDilation = 1.0f;
		if (UWorld* MyWorld = GetWorld())
		{
			CurrentTimeDilation = MyWorld->GetWorldSettings()->GetEffectiveTimeDilation();
		}
		Rotation.Yaw += 36.f * TurnTableSpeedScaling * DeltaTime / FMath::Max(CurrentTimeDilation, KINDA_SMALL_NUMBER);
		SetRelativeRotation(Rotation);
	}
	
    // Brute force approach to ensure that when materials are changed the names are cached parameter names are updated 
	bCachedMaterialParameterIndicesAreDirty = true;
	
	// Force retargeting data to be re-cached to take into account skeleton edits.
	if (bRequiredBonesUpToDateDuringTick)
	{
		bRequiredBonesUpToDate = false;
	}

	if (bTrackAttachedInstanceLOD)
	{
		UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(AnimScriptInstance);
		USkeletalMeshComponent* TargetMeshComp = PreviewInstance->GetDebugSkeletalMeshComponent();
		if (TargetMeshComp && TargetMeshComp->GetPredictedLODLevel() + 1 != GetForcedLOD())
		{
			SetDebugForcedLOD(TargetMeshComp->GetPredictedLODLevel() + 1);
		}
	}

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	// The tick from our super will call ShouldRunClothTick on us which will 'consume' this flag.
	// flip this flag here to only allow a single tick.
	bPerformSingleClothingTick = false;

	// If we have clothing selected we need to skin the asset for the editor tools
	RefreshSelectedClothingSkinnedPositions();
	return;
}

void UDebugSkelMeshComponent::RefreshSelectedClothingSkinnedPositions()
{
	if (GetSkeletalMeshAsset() && SelectedClothingGuidForPainting.IsValid())
	{
		const TObjectPtr<UClothingAssetBase>* const Asset = GetSkeletalMeshAsset()->GetMeshClothingAssets().FindByPredicate(
			[this](const TObjectPtr<UClothingAssetBase>& Item) -> bool
			{
				return Item && Item->GetAssetGuid() == SelectedClothingGuidForPainting;
			});

		if (const UClothingAssetCommon* const ConcreteAsset = Cast<const UClothingAssetCommon>(Asset ? *Asset : nullptr))
		{
			if (ConcreteAsset->LodData.IsValidIndex(SelectedClothingLodForPainting))
			{
				TArray<FMatrix44f> RefToLocals;
				GetCurrentRefToLocalMatrices(RefToLocals, 0);  // Pass LOD0 to collect all bones

				const FClothLODDataCommon& LodData = ConcreteAsset->LodData[SelectedClothingLodForPainting];

				ClothingMeshUtils::SkinPhysicsMesh(
					ConcreteAsset->UsedBoneIndices,
					LodData.PhysicalMeshData,
					FTransform::Identity,
					RefToLocals.GetData(),
					RefToLocals.Num(),
					SkinnedSelectedClothingPositions,
					SkinnedSelectedClothingNormals);

				RebuildCachedClothBounds();
				return;
			}
		}
	}
	SkinnedSelectedClothingNormals.Reset();
	SkinnedSelectedClothingPositions.Reset();
}

void UDebugSkelMeshComponent::GetUsedMaterials(TArray<UMaterialInterface *>& OutMaterials, bool bGetDebugMaterials /*= false*/) const
{
	USkeletalMeshComponent::GetUsedMaterials(OutMaterials, bGetDebugMaterials);

	if (bGetDebugMaterials)
	{
		OutMaterials.Add(GEngine->ClothPaintMaterialInstance);
		OutMaterials.Add(GEngine->ClothPaintMaterialWireframeInstance);
		OutMaterials.Add(GEngine->ClothPaintOpaqueMaterialInstance);
		OutMaterials.Add(GEngine->ClothPaintOpaqueMaterialWireframeInstance);
	}
}


// UE_DEPRECATED(5.7, "Use GetClothingSimulationInstances instead.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
IClothingSimulation* UDebugSkelMeshComponent::GetMutableClothingSimulation()
{
	if (GetClothingSimulationInstances().Num())
	{
		// This function seems unused, and therefore a const_cast is probably a better idea than compromising the const version of GetClothingSimulation()
		if (IClothingSimulationInterface* const ClothingSimulation =
			const_cast<IClothingSimulationInterface*>(GetClothingSimulationInstances()[0].GetClothingSimulation()))
		{
			return ClothingSimulation->DynamicCastToIClothingSimulation();
		}
	}
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDebugSkelMeshComponent::RebuildCachedClothBounds()
{
	FBox ClothBBox(ForceInit);
	
	for ( int32 Index = 0; Index < SkinnedSelectedClothingPositions.Num(); ++Index )
	{
		ClothBBox += (FVector)SkinnedSelectedClothingPositions[Index];
	}

	CachedClothBounds = FBoxSphereBounds(ClothBBox);
}

void UDebugSkelMeshComponent::ShowReferencePose(bool bRefPose)
{
	if (bRefPose)
	{
		EnablePreview(true, nullptr);
	}
}

bool UDebugSkelMeshComponent::IsReferencePoseShown() const
{
	return (IsPreviewOn() && PreviewInstance->GetCurrentAsset() == nullptr);
}

/***************************************************
 * FDebugSkelMeshSceneProxyCommon
 ***************************************************/
void FDebugSkelMeshSceneProxyCommon::GetDynamicMeshElements(
	const FPrimitiveSceneProxy* SceneProxy,
	const FSkeletalMeshObject* MeshObject,
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	uint32 VisibilityMap,
	FMeshElementCollector& Collector) const
{
	if (MeshObject && DynamicData && (DynamicData->bDrawNormals || DynamicData->bDrawTangents || DynamicData->bDrawBinormals))
	{
		for(int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if(VisibilityMap & (1 << ViewIndex))
			{
				MeshObject->DrawVertexElements(Collector.GetPDI(ViewIndex), SceneProxy->GetLocalToWorld(), DynamicData->bDrawNormals, DynamicData->bDrawTangents, DynamicData->bDrawBinormals);
			}
		}
	}

	if(DynamicData && DynamicData->ClothingSimDataIndexWhenPainting != INDEX_NONE && DynamicData->bDrawClothPaintPreview)
	{
		if(DynamicData->SkinnedPositions.Num() > 0 && DynamicData->ClothingVisiblePropertyValues.Num() > 0)
		{
			if (Views.Num())
			{
				FDynamicMeshBuilder MeshBuilderSurface(Views[0]->GetFeatureLevel());
				FDynamicMeshBuilder MeshBuilderWireframe(Views[0]->GetFeatureLevel());

				const TArray<uint32>& Indices = DynamicData->ClothingSimIndices;
				const TArray<FVector3f>& Vertices = DynamicData->SkinnedPositions;
				const TArray<FVector3f>& Normals = DynamicData->SkinnedNormals;

				float* ValueArray = DynamicData->ClothingVisiblePropertyValues.GetData();

				const int32 NumVerts = Vertices.Num();

				const FLinearColor Magenta = FLinearColor(1.0f, 0.0f, 1.0f);
				for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
				{
					FDynamicMeshVertex Vert;

					Vert.Position = Vertices[VertIndex];
					Vert.TextureCoordinate[0] = { 1.0f, 1.0f };
					Vert.TangentZ = DynamicData->bFlipNormal ? -Normals[VertIndex] : Normals[VertIndex];

					float CurrValue = ValueArray[VertIndex];
					float Range = DynamicData->PropertyViewMax - DynamicData->PropertyViewMin;
					float ClampedViewValue = FMath::Clamp(CurrValue, DynamicData->PropertyViewMin, DynamicData->PropertyViewMax);
					const FLinearColor Color = CurrValue == 0.0f ? Magenta : (FLinearColor::White * ((ClampedViewValue - DynamicData->PropertyViewMin) / Range));
					Vert.Color = Color.ToFColor(true);

					MeshBuilderSurface.AddVertex(Vert);
					MeshBuilderWireframe.AddVertex(Vert);
				}

				const int32 NumIndices = Indices.Num();
				for (int32 TriBaseIndex = 0; TriBaseIndex < NumIndices; TriBaseIndex += 3)
				{
					if (DynamicData->bFlipNormal)
					{
						MeshBuilderSurface.AddTriangle(Indices[TriBaseIndex], Indices[TriBaseIndex + 2], Indices[TriBaseIndex + 1]);
						MeshBuilderWireframe.AddTriangle(Indices[TriBaseIndex], Indices[TriBaseIndex + 2], Indices[TriBaseIndex + 1]);
					}
					else
					{
						MeshBuilderSurface.AddTriangle(Indices[TriBaseIndex], Indices[TriBaseIndex + 1], Indices[TriBaseIndex + 2]);
						MeshBuilderWireframe.AddTriangle(Indices[TriBaseIndex], Indices[TriBaseIndex + 1], Indices[TriBaseIndex + 2]);
					}
				}

				// Set material params
				UMaterialInstanceDynamic* SurfaceMID = (DynamicData->bDoubleSidedOpaqueCloth && GEngine->ClothPaintOpaqueMaterialInstance) ? GEngine->ClothPaintOpaqueMaterialInstance : GEngine->ClothPaintMaterialInstance;
				check(SurfaceMID);
				UMaterialInstanceDynamic* WireMID = (DynamicData->bDoubleSidedOpaqueCloth && GEngine->ClothPaintOpaqueMaterialWireframeInstance) ? GEngine->ClothPaintOpaqueMaterialWireframeInstance : GEngine->ClothPaintMaterialWireframeInstance;
				check(WireMID);

				FMaterialRenderProxy* MatProxySurface = SurfaceMID->GetRenderProxy();
				FMaterialRenderProxy* MatProxyWireframe = WireMID->GetRenderProxy();

				if (MatProxySurface && MatProxyWireframe)
				{
					const int32 NumViews = Views.Num();
					for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
					{
						const FSceneView* View = Views[ViewIndex];
						MeshBuilderSurface.GetMesh(SceneProxy->GetLocalToWorld(), MatProxySurface, SDPG_Foreground, false, false, ViewIndex, Collector);
						MeshBuilderWireframe.GetMesh(SceneProxy->GetLocalToWorld(), MatProxyWireframe, SDPG_Foreground, false, false, ViewIndex, Collector);
					}
				}
			}
		}
	}
}

/***************************************************
 * FDebugSkelMeshSceneProxy
 ***************************************************/
FDebugSkelMeshSceneProxy::FDebugSkelMeshSceneProxy(const UDebugSkelMeshComponent* InComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, FLinearColor InWireframeOverlayColor)
	: FSkeletalMeshSceneProxy(InComponent, InSkelMeshRenderData)
{
	SetWireframeColor(InWireframeOverlayColor);

	if (GEngine->ClothPaintMaterial)
	{
		MaterialRelevance |= GEngine->ClothPaintMaterial->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
	}
}

SIZE_T FDebugSkelMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FDebugSkelMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	if (!DynamicData || DynamicData->bDrawMesh)
	{
		GetMeshElementsConditionallySelectable(Views, ViewFamily, IsSelectable(), VisibilityMap, Collector);
	}
	FDebugSkelMeshSceneProxyCommon::GetDynamicMeshElements(this, GetMeshObject(), Views, ViewFamily, VisibilityMap, Collector);
}

/***************************************************
 * FDebugSkelMeshSceneProxyNanite
 ***************************************************/
FDebugSkelMeshSceneProxyNanite::FDebugSkelMeshSceneProxyNanite(const Nanite::FMaterialAudit& Materials, const UDebugSkelMeshComponent* InComponent, FSkeletalMeshRenderData* InSkelMeshRenderData, FLinearColor InWireframeOverlayColor)
	: Nanite::FSkinnedSceneProxy(Materials, InComponent, InSkelMeshRenderData)
{
	SetWireframeColor(InWireframeOverlayColor);
}

SIZE_T FDebugSkelMeshSceneProxyNanite::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FDebugSkelMeshSceneProxyNanite::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	if (!DynamicData || DynamicData->bDrawMesh)
	{
		Nanite::FSkinnedSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);
	}
	FDebugSkelMeshSceneProxyCommon::GetDynamicMeshElements(this, GetMeshObject(), Views, ViewFamily, VisibilityMap, Collector);
}


FDebugSkelMeshDynamicData::FDebugSkelMeshDynamicData(UDebugSkelMeshComponent* InComponent)
	: bDrawMesh(InComponent->bDrawMesh)
	, bDrawNormals(InComponent->bDrawNormals)
	, bDrawTangents(InComponent->bDrawTangents)
	, bDrawBinormals(InComponent->bDrawBinormals)
	, bDrawClothPaintPreview(InComponent->bShowClothData)
	, bFlipNormal(InComponent->bClothFlipNormal)
	, bDoubleSidedOpaqueCloth(!InComponent->bClothCullBackface && InComponent->ClothMeshOpacity == 1.f)
	, ClothingSimDataIndexWhenPainting(INDEX_NONE)
	, PropertyViewMin(InComponent->MinClothPropertyView)
	, PropertyViewMax(InComponent->MaxClothPropertyView)
{
	if(InComponent->SelectedClothingGuidForPainting.IsValid())
	{
		SkinnedPositions = InComponent->SkinnedSelectedClothingPositions;
		SkinnedNormals = InComponent->SkinnedSelectedClothingNormals;

		if(USkeletalMesh* Mesh = InComponent->GetSkeletalMeshAsset())
		{
			const int32 NumClothingAssets = Mesh->GetMeshClothingAssets().Num();
			for(int32 ClothingAssetIndex = 0; ClothingAssetIndex < NumClothingAssets; ++ClothingAssetIndex)
			{
				UClothingAssetBase* BaseAsset = Mesh->GetMeshClothingAssets()[ClothingAssetIndex];
				if(BaseAsset && BaseAsset->GetAssetGuid() == InComponent->SelectedClothingGuidForPainting)
				{
					ClothingSimDataIndexWhenPainting = ClothingAssetIndex;

					if(UClothingAssetCommon* ConcreteAsset = Cast<UClothingAssetCommon>(BaseAsset))
					{
						if(ConcreteAsset->LodData.IsValidIndex(InComponent->SelectedClothingLodForPainting))
						{
							const FClothLODDataCommon& LodData = ConcreteAsset->LodData[InComponent->SelectedClothingLodForPainting];

							ClothingSimIndices = LodData.PhysicalMeshData.Indices;

							if(LodData.PointWeightMaps.IsValidIndex(InComponent->SelectedClothingLodMaskForPainting))
							{
								const FPointWeightMap& Mask = LodData.PointWeightMaps[InComponent->SelectedClothingLodMaskForPainting];

								ClothingVisiblePropertyValues = Mask.Values;
							}
						}
					}

					break;
				}
			}
		}
	}

	// Set material params at construction time (SetScalarParameterValue can't be called in render thread)
	if (UMaterialInstanceDynamic* const SurfaceMID = GEngine->ClothPaintMaterialInstance)
	{
		SurfaceMID->SetScalarParameterValue(FName("ClothOpacity"), InComponent->ClothMeshOpacity);
		SurfaceMID->SetScalarParameterValue(FName("BackfaceCull"), InComponent->bClothCullBackface ? 1.f : 0.f);
	}
	if (UMaterialInstanceDynamic* const WireMID = GEngine->ClothPaintMaterialWireframeInstance)
	{
		WireMID->SetScalarParameterValue(FName("ClothOpacity"), InComponent->ClothMeshOpacity);
		WireMID->SetScalarParameterValue(FName("BackfaceCull"), 1.f);
	}
}

FScopedSuspendAlternateSkinWeightPreview::FScopedSuspendAlternateSkinWeightPreview(USkeletalMesh* SkeletalMesh)
{
	SuspendedComponentArray.Empty(2);
	if (SkeletalMesh != nullptr)
	{
		// Now iterate over all skeletal mesh components and unregister them from the world, we will reregister them in the destructor
		for (TObjectIterator<UDebugSkelMeshComponent> It; It; ++It)
		{
			UDebugSkelMeshComponent* DebugSKComp = *It;
			if (DebugSKComp->GetSkeletalMeshAsset() == SkeletalMesh)
			{
				const FName ProfileName = DebugSKComp->GetCurrentSkinWeightProfileName();
				if (ProfileName != NAME_None)
				{
					DebugSKComp->ClearSkinWeightProfile();
					TTuple<UDebugSkelMeshComponent*, FName> ComponentTupple;
					ComponentTupple.Key = DebugSKComp;
					ComponentTupple.Value = ProfileName;
					SuspendedComponentArray.Add(ComponentTupple);
				}
			}
		}
	}
}

FScopedSuspendAlternateSkinWeightPreview::~FScopedSuspendAlternateSkinWeightPreview()
{
	//Put back the skin weight profile for all editor debug component
	for (const TTuple<UDebugSkelMeshComponent*, FName>& ComponentTupple : SuspendedComponentArray)
	{
		ComponentTupple.Key->SetSkinWeightProfile(ComponentTupple.Value);
	}
	SuspendedComponentArray.Empty();
}
