// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontage.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/EditorAnimBaseObj.h"
#include "Animation/AnimSequenceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAnimBaseObj)

#define LOCTEXT_NAMESPACE "SSkeletonTree"

UEditorAnimBaseObj::UEditorAnimBaseObj(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEditorAnimBaseObj::InitFromAnim(UAnimSequenceBase* AnimObjectIn, FOnAnimObjectChange OnChangeIn)
{
	AnimObject = AnimObjectIn;
	OnChange = OnChangeIn;
}

bool UEditorAnimBaseObj::ApplyChangesToMontage()
{
	return false;
}

void UEditorAnimBaseObj::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	//Handle undo from the details panel
	AnimObject->Modify();
}

void UEditorAnimBaseObj::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(OnChange.IsBound() && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (ApplyChangesToMontage())
		{
			OnChange.Execute(this, PropertyChangeRequiresRebuild(PropertyChangedEvent));
		}
	}
}

#undef LOCTEXT_NAMESPACE
