// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontage.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/EditorAnimCompositeSegment.h"
#include "Animation/AnimComposite.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAnimCompositeSegment)



UEditorAnimCompositeSegment::UEditorAnimCompositeSegment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AnimSegmentIndex = 0;
}

void UEditorAnimCompositeSegment::InitAnimSegment(int32 AnimSegmentIndexIn)
{
	AnimSegmentIndex = AnimSegmentIndexIn;
	if(UAnimComposite* Composite = Cast<UAnimComposite>(AnimObject))
	{
		if(Composite->AnimationTrack.AnimSegments.IsValidIndex(AnimSegmentIndex))
		{
			AnimSegment = Composite->AnimationTrack.AnimSegments[AnimSegmentIndex];
		}
	}
}

void UEditorAnimCompositeSegment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimReference))
	{
		AnimSegment.UpdateCachedPlayLength();			
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UEditorAnimCompositeSegment::ApplyChangesToMontage()
{
	if(UAnimComposite* Composite = Cast<UAnimComposite>(AnimObject))
	{
		if(Composite->AnimationTrack.AnimSegments.IsValidIndex(AnimSegmentIndex))
		{
			if (AnimSegment.GetAnimReference() && Composite->GetSkeleton()->IsCompatibleForEditor(AnimSegment.GetAnimReference()->GetSkeleton()))
			{
				Composite->AnimationTrack.AnimSegments[AnimSegmentIndex] = AnimSegment;
				return true;
			}
			else
			{
				AnimSegment.SetAnimReference(Composite->AnimationTrack.AnimSegments[AnimSegmentIndex].GetAnimReference(), false);
				return false;
			}
		}
	}

	return false;
}

bool UEditorAnimCompositeSegment::PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Changing the end or start time of the segment length can't change the order of the montage segments.
	// Return false so that the montage editor does not fully rebuild its UI and we can keep this UEditorAnimSegment object 
	// in the details view. (A better solution would be handling the rebuilding in a way that didn't possibly invalidate the UEditorMontageObj in the details view)
	return !(PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimEndTime) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimStartTime) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimPlayRate) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, LoopingCount));
}
