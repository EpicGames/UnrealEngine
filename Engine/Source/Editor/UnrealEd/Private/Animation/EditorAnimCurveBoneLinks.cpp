// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorAnimCurveBones.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/EditorAnimCurveBoneLinks.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAnimCurveBoneLinks)

#define LOCTEXT_NAMESPACE "UEditorAnimCurveBoneLinks"

UEditorAnimCurveBoneLinks::UEditorAnimCurveBoneLinks(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MaxLOD (0xFF)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UEditorAnimCurveBoneLinks::Initialize(const TWeakPtr<class IEditableSkeleton> InEditableSkeleton, const FSmartName& InCurveName, FOnAnimCurveBonesChange OnChangeIn)
{
	AnimCurveMetaData = nullptr;
	CurveName = InCurveName.DisplayName;
	OnChange = OnChangeIn;
	SetFlags(RF_Transactional);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEditorAnimCurveBoneLinks::Initialize(UAnimCurveMetaData* InAnimCurveMetaData, const FName& InCurveName, FOnAnimCurveBonesChange OnChangeIn)
{
	AnimCurveMetaData = InAnimCurveMetaData;
	CurveName = InCurveName;
	OnChange = OnChangeIn;
	SetFlags(RF_Transactional);
}

void UEditorAnimCurveBoneLinks::Refresh(const FName& InCurveName, const TArray<FBoneReference>& CurrentLinks, uint8 InMaxLOD)
{
	if (AnimCurveMetaData.IsValid())
	{
		// double check the name
		CurveName = InCurveName;
		ConnectedBones = CurrentLinks;
		MaxLOD = InMaxLOD;
	}
}

void UEditorAnimCurveBoneLinks::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(OnChange.IsBound())
	{
		OnChange.Execute(this);
	}
}

#undef LOCTEXT_NAMESPACE
