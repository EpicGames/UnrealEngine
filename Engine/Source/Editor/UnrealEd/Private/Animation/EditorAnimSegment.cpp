// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontage.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/EditorAnimSegment.h"
#include "Animation/AnimMontage.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorAnimSegment)



UEditorAnimSegment::UEditorAnimSegment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	AnimSlotIndex = 0;
	AnimSegmentIndex = 0;
}

void UEditorAnimSegment::InitAnimSegment(int32 AnimSlotIndexIn, int32 AnimSegmentIndexIn)
{
	AnimSlotIndex = AnimSlotIndexIn;
	AnimSegmentIndex = AnimSegmentIndexIn;
	if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimObject))
	{
		if(Montage->SlotAnimTracks.IsValidIndex(AnimSlotIndex) && Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments.IsValidIndex(AnimSegmentIndex))
		{
			AnimSegment = Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments[AnimSegmentIndex];
		}
	}
}

void UEditorAnimSegment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimReference))
	{
		AnimSegment.UpdateCachedPlayLength();			
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UEditorAnimSegment::ApplyChangesToMontage()
{
	if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimObject))
	{
		if(Montage->SlotAnimTracks.IsValidIndex(AnimSlotIndex) && Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments.IsValidIndex(AnimSegmentIndex) )
		{
			if (AnimSegment.GetAnimReference() && Montage->GetSkeleton()->IsCompatibleForEditor(AnimSegment.GetAnimReference()->GetSkeleton()))
			{
				Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments[AnimSegmentIndex] = AnimSegment;
				Montage->UpdateLinkableElements(AnimSlotIndex, AnimSegmentIndex);

				// Need to update linkable elements for segments further along.
				int32 NumSegments = Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments.Num();
				for(int32 SegmentIdx = AnimSegmentIndex + 1 ; SegmentIdx < NumSegments ; ++SegmentIdx)
				{
					Montage->UpdateLinkableElements(AnimSlotIndex, SegmentIdx);
				}

				return true;
			}
			else
			{
				AnimSegment.SetAnimReference(Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments[AnimSegmentIndex].GetAnimReference());
				return false;
			}
		}
	}

	return false;
}

bool UEditorAnimSegment::PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Changing the end or start time of the segment length can't change the order of the montage segments.
	// Return false so that the montage editor does not fully rebuild its UI and we can keep this UEditorAnimSegment object 
	// in the details view. (A better solution would be handling the rebuilding in a way that didn't possibly invalidate the UEditorMontageObj in the details view)
	return !(PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimEndTime) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimStartTime) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimPlayRate) || PropertyName == GET_MEMBER_NAME_CHECKED(FAnimSegment, LoopingCount));
}
