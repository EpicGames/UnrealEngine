// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontage.cpp: Montage classes that contains slots
=============================================================================*/ 

#include "Animation/EditorCompositeSection.h"
#include "Animation/AnimMetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorCompositeSection)


UEditorCompositeSection::UEditorCompositeSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SectionIndex = INDEX_NONE;
}

// since meta data is instanced, they have to copied manually with correct Outer
// when we move between editor composite section and montage composite section
void CopyMetaData(FCompositeSection& Source, FCompositeSection& Dest, UObject* DestOuter)
{
	const int32 TotalMetaData = Source.MetaData.Num();
	Dest.MetaData.Reset(TotalMetaData);
	Dest.MetaData.AddZeroed(TotalMetaData);

	for (int32 Idx = 0; Idx < TotalMetaData; ++Idx)
	{
		UAnimMetaData* SourceMetaData = Source.MetaData[Idx];
		if (SourceMetaData)
		{
			TSubclassOf<UAnimMetaData> SourceMetaDataClass = SourceMetaData->GetClass();
			UAnimMetaData* NewMetaData = NewObject<UAnimMetaData>(DestOuter, SourceMetaDataClass, NAME_None, RF_NoFlags, SourceMetaData);
			Dest.MetaData[Idx] = NewMetaData;
		}
	}
}

void UEditorCompositeSection::InitSection(int32 SectionIndexIn)
{
	SectionIndex = SectionIndexIn;
	if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimObject))
	{
		if(Montage->CompositeSections.IsValidIndex(SectionIndex))
		{
			CompositeSection = Montage->CompositeSections[SectionIndex];
			CopyMetaData(Montage->CompositeSections[SectionIndex], CompositeSection, this);
		}
	}
}

bool UEditorCompositeSection::ApplyChangesToMontage()
{
	if(UAnimMontage* Montage = Cast<UAnimMontage>(AnimObject))
	{
		if(Montage->CompositeSections.IsValidIndex(SectionIndex))
		{
			CompositeSection.OnChanged(CompositeSection.GetTime());
			Montage->CompositeSections[SectionIndex] = CompositeSection;
			CopyMetaData(CompositeSection, Montage->CompositeSections[SectionIndex], Montage);
			return true;
		}
	}

	return false;
}

