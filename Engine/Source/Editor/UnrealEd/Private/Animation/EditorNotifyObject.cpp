// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/EditorNotifyObject.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorNotifyObject)

UEditorNotifyObject::UEditorNotifyObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

bool UEditorNotifyObject::ApplyChangesToMontage()
{
	if(AnimObject)
	{
		for(FAnimNotifyEvent& Notify : AnimObject->Notifies)
		{
			if(Notify.Guid == Event.Guid)
			{
				Event.OnChanged(Event.GetTime());

				// If we have a duration this is a state notify
				if(Event.GetDuration() > 0.0f)
				{
					Event.EndLink.OnChanged(Event.EndLink.GetTime());

					// Always keep link methods in sync between notifies and duration links
					if(Event.GetLinkMethod() != Event.EndLink.GetLinkMethod())
					{
						Event.EndLink.ChangeLinkMethod(Event.GetLinkMethod());
					}
				}
				Notify = Event;
				break;
			}
		}
	}

	return true;
}

void UEditorNotifyObject::InitialiseNotify(const FAnimNotifyEvent& InNotify)
{
	if(AnimObject)
	{
		Event = InNotify;
		
		TryToCacheNotifyName();
	}
}

bool UEditorNotifyObject::PropertyChangeRequiresRebuild(FPropertyChangedEvent& PropertyChangedEvent)
{
	// We don't need to rebuild the track UI when we change the properties of a notify
	if(PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UEditorNotifyObject, Event) && (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNotifyEvent, Notify) || PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNotifyEvent, NotifyStateClass)))
	{
		const bool bIsNotifyNameUnchanged = !TryToCacheNotifyName();
		if (bIsNotifyNameUnchanged)
		{
			return false;
		}
	}

	return Super::PropertyChangeRequiresRebuild(PropertyChangedEvent);
}

bool UEditorNotifyObject::TryToCacheNotifyName()
{
	bool bChanged = false;

	const auto TrySetName = [&bChanged, this](FName InName)
	{
		if (InName != CachedNotifyName)
		{
			CachedNotifyName = InName;
			bChanged = true;
		}
	};
	
	if (Event.Notify)
	{
		FName NewName = FName(Event.Notify->GetNotifyName());
		TrySetName(NewName);
	}
	else if (Event.NotifyStateClass)
	{
		FName NewName = FName(Event.NotifyStateClass->GetNotifyName());
		TrySetName(NewName);
	}
	else
	{
		TrySetName(Event.NotifyName);
	}

	return bChanged;
}
