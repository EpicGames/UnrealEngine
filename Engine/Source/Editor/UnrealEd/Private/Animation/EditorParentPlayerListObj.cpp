// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/EditorParentPlayerListObj.h"
#include "AnimGraphNode_Base.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorParentPlayerListObj)

UEditorParentPlayerListObj::UEditorParentPlayerListObj(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AnimBlueprint(NULL)
{

}

FAnimParentNodeAssetOverride& UEditorParentPlayerListObj::AddOverridableNode(UAnimGraphNode_Base* Node)
{
	FAnimParentNodeAssetOverride* ExistingEntry = Overrides.FindByPredicate([Node](const FAnimParentNodeAssetOverride& Other)
	{
		return Other.ParentNodeGuid == Node->NodeGuid;
	});

	if(!ExistingEntry)
	{
		Overrides.AddZeroed();
		Overrides.Last().ParentNodeGuid = Node->NodeGuid;
		Overrides.Last().NewAsset = Node->GetAnimationAsset();

		GuidToVisualNodeMap.Add(Node->NodeGuid, Node);
	}

	return ExistingEntry ? *ExistingEntry : Overrides.Last();
}

void UEditorParentPlayerListObj::InitialiseFromBlueprint(UAnimBlueprint* Blueprint)
{
	AnimBlueprint = Blueprint;

	Overrides.Empty();

	TArray<UBlueprint*> BlueprintHierarchy;
	Blueprint->GetBlueprintHierarchyFromClass(Blueprint->GetAnimBlueprintGeneratedClass(), BlueprintHierarchy);

	// Search from 1 as 0 is this class and we're looking for it's parents
	for(int32 BPIdx = 1 ; BPIdx < BlueprintHierarchy.Num() ; ++BPIdx)
	{
		UBlueprint* CurrBP = BlueprintHierarchy[BPIdx];

		TArray<UEdGraph*> Graphs;
		CurrBP->GetAllGraphs(Graphs);

		for(UEdGraph* Graph : Graphs)
		{
			for(UEdGraphNode* Node : Graph->Nodes)
			{
				UAnimGraphNode_Base* AnimNode = Cast<UAnimGraphNode_Base>(Node);
				if(AnimNode && (AnimNode->GetAnimationAsset() || AnimNode->GetAnimationAssetClass()))
				{
					FAnimParentNodeAssetOverride& Override = AddOverridableNode(AnimNode);

					// Check the blueprint for saved overrides
					FAnimParentNodeAssetOverride* SavedOverride = AnimBlueprint->GetAssetOverrideForNode(Override.ParentNodeGuid);

					if(SavedOverride)
					{
						Override.NewAsset = SavedOverride->NewAsset;
					}
				}
			}
		}
	}
}

void UEditorParentPlayerListObj::ApplyOverrideToBlueprint(FAnimParentNodeAssetOverride& Override)
{
	FAnimParentNodeAssetOverride* ExistingOverride = AnimBlueprint->ParentAssetOverrides.FindByPredicate([Override](const FAnimParentNodeAssetOverride& Other)
	{
		return Other.ParentNodeGuid == Override.ParentNodeGuid;
	});

	UAnimBlueprintGeneratedClass* GenClass = AnimBlueprint->GetAnimBlueprintGeneratedClass();

	FScopedTransaction Transaction(NSLOCTEXT("AnimOverrideEditorObj", "ApplyToBlueprintTransaction", "Apply an override to a blueprint."));
	AnimBlueprint->Modify();

	if(ExistingOverride)
	{
		UAnimGraphNode_Base* ExistingNode = GetVisualNodeFromGuid(ExistingOverride->ParentNodeGuid);
		check(ExistingNode);

		FAnimParentNodeAssetOverride* ParentOverride = AnimBlueprint->GetAssetOverrideForNode(Override.ParentNodeGuid, true);

		if((ParentOverride && Override.NewAsset == ParentOverride->NewAsset) || Override.NewAsset == ExistingNode->GetAnimationAsset())
		{
			// If the asset is the same as a parent, or failing that same as the root; remove it
			FAnimParentNodeAssetOverride OverrideToRemove = *ExistingOverride;
			AnimBlueprint->ParentAssetOverrides.Remove(OverrideToRemove);
		}
		else
		{
			// Otherwise just update the override
			ExistingOverride->NewAsset = Override.NewAsset;
		}
	}
	else
	{
		UAnimGraphNode_Base* GraphNode = GetVisualNodeFromGuid(Override.ParentNodeGuid);
		Override.ParentNodeGuid = GraphNode->NodeGuid;
		// Can't find the override so add it to the blueprint.
		AnimBlueprint->ParentAssetOverrides.Add(Override);
	}

	AnimBlueprint->NotifyOverrideChange(Override);
	FBlueprintEditorUtils::MarkBlueprintAsModified(AnimBlueprint);

}

UAnimGraphNode_Base* UEditorParentPlayerListObj::GetVisualNodeFromGuid(FGuid InGuid) const
{
	UAnimGraphNode_Base* const* Result = GuidToVisualNodeMap.Find(InGuid);
	if(Result)
	{
		return *Result;
	}
	return NULL;
}

const UAnimBlueprint* UEditorParentPlayerListObj::GetBlueprint() const
{
	return AnimBlueprint;
}

