// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/EditorSkeletonNotifyObj.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorSkeletonNotifyObj)

UEditorSkeletonNotifyObj::UEditorSkeletonNotifyObj(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

