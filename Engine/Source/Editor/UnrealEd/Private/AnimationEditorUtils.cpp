// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimBlueprint.h"
#include "Factories/AnimBlueprintFactory.h"
#include "Factories/AnimCompositeFactory.h"
#include "Factories/AnimMontageFactory.h"
#include "Factories/BlendSpaceFactory1D.h"
#include "Factories/AimOffsetBlendSpaceFactory1D.h"
#include "Factories/BlendSpaceFactoryNew.h"
#include "Factories/AimOffsetBlendSpaceFactoryNew.h"
#include "Engine/PoseWatch.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimCompress.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "AnimationGraph.h"
#include "AnimStateNodeBase.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimNodeBase.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimationStateMachineGraph.h"
#include "K2Node_Composite.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IMainFrameModule.h"

#define LOCTEXT_NAMESPACE "AnimationEditorUtils"

/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////

///////////////////////////////////////////////////////////////////////////////
// Create Animation dialog to determine a newly created asset's name
///////////////////////////////////////////////////////////////////////////////

FText SCreateAnimationAssetDlg::LastUsedAssetPath;

void SCreateAnimationAssetDlg::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));
	AssetName = FText::FromString(FPackageName::GetLongPackageAssetName(InArgs._DefaultAssetPath.ToString()));

	const bool bPrioritizeLastPathUsed = InArgs._PrioritizeLastPathUsed;
	
	if (AssetPath.IsEmpty() || (bPrioritizeLastPathUsed && !LastUsedAssetPath.IsEmpty()))
	{
		AssetPath = LastUsedAssetPath;

		// If the AssetPath is still empty, fall back to the Game dir
		if (AssetPath.IsEmpty())
		{
			AssetPath = FText::FromString(TEXT("/Game"));
		}
	}
	else
	{
		LastUsedAssetPath = AssetPath;
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SCreateAnimationAssetDlg::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SCreateAnimationAssetDlg_Title", "Create a New Animation Asset"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectPath", "Select Path to create animation"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]

					+ SVerticalBox::Slot()
						.FillHeight(1)
						.Padding(3)
						[
							ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SSeparator)
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(3)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 0, 10, 0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("AnimationName", "Animation Name"))
							]

							+ SHorizontalBox::Slot()
								[
									SNew(SEditableTextBox)
									.Text(AssetName)
									.OnTextCommitted(this, &SCreateAnimationAssetDlg::OnNameChange)
									.MinDesiredWidth(250)
								]
						]

				]
			]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(5)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text(LOCTEXT("OK", "OK"))
						.OnClicked(this, &SCreateAnimationAssetDlg::OnButtonClick, EAppReturnType::Ok)
					]
					+ SUniformGridPanel::Slot(1, 0)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
							.Text(LOCTEXT("Cancel", "Cancel"))
							.OnClicked(this, &SCreateAnimationAssetDlg::OnButtonClick, EAppReturnType::Cancel)
						]
				]
		]);
}

void SCreateAnimationAssetDlg::OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo)
{
	AssetName = NewName;
}

void SCreateAnimationAssetDlg::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
	LastUsedAssetPath = AssetPath;
}

FReply SCreateAnimationAssetDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	if (ButtonID != EAppReturnType::Cancel)
	{
		if (!ValidatePackage())
		{
			// reject the request
			return FReply::Handled();
		}
	}

	RequestDestroyWindow();

	return FReply::Handled();
}

/** Ensures supplied package name information is valid */
bool SCreateAnimationAssetDlg::ValidatePackage()
{
	FText Reason;
	FString FullPath = GetFullAssetPath();

	if (!FPackageName::IsValidLongPackageName(FullPath, false, &Reason)
		|| !FName(*AssetName.ToString()).IsValidObjectName(Reason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason);
		return false;
	}

	return true;
}

EAppReturnType::Type SCreateAnimationAssetDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SCreateAnimationAssetDlg::GetAssetPath()
{
	return AssetPath.ToString();
}

FString SCreateAnimationAssetDlg::GetAssetName()
{
	return AssetName.ToString();
}

FString SCreateAnimationAssetDlg::GetFullAssetPath()
{
	return AssetPath.ToString() + "/" + AssetName.ToString();
}

/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////

/////////////////////////////////////////////////////
// Dialog to prompt user to select an animation compression settings asset.
/////////////////////////////////////////////////////

SAnimationCompressionSelectionDialog::SAnimationCompressionSelectionDialog()
	: bValidAssetChosen(false)
{}

SAnimationCompressionSelectionDialog::~SAnimationCompressionSelectionDialog()
{}

void SAnimationCompressionSelectionDialog::SetOnAssetSelected(const FOnAssetSelected& InHandler)
{
	OnAssetSelectedHandler = InHandler;
}

void SAnimationCompressionSelectionDialog::DoSelectAsset(const FAssetData& SelectedAsset)
{
	bValidAssetChosen = true;
	OnAssetSelectedHandler.ExecuteIfBound(SelectedAsset);

	CloseDialog();
}

FReply SAnimationCompressionSelectionDialog::OnConfirmClicked()
{
	TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	if (SelectedAssets.Num() > 0)
	{
		DoSelectAsset(SelectedAssets[0]);
	}

	return FReply::Handled();
}

FReply SAnimationCompressionSelectionDialog::OnCancelClicked()
{
	CloseDialog();

	return FReply::Handled();
}

void SAnimationCompressionSelectionDialog::CloseDialog()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

void SAnimationCompressionSelectionDialog::OnAssetSelected(const FAssetData& AssetData)
{
	CurrentlySelectedAssets = GetCurrentSelectionDelegate.Execute();
}

void SAnimationCompressionSelectionDialog::OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType)
{
	const bool bCorrectActivationMethod = ActivationType == EAssetTypeActivationMethod::DoubleClicked || ActivationType == EAssetTypeActivationMethod::Opened;
	if (SelectedAssets.Num() > 0 && bCorrectActivationMethod)
	{
		DoSelectAsset(SelectedAssets[0]);
	}
}

bool SAnimationCompressionSelectionDialog::IsConfirmButtonEnabled() const
{
	return CurrentlySelectedAssets.Num() > 0;
}

void SAnimationCompressionSelectionDialog::Construct(const FArguments& InArgs, const FAnimationCompressionSelectionDialogConfig& InConfig)
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Push(UAnimBoneCompressionSettings::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAnimationCompressionSelectionDialog::OnAssetSelected);
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SAnimationCompressionSelectionDialog::OnAssetsActivated);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.SaveSettingsName = TEXT("AnimationCompressionSelectionDialog");
	AssetPickerConfig.bCanShowFolders = false;
	AssetPickerConfig.bCanShowDevelopersFolder = true;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.InitialAssetSelection = InConfig.DefaultSelectedAsset != nullptr ? InConfig.DefaultSelectedAsset : FAnimationUtils::GetDefaultAnimationBoneCompressionSettings();
	AssetPickerConfig.bForceShowEngineContent = true;

	if (AssetPickerConfig.InitialAssetSelection.IsValid())
	{
		CurrentlySelectedAssets.Add(AssetPickerConfig.InitialAssetSelection);
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetPicker = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

	// The root widget in this dialog.
	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	// Asset view
	MainVerticalBox->AddSlot()
		.FillHeight(1)
		.Padding(0, 0, 0, 4)
		[
			SNew(SSplitter)

			+ SSplitter::Slot()
			.Value(0.75f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					AssetPicker.ToSharedRef()
				]
			]
		];

	// Buttons and asset name
	TSharedRef<SHorizontalBox> ButtonsAndNameBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(4, 3)
		[
			SNew(SButton)
			.Text(LOCTEXT("AnimationCompressionSelectionDialogSelectButton", "Select"))
			.ContentPadding(FMargin(8, 2, 8, 2))
			.IsEnabled(this, &SAnimationCompressionSelectionDialog::IsConfirmButtonEnabled)
			.OnClicked(this, &SAnimationCompressionSelectionDialog::OnConfirmClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		.Padding(4, 3)
		[
			SNew(SButton)
			.ContentPadding(FMargin(8, 2, 8, 2))
			.Text(LOCTEXT("AnimationCompressionSelectionDialogCancelButton", "Cancel"))
			.OnClicked(this, &SAnimationCompressionSelectionDialog::OnCancelClicked)
		];

	MainVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(0)
		[
			ButtonsAndNameBox
		];

	ChildSlot
		[
			MainVerticalBox
		];
}

/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////
/////////////////////////////////////////////////////

/////////////////////////////////////////////////////
// Animation editor utility functions
/////////////////////////////////////////////////////

namespace AnimationEditorUtils
{
	FAssetData CreateModalAnimationCompressionSelectionDialog(const FAnimationCompressionSelectionDialogConfig& InConfig)
	{
		struct FModalResult
		{
			void OnAssetSelected(const FAssetData& SelectedAsset)
			{
				SavedResult = SelectedAsset;
			}

			FAssetData SavedResult;
		};

		FModalResult ModalWindowResult;
		auto OnAssetSelectedDelegate = SAnimationCompressionSelectionDialog::FOnAssetSelected::CreateRaw(&ModalWindowResult, &FModalResult::OnAssetSelected);

		TSharedRef<SAnimationCompressionSelectionDialog> Dialog = SNew(SAnimationCompressionSelectionDialog, InConfig);
		Dialog->SetOnAssetSelected(OnAssetSelectedDelegate);

		const FVector2D DefaultWindowSize(400.0f, 500.0f);
		const FVector2D WindowSize = InConfig.WindowSizeOverride.IsZero() ? DefaultWindowSize : InConfig.WindowSizeOverride;
		const FText WindowTitle = InConfig.DialogTitleOverride.IsEmpty() ? LOCTEXT("GenericAnimationCompressionSelectionDialogWindowHeader", "Select compression settings") : InConfig.DialogTitleOverride;

		TSharedRef<SWindow> DialogWindow =
			SNew(SWindow)
			.Title(WindowTitle)
			.ClientSize(WindowSize);

		DialogWindow->SetContent(Dialog);

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if (MainFrameParentWindow.IsValid())
		{
			FSlateApplication::Get().AddModalWindow(DialogWindow, MainFrameParentWindow.ToSharedRef());
		}

		return ModalWindowResult.SavedResult;
	}

	/** Creates a unique package and asset name taking the form InBasePackageName+InSuffix */
	void CreateUniqueAssetName(const FString& InBasePackageName, const FString& InSuffix, FString& OutPackageName, FString& OutAssetName) 
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(InBasePackageName, InSuffix, OutPackageName, OutAssetName);
	}

	void CreateAnimationAssets(const TArray<TSoftObjectPtr<UObject>>& SkeletonsOrSkeletalMeshes, TSubclassOf<UAnimationAsset> AssetClass, const FString& InPrefix, FAnimAssetCreated AssetCreated, UObject* NameBaseObject /*= nullptr*/, bool bDoNotShowNameDialog /*= false*/, bool bAllowReplaceExisting /*= false*/)
	{
		TArray<UObject*> ObjectsToSync;
		for (auto SkelIt = SkeletonsOrSkeletalMeshes.CreateConstIterator(); SkelIt; ++SkelIt)
		{
			UObject* SkeletonOrSkeletalMeshObject = SkelIt->LoadSynchronous();
			
			USkeletalMesh* SkeletalMesh = nullptr;
			USkeleton* Skeleton = Cast<USkeleton>(SkeletonOrSkeletalMeshObject);
			if (Skeleton == nullptr)
			{
				SkeletalMesh = Cast<USkeletalMesh>(SkeletonOrSkeletalMeshObject);
				if (SkeletalMesh)
				{
					Skeleton = SkeletalMesh->GetSkeleton();				
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Invalid skeleton or skeletal mesh passed to CreateAnimationAssets. No asset will be generated.");
				}
			}

			if (Skeleton)
			{
				FString Name;
				FString PackageName;
				FString AssetPath = (NameBaseObject)? NameBaseObject->GetOutermost()->GetName(): Skeleton->GetOutermost()->GetName();
				// Determine an appropriate name
				CreateUniqueAssetName(AssetPath, InPrefix, PackageName, Name);

				if (bDoNotShowNameDialog == false)
				{
					// set the unique asset as a default name
					TSharedRef<SCreateAnimationAssetDlg> NewAnimDlg =
						SNew(SCreateAnimationAssetDlg)
						.DefaultAssetPath(FText::FromString(PackageName))
						.PrioritizeLastPathUsed(true);

					// show a dialog to determine a new asset name
					if (NewAnimDlg->ShowModal() == EAppReturnType::Cancel)
					{
						return;
					}

					PackageName = NewAnimDlg->GetFullAssetPath();
					Name = NewAnimDlg->GetAssetName();
				}

				// Create the asset, and assign its skeleton
				FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

				UAnimationAsset* NewAsset = nullptr;

				if (bAllowReplaceExisting)
				{
					UPackage* ExistingPackage = LoadPackage(nullptr, *PackageName, LOAD_None);
					UObject* ExistingObject = StaticFindObject(AssetClass.Get(), ExistingPackage, *Name);
					if (ExistingObject)
					{
						EAppReturnType::Type UserResponse = FMessageDialog::Open(
							EAppMsgType::YesNo,
							FText::Format(LOCTEXT("CreateAnimationAssetsAlreadyExists", "Do you want to replace the existing asset?\n\nAn asset already exists at the import location: {0}"), FText::FromString(PackageName)));

						if (UserResponse == EAppReturnType::Yes)
						{
							NewAsset = Cast<UAnimationAsset>(ExistingObject);
						}
						else
						{
							return;
						}
					}
				}
				
				if (!NewAsset)
				{
					NewAsset = Cast<UAnimationAsset>(AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), AssetClass, nullptr));
				}
				
				if(NewAsset)
				{
					NewAsset->SetSkeleton(Skeleton);

					if (UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(NewAsset))
					{
						SequenceBase->GetController().InitializeModel();
					}
					
					if (SkeletalMesh)
					{
						NewAsset->SetPreviewMesh(SkeletalMesh);
					}
					NewAsset->MarkPackageDirty();

					ObjectsToSync.Add(NewAsset);
				}
			}
		}

		if (AssetCreated.IsBound())
		{
			if (!AssetCreated.Execute(ObjectsToSync))
			{
				// Rename the objects we created out of the way
				for (UObject* ObjectToDelete : ObjectsToSync)
				{
					// Notify the asset registry
					FAssetRegistryModule::AssetDeleted(ObjectToDelete);
					ObjectToDelete->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
				}
			}
		}
	}

	void CreateNewAnimBlueprint(TArray<TSoftObjectPtr<UObject>> SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser)
	{
		TArray<TWeakObjectPtr<UObject>> SkeletonsOrSkeletalMeshesLoaded;
		for (TSoftObjectPtr<UObject>& SkeletonsOrSkeletalMesh : SkeletonsOrSkeletalMeshes)
		{
			if (UObject* SkeletonOrSkeletalMeshObject = SkeletonsOrSkeletalMesh.LoadSynchronous())
			{
				SkeletonsOrSkeletalMeshesLoaded.Add(SkeletonOrSkeletalMeshObject);
			}
		}

		CreateNewAnimBlueprint(SkeletonsOrSkeletalMeshesLoaded, AssetCreated, bInContentBrowser);
	}
	
	void CreateNewAnimBlueprint(TArray<TWeakObjectPtr<UObject>> SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser)
    {
		const FString DefaultSuffix = TEXT("_AnimBlueprint");

		if (SkeletonsOrSkeletalMeshes.Num() == 1)
		{
			UObject* SkeletonOrSkeletalMeshObject = SkeletonsOrSkeletalMeshes[0].Get();
			
			USkeletalMesh* SkeletalMesh = nullptr;
			USkeleton* Skeleton = Cast<USkeleton>(SkeletonOrSkeletalMeshObject);
			if (Skeleton == nullptr)
			{
				SkeletalMesh = CastChecked<USkeletalMesh>(SkeletonOrSkeletalMeshObject);
				Skeleton = SkeletalMesh->GetSkeleton();
			}

			if (Skeleton)
			{
				// Determine an appropriate name for inline-rename
				FString Name;
				FString PackageName;
				CreateUniqueAssetName(Skeleton->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

				UAnimBlueprintFactory* Factory = NewObject<UAnimBlueprintFactory>();
				Factory->TargetSkeleton = Skeleton;
				Factory->PreviewSkeletalMesh = SkeletalMesh;

				if (bInContentBrowser)
				{
					FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackageName), UAnimBlueprint::StaticClass(), Factory);
				}
				else
				{
					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UAnimBlueprint* NewAsset = CastChecked<UAnimBlueprint>(AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UAnimBlueprint::StaticClass(), Factory));

					if (NewAsset && AssetCreated.IsBound())
					{
						TArray<UObject*> NewObjects;
						NewObjects.Add(NewAsset);
						if (!AssetCreated.Execute(NewObjects))
						{
							//Destroy the assets we just create
							for (UObject* ObjectToDelete : NewObjects)
							{
								ObjectToDelete->ClearFlags(RF_Standalone | RF_Public);
								ObjectToDelete->RemoveFromRoot();
								ObjectToDelete->MarkAsGarbage();
							}
							CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
						}
					}
				}
			}
		}
		else
		{
			TArray<UObject*> AssetsToSync;
			for (auto ObjIt = SkeletonsOrSkeletalMeshes.CreateConstIterator(); ObjIt; ++ObjIt)
			{
				USkeletalMesh* SkeletalMesh = nullptr;
				USkeleton* Skeleton = Cast<USkeleton>(ObjIt->Get());
				if (Skeleton == nullptr)
				{
					SkeletalMesh = CastChecked<USkeletalMesh>(ObjIt->Get());
					Skeleton = SkeletalMesh->GetSkeleton();
				}

				if(Skeleton)
				{
					// Determine an appropriate name
					FString Name;
					FString PackageName;
					CreateUniqueAssetName(Skeleton->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

					// Create the anim blueprint factory used to generate the asset
					UAnimBlueprintFactory* Factory = NewObject<UAnimBlueprintFactory>();
					Factory->TargetSkeleton = Skeleton;
					Factory->PreviewSkeletalMesh = SkeletalMesh;

					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UAnimBlueprint::StaticClass(), Factory);

					if (NewAsset)
					{
						AssetsToSync.Add(NewAsset);
					}
				}
			}

			if (AssetCreated.IsBound())
			{
				if (!AssetCreated.Execute(AssetsToSync))
				{
					//Destroy the assets we just create
					for (UObject* ObjectToDelete : AssetsToSync)
					{
						ObjectToDelete->ClearFlags(RF_Standalone | RF_Public);
						ObjectToDelete->RemoveFromRoot();
						ObjectToDelete->MarkAsGarbage();
					}
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}
			}
		}
	}

	bool CanCreateAssetOfType(const UClass* InClass)
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		return AssetTools.IsAssetClassSupported(InClass);
	}

	void FillCreateAssetMenu(FMenuBuilder& MenuBuilder, const TArray<TSoftObjectPtr<UObject>>& SkeletonsOrSkeletalMeshes, FAnimAssetCreated AssetCreated, bool bInContentBrowser)
	{
		const bool bAllowReplaceExisting = false;

		MenuBuilder.BeginSection("CreateAnimAssets", LOCTEXT("CreateAnimAssetsMenuHeading", "Anim Assets"));
		{
			if(CanCreateAssetOfType(UAnimBlueprint::StaticClass()))
			{
				// only allow for content browser until we support multi assets so we can open new persona with this BP
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Skeleton_NewAnimBlueprint", "Anim Blueprint"),
					LOCTEXT("Skeleton_NewAnimBlueprintTooltip", "Creates an Anim Blueprint using the selected skeleton."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimBlueprint"),
					FUIAction(
						FExecuteAction::CreateStatic(&CreateNewAnimBlueprint, SkeletonsOrSkeletalMeshes, AssetCreated, bInContentBrowser),
						FCanExecuteAction()
						)
					);
			}

			if(CanCreateAssetOfType(UAnimComposite::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Skeleton_NewAnimComposite", "Anim Composite"),
					LOCTEXT("Skeleton_NewAnimCompositeTooltip", "Creates an AnimComposite using the selected skeleton."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimComposite"),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UAnimCompositeFactory, UAnimComposite>, SkeletonsOrSkeletalMeshes, FString("_Composite"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}

			if(CanCreateAssetOfType(UAnimMontage::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Skeleton_NewAnimMontage", "Anim Montage"),
					LOCTEXT("Skeleton_NewAnimMontageTooltip", "Creates an AnimMontage using the selected skeleton."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimMontage"),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UAnimMontageFactory, UAnimMontage>, SkeletonsOrSkeletalMeshes, FString("_Montage"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CreateBlendSpace", LOCTEXT("CreateBlendSpaceMenuHeading", "Blend Spaces"));
		{
			if (CanCreateAssetOfType(UBlendSpace::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SkeletalMesh_New2DBlendspace", "Blend Space"),
					LOCTEXT("SkeletalMesh_New2DBlendspaceTooltip", "Creates a Blend Space using the selected skeleton."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlendSpace"),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UBlendSpaceFactoryNew, UBlendSpace>, SkeletonsOrSkeletalMeshes, FString("_BlendSpace"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}

			if (CanCreateAssetOfType(UBlendSpace1D::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SkeletalMesh_New1DBlendspace", "Blend Space 1D"),
					LOCTEXT("SkeletalMesh_New1DBlendspaceTooltip", "Creates a 1D Blend Space using the selected skeleton."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlendSpace1D"),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UBlendSpaceFactory1D, UBlendSpace1D>, SkeletonsOrSkeletalMeshes, FString("_BlendSpace1D"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("CreateAimOffset", LOCTEXT("CreateAimOffsetMenuHeading", "Aim Offsets"));
		{
			if (CanCreateAssetOfType(UAimOffsetBlendSpace::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SkeletalMesh_New2DAimOffset", "Aim Offset"),
					LOCTEXT("SkeletalMesh_New2DAimOffsetTooltip", "Creates a Aim Offset blendspace using the selected skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UAimOffsetBlendSpaceFactoryNew, UAimOffsetBlendSpace>, SkeletonsOrSkeletalMeshes, FString("_AimOffset2D"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}

			if (CanCreateAssetOfType(UAimOffsetBlendSpace1D::StaticClass()))
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("SkeletalMesh_New1DAimOffset", "Aim Offset 1D"),
					LOCTEXT("SkeletalMesh_New1DAimOffsetTooltip", "Creates a 1D Aim Offset blendspace using the selected skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateStatic(&ExecuteNewAnimAsset<UAimOffsetBlendSpaceFactory1D, UAimOffsetBlendSpace1D>, SkeletonsOrSkeletalMeshes, FString("_AimOffset1D"), AssetCreated, bInContentBrowser, bAllowReplaceExisting),
						FCanExecuteAction()
						)
					);
			}
		}
		MenuBuilder.EndSection();
	}

	bool ApplyCompressionAlgorithm(TArray<UAnimSequence*>& AnimSequencePtrs, UAnimBoneCompressionSettings* OverrideSettings)
	{
		const bool bProceed = (AnimSequencePtrs.Num() > 1)? EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo,
			FText::Format(NSLOCTEXT("UnrealEd", "AboutToCompressAnimations_F", "About to compress {0} animations.  Proceed?"), FText::AsNumber(AnimSequencePtrs.Num()))) : true;
		if(bProceed)
		{
			GWarn->BeginSlowTask(LOCTEXT("AnimCompressing", "Compressing"), true);

			{
				UE::Anim::Compression::FAnimationCompressionMemorySummaryScope Scope;
				for (UAnimSequence* AnimSeq : AnimSequencePtrs)
				{
					if (OverrideSettings != nullptr)
					{
						AnimSeq->BoneCompressionSettings = OverrideSettings;
					}

					// Clear CompressCommandletVersion so we can recompress these animations later.
					AnimSeq->CompressCommandletVersion = 0;
					AnimSeq->ClearAllCachedCookedPlatformData();
					AnimSeq->CacheDerivedDataForCurrentPlatform();
				}
			}

			GWarn->EndSlowTask();

			return true;
		}

		return false;
	}

	bool IsAnimGraph(UEdGraph* Graph)
	{
		return Cast<UAnimationGraph>(Graph) != nullptr;
	}

	void RegenerateSubGraphArrays(UAnimBlueprint* Blueprint)
	{
		// The anim graph should be the first function graph on the blueprint
		if(Blueprint->FunctionGraphs.Num() > 0)
		{
			if(UAnimationGraph* AnimGraph = Cast<UAnimationGraph>(Blueprint->FunctionGraphs[0]))
			{
				RegenerateGraphSubGraphs(Blueprint, AnimGraph);
			}
		}
	}

	void RegenerateGraphSubGraphs(UAnimBlueprint* OwningBlueprint, UEdGraph* GraphToFix)
	{
		TArray<UEdGraph*> ChildGraphs;
		FindChildGraphsFromNodes(GraphToFix, ChildGraphs);

		for(UEdGraph* Child : ChildGraphs)
		{
			RegenerateGraphSubGraphs(OwningBlueprint, Child);
		}

		if(ChildGraphs != GraphToFix->SubGraphs)
		{
			UE_LOGF(LogAnimation, Log, "Fixed missing or duplicated graph entries in SubGraph array for graph %ls in AnimBP %ls", *GraphToFix->GetName(), *OwningBlueprint->GetName());
			GraphToFix->SubGraphs = ChildGraphs;
		}
	}

	void RemoveDuplicateSubGraphs(UEdGraph* GraphToClean)
	{
		TArray<UEdGraph*> NewSubGraphArray;

		for(UEdGraph* SubGraph : GraphToClean->SubGraphs)
		{
			NewSubGraphArray.AddUnique(SubGraph);
		}

		if(NewSubGraphArray.Num() != GraphToClean->SubGraphs.Num())
		{
			GraphToClean->SubGraphs = NewSubGraphArray;
		}
	}

	void FindChildGraphsFromNodes(UEdGraph* GraphToSearch, TArray<UEdGraph*>& ChildGraphs)
	{
		for(UEdGraphNode* CurrentNode : GraphToSearch->Nodes)
		{
			if(UAnimGraphNode_StateMachineBase* StateMachine = Cast<UAnimGraphNode_StateMachineBase>(CurrentNode))
			{
				ChildGraphs.AddUnique(StateMachine->EditorStateMachineGraph);
			}
			else if(UAnimStateNodeBase* StateNode = Cast<UAnimStateNodeBase>(CurrentNode))
			{
				UEdGraph* BoundGraph = StateNode->GetBoundGraph();
				if (BoundGraph == nullptr)
				{
					continue;
				}

				ChildGraphs.AddUnique(BoundGraph);

				if(UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(StateNode))
				{
					if(TransitionNode->CustomTransitionGraph)
					{
						ChildGraphs.AddUnique(TransitionNode->CustomTransitionGraph);
					}
				}
			}
			else if(UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(CurrentNode))
			{
				ChildGraphs.AddUnique(CompositeNode->BoundGraph);
			}
		}
	}

	static FOnPoseWatchesChanged OnPoseWatchesChangedDelegate;

	void SetPoseWatch(UPoseWatch* PoseWatch, UAnimBlueprint* AnimBlueprintIfKnown)
	{
#if WITH_EDITORONLY_DATA
			if (UAnimGraphNode_Base* TargetNode = Cast<UAnimGraphNode_Base>(PoseWatch->Node))
			{
				UAnimBlueprint* AnimBlueprint = AnimBlueprintIfKnown ? AnimBlueprintIfKnown : Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(TargetNode));
				if ((AnimBlueprint != nullptr) && (AnimBlueprint->GeneratedClass != nullptr))
				{
					if (UAnimBlueprintGeneratedClass* AnimBPGenClass = Cast<UAnimBlueprintGeneratedClass>(*AnimBlueprint->GeneratedClass))
					{
						// Find the insertion point from the debugging data
						const int32 LinkID = AnimBPGenClass->GetLinkIDForNode<FAnimNode_Base>(TargetNode);

						for (const TObjectPtr<UPoseWatchElement>& PoseWatchElement : PoseWatch->GetElements())
						{
							if (UPoseWatchPoseElement* PoseWatchPoseElement = Cast<UPoseWatchPoseElement>(PoseWatchElement.Get()))
							{
								AnimBPGenClass->GetAnimBlueprintDebugData().AddPoseWatch(LinkID, PoseWatchPoseElement);
							}
						}

						OnPoseWatchesChangedDelegate.Broadcast(AnimBlueprint, TargetNode);
					}
				}
			}
#endif
	}

	void RemovePoseWatchesFromGraph(UAnimBlueprint* AnimBlueprint, class UEdGraph* Graph)
	{
#if WITH_EDITORONLY_DATA
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			RemovePoseWatchFromNode(Node, AnimBlueprint);
		}
#endif
	}

	UPoseWatch* FindPoseWatchForNode(const UEdGraphNode* Node, UAnimBlueprint* AnimBlueprintIfKnown)
	{
#if WITH_EDITORONLY_DATA
		UAnimBlueprint* AnimBlueprint = AnimBlueprintIfKnown ? AnimBlueprintIfKnown : Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(Node));

		if(AnimBlueprint)
		{
			// iterate backwards so we can remove invalid pose watches as we go
			for (int32 Index = AnimBlueprint->PoseWatches.Num() - 1; Index >= 0; --Index)
			{
				UPoseWatch* PoseWatch = AnimBlueprint->PoseWatches[Index];
				if (PoseWatch == nullptr || PoseWatch->Node == nullptr)
				{
					AnimBlueprint->PoseWatches.RemoveAtSwap(Index);
					continue;
				}

				// Return this pose watch if the node location matches the given node
				if (PoseWatch->Node == Node)
				{
					return PoseWatch;
				}
			}
		}

		return nullptr;
#endif
	}

	UPoseWatch* MakePoseWatchForNode(UAnimBlueprint* AnimBlueprint, UEdGraphNode* Node)
	{
#if WITH_EDITORONLY_DATA
		check(CastChecked<UAnimGraphNode_Base>(Node)->IsPoseWatchable());
		UPoseWatch* NewPoseWatch = NewObject<UPoseWatch>(AnimBlueprint);
		NewPoseWatch->Node = Node;
		NewPoseWatch->SetUniqueDefaultLabel();
		NewPoseWatch->AddElement<UPoseWatchPoseElement>(LOCTEXT("PoseWatchElementLabel_PoseWatch", "Pose Watch"), TEXT("AnimGraph.PoseWatch.Icon"));
		AnimBlueprint->PoseWatches.Add(NewPoseWatch);
		SetPoseWatch(NewPoseWatch, AnimBlueprint);
		return NewPoseWatch;
#else
		return nullptr;
#endif
	}

	void RemovePoseWatch(UPoseWatch* PoseWatch, UAnimBlueprint* AnimBlueprintIfKnown)
	{
#if WITH_EDITORONLY_DATA
		if (UAnimGraphNode_Base* TargetNode = Cast<UAnimGraphNode_Base>(PoseWatch->Node))
		{
			UAnimBlueprint* AnimBlueprint = AnimBlueprintIfKnown ? AnimBlueprintIfKnown : Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(TargetNode));
			if (AnimBlueprint)
			{
				AnimBlueprint->PoseWatches.Remove(PoseWatch);
				if (UAnimBlueprintGeneratedClass* AnimBPGenClass = AnimBlueprint->GetAnimBlueprintGeneratedClass())
				{
					int32 LinkID = AnimBPGenClass->GetLinkIDForNode<FAnimNode_Base>(Cast<UAnimGraphNode_Base>(PoseWatch->Node));
					AnimBPGenClass->GetAnimBlueprintDebugData().RemovePoseWatch(LinkID);
					OnPoseWatchesChangedDelegate.Broadcast(AnimBlueprint, TargetNode);
				}
			}
		}
#endif
	}

	void RemovePoseWatchFromNode(UEdGraphNode* Node, UAnimBlueprint* AnimBlueprint)
	{
#if WITH_EDITORONLY_DATA
		if (AnimBlueprint)
		{
			for (UPoseWatch* SomePoseWatch : AnimBlueprint->PoseWatches)
			{
				if (SomePoseWatch->Node == Node)
				{
					if (UAnimBlueprintGeneratedClass* AnimBPGenClass = AnimBlueprint->GetAnimBlueprintGeneratedClass())
					{
						int32 LinkID = AnimBPGenClass->GetLinkIDForNode<FAnimNode_Base>(Cast<UAnimGraphNode_Base>(SomePoseWatch->Node));
						AnimBPGenClass->GetAnimBlueprintDebugData().RemovePoseWatch(LinkID);

						OnPoseWatchesChangedDelegate.Broadcast(AnimBlueprint, Node);
					}

					SomePoseWatch->OnRemoved();
					return;
				}
			}
		}
#endif
	}

	FOnPoseWatchesChanged& OnPoseWatchesChanged()
	{
		return OnPoseWatchesChangedDelegate;
	}

	int32 GetPoseWatchNodeLinkID(UPoseWatch* PoseWatch, UAnimBlueprintGeneratedClass*& OutAnimBPGenClass)
	{
#if WITH_EDITORONLY_DATA
		if (UAnimGraphNode_Base* TargetNode = Cast<UAnimGraphNode_Base>(PoseWatch->Node))
		{
			UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(TargetNode));

			if ((AnimBlueprint != nullptr) && (AnimBlueprint->GeneratedClass != nullptr))
			{
				if (UAnimBlueprintGeneratedClass* AnimBPGenClass = Cast<UAnimBlueprintGeneratedClass>(*AnimBlueprint->GeneratedClass))
				{
					// Find the insertion point from the debugging data
					OutAnimBPGenClass = AnimBPGenClass;
					return AnimBPGenClass->GetLinkIDForNode<FAnimNode_Base>(TargetNode);
				}
			}
		}
#endif
		return INDEX_NONE;
	}

	void SetupDebugLinkedAnimInstances(UAnimBlueprint* InAnimBlueprint, UObject* InRootObjectBeingDebugged)
	{
		check(IsInGameThread());
		
		static bool bSettingDebugInstances = false;

		if(!bSettingDebugInstances)
		{
			TGuardValue<bool> GuardValue(bSettingDebugInstances, true);
			if(InRootObjectBeingDebugged)
			{
				if(const USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(InRootObjectBeingDebugged->GetOuter()))
				{
					// See if we have any linked instances
					const TArray<UAnimInstance*> LinkedInstances = Component->GetLinkedAnimInstances();
					for(UAnimInstance* LinkedInstance : LinkedInstances)
					{
						if(UAnimBlueprint* LinkedAnimBlueprint = Cast<UAnimBlueprint>(LinkedInstance->GetClass()->ClassGeneratedBy))
						{
							LinkedAnimBlueprint->SetObjectBeingDebugged(LinkedInstance);
						}
					}
				}
			}
			else if(UObject* OldDebuggedObject = InAnimBlueprint->GetObjectBeingDebugged())
			{
				if(const USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(OldDebuggedObject->GetOuter()))
				{
					// See if we have any linked instances
					const TArray<UAnimInstance*> LinkedInstances = Component->GetLinkedAnimInstances();
					for(UAnimInstance* LinkedInstance : LinkedInstances)
					{
						if(UAnimBlueprint* LinkedAnimBlueprint = Cast<UAnimBlueprint>(LinkedInstance->GetClass()->ClassGeneratedBy))
						{
							LinkedAnimBlueprint->SetObjectBeingDebugged(nullptr);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
