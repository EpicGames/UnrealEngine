// Copyright Epic Games, Inc. All Rights Reserved.

// UE_DEPRECATED(5.7, "Apex clothing is no longer supported, this implementation will be removed")

#if 0  // Workaround the first inclusion rule to help the deprecation
#include "ApexClothingUtils.h"
#endif

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ClothingAsset.h"

#define LOCTEXT_NAMESPACE "ApexClothingUtils"

namespace ApexClothingUtils
{

//enforces a call of "OnRegister" to update vertex factories
void ReregisterSkelMeshComponents(USkeletalMesh* SkelMesh)
{
	for( TObjectIterator<USkeletalMeshComponent> It; It; ++It )
	{
		USkeletalMeshComponent* MeshComponent = *It;
		if( MeshComponent && 
			!MeshComponent->IsTemplate() &&
			MeshComponent->GetSkeletalMeshAsset() == SkelMesh )
		{
			MeshComponent->ReregisterComponent();
		}
	}
}

void RefreshSkelMeshComponents(USkeletalMesh* SkelMesh)
{
	for( TObjectIterator<USkeletalMeshComponent> It; It; ++It )
	{
		USkeletalMeshComponent* MeshComponent = *It;
		if( MeshComponent && 
			!MeshComponent->IsTemplate() &&
			MeshComponent->GetSkeletalMeshAsset() == SkelMesh )
		{
			MeshComponent->RecreateRenderState_Concurrent();
		}
	}
}

void RestoreAllClothingSections(USkeletalMesh* SkelMesh, uint32 LODIndex, uint32 AssetIndex)
{
	if(FSkeletalMeshModel* Resource = SkelMesh->GetImportedModel())
	{
		for(FSkeletalMeshLODModel& LodModel : Resource->LODModels)
		{
			for(FSkelMeshSection& Section : LodModel.Sections)
			{
				if(Section.HasClothingData())
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ClothingAssetUtils::ClearSectionClothingData(Section);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					if (FSkelMeshSourceSectionUserData* UserSectionData = LodModel.UserSectionsData.Find(Section.OriginalDataSectionIndex))
					{
						UserSectionData->CorrespondClothAssetIndex = INDEX_NONE;
						UserSectionData->ClothingData.AssetLodIndex = INDEX_NONE;
						UserSectionData->ClothingData.AssetGuid = FGuid();
					}
				}
			}
		}
	}
}

void RemoveAssetFromSkeletalMesh(USkeletalMesh* SkelMesh, uint32 AssetIndex, bool bReleaseAsset, bool bRecreateSkelMeshComponent)
{
	FSkeletalMeshModel* ImportedResource= SkelMesh->GetImportedModel();
	int32 NumLODs = ImportedResource->LODModels.Num();

	for(int32 LODIdx=0; LODIdx < NumLODs; LODIdx++)
	{
		RestoreAllClothingSections(SkelMesh, LODIdx, AssetIndex);
	}

	SkelMesh->GetClothingAssets_DEPRECATED().RemoveAt(AssetIndex);	//have to remove the asset from the array so that new actors are not created for asset pending deleting

	SkelMesh->PostEditChange(); // update derived data

	ReregisterSkelMeshComponents(SkelMesh);

	if(bRecreateSkelMeshComponent)
	{
		// Refresh skeletal mesh components
		RefreshSkelMeshComponents(SkelMesh);
	}
}

} // namespace ApexClothingUtils

#undef LOCTEXT_NAMESPACE
