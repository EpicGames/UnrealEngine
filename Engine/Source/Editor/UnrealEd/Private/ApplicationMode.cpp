// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Docking/LayoutService.h"
#include "UnrealEdMisc.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LogApplicationMode, Log, All);

#define LOCTEXT_NAMESPACE "ApplicationMode"

/////////////////////////////////////////////////////
// FApplicationMode

FApplicationMode::FApplicationMode(FName InModeName) :
	ModeName(InModeName),
	ToolbarExtender(MakeShareable(new FExtender)),
	WorkspaceMenuCategory(FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_AssetEditor", "Asset Editor")))
{
}

FApplicationMode::FApplicationMode(FName InModeName, FText(*GetLocalizedMode)(const FName)) :
	ModeName(InModeName),
	ToolbarExtender(MakeShareable(new FExtender)),
	WorkspaceMenuCategory(FWorkspaceItem::NewGroup(FText::Format(LOCTEXT("WorkspaceMenu_ApplicationMode", "{0} Editor"), GetLocalizedMode(InModeName))))
{
}

void FApplicationMode::DeactivateMode()
{
	// Save the layout to INI
	TSharedPtr<FWorkflowCentricApplication> Host = GetHost();
	check(Host.IsValid());
	TSharedPtr<FTabManager> TabManager = Host->GetAssociatedTabManager();
	check(TabManager.IsValid());
	FLayoutSaveRestore::SaveToConfig(LayoutIni, TabManager->PersistLayout());

	// Unregister the tabs
	/*
	for (int32 Index = 0; Index < AllowableTabs.Num(); ++Index)
	{
		const FName TabName = AllowableTabs[Index];

		PinnedTabManager->UnregisterTabSpawner(TabName);
	}
	*/
}

TSharedRef<FTabManager::FLayout> FApplicationMode::ActivateMode()
{
	RegisterTabFactoriesWithHost();

	/*
	for (int32 Index = 0; Index < AllowableTabs.Num(); ++Index)
	{
		const FName TabName = AllowableTabs[Index];

		PinnedTabManager->RegisterTabSpawner(TabName, FOnSpawnTab::CreateSP(this, &FApplicationMode::SpawnTab, TabName));
	}
	*/

	// Try loading the layout from INI
	check(TabLayout.IsValid());
	return FLayoutSaveRestore::LoadFromConfig(LayoutIni, TabLayout.ToSharedRef());
}

void FApplicationMode::AddTabFactory(FCreateWorkflowTabFactory FactoryCreator)
{
	if (FactoryCreator.IsBound())
	{
		TSharedPtr<FWorkflowCentricApplication> Host = GetHost();
		check(Host.IsValid());
		ApplicationModeTabFactories.RegisterFactory(FactoryCreator.Execute(Host));
	}
}

void FApplicationMode::RemoveTabFactory(FName TabFactoryID)
{
	ApplicationModeTabFactories.UnregisterFactory(TabFactoryID);
}

void FApplicationMode::RegisterTabFactoriesWithHost()
{
	TSharedPtr<FWorkflowCentricApplication> Host = GetHost();
	check(Host.IsValid());
	RegisterTabFactoriesWithManager(Host->GetAssociatedTabManager());
}

void FApplicationMode::RegisterTabFactoriesWithManager(const TSharedPtr<FTabManager>& InTabManager)
{
	if (InTabManager)
	{
		RegisterTabFactories(InTabManager);
	}
}

void FApplicationMode::RegisterTabFactoriesWithAppAndManager(FWorkflowCentricApplication* InApp, const TSharedRef<FTabManager>& InTabManager)
{
	if (InApp)
	{
		InApp->PushTabFactories(ApplicationModeTabFactories);
	}
}

void FApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	check(InTabManager.IsValid());
	RegisterTabFactoriesWithAppAndManager(GetHost().Get(), InTabManager.ToSharedRef());
}

void FApplicationMode::ShareHost(const FApplicationMode* Mode)
{
	if (Mode)
	{
		HostWeakPtr = Mode->HostWeakPtr;
	}
	else
	{
		if (HostWeakPtr.IsUnique())
		{
			SetHost(nullptr);
		}
		else
		{
			HostWeakPtr = MakeShared<TWeakPtr<FWorkflowCentricApplication>>();
		}
	}
}

#undef LOCTEXT_NAMESPACE
