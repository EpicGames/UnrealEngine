// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinitionDefault.h"

#include "AssetToolsModule.h"
#include "IAssetStatusInfoProvider.h"
#include "EditorFramework/ThumbnailInfo.h"
#include "ISourceControlModule.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Toolkits/SimpleAssetEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinitionDefault)

#define LOCTEXT_NAMESPACE "AssetDefinitionDefault"

EAssetCommandResult UAssetDefinitionDefault::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (GetAssetOpenSupport(FAssetOpenSupportArgs(OpenArgs.OpenMethod)).IsSupported)
	{
		FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, OpenArgs.LoadObjects<UObject>());
		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

EAssetCommandResult UAssetDefinitionDefault::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	// Dump assets to temp text files
	FString OldTextFilename = AssetTools.DumpAssetToTempFile(DiffArgs.OldAsset);
	FString NewTextFilename = AssetTools.DumpAssetToTempFile(DiffArgs.NewAsset);
	FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

	AssetTools.CreateDiffProcess(DiffCommand, OldTextFilename, NewTextFilename);

	return EAssetCommandResult::Handled;
}

namespace UE::Editor
{
	UThumbnailInfo* FindOrCreateThumbnailInfo(UObject* AssetObject, TSubclassOf<UThumbnailInfo> ThumbnailClass)
	{
		if (AssetObject && ThumbnailClass)
		{
			if (const FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(AssetObject->GetClass(), "ThumbnailInfo"))
			{
				// Is the property marked as Instanced?
				if (ObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance | CPF_ExportObject | CPF_InstancedReference))
				{
					// Get the thumbnail.
					UThumbnailInfo* ThumbnailInfo = Cast<UThumbnailInfo>(ObjectProperty->GetObjectPropertyValue_InContainer(AssetObject));
					if (ThumbnailInfo && ThumbnailInfo->GetClass() == ThumbnailClass)
					{
						return ThumbnailInfo;
					}

					// We couldn't find it, need to initialize it.
					ThumbnailInfo = NewObject<UThumbnailInfo>(AssetObject, ThumbnailClass, NAME_None, RF_Transactional);
					ObjectProperty->SetObjectPropertyValue_InContainer(AssetObject, ThumbnailInfo);

					return ThumbnailInfo;
				}
			}
		}
		
		return nullptr;
	}

	bool TrySetExistingThumbnailInfo(UObject* InAssetObject, UThumbnailInfo* InThumbnailInfo)
	{
		if (InAssetObject)
		{
			if (const FObjectProperty* ObjectProperty = FindFProperty<FObjectProperty>(InAssetObject->GetClass(), "ThumbnailInfo"))
			{
				// Is the property marked as Instanced?
				if (ObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance | CPF_ExportObject | CPF_InstancedReference))
				{
					ObjectProperty->SetObjectPropertyValue_InContainer(InAssetObject, InThumbnailInfo);
					return true;
				}
			}
		}
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
