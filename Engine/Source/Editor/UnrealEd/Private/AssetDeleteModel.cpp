// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDeleteModel.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"

#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AutoReimport/AutoReimportUtilities.h"
#include "AutoReimport/AutoReimportManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/ReferencerFinder.h"

#define LOCTEXT_NAMESPACE "FAssetDeleteModel"

FAssetDeleteModel::FAssetDeleteModel( const TArray<UObject*>& InObjectsToDelete )
	: State( StartScanning )
	, bPendingObjectsCanBeReplaced(false)
	, bIsAnythingReferencedInMemoryByNonUndo(false)
	, bIsAnythingReferencedInMemoryByUndo(false)
	, PendingDeleteIndex(0)
	, ObjectsDeleted(0)
{
	// Take a weak copy in-case GC purges any objects from this array.
	TArray<TWeakObjectPtr<UObject>> WeakObjectsToDelete;
	WeakObjectsToDelete.Reserve( InObjectsToDelete.Num() );
	for ( UObject* ObjectToDelete : InObjectsToDelete )
	{
		WeakObjectsToDelete.Add( ObjectToDelete );
	}

	// Purge unclaimed objects.
	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

	// Create a pending delete object for each object needing to be deleted so that we can store reference tracing information.
	for ( const TWeakObjectPtr<UObject>& WeakObjectToDelete : WeakObjectsToDelete )
	{
		UObject* ObjectToDelete = WeakObjectToDelete.Get();
		if ( ObjectToDelete )
		{
			AddObjectToDelete( ObjectToDelete );
		}
	}
}

FAssetDeleteModel::~FAssetDeleteModel()
{
}

void FAssetDeleteModel::AddObjectToDelete( UObject* InObject )
{
	if (!PendingDeletes.ContainsByPredicate([=](const TSharedPtr<FPendingDelete>& A) { return A->GetObject() == InObject; }))
	{
		PendingDeletes.Add(MakeShareable(new FPendingDelete(InObject)));
	}

	PrepareToDelete(InObject);

	SetState(StartScanning);
}

void FAssetDeleteModel::SetState( EState NewState )
{
	if ( State != NewState )
	{
		State = NewState;
		if ( StateChanged.IsBound() )
		{
			StateChanged.Broadcast( NewState );
		}
	}
}

void FAssetDeleteModel::Tick( const float InDeltaTime )
{
	const double MaxTickSeconds   = 0.100;
	double       StartTickSeconds = FPlatformTime::Seconds();

	switch ( State )
	{
	case Waiting:
		break;
	case StartScanning:
		OnDiskReferences = TSet<FName>();
		bIsAnythingReferencedInMemoryByNonUndo = false;
		bIsAnythingReferencedInMemoryByUndo = false;
		SourceFileToAssetCount.Empty();
		PendingDeleteIndex = 0;

		SetState(Scanning);
		break;
	case Scanning:
		while (PendingDeleteIndex < PendingDeletes.Num() && (FPlatformTime::Seconds() - StartTickSeconds) < MaxTickSeconds)
		{
			TSharedPtr<FPendingDelete>& PendingDelete = PendingDeletes[PendingDeleteIndex];

			PendingDelete->CheckForReferences();

			// Add references are not part of the objects being deleted.
			int NonPendingDeletedExternalOnDiskReferences = 0;
			for ( FName& Reference : PendingDelete->DiskReferences )
			{
				if ( !IsAssetInPendingDeletes( Reference ) )
				{
					OnDiskReferences.Add( Reference );
					NonPendingDeletedExternalOnDiskReferences++;
				}
			}
			PendingDelete->RemainingDiskReferences = NonPendingDeletedExternalOnDiskReferences;

			// Count up all the external references
			int32 NonPendingDeletedExternalInMemoryReferences = 0;

			for ( const FReferencerInformation& Reference : PendingDelete->MemoryReferences.ExternalReferences )
			{
				if ( !IsObjectInPendingDeletes( Reference.Referencer ) )
				{
					NonPendingDeletedExternalInMemoryReferences++;
				}
			}
			PendingDelete->RemainingMemoryReferences = NonPendingDeletedExternalInMemoryReferences;

			DiscoverSourceFileReferences(*PendingDelete);

			bIsAnythingReferencedInMemoryByNonUndo |= PendingDelete->RemainingMemoryReferences > 0;
			bIsAnythingReferencedInMemoryByUndo |= PendingDelete->IsReferencedInMemoryByUndo();

			PendingDeleteIndex++;
		}

		if (PendingDeleteIndex >= PendingDeletes.Num())
		{
			SetState(UpdateActions);
		}
		break;
	case UpdateActions:
		bPendingObjectsCanBeReplaced = ComputeCanReplaceReferences();
		SetState(Finished);
		break;
	case Finished:
		break;
	}
}

void FAssetDeleteModel::DiscoverSourceFileReferences(FPendingDelete& PendingDelete)
{
	if (!GetDefault<UEditorLoadingSavingSettings>()->bMonitorContentDirectories)
	{
		return;
	}
	
	// Start by extracting the files from the object
	TArray<FString> SourceContentFiles;
	Utils::ExtractSourceFilePaths(PendingDelete.GetObject(), SourceContentFiles);

	if (GUnrealEd && GUnrealEd->AutoReimportManager )
	{
		TArray<FPathAndMountPoint> MonitoredDirectories = GUnrealEd->AutoReimportManager->GetMonitoredDirectories();

		// Remove anything that's not under a monitored, mounted path, or doesn't exist
		SourceContentFiles.RemoveAll([&] (const FString& InFilename) {
			for ( const FPathAndMountPoint& Dir : MonitoredDirectories )
			{
				if ( !Dir.MountPoint.IsEmpty() && InFilename.StartsWith(Dir.Path) )
				{
					if ( FPaths::FileExists(InFilename) )
					{
						return false;
					}
					else
					{
						return true;
					}
				}
			}
			return true;
		});
	}

	// Now accumulate references to the same source content file. We only offer to delete a file if it is only referenced by the deleted object(s)
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	for (const FString& SourcePath : SourceContentFiles)
	{
		int32* NumReferences = SourceFileToAssetCount.Find(SourcePath);
		if (NumReferences)
		{
			(*NumReferences)--;
		}
		else
		{
			const int32 NumReferencingAssets = Utils::FindAssetsPertainingToFile(AssetRegistry, SourcePath).Num();
			SourceFileToAssetCount.Add(SourcePath, NumReferencingAssets - 1);
		}
	}
}

bool FAssetDeleteModel::HasAnySourceContentFilesToDelete() const
{
	for (const auto& Pair : SourceFileToAssetCount)
	{
		if (Pair.Value == 0)
		{
			return true;
		}
	}
	
	return false;
}

void FAssetDeleteModel::DeleteSourceContentFiles()
{
	IFileManager& FileManager = IFileManager::Get();
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	for (const auto& Pair : SourceFileToAssetCount)
	{
		const auto& Path = Pair.Key;
		// We can only delete this path if there are no (non-deleted) objects referencing it
		if (Pair.Value != 0)
		{
			continue;
		}

		// One way or another this file is going to be deleted, but we don't want the import manager to react to the deletion
		GUnrealEd->AutoReimportManager->IgnoreDeletedFile(Path);

		if (ISourceControlModule::Get().IsEnabled())
		{
			const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Path, EStateCacheUsage::ForceUpdate);
			const bool bIsSourceControlled = SourceControlState.IsValid() && SourceControlState->IsSourceControlled();

			if (bIsSourceControlled)
			{
				// The file is managed by source control. Delete it through there.
				TArray<FString> DeleteFilenames;
				DeleteFilenames.Add(Path);

				// Revert the file if it is checked out
				const bool bIsAdded = SourceControlState->IsAdded();
				if (SourceControlState->IsCheckedOut() || bIsAdded || SourceControlState->IsDeleted())
				{
					SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), DeleteFilenames);
				}

				// If it wasn't already marked as an add, we can ask the source control provider to delete the file
				if (!bIsAdded)
				{
					// Open the file for delete
					SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), DeleteFilenames);
					continue;
				}
			}
		}

		// We'll just delete it ourself
		FileManager.Delete(*Path, false /* RequireExists */, true /* Even if read only */, true /* Quiet */);
	}
}

bool FAssetDeleteModel::CanDelete() const
{
	return !CanForceDelete();
}

bool FAssetDeleteModel::DoDelete()
{
	if ( !CanDelete() )
	{
		return false;
	}

	TArray<UObject*> ObjectsToDelete;
	for ( TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		ObjectsToDelete.Add( PendingDelete->GetObject() );
	}

	ObjectsDeleted = ObjectTools::DeleteObjectsUnchecked(ObjectsToDelete);

	return true;
}

bool FAssetDeleteModel::CanForceDelete() const
{
	// We can only force delete when they are still referenced in memory or still referenced on disk.
	return bIsAnythingReferencedInMemoryByNonUndo || (OnDiskReferences.Num() > 0);
}

bool FAssetDeleteModel::DoForceDelete()
{
	if ( !CanForceDelete() )
	{
		return false;
	}

	TArray<UObject*> ObjectsToForceDelete;
	for ( TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		ObjectsToForceDelete.Add(PendingDelete->GetObject());
	}

	ObjectsDeleted = ObjectTools::ForceDeleteObjects(ObjectsToForceDelete, false);

	return true;
}

bool FAssetDeleteModel::ComputeCanReplaceReferences()
{
	TArray<UObject*> PendingDeletedObjects;
	for ( const TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		PendingDeletedObjects.Add(PendingDelete->GetObject());
	}

	return ObjectTools::AreObjectsOfEquivalantType(PendingDeletedObjects);
}

bool FAssetDeleteModel::CanReplaceReferences() const
{
	return bPendingObjectsCanBeReplaced;
}

bool FAssetDeleteModel::CanReplaceReferencesWith( const FAssetData& InAssetData ) const
{
	// First make sure that it's not an object we're preparing to delete
	if ( IsAssetInPendingDeletes( InAssetData.PackageName ) )
	{
		return true;
	}

	const UClass* FirstPendingDelete = PendingDeletes[0]->GetObject()->GetClass();
	const UClass* AssetDataClass = InAssetData.GetClass();

	// If the class isn't loaded we can't compare them, so just return true that we'll be
	// filtering it from the list.
	if ( AssetDataClass == nullptr )
	{
		return true;
	}

	// Filter out blueprints of different types
	if (FirstPendingDelete->IsChildOf(UBlueprint::StaticClass()) && AssetDataClass->IsChildOf(UBlueprint::StaticClass()))
	{
		// Get BP native parent classes
		UClass* OriginalBPParentClass = CastChecked<UBlueprint>(PendingDeletes[0]->GetObject())->ParentClass;
		const FString NativeClassName = InAssetData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);

		if (!NativeClassName.IsEmpty())
		{
			UClass* NativeParentClassToTest = UClass::TryFindTypeSlow<UClass>(NativeClassName);
			UClass* NativeParentClassToReplace = FBlueprintEditorUtils::FindFirstNativeClass(OriginalBPParentClass);

			if (!NativeParentClassToTest || !NativeParentClassToTest->IsChildOf(NativeParentClassToReplace))
			{
				// If we couldn't determine the asset parent class (e.g. because the NativeParentClass tag wasn't present in the FAssetData), or if
				// the asset parent class wasn't equal to or derived from the pending delete BP class, filter i
				return true;
			}
		}
	}

	// Only show objects that are of replaceable because their classes are compatible.
	return !ObjectTools::AreClassesInterchangeable( FirstPendingDelete, AssetDataClass );
}

bool FAssetDeleteModel::DoReplaceReferences(const FAssetData& ReplaceReferencesWith )
{
	if ( !CanReplaceReferences() )
	{
		return false;
	}

	// Find which object the user has elected to be the "object to consolidate to"
	UObject* ObjectToConsolidateTo = ReplaceReferencesWith.GetAsset();
	check( ObjectToConsolidateTo );

	TArray<UObject*> FinalConsolidationObjects;
	for ( TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		FinalConsolidationObjects.Add(PendingDelete->GetObject());
	}

	// The consolidation action clears the array, so we need to save the count.
	int32 ObjectsBeingDeletedCount = FinalConsolidationObjects.Num();

	// Perform the object consolidation
	bool bShowDeleteConfirmation = false;
	ObjectTools::FConsolidationResults ConsResults = ObjectTools::ConsolidateObjects( ObjectToConsolidateTo, FinalConsolidationObjects, bShowDeleteConfirmation );

	// If the consolidation went off successfully with no failed objects, prompt the user to checkout/save the packages dirtied by the operation
	if ( ConsResults.DirtiedPackages.Num() > 0 && ConsResults.FailedConsolidationObjs.Num() == 0)
	{
		FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
		SaveParams.bCheckDirty = false;
		SaveParams.bPromptToSave = true;
		SaveParams.bIsExplicitSave = true;

		FEditorFileUtils::PromptForCheckoutAndSave( ObjectPtrDecay(ConsResults.DirtiedPackages), SaveParams);
	}
	// If the consolidation resulted in failed (partially consolidated) objects, do not save, and inform the user no save attempt was made
	else if ( ConsResults.FailedConsolidationObjs.Num() > 0)
	{
		//DisplayMessage( true, LOCTEXT( "Consolidate_WarningPartial", "Not all objects could be consolidated, no save has occurred" ).ToString() );
	}

	ObjectsDeleted = ObjectsBeingDeletedCount - ( ConsResults.FailedConsolidationObjs.Num() + ConsResults.InvalidConsolidationObjs.Num() );

	return true;
}

bool FAssetDeleteModel::IsObjectInPendingDeletes( const UObject* InObject ) const
{
	for ( const TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		if ( PendingDelete->IsObjectContained(InObject) )
		{
			return true;
		}
	}

	return false;
}

bool FAssetDeleteModel::IsAssetInPendingDeletes( const FName& PackageName ) const
{
	for ( const TSharedPtr< FPendingDelete >& PendingDelete : PendingDeletes )
	{
		if ( PendingDelete->IsAssetContained(PackageName) )
		{
			return true;
		}
	}

	return false;
}

float FAssetDeleteModel::GetProgress() const
{
	return PendingDeleteIndex / (float)PendingDeletes.Num();
}

FText FAssetDeleteModel::GetProgressText() const
{
	if ( PendingDeleteIndex < PendingDeletes.Num() )
	{
		return FText::FromString(PendingDeletes[PendingDeleteIndex]->GetObject()->GetName());
	}
	else
	{
		return LOCTEXT( "Done", "Done!" );
	}
}

bool FAssetDeleteModel::GoToNextReferenceInLevel() const
{
	// Clear the current selection
	GEditor->SelectNone(false, false);

	UWorld* RepresentingWorld = NULL;

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if ( Context.WorldType == EWorldType::PIE )
		{
			RepresentingWorld = Context.World();
			break;
		}
		else if ( Context.WorldType == EWorldType::Editor )
		{
			RepresentingWorld = Context.World();
		}
	}

	// If there is no world we definitely can't find any actors in the level
	if ( !RepresentingWorld )
	{
		return false;
	}

	bool bRepresentingPlayWorld = RepresentingWorld->WorldType == EWorldType::PIE;

	// @todo I'm not sure what should be done if PIE is active...
	if ( bRepresentingPlayWorld )
	{
		return false;
	}

	// Go over every pending deleted object, and for all of its references go to the first
	// one in the current world.
	for ( const TSharedPtr<FPendingDelete>& PendingDelete : PendingDeletes )
	{
		for ( const FReferencerInformation& Reference : PendingDelete->MemoryReferences.ExternalReferences )
		{
			const AActor* ReferencingActor = Cast<const AActor>(Reference.Referencer);

			// If the referencer isn't an actor, it might be a component.
			if ( !ReferencingActor )
			{
				const UActorComponent* ReferencingComponent = Cast<const UActorComponent>(Reference.Referencer);
				if ( ReferencingComponent )
				{
					ReferencingActor = ReferencingComponent->GetOwner();
				}
			}

			if ( ReferencingActor )
			{
				if ( ReferencingActor->GetWorld() == RepresentingWorld )
				{
					GEditor->SelectActor(const_cast<AActor*>( ReferencingActor ), true, true, true);

					// Point the camera at it
					GUnrealEd->Exec(ReferencingActor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));

					return true;
				}
			}
		}
	}
	
	return false;
}

int32 FAssetDeleteModel::GetDeletedObjectCount() const
{
	return ObjectsDeleted;
}

void FAssetDeleteModel::PrepareToDelete(UObject* InObject)
{
	if ( UObjectRedirector* RedirectorObj = Cast<UObjectRedirector>(InObject) )
	{
		// Add all redirectors found in this package to the redirectors to delete list.
		// All redirectors in this package should be fixed up.
		UPackage* RedirectorPackage = RedirectorObj->GetOutermost();
		TArray<UObject*> AssetsInRedirectorPackage;
		
		GetObjectsWithOuter(RedirectorPackage, AssetsInRedirectorPackage, EGetObjectsFlags::None);
		bool bContainsAtLeastOneOtherAsset = false;

		for ( auto ObjIt = AssetsInRedirectorPackage.CreateConstIterator(); ObjIt; ++ObjIt )
		{
			if ( UObjectRedirector* Redirector = Cast<UObjectRedirector>(*ObjIt) )
			{
				Redirector->RemoveFromRoot();
				continue;
			}
			
			bContainsAtLeastOneOtherAsset = true;
		}

		if ( !bContainsAtLeastOneOtherAsset )
		{
			RedirectorPackage->RemoveFromRoot();
		}
	}
}

// FPendingDelete
//-----------------------------------------------------------------

FPendingDelete::FPendingDelete(UObject* InObject)
	: RemainingDiskReferences(0)
	, RemainingMemoryReferences(0)
	, Object(InObject)
	, bReferencesChecked(false)
	, bIsReferencedInMemoryByNonUndo(false)
	, bIsReferencedInMemoryByUndo(false)
	, bIsInternal(false)
{
	// Blueprints actually contain 3 assets, the UBlueprint, GeneratedClass and SkeletonGeneratedClass
	// we need to add them all to the pending deleted objects so that the memory reference detection system
	// can know those references don't count towards the in memory references.
	if ( UBlueprint* BlueprintObj = Cast<UBlueprint>(InObject) )
	{
		if ( BlueprintObj->GeneratedClass != NULL )
		{
			InternalObjects.Add(BlueprintObj->GeneratedClass);
		}

		if ( BlueprintObj->SkeletonGeneratedClass != NULL )
		{
			InternalObjects.Add(BlueprintObj->SkeletonGeneratedClass);
		}
	}

	// Filter out any non assets
	if ( !FAssetData::IsUAsset(InObject) )
	{
		bIsInternal = true;
	}
}

bool FPendingDelete::IsObjectContained(const UObject* InObject) const
{
	const UObject* InObjectParent = InObject;

	// If the objects are in the same package then it should be safe to delete them since the package
	// will be marked for garbage collection.
	if ( Object->GetOutermost() == InObject->GetOutermost() )
	{
		return true;
	}

	// We need to check if the object or any of it's parents are children of the object being deleted, and so
	// can safely be ignored.
	while ( InObjectParent != NULL )
	{
		if ( Object == InObjectParent )
		{
			return true;
		}

		// Also check if it's a child of any of the internal objects.
		for ( int32 InternalIndex = 0; InternalIndex < InternalObjects.Num(); InternalIndex++ )
		{
			if ( InternalObjects[InternalIndex] == InObjectParent )
			{
				return true;
			}
		}

		InObjectParent = InObjectParent->GetOuter();
	}

	return false;
}

bool FPendingDelete::IsAssetContained(const FName& PackageName) const
{
	if ( Object->GetOutermost()->GetFName() == PackageName )
	{
		return true;
	}

	return false;
}


void FPendingDelete::CheckForReferences()
{
	if (bReferencesChecked)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPendingDelete::CheckForReferences)
	bReferencesChecked = true;

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	AssetRegistryModule.Get().GetReferencers(Object->GetOutermost()->GetFName(), DiskReferences);

	IConsoleVariable* UseLegacyGetReferencersForDeletion = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.UseLegacyGetReferencersForDeletion"));
	if (UseLegacyGetReferencersForDeletion == nullptr || !UseLegacyGetReferencersForDeletion->GetBool())
	{
		// This new version uses the fast reference collector to gather referencers and handles transaction referencers in a single pass
		ObjectTools::GatherObjectReferencersForDeletion(Object, bIsReferencedInMemoryByNonUndo, bIsReferencedInMemoryByUndo, &MemoryReferences);
	}
	else
	{
		// Check and see whether we are referenced by any objects that won't be garbage collected (*including* the undo buffer)
		FReferencerInformationList ReferencesIncludingUndo;
		bool bReferencedInMemoryOrUndoStack = IsReferenced(Object, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &ReferencesIncludingUndo);

		// Determine the in-memory references, *excluding* the undo buffer
		if (GEditor && GEditor->Trans)
		{
			GEditor->Trans->DisableObjectSerialization();
		}
		bIsReferencedInMemoryByNonUndo = IsReferenced(Object, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &MemoryReferences);
		if (GEditor && GEditor->Trans)
		{
			GEditor->Trans->EnableObjectSerialization();
		}

		// see if this object is the transaction buffer - set a flag so we know we need to clear the undo stack
		const int32 TotalReferenceCount = ReferencesIncludingUndo.ExternalReferences.Num() + ReferencesIncludingUndo.InternalReferences.Num();
		const int32 NonUndoReferenceCount = MemoryReferences.ExternalReferences.Num() + MemoryReferences.InternalReferences.Num();

		bIsReferencedInMemoryByUndo = TotalReferenceCount > NonUndoReferenceCount;

		// If the object itself isn't in the transaction buffer, check to see if it's a Blueprint asset. We might have instances of the
		// Blueprint in the transaction buffer, in which case we also want to both alert the user and clear it prior to deleting the asset.
		if (!bIsReferencedInMemoryByUndo)
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(Object);
			if (Blueprint && Blueprint->GeneratedClass)
			{
				TArray<FReferencerInformation> ExternalMemoryReferences = MemoryReferences.ExternalReferences;
				for (auto RefIt = ExternalMemoryReferences.CreateIterator(); RefIt && !bIsReferencedInMemoryByUndo; ++RefIt)
				{
					FReferencerInformation& RefInfo = *RefIt;
					if (RefInfo.Referencer->IsA(Blueprint->GeneratedClass))
					{
						if (IsReferenced(RefInfo.Referencer, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &ReferencesIncludingUndo))
						{
							if (GEditor && GEditor->Trans)
							{
								GEditor->Trans->DisableObjectSerialization();
							}

							FReferencerInformationList ReferencesExcludingUndo;
							if (IsReferenced(RefInfo.Referencer, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &ReferencesExcludingUndo))
							{
								bIsReferencedInMemoryByUndo = (ReferencesIncludingUndo.InternalReferences.Num() + ReferencesIncludingUndo.ExternalReferences.Num()) > (ReferencesExcludingUndo.InternalReferences.Num() + ReferencesExcludingUndo.ExternalReferences.Num());
							}
							if (GEditor && GEditor->Trans)
							{
								GEditor->Trans->EnableObjectSerialization();
							}
						}
					}
				}
			}
		}
	}
}

bool FPendingDelete::operator == ( const FPendingDelete& Other ) const
{
	return Object == Other.Object;
}

#undef LOCTEXT_NAMESPACE
