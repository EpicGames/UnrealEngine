// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditorModeManager.h"
#include "Engine/Selection.h"
#include "PreviewScene.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

//////////////////////////////////////////////////////////////////////////
// FAssetEditorModeManager

FAssetEditorModeManager::FAssetEditorModeManager()
{
	{
		UTypedElementSelectionSet* ActorAndComponentsSelectionSet = NewObject<UTypedElementSelectionSet>(GetTransientPackage(), NAME_None, RF_Transactional);

		ActorSet = USelection::CreateActorSelection(GetTransientPackage(), NAME_None, RF_Transactional);
		ActorSet->SetElementSelectionSet(ActorAndComponentsSelectionSet);
		ActorSet->AddToRoot();

		ComponentSet = USelection::CreateComponentSelection(GetTransientPackage(), NAME_None, RF_Transactional);
		ComponentSet->SetElementSelectionSet(ActorAndComponentsSelectionSet);
		ComponentSet->AddToRoot();
	}

	ObjectSet = USelection::CreateObjectSelection(GetTransientPackage(), NAME_None, RF_Transactional);
	ObjectSet->SetElementSelectionSet(NewObject<UTypedElementSelectionSet>(ObjectSet, NAME_None, RF_Transactional));
	ObjectSet->AddToRoot();
}

FAssetEditorModeManager::~FAssetEditorModeManager()
{
	// We may be destroyed after the UObject system has already shutdown, 
	// which would mean that these instances will be garbage
	if (UObjectInitialized())
	{
		if (!ActorSet->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			check(ActorSet->GetElementSelectionSet() == ComponentSet->GetElementSelectionSet());
			if (UTypedElementSelectionSet* ActorAndComponentsSelectionSet = ActorSet->GetElementSelectionSet())
			{
				ActorAndComponentsSelectionSet->ClearSelection(FTypedElementSelectionOptions());
			}
		}

		ActorSet->RemoveFromRoot();
		ComponentSet->RemoveFromRoot();

		if (!ObjectSet->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			if (UTypedElementSelectionSet* ObjectSelectionSet = ObjectSet->GetElementSelectionSet())
			{
				ObjectSelectionSet->ClearSelection(FTypedElementSelectionOptions());
			}
		}

		ObjectSet->RemoveFromRoot();
	}
}

USelection* FAssetEditorModeManager::GetSelectedActors() const
{
	return ActorSet;
}

USelection* FAssetEditorModeManager::GetSelectedObjects() const
{
	return ObjectSet;
}

USelection* FAssetEditorModeManager::GetSelectedComponents() const
{
	return ComponentSet;
}

UWorld* FAssetEditorModeManager::GetWorld() const
{
	return (PreviewSceneWorld.IsValid()) ? PreviewSceneWorld.Get() : GEditor->GetEditorWorldContext().World();
}

void FAssetEditorModeManager::SetPreviewScene(FPreviewScene* NewPreviewScene)
{
	PreviewScene = NewPreviewScene;

	// Due to destruction order, we might get a call from FEditorModeTools::OnWorldCleanup with an already destoyed PreviewScene,
	// but we sill have to return the correct World the editor was using to perform a correct shutdown,
	// so caching the PreviewScene World when assigning (and clearing if null preview is set)
	PreviewSceneWorld = (NewPreviewScene != nullptr) ? NewPreviewScene->GetWorld() : nullptr;
}

FPreviewScene* FAssetEditorModeManager::GetPreviewScene() const
{
	return PreviewScene;
}
