// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetEditorViewportLayout.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Framework/Docking/LayoutService.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SCanvas.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Docking/SDockTab.h"
#include "EditorViewportLayout.h"
#include "EditorViewportTypeDefinition.h"
#include "EditorViewportLayoutEntity.h"
#include "EditorViewportCommands.h"
#include "SAssetEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EditorViewportTabContent.h"
#include "EditorViewportLayoutOnePane.h"
#include "EditorViewportLayoutTwoPanes.h"
#include "EditorViewportLayoutThreePanes.h"
#include "EditorViewportLayoutFourPanes.h"
#include "EditorViewportLayout2x2.h"


#define LOCTEXT_NAMESPACE "AssetEditorViewportToolBar"


namespace ViewportLayoutDefs
{
	/** How many seconds to interpolate from restored to maximized state */
	static const float MaximizeTransitionTime = 0.15f;

	/** How many seconds to interpolate from maximized to restored state */
	static const float RestoreTransitionTime = 0.2f;

	/** Default maximized state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeMaximized = true;

	/** Default immersive state for new layouts - will only be applied when no config data is restoring state */
	static const bool bDefaultShouldBeImmersive = false;
}


// AssetEditorViewport ////////////////////////////////////////////////


void SAssetEditorViewportsOverlay::Construct( const FArguments& InArgs )
{
	const TSharedRef<SWidget>& ContentWidget = InArgs._Content.Widget;
	ViewportTab = InArgs._ViewportTab;

	ChildSlot
		[
			SAssignNew( OverlayWidget, SOverlay )
			+ SOverlay::Slot()
		[
			ContentWidget
		]
		];
}

void SAssetEditorViewportsOverlay::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedSize = AllottedGeometry.Size;
}

SOverlay::FScopedWidgetSlotArguments SAssetEditorViewportsOverlay::AddSlot()
{
	return OverlayWidget->AddSlot();
}

void SAssetEditorViewportsOverlay::RemoveSlot()
{
	return OverlayWidget->RemoveSlot();
}

const FVector2D& SAssetEditorViewportsOverlay::GetCachedSize() const
{
	return CachedSize;
}

TSharedPtr<FViewportTabContent> SAssetEditorViewportsOverlay::GetViewportTab() const
{
	return ViewportTab;
}

void FAssetEditorViewportPaneLayout::LoadConfig(const FString& LayoutString, TFunction<void(const FString&, const FName)> LoadAdditionalLayoutInfoCallback)
{
	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
	LoadLayoutString(SpecificLayoutString);

	if (LoadAdditionalLayoutInfoCallback)
	{
		LoadAdditionalLayoutInfoCallback(SpecificLayoutString, PerspectiveViewportConfigKey);
	}
}

void FAssetEditorViewportPaneLayout::SaveConfig(const FString& LayoutString, TFunction<void(const FString&)> SaveAdditionalLayoutInfoCallback) const
{
	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
	SaveLayoutString(SpecificLayoutString);

	if (SaveAdditionalLayoutInfoCallback)
	{
		SaveAdditionalLayoutInfoCallback(SpecificLayoutString);
	}
}

FString FAssetEditorViewportPaneLayout::GetTypeSpecificLayoutString(const FString& LayoutString) const
{
	if (LayoutString.IsEmpty())
	{
		return LayoutString;
	}
	return FString::Printf(TEXT("%s.%s"), *GetLayoutTypeName().ToString(), *LayoutString);
}

// FAssetEditorViewportLayout /////////////////////////////

FAssetEditorViewportLayout::FAssetEditorViewportLayout()
{
}


FAssetEditorViewportLayout::~FAssetEditorViewportLayout()
{
	for (auto& Pair : Viewports)
	{
		Pair.Value->OnLayoutDestroyed();
	}
}

TSharedRef<SWidget> FAssetEditorViewportLayout::FactoryViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs)
{
	TSharedPtr<SAssetEditorViewport> EditorViewport;
	TSharedPtr<FEditorViewportTabContent> PinnedTabContent = ParentTabContent.Pin();
	if (PinnedTabContent.IsValid())
	{
		EditorViewport = PinnedTabContent->CreateSlateViewport(InTypeName, ConstructionArgs);
	}
	else
	{
		EditorViewport = SNew(SAssetEditorViewport, ConstructionArgs);
	}

	TSharedRef<IEditorViewportLayoutEntity> LayoutEntity = MakeShareable(new FEditorViewportLayoutEntity(EditorViewport));
	Viewports.Add(ConstructionArgs.ConfigKey, LayoutEntity);
	return LayoutEntity->AsWidget();
}

void FAssetEditorViewportLayout::FactoryPaneConfigurationFromTypeName(const FName& InLayoutConfigTypeName)
{
	//The items in these ifs should match the names in namespace EditorViewportConfigurationNames
	if (InLayoutConfigTypeName == EditorViewportConfigurationNames::TwoPanesHoriz) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutTwoPanesHoriz);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::TwoPanesVert) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutTwoPanesVert);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::FourPanes2x2) LayoutConfiguration = MakeShareable(new FEditorViewportLayout2x2);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::ThreePanesLeft) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutThreePanesLeft);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::ThreePanesRight) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutThreePanesRight);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::ThreePanesTop) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutThreePanesTop);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::ThreePanesBottom) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutThreePanesBottom);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::FourPanesLeft) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutFourPanesLeft);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::FourPanesRight) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutFourPanesRight);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::FourPanesBottom) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutFourPanesBottom);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::FourPanesTop) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutFourPanesTop);
	else if (InLayoutConfigTypeName == EditorViewportConfigurationNames::OnePane) LayoutConfiguration = MakeShareable(new FEditorViewportLayoutOnePane);

	if (!LayoutConfiguration.IsValid())
	{
		LayoutConfiguration = MakeShareable(new FEditorViewportLayoutOnePane);
	}
}

const FName FAssetEditorViewportLayout::GetActivePaneConfigurationTypeName() const
{
	return LayoutConfiguration.IsValid() ? LayoutConfiguration->GetLayoutTypeName() : NAME_None;
}

TSharedRef<SWidget> FAssetEditorViewportLayout::BuildViewportLayout(TSharedPtr<SDockTab> InParentDockTab, TSharedPtr<FEditorViewportTabContent> InParentTab, const FString& LayoutString)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetEditorViewportLayout::BuildViewportLayout);

	// We don't support reconfiguring an existing layout object, as this makes handling of transitions
	// particularly difficult.  Instead just destroy the old layout and create a new layout object.
	check(!ParentTab.IsValid());
	ParentTab = InParentDockTab;
	ParentTabContent = InParentTab;

	// We use an overlay so that we can draw a maximized viewport on top of the other viewports
	TSharedPtr<SBorder> ViewportsBorder;
	TSharedRef<SAssetEditorViewportsOverlay> ViewportsOverlay =
		SNew(SAssetEditorViewportsOverlay)
		.ViewportTab(InParentTab)
		[
			SAssignNew(ViewportsBorder, SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Visibility(this, &FAssetEditorViewportLayout::OnGetNonMaximizedVisibility)
		];

	ViewportsOverlayPtr = ViewportsOverlay;

	// You must have a valid layout configuration before building the layout.
	if (!ensureMsgf(LayoutConfiguration.IsValid(), TEXT("No valid layout configuration for the viewport layout was found.")))
	{
		return ViewportsOverlay;
	}

	// Don't set the content until the OverlayPtr has been set, because it access this when we want to start with the viewports maximized.
	TSharedRef<SWidget> ViewportLayoutWidget = LayoutConfiguration->MakeViewportLayout(this->AsShared(), LayoutString);
	LoadConfig(LayoutString);
	ViewportsBorder->SetContent(ViewportLayoutWidget);

	return ViewportsOverlay;
}

TSharedRef<SWidget> FAssetEditorViewportLayout::MakeViewportLayout(const FString& LayoutString)
{
	return LayoutConfiguration->MakeViewportLayout(this->AsShared(), LayoutString);
}

void FAssetEditorViewportLayout::LoadConfig(const FString& LayoutString)
{
}

void FAssetEditorViewportLayout::SaveConfig(const FString& LayoutString) const
{
	if (!LayoutString.IsEmpty())
	{
		FString LayoutTypeString = LayoutConfiguration->GetLayoutTypeName().ToString();

		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();
		GConfig->SetString(*IniSection, *(LayoutString + TEXT(".LayoutType")), *LayoutTypeString, GEditorPerProjectIni);
	}
}

EVisibility FAssetEditorViewportLayout::OnGetNonMaximizedVisibility() const
{
	return EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE

