// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/AssetGuideline.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetGuideline)

#if WITH_EDITOR


#include "Editor.h"
#include "ISettingsEditorModule.h"
#include "GameProjectGenerationModule.h"
#include "TimerManager.h"

#include "Interfaces/Interface_AssetUserData.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Application/SlateApplicationBase.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Engine/EngineTypes.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Docking/TabManager.h"
#include "Internationalization/Regex.h"
#include "Logging/StructuredLog.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "AssetGuideine"

DEFINE_LOG_CATEGORY_STATIC(LogAssetGuideline, Log, All);

bool UAssetGuideline::bAssetGuidelinesEnabled = true;

bool UAssetGuideline::IsPostLoadThreadSafe() const
{
	return true;
}

void UAssetGuideline::PostLoad()
{
	Super::PostLoad();

	// If we fail to package, this can trigger a re-build & load of failed assets 
	// via the UBT with 'WITH_EDITOR' on, but slate not initialized. Skip that.
	if (!FSlateApplicationBase::IsInitialized())
	{
		return;
	}

	static TArray<FName> TestedGuidelines;
	if (TestedGuidelines.Contains(GuidelineName))
	{
		return;
	}
	TestedGuidelines.AddUnique(GuidelineName);

	FString NeededPlugins;
	TArray<FString> IncorrectPlugins;
	for (const FString& Plugin : Plugins)
	{
		TSharedPtr<IPlugin> NeededPlugin = IPluginManager::Get().FindPlugin(Plugin);
		if (NeededPlugin.IsValid())
		{
			if (!NeededPlugin->IsEnabled())
			{
				NeededPlugins += NeededPlugin->GetFriendlyName() + "\n";
				IncorrectPlugins.Add(Plugin);
			}
		}
		else
		{
			NeededPlugins += Plugin + "\n";;
			IncorrectPlugins.Add(Plugin);
		}
	}

	FString NeededProjectSettings;
	TArray<FIniStringValue> IncorrectProjectSettings;
	for (const FIniStringValue& ProjectSetting : ProjectSettings)
	{
		if (IConsoleManager::Get().FindConsoleVariable(*ProjectSetting.Key))
		{
			FString Branch = ProjectSetting.Branch;
			// If the branch was specified, or we can parse it from the filename, use that to read
			// the ini value, as it will take Base inis into account.
			if (!Branch.IsEmpty()
				|| TryDetectIniBranchFromFilename(ProjectSetting.Filename, Branch))
			{
				FString CurrentIniValue;
				if (GConfig->GetString(*ProjectSetting.Section, *ProjectSetting.Key, CurrentIniValue, Branch))
				{
					if (CurrentIniValue != ProjectSetting.Value)
					{
						NeededProjectSettings += FString::Printf(TEXT("[%s] %s = %s\n"), *ProjectSetting.Section, *ProjectSetting.Key, *ProjectSetting.Value);
						IncorrectProjectSettings.Add(ProjectSetting);
					}
 
					continue;
				}
			}
 
			// No branch could be detected. Read directly from the specified file.
			FString CurrentIniValue;
			FString FilenamePath = FConfigCacheIni::NormalizeConfigIniPath(FPaths::ProjectDir() + ProjectSetting.Filename);
			if (GConfig->GetString(*ProjectSetting.Section, *ProjectSetting.Key, CurrentIniValue, FilenamePath))
			{
				if (CurrentIniValue != ProjectSetting.Value)
				{
					NeededProjectSettings += FString::Printf(TEXT("[%s]  %s = %s\n"), *ProjectSetting.Section, *ProjectSetting.Key, *ProjectSetting.Value);
					IncorrectProjectSettings.Add(ProjectSetting);
				}
			}
			else
			{
				NeededProjectSettings += FString::Printf(TEXT("[%s]  %s = %s\n"), *ProjectSetting.Section, *ProjectSetting.Key, *ProjectSetting.Value);
				IncorrectProjectSettings.Add(ProjectSetting);
			}
		}
	}

	if (!NeededPlugins.IsEmpty() || !NeededProjectSettings.IsEmpty())
	{
		FText SubText;
		FText TitleText;
		{
			FText AssetName = FText::AsCultureInvariant(GetPackage() ? GetPackage()->GetFName().ToString() : GetFName().ToString());

			FText MissingPlugins = NeededPlugins.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("MissingPlugins", "Needed Plugins:\n{0}"), FText::AsCultureInvariant(NeededPlugins));
			FText PluginWarning = NeededPlugins.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("PluginWarning", "Asset '{0}' needs the plugins listed above. Related assets may not display properly.\nAttemping to save this asset or related assets may result in irreversible modification due to missing plugins."), AssetName);

			FText MissingProjectSettings = NeededProjectSettings.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("MissingProjectSettings", "Needed project settings: \n{0}"), FText::AsCultureInvariant(NeededProjectSettings));
			FText ProjectSettingWarning = NeededProjectSettings.IsEmpty() ? FText::GetEmpty() : FText::Format(LOCTEXT("ProjectSettingWarning", "Asset '{0}' needs the project settings listed above. Related assets may not display properly."), AssetName);

			FFormatNamedArguments SubTextArgs;
			SubTextArgs.Add("PluginSubText", NeededPlugins.IsEmpty() ? FText::GetEmpty() : FText::Format(FText::AsCultureInvariant("{0}{1}\n"), MissingPlugins, PluginWarning));
			SubTextArgs.Add("ProjectSettingSubText", NeededProjectSettings.IsEmpty() ? FText::GetEmpty() : FText::Format(FText::AsCultureInvariant("{0}{1}\n"), MissingProjectSettings, ProjectSettingWarning));
			SubText = FText::Format(LOCTEXT("SubText", "{PluginSubText}{ProjectSettingSubText}"), SubTextArgs);

			FText NeedPlugins = LOCTEXT("NeedPlugins", "Missing Plugins!");
			FText NeedProjectSettings = LOCTEXT("NeedProjectSettings", "Missing Project Settings!");
			FText NeedBothGuidelines = LOCTEXT("NeedBothGuidelines", "Missing Plugins & Project Settings!");
			TitleText = !NeededPlugins.IsEmpty() && !NeededProjectSettings.IsEmpty() ? NeedBothGuidelines : !NeededPlugins.IsEmpty() ? NeedPlugins : NeedProjectSettings;
		}

		if (!bAssetGuidelinesEnabled)
		{
			UE_LOGF(LogAssetGuideline, Warning, "%ls %ls", *TitleText.ToString(), *SubText.ToString());
			return;
		}

		auto WarningHyperLink = [](bool NeedPluginLink, bool NeedProjectSettingLink)
		{
			if (NeedProjectSettingLink)
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("ProjectSettings"));
			}

			if (NeedPluginLink)
			{
				FGlobalTabmanager::Get()->TryInvokeTab(FName("PluginsEditor"));
			}
		};


		TPromise<TWeakPtr<SNotificationItem>> TextNotificationPromise;
		TPromise<TWeakPtr<SNotificationItem>> HyperlinkNotificationPromise;
		auto GetTextFromState = [NotificationFuture = TextNotificationPromise.GetFuture().Share(), SubText]()
		{
			SNotificationItem::ECompletionState State = SNotificationItem::CS_None;
			if (TSharedPtr<SNotificationItem> Notification = NotificationFuture.Get().Pin())
			{
				State = Notification->GetCompletionState();
			}

			switch (State)
			{
			case SNotificationItem::CS_Success: return LOCTEXT("RestartNeeded", "Plugins & project settings updated, but will be out of sync until restart.");
			case SNotificationItem::CS_Fail: return LOCTEXT("ChangeFailure", "Failed to change plugins & project settings.");
			}

			return SubText;
		};

		FText HyperlinkText = FText::GetEmpty();
		if (!NeededPlugins.IsEmpty())
		{
			HyperlinkText = LOCTEXT("PluginHyperlinkText", "Open Plugin Browser");
		}
		else if (!NeededProjectSettings.IsEmpty())
		{
			HyperlinkText = LOCTEXT("ProjectSettingsHyperlinkText", "Open Project Settings");
		}

		/* Gets text based on current notification state*/
		auto GetHyperlinkTextFromState = [NotificationFuture = HyperlinkNotificationPromise.GetFuture().Share(), HyperlinkText]()
		{
			SNotificationItem::ECompletionState State = SNotificationItem::CS_None;
			if (TSharedPtr<SNotificationItem> Notification = NotificationFuture.Get().Pin())
			{
				State = Notification->GetCompletionState();
			}

			// Make hyperlink text on success or fail empty, so that the box auto-resizes correctly.
			switch (State)
			{
			case SNotificationItem::CS_Success: return FText::GetEmpty();
			case SNotificationItem::CS_Fail: return FText::GetEmpty();
			}

			return HyperlinkText;
		};


		FNotificationInfo Info(TitleText);
		Info.bFireAndForget = false;
		Info.FadeOutDuration = 0.0f;
		Info.ExpireDuration = 0.0f;
		Info.WidthOverride = FOptionalSize();
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("GuidelineEnableMissing", "Enable Missing"), 
			LOCTEXT("GuidelineEnableMissingTT", "Attempt to automatically set missing plugins / project settings"), 
			FSimpleDelegate::CreateUObject(this, &UAssetGuideline::EnableMissingGuidelines, IncorrectPlugins, IncorrectProjectSettings),
			SNotificationItem::CS_None));

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("GuidelineDismiss", "Dismiss"),
			LOCTEXT("GuidelineDismissTT", "Dismiss this notification."), 
			FSimpleDelegate::CreateUObject(this, &UAssetGuideline::DismissNotifications),
			SNotificationItem::CS_None));

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("GuidelineRemove", "Remove Guideline"),
			LOCTEXT("GuidelineRemoveTT", "Remove guideline from this asset. Preventing this notifcation from showing up again."),
			FSimpleDelegate::CreateUObject(this, &UAssetGuideline::RemoveAssetGuideline),
			SNotificationItem::CS_None));

		Info.Text = TitleText;
		Info.SubText = MakeAttributeLambda(GetTextFromState);
		
		Info.HyperlinkText = MakeAttributeLambda(GetHyperlinkTextFromState);
		Info.Hyperlink = FSimpleDelegate::CreateLambda(WarningHyperLink, !NeededPlugins.IsEmpty(), !NeededProjectSettings.IsEmpty());


		NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		if (TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin())
		{
			TextNotificationPromise.SetValue(NotificationPtr);
			HyperlinkNotificationPromise.SetValue(NotificationPtr);
			NotificationPin->SetCompletionState(SNotificationItem::CS_None);
		}
	}
}

void UAssetGuideline::BeginDestroy()
{
	DismissNotifications();

	Super::BeginDestroy();
}

bool UAssetGuideline::TryDetectIniBranchFromFilename(const FString& Filename, FString& OutIniBranch)
{
	const FRegexPattern DefaultPathPattern(TEXT("^/?Config/Default(\\w+)\\.ini$"), ERegexPatternFlags::CaseInsensitive);
	FRegexMatcher Matcher(DefaultPathPattern, Filename);
	if (Matcher.FindNext())
	{
		OutIniBranch = Matcher.GetCaptureGroup(1);
		// The regex should ensure this capture group is non-empty
		check(!OutIniBranch.IsEmpty());

		UE_LOGFMT(LogAssetGuideline, Log, "Detected ini branch name {Branch} from filename {Filename}", OutIniBranch, Filename);
		return true;
	}
	else
	{
		UE_LOGFMT(LogAssetGuideline, Error, "Failed to detect ini branch name from filename {Filename}. Please specify the ini branch explicitly in the asset guideline.", Filename);
		return false;
	}
}

void UAssetGuideline::EnableMissingGuidelines(TArray<FString> IncorrectPlugins, TArray<FIniStringValue> IncorrectProjectSettings)
{
	if (TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin())
	{
		bool bSuccess = true;

		if (IncorrectPlugins.Num() > 0)
		{
			FGameProjectGenerationModule::Get().TryMakeProjectFileWriteable(FPaths::GetProjectFilePath());
			bSuccess = !FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FPaths::GetProjectFilePath());
		}

		if (bSuccess)
		{
			for (const FString& IncorrectPlugin : IncorrectPlugins)
			{
				FText FailMessage;
				bool bPluginEnabled = IProjectManager::Get().SetPluginEnabled(IncorrectPlugin, true, FailMessage);

				if (bPluginEnabled && IProjectManager::Get().IsCurrentProjectDirty())
				{
					bPluginEnabled = IProjectManager::Get().SaveCurrentProjectToDisk(FailMessage);
				}

				if (!bPluginEnabled)
				{
					bSuccess = false;
					break;
				}
			}
		}

		TSet<FString> ConfigFilesToFlush;
		if (bSuccess)
		{
			for (const FIniStringValue& IncorrectProjectSetting : IncorrectProjectSettings)
			{
				FString Branch = IncorrectProjectSetting.Branch;
				if (Branch.Len() == 0
					&& !TryDetectIniBranchFromFilename(IncorrectProjectSetting.Filename, Branch))
				{
					// Branch wasn't specified and couldn't be detected from the filename
					bSuccess = false;
					break;
				}

				const FString FilenamePath = FConfigCacheIni::NormalizeConfigIniPath(FPaths::ProjectDir() + IncorrectProjectSetting.Filename);

				// Prompt the user to make the file writeable if necessary
				FGameProjectGenerationModule::Get().TryMakeProjectFileWriteable(FilenamePath);
				if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FilenamePath))
				{
					UE_LOGFMT(LogAssetGuideline, Error, "Failed to make {Filename} writeable", IncorrectProjectSetting.Filename);

					bSuccess = false;
					break;
				}

				// Flush pending writes from any other code systems, so that they're not lost when we reload this branch
				GConfig->Flush(false, Branch);

				// Create a sandbox FConfigCache
				FConfigCacheIni Config(EConfigCacheType::Temporary);

				// Add an empty file to the config so it doesn't read in the original file (see FConfigCacheIni.Find())
				FConfigFile& NewFile = Config.Add(FilenamePath, FConfigFile());

				// Set the key in the temp file and write it out to the actual file
				NewFile.SetString(*IncorrectProjectSetting.Section, *IncorrectProjectSetting.Key, *IncorrectProjectSetting.Value);
				if (!NewFile.UpdateSinglePropertyInSection(*FilenamePath, *IncorrectProjectSetting.Key, *IncorrectProjectSetting.Section))
				{
					UE_LOGFMT(LogAssetGuideline, Error, "Failed to write ini file {Filename}", FilenamePath);

					bSuccess = false;
					break;
				}

				// Reload the branch from disk to synchronise the in-memory value with the on-disk value
				FConfigContext Context = FConfigContext::ForceReloadIntoGConfig();
				// No need to write the combined ini out, as all the necessary parts are already synced
				Context.bWriteDestIni = false;
				if (!Context.Load(*Branch))
				{
					UE_LOGFMT(LogAssetGuideline, Error, "Failed to reload ini branch {Branch}", Branch);

					bSuccess = false;
					break;
				}
			}
		}

		if (bSuccess)
		{
			auto ShowRestartPrompt = []()
			{
				FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
			};

			FTimerHandle NotificationFadeTimer;
			GEditor->GetTimerManager()->SetTimer(NotificationFadeTimer, FTimerDelegate::CreateLambda(ShowRestartPrompt), 3.0f, false);
		}

		NotificationPin->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		NotificationPin->ExpireAndFadeout();
		NotificationPtr.Reset();
	}
}

void UAssetGuideline::DismissNotifications()
{
	if (TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin())
	{
		NotificationPin->SetCompletionState(SNotificationItem::CS_None);
		NotificationPin->ExpireAndFadeout();
		NotificationPtr.Reset();
	}
}

void UAssetGuideline::RemoveAssetGuideline()
{
	if (TSharedPtr<SNotificationItem> NotificationPin = NotificationPtr.Pin())
	{
		if (IInterface_AssetUserData* UserDataOuter = Cast<IInterface_AssetUserData>(GetOuter()))
		{
			UserDataOuter->RemoveUserDataOfClass(UAssetGuideline::StaticClass());
			GetOuter()->MarkPackageDirty();
		}
		NotificationPin->SetCompletionState(SNotificationItem::CS_None);
		NotificationPin->ExpireAndFadeout();
		NotificationPtr.Reset();
	}
}

#undef LOCTEXT_NAMESPACE 

#endif // WITH_EDITOR
