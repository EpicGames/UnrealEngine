// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetImportTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetImportTask)

DEFINE_LOG_CATEGORY(LogAssetImportTask);

UAssetImportTask::UAssetImportTask()
{

}

const TArray<UObject*>& UAssetImportTask::GetObjects() const
{
	if (AsyncResults.IsValid())
	{
		AsyncResults->WaitUntilDone();
		return AsyncResults->GetImportedObjects();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ToRawPtrTArrayUnsafe(Result);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetImportTask::IsAsyncImportComplete() const
{
	return !AsyncResults.IsValid() || AsyncResults->GetStatus() == UE::Interchange::FImportResult::EStatus::Done;
}
