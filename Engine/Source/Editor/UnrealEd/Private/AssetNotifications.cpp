// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetNotifications.h"
#include "Animation/Skeleton.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AssetNotifications"

void FAssetNotifications::SkeletonNeedsToBeSaved(const USkeleton* Skeleton)
{
	FFormatNamedArguments Args;
	Args.Add( TEXT("SkeletonName"), FText::FromString( Skeleton->GetName() ) );
	FNotificationInfo Info( FText::Format( LOCTEXT("SkeletonNeedsToBeSaved", "Skeleton {SkeletonName} needs to be saved"), Args ) );
	Info.ExpireDuration = 5.0f;
	Info.bUseLargeFont = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_None );
	}
}

void FAssetNotifications::CannotEditCookedAsset(const class UObject* Asset)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), Asset ? FText::FromString(Asset->GetName()) : FText::GetEmpty());
	FNotificationInfo Info(FText::Format(LOCTEXT("CannotEditCookedAsset", "Cannot edit cooked asset {AssetName}"), Args));
	Info.ExpireDuration = 5.0f;
	Info.bUseLargeFont = false;
	Info.bUseSuccessFailIcons = true;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

#undef LOCTEXT_NAMESPACE
