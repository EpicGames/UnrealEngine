// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPrivatizeModel.h"
#include "Engine/Blueprint.h"
#include "Misc/AssetAccessRestrictions.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "FAssetPrivatizeModel"

FPendingPrivateAsset::FPendingPrivateAsset(UObject* InObject, const EAssetAccessSpecifier InTargetAssetAccessSpecifier, UPackage* InOwningPackage)
	: Object(InObject)
	, OwningPackage(InOwningPackage)
	, TargetAssetAccessSpecifier(InTargetAssetAccessSpecifier)
	, bIsReferencedInMemoryByNonUndo(false)
	, bIsReferencedInMemoryByUndo(false)
	, bReferencesChecked(false)
{
	if (!OwningPackage)
	{
		OwningPackage = Object->GetPackage();
	}

	OwningPackageMountPointName = FPackageName::GetPackageMountPoint(OwningPackage->GetFName().ToString());
}

bool FPendingPrivateAsset::IsReferenceIllegal(const FName& InReference)
{
	FString InReferenceString = InReference.ToString();
	// We only care about references that can be saved to disk and committed to source control
	if (InReferenceString.StartsWith(TEXT("/Engine/Transient")) ||
		FPackageName::IsMemoryPackage(InReferenceString) || FPackageName::IsTempPackage(InReferenceString))
	{
		return false;
	}

	FName ReferenceMountPointName = FPackageName::GetPackageMountPoint(InReferenceString);
	if (ReferenceMountPointName.IsNone())
	{
		return false;
	}

	if (ReferenceMountPointName == OwningPackageMountPointName)
	{
		return false;
	}

	if (TargetAssetAccessSpecifier == EAssetAccessSpecifier::Private)
	{
		// A reference is illegal if it refers to a private asset that is not in the same mount point
		return true;
	}
	else if (TargetAssetAccessSpecifier == EAssetAccessSpecifier::EpicInternal)
	{
		if (UE::AssetAccessRestrictions::IsPathAllowedToReferenceEpicInternalAssets.IsBound())
		{
			return UE::AssetAccessRestrictions::IsPathAllowedToReferenceEpicInternalAssets.Execute(FNameBuilder(ReferenceMountPointName).ToView());
		}

		return false;
	}

	return true;
}

void FPendingPrivateAsset::CheckForIllegalReferences()
{
	if (bReferencesChecked)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPendingPrivateAsset::CheckForIllegalReferences)
		bReferencesChecked = true;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	AssetRegistryModule.Get().GetReferencers(OwningPackage->GetFName(), IllegalDiskReferences);

	IllegalDiskReferences.RemoveAll([this](const FName& Reference)
		{
			return !IsReferenceIllegal(Reference);
		});

	ObjectTools::GatherObjectReferencersForDeletion(Object, bIsReferencedInMemoryByNonUndo, bIsReferencedInMemoryByUndo, &IllegalMemoryReferences);

	IllegalMemoryReferences.ExternalReferences.RemoveAll([this](const FReferencerInformation& ReferenceInfo)
		{
			if (ReferenceInfo.Referencer->HasAnyFlags(RF_Transient))
			{
				return true;
			}
			return !IsReferenceIllegal(ReferenceInfo.Referencer->GetPackage()->GetFName());
		});

	IllegalMemoryReferences.InternalReferences.RemoveAll([this](const FReferencerInformation& ReferenceInfo)
		{
			if (ReferenceInfo.Referencer->HasAnyFlags(RF_Transient))
			{
				return true;
			}
			return !IsReferenceIllegal(ReferenceInfo.Referencer->GetPackage()->GetFName());
		});

	if (IllegalMemoryReferences.ExternalReferences.IsEmpty() && IllegalMemoryReferences.InternalReferences.IsEmpty())
	{
		bIsReferencedInMemoryByNonUndo = false;
	}
}

void FPendingPrivateAsset::DoPrivatize()
{
	if (OwningPackage->SetAssetAccessSpecifier(TargetAssetAccessSpecifier))
	{
		Object->PostEditChange();
	}
}

FAssetPrivatizeModel::FAssetPrivatizeModel(const TArray<UObject*>& InObjectsToPrivatize, const EAssetAccessSpecifier InAssetAccessSpecifier)
	: bIsAnythingReferencedInMemoryByNonUndo(false)
	, bIsAnythingReferencedInMemoryByUndo(false)
	, PendingPrivateIndex(0)
	, State(StartScanning)
	, ObjectsPrivatized(0)
	, TargetAssetAccessSpecifier(InAssetAccessSpecifier)
{
	const EAssetAccessSpecifier TargetSpecifier = InAssetAccessSpecifier;
	for (UObject* ObjectToPrivatize : InObjectsToPrivatize)
	{
		if (ObjectToPrivatize)
		{
			UPackage* ObjectPackage = ObjectToPrivatize->GetPackage();

			if (ObjectPackage && (ObjectPackage->GetAssetAccessSpecifier() != TargetSpecifier))
			{
				AddObjectToPrivatize(ObjectToPrivatize, ObjectPackage);
			}
		}
	}
}

void FAssetPrivatizeModel::AddObjectToPrivatize(UObject* InObject, UPackage* InOwningPackage)
{
	bool NewPendingAsset = !PendingPrivateAssets.ContainsByPredicate([=](const TSharedPtr<FPendingPrivateAsset>& PendingPrivateAsset)
		{
			return PendingPrivateAsset->GetObject() == InObject;
		});

	if (NewPendingAsset)
	{
		PendingPrivateAssets.Add(MakeShareable(new FPendingPrivateAsset(InObject, GetTargetAssetAccessSpecifier(), InOwningPackage)));
	}

	SetState(StartScanning);
}

void FAssetPrivatizeModel::SetState(EState NewState)
{
	if (State != NewState)
	{
		State = NewState;

		if (StateChanged.IsBound())
		{
			StateChanged.Broadcast(NewState);
		}
	}
}

float FAssetPrivatizeModel::GetProgress() const
{
	return PendingPrivateIndex / (float)PendingPrivateAssets.Num();
}

FText FAssetPrivatizeModel::GetProgressText() const
{
	if (PendingPrivateIndex < PendingPrivateAssets.Num())
	{
		return FText::FromString(PendingPrivateAssets[PendingPrivateIndex]->GetObject()->GetName());
	}
	else
	{
		return LOCTEXT("Done", "Done!");
	}
}

bool FAssetPrivatizeModel::CanPrivatize()
{
	return !CanForcePrivatize();
}

bool FAssetPrivatizeModel::DoPrivatize()
{
	if (!CanPrivatize())
	{
		return false;
	}

	for (const TSharedPtr<FPendingPrivateAsset>& PendingPrivateAsset : PendingPrivateAssets)
	{
		PendingPrivateAsset->DoPrivatize();
		ObjectsPrivatized++;
	}

	return true;
}

bool FAssetPrivatizeModel::CanForcePrivatize()
{
	return bIsAnythingReferencedInMemoryByNonUndo || bIsAnythingReferencedInMemoryByUndo || (IllegalOnDiskReferences.Num() > 0);
}

bool FAssetPrivatizeModel::DoForcePrivatize()
{
	TArray<UObject*> ObjectsToPrivatize;
	ObjectsToPrivatize.Reserve(PendingPrivateAssets.Num());

	TSet<UObject*> ObjectsToPrivatizeWithin;
	for (const TSharedPtr<FPendingPrivateAsset>& PendingPrivateAsset : PendingPrivateAssets)
	{
		ObjectsToPrivatize.Add(PendingPrivateAsset->GetObject());

		FReferencerInformationList& AssetIllegalReferencers = PendingPrivateAsset->IllegalMemoryReferences;
		for (FReferencerInformation& ExternalReference : AssetIllegalReferencers.ExternalReferences)
		{
			ObjectsToPrivatizeWithin.Add(ExternalReference.Referencer);
		}

		PendingPrivateAsset->DoPrivatize();
		ObjectsPrivatized++;
	}

	if (!ObjectsToPrivatize.IsEmpty() && !ObjectsToPrivatizeWithin.IsEmpty())
	{
		// Null out the illegal references
		ObjectTools::ForceReplaceReferences(nullptr, ObjectsToPrivatize, ObjectsToPrivatizeWithin);
	}

	return true;
}

void FAssetPrivatizeModel::ScanForReferences()
{
	double StartTickSeconds = FPlatformTime::Seconds();

	while (PendingPrivateIndex < PendingPrivateAssets.Num() && (FPlatformTime::Seconds() - StartTickSeconds) < MaxTickSeconds)
	{
		TSharedPtr<FPendingPrivateAsset>& PendingPrivateAsset = PendingPrivateAssets[PendingPrivateIndex];

		PendingPrivateAsset->CheckForIllegalReferences();

		IllegalOnDiskReferences.Append(PendingPrivateAsset->IllegalDiskReferences);

		bIsAnythingReferencedInMemoryByUndo |= PendingPrivateAsset->IsReferencedInMemoryByUndo();
		bIsAnythingReferencedInMemoryByNonUndo |= PendingPrivateAsset->IsReferencedInMemoryByNonUndo();
		PendingPrivateIndex++;
	}

	if (PendingPrivateIndex >= PendingPrivateAssets.Num())
	{
		SetState(Finished);
	}
}

void FAssetPrivatizeModel::Tick(const float InDeltaTime)
{
	switch (State)
	{
	case Waiting:
		break;
	case StartScanning:
		IllegalOnDiskReferences = TSet<FName>();
		bIsAnythingReferencedInMemoryByNonUndo = false;
		bIsAnythingReferencedInMemoryByUndo = false;
		PendingPrivateIndex = 0;
		SetState(Scanning);
		break;
	case Scanning:
		ScanForReferences();
		break;
	case Finished:
		break;
	default:
		break;
	}
}
#undef LOCTEXT_NAMESPACE