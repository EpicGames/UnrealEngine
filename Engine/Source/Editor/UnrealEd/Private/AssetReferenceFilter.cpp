// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/AssetReferenceFilter.h"
#include "AssetRegistry/AssetData.h"

IAssetReferenceFilter::FOnIsCrossPluginReferenceAllowed IAssetReferenceFilter::OnIsCrossPluginReferenceAllowedDelegate;

bool IAssetReferenceFilter::IsCrossPluginReferenceAllowed(const FAssetData& ReferencingAssetData, const FAssetData& ReferencedAssetData)
{
	return OnIsCrossPluginReferenceAllowedDelegate.IsBound() ? OnIsCrossPluginReferenceAllowedDelegate.Execute(ReferencingAssetData, ReferencedAssetData) : false;
}