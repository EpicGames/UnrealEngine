// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSelection.h"
#include "Engine/Level.h"
#include "Model.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UnrealType.h"
#include "GameFramework/Actor.h"
#include "ActorFactories/ActorFactory.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/Pawn.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Engine/Brush.h"
#include "Editor/GroupActor.h"
#include "Animation/SkeletalMeshActor.h"
#include "Particles/Emitter.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Components/DecalComponent.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"

#include "LevelUtils.h"

#include "ComponentAssetBroker.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/CollectionDragDropOp.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SnappingUtils.h"
#include "ActorEditorUtils.h"
#include "LevelEditorSubsystem.h"
#include "LevelEditorViewport.h"
#include "LandscapeProxy.h"
#include "Landscape.h"

#include "Editor/ActorPositioning.h"

#include "ObjectEditorUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelBounds.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Misc/MessageDialog.h"
#include "Subsystems/BrowseToAssetOverrideSubsystem.h"
#include "Subsystems/PlacementSubsystem.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"

namespace AssetSelectionUtils
{
	bool IsClassPlaceable(const UClass* Class)
	{
		const bool bIsAddable =
			Class
			&& !(Class->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Deprecated | CLASS_Abstract))
			&& Class->IsChildOf( AActor::StaticClass() );
		return bIsAddable;
	}

	// Blueprints handle their own Abstract flag, but we should check for deprecation and NotPlaceable here
	bool IsChildBlueprintPlaceable(const UClass* Class)
	{
		const bool bIsAddable =
			Class
			&& !(Class->HasAnyClassFlags(CLASS_NotPlaceable | CLASS_Deprecated))
			&& Class->IsChildOf(AActor::StaticClass());
		return bIsAddable;
	}
	
	void GetSelectedAssets( TArray<FAssetData>& OutSelectedAssets )
	{
		// Add the selection from the content browser module
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().GetSelectedAssets(OutSelectedAssets);
	}

	FSelectedActorInfo BuildSelectedActorInfo( const TArray<AActor*>& SelectedActors)
	{
		FSelectedActorInfo ActorInfo;
		if( SelectedActors.Num() > 0 )
		{
			// Get the class type of the first actor.
			AActor* FirstActor = SelectedActors[0];

			if( FirstActor && !FirstActor->IsTemplate() )
			{
				UBrowseToAssetOverrideSubsystem* BrowseToAssetOverrideSubsystem = UBrowseToAssetOverrideSubsystem::Get();

				UClass* FirstClass = FirstActor->GetClass();
				UObject* FirstArchetype = FirstActor->GetArchetype();

				ActorInfo.bAllSelectedAreBrushes = Cast< ABrush >( FirstActor ) != NULL;
				ActorInfo.SelectionClass = FirstClass;

				// Compare all actor types with the baseline.
				for ( int32 ActorIndex = 0; ActorIndex < SelectedActors.Num(); ++ActorIndex )
				{
					AActor* CurrentActor = SelectedActors[ ActorIndex ];

					if( CurrentActor->IsTemplate() )
					{
						continue;
					}

					ABrush* Brush = Cast< ABrush >( CurrentActor );
					if( !Brush)
					{
						ActorInfo.bAllSelectedAreBrushes = false;
					}
					else
					{
						if( !ActorInfo.bHaveBuilderBrush )
						{
							ActorInfo.bHaveBuilderBrush = FActorEditorUtils::IsABuilderBrush(Brush);
						}
						ActorInfo.bHaveBrush |= true;
						ActorInfo.bHaveBSPBrush |= (!Brush->IsVolumeBrush());
						ActorInfo.bHaveVolume |= Brush->IsVolumeBrush();
					}

					UClass* CurrentClass = CurrentActor->GetClass();
					if( FirstClass != CurrentClass )
					{
						ActorInfo.bAllSelectedActorsOfSameType = false;
						ActorInfo.SelectionClass = NULL;
						FirstClass = NULL;
					}
					else
					{
						ActorInfo.SelectionClass = CurrentActor->GetClass();
					}

					++ActorInfo.NumSelected;

					if( ActorInfo.bAllSelectedActorsBelongToCurrentLevel )
					{
						const ULevel* ActorLevel = CurrentActor->GetLevel();
						if( !ActorLevel || !ActorLevel->IsCurrentLevel() )
						{
							ActorInfo.bAllSelectedActorsBelongToCurrentLevel = false;
						}
					}

					if( ActorInfo.bAllSelectedActorsBelongToSameWorld )
					{
						if ( !ActorInfo.SharedWorld )
						{
							ActorInfo.SharedWorld = CurrentActor->GetWorld();
							check(ActorInfo.SharedWorld);
						}
						else
						{
							if( ActorInfo.SharedWorld != CurrentActor->GetWorld() )
							{
								ActorInfo.bAllSelectedActorsBelongToCurrentLevel = false;
								ActorInfo.SharedWorld = NULL;
							}
						}
					}

					// To prevent move to other level for Landscape if its components are distributed in streaming levels
					if (CurrentActor->IsA(ALandscape::StaticClass()))
					{
						ALandscape* Landscape = CastChecked<ALandscape>(CurrentActor);
						if (!Landscape || !Landscape->HasAllComponent())
						{
							if( !ActorInfo.bAllSelectedActorsBelongToCurrentLevel )
							{
								ActorInfo.bAllSelectedActorsBelongToCurrentLevel = true;
							}
						}
					}

					if ( ActorInfo.bSelectedActorsBelongToSameLevel )
					{
						ULevel* ActorLevel = CurrentActor->GetLevel();
						if ( !ActorInfo.SharedLevel )
						{
							// This is the first selected actor we've encountered.
							ActorInfo.SharedLevel = ActorLevel;
						}
						else
						{
							// Does this actor's level match the others?
							if ( ActorInfo.SharedLevel != ActorLevel )
							{
								ActorInfo.bSelectedActorsBelongToSameLevel = false;
								ActorInfo.SharedLevel = NULL;
							}
						}
					}


					AGroupActor* FoundGroup = Cast<AGroupActor>(CurrentActor);
					if(!FoundGroup)
					{
						FoundGroup = AGroupActor::GetParentForActor(CurrentActor);
					}
					if( FoundGroup )
					{
						if( !ActorInfo.bHaveSelectedSubGroup )
						{
							ActorInfo.bHaveSelectedSubGroup  = AGroupActor::GetParentForActor(FoundGroup) != NULL;
						}
						if( !ActorInfo.bHaveSelectedLockedGroup )
						{
							ActorInfo.bHaveSelectedLockedGroup = FoundGroup->IsLocked();
						}
						if( !ActorInfo.bHaveSelectedUnlockedGroup )
						{
							AGroupActor* FoundRoot = AGroupActor::GetRootForActor(CurrentActor);
							ActorInfo.bHaveSelectedUnlockedGroup = !FoundGroup->IsLocked() || ( FoundRoot && !FoundRoot->IsLocked() );
						}
					}
					else
					{
						++ActorInfo.NumSelectedUngroupedActors;
					}

					USceneComponent* RootComp = CurrentActor->GetRootComponent();
					if(RootComp != nullptr && RootComp->GetAttachParent() != nullptr)
					{
						ActorInfo.bHaveAttachedActor = true;
					}

					for (UActorComponent* Component : CurrentActor->GetComponents())
					{
						if (Component)
						{
							if( UStaticMeshComponent* SMComp = Cast<UStaticMeshComponent>(Component) )
							{
								if( SMComp->IsRegistered() )
								{
									ActorInfo.bHaveStaticMeshComponent = true;
								}
							}

							// Check for experimental/early-access classes in the component hierarchy
							bool bIsExperimental, bIsEarlyAccess;
							FString MostDerivedDevelopmentClassName;
							FObjectEditorUtils::GetClassDevelopmentStatus(Component->GetClass(), bIsExperimental, bIsEarlyAccess, MostDerivedDevelopmentClassName);

							ActorInfo.bHaveExperimentalClass |= bIsExperimental;
							ActorInfo.bHaveEarlyAccessClass |= bIsEarlyAccess;
						}
					}

					// Check for experimental/early-access classes in the actor hierarchy
					{
						bool bIsExperimental, bIsEarlyAccess;
						FString MostDerivedDevelopmentClassName;
						FObjectEditorUtils::GetClassDevelopmentStatus(CurrentClass, bIsExperimental, bIsEarlyAccess, MostDerivedDevelopmentClassName);

						ActorInfo.bHaveExperimentalClass |= bIsExperimental;
						ActorInfo.bHaveEarlyAccessClass |= bIsEarlyAccess;
					}

					if( CurrentActor->IsA( ALight::StaticClass() ) )
					{
						ActorInfo.bHaveLight = true;
					}

					if( CurrentActor->IsA( AStaticMeshActor::StaticClass() ) ) 
					{
						ActorInfo.bHaveStaticMesh = true;
						AStaticMeshActor* StaticMeshActor = CastChecked<AStaticMeshActor>( CurrentActor );
						if ( StaticMeshActor->GetStaticMeshComponent() )
						{
							UStaticMesh* StaticMesh = StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh();

							ActorInfo.bAllSelectedStaticMeshesHaveCollisionModels &= ( (StaticMesh && StaticMesh->GetBodySetup()) ? true : false );
						}
					}

					if( CurrentActor->IsA( ASkeletalMeshActor::StaticClass() ) )
					{
						ActorInfo.bHaveSkeletalMesh = true;
					}

					if( CurrentActor->IsA( APawn::StaticClass() ) )
					{
						ActorInfo.bHavePawn = true;
					}

					if( CurrentActor->IsA( AEmitter::StaticClass() ) )
					{
						ActorInfo.bHaveEmitter = true;
					}

					if ( CurrentActor->IsTemporarilyHiddenInEditor() )
					{
						ActorInfo.bHaveHidden = true;
					}

					if ( CurrentActor->IsA( ALandscapeProxy::StaticClass() ) )
					{
						ActorInfo.bHaveLandscape = true;
					}

					// Find our counterpart actor
					AActor* EditorWorldActor = EditorUtilities::GetEditorWorldCounterpartActor( CurrentActor );
					if( EditorWorldActor != NULL )
					{
						// Just count the total number of actors with counterparts
						++ActorInfo.NumSimulationChanges;
					}

					if (!ActorInfo.bHaveBrowseOverride && BrowseToAssetOverrideSubsystem->HasBrowseToAssetOverride(CurrentActor))
					{
						ActorInfo.bHaveBrowseOverride = true;
					}
				}

				if( ActorInfo.SelectionClass != NULL )
				{
					ActorInfo.SelectionStr = ActorInfo.SelectionClass->GetName();
				}
				else
				{
					ActorInfo.SelectionStr = TEXT("Actor");
				}


			}
		}

		// hack when only BSP is selected
		if( ActorInfo.SharedWorld == nullptr )
		{
			ActorInfo.SharedWorld = GWorld;
		}

		return ActorInfo;
	}

	FSelectedActorInfo GetSelectedActorInfo()
	{
		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects<AActor>( SelectedActors );
		return BuildSelectedActorInfo( SelectedActors );	
	}

	int32 GetNumSelectedSurfaces( UWorld* InWorld )
	{
		int32 NumSelectedSurfs = 0;
		UWorld* World = InWorld;
		if( !World )
		{
			World = GWorld;	// Fallback to GWorld
		}
		if( World )
		{
			const int32 NumLevels = World->GetNumLevels();
			for (int32 LevelIndex = 0; LevelIndex < NumLevels; LevelIndex++)
			{
				ULevel* Level = World->GetLevel(LevelIndex);
				UModel* Model = Level->Model;
				check(Model);
				const int32 NumSurfaces = Model->Surfs.Num();

				// Count the number of selected surfaces
				for (int32 Surface = 0; Surface < NumSurfaces; ++Surface)
				{
					FBspSurf *Poly = &Model->Surfs[Surface];

					if (Poly->PolyFlags & PF_Selected)
					{
						++NumSelectedSurfs;
					}
				}
			}
		}

		return NumSelectedSurfs;
	}

	bool IsAnySurfaceSelected( UWorld* InWorld )
	{
		UWorld* World = InWorld;
		if (!World)
		{
			World = GWorld;	// Fallback to GWorld
		}
		if (World)
		{
			const int32 NumLevels = World->GetNumLevels();
			for (int32 LevelIndex = 0; LevelIndex < NumLevels; LevelIndex++)
			{
				ULevel* Level = World->GetLevel(LevelIndex);
				UModel* Model = Level->Model;
				check(Model);
				const int32 NumSurfaces = Model->Surfs.Num();

				// Count the number of selected surfaces
				for (int32 Surface = 0; Surface < NumSurfaces; ++Surface)
				{
					FBspSurf *Poly = &Model->Surfs[Surface];

					if (Poly->PolyFlags & PF_Selected)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	bool IsBuilderBrushSelected()
	{
		bool bHasBuilderBrushSelected = false;

		for( FSelectionIterator SelectionIter = GEditor->GetSelectedActorIterator(); SelectionIter; ++SelectionIter )
		{
			AActor* Actor = Cast< AActor >(*SelectionIter);
			if( Actor && FActorEditorUtils::IsABuilderBrush( Actor ) )
			{
				bHasBuilderBrushSelected = true;
				break;
			}
		}

		return bHasBuilderBrushSelected;
	}
}

namespace ActorPlacementUtils
{
	bool IsLevelValidForActorPlacement(ULevel* InLevel, TArray<FTransform>& InActorTransforms)
	{
		if (FLevelUtils::IsLevelLocked(InLevel))
		{
			FNotificationInfo Info(NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevel", "The requested operation could not be completed because the level is locked."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return false;
		}
		if (InLevel && InLevel->OwningWorld && InLevel->OwningWorld->GetStreamingLevels().Num() == 0)
		{
			// if this is the only level
			return true;
		}
		if (GIsRunningUnattendedScript)
		{
			// Don't prompt user for checks in unattended mode
			return true;
		}
		if (InLevel && InLevel->GetPromptWhenAddingToLevelBeforeCheckout() && SourceControlHelpers::IsAvailable())
		{
			FString FileName = SourceControlHelpers::PackageFilename(InLevel->GetPathName());
			// Query file state also checks the source control status
			FSourceControlStatePtr SCState = ISourceControlModule::Get().GetProvider().GetState(FileName, EStateCacheUsage::Use);
			if (!(SCState->IsCheckedOut() || SCState->IsAdded() || SCState->CanAdd() || SCState->IsUnknown()))
			{
				if (EAppReturnType::Ok != FMessageDialog::Open(EAppMsgType::OkCancel, NSLOCTEXT("UnrealEd","LevelNotCheckedOutMsg", "This actor will be placed in a level that is in revision control but not currently checked out. Continue?"), NSLOCTEXT("UnrealEd", "LevelCheckout_Title", "Level Checkout Warning")))
				{
					return false;
				}
				else
				{
					InLevel->bPromptWhenAddingToLevelBeforeCheckout = false;
				}
			}
		}
		if (InLevel && InLevel->OwningWorld)
		{
			int32 LevelCount = InLevel->OwningWorld->GetStreamingLevels().Num();
			int32 NumLockedLevels = 0;
			// Check for streaming level count b/c we know there is > 1 streaming level
			for (ULevelStreaming* StreamingLevel : InLevel->OwningWorld->GetStreamingLevels())
			{
				StreamingLevel->bLocked ? NumLockedLevels++ : 0;
			}
			// If there is only one unlocked level, a) ours is the unlocked level b/c of the previous IsLevelLocked test and b) we shouldn't try to check for level bounds on the next test
			if (LevelCount - NumLockedLevels == 1)
			{
				return true;
			}
		}
		if (InLevel && InLevel->GetPromptWhenAddingToLevelOutsideBounds())
		{
			FBox CurrentLevelBounds(ForceInit);
			if (InLevel->LevelBoundsActor.IsValid())
			{
				CurrentLevelBounds = InLevel->LevelBoundsActor.Get()->GetComponentsBoundingBox();
			}
			else
			{
				CurrentLevelBounds = ALevelBounds::CalculateLevelBounds(InLevel);
			}

			FVector BoundsExtent = CurrentLevelBounds.GetExtent();
			if (BoundsExtent.X < GetDefault<ULevelEditorMiscSettings>()->MinimumBoundsForCheckingSize.X
				&& BoundsExtent.Y < GetDefault<ULevelEditorMiscSettings>()->MinimumBoundsForCheckingSize.Y
				&& BoundsExtent.Z < GetDefault<ULevelEditorMiscSettings>()->MinimumBoundsForCheckingSize.Z)
			{
				return true;
			}
			CurrentLevelBounds = CurrentLevelBounds.ExpandBy(BoundsExtent * (GetDefault<ULevelEditorMiscSettings>()->PercentageThresholdForPrompt / 100.0f));
			for (int32 ActorTransformIndex = 0; ActorTransformIndex < InActorTransforms.Num(); ++ActorTransformIndex)
			{
				FTransform ActorTransform = InActorTransforms[ActorTransformIndex];
				if (!CurrentLevelBounds.IsInsideOrOn(ActorTransform.GetLocation()))
				{
					if (EAppReturnType::Ok != FMessageDialog::Open(EAppMsgType::OkCancel, NSLOCTEXT("UnrealEd", "LevelBoundsMsg", "The actor will be placed outside the bounds of the current level. Continue?"), NSLOCTEXT("UnrealEd", "ActorPlacement_Title", "Actor Placement Warning")))
					{
						return false;
					}
					else
					{
						InLevel->bPromptWhenAddingToLevelOutsideBounds = false;
						break;
					}
				}
			}
		}
		return true;
	}
}

namespace AssetSelectionLocals {

UTypedElementSelectionSet* GetEditorSelectionSet()
{
	if (!GEditor)
	{
		return nullptr;
	}
	
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		if (FEditorModeTools* ModeManager = LevelEditorSubsystem->GetLevelEditorModeManager())
		{
			return ModeManager->GetEditorSelectionSet();
		}
	}

	return nullptr;
}

void ForEachObjectInHandles(TArray<FTypedElementHandle> Handles, TFunctionRef<void(UObject&)> Func)
{
	for (FTypedElementHandle Handle : Handles)
	{
		TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(Handle);
		if (!ObjectInterface)
		{
			continue;
		}

		UObject* Object = ObjectInterface.GetObject();
		if (!Object)
		{
			continue;
		}

		Func(*Object);
	}
}

/**
 * Creates an object using the specified factory.
 */
TArray<FTypedElementHandle> PlaceAssetUsingFactory(UObject* Asset, TScriptInterface<IAssetFactoryInterface> Factory, bool bSelectResult = true, EObjectFlags ObjectFlags = RF_Transactional, const FName Name = NAME_None)
{
	TArray<FTypedElementHandle> PlacedItems;
	if (!Factory)
	{
		return PlacedItems;
	}

	// Whereas going throught UPlacementSubsystem does not require the factory to be an actor factory,
	// other legacy paths require actor factories. These two pointers, when non-null, can be used for 
	// those paths.
	UActorFactory* ActorFactory = Cast<UActorFactory>(Factory.GetObject());
	AActor* NewActorTemplate = ActorFactory ? ActorFactory->GetDefaultActor(Asset) : nullptr;

	UWorld* OldWorld = nullptr;

	// The play world needs to be selected if it exists
	if (GIsEditor && GEditor->PlayWorld && !GIsPlayInEditorWorld)
	{
		OldWorld = SetPlayInEditorWorld(GEditor->PlayWorld);
	}

	// For Brushes/Volumes, use the default brush as the template rather than the factory default actor
	if (NewActorTemplate && NewActorTemplate->IsA(ABrush::StaticClass()) && GWorld->GetDefaultBrush() != nullptr)
	{
		NewActorTemplate = GWorld->GetDefaultBrush();
	}

	// TODO: FSnappedPositioningData should probably not require the use of an actor factory
	const FSnappedPositioningData PositioningData = FSnappedPositioningData(GCurrentLevelEditingViewportClient, GEditor->ClickLocation, GEditor->ClickPlane)
		.UseFactory(ActorFactory)
		.UsePlacementExtent(NewActorTemplate ? NewActorTemplate->GetPlacementExtent() : FVector3d::Zero());

	FTransform ActorTransform = FActorPositioning::GetSnappedSurfaceAlignedTransform(PositioningData);

	if (NewActorTemplate && GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled)
	{
		// HACK: If we are aligning rotation to surfaces, we have to factor in the inverse of the actor's rotation and translation so that the resulting transform after SpawnActor is correct.
		
		// TODO: Do this for non-actor placeable objects

		if (auto* RootComponent = NewActorTemplate->GetRootComponent())
		{
			RootComponent->UpdateComponentToWorld();
		}

		FVector OrigActorScale3D = ActorTransform.GetScale3D();
		ActorTransform = NewActorTemplate->GetTransform().Inverse() * ActorTransform;
		ActorTransform.SetScale3D(OrigActorScale3D);
	}

	// Do not fade snapping indicators over time if the viewport is not realtime
	bool bClearImmediately = !GCurrentLevelEditingViewportClient || !GCurrentLevelEditingViewportClient->IsRealtime();
	FSnappingUtils::ClearSnappingHelpers(bClearImmediately);

	ULevel* DesiredLevel = GWorld->GetCurrentLevel();

	bool bShouldSpawnObject = true;

	if ((ObjectFlags & RF_Transactional) != 0)
	{
		TArray<FTransform> SpawningActorTransforms;
		SpawningActorTransforms.Add(ActorTransform);
		// TODO: At the moment, the conditions for placing non-actor items in a level are the same as actor items, but
		// we should probably rename this function so it is not actor-specific.
		bShouldSpawnObject = ActorPlacementUtils::IsLevelValidForActorPlacement(DesiredLevel, SpawningActorTransforms);
	}

	if (bShouldSpawnObject)
	{
		FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "PlaceObject", "Place Object"), (ObjectFlags & RF_Transactional) != 0);

		// Create the object.
		UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>();
		if (PlacementSubsystem)
		{
			FAssetPlacementInfo PlacementInfo;
			PlacementInfo.AssetToPlace = FAssetData(Asset);
			PlacementInfo.PreferredLevel = DesiredLevel;
			PlacementInfo.NameOverride = Name;
			PlacementInfo.FinalizedTransform = ActorTransform;
			PlacementInfo.FactoryOverride = Factory;

			FPlacementOptions PlacementOptions;
			PlacementOptions.bIsCreatingPreviewElements = FLevelEditorViewportClient::IsDroppingPreviewActor();

			PlacedItems = PlacementSubsystem->PlaceAsset(PlacementInfo, PlacementOptions);

			ForEachObjectInHandles(PlacedItems, [ObjectFlags](UObject& PlacedObject)
			{
				PlacedObject.SetFlags(ObjectFlags);
			});
		}

		// If we fail to place using the placement subsystem above for some reason, we keep this legacy path that 
		// tries using an actor factory directly. We don't bother adding a fallback for non-actor factories, as
		// those got introduced after the existence of the placement subsystem, and should rely on it.
		if (!PlacedItems.Num() && ActorFactory)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.ObjectFlags = ObjectFlags;
			SpawnParams.Name = Name;
			SpawnParams.bTemporaryEditorActor = FLevelEditorViewportClient::IsDroppingPreviewActor();

			if (AActor* PlacedActor = ActorFactory->CreateActor(Asset, DesiredLevel, ActorTransform, SpawnParams))
			{
				FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(PlacedActor);
				if (ensure(Handle))
				{
					PlacedItems.Add(Handle);
				}
			}
		}
	}

	if (PlacedItems.Num())
	{
		if (bSelectResult)
		{
			// TODO: It would be nice not to use this old form of selection clearing, but it has the benefit
			// of clearing up legacy bsp selection as well...
			GEditor->SelectNone(false, true);

			UTypedElementSelectionSet* SelectionSet = GetEditorSelectionSet();
			if (ensure(SelectionSet))
			{
				FTypedElementSelectionOptions SelectionOptions;
				SelectionSet->SelectElements(PlacedItems, SelectionOptions);
			}
		}

		ForEachObjectInHandles(PlacedItems, [](UObject& PlacedObject)
		{
			if (AActor* Actor = Cast<AActor>(&PlacedObject))
			{
				Actor->InvalidateLightingCache();
			}

			PlacedObject.PostEditChange();
			PlacedObject.MarkPackageDirty();
		});

		GEditor->RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
	}

	// Restore the old world if there was one
	if (OldWorld)
	{
		RestoreEditorWorld(OldWorld);
	}

	return PlacedItems;
}

/**
 * Helper to pull out a single actor from an array of typed element handles. Used to convert
 * output in some legacy paths.
 */
AActor* GetActorFromTypedElementHandles(const TArray<FTypedElementHandle>& Handles)
{
	for (const FTypedElementHandle& Handle : Handles)
	{
		TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(Handle);
		if (!ObjectInterface)
		{
			continue;
		}

		if (AActor* Actor = ObjectInterface.GetObjectAs<AActor>())
		{
			return Actor;
		}
	}
	return nullptr;
}
}//end namespace AssetSelectionLocals


namespace AssetUtil
{
	TArray<FAssetData> ExtractAssetDataFromDrag(const FDragDropEvent &DragDropEvent)
	{
		return ExtractAssetDataFromDrag(DragDropEvent.GetOperation());
	}

	TArray<FAssetData> ExtractAssetDataFromDrag(const TSharedPtr<const FDragDropOperation>& Operation)
	{
		TArray<FAssetData> DroppedAssetData;

		if (!Operation.IsValid())
		{
			return DroppedAssetData;
		}

		if (Operation->IsOfType<FExternalDragOperation>())
		{
			TSharedPtr<const FExternalDragOperation> DragDropOp = StaticCastSharedPtr<const FExternalDragOperation>(Operation);
			if (DragDropOp->HasText())
			{
				TArray<FString> DroppedAssetStrings;
				const TCHAR AssetDelimiter[] = { AssetMarshalDefs::AssetDelimiter, TEXT('\0') };
				DragDropOp->GetText().ParseIntoArray(DroppedAssetStrings, AssetDelimiter, true);

				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
				IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

				for (const FString& DroppedAssetString : DroppedAssetStrings)
				{
					if (DroppedAssetString.Len() < NAME_SIZE && FName::IsValidXName(DroppedAssetString, INVALID_OBJECTPATH_CHARACTERS))
					{
						FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(DroppedAssetString));
						if (AssetData.IsValid())
						{
							DroppedAssetData.Add(AssetData);
						}
					}
				}
			}
		}
		else if (Operation->IsOfType<FCollectionDragDropOp>())
		{
			TSharedPtr<const FCollectionDragDropOp> DragDropOp = StaticCastSharedPtr<const FCollectionDragDropOp>(Operation);
			DroppedAssetData.Append(DragDropOp->GetAssets());
		}
		else if (Operation->IsOfType<FAssetDragDropOp>())
		{
			TSharedPtr<const FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<const FAssetDragDropOp>(Operation);

			if (!DragDropOp->RefreshedAssetData.IsSet())
			{
				TArray<FAssetData> Refreshed;
				if (IAssetRegistry* Registry = IAssetRegistry::Get())
				{
					for (const FAssetData& OriginalData : DragDropOp->GetAssets())
					{
						FAssetData RegistryData = Registry->GetAssetByObjectPath(OriginalData.GetSoftObjectPath());
						Refreshed.Add(RegistryData.IsValid() ? MoveTemp(RegistryData) : OriginalData);
					}
				}
				else
				{
					Refreshed = DragDropOp->GetAssets();
				}
				DragDropOp->RefreshedAssetData = MoveTemp(Refreshed);
			}

			DroppedAssetData.Append(DragDropOp->RefreshedAssetData.GetValue());
		}

		return DroppedAssetData;
	}

	FReply CanHandleAssetDrag( const FDragDropEvent &DragDropEvent )
	{
		TArray<FAssetData> DroppedAssetData = ExtractAssetDataFromDrag(DragDropEvent);
		for ( auto AssetIt = DroppedAssetData.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			const FAssetData& AssetData = *AssetIt;

			if ( AssetData.IsValid() && FComponentAssetBrokerage::GetPrimaryComponentForAsset( AssetData.GetClass() ) )
			{
				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}
}



/* ==========================================================================================================
FActorFactoryAssetProxy
========================================================================================================== */

void FActorFactoryAssetProxy::GenerateActorFactoryMenuItems( const FAssetData& AssetData, TArray<FMenuItem>* OutMenuItems, bool ExcludeStandAloneFactories  )
{
	FText UnusedErrorMessage;
	const FAssetData NoAssetData {};
	for ( int32 FactoryIdx = 0; FactoryIdx < GEditor->ActorFactories.Num(); FactoryIdx++ )
	{
		UActorFactory* Factory = GEditor->ActorFactories[FactoryIdx];

		const bool FactoryWorksWithoutAsset = Factory->CanCreateActorFrom( NoAssetData, UnusedErrorMessage );
		const bool FactoryWorksWithAsset = AssetData.IsValid() && Factory->CanCreateActorFrom( AssetData, UnusedErrorMessage );
		const bool FactoryWorks = FactoryWorksWithAsset || FactoryWorksWithoutAsset;

		if ( FactoryWorks )
		{
			FMenuItem MenuItem = FMenuItem( Factory, NoAssetData );
			if ( FactoryWorksWithAsset )
			{
				MenuItem = FMenuItem( Factory, AssetData );
			}

			if ( FactoryWorksWithAsset || ( !ExcludeStandAloneFactories && FactoryWorksWithoutAsset ) )
			{
				OutMenuItems->Add( MenuItem );
			}
		}
	}
}

/**
* Find the appropriate actor factory for an asset by type.
*
* @param	AssetData			contains information about an asset that to get a factory for
* @param	bRequireValidObject	indicates whether a valid asset object is required.  specify false to allow the asset
*								class's CDO to be used in place of the asset if no asset is part of the drag-n-drop
*
* @return	the factory that is responsible for creating actors for the specified asset type.
*/
UActorFactory* FActorFactoryAssetProxy::GetFactoryForAsset( const FAssetData& AssetData, bool bRequireValidObject/*=false*/ )
{
	UObject* Asset = NULL;
	UClass* AssetClass = AssetData.GetClass();
	
	if ( AssetData.IsAssetLoaded() )
	{
		Asset = AssetData.GetAsset();
	}
	else if ( !bRequireValidObject && AssetClass )
	{
		Asset = AssetClass->GetDefaultObject();
	}

	return FActorFactoryAssetProxy::GetFactoryForAssetObject( Asset );
}

/**
* Find the appropriate actor factory for an asset.
*
* @param	AssetObj	The asset that to find the appropriate actor factory for
*
* @return	The factory that is responsible for creating actors for the specified asset
*/
UActorFactory* FActorFactoryAssetProxy::GetFactoryForAssetObject( UObject* AssetObj )
{
	UActorFactory* Result = NULL;

	// Attempt to find a factory that is capable of creating the asset
	const TArray< UActorFactory *>& ActorFactories = GEditor->ActorFactories;
	FText UnusedErrorMessage;
	FAssetData AssetData( AssetObj );
	for ( int32 FactoryIdx = 0; Result == NULL && FactoryIdx < ActorFactories.Num(); ++FactoryIdx )
	{
		UActorFactory* ActorFactory = ActorFactories[FactoryIdx];
		// Check if the actor can be created using this factory, making sure to check for an asset to be assigned from the selector
		if ( ActorFactory->CanCreateActorFrom( AssetData, UnusedErrorMessage ) )
		{
			Result = ActorFactory;
		}
	}

	return Result;
}

AActor* FActorFactoryAssetProxy::AddActorForAsset( UObject* AssetObj, bool bSelectActor, EObjectFlags ObjectFlags, 
	UActorFactory* FactoryToUse /*= NULL*/, const FName Name )
{
	UE::AssetPlacementUtil::FExtraPlaceAssetOptions Options;
	Options.bSelectOutput = bSelectActor;
	Options.ObjectFlags = ObjectFlags;
	Options.FactoryToUse = FactoryToUse;
	Options.Name = Name;

	TArray<FTypedElementHandle> PlacedItems = UE::AssetPlacementUtil::PlaceAssetInCurrentLevel(AssetObj, Options);

	AActor* Actor = AssetSelectionLocals::GetActorFromTypedElementHandles(PlacedItems);

	ensureMsgf(Actor || PlacedItems.Num() == 0, TEXT("FActorFactoryAssetProxy::AddActorForAsset produced an object, "
		"but not an actor. Use UE::AssetFactoryUtils::AddObjectForAssetToCurrentLevel instead to use the result."));

	return Actor;
}

AActor* FActorFactoryAssetProxy::AddActorFromSelection( UClass* ActorClass, const FVector* ActorLocation, bool SelectActor, EObjectFlags ObjectFlags, UActorFactory* ActorFactory, const FName Name )
{
	using namespace AssetSelectionLocals;

	check( ActorClass != NULL );

	if( !ActorFactory )
	{
		// Look for an actor factory capable of creating actors of the actors type.
		ActorFactory = GEditor->FindActorFactoryForActorClass( ActorClass );
	}

	AActor* Result = NULL;
	FText ErrorMessage;

	if ( ActorFactory )
	{
		UObject* TargetObject = GEditor->GetSelectedObjects()->GetTop<UObject>();

		if( TargetObject && ActorFactory->CanCreateActorFrom( FAssetData(TargetObject), ErrorMessage ) )
		{
			// Attempt to add the actor
			TArray<FTypedElementHandle> PlacedItems = PlaceAssetUsingFactory(TargetObject, ActorFactory, SelectActor, ObjectFlags);
			Result = GetActorFromTypedElementHandles(PlacedItems);

			ensureMsgf(Result || PlacedItems.Num() == 0, TEXT("FActorFactoryAssetProxy::AddActorFromSelection produced a "
				"result, but not an actor."));
		}
	}

	return Result;
}

/**
* Determines if the provided actor is capable of having a material applied to it.
*
* @param	TargetActor	Actor to check for the validity of material application
*
* @return	true if the actor is valid for material application; false otherwise
*/
bool FActorFactoryAssetProxy::IsActorValidForMaterialApplication( AActor* TargetActor )
{
	bool bIsValid = false;

	//@TODO: PAPER2D: Extend this to support non mesh components (or make sprites a mesh component)

	// Check if the actor has a mesh or fog volume density. If so, it can likely have
	// a material applied to it. Otherwise, it cannot.
	if ( TargetActor )
	{
		for (UActorComponent* Component : TargetActor->GetComponents())
		{
			if (Cast<UMeshComponent>(Component))
			{
				bIsValid = true;
				break;
			}
		}
	}

	return bIsValid;
}
/**
* Attempts to apply the material to the specified actor.
*
* @param	TargetActor		the actor to apply the material to
* @param	MaterialToApply	the material to apply to the actor
* @param    OptionalMaterialSlot the material slot to apply to.
*
* @return	true if the material was successfully applied to the actor
*/
bool FActorFactoryAssetProxy::ApplyMaterialToActor( AActor* TargetActor, UMaterialInterface* MaterialToApply, int32 OptionalMaterialSlot )
{
	bool bResult = false;

	if ( TargetActor != NULL && MaterialToApply != NULL )
	{
		ALandscapeProxy* Landscape = Cast<ALandscapeProxy>(TargetActor);
		if (Landscape != NULL)
		{
			FProperty* MaterialProperty = FindFProperty<FProperty>(ALandscapeProxy::StaticClass(), "LandscapeMaterial");
			Landscape->PreEditChange(MaterialProperty);
			Landscape->LandscapeMaterial = MaterialToApply;
			FPropertyChangedEvent PropertyChangedEvent(MaterialProperty);
			Landscape->PostEditChangeProperty(PropertyChangedEvent);
			bResult = true;
		}
		else
		{
			TArray<UActorComponent*> EditableComponents;
			FActorEditorUtils::GetEditableComponents( TargetActor, EditableComponents );

			// Some actors could potentially have multiple mesh components, so we need to store all of the potentially valid ones
			// (or else perform special cases with IsA checks on the target actor)
			TArray<USceneComponent*> FoundMeshComponents;

			// Find which mesh the user clicked on first.
			for (UActorComponent* Component : TargetActor->GetComponents())
			{
				USceneComponent* SceneComp = Cast<USceneComponent>(Component);
				
				// Only apply the material to editable components.  Components which are not exposed are not intended to be changed.
				if (SceneComp && EditableComponents.Contains( SceneComp ) )
				{
					UMeshComponent* MeshComponent = Cast<UMeshComponent>(SceneComp);

					if((MeshComponent && MeshComponent->IsRegistered()) ||
						SceneComp->IsA<UDecalComponent>())
					{
						// Intentionally do not break the loop here, as there could be potentially multiple mesh components
						FoundMeshComponents.AddUnique( SceneComp );
					}
				}
			}

			if ( FoundMeshComponents.Num() > 0 )
			{
				// Check each component that was found
				for ( TArray<USceneComponent*>::TConstIterator MeshCompIter( FoundMeshComponents ); MeshCompIter; ++MeshCompIter )
				{
					USceneComponent* SceneComp = *MeshCompIter;
					bResult = FComponentEditorUtils::AttemptApplyMaterialToComponent(SceneComp, MaterialToApply, OptionalMaterialSlot);
				}
			}
		}
	}


	return bResult;
}


// AssetPlacementUtil

TArray<FTypedElementHandle> UE::AssetPlacementUtil::PlaceAssetInCurrentLevel(UObject* AssetObj, const FExtraPlaceAssetOptions& ExtraParms)
{
	using namespace AssetSelectionLocals;

	if (!AssetObj)
	{
		return TArray<FTypedElementHandle>();
	}

	const FAssetData AssetData(AssetObj, FAssetData::ECreationFlags::AllowBlueprintClass);

	if (!ExtraParms.FactoryToUse)
	{
		UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>();
		if (!PlacementSubsystem)
		{
			return TArray<FTypedElementHandle>();
		}
		TScriptInterface<IAssetFactoryInterface> AssetFactory = PlacementSubsystem->FindAssetFactoryFromAssetData(AssetData);
		return PlaceAssetUsingFactory(AssetObj, AssetFactory, ExtraParms.bSelectOutput, ExtraParms.ObjectFlags, ExtraParms.Name);
	}

	if (!ExtraParms.FactoryToUse->CanPlaceElementsFromAssetData(AssetData))
	{
		return TArray<FTypedElementHandle>();
	}

	return PlaceAssetUsingFactory(AssetObj, ExtraParms.FactoryToUse, ExtraParms.bSelectOutput, ExtraParms.ObjectFlags, ExtraParms.Name);
}
