// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetThumbnail.h"
#include "AssetThumbnailToolTip.h"
#include "AssetDefinitionRegistry.h"
#include "AssetStatusAssetDataInfoProvider.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"
#include "Engine/GameViewportClient.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Modules/ModuleManager.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Textures/SlateTextureData.h"
#include "Fonts/SlateFontInfo.h"
#include "Application/ThrottleManager.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SViewport.h"
#include "Styling/AppStyle.h"
#include "RenderingThread.h"
#include "Settings/ContentBrowserSettings.h"
#include "RenderUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Slate/SlateTextures.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ShaderCompiler.h"
#include "AssetCompilingManager.h"
#include "AssetDefinitionRegistry.h"
#include "AssetDefinitionAssetInfo.h"
#include "AssetDefinitionDefault.h"
#include "IAssetTools.h"
#include "AssetTypeActions_Base.h"
#include "AssetToolsModule.h"
#include "Styling/SlateIconFinder.h"
#include "ClassIconFinder.h"
#include "IAssetSystemInfoProvider.h"
#include "IVREditorModule.h"
#include "SAssetThumbnailEditModeTools.h"
#include "SDocumentationToolTip.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "ContentBrowserUtils.h"
#include "Widgets/Colors/SColorBlock.h"

#include "Misc/ObjectThumbnail.inl"

DEFINE_LOG_CATEGORY_STATIC(LogThumbnails, Log, All);

namespace AssetThumbnailPool
{
	const FObjectThumbnail* LoadThumbnailsFromPackage(const FAssetData& AssetData, FThumbnailMap& OutThumbnailMap)
	{
		FString PackageFilename;
		if (FPackageName::DoesPackageExist(AssetData.PackageName.ToString(), &PackageFilename))
		{
			const FName ObjectFullName = FName(*AssetData.GetFullName());
			TSet<FName> ObjectFullNames;
			ObjectFullNames.Add(ObjectFullName);

			ThumbnailTools::LoadThumbnailsFromPackage(PackageFilename, ObjectFullNames, OutThumbnailMap);
			return OutThumbnailMap.Find(ObjectFullName);
		}
		return nullptr;
	};
}

FName FAssetThumbnailPool::CustomThumbnailTagName = "CustomThumbnail";

template <>
struct TWidgetTypeTraits<class SAssetThumbnail>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SAssetThumbnail : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAssetThumbnail )
		: _Style("AssetThumbnail")
		, _ThumbnailPool(nullptr)
		, _AllowFadeIn(false)
		, _ForceGenericThumbnail(false)
		, _AllowHintText(true)
		, _AllowAssetSpecificThumbnailOverlay(false)
		, _AllowRealTimeOnHovered(true)
		, _Label(EThumbnailLabel::ClassName)
		, _HighlightedText(FText::GetEmpty())
		, _HintColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
		, _ClassThumbnailBrushOverride(NAME_None)
		, _AssetTypeColorOverride()
		, _Padding(0)
		, _BorderPadding(FMargin(2.f))
		, _GenericThumbnailSize(64)
		, _AllowAssetStatusThumbnailOverlay(false)
		, _AdditionalTooltipInSmallView(SNullWidget::NullWidget)
		, _ShowAssetColor(false)
		, _AssetBorderImageOverride()
		, _ShowAssetBorder(false)
		, _AlwaysExpandTooltip(false)
		, _ColorStripOrientation(EThumbnailColorStripOrientation::HorizontalBottomEdge)
		{}

		SLATE_ARGUMENT(FName, Style)
		SLATE_ARGUMENT(TSharedPtr<FAssetThumbnail>, AssetThumbnail)
		SLATE_ARGUMENT(TSharedPtr<FAssetThumbnailPool>, ThumbnailPool)
		SLATE_ARGUMENT(bool, AllowFadeIn)
		SLATE_ARGUMENT(bool, ForceGenericThumbnail)
		SLATE_ARGUMENT(bool, AllowHintText)
		SLATE_ATTRIBUTE(bool, AllowAssetSpecificThumbnailOverlay)
		SLATE_ATTRIBUTE(bool, AllowAssetSpecificThumbnailOverlayIndicator)
		SLATE_ARGUMENT(bool, AllowRealTimeOnHovered)
		SLATE_ARGUMENT(EThumbnailLabel::Type, Label)
		SLATE_ATTRIBUTE(FText, HighlightedText)
		SLATE_ATTRIBUTE(FLinearColor, HintColorAndOpacity)
		SLATE_ARGUMENT(FName, ClassThumbnailBrushOverride)
		SLATE_ARGUMENT(TOptional<FLinearColor>, AssetTypeColorOverride)
		SLATE_ARGUMENT(FMargin, Padding)
		SLATE_ATTRIBUTE(FMargin, BorderPadding)
		SLATE_ATTRIBUTE(int32, GenericThumbnailSize)
		SLATE_ARGUMENT(TSharedPtr<IAssetSystemInfoProvider>, AssetSystemInfoProvider)
		SLATE_ATTRIBUTE(bool, AllowAssetStatusThumbnailOverlay)
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, AdditionalTooltipInSmallView)
		SLATE_ATTRIBUTE(bool, ShowAssetColor)
		SLATE_ATTRIBUTE(const FSlateBrush*, AssetBorderImageOverride)
		SLATE_ARGUMENT(bool, ShowAssetBorder)
		SLATE_ARGUMENT(bool, CanDisplayEditModePrimitiveTools)
		SLATE_ATTRIBUTE(EVisibility, IsEditModeVisible)
		SLATE_ATTRIBUTE(bool, AlwaysExpandTooltip)
		SLATE_ARGUMENT(EThumbnailColorStripOrientation, ColorStripOrientation)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		Style = InArgs._Style;
		HighlightedText = InArgs._HighlightedText;
		Label = InArgs._Label;
		HintColorAndOpacity = InArgs._HintColorAndOpacity;
		bAllowHintText = InArgs._AllowHintText;
		bAllowRealTimeOnHovered = InArgs._AllowRealTimeOnHovered;
		ThumbnailBrush = nullptr;
		ClassIconBrush = nullptr;
		AssetThumbnail = InArgs._AssetThumbnail;
		bHasRenderedThumbnail = false;
		WidthLastFrame = 0;
		GenericThumbnailBorderPadding = 2.f;
		GenericThumbnailSize = InArgs._GenericThumbnailSize;
		ColorStripOrientation = InArgs._ColorStripOrientation;
		AssetSystemInfoProvider = InArgs._AssetSystemInfoProvider;
		AllowAssetSpecificThumbnailOverlay = InArgs._AllowAssetSpecificThumbnailOverlay;
		AllowAssetSpecificThumbnailOverlayIndicator = InArgs._AllowAssetSpecificThumbnailOverlayIndicator;
		AllowAssetStatusThumbnailOverlay = InArgs._AllowAssetStatusThumbnailOverlay;
		AssetBorderImageOverride = InArgs._AssetBorderImageOverride;
		ShowAssetColor = InArgs._ShowAssetColor;
		EditModeVisibility = InArgs._IsEditModeVisible;
		AlwaysExpandTooltip = InArgs._AlwaysExpandTooltip;
		AdditionalTooltipInSmallView = InArgs._AdditionalTooltipInSmallView;
		BorderPadding = InArgs._BorderPadding;
		AssetThumbnail->OnAssetDataChanged().AddSP(this, &SAssetThumbnail::OnAssetDataChanged);
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();

		UClass* Class = FindObjectSafe<UClass>(AssetData.AssetClassPath);
		static FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TSharedPtr<IAssetTypeActions> AssetTypeActions;
		if ( Class != NULL )
		{
			AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Class).Pin();
		}

		AssetTypeColorOverride = InArgs._AssetTypeColorOverride;
		AssetColor = FLinearColor::White;
		if( AssetTypeColorOverride.IsSet() )
		{
			AssetColor = AssetTypeColorOverride.GetValue();
		}
		else if ( AssetTypeActions.IsValid() )
		{
			AssetColor = AssetTypeActions->GetTypeColor();
		}

		TSharedRef<SOverlay> OverlayWidget = SNew(SOverlay);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Set our tooltip - this will refresh each time it's opened to make sure it's up-to-date
			SetToolTip(SNew(SAssetThumbnailToolTip)
				.AssetThumbnail(SharedThis(this))
				.AlwaysExpandTooltip(AlwaysExpandTooltip));
		}

		UpdateThumbnailClass(AssetTypeActions.Get());

		ClassThumbnailBrushOverride = InArgs._ClassThumbnailBrushOverride;

		AssetBackgroundBrushName = *(Style.ToString() + TEXT(".AssetBackground"));
		ClassBackgroundBrushName = *(Style.ToString() + TEXT(".ClassBackground"));

		// The generic representation of the thumbnail, for use before the rendered version, if it exists
		OverlayWidget->AddSlot()
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		.Padding(InArgs._Padding)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		[
			SAssignNew(AssetBackgroundWidget, SBorder)
			.BorderImage(GetAssetBackgroundBrush())
			.Padding(GenericThumbnailBorderPadding)		
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Visibility(this, &SAssetThumbnail::GetGenericThumbnailVisibility)
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SAssignNew(GenericLabelTextBlock, STextBlock)
					.Text(GetLabelText())
					.Font(GetTextFont())
					.Justification(ETextJustify::Center)
					.ColorAndOpacity(FAppStyle::GetColor(Style, ".ColorAndOpacity"))
					.HighlightText(HighlightedText)
				]

				+SOverlay::Slot()
				[
					SAssignNew(GenericThumbnailImage, SImage)
					.DesiredSizeOverride(this, &SAssetThumbnail::GetGenericThumbnailDesiredSize)
					.Image(this, &SAssetThumbnail::GetClassThumbnailBrush)
				]
			]
		];

		if ( InArgs._ThumbnailPool.IsValid() && !InArgs._ForceGenericThumbnail )
		{
			ViewportFadeAnimation = FCurveSequence();
			ViewportFadeCurve = ViewportFadeAnimation.AddCurve(0.f, 0.25f, ECurveEaseFunction::QuadOut);

			TSharedPtr<SViewport> Viewport = 
				SNew( SViewport )
				.EnableGammaCorrection(false)
PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO: XR Creative Framework relevant?
				// In VR editor every widget is in the world and gamma corrected by the scene renderer.  Thumbnails will have already been gamma
				// corrected and so they need to be reversed
				.ReverseGammaCorrection(IVREditorModule::Get().IsVREditorModeActive())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				.EnableBlending(true)
				.ViewportSize(AssetThumbnail->GetSize());

			Viewport->SetViewportInterface( AssetThumbnail.ToSharedRef() );
			AssetThumbnail->GetViewportRenderTargetTexture(); // Access the render texture to push it on the stack if it isn't already rendered

			InArgs._ThumbnailPool->OnThumbnailRendered().AddSP(this, &SAssetThumbnail::OnThumbnailRendered);
			InArgs._ThumbnailPool->OnThumbnailRenderFailed().AddSP(this, &SAssetThumbnail::OnThumbnailRenderFailed);

			if ( ShouldRender() && (!InArgs._AllowFadeIn || InArgs._ThumbnailPool->IsRendered(AssetThumbnail)) )
			{
				bHasRenderedThumbnail = true;
				ViewportFadeAnimation.JumpToEnd();
			}

			// The viewport for the rendered thumbnail, if it exists
			OverlayWidget->AddSlot()
			[
				SAssignNew(RenderedThumbnailWidget, SBorder)
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				.Padding(InArgs._Padding)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				.BorderImage(FStyleDefaults::GetNoBrush())
				.ColorAndOpacity(this, &SAssetThumbnail::GetViewportColorAndOpacity)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					Viewport.ToSharedRef()
				]
			];
		}

		if (ThumbnailClass.Get() && bIsClassType)
		{
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				FAssetDisplayInfo ClassInfo;
				ClassInfo.StatusIcon = MakeAttributeSP(this, &SAssetThumbnail::GetClassIconBrush);
				ClassInfo.Priority = FAssetStatusPriority(EStatusSeverity::Info, 1);
				ClassInfo.StatusDescription = MakeAttributeSP(this, &SAssetThumbnail::GetClassName);
				ClassInfo.IsVisible = EVisibility::Visible;
				OverlayInfo.Add(ClassInfo);
			}
			else
			{
				OverlayWidget->AddSlot()
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				.Padding(GetClassIconPadding())
				[
					SAssignNew(ClassIconWidget, SBorder)
					.BorderImage(FAppStyle::GetNoBrush())
					[
						SNew(SImage)
						.Image(this, &SAssetThumbnail::GetClassIconBrush)
					]
				];
			}
		}

		if( bAllowHintText )
		{
			OverlayWidget->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				.Padding(FMargin(2, 2, 2, 2))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(Style, ".HintBackground"))
					.BorderBackgroundColor(this, &SAssetThumbnail::GetHintBackgroundColor) //Adjust the opacity of the border itself
					.ColorAndOpacity(HintColorAndOpacity) //adjusts the opacity of the contents of the border
					.Visibility(this, &SAssetThumbnail::GetHintTextVisibility)
					.Padding(0)
					[
						SAssignNew(HintTextBlock, STextBlock)
						.Text(GetLabelText())
						.Font(GetHintTextFont())
						.ColorAndOpacity(FAppStyle::GetColor(Style, ".HintColorAndOpacity"))
						.HighlightText(HighlightedText)
					]
				];
		}

		TSharedRef<SWidget> ContentWidget = OverlayWidget;

		auto AddAssetColor = [&]()
		{
			// The asset color strip
			OverlayWidget->AddSlot()
			.HAlign(ColorStripOrientation == EThumbnailColorStripOrientation::HorizontalBottomEdge ? HAlign_Fill : HAlign_Right)
			.VAlign(ColorStripOrientation == EThumbnailColorStripOrientation::HorizontalBottomEdge ? VAlign_Bottom : VAlign_Fill)
			[
				SAssignNew(AssetColorStripBlockWidget, SColorBlock)
				.Color_Lambda([this] () { return AssetColor; })
				.Size(2.f)
				.Visibility(TAttribute<EVisibility>::CreateSP(this, &SAssetThumbnail::GetAssetColorVisibility))
			];
		};
		
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			if (ShowAssetColor.IsSet())
			{
				AddAssetColor();
			}
		}
		else
		{
			AddAssetColor();
		}

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Default border creation
			if (InArgs._ShowAssetBorder || AssetBorderImageOverride.IsSet())
			{
				// Image overlay to match the Hovered/Selected status, by design only part of the asset color should be highlighted
				const TSharedRef<SImage> OverlayImage = SNew(SImage)
					.Image(this, &SAssetThumbnail::GetThumbnailBorderBrush)
					.Visibility(EVisibility::HitTestInvisible);

				OverlayWidget->AddSlot()
				.Padding(TAttribute<FMargin>::CreateSP(this, &SAssetThumbnail::GetOverlayThumbnailBorderPadding))
				[
					OverlayImage
				];

				// Border used to contain the Thumbnail in the ratio
				const TSharedRef<SBorder> ThumbnailBorder = SNew(SBorder)
					.Padding(TAttribute<FMargin>::CreateSP(this, &SAssetThumbnail::GetThumbnailBorderPadding))
					.BorderImage(this, &SAssetThumbnail::GetThumbnailBorderBrush);

				if (AssetBorderImageOverride.IsSet())
				{
					OverlayImage->SetImage(AssetBorderImageOverride);
					ThumbnailBorder->SetBorderImage(AssetBorderImageOverride);
				}

				ThumbnailBorder->SetContent(OverlayWidget);
				ContentWidget = ThumbnailBorder;
			}
		}

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// AssetEditMode, do not create it if there is no config argument to show it
			if (EditModeVisibility.IsSet())
			{
				OverlayWidget->AddSlot()
				[
					SAssignNew(AssetThumbnailEditMode, SAssetThumbnailEditModeTools, AssetThumbnail)
					.SmallView(InArgs._CanDisplayEditModePrimitiveTools)
					.Visibility(EditModeVisibility)
				];
			}
		}

		const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Class);

		if (!AssetDefinition)
		{
			AssetDefinition = GetDefault<UAssetDefinitionDefault>();
		}

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			if (AllowAssetStatusThumbnailOverlay.IsSet() && AssetDefinition)
			{
				const TSharedPtr<FAssetStatusAssetDataInfoProvider> AssetDataInfoProvider = MakeShared<FAssetStatusAssetDataInfoProvider>(AssetData);
				AssetDefinition->GetAssetStatusInfo(AssetDataInfoProvider, OverlayInfo);
				// Sort the list based on Status Priority
				auto SortStatus = [] (const FAssetDisplayInfo& InFirstStatus, const FAssetDisplayInfo& InSecondStatus)
				{
					if (!InFirstStatus.Priority.IsSet())
					{
						return false;
					}

					if (!InSecondStatus.Priority.IsSet())
					{
						return true;
					}

					return InSecondStatus.Priority.Get() < InFirstStatus.Priority.Get();
				};
				OverlayInfo.Sort(SortStatus);

				const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
				for (int32 StatusIndex = 0; StatusIndex < OverlayInfo.Num(); StatusIndex++)
				{
					Statuses.Add(CreateStatusWidget(StatusIndex, OverlayInfo[StatusIndex]));
					HorizontalBox->AddSlot()
					[
						Statuses[StatusIndex].ToSharedRef()
					];
				}

				StatusOverflowWidget = CreateStatusOverflowWidget();
				HorizontalBox->AddSlot()
				[
					StatusOverflowWidget.ToSharedRef()
				];

				OverlayWidget->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				.Padding(StatusPadding)
				[
					SNew(SBorder)
					.Padding(StatusBorderPadding)
					.BorderImage(FAppStyle::GetBrush(Style, ".AssetThumbnailStatusBar"))
					.Visibility(this, &SAssetThumbnail::GetStatusBorderVisibility)
					[
						HorizontalBox
					]
				];
			}
		}

		if (AllowAssetSpecificThumbnailOverlay.IsSet())
		{
			FAssetActionThumbnailOverlayInfo OutThumbnailInfo;
			// Skip ALL older overlay if the new style is enabled
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				if (AssetDefinition && AssetDefinition->GetThumbnailActionOverlay(AssetData, OutThumbnailInfo))
				{
					constexpr int32 OverlayZOrder = 1;
					TSharedPtr<SWidget> TopLeftImage;
					TSharedPtr<SWidget> CenterImage;
					if (OutThumbnailInfo.IsActionPlayingDelegate.IsBound())
					{
						auto OnGetActionImage = [IsPlaying = OutThumbnailInfo.IsActionPlayingDelegate] ()
							{
								if (IsPlaying.IsBound() && IsPlaying.Execute())
								{
									return FAppStyle::GetBrush("ContentBrowser.AssetAction.StopIcon");
								}
								return FAppStyle::GetBrush("ContentBrowser.AssetAction.PlayIcon");
							};
						TopLeftImage = SNew(SImage).Image_Lambda(OnGetActionImage);
						CenterImage = SNew(SImage).Image_Lambda(OnGetActionImage);
					}
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					else if (OutThumbnailInfo.ActionImageWidget.IsValid())
					{
						// This approach use the same widget in 2 different place and that shouldn't happen, we have to create 2 separate widget, but keep it here for this release since we shouldn't break the API
						TopLeftImage = OutThumbnailInfo.ActionImageWidget.ToSharedRef();
						CenterImage = OutThumbnailInfo.ActionImageWidget.ToSharedRef();
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					
					if (TopLeftImage.IsValid())
					{
						constexpr float PaddingFromTopLeftBorder = 4.f;

						OverlayWidget->AddSlot()
						.ZOrder(OverlayZOrder)
						.VAlign(VAlign_Top)
						.HAlign(HAlign_Left)
						.Padding(PaddingFromTopLeftBorder, PaddingFromTopLeftBorder, 0.f, 0.f)
						[
							SNew(SBox)
							.WidthOverride(this, &SAssetThumbnail::GetPlayIndicatorSize)
							.HeightOverride(this, &SAssetThumbnail::GetPlayIndicatorSize)
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush(Style, ".AssetThumbnailBar"))
								.Padding(this, &SAssetThumbnail::GetPlayIndicatorPadding)
								.Visibility(this, &SAssetThumbnail::GetActionIconOverlayVisibility)
								[
									TopLeftImage.ToSharedRef()
								]
							]
						];
					}
					// Set the default style and padding for the Button
					OutThumbnailInfo.ActionButtonArgs
						.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>(Style, ".Action.Button"))
						.ContentPadding(this, &SAssetThumbnail::GetPlayButtonContentPadding);

					TAttribute<EVisibility> ActionVisibility = TAttribute<EVisibility>::CreateSP(this, &SAssetThumbnail::GetActionButtonVisibility);
					TSharedRef<SButton> ActionButton = SArgumentNew(OutThumbnailInfo.ActionButtonArgs, SButton);
					ActionButton->Construct(OutThumbnailInfo.ActionButtonArgs);
					ActionButton->SetVisibility(ActionVisibility);
					ActionButton->SetToolTipText(OutThumbnailInfo.ActionButtonArgs._ToolTipText);
					ActionButton->SetToolTip(OutThumbnailInfo.ActionButtonArgs._ToolTip);
					ActionButton->SetCursor(EMouseCursor::Default);
					TSharedRef<SWidget> ActionImageWidget = CenterImage.IsValid()
						? CenterImage.ToSharedRef()
						: SNew(SImage).Image(FAppStyle::GetBrush("ContentBrowser.AssetAction.PlayIcon"));

					ActionButton->SetContent(ActionImageWidget);

					constexpr float CenterImageSize = 32.f;

					OverlayWidget->AddSlot()
					.ZOrder(OverlayZOrder)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SBox)
						.WidthOverride(CenterImageSize)
						.HeightOverride(CenterImageSize)
						[
							ActionButton
						]
					];
				}
			}
			else if (AssetTypeActions.IsValid())
			{
				// Does the asset provide an additional thumbnail overlay?
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				TSharedPtr<SWidget> AssetSpecificThumbnailOverlay = AssetTypeActions->GetThumbnailOverlay(AssetData);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if (AssetSpecificThumbnailOverlay.IsValid())
				{
					OverlayWidget->AddSlot()
					[
						AssetSpecificThumbnailOverlay.ToSharedRef()
					];
				}
			}
		}

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ChildSlot
			[
				ContentWidget
			];
		}
		else
		{
			ChildSlot
			[
				OverlayWidget
			];
		}

		UpdateThumbnailVisibilities();

	}

	void UpdateThumbnailClass(const IAssetTypeActions* AssetTypeActions)
	{
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();
		ThumbnailClass = MakeWeakObjectPtr(const_cast<UClass*>(FClassIconFinder::GetIconClassForAssetData(AssetData, &bIsClassType)));
		if (ThumbnailClass.IsValid())
		{
			ClassName = FText::FromString(ThumbnailClass->GetName());
		}

		const FName AssetClassName = AssetThumbnail->GetAssetData().AssetClassPath.GetAssetName();

		ClassIconBrush = nullptr;
		ThumbnailBrush = nullptr;

		if (AssetTypeActions)
		{
			if (const FSlateBrush* AssetTypeThumbnail = AssetTypeActions->GetThumbnailBrush(AssetData, AssetClassName))
			{
				ThumbnailBrush = AssetTypeThumbnail;
			}

			if (const FSlateBrush* AssetTypeIcon = AssetTypeActions->GetIconBrush(AssetData, AssetClassName))
			{
				ClassIconBrush = AssetTypeIcon;
			}
		}

		if (!ThumbnailBrush)
		{
			// For non-class types, use the default based upon the actual asset class
			// This has the side effect of not showing a class icon for assets that don't have a proper thumbnail image available
			const FName DefaultThumbnail = (bIsClassType) ? NAME_None : FName(*FString::Printf(TEXT("ClassThumbnail.%s"), *AssetClassName.ToString()));
			ThumbnailBrush = FClassIconFinder::FindThumbnailForClass(ThumbnailClass.Get(), DefaultThumbnail);
		}
		if (!ClassIconBrush)
		{
			ClassIconBrush = FSlateIconFinder::FindIconBrushForClass(ThumbnailClass.Get());
		}
	}

	FSlateColor GetHintBackgroundColor() const
	{
		const FLinearColor Color = HintColorAndOpacity.Get();
		return FSlateColor( FLinearColor( Color.R, Color.G, Color.B, FMath::Lerp( 0.0f, 0.5f, Color.A ) ) );
	}

	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		SCompoundWidget::OnMouseEnter( MyGeometry, MouseEvent );

		if (AssetThumbnailEditMode.IsValid())
		{
			SetHover(true);
		}

		if ( bAllowRealTimeOnHovered )
		{
			AssetThumbnail->SetRealTime( true );
		}
	}
	
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		SCompoundWidget::OnMouseLeave( MouseEvent );

		if (AssetThumbnailEditMode.IsValid())
		{
			SetHover(AssetThumbnailEditMode->IsEditingThumbnail());
		}

		if ( bAllowRealTimeOnHovered )
		{
			AssetThumbnail->SetRealTime( false );
		}
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		if ( WidthLastFrame != AllottedGeometry.Size.X )
		{
			WidthLastFrame = static_cast<float>(AllottedGeometry.Size.X);

			// The width changed, update the font
			if ( GenericLabelTextBlock.IsValid() )
			{
				GenericLabelTextBlock->SetFont( GetTextFont() );
				GenericLabelTextBlock->SetWrapTextAt( GetTextWrapWidth() );
			}

			if ( HintTextBlock.IsValid() )
			{
				HintTextBlock->SetFont( GetHintTextFont() );
				HintTextBlock->SetWrapTextAt( GetTextWrapWidth() );
			}

			if (!OverlayInfo.IsEmpty())
			{
				const float ThumbnailWidth = WidthLastFrame - (StatusPadding * 2) - (StatusBorderPadding * 2) - GetAssetThumbnailBorderPadding();
				const int32 MaxShownStatus = FMath::FloorToInt32(ThumbnailWidth / DefaultStatusSize);
				constexpr int32 CutoffNumberBeforeResizing = 3;

				if (MaxShownStatus < CutoffNumberBeforeResizing)
				{
					StatusSize = ThumbnailWidth / 3.f;
				}
				else
				{
					StatusSize = DefaultStatusSize;
				}

				StatusSize = FMath::FloorToFloat(StatusSize);
			}

			if (WidthLastFrame < PlayIndicatorMaxSizeThreshold)
			{
				PlayIndicatorSize = (PlayIndicatorDefaultSize * WidthLastFrame) / PlayIndicatorMaxSizeThreshold;
				PlayIndicatorPadding = (PlayIndicatorDefaultPadding * WidthLastFrame) / PlayIndicatorMaxSizeThreshold;
				PlayButtonContentPadding = (PlayButtonContentDefaultPadding * WidthLastFrame) / PlayIndicatorMaxSizeThreshold;
			}
			else
			{
				PlayIndicatorSize = PlayIndicatorDefaultSize;
				PlayIndicatorPadding = PlayIndicatorDefaultPadding;
				PlayButtonContentPadding = PlayButtonContentDefaultPadding;
			}

			PlayButtonContentPadding = FMath::FloorToFloat(PlayButtonContentPadding);
			PlayIndicatorPadding = FMath::FloorToFloat(PlayIndicatorPadding);
			PlayIndicatorSize = FMath::FloorToFloat(PlayIndicatorSize);
		}
	}

	FOptionalSize GetPlayIndicatorSize() const
	{
		return FOptionalSize(PlayIndicatorSize);
	}

	FMargin GetPlayIndicatorPadding() const
	{
		return FMargin(PlayIndicatorPadding);
	}

	FMargin GetPlayButtonContentPadding() const
	{
		return FMargin(PlayButtonContentPadding);
	}

	TSharedRef<SDocumentationToolTip> GetDefaultToolTip() const
	{
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();

		const UClass* Class = FindObjectSafe<UClass>(AssetData.AssetClassPath);

		TArray<FAssetDisplayInfo> OutSystemInfo;
		TSharedRef<SBox> PromptBox = SNew(SBox);
		TSharedRef<SVerticalBox> OverallTooltipVBox = SNew(SVerticalBox);
		{
			const FSlateBrush* ClassIcon = FAppStyle::GetDefaultBrush();
			TOptional<FLinearColor> Color;
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetData.GetClass()))
			{
				ClassIcon = AssetDefinition->GetIconBrush(AssetData, AssetData.AssetClassPath.GetAssetName());
				Color = AssetDefinition->GetAssetColor();
			}

			if (AssetSystemInfoProvider.IsValid())
			{
				AssetSystemInfoProvider->PopulateAssetInfo(OutSystemInfo);
			}

			if (ClassIcon == nullptr || ClassIcon == FAppStyle::GetDefaultBrush())
			{
				ClassIcon = FSlateIconFinder::FindIconForClass(AssetData.GetClass()).GetIcon();
			}

			FText ClassNameText = NSLOCTEXT("AssetThumbnail", "ClassNameText", "Not Found");
			if (Class != NULL)
			{
				ClassNameText = Class->GetDisplayNameText();
			}
			else if (!AssetData.AssetClassPath.IsNull())
			{
				ClassNameText = FText::FromString(AssetData.AssetClassPath.ToString());
			}

			const FText NameText = FText::FromString(AssetData.AssetName.ToString());
			// Name/Type Slot
			OverallTooltipVBox->AddSlot()
			.AutoHeight()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(0.f, 0.f, 0.f, 6.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(NameText)
					.ColorAndOpacity(FStyleColors::White)
				]

				+ SVerticalBox::Slot()
				.Padding(0.f, 0.f, 0.f, 6.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.f, 0.f, 4.f, 0.f)
					[
						SNew(SBox)
						.WidthOverride(16.f)
						.HeightOverride(16.f)
						[
							SNew(SImage)
							.Image(ClassIcon)
							.ColorAndOpacity_Lambda([Color] () { return Color.IsSet() ? Color.GetValue() : FStyleColors::White;})
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ClassNameText)
					]
				]
			];

			if (AdditionalTooltipInSmallView.IsSet())
			{
				OverallTooltipVBox->AddSlot()
				[
					SNew(SBox)
					.Padding(this, &SAssetThumbnail::GetAdditionalSmallViewTooltipMargin)
					[
						AdditionalTooltipInSmallView.Get().ToSharedRef()
					]
				];
			}

			TSharedRef<SVerticalBox> StatusVerticalBox = SNew(SVerticalBox);

			for (const FAssetDisplayInfo& AssetStatusInfo : OverlayInfo)
			{
				TSharedRef<SLayeredImage> StatusLayeredImage = SNew(SLayeredImage).Image(AssetStatusInfo.StatusIcon);
				StatusLayeredImage->AddLayer(AssetStatusInfo.StatusIconOverlay);

				StatusVerticalBox->AddSlot()
					.Padding(0.f, 0.f, 0.f, 6.f)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility(AssetStatusInfo.IsVisible)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 4.f, 0.f)
						[
						   SNew(SBox)
						   .WidthOverride(16.f)
						   .HeightOverride(16.f)
						   [
								StatusLayeredImage
						   ]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(AssetStatusInfo.StatusDescription)
						]
					];
			}

			// Status
			OverallTooltipVBox->AddSlot()
			.AutoHeight()
			[
				StatusVerticalBox
			];

			// Separator
			OverallTooltipVBox->AddSlot()
			.Padding(0.f,0.f, 0.f, 6.f)
			.AutoHeight()
			[
				SNew(SSeparator)
				.Orientation(Orient_Horizontal)
				.Thickness(1.f)
				.ColorAndOpacity(COLOR("#484848FF"))
				.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			];

			// More info
			if (!OutSystemInfo.IsEmpty())
			{
				PromptBox->SetPadding(FMargin(0.f, 2.f, 0.f, 0.f));
				PromptBox->SetContent(
					SNew(SRichTextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(Style, ".Tooltip.MoreInfoText"))
					.Justification(ETextJustify::Center)
					.Text(NSLOCTEXT("AssetThumbnail", "MoreInfoTooltip", "Hold<WrappedCommand/>for more"))
					+ SRichTextBlock::WidgetDecorator(TEXT("WrappedCommand"), this, &SAssetThumbnail::OnCreateWidgetDecoratorWidget)
				);
			}
		}

		TSharedRef<SVerticalBox> ExtendedToolTipVerticalBox = SNew(SVerticalBox);
		TSharedRef<SBox> ExtendedToolTip = SNew(SBox)
			.Padding(FMargin(9.f, -9.f, 10.f, 6.f))
			[
				ExtendedToolTipVerticalBox
			];

		bool bWasSeparatorAddedForCollection = false;
		for (const FAssetDisplayInfo& SystemInfo : OutSystemInfo)
		{
			if (!SystemInfo.StatusTitle.IsSet() || !SystemInfo.StatusDescription.IsSet())
			{
				continue;
			}

			// StatusTitle currently used to add a separator for Collection, need to be changed in future version to allow more configurability
			const FName TitleName = FName(SystemInfo.StatusTitle.Get().ToString());
			if (!bWasSeparatorAddedForCollection && TitleName == TEXT("Collection(s)"))
			{
				bWasSeparatorAddedForCollection = true;
			
				// Separator
				ExtendedToolTipVerticalBox->AddSlot()
				.Padding(0.f,0.f, 0.f, 6.f)
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
					.Thickness(1.f)
					.ColorAndOpacity(COLOR("#484848FF"))
					.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				];
				continue;
			}
			
			if ((SystemInfo.IsVisible.IsSet() && SystemInfo.IsVisible.Get().IsVisible()) || !SystemInfo.IsVisible.IsSet())
			{
				AddToExtendedToolTipInfoBox(ExtendedToolTipVerticalBox, SystemInfo.StatusIcon, SystemInfo.StatusTitle.Get(), SystemInfo.StatusDescription.Get());
			}
		}

		return SNew(SDocumentationToolTip)
				.OverrideExtendedToolTipContent(ExtendedToolTip)
				.OverridePromptContent(PromptBox)
				.AlwaysExpandTooltip(AlwaysExpandTooltip)
				[
					OverallTooltipVBox
				];
	}

private:

	FMargin GetAdditionalSmallViewTooltipMargin() const
	{
		if (AdditionalTooltipInSmallView.IsSet())
		{
			const TSharedPtr<SWidget> AdditionalTooltip = AdditionalTooltipInSmallView.Get();
			const bool bIsValidAndNotNullWidget = AdditionalTooltip.IsValid() && AdditionalTooltip != SNullWidget::NullWidget;
			return bIsValidAndNotNullWidget && AdditionalTooltip->GetVisibility().IsVisible() ? FMargin(0.f, 0.f, 0.f, 6.f) : FMargin(0.f);
		}
		return FMargin(0.f);
	}

	FSlateWidgetRun::FWidgetRunInfo OnCreateWidgetDecoratorWidget(const FTextRunInfo& InRunInfo, const ISlateStyle* InStyle) const
	{
		TSharedRef<SWidget> CtrlAltWidget = SNew(SBox)
			.Padding(5.f, 0.f)
			[
				SNew(SBorder)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush(Style, ".ToolTip.CommandBorder"))
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>(Style, ".Tooltip.MoreInfoText"))
#if PLATFORM_MAC
					.Text(NSLOCTEXT("AssetThumbnail", "CommandOptionLabel", " Command + Option "))
#else
					.Text(NSLOCTEXT("AssetThumbnail", "CtrlAltLabel", " Ctrl + Alt "))
#endif
				]
			];
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const int16 Baseline = FontMeasure->GetBaseline(FStyleDefaults::GetFontInfo());

		return FSlateWidgetRun::FWidgetRunInfo(CtrlAltWidget, Baseline - 2);

	}

	void AddToExtendedToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const TAttribute<const FSlateBrush*>& Icon, const FText& Key, const FText& Value) const
	{
		InfoBox->AddSlot()
		.Padding(0.f, 0.f, 0.f, 6.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(SBox)
				.Visibility(Icon.IsSet() && Icon.Get() != nullptr? EVisibility::Visible : EVisibility::Collapsed)
				.WidthOverride(16.f)
				.HeightOverride(16.f)
				[
					SNew(SImage)
					.Image(Icon)
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(STextBlock)
				.Text(FText::Format(NSLOCTEXT("AssetThumbnailToolTip", "AssetViewTooltipFormat", "{0}:"), Key))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::White)
				.Text(Value)
				.WrapTextAt(700.0f)
			]
		];
	}

	const FSlateBrush* GetThumbnailBorderBrush() const
	{
		const FString ThumbnailBorder = TEXT(".AssetBorder");
		return FAppStyle::GetBrush(Style, StringCast<ANSICHAR>(*ThumbnailBorder).Get());
	}

	EVisibility GetAssetColorVisibility() const
	{
		// if the new style is not enabled always show it
		return !UE::Editor::ContentBrowser::IsNewStyleEnabled() || ShowAssetColor.Get() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FMargin GetOverlayThumbnailBorderPadding() const
	{
		constexpr float DefaultPadding = -2.f;
		// Need this to be negative for the Overlay so that it overlaps with the actual border which will keep the ratio correct and won't let the thumbnail over the limits.
		if (BorderPadding.IsSet())
		{
			// Get the negative of it so that the overlay border is always aligned correctly
			return BorderPadding.Get() * -1.f;
		}
		return FMargin(DefaultPadding);
	}

	FMargin GetThumbnailBorderPadding() const
	{
		constexpr float DefaultPadding = 2.f;
		// Asset strip margin
		if (BorderPadding.IsSet())
		{
			return BorderPadding.Get();
		}
		return FMargin(DefaultPadding);
	}

	EVisibility GetStatusBorderVisibility() const
	{
		// If not allowed hide it
		if (AllowAssetStatusThumbnailOverlay.Get())
		{
			for (const FAssetDisplayInfo& AssetStatus : OverlayInfo)
			{
				if (AssetStatus.IsVisible.IsSet() && AssetStatus.IsVisible.Get().IsVisible())
				{
					if (!EditModeVisibility.IsSet() || !EditModeVisibility.Get().IsVisible())
					{
						return EVisibility::Visible;
					}
				}
			}
		}
		return EVisibility::Collapsed;
	}

	EVisibility GetActionIconOverlayVisibility() const
	{
		const bool bIsEditModeVisible = EditModeVisibility.IsSet() && EditModeVisibility.Get().IsVisible();
		return AllowAssetSpecificThumbnailOverlayIndicator.Get() && !IsHovered() && !bIsEditModeVisible ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetActionButtonVisibility() const
	{
		return IsHovered() && AllowAssetSpecificThumbnailOverlay.Get() && (!EditModeVisibility.IsSet() || !EditModeVisibility.Get().IsVisible()) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TSharedRef<SWidget> CreateStatusWidget(int32 StatusIndex, const FAssetDisplayInfo& InStatusInfo) const
	{
		TSharedRef<SLayeredImage> StatusLayeredImage = SNew(SLayeredImage)
			.Image(InStatusInfo.StatusIcon)
			.DesiredSizeOverride(this, &SAssetThumbnail::GetStatusSizeForImage);
		StatusLayeredImage->AddLayer(InStatusInfo.StatusIconOverlay);

		return SNew(SBox)
			.WidthOverride(this, &SAssetThumbnail::GetStatusSize)
			.HeightOverride(this, &SAssetThumbnail::GetStatusSize)
			.Visibility(this, &SAssetThumbnail::GetStatusVisibility, StatusIndex)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				StatusLayeredImage
			];
	}

	EVisibility GetStatusVisibility(int32 StatusIndex) const
	{
		if (OverlayInfo.IsValidIndex(StatusIndex))
		{
			const FAssetDisplayInfo& AssetStatus = OverlayInfo[StatusIndex];
			if (AssetStatus.IsVisible.IsSet())
			{
				const EVisibility StatusVisibility = AssetStatus.IsVisible.Get();
				if (StatusVisibility != EVisibility::Hidden && StatusVisibility != EVisibility::Collapsed)
				{
					return GetStatusVisibilityBasedOnGeometry(StatusIndex);
				}
			}
		}
		return EVisibility::Collapsed;
	}

	TSharedRef<SWidget> CreateStatusOverflowWidget() const
	{
		return SNew(SBox)
			.WidthOverride(this, &SAssetThumbnail::GetStatusSize)
			.HeightOverride(this, &SAssetThumbnail::GetStatusSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Visibility(this, &SAssetThumbnail::GetStatusOverflowVisibility)
			[
				SNew(STextBlock)
				.Font(this, &SAssetThumbnail::GetStatusOverflowFont)
				.Text(this, &SAssetThumbnail::GetStatusOverflowText)
				.ColorAndOpacity(FStyleColors::Foreground)
			];
	}

	FSlateFontInfo GetStatusOverflowFont() const
	{
		FString FontStyle = TEXT(".StatusOverflowFont");
		if (StatusSize != DefaultStatusSize)
		{
			FontStyle = TEXT(".StatusOverflowFontSmall");
		}
		return FAppStyle::GetFontStyle(Style, StringCast<ANSICHAR>(*FontStyle).Get());
	}

	FOptionalSize GetStatusSize() const
	{
		return FOptionalSize(StatusSize);
	}

	TOptional<FVector2D> GetStatusSizeForImage() const
	{
		return TOptional(FVector2D(StatusSize));
	}

	float GetAssetThumbnailBorderPadding() const
	{
		// Already take into account Left and Right
		constexpr float DefaultPadding = 0.f;

		// Asset strip margin
		if (BorderPadding.IsSet())
		{
			// Get the negative of it so that the overlay border is always aligned correctly
			const FMargin BorderMargin = BorderPadding.Get();
			return BorderMargin.Left + BorderMargin.Right;
		}
		return DefaultPadding;
	}

	EVisibility GetStatusOverflowVisibility() const
	{
		const FGeometry ThumbnailGeometry = GetPaintSpaceGeometry();
		const float ThumbnailWidth = ThumbnailGeometry.GetAbsoluteSize().X - (StatusPadding * 2) - (StatusBorderPadding * 2) - GetAssetThumbnailBorderPadding();
		const int32 MaxShownStatus = FMath::FloorToInt32(ThumbnailWidth / StatusSize);

		int32 ShownStatus = 0;
		for (const FAssetDisplayInfo& Status : OverlayInfo)
		{
			if (Status.IsVisible.IsSet() && Status.IsVisible.Get().IsVisible())
			{
				ShownStatus++;
			}
		}
		return ShownStatus > MaxShownStatus? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText GetStatusOverflowText() const
	{
		FGeometry ThumbnailGeometry = GetPaintSpaceGeometry();
		const float ThumbnailWidth = ThumbnailGeometry.GetAbsoluteSize().X - (StatusPadding * 2) - (StatusBorderPadding * 2) - GetAssetThumbnailBorderPadding();
		const int32 MaxShownStatus = FMath::FloorToInt32(ThumbnailWidth / StatusSize);

		int32 ShownStatus = 0;
		for (const FAssetDisplayInfo& Status : OverlayInfo)
		{
			if (Status.IsVisible.IsSet() && Status.IsVisible.Get().IsVisible())
			{
				ShownStatus++;
			}
		}

		// We need to add 1 to the HiddenStatus since the Overflow "status" will occupy 1 extra slot
		constexpr int32 OccupiedStatusSlot = 1;
		const int32 HiddenStatus = ShownStatus - MaxShownStatus + OccupiedStatusSlot;
		return FText::Format(NSLOCTEXT("AssetThumbnail", "StatusOverflowText", "+{0}"), HiddenStatus);
	}

	EVisibility GetStatusVisibilityBasedOnGeometry(int32 InStatusIndex) const
	{
		int32 CollapsedStatusBeforeThis = 0;
		for (int32 StatusIndex = 0; StatusIndex < InStatusIndex; StatusIndex++)
		{
			CollapsedStatusBeforeThis += Statuses[StatusIndex]->GetVisibility().IsVisible() ? 0 : 1;
		}

		const FGeometry ThumbnailGeometry = GetPaintSpaceGeometry();
		const float ThumbnailWidth = ThumbnailGeometry.GetAbsoluteSize().X - (StatusPadding * 2) - (StatusBorderPadding * 2) - GetAssetThumbnailBorderPadding();
		const int32 StatusIndexConsideringHiddenOnes = (InStatusIndex - CollapsedStatusBeforeThis);
		const int32 ShownStatus = FMath::FloorToInt32(ThumbnailWidth / StatusSize);
		const int32 StatusIndexConsideringClippedStatusIfVisible = StatusOverflowWidget.IsValid() && StatusOverflowWidget->GetVisibility().IsVisible() ? StatusIndexConsideringHiddenOnes + 1 : StatusIndexConsideringHiddenOnes;
		return StatusIndexConsideringClippedStatusIfVisible < ShownStatus ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void OnAssetDataChanged()
	{
		if ( GenericLabelTextBlock.IsValid() )
		{
			GenericLabelTextBlock->SetText( GetLabelText() );
		}

		if ( HintTextBlock.IsValid() )
		{
			HintTextBlock->SetText( GetLabelText() );
		}

		// Check if the asset has a thumbnail.
		const FObjectThumbnail* ObjectThumbnail = NULL;
		FThumbnailMap ThumbnailMap;
		if( AssetThumbnail->GetAsset() )
		{
			FName FullAssetName = FName( *(AssetThumbnail->GetAssetData().GetFullName()) );
			TArray<FName> ObjectNames;
			ObjectNames.Add( FullAssetName );
			ThumbnailTools::ConditionallyLoadThumbnailsForObjects(ObjectNames, ThumbnailMap);
			ObjectThumbnail = ThumbnailMap.Find( FullAssetName );
		}

		bHasRenderedThumbnail = ObjectThumbnail && !ObjectThumbnail->IsEmpty();
		ViewportFadeAnimation.JumpToEnd();
		AssetThumbnail->GetViewportRenderTargetTexture(); // Access the render texture to push it on the stack if it isnt already rendered

		const FAssetData& AssetData = AssetThumbnail->GetAssetData();

		UClass* Class = FindObject<UClass>(AssetData.AssetClassPath);
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TSharedPtr<IAssetTypeActions> AssetTypeActions;
		if ( Class != NULL )
		{
			TWeakPtr<IAssetTypeActions> TypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Class);
			if (TypeActions.IsValid())
			{
				AssetTypeActions = TypeActions.Pin();
			}
		}

		UpdateThumbnailClass(AssetTypeActions.Get());

		AssetColor = FLinearColor::White;
		if( AssetTypeColorOverride.IsSet() )
		{
			AssetColor = AssetTypeColorOverride.GetValue();
		}
		else if ( AssetTypeActions.IsValid() )
		{
			AssetColor = AssetTypeActions->GetTypeColor();
		}

		UpdateThumbnailVisibilities();
	}

	FSlateFontInfo GetTextFont() const
	{
		return FAppStyle::GetFontStyle( WidthLastFrame <= 64 ? FAppStyle::Join(Style, ".FontSmall") : FAppStyle::Join(Style, ".Font") );
	}

	FSlateFontInfo GetHintTextFont() const
	{
		return FAppStyle::GetFontStyle( WidthLastFrame <= 64 ? FAppStyle::Join(Style, ".HintFontSmall") : FAppStyle::Join(Style, ".HintFont") );
	}

	float GetTextWrapWidth() const
	{
		return WidthLastFrame - GenericThumbnailBorderPadding * 2.f;
	}

	const FSlateBrush* GetAssetBackgroundBrush() const
	{
		return FAppStyle::GetBrush(AssetBackgroundBrushName);
	}

	const FSlateBrush* GetClassBackgroundBrush() const
	{

		return FAppStyle::GetBrush(ClassBackgroundBrushName);
	}

	FLinearColor GetViewportColorAndOpacity() const
	{
		return FLinearColor(1, 1, 1, ViewportFadeCurve.GetLerp());
	}
	
	EVisibility GetViewportVisibility() const
	{
		return bHasRenderedThumbnail ? EVisibility::Visible : EVisibility::Collapsed;
	}

	/** The height of the color element (if it's oriented along the bottom edge) or its width (if it's oriented along the right edge) */
	float GetAssetColorThickness() const
	{
		if (BorderPadding.IsSet())
		{
			// Get half the vertical or horizontal padding value
			// The user specifies the intended line thickness with a uniform padding value, so we need to compensate
			return ColorStripOrientation == EThumbnailColorStripOrientation::HorizontalBottomEdge
				? BorderPadding.Get().GetTotalSpaceAlong<EOrientation::Orient_Vertical>() / 2.f
				: BorderPadding.Get().GetTotalSpaceAlong<Orient_Horizontal>() / 2.f;
		}

		return 2.0f;
	}

	const FSlateBrush* GetClassThumbnailBrush() const
	{
		if (ClassThumbnailBrushOverride.IsNone())
		{
			return ThumbnailBrush;
		}
		else
		{
			// Instead of getting the override thumbnail directly from the editor style here get it from the
			// ClassIconFinder since it may have additional styles registered which can be searched by passing
			// it as a default with no class to search for.
			return FClassIconFinder::FindThumbnailForClass(nullptr, ClassThumbnailBrushOverride);
		}
	}

	EVisibility GetClassThumbnailVisibility() const
	{
		if(!bHasRenderedThumbnail)
		{
			const FSlateBrush* ClassThumbnailBrush = GetClassThumbnailBrush();
			if( (ClassThumbnailBrush && ThumbnailClass.Get())  || !ClassThumbnailBrushOverride.IsNone() )
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Collapsed;
	}

	EVisibility GetGenericThumbnailVisibility() const
	{
		return (bHasRenderedThumbnail && ViewportFadeAnimation.IsAtEnd()) ? EVisibility::Collapsed : EVisibility::Visible;
	}

	FText GetClassName() const
	{
		return ClassName;
	}

	const FSlateBrush* GetClassIconBrush() const
	{
		return ClassIconBrush;
	}

	FMargin GetClassIconPadding() const
	{
		if (ColorStripOrientation == EThumbnailColorStripOrientation::HorizontalBottomEdge)
		{
			const float Height = GetAssetColorThickness();
			return FMargin(0, 0, 0, Height);
		}
		else
		{
			const float Width = GetAssetColorThickness();
			return FMargin(0, 0, Width, 0);
		}
	}

	EVisibility GetHintTextVisibility() const
	{
		if ( bAllowHintText && ( bHasRenderedThumbnail || !GenericLabelTextBlock.IsValid() ) && HintColorAndOpacity.Get().A > 0 )
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	}

	void OnThumbnailRendered(const FAssetData& AssetData)
	{
		if ( !bHasRenderedThumbnail && AssetData == AssetThumbnail->GetAssetData() && ShouldRender() )
		{
			OnRenderedThumbnailChanged( true );
			ViewportFadeAnimation.Play( this->AsShared() );
		}
	}

	void OnThumbnailRenderFailed(const FAssetData& AssetData)
	{
		if ( bHasRenderedThumbnail && AssetData == AssetThumbnail->GetAssetData() )
		{
			OnRenderedThumbnailChanged( false );
		}
	}

	bool ShouldRender() const
	{
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();

		// Never render a thumbnail for an invalid asset
		if ( !AssetData.IsValid() )
		{
			return false;
		}

		if( AssetData.IsAssetLoaded() )
		{
			// Loaded asset, return true if there is a rendering info for it
			UObject* Asset = AssetData.GetAsset();
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo( Asset );
			if ( RenderInfo != NULL && RenderInfo->Renderer != NULL )
			{
				return true;
			}
		}

		const FObjectThumbnail* CachedThumbnail = ThumbnailTools::FindCachedThumbnail(*AssetData.GetFullName());
		if ( CachedThumbnail != NULL )
		{
			// There is a cached thumbnail for this asset, we should render it
			return !CachedThumbnail->IsEmpty();
		}

		if ( AssetData.AssetClassPath != UBlueprint::StaticClass()->GetClassPathName() )
		{
			// If we are not a blueprint, see if the CDO of the asset's class has a rendering info
			// Blueprints can't do this because the rendering info is based on the generated class
			UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);

			if ( AssetClass )
			{
				FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo( AssetClass->GetDefaultObject() );
				if ( RenderInfo != NULL && RenderInfo->Renderer != NULL )
				{
					return true;
				}
			}
		}
		
		// Always render thumbnails with custom thumbnails
		FString CustomThumbnailTagValue;
		if (AssetData.GetTagValue(FAssetThumbnailPool::CustomThumbnailTagName, CustomThumbnailTagValue))
		{
			return true;
		}
		
		// Unloaded blueprint or asset that may have a custom thumbnail, check to see if there is a thumbnail in the package to render
		FString PackageFilename;
		if ( FPackageName::DoesPackageExist(AssetData.PackageName.ToString(), &PackageFilename) )
		{
			TSet<FName> ObjectFullNames;
			FThumbnailMap ThumbnailMap;

			FName ObjectFullName = FName(*AssetData.GetFullName());
			ObjectFullNames.Add(ObjectFullName);

			ThumbnailTools::LoadThumbnailsFromPackage(PackageFilename, ObjectFullNames, ThumbnailMap);

			const FObjectThumbnail* ThumbnailPtr = ThumbnailMap.Find(ObjectFullName);
			if (ThumbnailPtr)
			{
				const FObjectThumbnail& ObjectThumbnail = *ThumbnailPtr;
				return ObjectThumbnail.GetImageWidth() > 0 && ObjectThumbnail.GetImageHeight() > 0 && ObjectThumbnail.GetCompressedDataSize() > 0;
			}
		}

		return false;
	}

	FText GetLabelText() const
	{
		if( Label != EThumbnailLabel::NoLabel )
		{
			if ( Label == EThumbnailLabel::ClassName )
			{
				return GetAssetClassDisplayName();
			}
			else if ( Label == EThumbnailLabel::AssetName ) 
			{
				return GetAssetDisplayName();
			}
		}
		return FText::GetEmpty();
	}

	FText GetDisplayNameForClass( UClass* Class, const FAssetData* AssetData = nullptr) const
	{
		FText ClassDisplayName;
		if ( Class )
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Class);
			
			if ( AssetTypeActions.IsValid() )
			{
				if (AssetData != nullptr)
				{
					ClassDisplayName = AssetTypeActions.Pin()->GetDisplayNameFromAssetData(*AssetData);
				}

				if (ClassDisplayName.IsEmpty())
				{
					ClassDisplayName = AssetTypeActions.Pin()->GetName();
				}
			}

			if ( ClassDisplayName.IsEmpty() )
			{
				ClassDisplayName = Class->GetDisplayNameText();
			}
		}

		return ClassDisplayName;
	}

	FText GetAssetClassDisplayName() const
	{
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();
		FTopLevelAssetPath AssetClass = AssetData.AssetClassPath;
		UClass* Class = FindObjectSafe<UClass>(AssetClass);

		if ( Class )
		{
			return GetDisplayNameForClass( Class, &AssetData );
		}

		return FText::FromString(AssetClass.GetAssetName().ToString());
	}

	FText GetAssetDisplayName() const
	{
		const FAssetData& AssetData = AssetThumbnail->GetAssetData();

		if ( AssetData.GetClass() == UClass::StaticClass() )
		{
			UClass* Class = Cast<UClass>( AssetData.GetAsset() );
			return GetDisplayNameForClass( Class );
		}

		return FText::FromName(AssetData.AssetName);
	}

	void OnRenderedThumbnailChanged( bool bInHasRenderedThumbnail )
	{
		bHasRenderedThumbnail = bInHasRenderedThumbnail;

		UpdateThumbnailVisibilities();
	}

	void UpdateThumbnailVisibilities()
	{
		// Either the generic label or thumbnail should be shown, but not both at once
		const EVisibility ClassThumbnailVisibility = GetClassThumbnailVisibility();
		if( GenericThumbnailImage.IsValid() )
		{
			GenericThumbnailImage->SetVisibility( ClassThumbnailVisibility );
		}
		if( GenericLabelTextBlock.IsValid() )
		{
			GenericLabelTextBlock->SetVisibility( (ClassThumbnailVisibility == EVisibility::Visible) ? EVisibility::Collapsed : EVisibility::Visible );
		}

		const EVisibility ViewportVisibility = GetViewportVisibility();
		if( RenderedThumbnailWidget.IsValid() )
		{
			RenderedThumbnailWidget->SetVisibility( ViewportVisibility );
			if( ClassIconWidget.IsValid() )
			{
				static const IConsoleVariable* CVarEnableContentBrowserNewStyle = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle"));
				const bool bEnableContentBrowserNewStyle = CVarEnableContentBrowserNewStyle && CVarEnableContentBrowserNewStyle->GetBool();

				if (bEnableContentBrowserNewStyle)
				{
					ClassIconWidget->SetVisibility(TAttribute<EVisibility>::CreateLambda([this, ViewportVisibility] () { return !EditModeVisibility.IsSet() || !EditModeVisibility.Get().IsVisible() ? ViewportVisibility : EVisibility::Collapsed; }));
				}
				else
				{
					ClassIconWidget->SetVisibility( ViewportVisibility );
				}
			}
		}
	}

	TOptional<FVector2D> GetGenericThumbnailDesiredSize() const
	{
		const int32 Size = GenericThumbnailSize.Get();

		return FVector2D(Size, Size);
	}

private:
	TSharedPtr<SAssetThumbnailEditModeTools> AssetThumbnailEditMode;
	TSharedPtr<STextBlock> GenericLabelTextBlock;
	TSharedPtr<STextBlock> HintTextBlock;
	TSharedPtr<SImage> GenericThumbnailImage;
	TSharedPtr<SBorder> ClassIconWidget;
	TSharedPtr<SBorder> RenderedThumbnailWidget;
	TSharedPtr<SBorder> AssetBackgroundWidget;
	TSharedPtr<SColorBlock> AssetColorStripBlockWidget;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	FCurveSequence ViewportFadeAnimation;
	FCurveHandle ViewportFadeCurve;

	FLinearColor AssetColor;
	TOptional<FLinearColor> AssetTypeColorOverride;

	float WidthLastFrame;
	float GenericThumbnailBorderPadding;
	bool bHasRenderedThumbnail;
	FName Style;
	TAttribute< FText > HighlightedText;
	EThumbnailLabel::Type Label;

	TAttribute< FLinearColor > HintColorAndOpacity;
	TAttribute<int32> GenericThumbnailSize;
	EThumbnailColorStripOrientation ColorStripOrientation;
	TSharedPtr<SWidget> StatusOverflowWidget;
	float StatusSize = 16.f;
	const float DefaultStatusSize = 16.f;
	const float StatusPadding = 4.f;
	const float StatusBorderPadding = 2.f;
	float PlayIndicatorPadding = 4.f;
	const float PlayIndicatorDefaultPadding = 4.f;
	float PlayButtonContentPadding = 8.f;
	const float PlayButtonContentDefaultPadding = 8.f;
	float PlayIndicatorSize = 20.f;
	const float PlayIndicatorMaxSizeThreshold = 64.f;
	const float PlayIndicatorDefaultSize = 20.f;
	TAttribute<FMargin> BorderPadding;
	TArray<FAssetDisplayInfo> OverlayInfo;
	TArray<TSharedPtr<SWidget>> Statuses;
	TSharedPtr<IAssetSystemInfoProvider> AssetSystemInfoProvider;
	TAttribute<bool> ShowAssetColor;
	TAttribute<const FSlateBrush*> AssetBorderImageOverride;
	TAttribute<EVisibility> EditModeVisibility;
	TAttribute<bool> AlwaysExpandTooltip;
	TAttribute<TSharedPtr<SWidget>> AdditionalTooltipInSmallView;
	TAttribute<bool> AllowAssetSpecificThumbnailOverlay;
	TAttribute<bool> AllowAssetSpecificThumbnailOverlayIndicator;
	TAttribute<bool> AllowAssetStatusThumbnailOverlay;
	bool bAllowHintText;
	bool bAllowRealTimeOnHovered;

	/** The name of the thumbnail which should be used instead of the class thumbnail. */
	FName ClassThumbnailBrushOverride;

	FName AssetBackgroundBrushName;
	FName ClassBackgroundBrushName;

	const FSlateBrush* ThumbnailBrush;

	const FSlateBrush* ClassIconBrush;
	FText ClassName;

	/** The class to use when finding the thumbnail. */
	TWeakObjectPtr<UClass> ThumbnailClass;
	/** Are we showing a class type? (UClass, UBlueprint) */
	bool bIsClassType;
};

void SAssetThumbnailToolTip::Construct(const FArguments& InArgs)
{
	AssetThumbnail = InArgs._AssetThumbnail;

	SToolTip::Construct(
		SToolTip::FArguments()
		.TextMargin(FMargin(1.f, -3.f))
		.BorderImage(FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border"))
		);
}

bool SAssetThumbnailToolTip::IsEmpty() const
{
	return !AssetThumbnail.IsValid();
}

void SAssetThumbnailToolTip::OnOpening()
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		TSharedPtr<SAssetThumbnail> AssetViewItemPin = AssetThumbnail.Pin();
		if (AssetViewItemPin.IsValid())
		{
			TSharedRef<SDocumentationToolTip> AssetToolTipRef = AssetViewItemPin->GetDefaultToolTip();
			SetContentWidget(AssetToolTipRef);
			AssetToolTip = AssetToolTipRef.ToSharedPtr();
		}
	}
}

void SAssetThumbnailToolTip::OnClosed()
{
	ResetContentWidget();
}

bool SAssetThumbnailToolTip::IsInteractive() const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		TSharedPtr<SAssetThumbnail> AssetViewItemPin = AssetThumbnail.Pin();

		// Use the SDocumentationTooltip IsInteractive only if we have something to show and if it should not be expanded by default.
		// This way we can avoid our tooltip to be kept in place when hovering.
		const bool bShouldAlwaysBeExpanded = AlwaysExpandTooltip.IsSet() && AlwaysExpandTooltip.Get();
		if (!IsEmpty() && AssetViewItemPin.IsValid() && AssetToolTip.IsValid() && !bShouldAlwaysBeExpanded)
		{
			return AssetToolTip->IsInteractive();
		}
		return false;
	}
	return false;
}

FAssetThumbnail::FAssetThumbnail( UObject* InAsset, uint32 InWidth, uint32 InHeight, const TSharedPtr<class FAssetThumbnailPool>& InThumbnailPool )
	: ThumbnailPool(InThumbnailPool)
	, AssetData(InAsset ? FAssetData(InAsset) : FAssetData())
	, Width( InWidth )
	, Height( InHeight )
{
	if ( InThumbnailPool.IsValid() )
	{
		InThumbnailPool->AddReferencer(*this);
	}
}

FAssetThumbnail::FAssetThumbnail( const FAssetData& InAssetData , uint32 InWidth, uint32 InHeight, const TSharedPtr<class FAssetThumbnailPool>& InThumbnailPool )
	: ThumbnailPool( InThumbnailPool )
	, AssetData ( InAssetData )
	, Width( InWidth )
	, Height( InHeight )
{
	if ( InThumbnailPool.IsValid() )
	{
		InThumbnailPool->AddReferencer(*this);
	}
}

FAssetThumbnail::~FAssetThumbnail()
{
	if ( ThumbnailPool.IsValid() )
	{
		ThumbnailPool.Pin()->RemoveReferencer(*this);
	}
}

FIntPoint FAssetThumbnail::GetSize() const
{
	return FIntPoint( Width, Height );
}

FSlateShaderResource* FAssetThumbnail::GetViewportRenderTargetTexture() const
{
	FSlateTexture2DRHIRef* Texture = NULL;
	if ( ThumbnailPool.IsValid() )
	{
		Texture = ThumbnailPool.Pin()->AccessTexture( AssetData, Width, Height );
	}
	if( !Texture || !Texture->IsValid() )
	{
		return NULL;
	}

	return Texture;
}

UObject* FAssetThumbnail::GetAsset() const
{
	if ( AssetData.IsValid() )
	{
		return AssetData.GetSoftObjectPath().ResolveObject();
	}
	else
	{
		return NULL;
	}
}

const FAssetData& FAssetThumbnail::GetAssetData() const
{
	return AssetData;
}

void FAssetThumbnail::SetAsset( const UObject* InAsset )
{
	SetAsset( FAssetData(InAsset) );
}

void FAssetThumbnail::SetAsset( const FAssetData& InAssetData )
{
	if ( ThumbnailPool.IsValid() )
	{
		ThumbnailPool.Pin()->RemoveReferencer(*this);
	}

	if ( InAssetData.IsValid() )
	{
		AssetData = InAssetData;
		if ( ThumbnailPool.IsValid() )
		{
			ThumbnailPool.Pin()->AddReferencer(*this);
		}
	}
	else
	{
		AssetData = FAssetData();
	}

	AssetDataChangedEvent.Broadcast();
}

TSharedRef<SWidget> FAssetThumbnail::MakeThumbnailWidget( const FAssetThumbnailConfig& InConfig )
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TAttribute<bool> AssetThumbnailOverlayAttribute =
		InConfig.AllowAssetSpecificThumbnailOverlay.IsSet()
		? InConfig.AllowAssetSpecificThumbnailOverlay
		: InConfig.bAllowAssetSpecificThumbnailOverlay;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// If not set use the PlayButton attribute instead
	const TAttribute<bool> AssetThumbnailOverlayIndicatorAttribute =
		InConfig.AllowAssetSpecificThumbnailOverlayIndicator.IsSet()
		? InConfig.AllowAssetSpecificThumbnailOverlayIndicator
		: AssetThumbnailOverlayAttribute;

	TSharedPtr<SWidget> ThumbnailWidget = nullptr;
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		ThumbnailWidget =
			SNew(SAssetThumbnail)
			.AssetThumbnail(SharedThis(this))
			.ThumbnailPool(ThumbnailPool.Pin())
			.AllowFadeIn(InConfig.bAllowFadeIn)
			.ForceGenericThumbnail(InConfig.bForceGenericThumbnail)
			.Label(InConfig.ThumbnailLabel)
			.HighlightedText(InConfig.HighlightedText)
			.HintColorAndOpacity(InConfig.HintColorAndOpacity)
			.AllowHintText(InConfig.bAllowHintText)
			.AllowRealTimeOnHovered(InConfig.bAllowRealTimeOnHovered)
			.ClassThumbnailBrushOverride(InConfig.ClassThumbnailBrushOverride)
			.AllowAssetSpecificThumbnailOverlay(AssetThumbnailOverlayAttribute)
			.AllowAssetSpecificThumbnailOverlayIndicator(AssetThumbnailOverlayIndicatorAttribute)
			.AssetTypeColorOverride(InConfig.AssetTypeColorOverride)
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			.Padding(InConfig.Padding)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			.BorderPadding(InConfig.BorderPadding)
			.GenericThumbnailSize(InConfig.GenericThumbnailSize)
			.AssetSystemInfoProvider(InConfig.AssetSystemInfoProvider)
			.AdditionalTooltipInSmallView(InConfig.AdditionalTooltipInSmallView)
			.AllowAssetStatusThumbnailOverlay(InConfig.AllowAssetStatusThumbnailOverlay)
			.ShowAssetColor(InConfig.ShowAssetColor)
			.CanDisplayEditModePrimitiveTools(InConfig.bCanDisplayEditModePrimitiveTools)
			.IsEditModeVisible(InConfig.IsEditModeVisible)
			.AssetBorderImageOverride(InConfig.AssetBorderImageOverride)
			.ShowAssetBorder(InConfig.ShowAssetBorder)
			.AlwaysExpandTooltip(InConfig.AlwaysExpandTooltip)
			.ColorStripOrientation(InConfig.ColorStripOrientation);
	}
	else
	{
		ThumbnailWidget =
			SNew(SAssetThumbnail)
			.AssetThumbnail(SharedThis(this))
			.ThumbnailPool(ThumbnailPool.Pin())
			.AllowFadeIn(InConfig.bAllowFadeIn)
			.ForceGenericThumbnail(InConfig.bForceGenericThumbnail)
			.Label(InConfig.ThumbnailLabel)
			.HighlightedText(InConfig.HighlightedText)
			.HintColorAndOpacity(InConfig.HintColorAndOpacity)
			.AllowHintText(InConfig.bAllowHintText)
			.AllowRealTimeOnHovered(InConfig.bAllowRealTimeOnHovered)
			.ClassThumbnailBrushOverride(InConfig.ClassThumbnailBrushOverride)
			.AllowAssetSpecificThumbnailOverlay(AssetThumbnailOverlayAttribute)
			.AssetTypeColorOverride(InConfig.AssetTypeColorOverride)
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			.Padding(InConfig.Padding)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			.GenericThumbnailSize(InConfig.GenericThumbnailSize)
			.ColorStripOrientation(InConfig.ColorStripOrientation);
	}

	return ThumbnailWidget.ToSharedRef();
}

void FAssetThumbnail::RefreshThumbnail()
{
	if ( ThumbnailPool.IsValid() && AssetData.IsValid() )
	{
		ThumbnailPool.Pin()->RefreshThumbnail( SharedThis(this) );
	}
}

void FAssetThumbnail::SetRealTime(bool bRealTime)
{
	if (ThumbnailPool.IsValid() && AssetData.IsValid())
	{
		ThumbnailPool.Pin()->SetRealTimeThumbnail( SharedThis(this), bRealTime );
	}
}

FAssetThumbnailPool::FAssetThumbnailPool( uint32 InNumInPool, double InMaxFrameTimeAllowance, uint32 InMaxRealTimeThumbnailsPerFrame )
	: NumInPool( InNumInPool )
	, MaxRealTimeThumbnailsPerFrame( InMaxRealTimeThumbnailsPerFrame )
	, MaxFrameTimeAllowance( InMaxFrameTimeAllowance )
{
	FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FAssetThumbnailPool::OnAssetLoaded);

	UThumbnailManager::Get().GetOnThumbnailDirtied().AddRaw(this, &FAssetThumbnailPool::OnThumbnailDirtied);

	// Add the custom thumbnail tag to the list of tags that the asset registry can parse
	TSet<FName>& MetaDataTagsForAssetRegistry = UObject::GetMetaDataTagsForAssetRegistry();
	MetaDataTagsForAssetRegistry.Add(CustomThumbnailTagName);
}

FAssetThumbnailPool::~FAssetThumbnailPool()
{
	UThumbnailManager* ThumbnailManager = UThumbnailManager::TryGet();
	if ( IsValid( ThumbnailManager ) &&
		!ThumbnailManager->HasAnyFlags( RF_BeginDestroyed | RF_FinishDestroyed ) )
	{
		ThumbnailManager->GetOnThumbnailDirtied().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);

	// Release all the texture resources
	ReleaseResources();
}

FAssetThumbnailPool::FThumbnailInfo::~FThumbnailInfo()
{
	if( ThumbnailTexture || ThumbnailRenderTarget )
	{
		ENQUEUE_RENDER_COMMAND(ReleaseThumbnailTextures)(
		[ThumbnailTexture = ThumbnailTexture, ThumbnailRenderTarget = ThumbnailRenderTarget](FRHICommandListImmediate& RHICmdList)
		{
			if (ThumbnailTexture)
			{
				ThumbnailTexture->ClearTextureData();
				ThumbnailTexture->ReleaseResource();
				delete ThumbnailTexture;
			}
			if (ThumbnailRenderTarget)
			{
				ThumbnailRenderTarget->ReleaseResource();
				delete ThumbnailRenderTarget;
			}
		});
		
		ThumbnailTexture = nullptr;
		ThumbnailRenderTarget = nullptr;
	}
}

void FAssetThumbnailPool::ReleaseResources()
{
	// Clear all pending render requests
	ThumbnailsToRenderStack.Empty();
	RealTimeThumbnails.Empty();
	RealTimeThumbnailsToRender.Empty();

	// Wait for all resources to be released
	FlushRenderingCommands();

	auto EnsureThumbnailIsUnique = [](const TSharedRef<FThumbnailInfo>& Thumb)
	{
		if ( !Thumb.IsUnique() )
		{
			ensureMsgf(0, TEXT("Thumbnail info for '%s' is still referenced by '%d' other objects"), *Thumb->AssetData.GetObjectPathString(), Thumb.GetSharedReferenceCount());
		}
	};
	
	// Make sure there are no more references to any of our thumbnails now that rendering commands have been flushed
	for(const TPair<FThumbId, TSharedRef<FThumbnailInfo>>& Pair : ThumbnailToTextureMap)
	{
		EnsureThumbnailIsUnique(Pair.Value);
	}
	ThumbnailToTextureMap.Empty();
	
	for(const TSharedRef<FThumbnailInfo>& Thumbnail : FreeThumbnails)
	{
		EnsureThumbnailIsUnique(Thumbnail);
	}
	FreeThumbnails.Empty();
}

TStatId FAssetThumbnailPool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT( FAssetThumbnailPool, STATGROUP_Tickables );
}

void FAssetThumbnailPool::ProcessPendingObjectThumbnails()
{
	const bool bShadersAreCompiling = (GShaderCompilingManager && GShaderCompilingManager->IsCompiling());

	if (bShadersAreCompiling)
	{
		for ( TMap< FSoftObjectPath, ThumbnailTools::FAsyncObjectThumbnail >::TIterator PendingObjectThumbnailIt = PendingObjectThumbnails.CreateIterator(); PendingObjectThumbnailIt; ++PendingObjectThumbnailIt )
		{
			FSoftObjectPath ObjectPath = PendingObjectThumbnailIt.Key();

			if ( UObject* Asset = ObjectPath.ResolveObject() )
			{
				UE_LOGF(LogThumbnails, Verbose, "Shaders are compiling, throwing away pending object thumbnail for %ls and retrying", *Asset->GetFullName());
				RefreshThumbnailsFor(Asset);
			}
		}

		PendingObjectThumbnails.Empty();
	}
	else
	{
		for ( TMap< FSoftObjectPath, ThumbnailTools::FAsyncObjectThumbnail >::TIterator PendingObjectThumbnailIt = PendingObjectThumbnails.CreateIterator(); PendingObjectThumbnailIt; ++PendingObjectThumbnailIt )
		{
			FSoftObjectPath ObjectPath = PendingObjectThumbnailIt.Key();

			if ( UObject* Asset = ObjectPath.ResolveObject() )
			{
				if ( !PendingObjectThumbnailIt.Value().IsObjectThumbnailReady() && PendingObjectThumbnailIt.Value().IsTextureReadbackReady() )
				{
					UE_LOGF(LogThumbnails, Verbose, "Enqueue thumbnail readback data copy for %ls", *Asset->GetFullName());
					PendingObjectThumbnailIt.Value().EnqueueReadbackDataCopy();
				}
				else if ( PendingObjectThumbnailIt.Value().IsObjectThumbnailReady() )
				{
					LocalObjectThumbnailsCache.Add(ObjectPath, PendingObjectThumbnailIt.Value().ConsumeObjectThumbnailUnchecked());
					UE_LOGF(LogThumbnails, Verbose, "Cached object thumbnail for %ls", *Asset->GetFullName());
				
					PendingObjectThumbnailIt.RemoveCurrent();
				}
			}
			else
			{
				UE_LOGF(LogThumbnails, Verbose, "Invalid pending object thumbnail for %ls, removing.", *ObjectPath.ToString());
				PendingObjectThumbnailIt.RemoveCurrent();
			}
		}
	}
}

void FAssetThumbnailPool::ProcessRecentlyLoadedAssets()
{
	// If there were any assets loaded since last frame that we are currently displaying thumbnails for, push them on the render stack now.
	if ( RecentlyLoadedAssets.Num() > 0 )
	{
		for ( const FSoftObjectPath& RecentlyLoadedAsset : RecentlyLoadedAssets )
		{
			RefreshThumbnailsFor(RecentlyLoadedAsset);
		}

		RecentlyLoadedAssets.Empty();
	}
}

bool FAssetThumbnailPool::IsTickable() const
{
	return RecentlyLoadedAssets.Num() > 0 || ThumbnailsToRenderStack.Num() > 0 || RealTimeThumbnails.Num() > 0 || PendingObjectThumbnails.Num() > 0;
}

UE_TRACE_EVENT_BEGIN(Cpu, FAssetThumbnailPool_Tick_RenderingThumbnail, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ObjectPath)
	UE_TRACE_EVENT_FIELD(bool, IsRealTime)
UE_TRACE_EVENT_END()

void FAssetThumbnailPool::Tick( float DeltaTime )
{
	// If throttling do not tick unless drag dropping which could have a thumbnail as the cursor decorator
	if (FSlateApplication::IsInitialized() && !FSlateApplication::Get().IsDragDropping() && !FSlateThrottleManager::Get().IsAllowingExpensiveTasks() && !FSlateApplication::Get().AnyMenusVisible())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetThumbnailPool::Tick);
	
	ProcessPendingObjectThumbnails();
	ProcessRecentlyLoadedAssets();

	// If we have dynamic thumbnails and we are done rendering the last batch of dynamic thumbnails, start a new batch as long as real-time thumbnails are enabled
	const bool bIsInPIEOrSimulate = GEditor->PlayWorld != NULL || GEditor->bIsSimulatingInEditor;
	const bool bShouldUseRealtimeThumbnails = GetDefault<UContentBrowserSettings>()->RealTimeThumbnails && !bIsInPIEOrSimulate;
	if ( bShouldUseRealtimeThumbnails && RealTimeThumbnails.Num() > 0 && RealTimeThumbnailsToRender.Num() == 0 )
	{
		double CurrentTime = FPlatformTime::Seconds();
		for ( int32 ThumbIdx = RealTimeThumbnails.Num() - 1; ThumbIdx >= 0; --ThumbIdx )
		{
			const TSharedRef<FThumbnailInfo>& Thumb = RealTimeThumbnails[ThumbIdx];
			if ( Thumb->AssetData.IsAssetLoaded() )
			{
				// Only render thumbnails that have been requested recently
				if ( (CurrentTime - Thumb->LastAccessTime) < 1.f )
				{
					RealTimeThumbnailsToRender.Add(Thumb);
				}
			}
			else
			{
				RealTimeThumbnails.RemoveAt(ThumbIdx);
			}
		}
	}

	uint32 NumRealTimeThumbnailsRenderedThisFrame = 0;
	// If there are any thumbnails to render, pop one off the stack and render it.
	if( ThumbnailsToRenderStack.Num() + RealTimeThumbnailsToRender.Num() > 0 )
	{
		TArray<TSharedRef<FThumbnailInfo>, TInlineAllocator<4>> ThumbnailsToRetry;
		TArray<TSharedRef<FThumbnailInfo>, TInlineAllocator<4>> RealTimeThumbnailsToRetry;

		double FrameStartTime = FPlatformTime::Seconds();
		// Render as many thumbnails as we are allowed to
		while( ThumbnailsToRenderStack.Num() + RealTimeThumbnailsToRender.Num() > 0 && FPlatformTime::Seconds() - FrameStartTime < MaxFrameTimeAllowance )
		{
			bool bIsRealTimeThumbnail = false;
			TSharedPtr<FThumbnailInfo> Info;
			if ( ThumbnailsToRenderStack.Num() > 0 )
			{
				Info = ThumbnailsToRenderStack.Pop();
			}
			else if (RealTimeThumbnailsToRender.Num() > 0 && NumRealTimeThumbnailsRenderedThisFrame < MaxRealTimeThumbnailsPerFrame )
			{
				Info = RealTimeThumbnailsToRender.Pop();
				NumRealTimeThumbnailsRenderedThisFrame++;
				bIsRealTimeThumbnail = true;
			}
			else
			{
				// No thumbnails left to render or we don't want to render any more
				break;
			}

			if( Info.IsValid() )
			{
				bool bRetryOnFailure = false;
				TSharedRef<FThumbnailInfo> InfoRef = Info.ToSharedRef();

				if ( InfoRef->AssetData.IsValid() )
				{
#if CPUPROFILERTRACE_ENABLED
					UE_TRACE_LOG_SCOPED_T(Cpu, FAssetThumbnailPool_Tick_RenderingThumbnail, CpuChannel)
						<< FAssetThumbnailPool_Tick_RenderingThumbnail.ObjectPath(*InfoRef->AssetData.GetObjectPathString())
						<< FAssetThumbnailPool_Tick_RenderingThumbnail.IsRealTime(bIsRealTimeThumbnail);
#endif // CPUPROFILERTRACE_ENABLED

					FAssetData CustomThumbnailAsset;
					// Check if a different asset should be used to generate the thumbnail for this asset
					FString CustomThumbnailTagValue;
					if (InfoRef->AssetData.GetTagValue(CustomThumbnailTagName, CustomThumbnailTagValue))
					{
						if (FPackageName::IsValidObjectPath(CustomThumbnailTagValue))
						{
							CustomThumbnailAsset = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get().GetAssetByObjectPath(FSoftObjectPath(CustomThumbnailTagValue));
						}
					}

					bool bLoadedThumbnail = LoadThumbnail(InfoRef, bRetryOnFailure, CustomThumbnailAsset);

					// If we failed to load a custom thumbnail, then load the custom thumbnail's asset and try again
					if (!bLoadedThumbnail && !bRetryOnFailure && CustomThumbnailAsset.IsValid())
					{
						if (UPackage* CustomThumbnailPackage = CustomThumbnailAsset.GetPackage())
						{
							UObject* CustomThumbnail = FindObjectFast<UObject>(CustomThumbnailPackage, CustomThumbnailAsset.AssetName);
							if (!CustomThumbnail)
							{
								// Because the custom thumbnail asset can be GCed (RF_Standalone flag cleared), 
								// its package might need to be reloaded
								CustomThumbnailPackage = LoadPackage(NULL, *CustomThumbnailAsset.PackageName.ToString(), LOAD_None);
								CustomThumbnail = FindObjectFast<UObject>(CustomThumbnailPackage, CustomThumbnailAsset.AssetName);
							}

							if (CustomThumbnail)
							{
								bLoadedThumbnail = LoadThumbnail(InfoRef, bRetryOnFailure, CustomThumbnailAsset);
								if (!bRetryOnFailure)
								{
									// Clear RF_Standalone flag on the loaded custom thumbnail asset so it gets GCed eventually
									CustomThumbnail->ClearFlags(RF_Standalone);
								}
							}
						}
					}

					if ( bLoadedThumbnail )
					{
						// Mark it as updated
						InfoRef->LastUpdateTime = FPlatformTime::Seconds();

						// Notify listeners that a thumbnail has been rendered
						ThumbnailRenderedEvent.Broadcast(InfoRef->AssetData);
					}
					// Do not send a failure event for this asset yet if shaders are still compiling or the asset itself is compiling.
					// The failure event will disable the rendering of this asset for good and we need to have a chance to 
					// re-render it when everything settles down.
					else if (!bRetryOnFailure)
					{
						// Notify listeners that a thumbnail render has failed
						ThumbnailRenderFailedEvent.Broadcast(InfoRef->AssetData);
						UE_LOGF(LogThumbnails, Verbose, "No thumbnail found for %ls", *InfoRef->AssetData.GetFullName());
					}
					else // bRetryOnFailure
					{
						if (bIsRealTimeThumbnail)
						{
							RealTimeThumbnailsToRetry.Push(InfoRef);
						}
						else
						{
							ThumbnailsToRetry.Push(InfoRef);
						}
					}
				}
			}
		}

		ThumbnailsToRenderStack.Append(ThumbnailsToRetry);
		RealTimeThumbnailsToRender.Append(RealTimeThumbnailsToRetry);
	}
}

bool FAssetThumbnailPool::LoadThumbnail(TSharedRef<FThumbnailInfo> ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender)
{
	const FAssetData& AssetData = CustomAssetToRender.IsValid() ? CustomAssetToRender : ThumbnailInfo->AssetData;

	constexpr bool bLoad = false;
	UObject* Asset = AssetData.FastGetAsset(bLoad);
	const bool bIsAssetLoaded = IsValid(Asset);

	if (bIsAssetLoaded)
	{
		return LoadThumbnailForLoadedAsset(ThumbnailInfo, bOutRetryOnFailure, CustomAssetToRender);
	}
	else
	{
		return LoadThumbnailForUnloadedAsset(ThumbnailInfo, CustomAssetToRender);
	}
}

bool FAssetThumbnailPool::LoadThumbnailForLoadedAsset(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender)
{
	const FAssetData& AssetData = CustomAssetToRender.IsValid() ? CustomAssetToRender : ThumbnailInfo->AssetData;

	const FObjectThumbnail* FoundThumbnail = nullptr;
	FThumbnailMap ThumbnailMap;

	TArray<TFunction<bool()>, TInlineAllocator<3>> LoadThumbnailSourcesPriority;

	TFunction<bool()> ThumbnailFromCacheSource = 
		[this, ThumbnailInfo, CustomAssetToRender]()
		{
			if (LoadThumbnailFromObjectThumbnailInCache(ThumbnailInfo, CustomAssetToRender))
			{
				UE_LOGF(LogThumbnails, Verbose, "Thumbnail found in cache for %ls", *ThumbnailInfo->AssetData.GetFullName());
				return true;
			}

			return false;
		};

	TFunction<bool()> ThumbnailFromDiskSource =
		[this, ThumbnailInfo, CustomAssetToRender]()
		{
			if (LoadThumbnailFromObjectThumbnailOnDisk(ThumbnailInfo, CustomAssetToRender))
			{
				UE_LOGF(LogThumbnails, Verbose, "Thumbnail found on disk for %ls", *ThumbnailInfo->AssetData.GetFullName());
				return true;
			}

			return false;
		};

	TFunction<bool()> ThumbnailFromRenderSource = 
		[this, ThumbnailInfo, &bOutRetryOnFailure, CustomAssetToRender]()
		{
			if (MakeFreshThumbnailFromRender(ThumbnailInfo, bOutRetryOnFailure, CustomAssetToRender))
			{
				UE_LOGF(LogThumbnails, Verbose, "Fresh thumbnail rendered for %ls", *ThumbnailInfo->AssetData.GetFullName());
				return true;
			}

			return false;
		};

	const bool bIsCookedAsset = AssetData.PackageFlags & PKG_Cooked;
	const bool bIsInitialThumbnailLoad = (ThumbnailInfo->LastUpdateTime <= 0.0);

	if (bIsCookedAsset)
	{
		if (bIsInitialThumbnailLoad)
		{
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromCacheSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromDiskSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromRenderSource));
		}
		else
		{
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromDiskSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromRenderSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromCacheSource));
		}
	}
	else // Uncooked asset
	{
		if (bIsInitialThumbnailLoad)
		{
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromCacheSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromRenderSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromDiskSource));
		}
		else
		{
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromRenderSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromCacheSource));
			LoadThumbnailSourcesPriority.Add(MoveTemp(ThumbnailFromDiskSource));
		}
	}

	for (const TFunction<bool()>& LoadThumbnailFromSource : LoadThumbnailSourcesPriority)
	{
		if (LoadThumbnailFromSource())
		{
			return true;
		}
		else if (bOutRetryOnFailure)
		{
			return false;
		}
	}

	return false;
}

bool FAssetThumbnailPool::LoadThumbnailForUnloadedAsset(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender)
{
	if (LoadThumbnailFromObjectThumbnailInCache(ThumbnailInfo, CustomAssetToRender))
	{
		UE_LOGF(LogThumbnails, Verbose, "Thumbnail found in cache for %ls", *ThumbnailInfo->AssetData.GetFullName());
		return true;
	}

	if (LoadThumbnailFromObjectThumbnailOnDisk(ThumbnailInfo, CustomAssetToRender))
	{
		UE_LOGF(LogThumbnails, Verbose, "Thumbnail found on disk for %ls", *ThumbnailInfo->AssetData.GetFullName());
		return true;
	}

	return false;
}

bool FAssetThumbnailPool::LoadThumbnailFromObjectThumbnailInCache(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender)
{
	const FAssetData& AssetData = CustomAssetToRender.IsValid() ? CustomAssetToRender : ThumbnailInfo->AssetData;

	const FObjectThumbnail* FoundObjectThumbnail = LocalObjectThumbnailsCache.Find(ThumbnailInfo->AssetData.GetSoftObjectPath());

	if (!FoundObjectThumbnail || FoundObjectThumbnail->GetImageWidth() != ThumbnailInfo->Width || FoundObjectThumbnail->GetImageHeight() != ThumbnailInfo->Height)
	{
		FoundObjectThumbnail = ThumbnailTools::FindCachedThumbnail(AssetData.GetFullName());
	}

	if (FoundObjectThumbnail && !FoundObjectThumbnail->IsDirty())
	{
		return CopyObjectThumbnailImageDataToSlateTextureData(ThumbnailInfo, *FoundObjectThumbnail);
	}
	else
	{
		if (FoundObjectThumbnail && FoundObjectThumbnail->IsDirty())
		{
			UE_LOGF(LogThumbnails, Verbose, "Ignoring cached object thumbnail for %ls because its dirty", *AssetData.GetFullName());
		}

		return false;
	}
}

bool FAssetThumbnailPool::LoadThumbnailFromObjectThumbnailOnDisk(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FAssetData& CustomAssetToRender)
{
	const FAssetData& AssetData = CustomAssetToRender.IsValid() ? CustomAssetToRender : ThumbnailInfo->AssetData;

	FThumbnailMap ThumbnailMap;
	const FObjectThumbnail* FoundObjectThumbnail = AssetThumbnailPool::LoadThumbnailsFromPackage(AssetData, ThumbnailMap);

	if (FoundObjectThumbnail)
	{
		return CopyObjectThumbnailImageDataToSlateTextureData(ThumbnailInfo, *FoundObjectThumbnail);
	}
	else
	{
		return false;
	}
}

bool FAssetThumbnailPool::MakeFreshThumbnailFromRender(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, bool& bOutRetryOnFailure, const FAssetData& CustomAssetToRender)
{
	const FAssetData& AssetData = CustomAssetToRender.IsValid() ? CustomAssetToRender : ThumbnailInfo->AssetData;

	// Render a fresh thumbnail from the loaded asset if possible
	UObject* Asset = AssetData.FastGetAsset();
	if (!IsValid(Asset))
	{
		return false;
	}

	const bool bAreShadersCompiling = (GShaderCompilingManager && GShaderCompilingManager->IsCompiling());
	if (!bAreShadersCompiling)
	{
		// Avoid rendering the thumbnail of an asset that is currently edited asynchronously
		const IInterface_AsyncCompilation* Interface_AsyncCompilation = Cast<IInterface_AsyncCompilation>(Asset);
		const bool bIsAssetStillCompiling = Interface_AsyncCompilation && Interface_AsyncCompilation->IsCompiling();

		if (!bIsAssetStillCompiling)
		{
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Asset);
			if (RenderInfo != nullptr && RenderInfo->Renderer != nullptr)
			{
				FThumbnailInfo_RenderThread ThumbInfo = ThumbnailInfo.Get();

				auto EnqueueThumbnailRender = [this, Asset, ThumbInfo, ThumbnailInfo]()
				{
					ENQUEUE_RENDER_COMMAND(SyncSlateTextureCommand)(
					[ThumbInfo](FRHICommandListImmediate& RHICmdList)
					{
						if (ThumbInfo.ThumbnailTexture->GetTypedResource() != ThumbInfo.ThumbnailRenderTarget->GetTextureRHI())
						{
							ThumbInfo.ThumbnailTexture->ClearTextureData();
							ThumbInfo.ThumbnailTexture->ReleaseRHI();
							ThumbInfo.ThumbnailTexture->SetRHIRef(ThumbInfo.ThumbnailRenderTarget->GetTextureRHI(), ThumbInfo.Width, ThumbInfo.Height);
						}
					});

					TOptional<ThumbnailTools::FAsyncObjectThumbnail> OptionalAsyncObjectThumbnail =
						ThumbnailTools::RenderThumbnailAsync(
							Asset,
							ThumbnailInfo->Width,
							ThumbnailInfo->Height,
							ThumbnailTools::EThumbnailTextureFlushMode::NeverFlush,
							ThumbnailInfo->ThumbnailRenderTarget
						);

					if (OptionalAsyncObjectThumbnail)
					{
						PendingObjectThumbnails.Add(ThumbnailInfo->AssetData.GetSoftObjectPath(), MoveTemp(OptionalAsyncObjectThumbnail.GetValue()));
						UE_LOGF(LogThumbnails, Verbose, "Future Thumbnail Object being tracked for %ls (%dx%d)", *Asset->GetFullName(), ThumbnailInfo->Width, ThumbnailInfo->Height);
					}
				};

				EThumbnailRenderFrequency ThumbnailRenderFrequency = RenderInfo->Renderer->GetThumbnailRenderFrequency(Asset);

				switch (ThumbnailRenderFrequency)
				{
					case EThumbnailRenderFrequency::Realtime:
					{
						EnqueueThumbnailRender();
						return true;
					}
					case EThumbnailRenderFrequency::OnPropertyChange:
					{
						if (ThumbnailInfo->LastUpdateTime <= 0.0)
						{
							EnqueueThumbnailRender();
							return true;
						}
						break;
					}
					case EThumbnailRenderFrequency::OnAssetSave:
					{
						// We don't allow rendering fresh thumbnails on demand for this frequency
						return false;
					}
					case EThumbnailRenderFrequency::Once:
					{
						// Eagerly return if we aren't interested in cached thumbnails
						return true;
					}
					default:
					{
						break;
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogThumbnails, Verbose, "Asset is compiling, can't render thumbnail for %ls. Will retry.", *AssetData.GetFullName());
			bOutRetryOnFailure = true;
		}
	}
	else
	{
		UE_LOGF(LogThumbnails, Verbose, "Shaders are compiling, can't render thumbnail for %ls. Will retry.", *AssetData.GetFullName());
		bOutRetryOnFailure = true;
	}

	return false;
}

bool FAssetThumbnailPool::CopyObjectThumbnailImageDataToSlateTextureData(const TSharedRef<FThumbnailInfo>& ThumbnailInfo, const FObjectThumbnail& ObjectThumbnail)
{
	FImageView Image = ObjectThumbnail.GetImage();

	if ( Image.GetNumPixels() > 0 )
	{
		// Make bulk data for updating the texture memory later
		FSlateTextureData* BulkData = new FSlateTextureData(Image);

		// Update the texture RHI
		FThumbnailInfo_RenderThread ThumbInfo = ThumbnailInfo.Get();
		ENQUEUE_RENDER_COMMAND(ClearSlateTextureCommand)(
			[ThumbInfo, BulkData](FRHICommandListImmediate& RHICmdList)
			{
				if (ThumbInfo.ThumbnailTexture->GetTypedResource() == ThumbInfo.ThumbnailRenderTarget->GetTextureRHI())
				{
					ThumbInfo.ThumbnailTexture->SetRHIRef(NULL, ThumbInfo.Width, ThumbInfo.Height);
				}

				ThumbInfo.ThumbnailTexture->SetTextureData(MakeShareable(BulkData));
				ThumbInfo.ThumbnailTexture->UpdateRHI(RHICmdList);
			});

		return true;
	}

	return false;
}

FSlateTexture2DRHIRef* FAssetThumbnailPool::AccessTexture( const FAssetData& AssetData, uint32 Width, uint32 Height )
{
	if(!AssetData.IsValid() || Width == 0 || Height == 0)
	{
		return nullptr;
	}
	
	FThumbId ThumbId( AssetData.GetSoftObjectPath(), Width, Height ) ;
	// Check to see if a thumbnail for this asset exists.  If so we don't need to render it
	const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find( ThumbId );
	TSharedPtr<FThumbnailInfo> ThumbnailInfo;
	if( ThumbnailInfoPtr )
	{
		ThumbnailInfo = *ThumbnailInfoPtr;
	}
	else
	{
		// Check if there is any thumbnail in the free pool that can be used
		// If there is, use it. This does not change the overall number of resources used
		for (TArray<TSharedRef<FThumbnailInfo>>::TIterator It(FreeThumbnails); It; ++It)
		{
			TSharedRef<FThumbnailInfo>& FreeThumbnail = *It;
			if (FreeThumbnail->Height == Height && FreeThumbnail->Width == Width)
			{
				ThumbnailInfo = FreeThumbnail;
				It.RemoveCurrentSwap();
				break;
			}
		}

		// If we didn't find one and we have reached the pool limit, remove any thumbnail from the free thumbnails if any. This will free up a slot
		// Otherwise, if there are no free thumbnails, evict the LRU used thumbnail to free up a slot.
		// Note, with luck, the LRU element might be suitable for reuse
		if( !ThumbnailInfo && (FreeThumbnails.Num() + ThumbnailToTextureMap.Num() == NumInPool) )
		{
			if (!FreeThumbnails.IsEmpty())
			{
				FreeThumbnails.Pop();
			}
			else
			{
				// Find the thumbnail which was accessed last and potentially use it for the new thumbnail
				double LastAccessTime = std::numeric_limits<double>::max();
				const FThumbId* AssetToRemove = nullptr;
				for( TMap< FThumbId, TSharedRef<FThumbnailInfo> >::TConstIterator It(ThumbnailToTextureMap); It; ++It )
				{
					if( It.Value()->LastAccessTime < LastAccessTime )
					{
						LastAccessTime = It.Value()->LastAccessTime;
						AssetToRemove = &It.Key();
					}
				}

				check( AssetToRemove );
				// Remove the old mapping. If matching our desired size, we reuse it, otherwise we discard it
				ThumbnailInfo = ThumbnailToTextureMap.FindAndRemoveChecked( *AssetToRemove );
				if (ThumbnailInfo->Height != Height || ThumbnailInfo->Width != Width)
				{
					ThumbnailInfo = nullptr;
				}
			}
		}
		
		// If there are no free thumbnail resources or space in the pool, create a new one
		if( !ThumbnailInfo )
		{
			// There are no free thumbnail resources
			check( (uint32)(FreeThumbnails.Num() + ThumbnailToTextureMap.Num()) < NumInPool );
			// The pool isn't used up so just make a new texture 

			// Make new thumbnail info if it doesn't exist
			ThumbnailInfo = MakeShareable( new FThumbnailInfo(Width, Height) );
			
			// Set the thumbnail and asset on the info. It is NOT safe to change or NULL these pointers until ReleaseResources.
			ThumbnailInfo->ThumbnailTexture = new FSlateTexture2DRHIRef( Width, Height, PF_B8G8R8A8, NULL, TexCreate_None );
			ThumbnailInfo->ThumbnailRenderTarget = new FSlateTextureRenderTarget2DResource(FLinearColor::Black, Width, Height, PF_B8G8R8A8, SF_Point, TA_Wrap, TA_Wrap, 0.0f);

			BeginInitResource( ThumbnailInfo->ThumbnailTexture );
			BeginInitResource( ThumbnailInfo->ThumbnailRenderTarget );
		}


		check( ThumbnailInfo.IsValid() );
		TSharedRef<FThumbnailInfo> ThumbnailRef = ThumbnailInfo.ToSharedRef();

		// Map the object to its thumbnail info
		ThumbnailToTextureMap.Add( ThumbId, ThumbnailRef );

		ThumbnailInfo->AssetData = AssetData;
	
		// Mark this thumbnail as needing to be updated
		ThumbnailInfo->LastUpdateTime = -1.0;

		// Request that the thumbnail be rendered as soon as possible
		ThumbnailsToRenderStack.Push( ThumbnailRef );
		UE_LOGF(LogThumbnails, Verbose, "Queued thumbnail to render for %ls (%dx%d)", *ThumbnailInfo->AssetData.GetFullName(), Width, Height);
	}

	// This thumbnail was accessed, update its last time to the current time
	// We'll use LastAccessTime to determine the order to recycle thumbnails if the pool is full
	ThumbnailInfo->LastAccessTime = FPlatformTime::Seconds();

	return ThumbnailInfo->ThumbnailTexture;
}

void FAssetThumbnailPool::AddReferencer( const FAssetThumbnail& AssetThumbnail )
{
	FIntPoint Size = AssetThumbnail.GetSize();
	if ( !AssetThumbnail.GetAssetData().IsValid() || Size.X == 0 || Size.Y == 0 )
	{
		// Invalid referencer
		return;
	}

	// Generate a key and look up the number of references in the RefCountMap
	FThumbId ThumbId( AssetThumbnail.GetAssetData().GetSoftObjectPath(), Size.X, Size.Y );
	int32* RefCountPtr = RefCountMap.Find(ThumbId);

	if ( RefCountPtr )
	{
		// Already in the map, increment a reference
		(*RefCountPtr)++;
	}
	else
	{
		// New referencer, add it to the map with a RefCount of 1
		RefCountMap.Add(ThumbId, 1);
	}
}

void FAssetThumbnailPool::RemoveReferencer( const FAssetThumbnail& AssetThumbnail )
{
	FIntPoint Size = AssetThumbnail.GetSize();
	const FSoftObjectPath ObjectPath = AssetThumbnail.GetAssetData().GetSoftObjectPath();
	if ( ObjectPath.IsNull() || Size.X == 0 || Size.Y == 0 )
	{
		// Invalid referencer
		return;
	}

	// Generate a key and look up the number of references in the RefCountMap
	FThumbId ThumbId( ObjectPath, Size.X, Size.Y ) ;
	int32* RefCountPtr = RefCountMap.Find(ThumbId);

	// This should complement an AddReferencer so this entry should be in the map
	if ( RefCountPtr )
	{
		// Decrement the RefCount
		(*RefCountPtr)--;

		// If we reached zero, free the thumbnail and remove it from the map.
		if ( (*RefCountPtr) <= 0 )
		{
			RefCountMap.Remove(ThumbId);
			FreeThumbnail(ObjectPath, Size.X, Size.Y);
		}
	}
	else
	{
		// This FAssetThumbnail did not reference anything or was deleted after the pool was deleted.
	}
}

bool FAssetThumbnailPool::IsInRenderStack( const TSharedPtr<FAssetThumbnail>& Thumbnail ) const
{
	const FAssetData& AssetData = Thumbnail->GetAssetData();
	const uint32 Width = Thumbnail->GetSize().X;
	const uint32 Height = Thumbnail->GetSize().Y;

	if ( ensure(AssetData.IsValid()) && ensure(Width > 0) && ensure(Height > 0) )
	{
		FThumbId ThumbId(AssetData.GetSoftObjectPath(), Width, Height);
		const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find( ThumbId );
		if ( ThumbnailInfoPtr )
		{
			return ThumbnailsToRenderStack.Contains(*ThumbnailInfoPtr);
		}
	}

	return false;
}

bool FAssetThumbnailPool::IsRendered(const TSharedPtr<FAssetThumbnail>& Thumbnail) const
{
	const FAssetData& AssetData = Thumbnail->GetAssetData();
	const uint32 Width = Thumbnail->GetSize().X;
	const uint32 Height = Thumbnail->GetSize().Y;

	if (ensure(AssetData.IsValid()) && ensure(Width > 0) && ensure(Height > 0))
	{
		FThumbId ThumbId(AssetData.GetSoftObjectPath(), Width, Height);
		const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find(ThumbId);
		if (ThumbnailInfoPtr)
		{
			return (*ThumbnailInfoPtr).Get().LastUpdateTime >= 0.f;
		}
	}

	return false;
}

void FAssetThumbnailPool::PrioritizeThumbnails( const TArray< TSharedPtr<FAssetThumbnail> >& ThumbnailsToPrioritize, uint32 Width, uint32 Height )
{
	if ( ensure(Width > 0) && ensure(Height > 0) )
	{
		TSet<FSoftObjectPath> ObjectPathList;
		for ( int32 ThumbIdx = 0; ThumbIdx < ThumbnailsToPrioritize.Num(); ++ThumbIdx )
		{
			ObjectPathList.Add(ThumbnailsToPrioritize[ThumbIdx]->GetAssetData().GetSoftObjectPath());
		}

		TArray< TSharedRef<FThumbnailInfo> > FoundThumbnails;
		for ( int32 ThumbIdx = ThumbnailsToRenderStack.Num() - 1; ThumbIdx >= 0; --ThumbIdx )
		{
			const TSharedRef<FThumbnailInfo>& ThumbnailInfo = ThumbnailsToRenderStack[ThumbIdx];
			if (ThumbnailInfo->Width == Width && ThumbnailInfo->Height == Height && ObjectPathList.Contains(ThumbnailInfo->AssetData.GetSoftObjectPath()))
			{
				FoundThumbnails.Add(ThumbnailInfo);
				ThumbnailsToRenderStack.RemoveAt(ThumbIdx);
			}
		}

		for ( int32 ThumbIdx = 0; ThumbIdx < FoundThumbnails.Num(); ++ThumbIdx )
		{
			ThumbnailsToRenderStack.Push(FoundThumbnails[ThumbIdx]);
		}
	}
}

void FAssetThumbnailPool::RefreshThumbnail( const TSharedPtr<FAssetThumbnail>& ThumbnailToRefresh )
{
	const FAssetData& AssetData = ThumbnailToRefresh->GetAssetData();
	const uint32 Width = ThumbnailToRefresh->GetSize().X;
	const uint32 Height = ThumbnailToRefresh->GetSize().Y;

	if ( ensure(AssetData.IsValid()) && ensure(Width > 0) && ensure(Height > 0) )
	{
		FThumbId ThumbId(AssetData.GetSoftObjectPath(), Width, Height);
		RefreshThumbnailFor(ThumbId);
	}
}

void FAssetThumbnailPool::SetRealTimeThumbnail( const TSharedPtr<FAssetThumbnail>& Thumbnail, bool bRealTimeThumbnail )
{
	const FAssetData& AssetData = Thumbnail->GetAssetData();
	const uint32 Width = Thumbnail->GetSize().X;
	const uint32 Height = Thumbnail->GetSize().Y;

	if (ensure(AssetData.IsValid()) && ensure(Width > 0) && ensure(Height > 0) )
	{
		FThumbId ThumbId(AssetData.GetSoftObjectPath(), Width, Height);
		const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find( ThumbId );
		if ( ThumbnailInfoPtr )
		{
			if ( bRealTimeThumbnail )
			{
				UE_LOGF(LogThumbnails, Verbose, "Setting real-time thumbnail ON for %ls", *(*ThumbnailInfoPtr)->AssetData.GetFullName());
				RealTimeThumbnails.AddUnique(*ThumbnailInfoPtr);
			}
			else
			{
				RealTimeThumbnails.Remove(*ThumbnailInfoPtr);
			}
		}
	}
}

void FAssetThumbnailPool::FreeThumbnail( const FSoftObjectPath& ObjectPath, uint32 Width, uint32 Height )
{
	if(ObjectPath.IsValid() && Width != 0 && Height != 0)
	{
		FThumbId ThumbId( ObjectPath, Width, Height ) ;

		const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find(ThumbId);
		if( ThumbnailInfoPtr )
		{
			UE_LOGF(LogThumbnails, Verbose, "Freeing thumbnail for %ls", *(*ThumbnailInfoPtr)->AssetData.GetFullName());

			TSharedRef<FThumbnailInfo> ThumbnailInfo = *ThumbnailInfoPtr;
			ThumbnailToTextureMap.Remove(ThumbId);
			ThumbnailsToRenderStack.Remove(ThumbnailInfo);
			RealTimeThumbnails.Remove(ThumbnailInfo);
			RealTimeThumbnailsToRender.Remove(ThumbnailInfo);

			FSlateTexture2DRHIRef* ThumbnailTexture = ThumbnailInfo->ThumbnailTexture;
			ENQUEUE_RENDER_COMMAND(ReleaseThumbnailTextureData)(
				[ThumbnailTexture](FRHICommandListImmediate& RHICmdList)
				{
					ThumbnailTexture->ClearTextureData();
				});

			FreeThumbnails.Add( ThumbnailInfo );
		}

		// Compress the FObjectThumbnail if there's one to reduce memory usage while the thumbnail isn't in the pool.
		if (FObjectThumbnail* ObjectThumbnail = LocalObjectThumbnailsCache.Find(ObjectPath))
		{
			if (ObjectThumbnail->GetCompressedDataSize() == 0)
			{
				ObjectThumbnail->CompressImageData();
			}

			ObjectThumbnail->EmptyUncompressedImageData();
		}
	}
}

void FAssetThumbnailPool::RefreshThumbnailsFor( const FSoftObjectPath& ObjectPath )
{
	for ( auto ThumbIt = ThumbnailToTextureMap.CreateIterator(); ThumbIt; ++ThumbIt)
	{
		if ( ThumbIt.Key().ObjectPath == ObjectPath )
		{
			RefreshThumbnailFor(ThumbIt.Key());
		}
	}
}

void FAssetThumbnailPool::RefreshThumbnailFor( const FThumbId ThumbId )
{
	const TSharedRef<FThumbnailInfo>* ThumbnailInfoPtr = ThumbnailToTextureMap.Find( ThumbId );
	if ( ThumbnailInfoPtr )
	{
		ThumbnailsToRenderStack.AddUnique( *ThumbnailInfoPtr );
		UE_LOGF(LogThumbnails, Verbose, "Refresh thumbnail requested for %ls", *(*ThumbnailInfoPtr)->AssetData.GetFullName());
	}
}

void FAssetThumbnailPool::OnAssetLoaded( UObject* Asset )
{
	if ( Asset != nullptr )
	{
		UE_LOGF(LogThumbnails, Verbose, "OnAssetLoaded %ls", *Asset->GetFullName());
		RecentlyLoadedAssets.Emplace( Asset );
	}
}

void FAssetThumbnailPool::OnThumbnailDirtied( const FSoftObjectPath& ObjectPath )
{
	UE_LOGF(LogThumbnails, Verbose, "OnThumbnailDirtied %ls", *ObjectPath.ToString());

	// Dirty any cached ObjectThumbnail for this ObjectPath
	if (FObjectThumbnail* ObjectThumbnail = LocalObjectThumbnailsCache.Find(ObjectPath))
	{
		ObjectThumbnail->MarkAsDirty();
	}

	RefreshThumbnailsFor( ObjectPath );
}
