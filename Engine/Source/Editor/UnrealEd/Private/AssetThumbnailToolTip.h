// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SToolTip.h"

class SAssetThumbnail;
class SDocumentationToolTip;

// AssetThumbnail ToolTip, Implementation is inside AssetThumbnail.cpp
class SAssetThumbnailToolTip : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SAssetThumbnailToolTip)
		: _AssetThumbnail()
		, _AlwaysExpandTooltip(false)
	{ }

		SLATE_ARGUMENT(TSharedPtr<SAssetThumbnail>, AssetThumbnail)
		SLATE_ATTRIBUTE(bool, AlwaysExpandTooltip)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// IToolTip interface
	virtual bool IsEmpty() const override;

	virtual void OnOpening() override;

	virtual void OnClosed() override;

	virtual bool IsInteractive() const override;
private:
	TWeakPtr<SAssetThumbnail> AssetThumbnail;
	TAttribute<bool> AlwaysExpandTooltip;
	TSharedPtr<SDocumentationToolTip> AssetToolTip;
};
