// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimport/AssetSourceFilenameCache.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "CoreGlobals.h"
#include "EditorFramework/AssetImportData.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

FAssetSourceFilenameCache::FAssetSourceFilenameCache()
	: CachePipe(UE_SOURCE_LOCATION)
{
	if (IsEngineExitRequested())
	{
		// This can get created for the first time on shutdown, if so don't do anything
		return;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	AssetRegistry.OnAssetsAdded().AddRaw(this, &FAssetSourceFilenameCache::HandleOnAssetsAdded);
	AssetRegistry.OnAssetsRemoved().AddRaw(this, &FAssetSourceFilenameCache::HandleOnAssetsRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FAssetSourceFilenameCache::HandleOnAssetRenamed);

	UAssetImportData::OnImportDataChanged.AddRaw(this, &FAssetSourceFilenameCache::HandleOnAssetUpdated);

	TArray<FAssetData> Assets;
	if (AssetRegistry.GetAllAssets(Assets))
	{
		HandleOnAssetsAdded(Assets);
	}
}

FAssetSourceFilenameCache& FAssetSourceFilenameCache::Get()
{
	static FAssetSourceFilenameCache Cache;
	return Cache;
}

void FAssetSourceFilenameCache::Shutdown()
{
	FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule)
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetsAdded().RemoveAll(this);
			AssetRegistry->OnAssetsRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
		}
	}
	
	AssetRenamedEvent.Clear();

	UAssetImportData::OnImportDataChanged.RemoveAll(this);
}

TOptional<FAssetImportInfo> FAssetSourceFilenameCache::ExtractAssetImportInfo(const FAssetData& AssetData)
{
	static const FName LegacySourceFilePathName("SourceFile");

	FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag(UObject::SourceFileTagName());
	if (Result.IsSet())
	{
		return FAssetImportInfo::FromJson(Result.GetValue());
	}
	else
	{
		FAssetDataTagMapSharedView::FFindTagResult ResultLegacy = AssetData.TagsAndValues.FindTag(LegacySourceFilePathName);
		if (ResultLegacy.IsSet())
		{
			FAssetImportInfo Legacy;
			Legacy.Insert(ResultLegacy.GetValue());
			return Legacy;
		}
		else
		{
			return TOptional<FAssetImportInfo>();
		}
	}
}

void FAssetSourceFilenameCache::HandleOnAssetsAdded(TConstArrayView<FAssetData> Assets)
{
	CachePipe.Launch(UE_SOURCE_LOCATION, [this, Assets=TArray<FAssetData>(Assets)](){
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetSourceFilenameCache::HandleOnAssetsAdded);
		const uint32 AssetCount = Assets.Num();
		TArray<TOptional<FAssetImportInfo>> ImportDataList;
		ImportDataList.SetNum(AssetCount);
		ParallelFor(Assets.Num(), [&Assets, &ImportDataList](int32 Index)
			{
				ImportDataList[Index] = ExtractAssetImportInfo(Assets[Index]);
			});

		SourceFileToObjectPathCache.Reserve(SourceFileToObjectPathCache.Num() + Assets.Num());
		for (uint32 Index = 0; Index < AssetCount; Index++)
		{
			const TOptional<FAssetImportInfo>& ImportData = ImportDataList[Index];
			if (ImportData.IsSet())
			{
				for (const auto& SourceFile : ImportData->SourceFiles)
				{
					SourceFileToObjectPathCache.FindOrAdd(FPaths::GetCleanFilename(SourceFile.RelativeFilename)).Add(Assets[Index].GetSoftObjectPath());
				}
			}
		}
	}, LowLevelTasks::ETaskPriority::BackgroundNormal);
}

void FAssetSourceFilenameCache::HandleOnAssetsRemoved(TConstArrayView<FAssetData> Assets)
{
	CachePipe.Launch(UE_SOURCE_LOCATION, [this, Assets=TArray<FAssetData>(Assets)](){
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetSourceFilenameCache::HandleOnAssetsRemoved);
		for (const FAssetData& AssetData : Assets)
		{
			TOptional<FAssetImportInfo> ImportData = ExtractAssetImportInfo(AssetData);
			if (ImportData.IsSet())
			{
				for (auto& SourceFile : ImportData->SourceFiles)
				{
					FString CleanFilename = FPaths::GetCleanFilename(SourceFile.RelativeFilename);
					if (auto* Objects = SourceFileToObjectPathCache.Find(CleanFilename))
					{
						Objects->Remove(AssetData.GetSoftObjectPath());
						if (Objects->Num() == 0)
						{
							SourceFileToObjectPathCache.Remove(CleanFilename);
						}
					}
				}
			}
		}
	}, LowLevelTasks::ETaskPriority::BackgroundNormal);
}

void FAssetSourceFilenameCache::HandleOnAssetRenamed(const FAssetData& AssetData, const FString& OldPath)
{
	// Capture by ref as we wait inline
	CachePipe.Launch(UE_SOURCE_LOCATION, [this, &AssetData, &OldPath]() {
		TOptional<FAssetImportInfo> ImportData = ExtractAssetImportInfo(AssetData);
		if (ImportData.IsSet())
		{
			for (auto& SourceFile : ImportData->SourceFiles)
			{
				FString CleanFilename = FPaths::GetCleanFilename(SourceFile.RelativeFilename);

				if (auto* Objects = SourceFileToObjectPathCache.Find(CleanFilename))
				{
					Objects->Remove(FSoftObjectPath(OldPath));
					if (Objects->Num() == 0)
					{
						SourceFileToObjectPathCache.Remove(CleanFilename);
					}
				}

				SourceFileToObjectPathCache.FindOrAdd(CleanFilename).Add(AssetData.GetSoftObjectPath());
			}
		}
	}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::Inline).Wait();

	AssetRenamedEvent.Broadcast(AssetData, OldPath);
}

void FAssetSourceFilenameCache::HandleOnAssetUpdated(const FAssetImportInfo& OldData, const UAssetImportData* ImportData)
{
	// Run inline on pipe to safely use ImportData
	CachePipe.Launch(UE_SOURCE_LOCATION, [this, &OldData, ImportData]() {
		FSoftObjectPath ObjectPath = FSoftObjectPath(ImportData->GetOuter());

		for (const auto& SourceFile : OldData.SourceFiles)
		{
			FString CleanFilename = FPaths::GetCleanFilename(SourceFile.RelativeFilename);
			if (auto* Objects = SourceFileToObjectPathCache.Find(CleanFilename))
			{
				Objects->Remove(ObjectPath);
				if (Objects->Num() == 0)
				{
					SourceFileToObjectPathCache.Remove(CleanFilename);
				}
			}
		}

		for (const auto& SourceFile : ImportData->SourceData.SourceFiles)
		{
			SourceFileToObjectPathCache.FindOrAdd(FPaths::GetCleanFilename(SourceFile.RelativeFilename)).Add(ObjectPath);
		}
	}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::Inline).Wait();
}

TArray<FAssetData> FAssetSourceFilenameCache::GetAssetsPertainingToFile(const IAssetRegistry& Registry, const FString& AbsoluteFilename) const
{
	return CachePipe.Launch(UE_SOURCE_LOCATION, [this, &Registry, &AbsoluteFilename]()
	{
		TArray<FAssetData> Assets;
		if (const auto* ObjectPaths = SourceFileToObjectPathCache.Find(FPaths::GetCleanFilename(AbsoluteFilename)))
		{
			for (const FSoftObjectPath& Path : *ObjectPaths)
			{
				FAssetData Asset = Registry.GetAssetByObjectPath(Path);
				TOptional<FAssetImportInfo> ImportInfo = ExtractAssetImportInfo(Asset);
				if (ImportInfo.IsSet())
				{
					auto AssetPackagePath = FPackageName::LongPackageNameToFilename(Asset.PackagePath.ToString() / TEXT(""));

					// Attempt to find the matching source filename in the list of imported sorce files (generally there are only one of these)
					const bool bWasImportedFromFile = ImportInfo->SourceFiles.ContainsByPredicate([&](const FAssetImportInfo::FSourceFile& File){

						if (AbsoluteFilename == FPaths::ConvertRelativePathToFull(AssetPackagePath / File.RelativeFilename) ||
							AbsoluteFilename == FPaths::ConvertRelativePathToFull(File.RelativeFilename))
						{
							return true;
						}

						return false;
					});

					if (bWasImportedFromFile)
					{
						Assets.Emplace();
						Swap(Asset, Assets[Assets.Num() - 1]);
					}
				}
			}
		}

		return Assets;
	}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::Inline).GetResult();
}
