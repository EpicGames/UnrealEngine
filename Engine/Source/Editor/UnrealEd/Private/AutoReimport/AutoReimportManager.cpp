// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimport/AutoReimportManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "UObject/GCObject.h"
#include "Styling/SlateTypes.h"
#include "EditorReimportHandler.h"
#include "Misc/Attribute.h"
#include "TickableEditorObject.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Factories/Factory.h"
#include "EditorFramework/AssetImportData.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "FileHelpers.h"

#include "AutoReimport/AutoReimportUtilities.h"
#include "Logging/MessageLog.h"
#include "AutoReimport/ContentDirectoryMonitor.h"

#include "PackageTools.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AutoReimport/ReimportFeedbackContext.h"
#include "MessageLogModule.h"
#include "AutoReimport/AssetSourceFilenameCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutoReimportManager)

#define LOCTEXT_NAMESPACE "AutoReimportManager"

enum class EStateMachineNode { CallOnce, CallMany };

/** Enum and value to specify the current state of our processing */
enum class ECurrentState : uint8
{
	Idle, Paused, Aborting, PromptUser, Initializing, ProcessAdditions, ProcessModifications, ProcessDeletions, SavePackages
};

/** A simple state machine class that calls functions mapped on enum values. If any function returns a new enum type, it moves onto that function */
struct FStateMachine
{
	typedef TFunction<TOptional<ECurrentState>(const DirectoryWatcher::FTimeLimit&)> FunctionType;

	/** Constructor that specifies the initial state of the machine */
	FStateMachine(ECurrentState InitialState) : CurrentState(InitialState){}

	/** Add an enum->function mapping for this state machine */
	void Add(ECurrentState State, EStateMachineNode NodeType, FunctionType&& Function)
	{
		Nodes.Add(State, FStateMachineNode(NodeType, MoveTemp(Function)));
	}

	/** Set a new state for this machine */
	void SetState(ECurrentState NewState)
	{
		CurrentState = NewState;
	}

	/** Tick this state machine with the given time limit. Will continuously enumerate the machine until TimeLimit is reached */
	void Tick(const DirectoryWatcher::FTimeLimit& TimeLimit)
	{
		while (!TimeLimit.Exceeded())
		{
			const FStateMachineNode& State = Nodes[CurrentState];
			TOptional<ECurrentState> NewState = State.Endpoint(TimeLimit);
			if (NewState.IsSet())
			{
				CurrentState = NewState.GetValue();
			}
			else if (State.Type == EStateMachineNode::CallOnce)
			{
				break;
			}
		}
	}

private:

	/** The current state of this machine */
	ECurrentState CurrentState;

private:

	struct FStateMachineNode
	{
		FStateMachineNode(EStateMachineNode InType, FunctionType&& InEndpoint) : Endpoint(MoveTemp(InEndpoint)), Type(InType) {}

		/** The function endpoint for this node */
		FunctionType Endpoint;

		/** Whether this endpoint should be called multiple times in a frame, or just once */
		EStateMachineNode Type;
	};

	/** A map of enum value -> callback information */
	TMap<ECurrentState, FStateMachineNode> Nodes;
};



/* Deals with auto reimporting of objects when the objects file on disk is modified*/
class FAutoReimportManager : public FTickableEditorObject, public FGCObject, public TSharedFromThis<FAutoReimportManager>
{
public:

	FAutoReimportManager();
	~FAutoReimportManager();

	/** Get a list of currently monitored directories */
	TArray<FPathAndMountPoint> GetMonitoredDirectories() const;

	/** Report an external change to the manager, such that a subsequent equal change reported by the os be ignored */
	void IgnoreNewFile(const FString& Filename);
	void IgnoreFileModification(const FString& Filename);
	void IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename);
	void IgnoreDeletedFile(const FString& Filename);

	/** Add a directory to watch at runtime. Directories added this way are not saved in the settings. Returns a handle that can be passed to RemoveRuntimeMonitoredDirectory.*/
	int32 AddRuntimeMonitoredDirectory(const FAutoReimportDirectoryConfig& Directory);

	/** Remove runtime directory with the given handle. */
	void RemoveRuntimeMonitoredDirectory(const int32 Handle);

	/** Destroy this manager */
	void Destroy();

private:

	/** FTickableEditorObject interface*/
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }

	/** Automation testing will tick its own file cache which can result in race conditions with the reimport manager.  */
	virtual bool IsTickable() const override { return !GIsAutomationTesting; }

	virtual TStatId GetStatId() const override;

	/** FGCObject interface*/
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAutoReimportManager");
	}

private:

	/** Called when a new asset path has been mounted or unmounted */
	void OnContentPathChanged(const FString& InAssetPath, const FString& FileSystemPath);

	/** Called when an asset has been renamed */
	void OnAssetRenamed(const FAssetData& AssetData, const FString& OldPath);

private:

	/** Callback for when an editor user setting has changed */
	void HandleLoadingSavingSettingChanged(FName PropertyName);

	/** Schedule monitors to be updated after a delay */
	void SetMonitorsDirty();

	/** Set up monitors to all the monitored content directories */
	void SetUpDirectoryMonitors();

	/** Retrieve a semi-colon separated string of file extensions supported by all available editor import factories */
	static FString GetAllFactoryExtensions();

private:
	
	/** Get the number of unprocessed changes that are not part of the current processing operation */
	int32 GetNumUnprocessedChanges() const;

	/** Populate the message log with a list of pending changes to files */
	void PopupateMessageLogWithPendingChanges(FMessageLog& MessageLog) const;

private:

	/** A state machine holding information about the current state of the manager */
	FStateMachine StateMachine;

private:

	/** Idle processing */
	TOptional<ECurrentState> Idle();

	/** Prompt the user whether they would like to import the changes */
	TOptional<ECurrentState> PromptUser();

	/** Set up the initial work for the import operation */
	TOptional<ECurrentState> InitializeOperation();

	/** Process any remaining pending additions we have */
	TOptional<ECurrentState> ProcessAdditions(const DirectoryWatcher::FTimeLimit& TimeLimit);

	/** Save any packages that were created inside ProcessAdditions */
	TOptional<ECurrentState> SavePackages();

	/** Process any remaining pending modifications we have */
	TOptional<ECurrentState> ProcessModifications(const DirectoryWatcher::FTimeLimit& TimeLimit);

	/** Process any remaining pending deletions we have */
	TOptional<ECurrentState> ProcessDeletions();

	/** Wait for a user's input. Just updates the progress text for now */
	TOptional<ECurrentState> Paused();

	/** Abort the process */
	TOptional<ECurrentState> Abort();

	/** Cleanup an operation that just processed some changes */
	void Cleanup();

	/** Check whether we should pause the operation or not */
	TOptional<ECurrentState> HandlePauseAbort(ECurrentState InCurrentState);

	/** Get the text to display on the confirmation notification */
	FText GetConfirmNotificationText() const;

private:

	/** Feedback context that can selectively override the global context to consume progress events for saving of assets */
	TSharedPtr<FReimportFeedbackContext> FeedbackContextOverride;

	/** Array of objects that detect changes to directories */
	TArray<TUniquePtr<FContentDirectoryMonitor>> DirectoryMonitors;

	/** A list of packages to save when we've added a bunch of assets */
	TArray<TObjectPtr<UPackage>> PackagesToSave;

	/** Reentracy guard for when we are making changes to assets */
	bool bGuardAssetChanges;

	/** A timeout used to refresh directory monitors when the user has made an interactive change to the settings */
	DirectoryWatcher::FTimeLimit ResetMonitorsTimeout;

	/** User confirmation popup */
	TSharedPtr<SNotificationItem> ConfirmNotification;

	/** The paused state of the state machine */
	ECurrentState PausedState;

	/** Monitored directories added at runtime */
	TMap<int32, FAutoReimportDirectoryConfig> RuntimeMonitoredDirectories;

	int32 NextRuntimeMonitoredDirectoryHandle = 0;
private:

	void OnPauseClicked()
	{
		switch (State)
		{
		case EProcessState::Paused: State = EProcessState::Running; break;
		case EProcessState::Running: State = EProcessState::Paused; break;
		default: break;
		}
	}
	void OnAbortClicked() { State = EProcessState::Aborted; }

	/** Flags for paused/aborted */
	enum class EProcessState { Running, Paused, Aborted };
	EProcessState State;
};

FAutoReimportManager::FAutoReimportManager()
	: StateMachine(ECurrentState::Idle)
	, bGuardAssetChanges(false)
	, PausedState(ECurrentState::Idle)
{
	UEditorLoadingSavingSettings* Settings = GetMutableDefault<UEditorLoadingSavingSettings>();
	Settings->OnSettingChanged().AddRaw(this, &FAutoReimportManager::HandleLoadingSavingSettingChanged);

	FPackageName::OnContentPathMounted().AddRaw(this, &FAutoReimportManager::OnContentPathChanged);
	FPackageName::OnContentPathDismounted().AddRaw(this, &FAutoReimportManager::OnContentPathChanged);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (!MessageLogModule.IsRegisteredLogListing("AssetReimport"))
	{
		MessageLogModule.RegisterLogListing("AssetReimport", LOCTEXT("AssetReimportLabel", "Asset Reimport"));
	}

	FAssetSourceFilenameCache::Get().OnAssetRenamed().AddRaw(this, &FAutoReimportManager::OnAssetRenamed);

	// Only set this up for content directories if the user has this enabled
	if (Settings->bMonitorContentDirectories)
	{
		SetUpDirectoryMonitors();
	}
	
	State = EProcessState::Running;

	StateMachine.Add(ECurrentState::Idle,					EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->Idle();					});
	StateMachine.Add(ECurrentState::Paused,					EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->Paused();					});
	StateMachine.Add(ECurrentState::Aborting,				EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->Abort();					});
	StateMachine.Add(ECurrentState::PromptUser,				EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->PromptUser();				});
	StateMachine.Add(ECurrentState::Initializing,			EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->InitializeOperation();		});
	StateMachine.Add(ECurrentState::ProcessAdditions,		EStateMachineNode::CallMany,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->ProcessAdditions(T);		});
	StateMachine.Add(ECurrentState::ProcessModifications,	EStateMachineNode::CallMany,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->ProcessModifications(T);	});
	StateMachine.Add(ECurrentState::ProcessDeletions,		EStateMachineNode::CallMany,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->ProcessDeletions();		});
	StateMachine.Add(ECurrentState::SavePackages,			EStateMachineNode::CallOnce,	[this](const DirectoryWatcher::FTimeLimit& T){ return this->SavePackages();			});
}

FAutoReimportManager::~FAutoReimportManager()
{
	if (ConfirmNotification.IsValid())
	{
		ConfirmNotification->SetText(FText());
	}
}

TArray<FPathAndMountPoint> FAutoReimportManager::GetMonitoredDirectories() const
{
	TArray<FPathAndMountPoint> Dirs;
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Dirs.Emplace(Monitor->GetDirectory(), Monitor->GetMountPoint());
	}
	return Dirs;
}

void FAutoReimportManager::IgnoreNewFile(const FString& Filename)
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		if (Filename.StartsWith(Monitor->GetDirectory()))
		{
			Monitor->IgnoreNewFile(Filename);
		}
	}
}

void FAutoReimportManager::IgnoreFileModification(const FString& Filename)
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		if (Filename.StartsWith(Monitor->GetDirectory()))
		{
			Monitor->IgnoreFileModification(Filename);
		}
	}
}

void FAutoReimportManager::IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename)
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		const bool bSrcInFolder = SrcFilename.StartsWith(Monitor->GetDirectory());
		const bool bDstInFolder = DstFilename.StartsWith(Monitor->GetDirectory());

		if (bSrcInFolder && bDstInFolder)
		{
			Monitor->IgnoreMovedFile(SrcFilename, DstFilename);
		}
		else if (bSrcInFolder)
		{
			Monitor->IgnoreDeletedFile(SrcFilename);
		}
		else if (bDstInFolder)
		{
			Monitor->IgnoreNewFile(DstFilename);
		}
	}
}

void FAutoReimportManager::IgnoreDeletedFile(const FString& Filename)
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		if (Filename.StartsWith(Monitor->GetDirectory()))
		{
			Monitor->IgnoreDeletedFile(Filename);
		}
	}
}

void FAutoReimportManager::Destroy()
{
	FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule)
	{
		FAssetSourceFilenameCache::Get().OnAssetRenamed().RemoveAll(this);
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnInMemoryAssetDeleted().RemoveAll(this);
		}
	}
	
	if (UEditorLoadingSavingSettings* Settings = GetMutableDefault<UEditorLoadingSavingSettings>())
	{
		Settings->OnSettingChanged().RemoveAll(this);
	}

	FPackageName::OnContentPathMounted().RemoveAll(this);
	FPackageName::OnContentPathDismounted().RemoveAll(this);

	// Force a save of all the caches
	DirectoryMonitors.Empty();
}

void FAutoReimportManager::OnAssetRenamed(const FAssetData& AssetData, const FString& OldPath)
{
	if (bGuardAssetChanges)
	{
		return;
	}

	// This code moves a source content file that reside alongside assets when the assets are renamed. We do this under the following conditions:
	// 	1. The sourcefile is solely referenced from the the asset that has been moved
	//	2. Said asset only references a single file
	//
	// Additionally, we rename the source file if it matched the name of the asset before the rename/move.
	//	- If we rename the source file, then we also update the reimport paths for the asset

	TOptional<FAssetImportInfo> ImportInfo = FAssetSourceFilenameCache::ExtractAssetImportInfo(AssetData);
	if (!ImportInfo.IsSet() || ImportInfo->SourceFiles.Num() != 1)
	{
		return;
	}
	
	const FString& RelativeFilename = ImportInfo->SourceFiles[0].RelativeFilename;

	FString OldPackagePath = FPackageName::GetLongPackagePath(OldPath) / TEXT("");
	FString NewReimportPath;

	// We move the file with the asset provided it is the only file referenced, and sits right beside the uasset file
	if (!RelativeFilename.GetCharArray().ContainsByPredicate([](const TCHAR Char) { return Char == '/' || Char == '\\'; }))
	{
		// File resides in the same folder as the asset, so we can potentially rename the source file too
		const FString AbsoluteSrcPath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(OldPackagePath));
		const FString AbsoluteDstPath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(AssetData.PackagePath.ToString() / TEXT("")));

		const FString OldAssetName = FPackageName::GetLongPackageAssetName(FPackageName::ObjectPathToPackageName(OldPath));
		FString NewFileName = FPaths::GetBaseFilename(RelativeFilename);

		bool bRequireReimportPathUpdate = false;
		if (UPackageTools::SanitizePackageName(NewFileName) == OldAssetName)
		{
			NewFileName = AssetData.AssetName.ToString();
			bRequireReimportPathUpdate = true;
		}

		const FString SrcFile = AbsoluteSrcPath / RelativeFilename;
		const FString DstFile = AbsoluteDstPath / NewFileName + TEXT(".") + FPaths::GetExtension(RelativeFilename);

		// We can't do this if multiple assets reference the same file. We should be checking for > 1 referencing asset, but the asset registry
		// filter lookup won't return the recently renamed package because it will be Empty by now, so we check for *anything* referencing the asset (assuming that we'll never find *this* asset).
		const IAssetRegistry& Registry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		if (Utils::FindAssetsPertainingToFile(Registry, SrcFile).Num() == 0)
		{
			if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*DstFile) &&
				IFileManager::Get().Move(*DstFile, *SrcFile, false /*bReplace */, false, true /* attributes */, true /* don't retry */))
			{
				IgnoreMovedFile(SrcFile, DstFile);

				if (bRequireReimportPathUpdate)
				{
					NewReimportPath = DstFile;
				}
			}
		}
	}

	if (NewReimportPath.IsEmpty() && FPackageName::GetLongPackagePath(OldPath) != AssetData.PackagePath.ToString())
	{
		// The asset has been moved, try and update its referenced path
		FString OldSourceFilePath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(OldPackagePath), RelativeFilename);
		if (FPaths::FileExists(OldSourceFilePath))
		{
			NewReimportPath = MoveTemp(OldSourceFilePath);
		}
	}

	if (!NewReimportPath.IsEmpty())
	{
		TArray<FString> Paths;
		Paths.Add(NewReimportPath);

		// Update the reimport file names
		FReimportManager::Instance()->UpdateReimportPaths(AssetData.GetAsset(), Paths);
	}
}

int32 FAutoReimportManager::GetNumUnprocessedChanges() const
{
	return Utils::Reduce(DirectoryMonitors, [](const TUniquePtr<FContentDirectoryMonitor>& Monitor, int32 Total){
		return Total + Monitor->GetNumUnprocessedChanges();
	}, 0);
}

void FAutoReimportManager::PopupateMessageLogWithPendingChanges(FMessageLog& MessageLog) const
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		const FString& BasePath = Monitor->GetDirectory();

		Monitor->IterateUnprocessedChanges([&MessageLog, &BasePath](const DirectoryWatcher::FUpdateCacheTransaction& Transaction, const FDateTime& TimeOfChange){
			FText ThisMessage;
			FString FullFilename = BasePath / Transaction.Filename.Get();
			switch(Transaction.Action)
			{
				case DirectoryWatcher::EFileAction::Added:
					ThisMessage = FText::Format(LOCTEXT("PendingChange_Add", "'{0}' has been created."), FText::FromString(FullFilename));
					break;

				case DirectoryWatcher::EFileAction::Removed:
					ThisMessage = FText::Format(LOCTEXT("PendingChange_Delete", "'{0}' has been deleted."), FText::FromString(FullFilename));
					break;

				case DirectoryWatcher::EFileAction::Modified:
					ThisMessage = FText::Format(LOCTEXT("PendingChange_Modified", "'{0}' has been modified."), FText::FromString(FullFilename));
					break;

				case DirectoryWatcher::EFileAction::Moved:
					ThisMessage = FText::Format(LOCTEXT("PendingChange_Moved", "'{0}' has been moved/renamed to {1}."), FText::FromString(BasePath / Transaction.MovedFromFilename.Get()), FText::FromString(FullFilename));
					break;
			}
			MessageLog.Message(EMessageSeverity::Info, ThisMessage);
			return true;
		});
	}
}

FText FAutoReimportManager::GetConfirmNotificationText() const
{
	const int32 TotalWork = GetNumUnprocessedChanges();
	if (TotalWork == 1)
	{
		return LOCTEXT("UserConfirmationTextSingle", "A change to a source content file has been detected.\nWould you like to import it?");
	}
	else if (TotalWork > 1)
	{
		return FText::Format(LOCTEXT("UserConfirmationTextMultiple", "{0} changes to source content files have been detected.\nWould you like to import them?"), FText::AsNumber(TotalWork));
	}
	return FText();
}

TOptional<ECurrentState> FAutoReimportManager::PromptUser()
{
	UEditorLoadingSavingSettings* Settings = GetMutableDefault<UEditorLoadingSavingSettings>();

	// Send out a notification asking for confirmation
	if (Settings->bPromptBeforeAutoImporting && !ConfirmNotification.IsValid())
	{
		FNotificationInfo Info(FText::GetEmpty());
		Info.bFireAndForget = false;
		Info.bUseLargeFont = false;

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("ImportButtonText", "Import"),
			FText(),
			FSimpleDelegate::CreateLambda([this, Settings]{
				Settings->PostEditChange();
				State = EProcessState::Running;
			}),
			SNotificationItem::CS_None
		));
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("DontImportButtonText", "Don't Import"),
			FText(),
			FSimpleDelegate::CreateLambda([this, Settings]{
				// going back into idle
				PausedState = ECurrentState::Aborting;
				State = EProcessState::Running;

				if (!Settings->bPromptBeforeAutoImporting)
				{
					// User clicked Don't import, with a don't show again. Disable auto reimport
					Settings->bMonitorContentDirectories = false;
					Settings->PostEditChange();
				}
			}),
			SNotificationItem::CS_None
		));

		Info.CheckBoxText = LOCTEXT("DontAskAgain", "Don't ask again");
		Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(TAttribute<ECheckBoxState>::FGetter::CreateStatic([]{
			return GetDefault<UEditorLoadingSavingSettings>()->bPromptBeforeAutoImporting ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
		}));
		Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic([](ECheckBoxState NewState){
			GetMutableDefault<UEditorLoadingSavingSettings>()->bPromptBeforeAutoImporting = NewState == ECheckBoxState::Unchecked;
		});

		Info.Hyperlink = FSimpleDelegate::CreateLambda([this]{
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

			FMessageLog MessageLog("AssetReimport");
			MessageLog.NewPage(FText::Format(LOCTEXT("WhatChangedMessageLogPageLabel", "Detailed File System Changes from {0}"), FText::AsTime(FDateTime::UtcNow())));
			PopupateMessageLogWithPendingChanges(MessageLog);
			MessageLogModule.OpenMessageLog("AssetReimport");
		});
		Info.HyperlinkText = LOCTEXT("UserConfirmationHyperlink", "What Changed?");

		ConfirmNotification = FSlateNotificationManager::Get().AddNotification(Info);

		State = EProcessState::Paused;
	}

	// Keep ticking the monitors
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Monitor->Tick();
	}

	if (ConfirmNotification.IsValid())
	{
		ConfirmNotification->SetText(GetConfirmNotificationText());
	}

	// Keep ticking this if we're paused
	if (State == EProcessState::Paused)
	{
		return TOptional<ECurrentState>();
	}

	if (ConfirmNotification.IsValid())
	{
		ConfirmNotification->SetEnabled(false);
		ConfirmNotification->SetCompletionState(SNotificationItem::CS_Success);
		ConfirmNotification->ExpireAndFadeout();
		ConfirmNotification = nullptr;
	}

	if (PausedState != ECurrentState::PromptUser)
	{
		return PausedState;
	}

	PausedState = ECurrentState::Idle;

	// This can get set by the user while we're prompting for action
	if (!Settings->bMonitorContentDirectories)
	{
		return ECurrentState::Aborting;
	}
	else
	{
		return ECurrentState::Initializing;
	}
}

TOptional<ECurrentState> FAutoReimportManager::InitializeOperation()
{
	int32 TotalWork = 0;
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		TotalWork += Monitor->StartProcessing();
	}

	if (TotalWork > 0)
	{
		if (!FeedbackContextOverride.IsValid())
		{
			// Create a new feedback context override
			FeedbackContextOverride = MakeShareable(new FReimportFeedbackContext(
				FSimpleDelegate::CreateSP(this, &FAutoReimportManager::OnPauseClicked),
				FSimpleDelegate::CreateSP(this, &FAutoReimportManager::OnAbortClicked))
			);
		}

		FeedbackContextOverride->Show(TotalWork);
		return ECurrentState::ProcessAdditions;
	}

	return ECurrentState::Idle;
}

TOptional<ECurrentState> FAutoReimportManager::ProcessAdditions(const DirectoryWatcher::FTimeLimit& TimeLimit)
{
	TOptional<ECurrentState> NewState = HandlePauseAbort(ECurrentState::ProcessAdditions);
	if (NewState.IsSet())
	{
		return NewState;
	}

	// Override the global feedback context while we do this to avoid popping up dialogs
	TGuardValue<FFeedbackContext*> ScopedContextOverride(GWarn, FeedbackContextOverride.Get());
	TGuardValue<bool> ScopedAssetChangesGuard(bGuardAssetChanges, true);

	TMap<FString, TArray<UFactory*>> Factories;

	TArray<FString> FactoryExtensions;
	FactoryExtensions.Reserve(16);

	// Get the list of valid factories
	for (TObjectIterator<UClass> It ; It ; ++It)
	{
		UClass* CurrentClass = (*It);

		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->bEditorImport && Factory->ImportPriority >= 0)
			{
				FactoryExtensions.Reset();
				Factory->GetSupportedFileExtensions(FactoryExtensions);

				for (const FString& Ext : FactoryExtensions)
				{
					Factories.FindOrAdd(Ext).Add(Factory);
				}
			}
		}
	}

	for (TTuple<FString, TArray<UFactory*>>& Pair : Factories)
	{
		Pair.Value.Sort([](const UFactory& A, const UFactory& B) { return A.ImportPriority > B.ImportPriority; });
	}

	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Monitor->ProcessAdditions(TimeLimit, PackagesToSave, Factories, *FeedbackContextOverride);
		if (TimeLimit.Exceeded())
		{
			return TOptional<ECurrentState>();
		}
	}

	return ECurrentState::ProcessModifications;
}

TOptional<ECurrentState> FAutoReimportManager::ProcessModifications(const DirectoryWatcher::FTimeLimit& TimeLimit)
{
	TOptional<ECurrentState> NewState = HandlePauseAbort(ECurrentState::ProcessModifications);
	if (NewState.IsSet())
	{
		return NewState;
	}

	// Override the global feedback context while we do this to avoid popping up dialogs
	TGuardValue<FFeedbackContext*> ScopedContextOverride(GWarn, FeedbackContextOverride.Get());
	TGuardValue<bool> ScopedAssetChangesGuard(bGuardAssetChanges, true);

	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Monitor->ProcessModifications(TimeLimit, PackagesToSave, *FeedbackContextOverride);

		if (TimeLimit.Exceeded())
		{
			return TOptional<ECurrentState>();
		}
	}

	return ECurrentState::ProcessDeletions;
}

TOptional<ECurrentState> FAutoReimportManager::ProcessDeletions()
{
	TOptional<ECurrentState> NewState = HandlePauseAbort(ECurrentState::ProcessDeletions);
	if (NewState.IsSet())
	{
		return NewState;
	}

	TGuardValue<bool> ScopedAssetChangesGuard(bGuardAssetChanges, true);

	TArray<FAssetData> AssetsToDelete;
	int32 TotalWork = 0;

	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		TotalWork += Monitor->GetDeletedFilesNum();
		Monitor->ExtractAssetsToDelete(AssetsToDelete);
	}

	FeedbackContextOverride->MainTask->EnterProgressFrame(static_cast<float>(TotalWork));

	if (AssetsToDelete.Num() > 0)
	{
		for (const FAssetData& AssetData : AssetsToDelete)
		{
			FeedbackContextOverride->AddMessage(EMessageSeverity::Info, FText::Format(LOCTEXT("Success_DeletedAsset", "Attempting to delete {0} (its source file has been removed)."), FText::FromName(AssetData.AssetName)));
		}

		ObjectTools::DeleteAssets(AssetsToDelete);
	}

	return ECurrentState::SavePackages;
}

TOptional<ECurrentState> FAutoReimportManager::SavePackages()
{
	// We don't override the context specifically when saving packages so the user gets proper feedback
	//TGuardValue<FFeedbackContext*> ScopedContextOverride(GWarn, FeedbackContextOverride.Get());

	TGuardValue<bool> ScopedAssetChangesGuard(bGuardAssetChanges, true);

	if (PackagesToSave.Num() > 0)
	{
		const bool bAlreadyCheckedOut = false;
		const bool bCheckDirty = false;
		const bool bPromptToSave = false;
		FEditorFileUtils::PromptForCheckoutAndSave(ObjectPtrDecay(PackagesToSave), bCheckDirty, bPromptToSave, nullptr, bAlreadyCheckedOut);

		PackagesToSave.Empty();
	}

	Cleanup();
	return ECurrentState::Idle;
}

TOptional<ECurrentState> FAutoReimportManager::HandlePauseAbort(ECurrentState InCurrentState)
{
	if (State == EProcessState::Aborted)
	{
		return ECurrentState::Aborting;
	}
	else if (State == EProcessState::Paused)
	{
		PausedState = InCurrentState;
		return ECurrentState::Paused;
	}

	return TOptional<ECurrentState>();
}

TOptional<ECurrentState> FAutoReimportManager::Paused()
{
	TOptional<ECurrentState> NewState = HandlePauseAbort(PausedState);
	if (NewState.IsSet())
	{
		return NewState.GetValue();
	}

	// No longer paused
	return PausedState;
}

TOptional<ECurrentState> FAutoReimportManager::Abort()
{
	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Monitor->Abort();
	}

	PackagesToSave.Empty();

	Cleanup();
	return ECurrentState::Idle;
}

TOptional<ECurrentState> FAutoReimportManager::Idle()
{
	// Check whether we need to reset the monitors or not
	if (ResetMonitorsTimeout.Exceeded())
	{
		const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
		if (Settings->bMonitorContentDirectories)
		{
			DirectoryMonitors.Empty();
			SetUpDirectoryMonitors();
		}
		else
		{
			// Destroy all the existing monitors, including their file caches
			for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
			{
				Monitor->Destroy();
			}
			DirectoryMonitors.Empty();
		}

		ResetMonitorsTimeout = DirectoryWatcher::FTimeLimit();
		return TOptional<ECurrentState>();
	}

	for (const TUniquePtr<FContentDirectoryMonitor>& Monitor : DirectoryMonitors)
	{
		Monitor->Tick();
	}

	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		return TOptional<ECurrentState>();
	}

	if (GetNumUnprocessedChanges() > 0)
	{
		PausedState = ECurrentState::PromptUser;
		return ECurrentState::PromptUser;
	}

	return TOptional<ECurrentState>();
}

void FAutoReimportManager::Cleanup()
{
	if (FeedbackContextOverride.IsValid())
	{
		FeedbackContextOverride->Hide();
	}
}

void FAutoReimportManager::Tick(float DeltaTime)
{
	// Never spend more than a 60fps frame doing this work (meaning we shouldn't drop below 30fps), we can do more if we're throttling CPU usage (ie editor running in background)
	const DirectoryWatcher::FTimeLimit TimeLimit(GEditor->ShouldThrottleCPUUsage() ? 1 / 6.f : 1.f / 60.f);
	StateMachine.Tick(TimeLimit);
}

TStatId FAutoReimportManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAutoReimportManager, STATGROUP_Tickables);
}

void FAutoReimportManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(PackagesToSave);
}

void FAutoReimportManager::HandleLoadingSavingSettingChanged(FName PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UEditorLoadingSavingSettings, bMonitorContentDirectories) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UEditorLoadingSavingSettings, AutoReimportDirectorySettings))
	{
		SetMonitorsDirty();
	}
}

void FAutoReimportManager::SetMonitorsDirty()
{
	ResetMonitorsTimeout = DirectoryWatcher::FTimeLimit(5.f);
}

void FAutoReimportManager::OnContentPathChanged(const FString& InAssetPath, const FString& FileSystemPath)
{
	const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
	if (Settings->bMonitorContentDirectories)
	{
		DirectoryMonitors.Empty();
		SetUpDirectoryMonitors();
	}
}

int32 FAutoReimportManager::AddRuntimeMonitoredDirectory(const FAutoReimportDirectoryConfig& Directory)
{
	const int32 NewHandle = NextRuntimeMonitoredDirectoryHandle++;
	RuntimeMonitoredDirectories.Add(NewHandle, Directory);
	SetMonitorsDirty();
	return NewHandle;
}

void FAutoReimportManager::RemoveRuntimeMonitoredDirectory(const int32 Handle)
{
	if (RuntimeMonitoredDirectories.Remove(Handle))
	{
		SetMonitorsDirty();
	}
}

void FAutoReimportManager::SetUpDirectoryMonitors()
{
	struct FParsedSettings
	{
		FString SourceDirectory;
		FString MountPoint;
		DirectoryWatcher::FMatchRules Rules;
	};

	TArray<FParsedSettings> FinalArray;
	const FString SupportedExtensions = GetAllFactoryExtensions();
	auto TryAddMapping = [&FinalArray, &SupportedExtensions](const FAutoReimportDirectoryConfig& Setting, bool bAllowEmptyMountPoint)
		{
			FParsedSettings NewMapping;
			NewMapping.SourceDirectory = Setting.SourceDirectory;
			NewMapping.MountPoint = Setting.MountPoint;

			if (!FAutoReimportDirectoryConfig::ParseSourceDirectoryAndMountPoint(NewMapping.SourceDirectory, NewMapping.MountPoint))
			{
				// Allow the parse failure if it's just because the mount point is empty.
				if (!(bAllowEmptyMountPoint && FPaths::DirectoryExists(NewMapping.SourceDirectory) && NewMapping.MountPoint.IsEmpty()))
				{
					return;
				}
			}

			// Only include extensions that match a factory
			NewMapping.Rules.SetApplicableExtensions(SupportedExtensions);
			for (const FAutoReimportWildcard& WildcardConfig : Setting.Wildcards)
			{
				NewMapping.Rules.AddWildcardRule(WildcardConfig.Wildcard, WildcardConfig.bInclude);
			}

			FinalArray.Add(NewMapping);
		};
	
	for (const FAutoReimportDirectoryConfig& Setting : GetDefault<UEditorLoadingSavingSettings>()->AutoReimportDirectorySettings)
	{
		TryAddMapping(Setting, false);
	}

	for (const TPair<int32, FAutoReimportDirectoryConfig>& Setting : RuntimeMonitoredDirectories)
	{
		TryAddMapping(Setting.Value, true);
	}

	for (int32 Index = 0; Index < FinalArray.Num(); ++Index)
	{
		const FParsedSettings& Mapping = FinalArray[Index];

		// We only create a directory monitor if there are no other's watching parent directories of this one
		for (int32 OtherIndex = Index + 1; OtherIndex < FinalArray.Num(); ++OtherIndex)
		{
			if (FinalArray[Index].SourceDirectory.StartsWith(FinalArray[OtherIndex].SourceDirectory))
			{
				UE_LOGF(LogAutoReimportManager, Warning, "Unable to watch directory %ls as it will conflict with another watching %ls.", *FinalArray[Index].SourceDirectory, *FinalArray[OtherIndex].SourceDirectory);
				goto next;
			}
		}

		DirectoryMonitors.Emplace(MakeUnique<FContentDirectoryMonitor>(Mapping.SourceDirectory, Mapping.Rules, Mapping.MountPoint));

	next:
		continue;
	}
}

FString FAutoReimportManager::GetAllFactoryExtensions()
{
	FString AllExtensions;

	// Use a scratch buffer to avoid unnecessary re-allocation
	FString Scratch;
	Scratch.Reserve(32);

	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (Class->IsChildOf(UFactory::StaticClass()) && !Class->HasAnyClassFlags(CLASS_Abstract))
		{
			UFactory* Factory = Cast<UFactory>(Class->GetDefaultObject());
			if (Factory->bEditorImport)
			{
				for (const FString& Format : Factory->Formats)
				{
					int32 Index = INDEX_NONE;
					if (Format.FindChar(';', Index) && Index > 0)
					{
						Scratch.GetCharArray().Reset();
						// Include the ;
						Scratch.AppendChars(*Format, Index + 1);

						if (AllExtensions.Find(Scratch) == INDEX_NONE)
						{
							AllExtensions += Scratch;
						}
					}
				}
			}
		}
	}

	return AllExtensions;
}

UAutoReimportManager::UAutoReimportManager(const FObjectInitializer& Init)
	: Super(Init)
{
}

UAutoReimportManager::~UAutoReimportManager()
{
}

void UAutoReimportManager::Initialize()
{
	Implementation = MakeShareable(new FAutoReimportManager);
}

void UAutoReimportManager::IgnoreNewFile(const FString& Filename)
{
	Implementation->IgnoreNewFile(Filename);
}

void UAutoReimportManager::IgnoreFileModification(const FString& Filename)
{
	Implementation->IgnoreFileModification(Filename);
}

void UAutoReimportManager::IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename)
{
	Implementation->IgnoreMovedFile(SrcFilename, DstFilename);
}

void UAutoReimportManager::IgnoreDeletedFile(const FString& Filename)
{
	Implementation->IgnoreDeletedFile(Filename);
}

TArray<FPathAndMountPoint> UAutoReimportManager::GetMonitoredDirectories() const
{
	return Implementation->GetMonitoredDirectories();
}

int32 UAutoReimportManager::AddRuntimeMonitoredDirectory(const FAutoReimportDirectoryConfig& Directory)
{
	return Implementation->AddRuntimeMonitoredDirectory(Directory);
}

void UAutoReimportManager::RemoveRuntimeMonitoredDirectory(const int32 Handle)
{
	Implementation->RemoveRuntimeMonitoredDirectory(Handle);
}

void UAutoReimportManager::BeginDestroy()
{
	Super::BeginDestroy();
	if (Implementation.IsValid())
	{
		Implementation->Destroy();
		Implementation = nullptr;
	}
}


#undef LOCTEXT_NAMESPACE

