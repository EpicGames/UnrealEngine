// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimport/AutoReimportUtilities.h"

#include "AutoReimport/AssetSourceFilenameCache.h"
#include "EditorFramework/AssetImportData.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

struct FAssetData;

DEFINE_LOG_CATEGORY(LogAutoReimportManager);

namespace Utils
{
	TArray<FAssetData> FindAssetsPertainingToFile(const IAssetRegistry& Registry, const FString& AbsoluteFilename)
	{
		return FAssetSourceFilenameCache::Get().GetAssetsPertainingToFile(Registry, AbsoluteFilename);
	}

	TArray<FString> ExtractSourceFilePaths(UObject* Object)
	{
		TArray<FString> Temp;
		ExtractSourceFilePaths(Object, Temp);
		return Temp;
	}

	void ExtractSourceFilePaths(UObject* Object, TArray<FString>& OutSourceFiles)
	{
		FAssetRegistryTagsContextData TagList(Object, EAssetRegistryTagsCaller::Uncategorized);
		Object->GetAssetRegistryTags(TagList);

		const FName TagName = UObject::SourceFileTagName();
		for (const TPair<FName, UObject::FAssetRegistryTag>& Pair : TagList.Tags)
		{
			const UObject::FAssetRegistryTag& Tag = Pair.Value;
			if (Tag.Name == TagName)
			{
				int32 PreviousNum = OutSourceFiles.Num();

				TOptional<FAssetImportInfo> ImportInfo = FAssetImportInfo::FromJson(Tag.Value);
				if (ImportInfo.IsSet())
				{
					for (const auto& File : ImportInfo->SourceFiles)
					{
						OutSourceFiles.Emplace(UAssetImportData::ResolveImportFilename(File.RelativeFilename, Object->GetOutermost()));
					}
				}
				break;
			}
		}
	}
}
