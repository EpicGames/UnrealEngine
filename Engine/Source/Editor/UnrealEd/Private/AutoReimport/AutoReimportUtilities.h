// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogMacros.h"

class IAssetRegistry;
class UObject;
struct FAssetData;

DECLARE_LOG_CATEGORY_EXTERN(LogAutoReimportManager, Log, All);

namespace Utils
{
	/** Reduce the array to the specified accumulator */
	template<typename T, typename P, typename A>
	A Reduce(const TArray<T>& InArray, P Predicate, A Accumulator)
	{
		for (const T& X : InArray)
		{
			Accumulator = Predicate(X, Accumulator);
		}
		return Accumulator;
	}

	/** Find a list of assets that were once imported from the specified filename */
	TArray<FAssetData> FindAssetsPertainingToFile(const IAssetRegistry& Registry, const FString& AbsoluteFilename);

	/** Extract any source file paths from the specified object */
	TArray<FString> ExtractSourceFilePaths(UObject* Object);
	void ExtractSourceFilePaths(UObject* Object, TArray<FString>& OutSourceFiles);
}
