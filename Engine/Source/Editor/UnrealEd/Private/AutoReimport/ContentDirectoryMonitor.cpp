// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimport/ContentDirectoryMonitor.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "EditorReimportHandler.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Factories/Factory.h"
#include "Factories/SceneImportFactory.h"
#include "EditorFramework/AssetImportData.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"

#include "AutoReimport/AutoReimportUtilities.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageTools.h"
#include "ObjectTools.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AutoReimport/ReimportFeedbackContext.h"
#include "AutoReimport/AssetSourceFilenameCache.h"
#include "Misc/NamePermissionList.h"

#include "InterchangeManager.h"
#include "InterchangeSceneImportAsset.h"

#define LOCTEXT_NAMESPACE "ContentDirectoryMonitor"

bool IsAssetDirty(UObject* Asset)
{
	UPackage* Package = Asset ? Asset->GetOutermost() : nullptr;
	return Package ? Package->IsDirty() : false;
}

/** Generate a config from the specified options, to pass to FFileCache on construction */
DirectoryWatcher::FFileCacheConfig GenerateFileCacheConfig(const FString& InPath, const DirectoryWatcher::FMatchRules& InMatchRules, const FString& InMountedContentPath)
{
	FString Directory = FPaths::ConvertRelativePathToFull(InPath);

	const FString& HashString = InMountedContentPath.IsEmpty() ? Directory : InMountedContentPath;
	const uint32 CRC = FCrc::MemCrc32(*HashString, HashString.Len()*sizeof(TCHAR));	
	FString CacheFilename = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir()) / TEXT("ReimportCache") / FString::Printf(TEXT("%u.bin"), CRC);

	DirectoryWatcher::FFileCacheConfig Config(Directory, MoveTemp(CacheFilename));
	Config.Rules = InMatchRules;
	// We always store paths inside content folders relative to the folder
	Config.PathType = DirectoryWatcher::EPathType::Relative;

	Config.bDetectChangesSinceLastRun = GetDefault<UEditorLoadingSavingSettings>()->bDetectChangesOnStartup;

	// It's safe to assume the asset registry is not re-loadable
	IAssetRegistry* Registry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	Config.CustomChangeLogic = [Directory, Registry](const DirectoryWatcher::FImmutableString& InRelativePath, const DirectoryWatcher::FFileData& FileData) -> TOptional<bool> {

		int32 TotalNumReferencingAssets = 0;

		TArray<FAssetData> Assets = FAssetSourceFilenameCache::Get().GetAssetsPertainingToFile(*Registry, Directory / InRelativePath.Get());

		if (Assets.Num() == 0)
		{
			return TOptional<bool>();
		}

		// We need to consider this as a changed file if the hash doesn't match any asset imported from that file
		for (FAssetData& Asset : Assets)
		{
			TOptional<FAssetImportInfo> Info = FAssetSourceFilenameCache::ExtractAssetImportInfo(Asset);

			// Check if the source file that this asset last imported was the same as the one we're going to reimport.
			// If it is, there's no reason to auto-reimport it
			if (Info.IsSet() && Info->SourceFiles.Num() == 1)
			{
				if (Info->SourceFiles[0].FileHash != FileData.FileHash)
				{
					return true;
				}
			}
		}

		return TOptional<bool>();
	};

	// We only detect changes for when the file *contents* have changed (not its timestamp)
	Config
		.DetectMoves(true)
		.DetectChangesFor(DirectoryWatcher::FFileCacheConfig::Timestamp, false)
		.DetectChangesFor(DirectoryWatcher::FFileCacheConfig::FileHash, true);

	return Config;
}

FContentDirectoryMonitor::FContentDirectoryMonitor(const FString& InDirectory, const DirectoryWatcher::FMatchRules& InMatchRules, const FString& InMountedContentPath)
	: Cache(GenerateFileCacheConfig(InDirectory, InMatchRules, InMountedContentPath))
	, MountedContentPath(InMountedContentPath)
	, LastSaveTime(0)
{
	Registry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
}

void FContentDirectoryMonitor::Destroy()
{
	Cache.Destroy();
}

void FContentDirectoryMonitor::IgnoreNewFile(const FString& Filename)
{
	Cache.IgnoreNewFile(Filename);
}

void FContentDirectoryMonitor::IgnoreFileModification(const FString& Filename)
{
	Cache.IgnoreFileModification(Filename);
}

void FContentDirectoryMonitor::IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename)
{
	Cache.IgnoreMovedFile(SrcFilename, DstFilename);
}

void FContentDirectoryMonitor::IgnoreDeletedFile(const FString& Filename)
{
	Cache.IgnoreDeletedFile(Filename);
}

void FContentDirectoryMonitor::Tick()
{
	Cache.Tick();

	// Immediately resolve any changes that we should not consider
	const FDateTime Threshold = FDateTime::UtcNow() - FTimespan::FromSeconds(GetDefault<UEditorLoadingSavingSettings>()->AutoReimportThreshold);

	TArray<DirectoryWatcher::FUpdateCacheTransaction> InsignificantTransactions = Cache.FilterOutstandingChanges([this, Threshold](const DirectoryWatcher::FUpdateCacheTransaction& Transaction, const FDateTime& TimeOfChange){
		return TimeOfChange <= Threshold && !ShouldConsiderChange(Transaction);
	});

	for (DirectoryWatcher::FUpdateCacheTransaction& Transaction : InsignificantTransactions)
	{
		Cache.CompleteTransaction(MoveTemp(Transaction));
	}

	const double Now = FPlatformTime::Seconds();
	if (Now - LastSaveTime > ResaveIntervalS)
	{
		LastSaveTime = Now;
		Cache.WriteCache();
	}
}

bool FContentDirectoryMonitor::ShouldConsiderChange(const DirectoryWatcher::FUpdateCacheTransaction& Transaction) const
{
	// If the file was removed, and nothing references it, there's nothing else to do
	if (Transaction.Action == DirectoryWatcher::EFileAction::Removed && FAssetSourceFilenameCache::Get().GetAssetsPertainingToFile(*Registry, Cache.GetDirectory() / Transaction.Filename.Get()).Num() == 0)
	{
		return false;
	}

	static const FName AssetToolsName("AssetTools");
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AssetToolsName).Get();
	if (!AssetTools.GetWritableFolderPermissionList()->PassesStartsWithFilter(Transaction.Filename.Get()))
	{
		return false;
	}

	return true;
}

int32 FContentDirectoryMonitor::GetNumUnprocessedChanges() const
{
	const FDateTime Threshold = FDateTime::UtcNow() - FTimespan::FromSeconds(GetDefault<UEditorLoadingSavingSettings>()->AutoReimportThreshold);

	int32 Total = 0;

	// Get all the changes that have happend beyond our import threshold
	Cache.IterateOutstandingChanges([this, &Total, Threshold](const DirectoryWatcher::FUpdateCacheTransaction& Transaction, const FDateTime& TimeOfChange){
		if (TimeOfChange <= Threshold && ShouldConsiderChange(Transaction))
		{
			++Total;
		}
		return true;
	});

	return Total;
}

void FContentDirectoryMonitor::IterateUnprocessedChanges(TFunctionRef<bool(const DirectoryWatcher::FUpdateCacheTransaction&, const FDateTime&)> InIter) const
{
	Cache.IterateOutstandingChanges(InIter);
}

int32 FContentDirectoryMonitor::StartProcessing()
{
	// We only process things that haven't changed for a given threshold
	IFileManager& FileManager = IFileManager::Get();
	const FDateTime Threshold = FDateTime::UtcNow() - FTimespan::FromSeconds(GetDefault<UEditorLoadingSavingSettings>()->AutoReimportThreshold);

	// Get all the changes that have happend beyond our import threshold
	TArray<DirectoryWatcher::FUpdateCacheTransaction> OutstandingChanges = Cache.FilterOutstandingChanges([this, Threshold](const DirectoryWatcher::FUpdateCacheTransaction& Transaction, const FDateTime& TimeOfChange){
		return TimeOfChange <= Threshold && ShouldConsiderChange(Transaction);
	});

	if (OutstandingChanges.Num() == 0)
	{
		return 0;
	}

	const UEditorLoadingSavingSettings* Settings = GetDefault<UEditorLoadingSavingSettings>();
	for (DirectoryWatcher::FUpdateCacheTransaction& Transaction : OutstandingChanges)
	{
		switch(Transaction.Action)
		{
			case DirectoryWatcher::EFileAction::Added:
				if (Settings->bAutoCreateAssets && !MountedContentPath.IsEmpty())
				{
					AddedFiles.Emplace(MoveTemp(Transaction));
				}
				else
				{
					Cache.CompleteTransaction(MoveTemp(Transaction));
				}
				break;

			case DirectoryWatcher::EFileAction::Moved:
			case DirectoryWatcher::EFileAction::Modified:
				ModifiedFiles.Emplace(MoveTemp(Transaction));
				break;

			case DirectoryWatcher::EFileAction::Removed:
				if (Settings->bAutoDeleteAssets && !MountedContentPath.IsEmpty())
				{
					DeletedFiles.Emplace(MoveTemp(Transaction));
				}
				else
				{
					Cache.CompleteTransaction(MoveTemp(Transaction));
				}
				break;
		}
	}

	return AddedFiles.Num() + ModifiedFiles.Num() + DeletedFiles.Num();
}

UObject* AttemptImport(UClass* InFactoryType, UPackage* Package, FName InName, bool& bCancelled, const FString& FullFilename)
{
	UObject* Asset = nullptr;

	if (UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), InFactoryType))
	{
		Factory->AddToRoot();
		if (Factory->ConfigureProperties())
		{
			if (UClass* SupportedClass = Factory->ResolveSupportedClass())
			{
				Asset = Factory->ImportObject(SupportedClass, Package, InName, RF_Public | RF_Standalone, FullFilename, nullptr, bCancelled);
			}
		}
		Factory->RemoveFromRoot();
	}

	return Asset;
}

void FContentDirectoryMonitor::ProcessAdditions(const DirectoryWatcher::FTimeLimit& TimeLimit, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, const TMap<FString, TArray<UFactory*>>& InFactoriesByExtension, FReimportFeedbackContext& Context)
{
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	const bool bUseInterchangeFramework = UInterchangeManager::IsInterchangeImportEnabled();

	bool bCancelled = false;
	for (int32 Index = 0; Index < AddedFiles.Num(); ++Index)
	{
		DirectoryWatcher::FUpdateCacheTransaction& Addition = AddedFiles[Index];

		if (bCancelled)
		{
			// Just update the cache immediately if the user cancelled
			Cache.CompleteTransaction(MoveTemp(Addition));
			Context.MainTask->EnterProgressFrame();
			continue;
		}

		const FString FullFilename = Cache.GetDirectory() + Addition.Filename.Get();

		FString NewAssetName = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(FullFilename));
		FString PackagePath = UPackageTools::SanitizePackageName(MountedContentPath / FPaths::GetPath(Addition.Filename.Get()) / NewAssetName);

		// Don't create assets for new files if assets already exist for the filename
		TArray<FAssetData> ExistingReferences = Utils::FindAssetsPertainingToFile(*Registry, FullFilename);
		if (ExistingReferences.Num() != 0)
		{
			// Treat this as a modified file that will attempt to reimport it (if applicable). We don't update the progress for this item until it is processed by ProcessModifications
			ModifiedFiles.Add(MoveTemp(Addition));
			continue;
		}

		// Move the progress on now that we know we're going to process the file
		Context.MainTask->EnterProgressFrame();

		// See if the asset can be imported using the Interchange before using the provided factory information
		bool bImportHandledThroughInterchange = false;
		if (bUseInterchangeFramework)
		{
			bool bSceneImport = false;
			UE::Interchange::FScopedSourceData ScopedSourceData(FullFilename);

			if (InterchangeManager.CanTranslateSourceData(ScopedSourceData.GetSourceData()))
			{
				FImportAssetParameters ImportAssetParameters;
				ImportAssetParameters.bIsAutomated = false;
				ImportAssetParameters.bFollowRedirectors = false;
				ImportAssetParameters.ReimportAsset = nullptr;
				ImportAssetParameters.bReplaceExisting = false;
				ImportAssetParameters.DestinationName = NewAssetName;

				// Priority is given to asset import. Try it first 
				bImportHandledThroughInterchange = InterchangeManager.ImportAsset(FPaths::GetPath(PackagePath), ScopedSourceData.GetSourceData(), ImportAssetParameters);
				if (!bImportHandledThroughInterchange)
				{
					// Try a level import then
					bImportHandledThroughInterchange = InterchangeManager.ImportScene(FPaths::GetPath(PackagePath), ScopedSourceData.GetSourceData(), ImportAssetParameters);
					if (!bImportHandledThroughInterchange)
					{
						// Add the message to the message log letting the user know that the import didn't work.
						Context.GetMessageLog().Message(EMessageSeverity::Info, FText::Format(LOCTEXT("Info_InterchangeFailedToImportAsset", "Interchange: Failed to import file {0}."), FText::FromString(FullFilename)));
					}
				}
			}
		}
		

		if (!bImportHandledThroughInterchange)
		{
			if (FPackageName::DoesPackageExist(*PackagePath))
			{
				// Package already exists, so try and import over the top of it, if it doesn't already have a source file path
				TArray<FAssetData> Assets;
				if (Registry->GetAssetsByPackageName(*PackagePath, Assets) && Assets.Num() == 1)
				{
					if (UObject* ExistingAsset = Assets[0].GetAsset())
					{
						// We're only eligible for reimport if the existing asset doesn't reference a source file already
						const bool bEligibleForReimport = !Utils::ExtractSourceFilePaths(ExistingAsset).ContainsByPredicate([&](const FString& In) {
							return !In.IsEmpty() && In == FullFilename;
							});

						if (bEligibleForReimport)
						{
							ReimportAssetWithNewSource(ExistingAsset, FullFilename, OutPackagesToSave, Context);
						}
					}
				}
			}
			else
			{
				UPackage* NewPackage = CreatePackage(*PackagePath);
				if (!ensure(NewPackage))
				{
					Context.AddMessage(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_FailedToCreateAsset", "Failed to create new asset ({0}) for file ({1})."), FText::FromString(NewAssetName), FText::FromString(FullFilename)));
				}
				else
				{
					Context.AddMessage(EMessageSeverity::Info, FText::Format(LOCTEXT("Info_CreatingNewAsset", "Importing new asset {0}."), FText::FromString(PackagePath)));

					// Make sure the destination package is loaded
					NewPackage->FullyLoad();

					UObject* NewAsset = nullptr;

					// Find a relevant factory for this file
					// @todo import: gmp: show dialog in case of multiple matching factories
					const FString Ext = FPaths::GetExtension(Addition.Filename.Get(), false);
					const TArray<UFactory*>* Factories = InFactoriesByExtension.Find(Ext);
					if (Factories && Factories->Num() != 0)
					{
						//Make sure all the scene factory are put at the end of the array. We give priority to asset factory before scene factory
						TArray<UFactory*> SortFactories;
						TArray<UFactory*> SceneFactories;
						for (UFactory* Factory : *Factories)
						{
							if (Factory->IsA(USceneImportFactory::StaticClass()))
							{
								SceneFactories.Add(Factory);
							}
							else
							{
								SortFactories.Add(Factory);
							}
						}
						if (SceneFactories.Num() > 0)
						{
							SortFactories.Append(SceneFactories);
						}
						// Prefer a factory if it explicitly can import. UFactory::FactoryCanImport returns false by default, even if the factory supports the extension, so we can't use it directly.
						UFactory* const* PreferredFactory = SortFactories.FindByPredicate([&](UFactory* F) { return F->FactoryCanImport(FullFilename); });
						if (PreferredFactory)
						{
							NewAsset = AttemptImport((*PreferredFactory)->GetClass(), NewPackage, *NewAssetName, bCancelled, FullFilename);
						}
						// If there was no preferred factory, just try them all until one succeeds
						else for (UFactory* Factory : SortFactories)
						{
							NewAsset = AttemptImport(Factory->GetClass(), NewPackage, *NewAssetName, bCancelled, FullFilename);

							if (bCancelled || NewAsset)
							{
								break;
							}
						}
					}

					// Verify if the package still exists after the import (it may have been cleaned up by the factory if the import was canceled).
					NewPackage = FindObject<UPackage>(nullptr, *PackagePath);
					if (NewPackage)
					{
						// If we didn't create an asset and the package was not cleaned up, unload and delete the package we just created
						if (!NewAsset)
						{
							TArray<UPackage*> Packages;
							Packages.Add(NewPackage);

							TGuardValue<bool> SuppressSlowTaskMessages(Context.bSuppressSlowTaskMessages, true);

							FText ErrorMessage;
							if (!UPackageTools::UnloadPackages(Packages, ErrorMessage))
							{
								Context.AddMessage(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_UnloadingPackage", "There was an error unloading a package: {0}."), ErrorMessage));
							}

							// Just add the message to the message log rather than add it to the UI
							// Factories may opt not to import the file, so we let them report errors if they do
							Context.GetMessageLog().Message(EMessageSeverity::Info, FText::Format(LOCTEXT("Info_FailedToImportAsset", "Failed to import file {0}."), FText::FromString(FullFilename)));
						}
						else if (!bCancelled)
						{
							FAssetRegistryModule::AssetCreated(NewAsset);
							GEditor->BroadcastObjectReimported(NewAsset);

							OutPackagesToSave.Add(NewPackage);
						}
					}

					// Refresh the supported class.  Some factories (e.g. FBX) only resolve their type after reading the file
					// ImportAssetType = Factory->ResolveSupportedClass();
					// @todo: analytics?
				}
			}
		}

		// Let the cache know that we've dealt with this change (it will be imported immediately)
		Cache.CompleteTransaction(MoveTemp(Addition));

		if (!bCancelled && TimeLimit.Exceeded())
		{
			// Remove the ones we've processed
			AddedFiles.RemoveAt(0, Index + 1);
			return;
		}
	}

	AddedFiles.Empty();
}

void FContentDirectoryMonitor::ProcessModifications(const DirectoryWatcher::FTimeLimit& TimeLimit, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context)
{
	FReimportManager* ReimportManager = FReimportManager::Instance();

	for (int32 Index = 0; Index < ModifiedFiles.Num(); ++Index)
	{
		Context.MainTask->EnterProgressFrame();

		DirectoryWatcher::FUpdateCacheTransaction& Change = ModifiedFiles[Index];
		const FString FullFilename = Cache.GetDirectory() + Change.Filename.Get();

		// Move the asset before reimporting it. We always reimport moved assets to ensure that their import path is up to date
		if (Change.Action == DirectoryWatcher::EFileAction::Moved)
		{
			const FString OldFilename = Cache.GetDirectory() + Change.MovedFromFilename.Get();
			const TArray<FAssetData> Assets = Utils::FindAssetsPertainingToFile(*Registry, OldFilename);

			if (Assets.Num() == 1)
			{
				UObject* Asset = Assets[0].GetAsset();
				if (Asset && Utils::ExtractSourceFilePaths(Asset).Num() == 1 && !MountedContentPath.IsEmpty())
				{
					UPackage* ExistingPackage = Asset->GetOutermost();

					const bool bAssetWasDirty = IsAssetDirty(Asset);

					const FString NewAssetName = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(Change.Filename.Get()));
					const FString PackagePath = UPackageTools::SanitizePackageName(MountedContentPath / FPaths::GetPath(Change.Filename.Get()));
					const FString FullDestPath = PackagePath / NewAssetName;

					const FText SrcPathText = FText::FromString(Assets[0].PackageName.ToString());
					const FText DstPathText = FText::FromString(FullDestPath);
					FPackagePath DestPackagePath;
					if (!FPackagePath::TryFromMountedName(FullDestPath, DestPackagePath))
					{
						Context.AddMessage(EMessageSeverity::Warning, FText::Format(LOCTEXT("MoveWarning_NotInMountedPath", "Can't move {0} to {1} - {1} is not in a mounted path."), SrcPathText, DstPathText));
					}
					else
					{
						if (ExistingPackage && ExistingPackage->GetLoadedPath() == DestPackagePath)
						{
							// No need to process this asset - it's already been moved to the right location
							Cache.CompleteTransaction(MoveTemp(Change));
							continue;
						}

						if (FPackageName::DoesPackageExist(DestPackagePath, &DestPackagePath))
						{
							Context.AddMessage(EMessageSeverity::Warning, FText::Format(LOCTEXT("MoveWarning_ExistingAsset", "Can't move {0} to {1} - one already exists."), SrcPathText, DstPathText));
						}
						else
						{
							TArray<FAssetRenameData> RenameData;
							RenameData.Emplace(Asset, PackagePath, NewAssetName);

							Context.AddMessage(EMessageSeverity::Info, FText::Format(LOCTEXT("Success_MovedAsset", "Moving asset {0} to {1}."), SrcPathText, DstPathText));

							FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get().RenameAssetsWithDialog(RenameData);

							TArray<FString> Filenames;
							Filenames.Add(FullFilename);

							// Update the reimport file names
							FReimportManager::Instance()->UpdateReimportPaths(Asset, Filenames);
							Asset->MarkPackageDirty();

							if (!bAssetWasDirty)
							{
								if (UPackage* NewPackage = Asset->GetOutermost())
								{
									OutPackagesToSave.Add(NewPackage);
								}
							}
						}
					}
				}
			}
		}
		else
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

			TArray<FString> NoReimportObjectPaths;
			TArray<UObject*> ReimportObjects;

			// Modifications or additions are treated the same by this point
			for (const FAssetData& AssetData : Utils::FindAssetsPertainingToFile(*Registry, FullFilename))
			{
				if (UClass* AssetClass = AssetData.GetClass())
				{
					TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetClass).Pin();
					if (!AssetTypeActions.IsValid() || !AssetTypeActions->IsImportedAsset())
					{
						NoReimportObjectPaths.Add(AssetData.GetObjectPathString());
					}
					else if (UObject* Asset = AssetData.GetAsset())
					{
						ReimportObjects.Add(Asset);
					}
				}
			}

			if (NoReimportObjectPaths.Num() > 0)
			{
				if (!GIsAutomationTesting && !FApp::IsUnattended() && !FApp::IsGame())
				{
					FText DialogMessage = LOCTEXT("ReimportInfo_NoReimportAvailable", "Some of the assets would need to be reimported manually.");
					FMessageDialog::Open(EAppMsgType::Ok, DialogMessage);
				}

				for (const FString& ObjectPathString : NoReimportObjectPaths)
				{
					Context.AddMessage(EMessageSeverity::Warning, FText::Format(LOCTEXT("ReimportWarning_NoReimport", "Reimport for asset {0} is unavailable. Please import them manually."), FText::FromString(ObjectPathString)));
				}
			}

			for(UObject* ReimportObject: ReimportObjects)
			{
				ReimportAsset(ReimportObject, FullFilename, OutPackagesToSave, Context);
			}
		}

		// Let the cache know that we've dealt with this change
		Cache.CompleteTransaction(MoveTemp(Change));

		if (TimeLimit.Exceeded())
		{
			ModifiedFiles.RemoveAt(0, Index + 1);
			return;
		}
	}

	ModifiedFiles.Empty();
}

void FContentDirectoryMonitor::ReimportAssetWithNewSource(UObject* InAsset, const FString& FullFilename, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context)
{
	TArray<FString> Filenames;
	Filenames.Add(FullFilename);
	FReimportManager::Instance()->UpdateReimportPaths(InAsset, Filenames);

	ReimportAsset(InAsset, FullFilename, OutPackagesToSave, Context);
}

void FContentDirectoryMonitor::ReimportAsset(UObject* Asset, const FString& FullFilename, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context)
{
	const bool bAssetWasDirty = IsAssetDirty(Asset);
	if (!FReimportManager::Instance()->Reimport(Asset, false /* Ask for new file */, false /* Show notification */))
	{
		Context.AddMessage(EMessageSeverity::Error, FText::Format(LOCTEXT("Error_FailedToReimportAsset", "Failed to reimport asset {0}."), FText::FromString(Asset->GetName())));
	}
	else
	{
		Context.AddMessage(EMessageSeverity::Info, FText::Format(LOCTEXT("Success_CreatedNewAsset", "Reimported asset {0} from {1}."), FText::FromString(Asset->GetName()), FText::FromString(FullFilename)));
		if (!bAssetWasDirty)
		{
			if (UPackage* NewPackage = Asset->GetOutermost())
			{
				OutPackagesToSave.Add(NewPackage);
			}
		}
	}
}

void FContentDirectoryMonitor::ExtractAssetsToDelete(TArray<FAssetData>& OutAssetsToDelete)
{
	for (DirectoryWatcher::FUpdateCacheTransaction& Deletion : DeletedFiles)
	{
		for (const FAssetData& AssetData : Utils::FindAssetsPertainingToFile(*Registry, Cache.GetDirectory() + Deletion.Filename.Get()))
		{
			OutAssetsToDelete.Add(AssetData);
		}

		// Let the cache know that we've dealt with this change (it will be imported in due course)
		Cache.CompleteTransaction(MoveTemp(Deletion));
	}

	DeletedFiles.Empty();
}

void FContentDirectoryMonitor::Abort()
{
	using namespace DirectoryWatcher;
	for (FUpdateCacheTransaction& Add : AddedFiles)
	{
		Cache.CompleteTransaction(MoveTemp(Add));
	}
	AddedFiles.Empty();

	for (FUpdateCacheTransaction& Mod : ModifiedFiles)
	{
		Cache.CompleteTransaction(MoveTemp(Mod));
	}
	ModifiedFiles.Empty();

	for (FUpdateCacheTransaction& Del : DeletedFiles)
	{
		Cache.CompleteTransaction(MoveTemp(Del));
	}
	DeletedFiles.Empty();

	for (FUpdateCacheTransaction& Change : Cache.GetOutstandingChanges())
	{
		Cache.CompleteTransaction(MoveTemp(Change));
	}
}


#undef LOCTEXT_NAMESPACE
