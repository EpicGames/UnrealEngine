// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "FileCache.h"

class FReimportFeedbackContext;
class IAssetRegistry;
class UFactory;

/** Class responsible for watching a specific content directory for changes */
class FContentDirectoryMonitor
{
public:

	/**
	 * Constructor.
	 * @param InDirectory			Content directory path to monitor. Assumed to be absolute.
	 * @param InMatchRules			Rules to select what will apply to this folder
	 * @param InMountedContentPath	(optional) Mounted content path (eg /Engine/, /Game/) to which InDirectory maps.
	 */
	FContentDirectoryMonitor(const FString& InDirectory, const DirectoryWatcher::FMatchRules& InMatchRules, const FString& InMountedContentPath = FString());

	/** Tick this monitor's cache to give it a chance to finish scanning for files */
	void Tick();

	/** Start processing any outstanding changes this monitor is aware of */
	int32 StartProcessing();

	/** Extract the files we need to import from our outstanding changes (happens first)*/ 
	void ProcessAdditions(const DirectoryWatcher::FTimeLimit& TimeLimit, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, const TMap<FString, TArray<UFactory*>>& InFactoriesByExtension, FReimportFeedbackContext& Context);

	/** Process the outstanding changes that we have cached */
	void ProcessModifications(const DirectoryWatcher::FTimeLimit& TimeLimit, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context);

	/** Returns the number of source files deleted in outstanding changes. */
	int32 GetDeletedFilesNum() const { return DeletedFiles.Num(); }

	/** Extract the assets we need to delete from our outstanding changes (happens last) */ 
	void ExtractAssetsToDelete(TArray<FAssetData>& OutAssetsToDelete);

	/** Abort the current processing operation */
	void Abort();

	/** Report an external change to the manager, such that a subsequent equal change reported by the os be ignored */
	void IgnoreNewFile(const FString& Filename);
	void IgnoreFileModification(const FString& Filename);
	void IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename);
	void IgnoreDeletedFile(const FString& Filename);

	/** Destroy this monitor including its cache */
	void Destroy();

	/** Get the directory that this monitor applies to */
	const FString& GetDirectory() const { return Cache.GetDirectory(); }

	/** Get the content mount point that this monitor applies to */
	const FString& GetMountPoint() const { return MountedContentPath; }

public:

	/** Get the number of outstanding changes that we potentially have to process (when not already processing) */
	int32 GetNumUnprocessedChanges() const;

	/** Iterate the current set of unprocessed file system changes */
	void IterateUnprocessedChanges(TFunctionRef<bool(const DirectoryWatcher::FUpdateCacheTransaction&, const FDateTime&)>) const;

private:

	/** Import a new asset into the specified package path, from the specified file */ 
	void ImportAsset(const FString& PackagePath, const FString& Filename, TArray<UPackage*>& OutPackagesToSave, const TMap<FString, TArray<UFactory*>>& InFactoriesByExtension, FReimportFeedbackContext& Context);

	/** Reimport a specific asset */
	void ReimportAsset(UObject* InAsset, const FString& FullFilename, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context);

	/** Set the specified asset to import from the specified file, then attempt to reimport it */
	void ReimportAssetWithNewSource(UObject* InAsset, const FString& FullFilename, TArray<TObjectPtr<UPackage>>& OutPackagesToSave, FReimportFeedbackContext& Context);

	/** Check whether we should consider a change for the specified transaction */
	bool ShouldConsiderChange(const DirectoryWatcher::FUpdateCacheTransaction& Transaction) const;

private:

	/** The file cache that monitors and reflects the content directory. */
	DirectoryWatcher::FFileCache Cache;

	/** The mounted content path for this monitor (eg /Game/) */
	FString MountedContentPath;

	/** A list of file system changes that are due to be processed */
	TArray<DirectoryWatcher::FUpdateCacheTransaction> AddedFiles, ModifiedFiles, DeletedFiles;

	/** The last time we attempted to save the cache file */
	double LastSaveTime;

	/** Cached asset registry ptr */
	IAssetRegistry* Registry;

	/** The interval between potential re-saves of the cache file */
	static const int32 ResaveIntervalS = 60;
};
