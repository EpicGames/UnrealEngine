// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimport/ReimportFeedbackContext.h"

#include "FileCacheUtilities.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Layout/Overscroll.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MessageLogModule.h"
#include "Misc/Attribute.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/SlowTaskStack.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ReimportContext"

class SWidgetStack : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWidgetStack){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SBox)
			.HeightOverride(45.0f)
			[
				SAssignNew(ScrollBox, SScrollBox)
				.Style(FAppStyle::Get(), "ScrollBoxNoShadow")
				.ScrollBarVisibility(EVisibility::Collapsed)
				.AllowOverscroll(EAllowOverscroll::No)
			]
		];
	}

	void Add(const TSharedRef<SWidget>& InWidget)
	{
		ScrollBox->AddSlot()
		[
			InWidget
		];

		++NumSlots;

		ScrollBox->ScrollToEnd();
	}
	
	int32 GetNumSlots() const { return NumSlots; }

	TSharedPtr<SScrollBox> ScrollBox;

	int32 NumSlots = 0;
	
};

/** Feedback context that overrides GWarn for import operations to prevent popup spam */
class SReimportFeedback : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SReimportFeedback) : _ExpireDuration(3.f) {}

		SLATE_ARGUMENT(TWeakPtr<FReimportFeedbackContext>, FeedbackContext)
		SLATE_ARGUMENT(float, ExpireDuration)
		SLATE_EVENT(FSimpleDelegate, OnExpired)

		SLATE_EVENT(FSimpleDelegate, OnPauseClicked)
		SLATE_EVENT(FSimpleDelegate, OnAbortClicked)

	SLATE_END_ARGS()


	virtual FVector2D ComputeDesiredSize(float LayoutScale) const override
	{
		auto Size = SCompoundWidget::ComputeDesiredSize(LayoutScale);
		// The width is determined by the top row, plus some padding
		Size.X = TopRow->GetDesiredSize().X + 100;
		return Size;
	}
	/** Construct this widget */
	void Construct(const FArguments& InArgs)
	{
		ExpireDuration = InArgs._ExpireDuration;
		OnExpired = InArgs._OnExpired;
		FeedbackContext = InArgs._FeedbackContext;

		bPaused = false;
		bExpired = false;

		auto OpenMessageLog = []{
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.OpenMessageLog("AssetReimport");
		};

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(TopRow, SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ProcessingChanges", "Processing source file changes..."))
					.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4,0,4,0))
				[
					SAssignNew(PauseButton, SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("PauseTooltip", "Temporarily pause processing of these source content files"))
					.OnClicked(this, &SReimportFeedback::OnPauseClicked, InArgs._OnPauseClicked)
					[
						SNew(SImage)
						.ColorAndOpacity(FLinearColor(.8f,.8f,.8f,1.f))
						.Image(this, &SReimportFeedback::GetPlayPauseBrush)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SAssignNew(AbortButton, SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("AbortTooltip", "Permanently abort processing of these source content files"))
					.OnClicked(this, &SReimportFeedback::OnAbortClicked, InArgs._OnAbortClicked)
					[
						SNew(SImage)
						.ColorAndOpacity(FLinearColor(0.8f,0.8f,0.8f,1.f))
						.Image(FAppStyle::GetBrush("GenericStop"))
					]
				]
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0, 1))
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(2.0f)
				[
					SAssignNew(ProgressBar, SProgressBar)
					.BorderPadding(FVector2D::ZeroVector)
					.Percent( this, &SReimportFeedback::GetProgressFraction )
				]
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0, 5, 0, 0))
			.AutoHeight()
			[
				SAssignNew(WidgetStack, SWidgetStack)
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0, 5, 0, 0))
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHyperlink)
				.Visibility(this, &SReimportFeedback::GetHyperlinkVisibility)
				.Text(LOCTEXT("OpenMessageLog", "Open message log"))
				.TextStyle(FCoreStyle::Get(), "SmallText")
				.OnNavigate_Lambda(OpenMessageLog)
			]
		];
	}

	/** Add a widget to this feedback's widget stack */
	void Add(const TSharedRef<SWidget>& Widget)
	{
		WidgetStack->Add(Widget);
	}

	/** Disable input to this widget's dynamic content (except the message log hyperlink) */
	void Disable()
	{

		ExpireTimeout = DirectoryWatcher::FTimeLimit(ExpireDuration);

		WidgetStack->SetVisibility(EVisibility::HitTestInvisible);
		PauseButton->SetVisibility(EVisibility::Collapsed);
		AbortButton->SetVisibility(EVisibility::Collapsed);
		ProgressBar->SetVisibility(EVisibility::Collapsed);
	}

	/** Enable, if previously disabled */
	void Enable()
	{
		ExpireTimeout = DirectoryWatcher::FTimeLimit();

		bPaused = false;
		WidgetStack->SetVisibility(EVisibility::Visible);
		PauseButton->SetVisibility(EVisibility::Visible);
		AbortButton->SetVisibility(EVisibility::Visible);
		ProgressBar->SetVisibility(EVisibility::Visible);
	}

private:
	
	TOptional<float> GetProgressFraction() const
	{
		auto PinnedContext = FeedbackContext.Pin();
		if (PinnedContext.IsValid() && PinnedContext->GetScopeStack().Num() >= 0 )
		{
			return PinnedContext->GetScopeStack().GetProgressFraction(0);
		}
		return 1.f;
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (bExpired)
		{
			return;
		}
		else if(ExpireTimeout.IsValid())
		{
			if (ExpireTimeout.Exceeded())
			{	
				OnExpired.ExecuteIfBound();
				bExpired = true;
			}
		}
	}

	/** Get the play/pause image */
	const FSlateBrush* GetPlayPauseBrush() const
	{
		return bPaused ? FAppStyle::GetBrush("GenericPlay") : FAppStyle::GetBrush("GenericPause");
	}

	/** Called when pause is clicked */
	FReply OnPauseClicked(FSimpleDelegate UserOnClicked)
	{
		bPaused = !bPaused;

		UserOnClicked.ExecuteIfBound();
		return FReply::Handled();
	}
	
	/** Called when abort is clicked */
	FReply OnAbortClicked(FSimpleDelegate UserOnClicked)
	{
		//Destroy();
		UserOnClicked.ExecuteIfBound();
		return FReply::Handled();
	}

	/** Get the visibility of the hyperlink to open the message log */
	EVisibility GetHyperlinkVisibility() const
	{
		return WidgetStack->GetNumSlots() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:

	/** The expire timeout used to fire OnExpired. Invalid when no timeout is set. */
	DirectoryWatcher::FTimeLimit ExpireTimeout;

	/** Amount of time to wait after this widget has been disabled before calling OnExpired */
	float ExpireDuration;

	/** Event that is called when this widget has been inactive and open for too long, and will fade out */
	FSimpleDelegate OnExpired;

	/** Whether we are paused and/or expired */
	bool bPaused, bExpired;

	/** The widget stack, displaying contextural information about the current state of the process */
	TSharedPtr<SWidgetStack> WidgetStack;
	TSharedPtr<SWidget> PauseButton, AbortButton, ProgressBar, TopRow;
	TWeakPtr<FReimportFeedbackContext> FeedbackContext;
};

FReimportFeedbackContext::FReimportFeedbackContext(const FSimpleDelegate& InOnPauseClicked, const FSimpleDelegate& InOnAbortClicked)
	: bSuppressSlowTaskMessages(false)
	, OnPauseClickedEvent(InOnPauseClicked)
	, OnAbortClickedEvent(InOnAbortClicked)
	, MessageLog("AssetReimport")
{
}

void FReimportFeedbackContext::Show(int32 TotalWork)
{
	// Important - we first destroy the old main task, then create a new one to ensure that they are (removed from/added to) the scope stack in the correct order
	MainTask.Reset();
	MainTask.Reset(new FScopedSlowTask(TotalWork, FText(), true, *this));

	if (NotificationContent.IsValid())
	{
		NotificationContent->Enable();
	}
	else
	{
		NotificationContent = SNew(SReimportFeedback)
			.FeedbackContext(AsShared())
			.OnExpired(this, &FReimportFeedbackContext::OnNotificationExpired)
			.OnPauseClicked(OnPauseClickedEvent)
			.OnAbortClicked(OnAbortClickedEvent);

		FNotificationInfo Info(AsShared());
		Info.bFireAndForget = false;

		Notification = FSlateNotificationManager::Get().AddNotification(Info);

		MessageLog.NewPage(FText::Format(LOCTEXT("MessageLogPageLabel", "Outstanding source content changes {0}"), FText::AsTime(FDateTime::Now())));
	}
}

void FReimportFeedbackContext::Hide()
{
	MainTask.Reset();

	if (Notification.IsValid())
	{
		NotificationContent->Disable();
		Notification->SetCompletionState(SNotificationItem::CS_Success);
	}
}

void FReimportFeedbackContext::OnNotificationExpired()
{
	if (Notification.IsValid())
	{
		MessageLog.Notify(FText(), EMessageSeverity::Error);
		Notification->Fadeout();

		NotificationContent = nullptr;
		Notification = nullptr;
	}
}

void FReimportFeedbackContext::AddMessage(EMessageSeverity::Type Severity, const FText& Message)
{
	MessageLog.Message(Severity, Message);
	AddWidget(SNew(STextBlock).Text(Message));
}

void FReimportFeedbackContext::AddWidget(const TSharedRef<SWidget>& Widget)
{
	if (NotificationContent.IsValid())
	{
		NotificationContent->Add(Widget);
	}
}

TSharedRef<SWidget> FReimportFeedbackContext::AsWidget()
{
	return NotificationContent.ToSharedRef();
}

void FReimportFeedbackContext::StartSlowTask(const FText& Task, bool bShowCancelButton)
{
	FFeedbackContext::StartSlowTask(Task, bShowCancelButton);

	if (NotificationContent.IsValid() && !bSuppressSlowTaskMessages && !Task.IsEmpty())
	{
		if (SlowTaskText.IsValid())
		{
			SlowTaskText->SetText(FText::Format(LOCTEXT("SlowTaskPattern_Default", "{0} (0%)"), Task));
		}
		else
		{
			NotificationContent->Add(SAssignNew(SlowTaskText, STextBlock).Text(Task));
		}
	}
}

void FReimportFeedbackContext::ProgressReported(const float TotalProgressInterp, FText DisplayMessage)
{
	if (SlowTaskText.IsValid())
	{
		SlowTaskText->SetText(FText::Format(LOCTEXT("SlowTaskPattern", "{0} ({1}%)"), DisplayMessage, FText::AsNumber(int(TotalProgressInterp * 100))));
	}
}

void FReimportFeedbackContext::FinalizeSlowTask()
{
	if (SlowTaskText.IsValid())
	{
		SlowTaskText->SetVisibility(EVisibility::Collapsed);
		SlowTaskText = nullptr;
	}

	FFeedbackContext::FinalizeSlowTask();
}

#undef LOCTEXT_NAMESPACE
