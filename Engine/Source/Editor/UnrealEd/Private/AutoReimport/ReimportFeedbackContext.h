// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/LogVerbosity.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Notifications/SNotificationList.h"

class FText;
class SWidget;

/** Feedback context that overrides GWarn for import operations to prevent popup spam */
class FReimportFeedbackContext : public FFeedbackContext, public INotificationWidget, public TSharedFromThis<FReimportFeedbackContext>
{
public:
	/** Constructor */
	FReimportFeedbackContext(const FSimpleDelegate& InOnPauseClicked, const FSimpleDelegate& InOnAbortClicked);

	/** Destroy this reimport context */
	void Hide();

	/** Revive the notification if it was destroyed */
	void Show(int32 TotalWork);

	/** Get the message log that this context is using */
	FMessageLog& GetMessageLog() { return MessageLog; }

	/** Add a message to the context (and message log) */
	void AddMessage(EMessageSeverity::Type Severity, const FText& Message);

	/** Add a custom widget to the context */
	void AddWidget(const TSharedRef<SWidget>& Widget);

	/** INotificationWidget and FFeedbackContext overrides */
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override {}
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override {}
	virtual void SerializeRecord(const UE::FLogRecord& Record) override {}
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override {}
	virtual TSharedRef<SWidget> AsWidget() override;
	virtual void StartSlowTask(const FText& Task, bool bShowCancelButton=false) override;
	virtual void ProgressReported(const float TotalProgressInterp, FText DisplayMessage) override;
	virtual void FinalizeSlowTask() override;

	/** True if we will suppress slow task messages from being shown on the UI, false if otherwise */
	bool bSuppressSlowTaskMessages;

	/** Publicly accessible task that defines the entire import progress */
	TUniquePtr<FScopedSlowTask> MainTask;

private:

	/** Called when our notification has expired */
	void OnNotificationExpired();

	FSimpleDelegate OnPauseClickedEvent, OnAbortClickedEvent;

	/** Message log for output of errors and log messages */
	FMessageLog MessageLog;

	/** Nested slow task text */
	TSharedPtr<class STextBlock> SlowTaskText;

	/** The notification that is shown when the context is active */
	TSharedPtr<class SNotificationItem> Notification;

	/** The notification content */
	TSharedPtr<class SReimportFeedback> NotificationContent;
};
