// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoSaveUtils.h"

#include "HAL/Platform.h"
#include "Misc/Paths.h"

FString AutoSaveUtils::GetAutoSaveDir()
{
	return FPaths::ProjectSavedDir() / TEXT("Autosaves");
}
