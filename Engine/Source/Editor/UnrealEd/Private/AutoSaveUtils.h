// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace AutoSaveUtils
{
	/** Get the root directory that we should save auto-saved assets and maps to */
	FString GetAutoSaveDir();
}
