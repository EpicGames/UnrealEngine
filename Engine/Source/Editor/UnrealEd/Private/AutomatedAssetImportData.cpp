// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedAssetImportData.h"

#include "Factories/Factory.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedAssetImportData)

DEFINE_LOG_CATEGORY(LogAutomatedImport);

UAutomatedAssetImportData::UAutomatedAssetImportData()
	: GroupName()
	, bReplaceExisting(false)
	, bSkipReadOnly(false)
{

}

bool UAutomatedAssetImportData::IsValid() const
{
	// This data is valid if there is at least one filename to import, there is a valid destination path
	// and either no factory was supported (automatic factory picking) or a valid factory was found
	return Filenames.Num() > 0 && !DestinationPath.IsEmpty() && (FactoryName.IsEmpty() || Factory != nullptr);
}

void UAutomatedAssetImportData::Initialize(TSharedPtr<FJsonObject> InImportGroupJsonData)
{
	ImportGroupJsonData = InImportGroupJsonData;
	if(Filenames.Num() > 0)
	{
		for (FString& Filename : Filenames)
		{
			// Make sure all filenames are absolute path
			Filename = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir(), Filename);
		}
		// If a factory doesn't have a vaild full path assume it is an unreal internal factory
		if(!FactoryName.IsEmpty() && !FactoryName.StartsWith("/Script/"))
		{
			FName FactoryFName = *FactoryName;
			// Find the factory among loaded factory classes
			EObjectFlags ExclusionFlags = RF_NoFlags;
			for (UFactory* TestFactory : TObjectRange<UFactory>(ExclusionFlags))
			{
				if (TestFactory->GetClass()->GetFName() == FactoryFName)
				{
					// factory has been found
					FactoryName = TestFactory->GetClass()->GetPathName();
					break;
				}
			}
		}

		if(!FactoryName.IsEmpty())
		{
			UClass* FactoryClass = LoadClass<UObject>(nullptr, *FactoryName, nullptr, LOAD_None, nullptr);
			if(FactoryClass)
			{
				UFactory* NewFactory = NewObject<UFactory>(GetTransientPackage(), FactoryClass);

				if(NewFactory->bEditorImport)
				{
					// Check that every file can be imported
					TArray<FString> InvalidFilesForFactory;
					for(const FString& Filename : Filenames)
					{
						if(!NewFactory->FactoryCanImport(Filename))
						{
							InvalidFilesForFactory.Add(Filename);
						}
					}

					if(InvalidFilesForFactory.Num() > 0)
					{
						UE_LOGF(LogAutomatedImport, Error, "Factory %ls could not import one or more of the source files", *FactoryName);
						for(const FString& InvalidFiles : InvalidFilesForFactory)
						{
							UE_LOGF(LogAutomatedImport, Error, "    %ls", *InvalidFiles);
						}
					}
					else
					{
						// All files are valid. Use this factory
						Factory = NewFactory;
					}
				}
			}
			else
			{
				UE_LOGF(LogAutomatedImport, Error, "Factory %ls could not be found", *FactoryName);
			}
		}
		else
		{
			UE_LOGF(LogAutomatedImport, Log, "Factory was not specified, will be set automatically");
		}
	}

	if(!DestinationPath.IsEmpty() && FPackageName::GetPackageMountPoint(DestinationPath) == NAME_None)
	{
		// Path doesnt have a valid moint point.  assume it is in /Game
		DestinationPath = TEXT("/Game") / DestinationPath;

		UE_LOGF(LogAutomatedImport, Warning, "DestinationPath has no valid mount point.  Assuming /Game is the mount point");
	}

	if (!LevelToLoad.IsEmpty())
	{
		FText FailReason;

		if (!FPackageName::IsValidLongPackageName(LevelToLoad, false, &FailReason))
		{
			UE_LOGF(LogAutomatedImport, Error, "Invalid level specified: %ls", *FailReason.ToString());
		}
	}

	FString PackagePath;
	FString FailureReason;
	if(!FPackageName::TryConvertFilenameToLongPackageName(DestinationPath, PackagePath, &FailureReason))
	{
		UE_LOGF(LogAutomatedImport, Error, "Invalid Destination Path (%ls): %ls", *DestinationPath, *FailureReason);
		DestinationPath.Empty();
	}
	else
	{
		// Package path is what we will use for importing.  So use that as the dest path now that it has been created
		DestinationPath = PackagePath;
	}
}

FString UAutomatedAssetImportData::GetDisplayName() const
{
	return GroupName.Len() > 0 ? GroupName : GetName();
}
