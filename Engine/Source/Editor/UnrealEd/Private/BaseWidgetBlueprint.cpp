// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseWidgetBlueprint.h"
#include "Components/Widget.h"
#include "Blueprint/WidgetTree.h"
#include "UObject/UObjectHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseWidgetBlueprint)

UBaseWidgetBlueprint::UBaseWidgetBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetTree = CreateDefaultSubobject<UWidgetTree>(TEXT("WidgetTree"));
	WidgetTree->SetFlags(RF_Transactional | RF_ArchetypeObject);
}

void UBaseWidgetBlueprint::PostLoad()
{
	Super::PostLoad();

	// Update any older widget trees that don't have the same flags.
	WidgetTree->SetFlags(RF_Transactional | RF_ArchetypeObject);

	// Clean up named slot content that somehow still refers to a parent widget.  Nothing
	// hosted inside of a named slot should have a Slot property set to anything.
	// This shouldn't be the case, but has happened in the past due to various paste bugs
	// so clean it up.
	ForEachSourceWidget([&](UWidget* InWidget) {
		if (INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(InWidget))
		{
			TArray<FName> SlotNames;
			NamedSlotHost->GetSlotNames(SlotNames);

			for (FName SlotName : SlotNames)
			{
				if (UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName))
				{
					if (SlotContent->Slot)
					{
						SlotContent->Slot = nullptr;
					}
				}
			}
		}
	});

	// Clean up any cases where the root widget some how still has a valid reference to a non-existent
	// slot parent.  This shouldn't be the case, but has happened in the past due to various paste bugs
	// so clean it up.
	if (WidgetTree->RootWidget && WidgetTree->RootWidget->Slot)
	{
		WidgetTree->RootWidget->Slot = nullptr;
	}
}

TArray<UWidget*> UBaseWidgetBlueprint::GetAllSourceWidgets()
{
	TArray<UWidget*> Ret;
	ForEachSourceWidgetImpl([&Ret](UWidget* Inner) {
		Ret.Push(Inner);
	});

	return Ret;
}

TArray<const UWidget*> UBaseWidgetBlueprint::GetAllSourceWidgets() const
{
	TArray<const UWidget*> Ret;
	ForEachSourceWidgetImpl([&Ret](UWidget* Inner) {
		Ret.Push(Inner);
	});

	return Ret;
}

void UBaseWidgetBlueprint::ForEachSourceWidget(TFunctionRef<void(UWidget*)> Fn)
{
	ForEachSourceWidgetImpl(Fn);
}

void UBaseWidgetBlueprint::ForEachSourceWidget(TFunctionRef<void(UWidget*)> Fn) const
{
	ForEachSourceWidgetImpl(Fn);
}

void UBaseWidgetBlueprint::ForEachSourceWidgetImpl(TFunctionRef<void(UWidget*)> Fn) const
{
	// This exists in order to facilitate working with collections of UWidgets wo/ 
	// relying on user implemented UWidget virtual functions. During blueprint compilation
	// it is bad practice to call those virtual functions until the class is fully formed
	// and reinstancing has finished. For instance, GetDefaultObject() calls in those user
	// functions may create a CDO before the class has been linked, or even before
	// all member variables have been generated:
	UWidgetTree* WidgetTreeForCapture = WidgetTree;
	ForEachObjectWithOuter(
		WidgetTree,
		[Fn, WidgetTreeForCapture](UObject* Inner)
		{
			if (UWidget* AsWidget = Cast<UWidget>(Inner))
			{
				Fn(AsWidget);
			}
		},
		EGetObjectsFlags::None,// Reject nested objects
		RF_NoFlags,
		EInternalObjectFlags::Garbage // Exclude Pending Kills
	);
}
