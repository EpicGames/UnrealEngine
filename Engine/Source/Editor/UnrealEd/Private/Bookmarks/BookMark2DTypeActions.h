// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bookmarks/IBookmarkTypeActions.h"
#include "EditorViewportClient.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"

#include "Engine/BookMark2D.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "LevelEditorViewport.h"

class FBookMark2DTypeActions : public IBookmarkTypeActions
{
public:

	virtual TSubclassOf<class UBookmarkBase> GetBookmarkClass() override
	{
		return UBookMark2D::StaticClass();
	}

	virtual void InitFromViewport(UBookmarkBase* InBookmark, FEditorViewportClient& InViewportClient) override
	{
		if (UBookMark2D* Bookmark = Cast<UBookMark2D>(InBookmark))
		{
			FVector Location = InViewportClient.GetViewLocation();
			Bookmark->Zoom2D = Location.X;
			Bookmark->Location = FIntPoint(Location.Y, Location.Z);
		}
	}

	virtual void JumpToBookmark(UBookmarkBase* InBookmark, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, FEditorViewportClient& InViewportClient) override
	{
		if (UBookMark2D* Bookmark = Cast<UBookMark2D>(InBookmark))
		{
			// Set all level editing cameras to this bookmark
			FVector Location(Bookmark->Zoom2D, Bookmark->Location.X, Bookmark->Location.Y);
			for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
			{
				ViewportClient->SetViewLocation(Location);
				ViewportClient->Invalidate();
			}
		}
	}
};