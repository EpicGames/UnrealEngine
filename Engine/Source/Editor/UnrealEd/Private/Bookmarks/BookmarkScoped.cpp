// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bookmarks/BookmarkScoped.h"

#include "Bookmarks/BookmarkSingleViewportActions.h"

FBookmarkScoped::FBookmarkScoped() :
	Actions(MakeShared<FBookmarkSingleViewportActions>())
{
	// Bookmark all viewports to restore when going out of scope
	for (FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		UBookMark* BookMark = NewObject<UBookMark>();
		BookMark->AddToRoot();
		Actions->InitFromViewport(BookMark, *ViewportClient);
		BookMarks.Add(ViewportClient, BookMark);
	}
}


FBookmarkScoped::~FBookmarkScoped()
{
	// Restore all viewport bookmarks
	for (FEditorViewportClient* ViewportClient : GEditor->GetAllViewportClients())
	{
		if (UBookMark** BookMark = BookMarks.Find(ViewportClient))
		{
			TSharedPtr<FBookmarkBaseJumpToSettings> Settings = MakeShared<FBookmarkJumpToSettings>();
			Actions->JumpToBookmark(*BookMark, Settings, *ViewportClient);
		}
	}

	for (TPair<FEditorViewportClient*, UBookMark*>& Pair : BookMarks)
	{
		Pair.Value->RemoveFromRoot();
	}
}
