// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bookmarks/IBookmarkTypeTools.h"
#include "Bookmarks/IBookmarkTypeActions.h"

#include "CoreMinimal.h"
#include "GameFramework/WorldSettings.h"
#include "EditorViewportClient.h"
#include "Engine/BookmarkBase.h"
#include "ScopedTransaction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Logging/LogMacros.h"

#define LOCTEXT_NAMESPACE "Bookmarks"

DEFINE_LOG_CATEGORY_STATIC(LogEditorBookmarks, Warning, Warning);

class FBookmarkTypeToolsImpl : public IBookmarkTypeTools
{
private:

	TArray<TSharedRef<IBookmarkTypeActions>> BookmarkTypeActions;

private:

	void Notify(const FText& InText, const bool bInSuccess = true) const
	{
		FNotificationInfo Info(InText);
		Info.bUseLargeFont = false;
		Info.bUseSuccessFailIcons = false;
			
		FSlateNotificationManager::Get().AddNotification(Info);	
	}

	static FORCEINLINE AWorldSettings* GetWorldSettings(FEditorViewportClient* InViewportClient)
	{
		return const_cast<AWorldSettings*>(GetWorldSettings(const_cast<const FEditorViewportClient*>(InViewportClient)));
	}

	static FORCEINLINE const AWorldSettings* GetWorldSettings(const FEditorViewportClient* InViewportClient)
	{
		if (InViewportClient != nullptr)
		{
			if (const UWorld* World = InViewportClient->GetWorld())
			{
				return World->GetWorldSettings();
			}
		}

		return nullptr;
	}

	const TSharedPtr<IBookmarkTypeActions> GetBookmarkTypeActions(const AWorldSettings& WorldSettings) const
	{
		return GetBookmarkTypeActions(WorldSettings.GetDefaultBookmarkClass().Get());
	}

	const TSharedPtr<IBookmarkTypeActions> GetBookmarkTypeActions(const UClass* Class) const
	{
		if (Class != nullptr)
		{
			auto FindBookmarkPred = [Class](const TSharedRef<IBookmarkTypeActions>& InActions)
			{
				// Explicitly check equality, not IsA.
				return Class == InActions->GetBookmarkClass();
			};

			if (const TSharedRef<IBookmarkTypeActions>* FoundActions = BookmarkTypeActions.FindByPredicate(FindBookmarkPred))
			{
				return *FoundActions;
			}
		}
		
		UE_LOGF(LogEditorBookmarks, Warning, "FBookmarkTypeToolsImpl::GetBookmarkTypeActions - Unable to get appropriate BookmarkTypeActions for Class %ls", *GetPathNameSafe(Class));
		return nullptr;
	}

public:

	/**
	 * Gets the current maximum number of bookmarks allowed.
	 *
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual const uint32 GetMaxNumberOfBookmarks(const FEditorViewportClient* InViewportClient) const override
	{
		if (const AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			return WorldSettings->GetMaxNumberOfBookmarks();
		}

		return 0;
	}

	/**
	 * Checks to see if a bookmark exists at a given index
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual const bool CheckBookmark(const uint32 InIndex, const FEditorViewportClient* InViewportClient) const override
	{
		if (const AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			return Bookmarks.IsValidIndex(InIndex) && Bookmarks[InIndex] != nullptr;
		}

		return false;
	}

	/**
	 * Sets the specified bookmark based on the given viewport, allocating it if necessary.
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void CreateOrSetBookmark(const uint32 InIndex, FEditorViewportClient* InViewportClient) const override
	{
		FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("SetBookmark", "Set Bookmark {0}"), InIndex));

		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			if (UBookmarkBase* Bookmark = WorldSettings->GetOrAddBookmark(InIndex, true))
			{
				const TSharedPtr<IBookmarkTypeActions> Actions = GetBookmarkTypeActions(Bookmark->GetClass());
				if (Actions.IsValid())
				{
					Actions->InitFromViewport(Bookmark, *InViewportClient);
				}

				const FText NotifyMessage = LOCTEXT("CreateOrSetBookmarkSuccessFormat", "Saved bookmark {0}");
				Notify(FText::Format(NotifyMessage, InIndex), true);
			}
			else
			{
				UE_LOGF(LogEditorBookmarks, Error, "FBookmarkTypeToolsImpl::CreateOrSetBookmark - Failed to create bookmark at Index %d", InIndex);

				const FText NotifyMessage = LOCTEXT("CreateOrSetBookmarkFailedFormat", "Failed to save bookmark {0}");
				Notify(FText::Format(NotifyMessage, InIndex), false);
			}
		}
	}

	/**
	 * Compacts the available bookmarks into mapped spaces.
	 * Does nothing if all mapped spaces are already filled, or if no bookmarks exist that are not mapped.
	 *
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual void CompactBookmarks(FEditorViewportClient* InViewportClient) const override
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("CompactedBookmarks", "Compacted Bookmarks"));

		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			WorldSettings->CompactBookmarks();
		}
	}

	/**
	 * Jumps to a bookmark from the list.
	 * 
	 * @param InIndex			Index of the bookmark to set.
	 * @param InSettings		Settings to used when jumping to the bookmark.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void JumpToBookmark(const uint32 InIndex, const TSharedPtr<struct FBookmarkBaseJumpToSettings> InSettings, FEditorViewportClient* InViewportClient) const override
	{
		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			bool bFailed = false;
			const FText FailNotifyMessage = LOCTEXT("JumpToBookmarkFailedFormat", "No bookmark at {0} to jump to");
			
			const TArray<UBookmarkBase*>& Bookmarks = WorldSettings->GetBookmarks();
			if (Bookmarks.IsValidIndex(InIndex))
			{
				if (UBookmarkBase* Bookmark = Bookmarks[InIndex])
				{
					const TSharedPtr<IBookmarkTypeActions> Actions = GetBookmarkTypeActions(Bookmark->GetClass());
					if (Actions.IsValid())
					{
						Actions->JumpToBookmark(Bookmark, InSettings, *InViewportClient);
					}
				}
				else
				{
					UE_LOGF(LogEditorBookmarks, Warning, "FBookmarkTypeToolsImpl::JumpToBookmark - Null Bookmark at index %d", InIndex);
				}
			}
			else
			{
				UE_LOGF(LogEditorBookmarks, Warning, "FBookmarkTypeToolsImpl::JumpToBookmark - Invalid bookmark index %d", InIndex);
			}

			if (bFailed)
			{
				Notify(FText::Format(FailNotifyMessage, InIndex), false);
			}
		}
	}

	/**
	 * Clears a bookmark from the list.
	 * 
	 * @param InIndex			Index of the bookmark to clear.
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmark.
	 */
	virtual void ClearBookmark(const uint32 InIndex, FEditorViewportClient* InViewportClient) const override
	{
		const FText ClearBookmarkMessage = FText::Format(LOCTEXT("ClearBookmarkSuccessFormat", "Cleared bookmark {0}"), InIndex);		
		FScopedTransaction ScopedTransaction(ClearBookmarkMessage);

		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			WorldSettings->ClearBookmark(InIndex);

			Notify(ClearBookmarkMessage, true);
		}
	}

	/**
	 * Clears all book marks
	 * 
	 * @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	 */
	virtual void ClearAllBookmarks(FEditorViewportClient* InViewportClient) const override
	{
		const FText ClearAllBookmarksMessage = LOCTEXT("ClearAllBookmarksSuccessFormat", "Cleared all bookmarks");
		FScopedTransaction ScopedTransaction(ClearAllBookmarksMessage);

		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			WorldSettings->ClearAllBookmarks();

			Notify(ClearAllBookmarksMessage, true);
		}
	}

	/**
	 * Gets the currently configured Bookmark class.
	 */
	virtual const TSubclassOf<UBookmarkBase> GetBookmarkClass(const FEditorViewportClient* InViewportClient) const override
	{
		if (const AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			return WorldSettings->GetDefaultBookmarkClass();
		}

		return nullptr;
	}

	/**
	 * Registers the given Bookmark Type Actions so they can be used by the editor.
	 *
	 * @param InActions			The Actions to register.		
	 */
	virtual void RegisterBookmarkTypeActions(const TSharedRef<IBookmarkTypeActions>& InActions) override
	{
		BookmarkTypeActions.Add(InActions);
	}

	/**
	 * Unregisters the given Bookmark Type Actions so they are no longer considered by the editor.
	 *
	 * @param InActions			The Actions to unregister.		
	 */
	virtual void UnregisterBookmarkTypeActions(const TSharedRef<IBookmarkTypeActions>& InActions) override
	{
		BookmarkTypeActions.Remove(InActions);
	}

	/**
	* Upgrades all bookmarks, ensuring they are of appropriate class.
	*
	* @param InViewportClient	Level editor viewport client used to reference the world which owns the bookmarks.
	* @param InWorldSettings	World Settings object that is requesting the upgrade.
	*/
	virtual void UpgradeBookmarks(class FEditorViewportClient* InViewportClient, AWorldSettings* InWorldSettings) const override
	{
		// TODO: Should this be guarded by a transaction?

		if (AWorldSettings* WorldSettings = GetWorldSettings(InViewportClient))
		{
			if (WorldSettings != InWorldSettings)
			{
				UE_LOGF(LogEditorBookmarks, Warning, "FBookmarkTypeToolsImpl::UpgradeBookmarks - Viewport client does not correspond to correct world (Viewport WorldSettings = %ls Expected WorldSettings = %ls)",
					*GetPathNameSafe(WorldSettings), *GetPathNameSafe(InWorldSettings));
				return;
			}

			// Make sure the new bookmark class and type actions are valid.
			if (UClass* NewBookmarkClass = InWorldSettings->GetDefaultBookmarkClass().Get())
			{
				const TSharedPtr<IBookmarkTypeActions> NewBookmarkActions = GetBookmarkTypeActions(*InWorldSettings);
				if (NewBookmarkActions.IsValid())
				{
					UClass* OldBookmarkClass = nullptr;
					TSharedPtr<IBookmarkTypeActions> OldBookmarkActions = nullptr;

					// Cache off our current viewport state.
					FViewportCameraTransform ViewportTransform = InViewportClient->GetViewTransform();
					EViewModeIndex ViewMode = InViewportClient->GetViewMode();

					const int32 NumBookmarks = InWorldSettings->GetMaxNumberOfBookmarks();
					const TArray<UBookmarkBase*>& AvailableBookmarks = InWorldSettings->GetBookmarks();

					// Here, we'll go through each existing bookmark, jump to it, then create a new bookmark
					// from that restored state.
					// This approach isn't going to be perfect due to potential incompatibility.
					// Alternatives may be to allow intermixing of Bookmark types, or completely obliterating bookmarks,
					// neither of which seem like great alternatives.
					for (int32 i = 0; i < NumBookmarks; ++i)
					{
						if (UBookmarkBase* OldBookmark = AvailableBookmarks[i])
						{
							if (OldBookmarkClass == nullptr ||
								OldBookmarkClass != OldBookmark->GetClass())
							{
								OldBookmarkClass = OldBookmark->GetClass();
								OldBookmarkActions = GetBookmarkTypeActions(OldBookmarkClass);
							}

							InWorldSettings->ClearBookmark(i);

							if (OldBookmarkActions.IsValid())
							{
								OldBookmarkActions->JumpToBookmark(OldBookmark, nullptr, *InViewportClient);
								UBookmarkBase* NewBookmark = InWorldSettings->GetOrAddBookmark(i, false);
								NewBookmarkActions->InitFromViewport(NewBookmark, *InViewportClient);
							}
						}
					}

					// Restore our viewport state.
					InViewportClient->SetViewLocation(ViewportTransform.GetLocation());
					InViewportClient->SetLookAtLocation(ViewportTransform.GetLookAt());
					InViewportClient->SetOrthoZoom(ViewportTransform.GetOrthoZoom());
					InViewportClient->SetViewRotation(ViewportTransform.GetRotation());
					InViewportClient->SetViewMode(ViewMode);
				}
			}
		}
	}
};

IBookmarkTypeTools& IBookmarkTypeTools::Get()
{
	static FBookmarkTypeToolsImpl Impl;
	return Impl;
}

#undef LOCTEXT_NAMESPACE
