// Copyright Epic Games, Inc. All Rights Reserved.


#include "BusyCursor.h"

#include "HAL/Platform.h"

static int32 GScopedBusyCursorReferenceCounter = 0;

FScopedBusyCursor::FScopedBusyCursor()
{
	// @todo Replace with a slate busy cursor.
}

FScopedBusyCursor::~FScopedBusyCursor()
{
}
