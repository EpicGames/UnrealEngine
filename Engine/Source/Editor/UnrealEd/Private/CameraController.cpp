// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CameraController.cpp: Implements controls for a camera with pseudo-physics
=============================================================================*/

#include "CameraController.h"
#include "EditorModeManager.h"
#include "EditorModes.h"


/** Constructor */
FEditorCameraController::FEditorCameraController()
	: Config(),
	  MovementVelocity( 0.0f, 0.0f, 0.0f ),
	  FOVVelocity( 0.0f ),
	  RotationVelocityEuler( 0.0f, 0.0f, 0.0f ),
	  OriginalFOVForRecoil( -1.0f ) 	// -1.0f here means 'initialize me on demand'
{
}



/**
 * Updates the position and orientation of the camera as well as other state (like velocity.)  Should be
 * called every frame.
 *
 * @param	UserImpulseData			Input data from the user this frame
 * @param	DeltaTime				Time interval since last update
 * @param	bAllowRecoilIfNoImpulse	True if we should recoil FOV if needed
 * @param	MovementSpeedScale		Scales the speed of movement
 * @param	InOutCameraPosition		[in, out] Camera position
 * @param	InOutCameraEuler		[in, out] Camera orientation
 * @param	InOutCameraFOV			[in, out] Camera field of view
 */
void FEditorCameraController::UpdateSimulation(
	const FCameraControllerUserImpulseData& UserImpulseData,
	const float DeltaTime,
	const bool bAllowRecoilIfNoImpulse,
	const float MovementSpeedScale,
	FVector& InOutCameraPosition,
	FVector& InOutCameraEuler,
	float& InOutCameraFOV )
{
	bool bAnyUserImpulse = false;


	// Apply dead zone test to user impulse data
	//ApplyImpulseDeadZone( UserImpulseData, FinalUserImpulse, bAnyUserImpulse );
	if( UserImpulseData.RotateYawVelocityModifier != 0.0f ||
		UserImpulseData.RotatePitchVelocityModifier != 0.0f ||
		UserImpulseData.RotateRollVelocityModifier != 0.0f ||
		UserImpulseData.MoveForwardBackwardImpulse != 0.0f ||
		UserImpulseData.MoveRightLeftImpulse!= 0.0f ||
		UserImpulseData.MoveWorldUpDownImpulse != 0.0f ||
		UserImpulseData.MoveLocalUpDownImpulse != 0.0f ||
		UserImpulseData.ZoomOutInImpulse != 0.0f ||
		UserImpulseData.RotateYawImpulse != 0.0f ||
		UserImpulseData.RotatePitchImpulse != 0.0f ||
		UserImpulseData.RotateRollImpulse != 0.0f
		)
	{
		bAnyUserImpulse = true;
	}

	FVector TranslationCameraEuler = InOutCameraEuler;
	if (Config.bPlanarCamera)
	{
		//remove roll
		TranslationCameraEuler.X = 0;
		//remove pitch
		TranslationCameraEuler.Y = 0;
	}
	// Movement
	UpdatePosition( UserImpulseData, DeltaTime, MovementSpeedScale, TranslationCameraEuler, InOutCameraPosition );



	// Rotation
	UpdateRotation( UserImpulseData, DeltaTime, InOutCameraEuler );



	// FOV
	UpdateFOV( UserImpulseData, DeltaTime, InOutCameraFOV );



	// Recoil camera FOV if we need to
	ApplyRecoil( DeltaTime, bAllowRecoilIfNoImpulse, bAnyUserImpulse, InOutCameraFOV );

}


void FEditorCameraController::ResetVelocity()
{
	MovementVelocity = FVector::ZeroVector;
	FOVVelocity = 0.f;
	RotationVelocityEuler = FVector::ZeroVector;
}


/**true if this camera currently has rotational velocity*/
bool FEditorCameraController::IsRotating (void) const
{
	if ((RotationVelocityEuler.X != 0.0f) || (RotationVelocityEuler.Y != 0.0f) || (RotationVelocityEuler.Z != 0.0f))
	{
		return true;
	}
	return false;
}


/**
 * Applies the dead zone setting to the incoming user impulse data
 *
 * @param	InUserImpulse	Input user impulse data
 * @param	OutUserImpulse	[out] Output user impulse data with dead zone applied
 * @param	bOutAnyImpulseData	[out] True if there was any user impulse this frame
 */
void FEditorCameraController::ApplyImpulseDeadZone( const FCameraControllerUserImpulseData& InUserImpulse,
													FCameraControllerUserImpulseData& OutUserImpulse,
													bool& bOutAnyImpulseData )
{
	FMemory::Memzero( &OutUserImpulse, sizeof( OutUserImpulse ) );

	// Keep track if there is any actual user input.  This is so that when all of the flight controls
	// are released, we can take action (such as resetting the camera FOV back to what it was.)
	bOutAnyImpulseData = false;

	if( FMath::Abs( InUserImpulse.MoveRightLeftImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.MoveRightLeftImpulse = InUserImpulse.MoveRightLeftImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.MoveForwardBackwardImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.MoveForwardBackwardImpulse = InUserImpulse.MoveForwardBackwardImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.MoveWorldUpDownImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.MoveWorldUpDownImpulse = InUserImpulse.MoveWorldUpDownImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.MoveLocalUpDownImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.MoveLocalUpDownImpulse = InUserImpulse.MoveLocalUpDownImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.RotateYawImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.RotateYawImpulse = InUserImpulse.RotateYawImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.RotatePitchImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.RotatePitchImpulse = InUserImpulse.RotatePitchImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.RotateRollImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.RotateRollImpulse = InUserImpulse.RotateRollImpulse;
		bOutAnyImpulseData = true;
	}

	if( FMath::Abs( InUserImpulse.ZoomOutInImpulse ) >= Config.ImpulseDeadZoneAmount )
	{
		OutUserImpulse.ZoomOutInImpulse = InUserImpulse.ZoomOutInImpulse;
		bOutAnyImpulseData = true;
	}

	// No dead zone for these
	OutUserImpulse.RotateYawVelocityModifier = InUserImpulse.RotateYawVelocityModifier;
	OutUserImpulse.RotatePitchVelocityModifier = InUserImpulse.RotatePitchVelocityModifier;
	OutUserImpulse.RotateRollVelocityModifier = InUserImpulse.RotateRollVelocityModifier;
	if( OutUserImpulse.RotateYawVelocityModifier != 0.0f ||
		OutUserImpulse.RotatePitchVelocityModifier != 0.0f ||
		OutUserImpulse.RotateRollVelocityModifier != 0.0f )
	{
		bOutAnyImpulseData = true;
	}
}



/**
 * Updates the camera position.  Called every frame by UpdateSimulation.
 *
 * @param	UserImpulse				User impulse data for the current frame
 * @param	DeltaTime				Time interval
 * @param	MovementSpeedScale		Additional movement accel/speed scale
 * @param	CameraEuler				Current camera rotation
 * @param	InOutCameraPosition		[in, out] Camera position
 */
void FEditorCameraController::UpdatePosition( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, const float MovementSpeedScale, const FVector& CameraEuler, FVector& InOutCameraPosition )
{
	// Compute local impulse
	FVector LocalSpaceImpulse;
	{
		// NOTE: Forward/back and right/left impulse are applied in local space, but up/down impulse is
		//		 applied in world space.  This is because it feels more intuitive to always move straight
		//		 up or down with those controls.
		LocalSpaceImpulse =
			FVector( UserImpulse.MoveForwardBackwardImpulse,		// Local space forward/back
					 UserImpulse.MoveRightLeftImpulse,				// Local space right/left
					 UserImpulse.MoveLocalUpDownImpulse );			// Local space up/down
	}


	// Compute world space acceleration
	FVector WorldSpaceAcceleration;
	{
		// Compute camera orientation, then rotate our local space impulse to world space
		const FQuat CameraOrientation = FQuat::MakeFromEuler( CameraEuler );
		FVector WorldSpaceImpulse = CameraOrientation.RotateVector( LocalSpaceImpulse );

		// Accumulate any world space impulse we may have
		// NOTE: Up/down impulse is applied in world space.  See above comments for more info.
		WorldSpaceImpulse +=
			FVector( 0.0f,											// World space forward/back
					 0.0f,											// World space right/left
					 UserImpulse.MoveWorldUpDownImpulse );			// World space up/down

		// Cap impulse by normalizing, but only if our magnitude is greater than 1.0
		//if( WorldSpaceImpulse.SizeSquared() > 1.0f )
		{
			//WorldSpaceImpulse = WorldSpaceImpulse.UnsafeNormal();
		}

		// Compute world space acceleration
		WorldSpaceAcceleration = WorldSpaceImpulse * Config.MovementAccelerationRate * MovementSpeedScale;
	}


	if( Config.bUsePhysicsBasedMovement )
	{
		// Accelerate the movement velocity
		MovementVelocity += WorldSpaceAcceleration * DeltaTime;


		// Apply damping
		{
			const float DampingFactor = FMath::Clamp( Config.MovementVelocityDampingAmount * DeltaTime, 0.0f, 0.75f );

			// Decelerate
			MovementVelocity += -MovementVelocity * DampingFactor;
		}
	}
	else
	{
		// No physics, so just use the acceleration as our velocity
		MovementVelocity = WorldSpaceAcceleration;
	}


	// Constrain maximum movement speed
	if( MovementVelocity.SizeSquared() > FMath::Square(Config.MaximumMovementSpeed * MovementSpeedScale) )
	{
		MovementVelocity = MovementVelocity.GetUnsafeNormal() * Config.MaximumMovementSpeed * MovementSpeedScale;
	}


	// Clamp velocity to a reasonably small number
	if( MovementVelocity.SizeSquared() < FMath::Square(KINDA_SMALL_NUMBER) )
	{
		MovementVelocity = FVector::ZeroVector;
	}


	// Update camera position
	InOutCameraPosition += MovementVelocity * DeltaTime;
}



/**
 * Updates the camera rotation.  Called every frame by UpdateSimulation.
 *
 * @param	UserImpulse			User impulse data for this frame
 * @param	DeltaTime			Time interval
 * @param	InOutCameraEuler	[in, out] Camera rotation
 */
void FEditorCameraController::UpdateRotation( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, FVector &InOutCameraEuler )
{
	FVector RotateImpulseEuler =
		FVector( UserImpulse.RotateRollImpulse,
		UserImpulse.RotatePitchImpulse,
		UserImpulse.RotateYawImpulse );

	FVector RotateVelocityModifierEuler =
		FVector( UserImpulse.RotateRollVelocityModifier,
		UserImpulse.RotatePitchVelocityModifier,
		UserImpulse.RotateYawVelocityModifier );

	// Iterate for each euler axis - yaw, pitch and roll
	for( int32 CurRotationAxis = 0; CurRotationAxis < 3; ++CurRotationAxis )
	{
		// This will serve as both our source and destination rotation value
		FVector::FReal& RotationVelocity = RotationVelocityEuler[ CurRotationAxis ];

		const FVector::FReal RotationImpulse = RotateImpulseEuler[CurRotationAxis];
		const FVector::FReal RotationVelocityModifier = RotateVelocityModifierEuler[ CurRotationAxis ];


		// Compute acceleration
		FVector::FReal RotationAcceleration = RotationImpulse * Config.RotationAccelerationRate;

		if( Config.bUsePhysicsBasedRotation || Config.bForceRotationalPhysics)
		{
			// Accelerate the rotation velocity
			RotationVelocity += RotationAcceleration * DeltaTime;

			// Apply velocity modifier.  This is used for mouse-look based camera rotation, where
			// we don't need to account for DeltaTime, since the value is based on an explicit number
			// of degrees per cursor pixel moved.
			RotationVelocity += RotationVelocityModifier;


			// Apply damping
			{
				const float DampingFactor = FMath::Clamp( Config.RotationVelocityDampingAmount * DeltaTime, 0.0f, 0.75f );

				// Decelerate
				RotationVelocity += -RotationVelocity * DampingFactor;
			}
		}
		else
		{
			// No physics, so just use the acceleration as our velocity
			RotationVelocity = RotationAcceleration;

			// Apply velocity modifier.  This is used for mouse-look based camera rotation, where
			// we don't need to account for DeltaTime, since the value is based on an explicit number
			// of degrees per cursor pixel moved.
			RotationVelocity += RotationVelocityModifier;
		}


		// Constrain maximum rotation speed
		RotationVelocity = FMath::Clamp(RotationVelocity, -Config.MaximumRotationSpeed, Config.MaximumRotationSpeed);

		// Clamp velocity to a reasonably small number
		if( FMath::Abs( RotationVelocity ) < KINDA_SMALL_NUMBER )
		{
			RotationVelocity = 0.0f;
		}


		// Update rotation
		InOutCameraEuler[ CurRotationAxis ] += RotationVelocity * DeltaTime;


		// Constrain final pitch rotation value to configured range
		if( CurRotationAxis == 1 )		// 1 == pitch
		{
			// Normalize the angle to -180 to 180.
			FVector::FReal Angle = FMath::Fmod(InOutCameraEuler[ CurRotationAxis ], 360.0);
			if (Angle > 180.)
			{
				Angle -= 360.;
			}
			else if (Angle < -180.)
			{
				Angle += 360.;
			}
			
			if (Config.bLockedPitch) 
			{
				// Clamp the angle.
				InOutCameraEuler[ CurRotationAxis ] =
					FMath::Clamp( Angle,
					(FVector::FReal)Config.MinimumAllowedPitchRotation,
					(FVector::FReal)Config.MaximumAllowedPitchRotation );
			}
		}
	}
}



/**
 * Update the field of view.  Called every frame by UpdateSimulation.
 *
 * @param	UserImpulse		User impulse data for this frame
 * @param	DeltaTime		Time interval
 * @param	InOutCameraFOV	[in, out] Camera field of view
 */
void FEditorCameraController::UpdateFOV( const FCameraControllerUserImpulseData& UserImpulse, const float DeltaTime, float& InOutCameraFOV )
{
	// Compute acceleration
	float FOVAcceleration = UserImpulse.ZoomOutInImpulse * Config.FOVAccelerationRate;


	// Is the user actively changing the FOV?
	if( FMath::Abs( FOVAcceleration ) > KINDA_SMALL_NUMBER )
	{
		// If we've never cached a FOV, then go ahead and do that now
		if( OriginalFOVForRecoil < 0.0f )
		{
			OriginalFOVForRecoil = InOutCameraFOV;
		}
	}


	if( Config.bUsePhysicsBasedFOV )
	{
		// Accelerate the FOV velocity
		FOVVelocity += FOVAcceleration * DeltaTime;


		// Apply damping
		{
			const float DampingFactor = FMath::Clamp( Config.FOVVelocityDampingAmount * DeltaTime, 0.0f, 0.75f );

			// Decelerate
			FOVVelocity += -FOVVelocity * DampingFactor;
		}
	}
	else
	{
		// No physics, so just use the acceleration as our velocity
		FOVVelocity = FOVAcceleration;
	}


	// Constrain maximum FOV speed
	FOVVelocity = FMath::Clamp<float>(FOVVelocity, -Config.MaximumFOVSpeed, Config.MaximumFOVSpeed );

	// Clamp velocity to a reasonably small number
	if( FMath::Abs( FOVVelocity ) < KINDA_SMALL_NUMBER )
	{
		FOVVelocity = 0.0f;
	}


	// Update camera FOV
	InOutCameraFOV += FOVVelocity * DeltaTime;


	// Constrain final FOV to configured range
	InOutCameraFOV = FMath::Clamp( InOutCameraFOV, Config.MinimumAllowedFOV, Config.MaximumAllowedFOV );
}



/**
 * Applies FOV recoil (if appropriate).  Called every frame by UpdateSimulation.
 *
 * @param	DeltaTime					Time interval
 * @param	bAllowRecoilIfNoImpulse		Whether recoil should be allowed if there wasn't any user impulse
 * @param	bAnyUserImpulse				True if there was user impulse data this iteration
 * @param	InOutCameraFOV				[in, out] Camera field of view
 */
void FEditorCameraController::ApplyRecoil( const float DeltaTime, const bool bAllowRecoilIfNoImpulse, bool bAnyUserImpulse, float& InOutCameraFOV )
{
	bool bIsRecoilingFOV = false;

	// Is the FOV 'recoil' feature enabled?  If so, we'll smoothly snap the FOV angle back to what
	// it was before the user started interacting with the camera.
	if( Config.bEnableFOVRecoil )
	{
		// We don't need to recoil if the user hasn't started changing the FOV yet
		if( OriginalFOVForRecoil >= 0.0f )
		{
			// If there isn't any user impulse, then go ahead and recoil the camera FOV
			if( !bAnyUserImpulse && bAllowRecoilIfNoImpulse )
			{
				// Kill any physics-based FOV velocity
				FOVVelocity = 0.0f;

				const float FOVDistance = FMath::Abs( InOutCameraFOV - OriginalFOVForRecoil );
				if( FOVDistance > 0.1f )
				{
					// Recoil speed in 'distances' per second
					const float CameraFOVRecoilSpeedScale = 10.0f;

					if( InOutCameraFOV < OriginalFOVForRecoil )
					{
						InOutCameraFOV += FOVDistance * DeltaTime * CameraFOVRecoilSpeedScale;
					}
					else
					{
						InOutCameraFOV -= FOVDistance * DeltaTime * CameraFOVRecoilSpeedScale;
					}

					// We've tinkered with the FOV, so make sure we don't cache these changes
					bIsRecoilingFOV = true;
				}
				else
				{
					// Close enough, so snap it!
					InOutCameraFOV = OriginalFOVForRecoil;

					// We're done done manipulating the FOV for now
					OriginalFOVForRecoil = -1.0f;
				}
			}
		}
	}
}
