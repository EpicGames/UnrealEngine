// Copyright Epic Games, Inc. All Rights Reserved.


#include "ClassIconFinder.h"

#include "AssetRegistry/AssetData.h"
#include "Blueprint/BlueprintSupport.h"
#include "Containers/UnrealString.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

const FSlateBrush* FClassIconFinder::FindIconForActors(const TArray< TWeakObjectPtr<AActor> >& InActors, UClass*& CommonBaseClass)
{
	// Get the common base class of the selected actors
	FSlateIcon CommonIcon;

	for( int32 ActorIdx = 0; ActorIdx < InActors.Num(); ++ActorIdx )
	{
		TWeakObjectPtr<const AActor> Actor = InActors[ActorIdx];
		UClass* ObjClass = Actor->GetClass();
		check(ObjClass);

		if (!CommonBaseClass)
		{
			CommonBaseClass = ObjClass;
		}
		while (!ObjClass->IsChildOf(CommonBaseClass))
		{
			CommonBaseClass = CommonBaseClass->GetSuperClass();
		}

		FSlateIcon ActorIcon = FindSlateIconForActor(Actor);

		if (!CommonIcon.IsSet())
		{
			CommonIcon = ActorIcon;
		}

		if (CommonIcon != ActorIcon)
		{
			CommonIcon = FSlateIconFinder::FindIconForClass(CommonBaseClass);
		}
	}

	return CommonIcon.GetOptionalIcon();
}

FSlateIcon FClassIconFinder::FindSlateIconForActor( const TWeakObjectPtr<const AActor>& InActor )
{
	// Actor specific overrides to normal per-class icons
	const AActor* Actor = InActor.Get();

	if ( Actor )
	{
		FName IconName = Actor->GetCustomIconName();
		if (IconName != NAME_None)
		{
			return FSlateIconFinder::FindIcon(IconName);
		}
	
		// Actor didn't specify an icon - fallback on the class icon
		return FSlateIconFinder::FindIconForClass(Actor->GetClass());
	}
	else
	{
		// If the actor reference is NULL it must have been deleted
		return FSlateIconFinder::FindIcon("ClassIcon.Deleted");
	}
}

const FSlateBrush* FClassIconFinder::FindIconForActor( const TWeakObjectPtr<const AActor>& InActor )
{
	return FindSlateIconForActor(InActor).GetOptionalIcon();
}

const UClass* FClassIconFinder::GetIconClassForBlueprint(const UBlueprint* InBlueprint)
{
	if ( !InBlueprint )
	{
		return nullptr;
	}

	// If we're loaded and have a generated class, just use that
	if ( const UClass* GeneratedClass = InBlueprint->GeneratedClass )
	{
		return GeneratedClass;
	}

	// We don't have a generated class, so instead try and use the parent class from our meta-data
	return GetIconClassForAssetData(FAssetData(InBlueprint));
}

const UClass* FClassIconFinder::GetIconClassForAssetData(const FAssetData& InAssetData, bool* bOutIsClassType)
{
	if ( bOutIsClassType )
	{
		*bOutIsClassType = false;
	}

	UClass* AssetClass = FindObjectSafe<UClass>(InAssetData.AssetClassPath);
	if ( !AssetClass )
	{
		return nullptr;
	}

	if ( AssetClass == UClass::StaticClass() )
	{
		if ( bOutIsClassType )
		{
			*bOutIsClassType = true;
		}

		return FindObject<UClass>(nullptr, *InAssetData.GetObjectPathString());
	}
	
	static const FName IgnoreClassThumbnail(TEXT("IgnoreClassThumbnail"));
	if ( (AssetClass->IsChildOf<UBlueprint>() || AssetClass->IsChildOf<UClass>()) &&
		!AssetClass->HasMetaDataHierarchical(IgnoreClassThumbnail))
	{
		if ( bOutIsClassType )
		{
			*bOutIsClassType = true;
		}

		// We need to use the asset data to get the parent class as the generated class may not be loaded
		FString ParentClassName;
		if ( !InAssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, ParentClassName) )
		{
			InAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassName);
		}
		if ( !ParentClassName.IsEmpty() )
		{
			UObject* Outer = nullptr;
			ResolveName(Outer, ParentClassName, false, false);
			return FindObject<UClass>(Outer, *ParentClassName);
		}
	}

	// Default to using the class for the asset type
	return AssetClass;
}
