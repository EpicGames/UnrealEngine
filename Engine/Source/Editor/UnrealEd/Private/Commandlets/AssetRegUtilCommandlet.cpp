// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AssetRegUtilCommandlet.cpp: General-purpose commandlet for anything which
	makes integral use of the asset registry.
=============================================================================*/

#include "Commandlets/AssetRegUtilCommandlet.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Engine/Texture.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/FileHelper.h"
#include "PackageHelperFunctions.h"
#include "Serialization/ArrayReader.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetRegUtilCommandlet)

DEFINE_LOG_CATEGORY(LogAssetRegUtil);

const static FName NAME_UnresolvedPackageName = FName(TEXT("UnresolvedPackageName"));

const static FName NAME_uasset(TEXT("uasset"));
const static FName NAME_umap(TEXT("umap"));
const static FName NAME_uexp(TEXT("uexp"));
const static FName NAME_ubulk(TEXT("ubulk"));
const static FName NAME_uptnl(TEXT("uptnl"));


struct FSortableDependencyEntry
{
	FSortableDependencyEntry(const FName& InLongPackageName, const FName& InFilePath, const FName& InExtension, const int32 InDepSet, const int32 InDepHierarchy, const int32 InDepOrder, bool InHasDependencies, TSet<FTopLevelAssetPath> &&InClasses)
		: LongPackageName(InLongPackageName)
		, FilePath(InFilePath)
		, Extension(InExtension)
		, Classes(MoveTemp(InClasses))
		, DepSet(InDepSet)
		, DepHierarchy(InDepHierarchy)
		, DepOrder(InDepOrder)
		, bHasDependencies(InHasDependencies)
		, bIsAsset(true)
	{ }

	// case for packages which arn't uassets
	FSortableDependencyEntry(const FName& InFilePath, const FName& InExtension, const int32 InDepSet)
		: LongPackageName(NAME_UnresolvedPackageName)
		, FilePath(InFilePath)
		, Extension( InExtension )
		, DepSet(InDepSet)
		, DepHierarchy(0)
		, DepOrder(0)
		, bHasDependencies(false)
		, bIsAsset(false)
	{ }

	FName LongPackageName;
	FName FilePath;
	FName Extension;
	TSet<FTopLevelAssetPath> Classes;
	int32 DepSet;
	int32 DepHierarchy;
	int32 DepOrder;
	bool bHasDependencies;
	bool bIsAsset;
};

/*
We want exports to be sorted in reverse hierarchical order, to replicate this kind of ordering as seen in a natural OpenOrder log:

	"../../../engine/Content/EngineMaterials/WorldGridMaterial.uasset" 274
	"../../../engine/Content/EngineMaterials/T_Default_Material_Grid_N.uasset" 275
	"../../../engine/Content/EngineMaterials/T_Default_Material_Grid_M.uasset" 276
	"../../../engine/Content/Functions/Engine_MaterialFunctions01/Opacity/CameraDepthFade.uasset" 277
	...
	"../../../engine/Content/EngineMaterials/T_Default_Material_Grid_N.uexp" 432
	"../../../engine/Content/EngineMaterials/T_Default_Material_Grid_M.uexp" 433
	"../../../engine/Content/Functions/Engine_MaterialFunctions01/Opacity/CameraDepthFade.uexp" 434
	"../../../engine/Content/EngineMaterials/WorldGridMaterial.uexp" 435
*/

struct FSortableDependencySortForHeaders
{
	bool operator()(const FSortableDependencyEntry& A, const FSortableDependencyEntry& B) const
	{
		return (A.DepHierarchy == B.DepHierarchy) ? A.DepOrder < B.DepOrder : A.DepHierarchy < B.DepHierarchy;
	}
};

struct FSortableDependencySortForExports
{
	bool operator()(const FSortableDependencyEntry& A, const FSortableDependencyEntry& B) const
	{
		return (A.DepHierarchy == B.DepHierarchy) ? A.DepOrder < B.DepOrder : A.DepHierarchy > B.DepHierarchy;
	}
};


struct FSortableDependencySort
{
	FSortableDependencySort(const TArray<FName>& InGroupExtensions, const TArray<FTopLevelAssetPath>& InGroupClasses, const TMap<FName, int32> InExtensionPriority)
		: GroupExtensions(InGroupExtensions)
		, GroupClasses(InGroupClasses)
		, ExtensionPriority(InExtensionPriority)
	{

	}

	const TArray<FName>& GroupExtensions;
	const TArray<FTopLevelAssetPath>& GroupClasses;
	const TMap<FName, int32> ExtensionPriority;

	bool operator()(const FSortableDependencyEntry& A, const FSortableDependencyEntry& B) const
	{
		// we want to sort everything in the order it came in on primarily (Ie the DepSet).
		bool bIsAExtensionGrouped = GroupExtensions.Contains(A.Extension);
		bool bIsBExtensionGrouped = GroupExtensions.Contains(B.Extension);

		// the extensions which are grouped we want to primarily sort on the grouping
		if (bIsAExtensionGrouped != bIsBExtensionGrouped)
		{
			return bIsAExtensionGrouped < bIsBExtensionGrouped;
		}


		bool bIsAClassGrouped = false;
		bool bIsBClassGrouped = false;
		FTopLevelAssetPath AClass;
		FTopLevelAssetPath BClass;
		for (const FTopLevelAssetPath& Class : GroupClasses)
		{
			if (A.Classes.Contains(Class))
			{
				AClass = Class;
				bIsAClassGrouped = true;
			}
			if (B.Classes.Contains(Class))
			{
				BClass = Class;
				bIsBClassGrouped = true;
			}
		}
		if (bIsAClassGrouped != bIsBClassGrouped)
		{
			return bIsAClassGrouped < bIsBClassGrouped;
		}

		if ((AClass != BClass) && bIsAClassGrouped && bIsBClassGrouped)
		{
			return BClass.Compare(AClass) < 0;
		}

		if (A.DepSet != B.DepSet)
		{
			return A.DepSet < B.DepSet;
		}

		const int32 *AExtPriority = ExtensionPriority.Find(A.Extension);
		const int32 *BExtPriority = ExtensionPriority.Find(B.Extension);


		if (!AExtPriority || !BExtPriority)
		{
			if (!AExtPriority && !BExtPriority)
			{
				return A.DepHierarchy == B.DepHierarchy ? A.DepOrder < B.DepOrder : A.DepHierarchy < B.DepHierarchy;
			}

			if (!AExtPriority)
			{
				return true;
			}
			return false;
		}

		if (*AExtPriority != *BExtPriority)
		{
			return *AExtPriority < *BExtPriority;
		}

		if (A.DepHierarchy != B.DepHierarchy)
		{
			if (A.Extension == NAME_uexp) // the case of uexp we actually want to reverse the hierarchy order
			{
				return A.DepHierarchy > B.DepHierarchy;
			}
			return A.DepHierarchy < B.DepHierarchy;
		}

		return A.DepOrder < B.DepOrder;

		// now everything else goes on either normal dependency order or reverse
	}
};

UAssetRegUtilCommandlet::UAssetRegUtilCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAssetRegUtilCommandlet::RecursivelyGrabDependencies(TArray<FSortableDependencyEntry>& OutSortableDependencies,
                                                          const int32& DepSet, int32& DepOrder, int32 DepHierarchy, TSet<FName>& ProcessedFiles, const TSet<FName>& OriginalSet, const FName& FilePath, const FName& PackageFName, const TArray<FTopLevelAssetPath>& FilterByClasses )
{
	bool bHasDependencies = false;
	//now walk the dependency tree for everything under this package
	TArray<FName> Dependencies;
	AssetRegistry->GetDependencies(PackageFName, Dependencies);

	TArray<FAssetData> AssetsData;
	AssetRegistry->GetAssetsByPackageName(PackageFName, AssetsData, true);
	TSet<FTopLevelAssetPath> Classes;
	Classes.Reserve(AssetsData.Num());
	for (const FAssetData& AssetData : AssetsData)
	{
		Classes.Add(AssetData.AssetClassPath);
		TArray<FTopLevelAssetPath> AncestorClasses;
		AssetRegistry->GetAncestorClassNames(AssetData.AssetClassPath, AncestorClasses);
		Classes.Append(AncestorClasses);
	}
	TSet<FTopLevelAssetPath> FilteredClasses;
	for (const FTopLevelAssetPath& FilterClass : FilterByClasses)
	{
		if (Classes.Contains(FilterClass))
		{
			FilteredClasses.Add(FilterClass);
		}
	}

	//keeping a simple path-only set around for the current hierarchy, so things don't get too slow if we end up unrolling a massive dependency tree.
	ProcessedFiles.Add(FilePath);
	FName ExtensionFName = FName(*FPaths::GetExtension(FilePath.ToString()));
	OutSortableDependencies.Add(FSortableDependencyEntry(PackageFName, FilePath, ExtensionFName, DepSet, DepHierarchy, DepOrder, Dependencies.Num()>0, MoveTemp(FilteredClasses)));

	++DepOrder;

	if (Dependencies.Num()>0)
	{
		//walk the dependencies in reverse order akin to how headers tend to be arranged in current load orders
		for (int32 DependencyIndex = Dependencies.Num() - 1; DependencyIndex >= 0; --DependencyIndex)
		{
			const FName& DepPackageName = Dependencies[DependencyIndex];
			const FString DepFilePath = FPackageName::LongPackageNameToFilename(DepPackageName.ToString(), TEXT(".uasset")).ToLower();
			const FName DepPathFName = FName(*DepFilePath);
			//if the package is in the main set, we already walked its dependencies, so we can stop early.
			if (!ProcessedFiles.Contains(DepPathFName) && OriginalSet.Contains(DepPathFName))
			{
				RecursivelyGrabDependencies(OutSortableDependencies, DepSet, DepOrder, DepHierarchy + 1, ProcessedFiles, OriginalSet, DepPathFName, DepPackageName, FilterByClasses);
			}
		}
	}
}

void UAssetRegUtilCommandlet::ReorderOrderFile(const FString& OrderFilePath, const FString& ReorderFileOutPath)
{
	TSet<FName> OriginalEntrySet;
	if (!LoadOrderFiles(OrderFilePath, OriginalEntrySet))
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Could not load specified order file.");
		return;
	}

	UE_LOGF(LogAssetRegUtil, Display, "Generating new file order via Asset Registry.");

	TArray<FSortableDependencyEntry> UnsortedEntries;
	UnsortedEntries.Empty(OriginalEntrySet.Num());

	// quick elimination tset
	TSet<FName> ProcessedFiles;
	ProcessedFiles.Reserve(OriginalEntrySet.Num());

	TSet<FName> AssetExtensions;
	AssetExtensions.Add(NAME_uasset);
	AssetExtensions.Add(NAME_umap);

	TSet<FName> ExtraAssetExtensions;
	ExtraAssetExtensions.Add(NAME_uexp);
	ExtraAssetExtensions.Add(NAME_ubulk);
	ExtraAssetExtensions.Add(NAME_uptnl);

	TArray<FTopLevelAssetPath> FilterByClasses;
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Material")));
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("MaterialFunction")));
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("MaterialInstance")));
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("BlueprintCore")));
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("ParticleEmitter")));
	FilterByClasses.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("ParticleModule")));


	int32 DepSet = 0; // this is the root set for the dependency (i.e files with a number probably came from the same core dependency)
	for (const FName& FilePath : OriginalEntrySet)
	{
		++DepSet;
		if (!ProcessedFiles.Contains(FilePath))
		{
			const FString FilePathExtension = FPaths::GetExtension(FilePath.ToString());
			const FName FNameExtension = FName(*FilePathExtension);
			if (AssetExtensions.Contains( FNameExtension) )
			{
				FString PackageName;
				if (FPackageName::TryConvertFilenameToLongPackageName(FilePath.ToString(), PackageName))
				{
					FName PackageFName(*PackageName);
					int32 DependencyOrderIndex = 0;
					RecursivelyGrabDependencies(UnsortedEntries, DepSet, DependencyOrderIndex, 0, ProcessedFiles, OriginalEntrySet, FilePath, PackageFName, FilterByClasses);
				}
				else
				{
					//special case for packages outside of our mounted paths, pick up the header and the export without any dependency-gathering.
					ProcessedFiles.Add(FilePath);
					UnsortedEntries.Add(FSortableDependencyEntry(NAME_UnresolvedPackageName, FilePath, FNameExtension, DepSet, 0, 0, false, TSet<FTopLevelAssetPath>()));
				}
			}
			else if ( ExtraAssetExtensions.Contains(FNameExtension) == false )
			{
				//not a package, no need to do special sorting/handling for headers and exports
				UnsortedEntries.Add(FSortableDependencyEntry(FilePath, FNameExtension, DepSet));
				ProcessedFiles.Add(FilePath);
			}
		}
	}

	for (int32 I = UnsortedEntries.Num()-1; I >= 0; --I)
	{
		const FSortableDependencyEntry& DependencyEntry = UnsortedEntries[I];
		if (DependencyEntry.bIsAsset)
		{
			// find all the uexp files and ubulk files and uptnl files
			FString StringPath = DependencyEntry.FilePath.ToString();
			for (const FName& ExtraAssetExtension : ExtraAssetExtensions)
			{
				FString ExtraAssetPath = FPaths::ChangeExtension(StringPath, ExtraAssetExtension.ToString());
				FName ExtraAssetPathFName = FName(*ExtraAssetPath);
				if (OriginalEntrySet.Contains(ExtraAssetPathFName))
				{
					check(!ProcessedFiles.Contains(ExtraAssetPathFName));
					ProcessedFiles.Add(ExtraAssetPathFName);
					TSet<FTopLevelAssetPath> Classes = DependencyEntry.Classes;
					UnsortedEntries.Add(FSortableDependencyEntry(DependencyEntry.LongPackageName, ExtraAssetPathFName, ExtraAssetExtension, DependencyEntry.DepSet, DependencyEntry.DepHierarchy, DependencyEntry.DepOrder, DependencyEntry.bHasDependencies, MoveTemp(Classes)));
				}
			}
		}
	}

	//if this were to fire, first guess would be that there's somehow a rogue export without a header
	check(OriginalEntrySet.Num() == ProcessedFiles.Num() && ProcessedFiles.Num() == UnsortedEntries.Num());


	TArray<FName> ShouldGroupExtensions;
	ShouldGroupExtensions.Add(NAME_ubulk);


	TMap<FName, int32> ExtensionPriority;
	ExtensionPriority.Add(NAME_umap, 0); 
	ExtensionPriority.Add(NAME_uasset, 0);
	ExtensionPriority.Add(NAME_uexp, 1);
	ExtensionPriority.Add(NAME_uptnl, 1);
	ExtensionPriority.Add(NAME_ubulk, 1);
	FSortableDependencySort DependencySortClass(ShouldGroupExtensions, FilterByClasses, ExtensionPriority);
	UnsortedEntries.Sort(DependencySortClass);


		
	UE_LOGF(LogAssetRegUtil, Display, "Writing output: %ls", *ReorderFileOutPath);
	FArchive* OutArc = IFileManager::Get().CreateFileWriter(*ReorderFileOutPath);
	if (OutArc)
	{
		//base from 1, to match existing order list convention
		uint64 NewOrderIndex = 1;
		for (const FSortableDependencyEntry& SortedEntry : UnsortedEntries)
		{
			const FString& FilePath = SortedEntry.FilePath.ToString();
			FString OutputLine = FString::Printf(TEXT("\"%s\" %llu\n"), *FilePath, NewOrderIndex++);
			OutArc->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OutputLine).Get()), OutputLine.Len());
		}
		OutArc->Close();
		delete OutArc;
	}
	else
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Could not open specified output file.");
	}
}

bool UAssetRegUtilCommandlet::MergeOrderFiles(TMap<FString, int64>& NewOrderMap, TMap<FString, int64>& PrevOrderMap)
{
	int64 PrevEntryCount = PrevOrderMap.Num() + 1;

	UE_LOGF(LogAssetRegUtil, Display, "Merge File Open Order intital count: %lld.", PrevEntryCount);

	NewOrderMap.ValueSort([](const uint64& A, const uint64& B) { return A < B; });

	// check in the new file open order all the new resources
	for (TMap<FString, int64>::TConstIterator It(NewOrderMap); It; ++It)
	{
		int64* match = PrevOrderMap.Find(It->Key);

		// Only add resources if we dont find them in the previous FOO
		if (match == nullptr)
		{
			PrevOrderMap.Add(It->Key, PrevEntryCount++);
		}
	}

	UE_LOGF(LogAssetRegUtil, Display, "Merge File Open Order final count: %lld.", PrevEntryCount);

	return true;
}

bool UAssetRegUtilCommandlet::GenerateOrderFile(TMap<FString, int64>& OutputOrderMap, const FString& ReorderFileOutPath)
{
	int NewOrderIndex = 1;
	UE_LOGF(LogAssetRegUtil, Display, "Writing output: %ls", *ReorderFileOutPath);
	FArchive* OutArc = IFileManager::Get().CreateFileWriter(*ReorderFileOutPath);
	if (OutArc)
	{
		OutputOrderMap.ValueSort([](const uint64& A, const uint64& B) { return A < B; });

		for (TMap<FString, int64>::TConstIterator It(OutputOrderMap); It; ++It)
		{
			FString OutputLine;
			OutputLine = FString::Printf(TEXT("\"%s\" %d\n"), *It->Key, NewOrderIndex++);
			OutArc->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OutputLine).Get()), OutputLine.Len());
		}

		OutArc->Close();
		delete OutArc;
		return true;
	}
	else
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Could not open specified output file.");
		return false;
	}
}

bool UAssetRegUtilCommandlet::LoadOrderFile(const FString& OrderFilePath, TMap<FString, int64>& OrderMap)
{
	UE_LOGF(LogAssetRegUtil, Display, "Parsing order package: %ls", *OrderFilePath);
	FString Text;
	if (!FFileHelper::LoadFileToString(Text, *OrderFilePath))
	{
		UE_LOGF(LogAssetRegUtil, Error, "Could not open file %ls", *OrderFilePath);
		return false;
	}

	TArray<FString> Lines;
	Text.ParseIntoArray(Lines, TEXT("\n"), true);

	OrderMap.Reserve(Lines.Num());
	for (int32 Index = 0; Index < Lines.Num(); ++Index)
	{
		int QuoteIndex;
		Lines[Index].ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
		Lines[Index].ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
		if (Lines[Index].FindLastChar('"', QuoteIndex))
		{
			FString ReadNum = Lines[Index].RightChop(QuoteIndex + 1);
			ReadNum.TrimStartInline();
			ReadNum.TrimEndInline();

			int64 OrderNumber = Index;
			if (ReadNum.IsNumeric())
			{
				OrderNumber = FCString::Atoi(*ReadNum);
			}
			else
			{
				return false;
			}

			Lines[Index].LeftInline(QuoteIndex + 1, EAllowShrinking::No);
			FString Name = Lines[Index].TrimQuotes();

			OrderMap.Add(Name, OrderNumber);
		}
	}

	return true;
}

bool UAssetRegUtilCommandlet::LoadOrderFiles(const FString& OrderFilePath, TSet<FName>& OrderFiles)
{
	UE_LOGF(LogAssetRegUtil, Display, "Parsing order file: %ls", *OrderFilePath);
	FString Text;
	if (!FFileHelper::LoadFileToString(Text, *OrderFilePath))
	{
		UE_LOGF(LogAssetRegUtil, Error, "Could not open file %ls", *OrderFilePath);
		return false;
	}

	TArray<FString> Lines;
	Text.ParseIntoArray(Lines, TEXT("\n"), true);

	// Parse each entry out of the order list. adapted from UnrealPak code, might want to unify somewhere.
	OrderFiles.Reserve(Lines.Num());
	for (int32 EntryIndex = 0; EntryIndex < Lines.Num(); ++EntryIndex)
	{
		Lines[EntryIndex].ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
		Lines[EntryIndex].ReplaceInline(TEXT("\n"), TEXT(""), ESearchCase::CaseSensitive);
		//discard the order number, assuming the list is in-order and has no special bits in use at this stage.
		int32 QuoteIndex;
		if (Lines[EntryIndex].FindLastChar('"', QuoteIndex))
		{
			FString ReadNum = Lines[EntryIndex].RightChop(QuoteIndex + 1);
			Lines[EntryIndex].LeftInline(QuoteIndex + 1, EAllowShrinking::No);
			//verify our expectations about the order, just in case something changes on the OpenOrder generation side
			ReadNum.TrimStartInline();
			if (ReadNum.IsNumeric())
			{
				const int32 ExplicitOrder = FCString::Atoi(*ReadNum);
				if (ExplicitOrder != EntryIndex + 1)
				{
					UE_LOGF(LogAssetRegUtil, Warning, "Unexpected order: %i vs %i", ExplicitOrder, EntryIndex + 1);
				}
			}
		}

		Lines[EntryIndex] = Lines[EntryIndex].TrimQuotes();

		const FString EntryPath = *Lines[EntryIndex].ToLower();
		const FName EntryFName = FName(*EntryPath);
		OrderFiles.Add(EntryFName);
	}

	return true;
}

bool UAssetRegUtilCommandlet::GeneratePartiallyUpdatedOrderFile(const FString& OldOrderFilePath, const FString& NewOrderFilePath, const FString& OutOrderFilePath, const float PatchSizePerfBalanceFactor)
{
	TSet<FName> OldOrderFileLineFullSet;
	TSet<FName> NewOrderFileLineSet;
	if (!LoadOrderFiles(OldOrderFilePath, OldOrderFileLineFullSet) ||
		!LoadOrderFiles(NewOrderFilePath, NewOrderFileLineSet))
	{
		return false;
	}

	TSet<FName> OldOrderFileLineSet;

	// Remove deleted files from old order file lines, don't use .Intersect so we keep the original order
	for (const FName& OldOrderFileIter : OldOrderFileLineFullSet)
	{
		if (NewOrderFileLineSet.Contains(OldOrderFileIter))
		{
			OldOrderFileLineSet.Add(OldOrderFileIter);
		}
	}

	// Add new files to old order file lines
	int LastFoundIndexInOldFileLines = -1;
	TArray<FName> OldOrderFileLineArray = OldOrderFileLineSet.Array();
	for (auto NewOrderFileIter = NewOrderFileLineSet.CreateConstIterator(); NewOrderFileIter; ++NewOrderFileIter)
	{
		int IndexInOldFileLines = OldOrderFileLineArray.IndexOfByKey(*NewOrderFileIter);

		if (IndexInOldFileLines != INDEX_NONE)
		{
			LastFoundIndexInOldFileLines = IndexInOldFileLines;
		}
		else
		{
			OldOrderFileLineArray.Insert(*NewOrderFileIter, LastFoundIndexInOldFileLines + 1);
			LastFoundIndexInOldFileLines++;
		}
	}
	OldOrderFileLineSet = TSet<FName>(OldOrderFileLineArray);

	// Write new file order to out file path
	FArchive* OutArc = IFileManager::Get().CreateFileWriter(*OutOrderFilePath);
	if (OutArc)
	{
		// Order number starts from 1
		int CurrentOrderNumber = 1;

		for (int i = 0; i < NewOrderFileLineSet.Num() * PatchSizePerfBalanceFactor; i++)
		{
			const FName OrderFileLine = NewOrderFileLineSet[FSetElementId::FromInteger(i)];
			FString OutputLine = FString::Printf(TEXT("\"%s\" %d\r\n"), *OrderFileLine.ToString(), CurrentOrderNumber);
			OutArc->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OutputLine).Get()), OutputLine.Len());
			CurrentOrderNumber++;

			OldOrderFileLineSet.Remove(OrderFileLine);
		}

		for (auto OldOrderFileIter = OldOrderFileLineSet.CreateIterator(); OldOrderFileIter; ++OldOrderFileIter)
		{
			const FName OrderFileLine = *OldOrderFileIter;
			FString OutputLine = FString::Printf(TEXT("\"%s\" %d\r\n"), *OrderFileLine.ToString(), CurrentOrderNumber);
			OutArc->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OutputLine).Get()), OutputLine.Len());
			CurrentOrderNumber++;
		}

		OutArc->Close();
		delete OutArc;
	}
	else
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Could not open specified output file.")
	}

	return true;
}

int32 UAssetRegUtilCommandlet::Main(const FString& CmdLineParams)
{
	// New deterministic FOO flavor
	bool bMergeFileOpenOrder = FParse::Param(*CmdLineParams, TEXT("MergeFileOpenOrder"));
	UE_LOGF(LogAssetRegUtil, Display, "AssetRegUtil cmdLineParams: %ls",*CmdLineParams);

	FString ReorderFile;
	FString ReorderOutput;

	if (bMergeFileOpenOrder)
	{
		if (!FParse::Value(*CmdLineParams, TEXT("ReorderFile="), ReorderFile, false))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Could not load ReorderFile: %ls", *ReorderFile);
			return 0;
		}

		if (!FParse::Value(*CmdLineParams, TEXT("ReorderOutput="), ReorderOutput, false))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Could not load ReorderOutput: %ls", *ReorderOutput);
			return 0;
		}

		FString PrevReorderFile;
		if (!FParse::Value(*CmdLineParams, TEXT("PrevReorderFile="), PrevReorderFile, false))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Could not load PrevReorderFile: %ls", *PrevReorderFile);
			return 0;
		}

		// Load the new FOO
		TMap<FString, int64> NewOrderMap;
		if (!LoadOrderFile(ReorderFile, NewOrderMap))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Could not load specified order file.");
			return 0;
		}

		// Load the previous FOO
		TMap<FString, int64> PrevOrderMap;
		if (!LoadOrderFile(PrevReorderFile, PrevOrderMap))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Could not load specified order file.");
			return 0;
		}

		MergeOrderFiles(NewOrderMap, PrevOrderMap);
		GenerateOrderFile(PrevOrderMap, ReorderOutput);
	}
	else
	{
		if (FParse::Value(*CmdLineParams, TEXT("ReorderFile="), ReorderFile, false))
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			AssetRegistry = &AssetRegistryModule.Get();

			UE_LOGF(LogAssetRegUtil, Display, "Populating the Asset Registry.");
			AssetRegistry->SearchAllAssets(true);

			if (!FParse::Value(*CmdLineParams, TEXT("ReorderOutput="), ReorderOutput, false))
			{
				//if nothing specified, base it on the input name
				ReorderOutput = FPaths::SetExtension(FPaths::SetExtension(ReorderFile, TEXT("")) + TEXT("Reordered"), FPaths::GetExtension(ReorderFile));
			}

			ReorderOrderFile(ReorderFile, ReorderOutput);

			// Generate partially-updated order file
			float PatchSizePerfBalanceFactor;	// Set the value close to 0.0 to favor patch size and close to 1.0 to favor streaming performance
			FParse::Value(*CmdLineParams, TEXT("PatchSizePerfBalanceFactor="), PatchSizePerfBalanceFactor);
			PatchSizePerfBalanceFactor = FMath::Clamp(PatchSizePerfBalanceFactor, 0.f, 1.f);

			FString OldOrderFilePath;
			if (FParse::Value(*CmdLineParams, TEXT("OldFileOpenOrderFile="), OldOrderFilePath, false))
			{
				UE_LOGF(LogAssetRegUtil, Display, "Generating partially-updated order file.");

				FString OutPartialOrderFilePath;
				if (!FParse::Value(*CmdLineParams, TEXT("PartialFileOpenOrderOutput="), OutPartialOrderFilePath, false))
				{
					//if nothing specified, base it on the input name
					OutPartialOrderFilePath = FPaths::SetExtension(FPaths::SetExtension(ReorderFile, TEXT("")) + TEXT("PartialUpdate"), FPaths::GetExtension(ReorderFile));
				}

				GeneratePartiallyUpdatedOrderFile(OldOrderFilePath, ReorderOutput, OutPartialOrderFilePath, PatchSizePerfBalanceFactor);
			}
		}
	}

	return 0;
}

UAssetRegistryDumpCommandlet::UAssetRegistryDumpCommandlet(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

int32 UAssetRegistryDumpCommandlet::Main(const FString& CmdLineParams)
{
	const FString* InputFileNamePtr;
	const FString* OutputDirectoryPtr;
	TArray<FString> Tokens, Switches;
	TMap<FString, FString> Params;
	ParseCommandLine(*CmdLineParams, Tokens, Switches, Params);
	InputFileNamePtr = Params.Find(TEXT("Input"));
	OutputDirectoryPtr = Params.Find(TEXT("OutDir"));
	if (!InputFileNamePtr || !OutputDirectoryPtr)
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Usage: -input <AssetRegistry.bin Filepath> -outdir <Directory to contain the dumped file(s)");
		return 1;
	}
	const FString& InputFileName = *InputFileNamePtr;
	const FString& OutputDirectory = *OutputDirectoryPtr;

	IFileManager& FileManager = IFileManager::Get();
	if (!FileManager.FileExists(*InputFileName))
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Could not open input file %ls", *InputFileName);
		return 3;
	}

	FArrayReader Bytes;
	if (!FFileHelper::LoadFileToArray(Bytes, *InputFileName))
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Failed to load file %ls", *InputFileName);
		return 3;
	}

	FAssetRegistryState State;
	if (!State.Load(Bytes))
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Failed to serialize file %ls as an AssetRegistry", *InputFileName);
		return 3;
	}

	FString DumpDir = FPaths::ConvertRelativePathToFull(OutputDirectory / TEXT("AssetRegistry"));
	if (FileManager.DirectoryExists(*DumpDir))
	{
		FString DeleteDirectory = OutputDirectory / FGuid::NewGuid().ToString();
		if (!FileManager.Move(*DeleteDirectory, *DumpDir))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Failed to move old directory %ls to delete staging directory %ls", *DumpDir, *DeleteDirectory);
			return 4;
		}
		if (!FileManager.DeleteDirectory(*DeleteDirectory, false /* RequireExists */, true /* Tree */))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Failed to delete temporary delete-staging directory %ls", *DeleteDirectory);
		}
	}
	if (!FileManager.MakeDirectory(*DumpDir, true /* Tree */))
	{
		UE_LOGF(LogAssetRegUtil, Warning, "Failed to create directory %ls", *DumpDir);
		return 4;
	}

	TArray<FString> Pages;
	TArray<FString> Arguments({ TEXT("ObjectPath"),TEXT("PackageName"),TEXT("Path"),TEXT("Class"),TEXT("Tag"), TEXT("DependencyDetails"), TEXT("PackageData") });
#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	State.Dump(Arguments, Pages, 10000 /* LinesPerPage */);
#endif
	int PageIndex = 0;
	TStringBuilder<256> FileName;
	int32 Result = 0;
	for (FString& PageText : Pages)
	{
		FileName.Reset();
		FileName.Appendf(TEXT("%s_%05d.txt"), *(DumpDir / TEXT("Page")), PageIndex++);
		PageText.ToLowerInline();
		if (!FFileHelper::SaveStringToFile(PageText, FileName.ToString()))
		{
			UE_LOGF(LogAssetRegUtil, Warning, "Failed to save file %ls", *FileName);
			Result = 4;
		}
	}

	return Result;
}
