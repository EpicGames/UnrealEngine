// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/AssetRegistryGenerator.h"

#include "Algo/Sort.h"
#include "Algo/StableSort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookSandbox.h"
#include "Cooker/CookWorkerClient.h"
#include "Cooker/MPCollector.h"
#include "CookMetadata.h"
#include "Commandlets/IChunkDataGenerator.h"
#include "Containers/RingBuffer.h"
#include "Engine/AssetManager.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameDelegates.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/xxhash.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/App.h"
#include "Misc/AsciiSet.h"
#include "Misc/ChunkDependencyInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "PakFileUtilities.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Stats/StatsMisc.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "Templates/Greater.h"
#include "Templates/UniquePtr.h"
#include "UObject/SoftObjectPath.h"
#include "Logging/StructuredLog.h"

#if WITH_EDITOR
#include "HAL/ThreadHeartBeat.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAssetRegistryGenerator, Log, All);

#define LOCTEXT_NAMESPACE "AssetRegistryGenerator"

LLM_DEFINE_TAG(Cooker_GeneratedAssetRegistry);

//////////////////////////////////////////////////////////////////////////
// Static functions
FName GetPackageNameFromDependencyPackageName(const FName RawPackageFName)
{
	FName PackageFName = RawPackageFName;
	if ((FPackageName::IsValidLongPackageName(RawPackageFName.ToString()) == false) &&
		(FPackageName::IsScriptPackage(RawPackageFName.ToString()) == false))
	{
		FText OutReason;
		if (!FPackageName::IsValidLongPackageName(RawPackageFName.ToString(), true, &OutReason))
		{
			const FText FailMessage = FText::Format(LOCTEXT("UnableToGeneratePackageName", "Unable to generate long package name for {0}. {1}"),
				FText::FromString(RawPackageFName.ToString()), OutReason);

			UE_LOGF(LogAssetRegistryGenerator, Warning, "%ls", *(FailMessage.ToString()));
			return NAME_None;
		}


		FString LongPackageName;
		if (FPackageName::SearchForPackageOnDisk(RawPackageFName.ToString(), &LongPackageName) == false)
		{
			return NAME_None;
		}
		PackageFName = FName(*LongPackageName);
	}

	// don't include script packages in dependencies as they are always in memory
	if (FPackageName::IsScriptPackage(PackageFName.ToString()))
	{
		// no one likes script packages
		return NAME_None;
	}
	return PackageFName;
}

/**
 * Checks if the provided file path is in the format supported by the BulkData CookedIndex system.
 * We expect two extensions in the path, the first will be all numbers and the second will be one
 * of the bulkdata types supported by the system, 'i.e <PackageName>.001.ubulk'.
 * @see FBulkDataCookedIndex for more info.
 */
static bool HasBulkDataCookedIndexExtension(FStringView Path)
{
	// Check that the extension at the end of the file is one of the bulkdata types that supports this feature
	if (!(Path.EndsWith(TEXT(".ubulk")) || Path.EndsWith(TEXT(".uptnl"))) || Path.Len() < 10)
	{
		return false;
	}

	// If the number of max digits changes then our assumptions about ExtensionSize should be reconsidered
	static_assert(FBulkDataCookedIndex::MAX_DIGITS == 3);

	// A valid extension of this type will always be 10 characters long
	const int32 ExtensionSize = 10;

	if (Path.Len() < ExtensionSize)
	{
		return false;
	}

	const int32 ExtensionStart = Path.Len() - ExtensionSize;

	if (Path[ExtensionStart] != TEXT('.'))
	{
		return false;
	}

	// Make sure that the first extension only has numeric characters, 0-9
	for (int32 Index = 1; Index < 4; Index++)
	{
		if (Path[ExtensionStart + Index] < TEXT('0') || Path[ExtensionStart + Index] > TEXT('9'))
		{
			return false;
		}
	}

	return true;
}

class FDefaultPakFileRules
{
public:
	void InitializeFromConfig(const ITargetPlatform* TargetPlatform)
	{
		if (bInitialized)
		{
			return;
		}

		FConfigFile ConfigFile;
		if (!FConfigCacheIni::LoadLocalIniFile(ConfigFile, TEXT("PakFileRules"), true /* bIsBaseIniName */))
		{
			return;
		}
		// Schema is defined in Engine\Config\BasePakFileRules.ini, see also GetPakFileRules in CopyBuildToStaging.Automation.cs

		FString IniPlatformName = TargetPlatform->IniPlatformName();
		for (const TPair<FString, FConfigSection>& Pair : AsConst(ConfigFile))
		{
			const FString& SectionName = Pair.Key;
			bool bMatchesAllPlatforms = true;
			bool bMatchesPlatform = false;
			FString ApplyToPlatformsValue;
			if (ConfigFile.GetString(*SectionName, TEXT("Platforms"), ApplyToPlatformsValue))
			{
				UE::String::ParseTokens(ApplyToPlatformsValue, ',',
					[&bMatchesAllPlatforms, &bMatchesPlatform, &IniPlatformName](FStringView Token)
					{
						bMatchesAllPlatforms = false;
						if (IniPlatformName == Token)
						{
							bMatchesPlatform = true;
						}
					}, UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);
			}
			if (!bMatchesPlatform && !bMatchesAllPlatforms)
			{
				continue;
			}

			FString OverridePaksValue;
			if (ConfigFile.GetString(*SectionName, TEXT("OverridePaks"), OverridePaksValue))
			{
				UE::String::ParseTokens(OverridePaksValue, ',', [this](FStringView Token)
					{
						ReferencedPaks.Add(FString(Token));
					}, UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);
			}
		}
		bInitialized = true;
	}

	bool IsChunkReferenced(int32 PakchunkIndex)
	{
		TStringBuilder<64> ChunkFileName;
		ChunkFileName.Append(FGenericPlatformMisc::GetPakFilenamePrefix());
		ChunkFileName.Appendf(TEXT("%d"), PakchunkIndex);
		FStringView ChunkFileNameView(ChunkFileName);
		return ReferencedPaks.ContainsByHash(GetTypeHash(ChunkFileNameView), ChunkFileNameView);
	}

private:
	bool bInitialized = false;
	TSet<FString> ReferencedPaks;

};

//////////////////////////////////////////////////////////////////////////
// FAssetRegistryGenerator

void FAssetRegistryGenerator::UpdateAssetManagerDatabase()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRegistryGenerator::UpdateAssetManagerDatabase);
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	UAssetManager::Get().UpdateManagementDatabase();
}

FAssetRegistryGenerator::FAssetRegistryGenerator(const ITargetPlatform* InPlatform)
	: AssetRegistry(FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get())
	, TargetPlatform(InPlatform)
	, bGenerateChunks(false)
	, HighestChunkId(0)
	, DependencyInfo(*GetMutableDefault<UChunkDependencyInfo>())
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	bool bOnlyHardReferences = false;
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	if (PackagingSettings)
	{
		bOnlyHardReferences = PackagingSettings->bChunkHardReferencesOnly;
	}	

	DependencyQuery = bOnlyHardReferences ? UE::AssetRegistry::EDependencyQuery::Hard : UE::AssetRegistry::EDependencyQuery::NoRequirements;

	InitializeChunkIdPakchunkIndexMapping();
}

FAssetRegistryGenerator::~FAssetRegistryGenerator()
{
}

bool FAssetRegistryGenerator::ShouldPlatformGenerateStreamingInstallManifest(const ITargetPlatform* Platform) const
{
	if (Platform)
	{
		FConfigFile PlatformIniFile;
		FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *Platform->IniPlatformName());
		FString ConfigString;
		if (PlatformIniFile.GetString(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bGenerateChunks"), ConfigString))
		{
			return FCString::ToBool(*ConfigString);
		}
	}

	return false;
}

static TMap<int64, int64> GetMaxChunkSizeOverrideFromChunkIndex(const ITargetPlatform* Platform)
{
	TMap<int64, int64> MaxChunkSizeOverrideFromChunkIndex;
	if (Platform)
	{
		FConfigFile PlatformIniFile;
		FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *Platform->IniPlatformName());
		TArray<FString> ConfigStringArray;
		if (PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("MaxChunkSizeOverrideFromChunkIndex"), ConfigStringArray))
		{
			for (const FString& ConfigString : ConfigStringArray)
			{
				FStringView ConfigStringView(ConfigString);
				ConfigStringView = ConfigStringView.TrimStartAndEnd();
				if (ConfigStringView.IsEmpty() || ConfigStringView[0] != TEXT('(') || ConfigStringView[ConfigStringView.Len() - 1] != TEXT(')'))
				{
					continue;
				}
				ConfigStringView.RemovePrefix(1);
				ConfigStringView.RemoveSuffix(1);

				TArray<FStringView> ConfigStringArrayNameSize;
				UE::String::ParseTokens(ConfigStringView, TEXTVIEW(","), [&](FStringView Element)
					{
						ConfigStringArrayNameSize.Add(Element);
					});

				if (ConfigStringArrayNameSize.Num() != 2 || ConfigStringArrayNameSize[0].Len() == 0 || ConfigStringArrayNameSize[1].Len() == 0)
				{
					continue;
				}

				const TCHAR* ConfigNameStringViewEnd = ConfigStringArrayNameSize[0].GetData();
				int64 PakIndex = FCString::Strtoi64(ConfigStringArrayNameSize[0].GetData(), (TCHAR**)&ConfigNameStringViewEnd, 10);
				if (ConfigNameStringViewEnd == ConfigStringArrayNameSize[0].GetData())
				{
					continue;
				}
				
				const TCHAR* ConfigSizeStringViewEnd = ConfigStringArrayNameSize[1].GetData();
				int64 ConfigSize = FCString::Strtoi64(ConfigStringArrayNameSize[1].GetData(), (TCHAR**)&ConfigSizeStringViewEnd, 10);

				if (ConfigSizeStringViewEnd == ConfigStringArrayNameSize[1].GetData())
				{
					continue;
				}

				MaxChunkSizeOverrideFromChunkIndex.Add(PakIndex, ConfigSize);
			}
		}
	}

	return MaxChunkSizeOverrideFromChunkIndex;

}


int64 FAssetRegistryGenerator::GetMaxChunkSizePerPlatform(const ITargetPlatform* Platform) const
{
	if (Platform)
	{
		FConfigFile PlatformIniFile;
		FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *Platform->IniPlatformName());
		FString ConfigString;
		if (PlatformIniFile.GetString(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("MaxChunkSize"), ConfigString))
		{
			return FCString::Atoi64(*ConfigString);
		}
	}

	return -1;
}

TArray<int32> FAssetRegistryGenerator::GetExistingPackageChunkAssignments(FName PackageFName, const TSet<FName>& StartupPackages)
{
	TArray<int32> ExistingChunkIDs;
	int32 PackageFNameHash = GetTypeHash(PackageFName);
	for (uint32 ChunkIndex = 0, MaxChunk = ChunkManifests.Num(); ChunkIndex < MaxChunk; ++ChunkIndex)
	{
		if (ChunkManifests[ChunkIndex] && ChunkManifests[ChunkIndex]->ContainsByHash(PackageFNameHash, PackageFName))
		{
			ExistingChunkIDs.AddUnique(ChunkIndex);
		}
	}

	if (StartupPackages.Contains(PackageFName))
	{
		ExistingChunkIDs.AddUnique(0);
	}

	return ExistingChunkIDs;
}

TArray<int32> FAssetRegistryGenerator::GetExplicitChunkIDs(const FName& PackageFName)
{
	TArray<int32> PackageInputChunkIds;
	const TArray<int32>* FoundIDs = ExplicitChunkIDs.Find(PackageFName);
	if (FoundIDs)
	{
		PackageInputChunkIds = *FoundIDs;
	}
	return PackageInputChunkIds;
}

static void ParseChunkLayerAssignment(TArray<FString> ChunkLayerAssignmentArray, TMap<int32, int32>& OutChunkLayerAssignment)
{
	OutChunkLayerAssignment.Empty();

	const TCHAR* PropertyChunkId = TEXT("ChunkId=");
	const TCHAR* PropertyLayerId = TEXT("Layer=");
	for (FString& Entry : ChunkLayerAssignmentArray)
	{
		// Remove parentheses
		Entry.TrimStartAndEndInline();
		Entry.ReplaceInline(TEXT("("), TEXT(""));
		Entry.ReplaceInline(TEXT(")"), TEXT(""));

		int32 ChunkId = -1;
		int32 LayerId = -1;
		FParse::Value(*Entry, PropertyChunkId, ChunkId);
		FParse::Value(*Entry, PropertyLayerId, LayerId);

		if (ChunkId >= 0 && LayerId >= 0 && !OutChunkLayerAssignment.Contains(ChunkId))
		{
			OutChunkLayerAssignment.Add(ChunkId, LayerId);
		}
	}
}

static void AssignLayerChunkDelegate(const FAssignLayerChunkMap* ChunkManifest, const FString& Platform, const int32 ChunkIndex, int32& OutChunkLayer)
{
	OutChunkLayer = 0;

	static TMap<FString, TUniquePtr<TMap<int32, int32>>> PlatformChunkLayerAssignments;
	TUniquePtr<TMap<int32, int32>>& ChunkLayerAssignment = PlatformChunkLayerAssignments.FindOrAdd(Platform);
	if (!ChunkLayerAssignment)
	{
		FConfigFile PlatformIniFile;
		FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *Platform);
		TArray<FString> ChunkLayerAssignmentArray;
		PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("ChunkLayerAssignment"), ChunkLayerAssignmentArray);

		ChunkLayerAssignment.Reset(new TMap<int32, int32>());
		ParseChunkLayerAssignment(ChunkLayerAssignmentArray, *ChunkLayerAssignment);
	}

	int32* LayerId = ChunkLayerAssignment->Find(ChunkIndex);
	if (LayerId)
	{
		OutChunkLayer = *LayerId;
	}
}

bool FAssetRegistryGenerator::GenerateStreamingInstallManifest(int64 InOverrideChunkSize, const TCHAR* InManifestSubDir,
	UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS)
{
	const FString Platform = TargetPlatform->PlatformName();
	FString ChunkManifestDir = GetChunkManifestDirectory(COTFS);
	if (InManifestSubDir)
	{
		ChunkManifestDir /= InManifestSubDir;
	}
	int64 MaxChunkSize = InOverrideChunkSize > 0 ? InOverrideChunkSize : GetMaxChunkSizePerPlatform(TargetPlatform);
	TMap<int64, int64> MaxChunkSizeOverrideFromIndex = GetMaxChunkSizeOverrideFromChunkIndex(TargetPlatform);

	if (!IFileManager::Get().MakeDirectory(*ChunkManifestDir, true /* Tree */))
	{
		UE_LOGF(LogAssetRegistryGenerator, Error, "Failed to create directory: %ls", *ChunkManifestDir);
		return false;
	}
	
	FString PakChunkListFilename = ChunkManifestDir / TEXT("pakchunklist.txt");
	FString PakChunkLayerInfoFilename = ChunkManifestDir / TEXT("pakchunklayers.txt");
	// List of pak file lists
	TUniquePtr<FArchive> PakChunkListFile(IFileManager::Get().CreateFileWriter(*PakChunkListFilename));
	// List of disc layer for each chunk
	TUniquePtr<FArchive> ChunkLayerFile(IFileManager::Get().CreateFileWriter(*PakChunkLayerInfoFilename));
	if (!PakChunkListFile || !ChunkLayerFile)
	{
		UE_LOGF(LogAssetRegistryGenerator, Error, "Failed to open output pakchunklist file %ls",
			!PakChunkListFile ? *PakChunkListFilename : *PakChunkLayerInfoFilename);
		return false;
	}

	FConfigFile PlatformIniFile;
	FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *TargetPlatform->IniPlatformName());

	// Update manifests for any encryption groups that contain non-asset files
	if (!TargetPlatform->HasSecurePackageFormat())
	{
		FContentEncryptionConfig ContentEncryptionConfig;
		UAssetManager::Get().GetContentEncryptionConfig(ContentEncryptionConfig);
		const FContentEncryptionConfig::TGroupMap& EncryptionGroups = ContentEncryptionConfig.GetPackageGroupMap();
		
		for (const FContentEncryptionConfig::TGroupMap::ElementType& GroupElement : EncryptionGroups)
		{
			const FName GroupName = GroupElement.Key;
			const FContentEncryptionConfig::FGroup& EncryptionGroup = GroupElement.Value;

			if (EncryptionGroup.NonAssetFiles.Num() > 0)
			{
				UE_LOGF(LogAssetRegistryGenerator, Log, "Updating non-asset files in manifest for group '%ls'", *GroupName.ToString());
				
				int32 ChunkID = UAssetManager::Get().GetContentEncryptionGroupChunkID(GroupName);
				int32 PakchunkIndex = GetPakchunkIndex(ChunkID);
				if (PakchunkIndex >= FinalChunkManifests.Num())
				{
					// Extend the array until it is large enough to hold the requested index, filling it in with nulls on all the newly added indices.
					// Note that this will temporarily break our contract that FinalChunkManifests does not contain null pointers; we fix up the contract
					// by replacing any remaining null pointers in the loop over FinalChunkManifests at the bottom of this function.
					FinalChunkManifests.AddDefaulted(PakchunkIndex - FinalChunkManifests.Num() + 1);
				}

				FChunkPackageSet* Manifest = FinalChunkManifests[PakchunkIndex].Get();
				if (Manifest == nullptr)
				{
					Manifest = new FChunkPackageSet();
					FinalChunkManifests[PakchunkIndex].Reset(Manifest);
				}

				for (const FString& NonAssetFile : EncryptionGroup.NonAssetFiles)
				{
					// Paths added as relative to the root. The staging code will need to map this onto the target path of all staged assets
					Manifest->Add(*NonAssetFile, FPaths::RootDir() / NonAssetFile);
				}
			}
		}
	}
	
	bool bEnableGameOpenOrderSort = false;
	bool bUseSecondaryOpenOrder = false;
	TArray<FString> OrderFileSpecStrings;
	{
		PlatformIniFile.GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bEnableAssetRegistryGameOpenOrderSort"), bEnableGameOpenOrderSort);
		PlatformIniFile.GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bPakUsesSecondaryOrder"), bUseSecondaryOpenOrder);
		PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("PakOrderFileSpecs"), OrderFileSpecStrings);
	}

	// if a game open order can be found then use that to sort the filenames
	FPakOrderMap OrderMap;
	bool bHaveGameOpenOrder = false;
	if (bEnableGameOpenOrderSort)
	{
		TArray<FPakOrderFileSpec> OrderFileSpecs;
		const FPakOrderFileSpec* SecondaryOrderSpec = nullptr;

		if (OrderFileSpecStrings.Num() == 0)
		{
			OrderFileSpecs.Add(FPakOrderFileSpec(TEXT("GameOpenOrder*.log")));
			if (bUseSecondaryOpenOrder)
			{
				OrderFileSpecs.Add(FPakOrderFileSpec(TEXT("CookerOpenOrder*.log")));
				SecondaryOrderSpec = &OrderFileSpecs.Last();
			}
		}
		else
		{
			UScriptStruct* Struct = FPakOrderFileSpec::StaticStruct();
			for (const FString& String : OrderFileSpecStrings)
			{
				FPakOrderFileSpec Spec;
				Struct->ImportText(*String, &Spec, nullptr, PPF_Delimited, nullptr, Struct->GetName());
				OrderFileSpecs.Add(Spec);
			}

		}

		TArray<FString> DirsToSearch;
		DirsToSearch.Add(FPaths::Combine(FPaths::ProjectDir(), TEXT("Build"), TEXT("FileOpenOrder")));
		FString PlatformsDir = FPaths::Combine(FPaths::ProjectDir(), TEXT("Platforms"), Platform, TEXT("Build"), TEXT("FileOpenOrder"));
		if (IFileManager::Get().DirectoryExists(*PlatformsDir))
		{
			DirsToSearch.Add(PlatformsDir);
		}
		else
		{
			DirsToSearch.Add(FPaths::Combine(FPaths::ProjectDir(), TEXT("Build"), Platform, TEXT("FileOpenOrder")));
		}

		TArray<FPakOrderMap> OrderMaps; // Indexes match with OrderFileSpecs
		OrderMaps.Reserve(OrderFileSpecs.Num());
		uint64 StartIndex = 0;
		for (const FPakOrderFileSpec& Spec : OrderFileSpecs)
		{
			// For each order map reserve it a contiguous integer range based on what indices were used by previous maps
			// After building each map we'll then merge in priority order
			UE_LOGF(LogAssetRegistryGenerator, Log, "Order file spec %ls starting at index %llu", *Spec.Pattern, StartIndex);

			FPakOrderMap& LocalOrderMap = OrderMaps.AddDefaulted_GetRef();

			TArray<FString> FoundFiles;
			for (const FString& Directory : DirsToSearch)
			{
				TArray<FString> LocalFoundFiles;
				IFileManager::Get().FindFiles(LocalFoundFiles, *FPaths::Combine(Directory, Spec.Pattern), true, false);

				for (const FString& Filename : LocalFoundFiles)
				{
					FoundFiles.Add(FPaths::Combine(Directory, Filename));
				}
			}
			
			FoundFiles.Sort();
			if (FoundFiles.Num() > 0)
			{
				bHaveGameOpenOrder = bHaveGameOpenOrder || (&Spec != SecondaryOrderSpec);
			}

			for (int32 i=0; i < FoundFiles.Num(); ++i)
			{
				const FString& Found = FoundFiles[i];
				UE_LOGF(LogAssetRegistryGenerator, Display, "Found order file %ls", *Found);
				if (LocalOrderMap.Num() == 0) 
				{
					LocalOrderMap.ProcessOrderFile(*Found, &Spec == SecondaryOrderSpec, false, StartIndex);
				}
				else
				{
					LocalOrderMap.ProcessOrderFile(*Found, &Spec == SecondaryOrderSpec, true);
				}
			}

			if (LocalOrderMap.Num() > 0)
			{
				check(LocalOrderMap.GetMaxIndex() >= StartIndex);
				StartIndex = LocalOrderMap.GetMaxIndex() + 1;
			}
		}

		TArray<int32> SpecIndicesByPriority;
		for (int32 i=0; i < OrderFileSpecs.Num(); ++i)
		{
			SpecIndicesByPriority.Add(i);
		}
		Algo::StableSortBy(SpecIndicesByPriority, [&OrderFileSpecs](int32 i) { return  OrderFileSpecs[i].Priority; }, TGreater<int32>());

		UE_LOGF(LogAssetRegistryGenerator, Log, "Merging order maps in priority order");
		for (int32 SpecIndex : SpecIndicesByPriority)
		{
			const FPakOrderFileSpec& Spec = OrderFileSpecs[SpecIndex];
			UE_LOGF(LogAssetRegistryGenerator, Log, "Merging order file spec %d (%ls) at priority (%d) ", SpecIndex, *Spec.Pattern, Spec.Priority);
			FPakOrderMap& LocalOrderMap = OrderMaps[SpecIndex];
			OrderMap.MergeOrderMap(MoveTemp(LocalOrderMap));
		}
	}

	TArray<FString> CompressedChunkWildcards;
	if (!TargetPlatform->IsServerOnly())
	{
		// Load the list of wildcards to specify which pakfiles should be compressed, if the targetplatform supports it.
		// This is only used in client platforms.
		PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("CompressedChunkWildcard"), CompressedChunkWildcards);
	}

	// Load the list of wildcards to specify which pakfiles have per-chunk compression. These are allowed to 
	// use individual compression settings even if the platform package doesn't want compression.
	// NOTE: If DDPI specifies a hardware compression setting of 'None', this won't work as expected because there will be no global compression settings to opt in to. In this case, 
	// set bForceUseProjectCompressionFormatIgnoreHardwareOverride=true and bCompressed=False in[/Script/UnrealEd.ProjectPackagingSettings], 
	// then setup whatever project compression settings you would like chunks to be able to opt in to.
	TArray<FString> AllowPerChunkCompressionWildcards;
	PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("AllowPerChunkCompressionWildcard"), AllowPerChunkCompressionWildcards);

	const TMap<int32, FString> PakChunkIdToStringOverride = GetPakChunkIdToStringOverrideMap();

	static bool bIsDLCCook = []()
		{
			FString DLCNameReturn;
			FParse::Value(FCommandLine::Get(), TEXT("DLCNAME="), DLCNameReturn);
			return !DLCNameReturn.IsEmpty();
		}();

	FDefaultPakFileRules DefaultPakFileRules;
	TArray<int32> ReferencedChunkIds;
	TArray<bool> FinalChunkIsReferenced;
	FinalChunkIsReferenced.SetNum(FinalChunkManifests.Num());
	ReferencedChunkIds.Reserve(FinalChunkManifests.Num());
	// Get the list of ChunkIds to report to ChunkDataGenerators' BeginGenerateChunkDataFiles.
	for (int32 PakchunkIndex = 0; PakchunkIndex < FinalChunkManifests.Num(); ++PakchunkIndex)
	{
		const FChunkPackageSet* Manifest = FinalChunkManifests[PakchunkIndex].Get();

		// Is this index a null placeholder that we added in the loop over EncryptedNonUFSFileGroups and then never filled in?  If so, 
		// fill it in with an empty FChunkPackageSet
		if (!Manifest)
		{
			FinalChunkManifests[PakchunkIndex].Reset(new FChunkPackageSet());
			Manifest = FinalChunkManifests[PakchunkIndex].Get();
		}

		FinalChunkIsReferenced[PakchunkIndex] = false;

		// Do not create any files, and do not tell ChunkDataGenerators that the chunk is referenced, if the chunk is
		// empty and is not referenced by rules applied during staging
		if (Manifest->IsEmpty())
		{
			// During a chunk'd DLC cook ignore DefaultPakFile Rules. Those are handled during main cooks.
			if (bIsDLCCook)
			{
				continue;
			}

			DefaultPakFileRules.InitializeFromConfig(TargetPlatform);
			if (!DefaultPakFileRules.IsChunkReferenced(PakchunkIndex))
			{
				continue;
			}
		}

		FinalChunkIsReferenced[PakchunkIndex] = true;
		ReferencedChunkIds.Add(PakchunkIndex);
	}
	// TOOD: Need to make a public interface for FCookSandbox and pass it into GenerateChunkDataFiles instead of the
	// internal FSandboxPlatformFile, in case any of the generators need to correctly map files in remapped plugins
	FSandboxPlatformFile& SandboxPlatformFile = InSandboxFile.GetSandboxPlatformFile();
	if (ChunkDataGenerators.Num() > 0)
	{
		IChunkDataGenerator::FBeginGenerateContext BeginContext;
		BeginContext.CurrentChunkIds = ReferencedChunkIds;
		BeginContext.TargetPlatform = TargetPlatform;
		BeginContext.SandboxFile = &SandboxPlatformFile;
		for (const TSharedRef<IChunkDataGenerator>& ChunkDataGenerator : ChunkDataGenerators)
		{
			ChunkDataGenerator->BeginGenerateChunkDataFiles(BeginContext);
		}
	}

	bool bSucceeded = true;
	TMap<FString, int64> PackageFileSizes;
	// generate per-chunk pak list files
	for (int32 PakchunkIndex = 0; PakchunkIndex < FinalChunkManifests.Num() && bSucceeded; ++PakchunkIndex)
	{
		const FChunkPackageSet* Manifest = FinalChunkManifests[PakchunkIndex].Get();
		check(Manifest); // This was created in the loop above if not already existing
		// Serialize chunk layers whether chunk is empty or not
		int32 TargetLayer = 0;
		FGameDelegates::Get().GetAssignLayerChunkDelegate().ExecuteIfBound(Manifest, Platform, PakchunkIndex, TargetLayer);

		FString LayerString = FString::Printf(TEXT("%d\r\n"), TargetLayer);
		ChunkLayerFile->Serialize(TCHAR_TO_ANSI(*LayerString), LayerString.Len());

		if (!FinalChunkIsReferenced[PakchunkIndex])
		{
			continue;
		}

		int64* MaxChunkSizeForPakIndex = MaxChunkSizeOverrideFromIndex.Find(PakchunkIndex);
		int64 PakChunkMaxChunkSize = MaxChunkSizeForPakIndex ? *MaxChunkSizeForPakIndex : MaxChunkSize;

		if (PakChunkMaxChunkSize > 0)
		{
			PackageFileSizes.Reset();
			PackageFileSizes.Reserve(Manifest->Num());
			const TMap<FName, const FAssetPackageData*>& PackageDataMap = State.GetAssetPackageDataMap();
			for (const TPair<FName, FString>& ManifestEntry : *Manifest)
			{
				const FAssetPackageData* const* ExistingPackageData = PackageDataMap.Find(ManifestEntry.Key);
				if (ExistingPackageData)
				{
					PackageFileSizes.Emplace(ManifestEntry.Value.Replace(TEXT("[Platform]"), *Platform), (*ExistingPackageData)->DiskSize);
				}
			}
		}

		// Split the chunk into subchunks as necessary and create and register a PakListFile for each subchunk
		int32 FilenameIndex = 0;
		TArray<FString> ChunkFilenames;
		Manifest->GenerateValueArray(ChunkFilenames);

		const FString PakChunkName = GetPakChunkNameForId(PakChunkIdToStringOverride, PakchunkIndex);
		bool bFinishedAllFiles = false;
		for (int32 SubChunkIndex = 0; !bFinishedAllFiles; ++SubChunkIndex)
		{
			const FString PakChunkFilename = (SubChunkIndex > 0)
				? FString(FGenericPlatformMisc::GetPakFilenamePrefix()) + FString::Printf(TEXT("%s_s%d.txt"), *PakChunkName, SubChunkIndex)
				: FString(FGenericPlatformMisc::GetPakFilenamePrefix()) + FString::Printf(TEXT("%s.txt"), *PakChunkName);

			const FString PakListFilename = FString::Printf(TEXT("%s/%s"), *ChunkManifestDir, *PakChunkFilename);
			TUniquePtr<FArchive> PakListFile(IFileManager::Get().CreateFileWriter(*PakListFilename));

			if (!PakListFile)
			{
				UE_LOGF(LogAssetRegistryGenerator, Error, "Failed to open output paklist file %ls", *PakListFilename);
				bSucceeded = false;
				break;
			}

			FString PakChunkOptions;
			for (const FString& CompressedChunkWildcard : CompressedChunkWildcards)
			{
				if (PakChunkFilename.MatchesWildcard(CompressedChunkWildcard))
				{
					PakChunkOptions += " compressed";
					break;
				}
			}

			for (const FString& AllowPerChunkCompressionWildcard : AllowPerChunkCompressionWildcards)
			{
				if (PakChunkFilename.MatchesWildcard(AllowPerChunkCompressionWildcard))
				{
					PakChunkOptions += " AllowPerChunkCompression";
					break;
				}
			}

			// For encryption chunks, PakchunkIndex equals ChunkID
			FGuid Guid = UAssetManager::Get().GetChunkEncryptionKeyGuid(PakchunkIndex);
			if (Guid.IsValid())
			{
				PakChunkOptions += TEXT(" encryptionkeyguid=") + Guid.ToString();
			}

			// If this chunk has a separate unique asset registry, add it to first subchunk's manifest here
			if (SubChunkIndex == 0)
			{
				// For chunks with unique asset registry name, pakchunkIndex should equal chunkid
				FName RegistryName = UAssetManager::Get().GetUniqueAssetRegistryName(PakchunkIndex);
				if (RegistryName != NAME_None)
				{
					FString AssetRegistryFilename = FString::Printf(TEXT("%s%sAssetRegistry%s.bin"),
						*InSandboxFile.GetSandboxDirectory(), *InSandboxFile.GetGameSandboxDirectoryName(),
						*RegistryName.ToString());
					ChunkFilenames.Add(AssetRegistryFilename);
				}
			}

			// Allow the extra data generation steps to run and add their output to the manifest
			if (ChunkDataGenerators.Num() > 0 && SubChunkIndex == 0)
			{
				TSet<FName> PackagesInChunk;
				PackagesInChunk.Reserve(Manifest->Num());
				for (const auto& ChunkManifestPair : *Manifest)
				{
					PackagesInChunk.Add(ChunkManifestPair.Key);
				}

				for (const TSharedRef<IChunkDataGenerator>& ChunkDataGenerator : ChunkDataGenerators)
				{
					ChunkDataGenerator->GenerateChunkDataFiles(PakchunkIndex, PackagesInChunk, TargetPlatform, &SandboxPlatformFile, ChunkFilenames);
				}
			}

			if (SubChunkIndex == 0)
			{
				if (bHaveGameOpenOrder)
				{		
					FString CookedDirectory = FPaths::ConvertRelativePathToFull( FPaths::Combine(FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), TEXT("[Platform]")) );
					FString RelativePath = TEXT("../../../");

					struct FFilePaths
					{
						FFilePaths(const FString& InFilename, FString&& InRelativeFilename, uint64 InFileOpenOrder) : Filename(InFilename), RelativeFilename(MoveTemp(InRelativeFilename)), FileOpenOrder(InFileOpenOrder)
						{ }
						FString Filename;
						FString RelativeFilename;
						uint64 FileOpenOrder;
					};

					TArray<FFilePaths> SortedFiles;
					SortedFiles.Empty(ChunkFilenames.Num());
					for (const FString& ChunkFilename : ChunkFilenames)
					{
						FString RelativeFilename = ChunkFilename.Replace(*CookedDirectory, *RelativePath);
						FPaths::RemoveDuplicateSlashes(RelativeFilename);
						FPaths::NormalizeFilename(RelativeFilename);
						if (FPaths::GetExtension(RelativeFilename).IsEmpty())
						{
							RelativeFilename = FPaths::SetExtension(RelativeFilename, TEXT("uasset")); // only use the uassets to decide which pak file these chunks should live in
						}
						RelativeFilename.ToLowerInline();
						uint64 FileOpenOrder = OrderMap.GetFileOrder(RelativeFilename, true /* bAllowUexpUBulkFallback */);
						/*if (FileOpenOrder != MAX_uint64)
						{
							UE_LOGF(LogAssetRegistryGenerator, Display, "Found file open order for %ls, %ll", *RelativeFilename, FileOpenOrder);
						}
						else
						{
							UE_LOGF(LogAssetRegistryGenerator, Display, "Didn't find openorder for %ls", *RelativeFilename, FileOpenOrder);
						}*/
						SortedFiles.Add(FFilePaths(ChunkFilename, MoveTemp(RelativeFilename), FileOpenOrder));
					}

					SortedFiles.Sort([&OrderMap, &CookedDirectory, &RelativePath](const FFilePaths& A, const FFilePaths& B)
					{
						uint64 AOrder = A.FileOpenOrder;
						uint64 BOrder = B.FileOpenOrder;

						if (AOrder == MAX_uint64 && BOrder == MAX_uint64)
						{
							return A.RelativeFilename.Compare(B.RelativeFilename, ESearchCase::IgnoreCase) < 0;
						}
						else 
						{
							return AOrder < BOrder;
						}
					});

					ChunkFilenames.Empty(SortedFiles.Num());
					for (int I = 0; I < SortedFiles.Num(); ++I)
					{
						ChunkFilenames.Add(MoveTemp(SortedFiles[I].Filename));
					}
				}
				else
				{
					// Sort so the order is consistent. If load order is important then it should be specified as a load order file to UnrealPak
					ChunkFilenames.Sort();
				}
			}

			int64 CurrentPakSize = 0;
			int64 NextFileSize = 0;
			FString NextFilename;
			bFinishedAllFiles = true;
			for (; FilenameIndex < ChunkFilenames.Num(); ++FilenameIndex)
			{
				FString Filename = ChunkFilenames[FilenameIndex];
				FString PakListLine = FPaths::ConvertRelativePathToFull(Filename.Replace(TEXT("[Platform]"), *Platform));
				if (PakChunkMaxChunkSize > 0)
				{
					const int64* ExistingPackageFileSize = PackageFileSizes.Find(PakListLine);
					const int64 PackageFileSize = ExistingPackageFileSize ? *ExistingPackageFileSize : 0;
					CurrentPakSize += PackageFileSize;
					if (PakChunkMaxChunkSize < CurrentPakSize)
					{
						// early out if we are over memory limit
						bFinishedAllFiles = false;
						NextFileSize = PackageFileSize;
						NextFilename = MoveTemp(PakListLine);
						break;
					}
				}
				
				FPaths::MakePathRelativeTo(PakListLine, *FPaths::RootDir());
				PakListLine.ReplaceInline(TEXT("/"), TEXT("\\"));
				PakListLine += TEXT("\r\n");
				PakListFile->Serialize(TCHAR_TO_ANSI(*PakListLine), PakListLine.Len());
			}

			const bool bAddedFilesToPakList = PakListFile->Tell() > 0;
			PakListFile->Close();

			if (!bFinishedAllFiles && !bAddedFilesToPakList)
			{
				const TCHAR* UnitsText = TEXT("MB");
				int32 Unit = 1000*1000;
				if (PakChunkMaxChunkSize < Unit * 10)
				{
					Unit = 1;
					UnitsText = TEXT("bytes");
				}
				UE_LOGFMT(LogAssetRegistryGenerator, Error, "Failed to add file {NextFilename} to paklist '{PakListFilename}'. The maximum size for a Pakfile is {MaxChunkFileSize}{UnitsText}, but the file to add is {ActualChunkFileSize}{UnitsText}.", 
					("NextFilename", NextFilename),
					("PakListFilename", PakListFilename),
					("MaxChunkFileSize", PakChunkMaxChunkSize / Unit),
					("UnitsText", UnitsText),
					("ActualChunkFileSize", (NextFileSize + Unit - 1) / Unit)	// Round the limit down and round the value up, so that the display always shows that the value is greater than the limit
				);
				bSucceeded = false;
				break;
			}

			// add this pakfilelist to our master list of pakfilelists
			FString PakChunkListLine = FString::Printf(TEXT("%s%s\r\n"), *PakChunkFilename, *PakChunkOptions);
			PakChunkListFile->Serialize(TCHAR_TO_ANSI(*PakChunkListLine), PakChunkListLine.Len());

			// Add layer information for this subchunk (we serialize it for the main chunk outside of this loop, hence the check).
			if (SubChunkIndex > 0)
			{
				ChunkLayerFile->Serialize(TCHAR_TO_ANSI(*LayerString), LayerString.Len());
			}
		}
	}

	ChunkLayerFile->Close();
	PakChunkListFile->Close();

	return bSucceeded;
}

void FAssetRegistryGenerator::CalculateChunkIdsAndAssignToManifest(const FName& PackageFName, const FString& PackagePathName,
	const FString& SandboxFilename, const FString& LastLoadedMapName, UE::Cook::FCookSandbox& InSandboxFile,
	const TSet<FName>& StartupPackages)
{
	TArray<int32> TargetChunks;
	TArray<int32> ExistingChunkIDs;

	if (!bGenerateChunks)
	{
		TargetChunks.AddUnique(0);
		ExistingChunkIDs.AddUnique(0);
	}
	else
	{
		UAssetManager& AssetManager = UAssetManager::Get();

		// Handle encrypted packages. Encrypted packages specified by GetEncryptedChunkIDsForPackage
		// use those specific chunks and nothing else.
		TArray<int32> EncryptedPackageChunkIDs = AssetManager.GetEncryptedChunkIDsForPackage(PackageFName).Array();
		if (EncryptedPackageChunkIDs.Num() != 0)
		{
			TargetChunks.Append(EncryptedPackageChunkIDs);
		}
		else
		{
			// Handle propagation of chunk assignments of Generated packages.
			// Generated packages by default use the chunks assigned to their Generator (unless encrypted, see above).
			// But generated packages for which the generator said it does not want chunk assignment from the generator
			// get their chunk assignment separately from the AssetManager.
			FName PackageNameThatDefinesChunks = PackageFName;
			UE::AssetRegistry::EDependencyProperty DependencyPropertyOfGenerator;
			FName GeneratorName = GetGeneratorPackage(PackageFName, this->State, DependencyPropertyOfGenerator);
			if (!GeneratorName.IsNone())
			{
				// CookGenerationHelper sets the Game flag on the dependency property if and only if it wants
				// the generated package to receive chunks from the generator,
				// see FGenerationHelper::FinishGeneratorPlatformSave.
				if (EnumHasAnyFlags(DependencyPropertyOfGenerator, UE::AssetRegistry::EDependencyProperty::Game))
				{
					PackageNameThatDefinesChunks = GeneratorName;
				}
			}

			TArray<int32> PackageChunkIDs = GetExplicitChunkIDs(PackageNameThatDefinesChunks);
			ExistingChunkIDs = GetExistingPackageChunkAssignments(PackageNameThatDefinesChunks, StartupPackages);
			PackageChunkIDs.Append(ExistingChunkIDs);
			AssetManager.GetPackageChunkIds(PackageNameThatDefinesChunks, TargetPlatform, PackageChunkIDs, TargetChunks);
		}
	}

	// Add the package to the manifest for every chunk the AssetManager found it should belong to
	for (const int32 PackageChunk : TargetChunks)
	{
		AddPackageToManifest(SandboxFilename, PackageFName, PackageChunk);
	}
	// Remove the package from the manifest for every chunk the AssetManager rejected from the existing chunks
	for (const int32 PackageChunk : ExistingChunkIDs)
	{
		if (!TargetChunks.Contains(PackageChunk))
		{
			RemovePackageFromManifest(PackageFName, PackageChunk);
		}
	}
}

bool FAssetRegistryGenerator::CleanManifestDirectories(UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	FString ChunkManifestDir = GetChunkManifestDirectory(COTFS);
	if (IFileManager::Get().DirectoryExists(*ChunkManifestDir))
	{
		if (!IFileManager::Get().DeleteDirectory(*ChunkManifestDir, false, true))
		{
			UE_LOGF(LogAssetRegistryGenerator, Error, "Failed to delete directory: %ls", *ChunkManifestDir);
			return false;
		}
	}

	FString ChunkListDir = FPaths::Combine(*FPaths::ProjectLogDir(), TEXT("ChunkLists"));
	if (IFileManager::Get().DirectoryExists(*ChunkListDir))
	{
		if (!IFileManager::Get().DeleteDirectory(*ChunkListDir, false, true))
		{
			UE_LOGF(LogAssetRegistryGenerator, Error, "Failed to delete directory: %ls", *ChunkListDir);
			return false;
		}
	}
	return true;
}

void FAssetRegistryGenerator::SetPreviousAssetRegistry(TUniquePtr<FAssetRegistryState>&& InPreviousState)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	check(InPreviousState);
	this->State = MoveTemp(*InPreviousState);
}

void FAssetRegistryGenerator::InjectEncryptionData(FAssetRegistryState& TargetState)
{
	UAssetManager& AssetManager = UAssetManager::Get();

	TMap<int32, FGuid> GuidCache;
	FContentEncryptionConfig EncryptionConfig;
	AssetManager.GetContentEncryptionConfig(EncryptionConfig);

	auto ApplyEncryptionTag = 
		[&GuidCache, &AssetManager, &TargetState](FAssetData* AssetData)
		{
			FString GuidString;
			const FAssetData::FChunkArrayView ChunkIDs = AssetData->GetChunkIDs();
			if (ChunkIDs.Num() > 1)
			{
				UE_LOGF(LogAssetRegistryGenerator, Warning, "Encrypted root asset '%ls' exists in two chunks. Only secondary assets should be shared between chunks.", *AssetData->GetObjectPathString());
			}
			else if (ChunkIDs.Num() == 1)
			{
				int32 ChunkID = ChunkIDs[0];
				FGuid Guid;

				if (GuidCache.Contains(ChunkID))
				{
					Guid = GuidCache[ChunkID];
				}
				else
				{
					Guid = GuidCache.Add(ChunkID, AssetManager.GetChunkEncryptionKeyGuid(ChunkID));
				}

				if (Guid.IsValid())
				{
					FAssetDataTagMap TagsAndValues = AssetData->TagsAndValues.CopyMap();
					TagsAndValues.Add(UAssetManager::GetEncryptionKeyAssetTagName(), Guid.ToString());
					FAssetData NewAssetData = FAssetData(AssetData->PackageName, AssetData->PackagePath, AssetData->AssetName, AssetData->AssetClassPath, TagsAndValues, ChunkIDs, AssetData->PackageFlags);
					NewAssetData.TaggedAssetBundles = AssetData->TaggedAssetBundles;
					TargetState.UpdateAssetData(AssetData, MoveTemp(NewAssetData));
				}
			}
		};

	for (FContentEncryptionConfig::TGroupMap::ElementType EncryptedAssetSetElement : EncryptionConfig.GetPackageGroupMap())
	{
		FName SetName = EncryptedAssetSetElement.Key;
		TSet<FName>& EncryptedRootAssets = EncryptedAssetSetElement.Value.PackageNames;

		for (FName EncryptedRootPackageName : EncryptedRootAssets)
		{
			TargetState.EnumerateMutableAssetsByPackageName(EncryptedRootPackageName,
				[&ApplyEncryptionTag](FAssetData* AssetData)
				{
					ApplyEncryptionTag(AssetData);

					return true; // Keep iterating assets in the package
				});
		}
	}

	// Since the encryption collections won't list all assets going into an encrypted chunk, repeat the search based on chunk id,
	// and look for any missed encrypted assets. Will not work for fast cook.
	TargetState.EnumerateAllMutableAssets(
		[&ApplyEncryptionTag](FAssetData& AssetData)
		{
			const FAssetData::FChunkArrayView ChunkIDs = AssetData.GetChunkIDs();
			const bool bEncrypted = ChunkIDs.ContainsByPredicate([](const int32 ChunkID) { return ChunkID >= 1000 && ChunkID < 2000; });
			const bool bHasTag = AssetData.TagsAndValues.Contains(UAssetManager::GetEncryptionKeyAssetTagName());

			if (bEncrypted && !bHasTag)
			{
				ApplyEncryptionTag(&AssetData);
			}

			return true;
		});
}

bool FAssetRegistryGenerator::SaveManifests(UE::Cook::FCookSandbox& InSandboxFile, UCookOnTheFlyServer& COTFS,
	int64 InOverrideChunkSize, const TCHAR* InManifestSubDir)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	if (!bGenerateChunks)
	{
		return true;
	}

	if (!GenerateStreamingInstallManifest(InOverrideChunkSize, InManifestSubDir, InSandboxFile, COTFS))
	{
		return false;
	}

	// Generate map for the platform abstraction
	TMultiMap<FString, int32> PakchunkMap;	// asset -> ChunkIDs map
	TSet<int32> PakchunkIndicesInUse;
	const FString PlatformName = TargetPlatform->PlatformName();

	// Collect all unique chunk indices and map all files to their chunks
	for (int32 PakchunkIndex = 0; PakchunkIndex < FinalChunkManifests.Num(); ++PakchunkIndex)
	{
		check(FinalChunkManifests[PakchunkIndex]);
		if (FinalChunkManifests[PakchunkIndex]->Num())
		{
			PakchunkIndicesInUse.Add(PakchunkIndex);
			for (const TPair<FName,FString>& Pair: *FinalChunkManifests[PakchunkIndex])
			{
				FString PlatFilename = Pair.Value.Replace(TEXT("[Platform]"), *PlatformName);
				PakchunkMap.Add(PlatFilename, PakchunkIndex);
			}
		}
	}

	// Sort our chunk IDs and file paths
	PakchunkMap.KeySort(TLess<FString>());
	PakchunkIndicesInUse.Sort(TLess<int32>());

	// Platform abstraction will generate any required platform-specific files for the chunks
	if (!TargetPlatform->GenerateStreamingInstallManifest(PakchunkMap, PakchunkIndicesInUse))
	{
		return false;
	}

	return true;
}

bool FAssetRegistryGenerator::ContainsMap(const FName& PackageName) const
{
	return PackagesContainingMaps.Contains(PackageName);
}

FAssetPackageData* FAssetRegistryGenerator::GetAssetPackageData(const FName& PackageName)
{
	return State.CreateOrGetAssetPackageData(PackageName);
}

TMap<int32, FString> FAssetRegistryGenerator::GetPakChunkIdToStringOverrideMap()
{
	TMap<int32, FString> ChunkIdStringOverride;
	UAssetManager::Get().GetPakChunkIdToStringMapping(ChunkIdStringOverride);
	constexpr FAsciiSet ValidCharacters(VALID_SAVEDDIRSUFFIX_CHARACTERS);

	TArray<int32> InvalidOverrides;
	for (const TPair<int32, FString>& PakChunkToName : ChunkIdStringOverride)
	{
		const FString& PakChunkName = PakChunkToName.Value;
		for (TCHAR Char : PakChunkName.GetCharArray())
		{
			if (Char == '\0')
			{
				break;
			}
			else if (!ValidCharacters.Contains(Char))
			{
				UE_LOGF(LogCook, Error, "GetPakChunkIdToStringMapping contains invalid string for chunk mapping. Character '%c' within %ls. Only the following character are allowed:\n%ls", Char, *PakChunkName, VALID_SAVEDDIRSUFFIX_CHARACTERS);
				InvalidOverrides.Add(PakChunkToName.Key);
				break;
			}
		}
	}
	for (const int32 InvalidOverride : InvalidOverrides)
	{
		ChunkIdStringOverride.Remove(InvalidOverride);
		UE_LOGF(LogCook, Log, "Removing PakChunk name override for chunk - %d due to invalid character.", InvalidOverride);
	}
	return ChunkIdStringOverride;
}

FString FAssetRegistryGenerator::GetPakChunkNameForId(const int32 PakChunkId)
{
	return GetPakChunkNameForId(GetPakChunkIdToStringOverrideMap(), PakChunkId);
}

FString FAssetRegistryGenerator::GetPakChunkNameForId(const TMap<int32, FString>& ChunkIdToStringOveride, const int32 PakChunkId)
{
	return ChunkIdToStringOveride.Contains(PakChunkId) ?
		FString::Printf(TEXT("%c%s%c"), NAMED_PAK_CHUNK_DELIMITER_CHAR, *ChunkIdToStringOveride[PakChunkId], NAMED_PAK_CHUNK_DELIMITER_CHAR) :
		FString::Printf(TEXT("%d"), PakChunkId);
}

void FAssetRegistryGenerator::UpdateCollectionAssetData()
{
	// Read out the per-platform settings use to build the list of collections to tag
	bool bTagAllCollections = false;
	TArray<FString> CollectionsToIncludeOrExclude;
	{
		const FString PlatformIniName = TargetPlatform->IniPlatformName();

		FConfigFile PlatformEngineIni;
		FConfigCacheIni::LoadLocalIniFile(PlatformEngineIni, TEXT("Engine"), true, (!PlatformIniName.IsEmpty() ? *PlatformIniName : ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName())));

		// The list of collections will either be a inclusive or a exclusive depending on the value of bTagAllCollections
		PlatformEngineIni.GetBool(TEXT("AssetRegistry"), TEXT("bTagAllCollections"), bTagAllCollections);
		PlatformEngineIni.GetArray(TEXT("AssetRegistry"), bTagAllCollections ? TEXT("CollectionsToExcludeAsTags") : TEXT("CollectionsToIncludeAsTags"), CollectionsToIncludeOrExclude);
	}

	// Build the list of collections we should tag for each asset
	TMap<FSoftObjectPath, TArray<FName>> AssetPathsToCollectionTags;
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		TArray<FCollectionNameType> CollectionNamesToTag;
		CollectionContainer->GetCollections(CollectionNamesToTag);
		if (bTagAllCollections)
		{
			CollectionNamesToTag.RemoveAll([&CollectionsToIncludeOrExclude](const FCollectionNameType& CollectionNameAndType)
			{
				return CollectionsToIncludeOrExclude.Contains(CollectionNameAndType.Name.ToString());
			});
		}
		else
		{
			CollectionNamesToTag.RemoveAll([&CollectionsToIncludeOrExclude](const FCollectionNameType& CollectionNameAndType)
			{
				return !CollectionsToIncludeOrExclude.Contains(CollectionNameAndType.Name.ToString());
			});
		}
		
		TArray<FSoftObjectPath> TmpAssetPaths;
		for (const FCollectionNameType& CollectionNameToTag : CollectionNamesToTag)
		{
			const FName CollectionTagName = *FString::Printf(TEXT("%s%s"), FAssetData::GetCollectionTagPrefix(), *CollectionNameToTag.Name.ToString());

			TmpAssetPaths.Reset();
			CollectionContainer->GetAssetsInCollection(CollectionNameToTag.Name, CollectionNameToTag.Type, TmpAssetPaths);

			for (const FSoftObjectPath& AssetPath : TmpAssetPaths)
			{
				TArray<FName>& CollectionTagsForAsset = AssetPathsToCollectionTags.FindOrAdd(AssetPath);
				CollectionTagsForAsset.AddUnique(CollectionTagName);
			}
		}
	}

	// Apply the collection tags to the asset registry state
	// Collection tags are queried only by the existence of the key, the value is never used. But Tag Values are not allowed
	// to be empty. Set the value for each tag to an arbitrary field, something short to avoid wasting memory. We use 1 (aka "true") for now.
	FStringView CollectionValue(TEXTVIEW("1"));
	for (const TPair<FSoftObjectPath, TArray<FName>>& AssetPathToCollectionTagsPair : AssetPathsToCollectionTags)
	{
		const FSoftObjectPath& AssetPath = AssetPathToCollectionTagsPair.Key;
		const TArray<FName>& CollectionTagsForAsset = AssetPathToCollectionTagsPair.Value;

		FAssetData* AssetData = State.GetMutableAssetByObjectPath(AssetPath);
		if (AssetData)
		{
			FAssetDataTagMap TagsAndValues = AssetData->TagsAndValues.CopyMap();
			for (const FName& CollectionTagName : CollectionTagsForAsset)
			{
				TagsAndValues.Add(CollectionTagName, FString(CollectionValue));
			}
			FAssetData NewAssetData(*AssetData);
			NewAssetData.TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(TagsAndValues));
			State.UpdateAssetData(AssetData, MoveTemp(NewAssetData));
		}
	}
}

void FAssetRegistryGenerator::Initialize()
{
	FGameDelegates::Get().GetAssignLayerChunkDelegate() = FAssignLayerChunkDelegate::CreateStatic(AssignLayerChunkDelegate);
}

void FAssetRegistryGenerator::InitializeAssetRegistryForCookOnTheFly(const FAssetRegistrySerializationOptions& DevelopmentSaveOptions)
{
	// If the asset registry is still doing its background scan, we need to wait for it to finish and tick it so that the results are flushed out
	AssetRegistry.WaitForCompletion();
	ensureMsgf(!AssetRegistry.IsLoadingAssets(), TEXT("Cannot initialize asset registry generator while asset registry is still scanning source assets "));
	AssetRegistry.InitializeTemporaryAssetRegistryState(State, DevelopmentSaveOptions);
}

static UE::Cook::ECookResult DiskSizeToCookResult(int64 DiskSize)
{
	if (DiskSize >= 0)
	{
		return UE::Cook::ECookResult::Succeeded;
	}
	switch (DiskSize)
	{
	case -2: return UE::Cook::ECookResult::NeverCookPlaceholder;
	case -3: return UE::Cook::ECookResult::NotAttempted;
	default: return UE::Cook::ECookResult::Failed;
	}
}

static int64 CookResultToDiskSize(UE::Cook::ECookResult CookResult)
{
	switch (CookResult)
	{
	case UE::Cook::ECookResult::Succeeded: return 0;
	case UE::Cook::ECookResult::NeverCookPlaceholder: return -2;
	case UE::Cook::ECookResult::NotAttempted: return -3;
	default: return -1;
	}
}

void FAssetRegistryGenerator::ComputePackageDifferences(const FComputeDifferenceOptions& Options,
	const FAssetRegistryState& PreviousState, FAssetRegistryDifference& OutDifference)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);

	TArray<FName> ModifiedScriptPackages;
	const TMap<FName, const FAssetPackageData*>& PreviousAssetPackageDataMap = PreviousState.GetAssetPackageDataMap();
	OutDifference.Packages.Reserve(PreviousAssetPackageDataMap.Num());

	AssetRegistry.EnumerateAllPackages(
		[this, &Options, &PreviousAssetPackageDataMap, &OutDifference]
		(FName PackageName, const FAssetPackageData& CurrentPackageDataRef)
	{
		const FAssetPackageData* CurrentPackageData = &CurrentPackageDataRef;
		const FAssetPackageData* PreviousPackageData = PreviousAssetPackageDataMap.FindRef(PackageName);

		if (!PreviousPackageData)
		{
			// A package that was not explored in the previous cook. No need to record it
		}
		else if (ComputePackageDifferences_IsPackageFileUnchanged(Options, PackageName, *CurrentPackageData,
			*PreviousPackageData))
		{
			if (PreviousPackageData->DiskSize < 0)
			{
				UE::Cook::ECookResult CookResult = DiskSizeToCookResult(PreviousPackageData->DiskSize);
				if (CookResult == UE::Cook::ECookResult::NeverCookPlaceholder)
				{
					OutDifference.Packages.Add(PackageName, EDifference::IdenticalNeverCookPlaceholder);
				}
				else
				{
					OutDifference.Packages.Add(PackageName, EDifference::IdenticalUncooked);
				}
			}
			else if (FPackageName::IsScriptPackage(WriteToString<256>(PackageName)))
			{
				OutDifference.Packages.Add(PackageName, EDifference::IdenticalScript);
			}
			else
			{
				OutDifference.Packages.Add(PackageName, EDifference::IdenticalCooked);
			}
		}
		else
		{
			if (PreviousPackageData->DiskSize < 0)
			{
				UE::Cook::ECookResult CookResult = DiskSizeToCookResult(PreviousPackageData->DiskSize);
				if (CookResult == UE::Cook::ECookResult::NeverCookPlaceholder)
				{
					OutDifference.Packages.Add(PackageName, EDifference::ModifiedNeverCookPlaceholder);
				}
				else
				{
					OutDifference.Packages.Add(PackageName, EDifference::ModifiedUncooked);
				}
			}
			else if (FPackageName::IsScriptPackage(WriteToString<256>(PackageName)))
			{
				OutDifference.Packages.Add(PackageName, EDifference::ModifiedScript);
			}
			else
			{
				OutDifference.Packages.Add(PackageName, EDifference::ModifiedCooked);
			}
		}
	});

	for (const TPair<FName, const FAssetPackageData*>& PackagePair : PreviousAssetPackageDataMap)
	{
		FName PackageName = PackagePair.Key;
		const FAssetPackageData* PreviousPackageData = PackagePair.Value;
		TOptional<FAssetPackageData> CurrentPackageDataCopy = AssetRegistry.GetAssetPackageDataCopy(PackageName);
		const FAssetPackageData* CurrentPackageData = CurrentPackageDataCopy.GetPtrOrNull();

		if (!CurrentPackageData)
		{
			// If it's a generated package, exclude it from the results list and do not remove it.
			// It will be evaluated for identical/modified/removed only if the generator package is cooked,
			// during the generator's process step.
			UE::AssetRegistry::EDependencyProperty UnusedGeneratorProperty;
			FName GeneratorName = GetGeneratorPackage(PackageName, PreviousState, UnusedGeneratorProperty);
			if (!GeneratorName.IsNone())
			{
				// Keep track of all generators and their list of generated
				TOptional<FAssetPackageData> GeneratorDataCopy = AssetRegistry.GetAssetPackageDataCopy(GeneratorName);
				const FAssetPackageData* GeneratorData = GeneratorDataCopy.GetPtrOrNull();
				if (!GeneratorData)
				{
					// Mark it as removed; it will be regenerated when the Generator cooks
					OutDifference.Packages.Add(PackageName, EDifference::RemovedCooked);
				}
				else
				{
					FGeneratorPackageInfo& Info = OutDifference.GeneratorPackages.FindOrAdd(GeneratorName);
					Info.Generated.Add(PackageName, CopyAssetPackageDataForIncrementalCook(*PreviousPackageData));
				}
			}
			else
			{
				if (PreviousPackageData->DiskSize < 0)
				{
					UE::Cook::ECookResult CookResult = DiskSizeToCookResult(PreviousPackageData->DiskSize);
					if (CookResult == UE::Cook::ECookResult::NeverCookPlaceholder)
					{
						OutDifference.Packages.Add(PackageName, EDifference::RemovedNeverCookPlaceholder);
					}
					else
					{
						OutDifference.Packages.Add(PackageName, EDifference::RemovedUncooked);
					}
				}
				else if (FPackageName::IsScriptPackage(WriteToString<256>(PackageName)))
				{
					OutDifference.Packages.Add(PackageName, EDifference::RemovedScript);
				}
				else
				{
					OutDifference.Packages.Add(PackageName, EDifference::RemovedCooked);
				}
			}
		}
	}

	// If a Generator package has been removed, remove all its generated packages
	for (TMap<FName, FGeneratorPackageInfo>::TIterator Iter(OutDifference.GeneratorPackages); Iter; ++Iter)
	{
		FName GeneratorName = Iter->Key;
		EDifference* GeneratorDifference = OutDifference.Packages.Find(GeneratorName);
		if (GeneratorDifference && *GeneratorDifference == EDifference::RemovedCooked)
		{
			for (const TPair<FName, FAssetPackageData>& Generated : Iter->Value.Generated)
			{
				OutDifference.Packages.Add(Generated.Key, EDifference::RemovedCooked);
			}
			Iter.RemoveCurrent();
		}
	}

	if (Options.bRecurseModifications)
	{
		const FStringView ExternalActorsFolderName(ULevel::GetExternalActorsFolderName());
		const FStringView ExternalObjectsFolderName = FPackagePath::GetExternalObjectsFolderName();

		// Recurse modified packages to their dependencies. This is needed because we only compare package guids
		TArray<FName> VisitStack;
		for (TPair<FName, EDifference>& Pair : OutDifference.Packages)
		{
			if (Pair.Value == EDifference::ModifiedCooked || Pair.Value == EDifference::ModifiedUncooked || Pair.Value == EDifference::ModifiedNeverCookPlaceholder
				|| (Pair.Value == EDifference::ModifiedScript && Options.bRecurseScriptModifications))
			{
				VisitStack.Add(Pair.Key);
			}
		}

		TSet<FName> Visited;
		Visited.Reserve(PreviousState.GetNumPackages());
		for (FName PackageName : VisitStack)
		{
			Visited.Add(PackageName);
		}
		while (!VisitStack.IsEmpty())
		{
			FName ModifiedPackage = VisitStack.Pop(EAllowShrinking::No);
			FString ModifiedPackageLeafName;

			// Read referencers from the current global assetregistry. If there are referencers in the old state that are not in the
			// current assetregistry, then they must have changed themselves and so are already in the modified set.
			TArray<FAssetIdentifier> Referencers;
			FAssetIdentifier ModifiedPackageAssetId(ModifiedPackage);
			AssetRegistry.GetReferencers(ModifiedPackageAssetId, Referencers, UE::AssetRegistry::EDependencyCategory::Package,
				UE::AssetRegistry::EDependencyQuery::Hard);
			AssetRegistry.GetReferencers(ModifiedPackageAssetId, Referencers, UE::AssetRegistry::EDependencyCategory::Package,
				UE::AssetRegistry::EDependencyQuery::Build);

			for (const FAssetIdentifier& Referencer : Referencers)
			{
				FName ReferencerPackageName = Referencer.PackageName;
				// EXTERNALACTOR_TODO: Replace this workaround for ExternalActors with a modification to the ExternalActor Packages' 
				// dependencies. External actors have an import dependency (hard, build, game) on their Map package because
				// the map package is their outer. But unless they have some other use of the Map package, we do not want to
				// mark them as modified even if their map package is modified. Doing so would mark all actors in the map as
				// modified anytime one of them changed.
				// Workaround: Detect external actors by naming convention and suppress their reference to the map package.
				// See also UAssetManager::ShouldSetManager
				TStringBuilder<256> ReferencerPackageNameStr(InPlace, ReferencerPackageName);
				bool bReferencerIsMapExternalPackage = false;
				for (FStringView ExternalFolderName : { ExternalActorsFolderName, ExternalObjectsFolderName })
				{
					int32 ExternalFolderIndex = UE::String::FindFirst(ReferencerPackageNameStr, ExternalFolderName, ESearchCase::IgnoreCase);
					if (ExternalFolderIndex != INDEX_NONE)
					{
						if (ModifiedPackageLeafName.IsEmpty())
						{
							ModifiedPackageLeafName = FPathViews::GetCleanFilename(WriteToString<256>(ModifiedPackage));
						}
						FStringView GeneratedRelativePath = ReferencerPackageNameStr.ToView().RightChop(ExternalFolderIndex + ExternalFolderName.Len());
						if (GeneratedRelativePath.Contains(ModifiedPackageLeafName))
						{
							bReferencerIsMapExternalPackage = true;
							break;
						}
					}
				}
				if (bReferencerIsMapExternalPackage)
				{
					// Suppress this reference; External actors should not be marked dirty if only their map package is dirty
					continue;
				}

				int32 ReferencerPackageHash = GetTypeHash(ReferencerPackageName);
				bool bAlreadyVisited;
				Visited.AddByHash(ReferencerPackageHash, ReferencerPackageName, &bAlreadyVisited);
				if (bAlreadyVisited)
				{
					continue;
				}

				EDifference* ReferencerDifference = OutDifference.Packages.FindByHash(ReferencerPackageHash, ReferencerPackageName);
				if (!ReferencerDifference)
				{
					// The referencer is not in the packages explored during the previous cook, so none of the previous cook
					// packages had it as a dependency, so we do not need to follow its referencers.
					continue;
				}

				// Convert this Referencer to modified. We do not need to handle Removed differences, because we found
				// it in the current assetregistry's dependency tree and so it can not be a Removed difference.
				switch (*ReferencerDifference)
				{
				case EDifference::IdenticalCooked: *ReferencerDifference = EDifference::ModifiedCooked; break;
				case EDifference::IdenticalUncooked: *ReferencerDifference = EDifference::ModifiedUncooked; break;
				case EDifference::IdenticalNeverCookPlaceholder: *ReferencerDifference = EDifference::ModifiedNeverCookPlaceholder; break;
				case EDifference::IdenticalScript: *ReferencerDifference = EDifference::ModifiedScript; break;
				default: break;
				}
				VisitStack.Add(ReferencerPackageName);
			}
		}
	}
}

bool FAssetRegistryGenerator::ComputePackageDifferences_IsPackageFileUnchanged(
	const FComputeDifferenceOptions& Options, FName PackageName, const FAssetPackageData& CurrentPackageData,
	const FAssetPackageData& PreviousPackageData)
{
	if (CurrentPackageData.GetPackageSavedHash() != PreviousPackageData.GetPackageSavedHash())
	{
		return false;
	}

	if (Options.bLegacyIterativeUseClassFilters)
	{
		if (!UE::TargetDomain::IsIncrementalCookEnabled(PackageName, false /* bAllowAllClasses */))
		{
			return false;
		}
	}

	return true;
}

FAssetPackageData FAssetRegistryGenerator::CopyAssetPackageDataForIncrementalCook(const FAssetPackageData& Source)
{
	FAssetPackageData Result;

	// Copy small scalars since they don't cost memory or much network bandwidth
	// Skip large scalars and containers that we don't need to save network bandwidth

	// CookedHash is not read by the cook
	// Result.CookedHash = Source.CookedHash;
	
	// PackageSavedHash is used during incremental cooks to compare whether the package is modified
	Result.SetPackageSavedHash(Source.GetPackageSavedHash());

	// ChunkHashes is not read by the cook
	// Result.ChunkHashes = Source.ChunkHashes;

	// ImportedClasses is a large container, but incremental cook needs it to calculate the current packagedigest
	Result.ImportedClasses = Source.ImportedClasses;
	Result.DiskSize = Source.DiskSize;
	Result.FileVersionUE = Source.FileVersionUE;
	Result.FileVersionLicenseeUE = Source.FileVersionLicenseeUE;
	Result.SetIsLicenseeVersion(Source.IsLicenseeVersion());
	Result.SetHasVirtualizedPayloads(Source.HasVirtualizedPayloads());

	// CustomVersions are not read by the cook
	//Result.SetCustomVersions(Source.GetCustomVersions());

	Result.Extension = Source.Extension;

	return Result;
}

FName FAssetRegistryGenerator::GetGeneratorPackage(FName PackageName, const FAssetRegistryState& InState,
	UE::AssetRegistry::EDependencyProperty& OutDependencyPropertyOfGenerator)
{
	OutDependencyPropertyOfGenerator = UE::AssetRegistry::EDependencyProperty::None;
	bool bGenerated = false;
	InState.EnumerateAssetsByPackageName(PackageName, [&bGenerated](const FAssetData* Asset)
		{
			bGenerated = (Asset->PackageFlags & PKG_CookGenerated) != 0;
			return false; // Stop iterating
		});
	if (!bGenerated)
	{
		return NAME_None;
	}

	TArray<FAssetDependency> Referencers;
	InState.GetReferencers(FAssetIdentifier(PackageName), Referencers, UE::AssetRegistry::EDependencyCategory::Package);
	if (Referencers.Num() != 1)
	{
		return NAME_None;
	}
	FAssetDependency& Dependency = Referencers[0];
	OutDependencyPropertyOfGenerator = Dependency.Properties;
	return Dependency.AssetId.PackageName;
}

void FAssetRegistryGenerator::ReportGeneratorPackages(const FAssetRegistryState& PreviousState,
	TMap<FName, FGeneratorPackageInfo>& OutGeneratorPackages, int32& OutNumNeverCookPlaceHolderPackages)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);

	OutGeneratorPackages.Reset();
	OutNumNeverCookPlaceHolderPackages = 0;

	for (const TPair<FName, const FAssetPackageData*>& PackagePair : PreviousState.GetAssetPackageDataMap())
	{
		FName PackageName = PackagePair.Key;
		const FAssetPackageData* PreviousPackageData = PackagePair.Value;
		bool bGenerated = false;
		PreviousState.EnumerateAssetsByPackageName(PackageName, [&bGenerated](const FAssetData* AssetData)
			{
				bGenerated = (AssetData->PackageFlags & PKG_CookGenerated) != 0;
				return false; // stop iterating
			});
		if (bGenerated)
		{
			// A generated package is always referenced by its generator, and it might be referenced by its fellow generated
			// packages. Look in its referencers for a non-generated package, and assume that is the generator.
			TArray<FAssetIdentifier> Referencers;
			PreviousState.GetReferencers(FAssetIdentifier(PackageName), Referencers,
				UE::AssetRegistry::EDependencyCategory::Package);
			for (TArray<FAssetIdentifier>::TIterator ReferencerIter(Referencers); ReferencerIter; ++ReferencerIter)
			{
				bool bReferencerIsGenerated = false;
				PreviousState.EnumerateAssetsByPackageName(ReferencerIter->PackageName,
					[&bReferencerIsGenerated](const FAssetData* AssetData)
					{
						bReferencerIsGenerated = (AssetData->PackageFlags & PKG_CookGenerated) != 0;
						return false; // stop iterating
					});
				if (bReferencerIsGenerated)
				{
					ReferencerIter.RemoveCurrent();
				}
			}
			FName GeneratorName = Referencers.Num() == 1 ? Referencers[0].PackageName : NAME_None;
			const FAssetPackageData* GeneratorData = !GeneratorName.IsNone()
				? PreviousState.GetAssetPackageData(GeneratorName) : nullptr;
			if (GeneratorData)
			{
				OutGeneratorPackages.FindOrAdd(GeneratorName).Generated.Add(PackageName,
					CopyAssetPackageDataForIncrementalCook(*PreviousPackageData));
			}
		}
		if (PreviousPackageData->DiskSize < 0)
		{
			UE::Cook::ECookResult CookResult = DiskSizeToCookResult(PreviousPackageData->DiskSize);
			if (CookResult == UE::Cook::ECookResult::NeverCookPlaceholder)
			{
				++OutNumNeverCookPlaceHolderPackages;
			}
		}
	}
}

void FAssetRegistryGenerator::RemoveStalePackages(
	const TMap<FName, UE::Cook::Artifact::FOplogPackageData>& OplogPackages)
{
	using namespace UE::Cook;

	TStringBuilder<256> PackageNameStr;

	// Delete stale script packages. We know which script packages are stale immediately, no LastReferencedDate timeout
	// required,by looking for them in the current editor binary.
	TSet<FAssetIdentifier> StaleScriptPackages;
	for (const TPair<FName, const FAssetPackageData*>& Pair : State.GetAssetPackageDataMap())
	{
		PackageNameStr.Reset();
		PackageNameStr << Pair.Key;
		if (FPackageName::IsScriptPackage(PackageNameStr))
		{
			if (!FindPackage(nullptr, *PackageNameStr))
			{
				StaleScriptPackages.Add(FAssetIdentifier(Pair.Key));
			}
		}
	}
	if (!StaleScriptPackages.IsEmpty())
	{
		State.RemovePackageDatas(StaleScriptPackages);
		State.RemoveDependsNodes(StaleScriptPackages);
	}

	// Delete stale content packages: if the PackageWriter removed them from the oplog for staleness then we need
	// to remove them from AssetRegistry is well.
	TArray<FAssetData*> RemoveDatas;
	for (const TPair<FName, const FAssetPackageData*>& Pair : State.GetAssetPackageDataMap())
	{
		FName PackageName = Pair.Key;
		const FAssetPackageData* PackageData = Pair.Value;

		// TODO: NeverCookPlaceholders were accidentally stripped from the AssetRegistry by CL 38685901 to support
		// build dependencies in the oplog; that changelist dropped them from the list of IgnorePackageDatas passed
		// into the AssetRegistryGenerator to keep for the DevelopmentAssetRegistry.
		// Culling of data from the DevelopmentAssetRegistry now only occurs here, so if we change this function to
		// keep NeverCookPlaceholders, we will change the behavior back to keeping them in the AssetRegistry. This
		// might no longer be needed. Investigate and enable or delete this block.
		static volatile bool bKeepNeverCookPlaceholders = false;
		if (bKeepNeverCookPlaceholders)
		{
			// Special case: NeverCookPlaceholders are not culled, because they are never put into the Oplog.
			// TODO: Need to add some way to cull stale NeverCookPlaceholders; add NeverCookPlaceholders to
			// the oplog?
			if (PackageData->DiskSize < 0)
			{
				ECookResult CookResult = DiskSizeToCookResult(PackageData->DiskSize);
				if (CookResult == ECookResult::NeverCookPlaceholder)
				{
					continue;
				}
			}
		}
		if (!OplogPackages.Contains(PackageName))
		{
			State.EnumerateMutableAssetsByPackageName(PackageName, [&RemoveDatas](FAssetData* AssetData)
				{
					RemoveDatas.Add(AssetData);
					return true; // keep enumerating
				});
		}
	}
	if (!RemoveDatas.IsEmpty())
	{
		TBitArray<FDefaultBitArrayAllocator> bRemoved;
		TBitArray<FDefaultBitArrayAllocator> bRemovedDependencyData;
		bRemoved.SetNumUninitialized(RemoveDatas.Num());
		bRemovedDependencyData.SetNumUninitialized(RemoveDatas.Num());
		State.RemoveAssetDatas(RemoveDatas, true /* bRemoveDependencyData */, bRemoved, bRemovedDependencyData);
	}
}

void FAssetRegistryGenerator::AddGlobalsToCookedAssetRegistry(UCookOnTheFlyServer& COTFS)
{
	const bool bCookingDLC = COTFS.IsCookingDLC();
	TStringBuilder<256> PackageNameStr;

	// Script packages: add Script packages from global assetregistry.
	// This addition of script packages is possibly no longer used, we include it for now as a legacy behavior.
	// We also take advantage of it in RemoveStalePackages to find all the stale /Script depends nodes.

	// Copy /Script PackageDatas from the current global AssetRegistry
	// DLC cooks skip this behavior to save time.
	if (!bCookingDLC)
	{
		AssetRegistry.EnumerateAllPackages([this, &PackageNameStr](FName PackageName, const FAssetPackageData& PackageData)
			{
				PackageNameStr.Reset();
				PackageNameStr << PackageName;
				if (FPackageName::IsScriptPackage(PackageNameStr))
				{
					*State.CreateOrGetAssetPackageData(PackageName) = PackageData;
				}
			});
	}

	// DependsNodes: 1) Remove dependencies to unrecorded assets. We copied dependencies for recorded assets from the
	// global assetregistry, but the targets of those dependencies could have included assets that were not cooked
	// (in this session or even in previous incremental cooks) because they are editoronly or NeverCook.
	
	// DependsNodes: 2) Manage Dependencies: add in manage dependenciesfrom the global AssetRegistry, and remove
	// any stale PrimaryAsset values from previous cooks. Mange dependencies are used by some editor tools that
	// analyze the cooked assetregistry such as AssetManagerEditorModule.
	TSet<FAssetIdentifier> RemoveNodes;
	for (TPair<FAssetIdentifier, FDependsNode*>& Pair : State.CachedDependsNodes)
	{
		const FAssetIdentifier& Id = Pair.Key;
		if (Id.IsValue())
		{
			// DependsNode for a SearchableName, keep it.
		}
		else if (Id.PrimaryAssetType.IsValid())
		{
			// DependsNode for a PrimaryAsset, remove it (to be added back below).
			RemoveNodes.Add(Pair.Key);
		}
		else if (Id.IsPackage())
		{
			// DependsNode for a Package. Either /Script or content. Keep the dependsnode for either type if and only
			// if we have a PackageAssetData for it.
			if (!State.CachedPackageData.Contains(Id.PackageName))
			{
				RemoveNodes.Add(Id);
			}
		}
		else if (!Id.PackageName.IsNone() && !Id.ObjectName.IsNone())
		{
			// A DependsNode to an object within a package. Invalid, remove it.
			RemoveNodes.Add(Id);
		}
		else
		{
			// Invalid depends node with all fields == NAME_None; remove it
			RemoveNodes.Add(Id);
		}
	}
	if (!RemoveNodes.IsEmpty())
	{
		State.RemoveDependsNodes(RemoveNodes, FAssetRegistryState::ERemoveDependsFlags::RemoveOrphans);
	}

	// Copy Manage dependencies from the current global AssetRegistry.
	// DLC cooks skip this behavior to save time.
	if (!bCookingDLC)
	{
		AssetRegistry.CopyManageDependenciesToState(State);
	}
}

void FAssetRegistryGenerator::FinalizeChunkIDs(const TSet<FName>& CookedPackages,
	const TSet<FName>& DevelopmentOnlyPackages, UE::Cook::FCookSandbox& InSandboxFile,
	bool bGenerateStreamingInstallManifest, const TSet<FName>& StartupPackages,
	const FAssetRegistrySerializationOptions& DevelopmentSaveOptions)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	// bGenerateNoChunks overrides bGenerateStreamingInstallManifest overrides ShouldPlatformGenerateStreamingInstallManifest
	// bGenerateChunks means we allow chunks other than 0 based on package ChunkIds, AND we generate a manifest for each chunk
	const UProjectPackagingSettings* PackagingSettings = Cast<UProjectPackagingSettings>(UProjectPackagingSettings::StaticClass()->GetDefaultObject());
	if (PackagingSettings->bGenerateNoChunks)
	{
		bGenerateChunks = false;
	}
	else if (bGenerateStreamingInstallManifest)
	{
		bGenerateChunks = true;
	}
	else
	{
		bGenerateChunks = ShouldPlatformGenerateStreamingInstallManifest(TargetPlatform);
	}

	// Development only packages should have been marked via UpdateAssetRegistryData with a negative size indicating why they were not cooked 
	for (FName DevelopmentOnlyPackage : DevelopmentOnlyPackages)
	{
		FAssetPackageData* PackageData = State.GetAssetPackageData(DevelopmentOnlyPackage);
		if (PackageData && PackageData->DiskSize >= 0)
		{
			// TODO UE-252126: Raise this logseverity to Warning once UE-252126 is fixed;
			// UE-252126 causes some low-priority occurrences of this warning.
			UE_LOGF(LogAssetRegistryGenerator, Display,
				"Package %ls is a development-only package but was not marked with DiskSize explaining why it was not cooked. Marking it as a generic skip.",
				*DevelopmentOnlyPackage.ToString());
			PackageData->DiskSize = -1;
		}
	}

	// Copy ExplicitChunkIDs and other data from the AssetRegistry into the maps we use during finalization
	State.EnumerateAllMutableAssets([&](FAssetData& AssetData)
	{
		for (int32 ChunkID : AssetData.GetChunkIDs())
		{
			if (ChunkID < 0)
			{
				UE_LOGF(LogAssetRegistryGenerator, Warning, "Out of range ChunkID: %d", ChunkID);
				ChunkID = 0;
			}
			
			ExplicitChunkIDs.FindOrAdd(AssetData.PackageName).AddUnique(ChunkID);
		}

		// Clear the Asset's chunk id list. We will fill it with the final IDs to use later on.
		// Chunk Ids are safe to modify in place so do a const cast
		AssetData.ClearChunkIDs();

		// Update whether the owner package contains a map
		if ((AssetData.PackageFlags & PKG_ContainsMap) != 0)
		{
			PackagesContainingMaps.Add(AssetData.PackageName);
		}
	});

	if (bGenerateChunks)
	{
		TSet<FName> AdditionalUncookedDependenciesForChunkSearch;
		TSet<FName> ProcessedPackages;
		ProcessedPackages.Reserve(CookedPackages.Num());
		TArray<FName> PackagesToProcess = CookedPackages.Array();
		while (PackagesToProcess.Num() > 0)
		{
			const FName PackageName = PackagesToProcess[0];
			PackagesToProcess.RemoveAtSwap(0);
			if (ProcessedPackages.Contains(PackageName))
			{
				continue;
			}

			TArray<FName> Dependencies;
			AssetRegistry.GetDependencies(PackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Game);
			AssetRegistry.GetDependencies(PackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Build);

			AdditionalUncookedDependenciesForChunkSearch.Append(Dependencies);
			PackagesToProcess.Append(Dependencies);
			ProcessedPackages.Add(PackageName);
		}
		TSet<FName> CookedPackagesAndMapDependencies = CookedPackages;
		CookedPackagesAndMapDependencies.Append(AdditionalUncookedDependenciesForChunkSearch);
		// Update the chunk map based upon what we have cooked and their game dependencies to account for external actors.
		UAssetManager::Get().UpdateCachedChunkMapAfterCook(CookedPackagesAndMapDependencies, StartupPackages);
	}
	// add all the packages to the unassigned package list
	for (FName CookedPackage : CookedPackages)
	{
		const FString SandboxPath = InSandboxFile.ConvertToAbsolutePathForExternalAppForWrite(*FPackageName::LongPackageNameToFilename(CookedPackage.ToString()));

		AllCookedPackageSet.Add(CookedPackage, SandboxPath);
		UnassignedPackageSet.Add(CookedPackage, SandboxPath);
	}

	// Capture list at start as elements will be removed during iteration
	TArray<FName> UnassignedPackageList;
	UnassignedPackageSet.GenerateKeyArray(UnassignedPackageList);

	// process the remaining unassigned packages
	for (FName PackageFName : UnassignedPackageList)
	{
		const FString& SandboxFilename = AllCookedPackageSet.FindChecked(PackageFName);
		const FString PackagePathName = PackageFName.ToString();

		CalculateChunkIdsAndAssignToManifest(PackageFName, PackagePathName, SandboxFilename, FString(), InSandboxFile,
			StartupPackages);
	}
	// anything that remains in the UnAssignedPackageSet is put in chunk0 by FixupPackageDependenciesForChunks

	FixupPackageDependenciesForChunks(InSandboxFile);
}

void FAssetRegistryGenerator::RegisterChunkDataGenerator(TSharedRef<IChunkDataGenerator> InChunkDataGenerator)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	ChunkDataGenerators.Add(MoveTemp(InChunkDataGenerator));
}

void FAssetRegistryGenerator::PreSave(const TSet<FName>& InCookedPackages)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	UAssetManager::Get().PreSaveAssetRegistry(TargetPlatform, InCookedPackages);
}

void FAssetRegistryGenerator::PostSave()
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	UAssetManager::Get().PostSaveAssetRegistry();
}

void FAssetRegistryGenerator::AddAssetToFileOrderRecursive(const FName& InPackageName, TArray<FName>& OutFileOrder, TSet<FName>& OutEncounteredNames, const TSet<FName>& InPackageNameSet, const TSet<FName>& InTopLevelAssets)
{
	if (!OutEncounteredNames.Contains(InPackageName))
	{
		OutEncounteredNames.Add(InPackageName);

		TArray<FName> Dependencies;
		AssetRegistry.GetDependencies(InPackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

		for (FName DependencyName : Dependencies)
		{
			if (InPackageNameSet.Contains(DependencyName))
			{
				if (!InTopLevelAssets.Contains(DependencyName))
				{
					AddAssetToFileOrderRecursive(DependencyName, OutFileOrder, OutEncounteredNames, InPackageNameSet, InTopLevelAssets);
				}
			}
		}

		OutFileOrder.Add(InPackageName);
	}
}

bool FAssetRegistryGenerator::SaveAssetRegistry(const FString& SandboxPath, bool bSerializeDevelopmentAssetRegistry, 
	bool bForceNoFilter, uint64& OutDevelopmentAssetRegistryHash, const TSet<FName>& CookedPackages,
	const FAssetRegistrySerializationOptions& DevelopmentSaveOptions)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	UE_LOGF(LogAssetRegistryGenerator, Display, "Saving asset registry v%d.", FAssetRegistryVersion::Type::LatestVersion);
	
	FAssetRegistrySerializationOptions SaveOptions;
	AssetRegistry.InitializeSerializationOptions(SaveOptions, TargetPlatform, UE::AssetRegistry::ESerializationTarget::ForGame);
	SaveOptions.bKeepDevelopmentAssetRegistryTags = FParse::Param(FCommandLine::Get(), TEXT("ARKeepDevTags"));

	if (bForceNoFilter)
	{
		SaveOptions.DisableFilters();
	}

	UpdateCollectionAssetData();

	// Write development first
	if (DevelopmentSaveOptions.bSerializeAssetRegistry)
	{
		FString PlatformSandboxPath = SandboxPath.Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
		const TCHAR* DevelopmentAssetRegistryFilename = GetDevelopmentAssetRegistryFilename();
		PlatformSandboxPath.ReplaceInline(TEXT("AssetRegistry.bin"), *FString::Printf(TEXT("Metadata/%s"), DevelopmentAssetRegistryFilename));

		if (bSerializeDevelopmentAssetRegistry)
		{
			// Make a copy of the state so it can be filtered independently
			FAssetRegistryState DevelopmentState;
			DevelopmentState.InitializeFromExisting(State, DevelopmentSaveOptions);
			// No need to call FilterTags; it is called by InitializeFromExisting

			// Create development registry data, used for DLC cooks, incremental cooks, and editor viewing
			FBufferArchive64 SerializedAssetRegistry;

			DevelopmentState.Save(SerializedAssetRegistry, DevelopmentSaveOptions);

			UE::Tasks::TTask<uint64> HashTask = UE::Tasks::Launch(TEXT("HashDevelopmentAssetRegistry"),
			 [&SerializedAssetRegistry]()
			{
				FMemoryView ToHash = MakeMemoryView(SerializedAssetRegistry);
				return UE::Cook::FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(ToHash);
			});

			// Save the generated registry
			FFileHelper::SaveArrayToFile(SerializedAssetRegistry, *PlatformSandboxPath);

			uint64 WaitStartTime = FPlatformTime::Cycles64();
			uint64 DevArXxHash = HashTask.GetResult();
			double WaitTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - WaitStartTime);

			UE_LOG(LogAssetRegistryGenerator, Display, 
				TEXT("Generated development asset registry %s num assets %d, size is %5.2fkb, ")
				TEXT("XxHash64[LE] = 0x%" UINT64_X_FMT ", waited on hash %.2f seconds"),
				*PlatformSandboxPath, State.GetNumAssets(), (float)SerializedAssetRegistry.Num() / 1024.f, 
				DevArXxHash, WaitTime
				);

			OutDevelopmentAssetRegistryHash = DevArXxHash;
		}

		if (bGenerateChunks || FParse::Param(FCommandLine::Get(), TEXT("ForceGenerateAllChunksInfo")))
		{
			FString ChunkListsPath = PlatformSandboxPath.Replace(*FString::Printf(TEXT("/%s"), DevelopmentAssetRegistryFilename), TEXT(""));

			// Write out CSV file with chunking information
			GenerateAssetChunkInformationCSV(ChunkListsPath, false);
		}
	}

	// Write runtime registry, filtering further the state that we wrote out mostly unfiltered for the
	// DevelopmentAssetRegistry above.
	if (SaveOptions.bSerializeAssetRegistry)
	{
		TMap<int32, FString> ChunkBucketNames;
		TMap<int32, TSet<int32>> ChunkBuckets;
		const int32 GenericChunkBucket = -1;
		ChunkBucketNames.Add(GenericChunkBucket, FString());

		State.FilterTags(SaveOptions);

		// When chunk manifests have been generated (e.g. cook by the book) serialize 
		// an asset registry for each chunk.
		if (FinalChunkManifests.Num() > 0)
		{
			// Pass over all chunks and build a mapping of chunk index to asset registry name. All chunks that don't have a unique registry are assigned to the "generic bucket"
			// which will be written to the master asset registry in chunk 0
			for (int32 PakchunkIndex = 0; PakchunkIndex < FinalChunkManifests.Num(); ++PakchunkIndex)
			{
				FChunkPackageSet* Manifest = FinalChunkManifests[PakchunkIndex].Get();
				if (Manifest == nullptr)
				{
					continue;
				}

				bool bAddToGenericBucket = true;

				// For chunks with unique asset registry name, pakchunkIndex should equal chunkid
				FName RegistryName = UAssetManager::Get().GetUniqueAssetRegistryName(PakchunkIndex);
				if (RegistryName != NAME_None)
				{
					ChunkBuckets.FindOrAdd(PakchunkIndex).Add(PakchunkIndex);
					ChunkBucketNames.FindOrAdd(PakchunkIndex) = RegistryName.ToString();
					bAddToGenericBucket = false;
				}

				if (bAddToGenericBucket)
				{
					ChunkBuckets.FindOrAdd(GenericChunkBucket).Add(PakchunkIndex);
				}
			}

			FString SandboxPathWithoutExtension = FPaths::ChangeExtension(SandboxPath, TEXT(""));
			FString SandboxPathExtension = FPaths::GetExtension(SandboxPath);

			for (TMap<int32, TSet<int32>>::ElementType& ChunkBucketElement : ChunkBuckets)
			{
				// Prune out the development only packages, and any assets that belong in a different chunk asset registry
				FAssetRegistryState NewState;
				NewState.InitializeFromExistingAndPrune(State, CookedPackages, TSet<FName>(), ChunkBucketElement.Value, SaveOptions);

				if (!TargetPlatform->HasSecurePackageFormat())
				{
					InjectEncryptionData(NewState);
				}

				// Create runtime registry data
				FArrayWriter SerializedAssetRegistry;
				SerializedAssetRegistry.SetFilterEditorOnly(true);

				NewState.Save(SerializedAssetRegistry, SaveOptions);

				// Save the generated registry
				FString PlatformSandboxPath = SandboxPathWithoutExtension.Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
				PlatformSandboxPath += ChunkBucketNames[ChunkBucketElement.Key] + TEXT(".") + SandboxPathExtension;

				FFileHelper::SaveArrayToFile(SerializedAssetRegistry, *PlatformSandboxPath);

				FString FilenameForLog;
				if (ChunkBucketElement.Key != GenericChunkBucket)
				{
					check(ChunkBucketElement.Key < FinalChunkManifests.Num());
					check(FinalChunkManifests[ChunkBucketElement.Key]);
					FilenameForLog = FString::Printf(TEXT("[chunkbucket %i] "), ChunkBucketElement.Key);
				}
				UE_LOGF(LogAssetRegistryGenerator, Display, "Generated asset registry %lsnum assets %d, size is %5.2fkb", *FilenameForLog, NewState.GetNumAssets(), (float)SerializedAssetRegistry.Num() / 1024.f);
			}
		}
		// If no chunk manifests have been generated (e.g. cook on the fly)
		else
		{
			// Prune out the development only packages
			State.PruneAssetData(CookedPackages, TSet<FName>(), SaveOptions);

			// Create runtime registry data
			FArrayWriter SerializedAssetRegistry;
			SerializedAssetRegistry.SetFilterEditorOnly(true);

			State.Save(SerializedAssetRegistry, SaveOptions);

			// Save the generated registry
			FString PlatformSandboxPath = SandboxPath.Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
			FFileHelper::SaveArrayToFile(SerializedAssetRegistry, *PlatformSandboxPath);

			int32 NumAssets = State.GetNumAssets();
			UE_LOGF(LogAssetRegistryGenerator, Display, "Generated asset registry num assets %d, size is %5.2fkb", NumAssets, (float)SerializedAssetRegistry.Num() / 1024.f);
		}
	}
	
	return true;
}

class FPackageCookerOpenOrderVisitor : public IPlatformFile::FDirectoryVisitor
{
	const UE::Cook::FCookSandbox& SandboxFile;
	const FString& PlatformSandboxPath;
	const TSet<FStringView>& ValidExtensions;
	TMultiMap<FString, FString>& PackageExtensions;

	// Scratch variables
	FString PackageName;
	FString AssetSourcePath;
	FString StandardAssetSourcePath;
	FString BaseAssetSourcePathBuffer;
public:
	FPackageCookerOpenOrderVisitor(
		const UE::Cook::FCookSandbox& InSandboxFile,
		const FString& InPlatformSandboxPath,
		const TSet<FStringView>& InValidExtensions,
		TMultiMap<FString, FString>& OutPackageExtensions) :
		SandboxFile(InSandboxFile),
		PlatformSandboxPath(InPlatformSandboxPath),
		ValidExtensions(InValidExtensions),
		PackageExtensions(OutPackageExtensions)
	{}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (bIsDirectory)
			return true;

		const TCHAR* Filename = FilenameOrDirectory;
		FStringView UnusedFilePath, FileBaseName, FileExtension;
		FPathViews::Split(Filename, UnusedFilePath, FileBaseName, FileExtension);
		if (ValidExtensions.Contains(FileExtension))
		{
			// if the file base name ends with an optional extension, ignore it. (i.e. .o.uasset/.o.uexp etc)
			if (FileBaseName.EndsWith(FPackagePath::GetOptionalSegmentExtensionModifier()))
			{
				return true;
			}

			AssetSourcePath = SandboxFile.ConvertFromSandboxPathInPlatformRoot(Filename, PlatformSandboxPath);
			StandardAssetSourcePath = FPaths::CreateStandardFilename(AssetSourcePath);
			FString* BaseAssetSourcePath = &StandardAssetSourcePath;
			if (StandardAssetSourcePath.EndsWith(TEXT(".m.ubulk")))
			{
				// '.' is an 'invalid' character in a filename; FilenameToLongPackageName will fail.
				BaseAssetSourcePathBuffer = StandardAssetSourcePath;
				BaseAssetSourcePathBuffer.RemoveFromEnd(TEXT(".m.ubulk"));
				BaseAssetSourcePath = &BaseAssetSourcePathBuffer;
			}
			else if (HasBulkDataCookedIndexExtension(StandardAssetSourcePath))
			{
				// 10 characters equals '.XXX.ubulk' or '.XXX.uptnl' extensions
				BaseAssetSourcePathBuffer = StandardAssetSourcePath.LeftChop(10);
				BaseAssetSourcePath = &BaseAssetSourcePathBuffer;
			}

			if (FPackageName::TryConvertFilenameToLongPackageName(*BaseAssetSourcePath, PackageName))
			{
				PackageExtensions.AddUnique(PackageName, StandardAssetSourcePath);
			}
		}

		return true;
	}
};

bool FAssetRegistryGenerator::WriteCookerOpenOrder(UE::Cook::FCookSandbox& InSandboxFile)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	TSet<FName> PackageNameSet;
	TSet<FName> MapList;
	State.EnumerateAllAssets([this, &PackageNameSet, &MapList](const FAssetData& AssetData)
	{
		PackageNameSet.Add(AssetData.PackageName);

		// REPLACE WITH PRIORITY

		if (ContainsMap(AssetData.PackageName))
		{
			MapList.Add(AssetData.PackageName);
		}
	});

	FString CookerFileOrderString;
	{
		TArray<FName> TopLevelMapPackageNames;
		TArray<FName> TopLevelPackageNames;

		for (FName PackageName : PackageNameSet)
		{
			TArray<FName> Referencers;
			AssetRegistry.GetReferencers(PackageName, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

			bool bIsTopLevel = true;
			bool bIsMap = MapList.Contains(PackageName);

			if (!bIsMap && Referencers.Num() > 0)
			{
				for (auto ReferencerName : Referencers)
				{
					if (PackageNameSet.Contains(ReferencerName))
					{
						bIsTopLevel = false;
						break;
					}
				}
			}

			if (bIsTopLevel)
			{
				if (bIsMap)
				{
					TopLevelMapPackageNames.Add(PackageName);
				}
				else
				{
					TopLevelPackageNames.Add(PackageName);
				}
			}
		}

		TArray<FName> FileOrder;
		TSet<FName> EncounteredNames;
		for (FName PackageName : TopLevelPackageNames)
		{
			AddAssetToFileOrderRecursive(PackageName, FileOrder, EncounteredNames, PackageNameSet, MapList);
		}

		for (FName PackageName : TopLevelMapPackageNames)
		{
			AddAssetToFileOrderRecursive(PackageName, FileOrder, EncounteredNames, PackageNameSet, MapList);
		}
		
		// Iterate sandbox folder and generate a map from package name to cooked files
		const TArray<FStringView> ValidExtensions =
		{
			TEXT("uasset"),
			TEXT("uexp"),
			TEXT("ubulk"),
			TEXT("uptnl"),
			TEXT("umap"),
			TEXT("ufont")
		};
		const TSet<FStringView> ValidExtensionSet(ValidExtensions);

		const FString SandboxPath = InSandboxFile.GetSandboxDirectory();
		const FString Platform = TargetPlatform->PlatformName();
		FString PlatformSandboxPath = SandboxPath.Replace(TEXT("[Platform]"), *Platform);

		// ZENTODO: Change this to work with Zen
		TMultiMap<FString, FString> CookedPackageFilesMap;
		FPackageCookerOpenOrderVisitor PackageSearch(InSandboxFile, PlatformSandboxPath, ValidExtensionSet, CookedPackageFilesMap);
		IFileManager::Get().IterateDirectoryRecursively(*PlatformSandboxPath, PackageSearch);

		int32 CurrentIndex = 0;
		for (FName PackageName : FileOrder)
		{
			TArray<FString> CookedFiles;
			CookedPackageFilesMap.MultiFind(PackageName.ToString(), CookedFiles);
			CookedFiles.Sort([&ValidExtensions](const FString& A, const FString& B)
			{
				return ValidExtensions.IndexOfByKey(FPaths::GetExtension(A, true)) < ValidExtensions.IndexOfByKey(FPaths::GetExtension(B, true));
			});

			for (const FString& CookedFile : CookedFiles)
			{
				TStringBuilder<256> Line;
				Line.Appendf(TEXT("\"%s\" %i\n"), *CookedFile, CurrentIndex++);
				CookerFileOrderString.Append(Line);
			}
		}
	}

	if (CookerFileOrderString.Len())
	{
		TStringBuilder<256> OpenOrderFilename;
		if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(TargetPlatform->PlatformName()).bIsConfidential)
		{
			OpenOrderFilename.Appendf(TEXT("%sPlatforms/%s/Build/FileOpenOrder/CookerOpenOrder.log"), *FPaths::ProjectDir(), *TargetPlatform->PlatformName());
		}
		else
		{
			OpenOrderFilename.Appendf(TEXT("%sBuild/%s/FileOpenOrder/CookerOpenOrder.log"), *FPaths::ProjectDir(), *TargetPlatform->PlatformName());
		}
		FFileHelper::SaveStringToFile(CookerFileOrderString, *OpenOrderFilename);
	}

	return true;
}

bool FAssetRegistryGenerator::GetPackageDependencyChain(FName SourcePackage, FName TargetPackage, TSet<FName>& VisitedPackages, TArray<FName>& OutDependencyChain)
{	
	//avoid crashing from circular dependencies.
	if (VisitedPackages.Contains(SourcePackage))
	{		
		return false;
	}
	VisitedPackages.Add(SourcePackage);

	if (SourcePackage == TargetPackage)
	{		
		OutDependencyChain.Add(SourcePackage);
		return true;
	}

	TArray<FName> SourceDependencies;
	if (GetPackageDependencies(SourcePackage, SourceDependencies, DependencyQuery) == false)
	{		
		return false;
	}

	int32 DependencyCounter = 0;
	while (DependencyCounter < SourceDependencies.Num())
	{		
		const FName& ChildPackageName = SourceDependencies[DependencyCounter];
		if (GetPackageDependencyChain(ChildPackageName, TargetPackage, VisitedPackages, OutDependencyChain))
		{
			OutDependencyChain.Add(SourcePackage);
			return true;
		}
		++DependencyCounter;
	}
	
	return false;
}

bool FAssetRegistryGenerator::GetPackageDependencies(FName PackageName, TArray<FName>& DependentPackageNames, UE::AssetRegistry::EDependencyQuery InDependencyQuery)
{
	return AssetRegistry.GetDependencies(PackageName, DependentPackageNames, UE::AssetRegistry::EDependencyCategory::Package, InDependencyQuery);
}

bool FAssetRegistryGenerator::GatherAllPackageDependencies(FName PackageName, TArray<FName>& DependentPackageNames)
{	
	if (GetPackageDependencies(PackageName, DependentPackageNames, DependencyQuery) == false)
	{
		return false;
	}

	TSet<FName> VisitedPackages;
	VisitedPackages.Append(DependentPackageNames);

	int32 DependencyCounter = 0;
	while (DependencyCounter < DependentPackageNames.Num())
	{
		const FName& ChildPackageName = DependentPackageNames[DependencyCounter];
		++DependencyCounter;
		TArray<FName> ChildDependentPackageNames;
		if (GetPackageDependencies(ChildPackageName, ChildDependentPackageNames, DependencyQuery) == false)
		{
			return false;
		}

		for (const auto& ChildDependentPackageName : ChildDependentPackageNames)
		{
			if (!VisitedPackages.Contains(ChildDependentPackageName))
			{
				DependentPackageNames.Add(ChildDependentPackageName);
				VisitedPackages.Add(ChildDependentPackageName);
			}
		}
	}

	return true;
}

/**
 * Helper struct to get the shortest reference chain in cook dependencies from one or more PackageNames
 * to the set of packages that are hard-imported into a chunk. Constructs the distance graph for
 * all packages reachable via cookdependencies from the input InSourceSet.
 */
class FAssetRegistryGenerator::FGetShortestReferenceChain
{
public:
	void Initialize(const FAssetRegistryGenerator::FChunkPackageSet* InSourceSet);
	FString Get(FName PackageName);

private:
	static constexpr int32 NoPath = MAX_int32;
	struct FVertexData
	{
		FName NextTowardSource = NAME_None;
		int32 SourceDistance = NoPath;
	};

private:
	TMap<FName, FVertexData> Vertices;
	bool bInitialized = false;
};

bool FAssetRegistryGenerator::GenerateAssetChunkInformationCSV(const FString& OutputPath, bool bWriteIndividualFiles)
{
	FString TmpString, TmpStringChunks;
	ANSICHAR HeaderText[] = "ChunkID, Package Name, Class Type, Hard or Soft Chunk, File Size, Other Chunks\n";

	// Get the list of assets sorted by packagename, so the CSV is consistent between cooks.
	TArray<FAssetData*> AssetDataList = State.GetMutableAssetsSortedByObjectPath();

	// Create file for all chunks
	TUniquePtr<FArchive> AllChunksFile(IFileManager::Get().CreateFileWriter(*FPaths::Combine(*OutputPath, TEXT("AllChunksInfo.csv"))));
	if (!AllChunksFile.IsValid())
	{
		return false;
	}

	AllChunksFile->Serialize(HeaderText, sizeof(HeaderText)-1);

	const TMap<int32, FString> PakChunkIdToStringOverride = GetPakChunkIdToStringOverrideMap();
	// Create file for each chunk if needed
	TArray<TUniquePtr<FArchive>> ChunkFiles;
	if (bWriteIndividualFiles)
	{
		for (int32 PakchunkIndex = 0; PakchunkIndex < FinalChunkManifests.Num(); ++PakchunkIndex)
		{
			const FString PakChunkName = GetPakChunkNameForId(PakChunkIdToStringOverride, PakchunkIndex);
			FArchive* ChunkFile = IFileManager::Get().CreateFileWriter(*FPaths::Combine(*OutputPath, *FString::Printf(TEXT("Chunks%sInfo.csv"), *PakChunkName)));
			if (ChunkFile == nullptr)
			{
				return false;
			}
			ChunkFile->Serialize(HeaderText, sizeof(HeaderText)-1);
			ChunkFiles.Add(TUniquePtr<FArchive>(ChunkFile));
		}
	}

	TMap<int32, FGetShortestReferenceChain> ReferenceChainFinderForChunk;

	for (const FAssetData* AssetDataPtr : AssetDataList)
	{
		const FAssetData& AssetData = *AssetDataPtr;
		const FAssetPackageData* PackageData = State.GetAssetPackageData(AssetData.PackageName);

		// Add only assets that have actually been cooked and belong to any chunk and that have a file size
		const FAssetData::FChunkArrayView ChunkIDs = AssetData.GetChunkIDs();
		if (PackageData != nullptr && ChunkIDs.Num() > 0 && PackageData->DiskSize > 0)
		{
			for (int32 PakchunkIndex : ChunkIDs)
			{
				const int64 FileSize = PackageData->DiskSize;
				FString SoftChain;
				bool bHardChunk = false;
				if (PakchunkIndex < ChunkManifests.Num())
				{
					bHardChunk = ChunkManifests[PakchunkIndex] && ChunkManifests[PakchunkIndex]->Contains(AssetData.PackageName);

					if (!bHardChunk)
					{
						FGetShortestReferenceChain& ChainFinder = ReferenceChainFinderForChunk.FindOrAdd(PakchunkIndex);
						ChainFinder.Initialize(ChunkManifests[PakchunkIndex].Get());
						SoftChain = ChainFinder.Get(AssetData.PackageName);
					}
				}

				// Build "other chunks" string or None if not part of
				TmpStringChunks.Empty(64);
				for (int32 OtherChunk : ChunkIDs)
				{
					if (OtherChunk != PakchunkIndex)
					{
						const FString OtherPakChunkName = GetPakChunkNameForId(PakChunkIdToStringOverride, OtherChunk);
						TmpString = FString::Printf(TEXT("%s "), *OtherPakChunkName);
					}
				}

				// Build csv line
				const FString PakChunkName = GetPakChunkNameForId(PakChunkIdToStringOverride, PakchunkIndex);
				TmpString = FString::Printf(TEXT("%s,%s,%s,%s,%lld,%s\n"),
					*PakChunkName,
					*AssetData.PackageName.ToString(),
					*AssetData.AssetClassPath.ToString(),
					bHardChunk ? TEXT("Hard") : *SoftChain,
					FileSize,
					ChunkIDs.Num() == 1 ? TEXT("None") : *TmpStringChunks
				);

				// Write line to all chunks file and individual chunks files if requested
				{
					auto Src = StringCast<ANSICHAR>(*TmpString, TmpString.Len());
					AllChunksFile->Serialize((ANSICHAR*)Src.Get(), Src.Length() * sizeof(ANSICHAR));
					
					if (bWriteIndividualFiles)
					{
						ChunkFiles[PakchunkIndex]->Serialize((ANSICHAR*)Src.Get(), Src.Length() * sizeof(ANSICHAR));
					}
				}
			}
		}
	}

	return true;
}

void FAssetRegistryGenerator::AddPackageToManifest(const FString& PackageSandboxPath, FName PackageName, int32 ChunkId)
{
	HighestChunkId = ChunkId > HighestChunkId ? ChunkId : HighestChunkId;
	int32 PakchunkIndex = GetPakchunkIndex(ChunkId);

	if (PakchunkIndex >= ChunkManifests.Num())
	{
		ChunkManifests.AddDefaulted(PakchunkIndex - ChunkManifests.Num() + 1);
	}
	if (!ChunkManifests[PakchunkIndex])
	{
		ChunkManifests[PakchunkIndex].Reset(new FChunkPackageSet());
	}
	ChunkManifests[PakchunkIndex]->Add(PackageName, PackageSandboxPath);
	// Now that the package has been assigned to a chunk, remove it from the unassigned set.
	UnassignedPackageSet.Remove(PackageName);
}


void FAssetRegistryGenerator::RemovePackageFromManifest(FName PackageName, int32 ChunkId)
{
	int32 PakchunkIndex = GetPakchunkIndex(ChunkId);

	if (ChunkManifests[PakchunkIndex])
	{
		ChunkManifests[PakchunkIndex]->Remove(PackageName);
	}
}

void FAssetRegistryGenerator::SubtractParentChunkPackagesFromChildChunks(const FChunkDependencyTreeNode& Node,
	const TSet<FName>& CumulativeParentPackages, TArray<TArray<FName>>& OutPackagesMovedBetweenChunks)
{
	if (FinalChunkManifests.Num() <= Node.ChunkID || !FinalChunkManifests[Node.ChunkID])
	{
		return;
	}
	FChunkPackageSet& NodeManifest = *FinalChunkManifests[Node.ChunkID];
	for (FName PackageName : CumulativeParentPackages)
	{
		// Remove any assets belonging to our parents.
		if (NodeManifest.Remove(PackageName) > 0)
		{
			OutPackagesMovedBetweenChunks[Node.ChunkID].Add(PackageName);
			UE_LOGF(LogAssetRegistryGenerator, Verbose, "Removed %ls from chunk %i because it is duplicated in another chunk.",
				*PackageName.ToString(), Node.ChunkID);
		}
	}

	if (!Node.ChildNodes.Num())
	{
		return;
	}

	// Add the current Chunk's assets
	TSet<FName> CumulativePackages;
	if (!NodeManifest.IsEmpty())
	{
		CumulativePackages.Reserve(CumulativeParentPackages.Num() + NodeManifest.Num());
		CumulativePackages.Append(CumulativeParentPackages);
		for (const TPair<FName, FString>& Pair : NodeManifest)
		{
			CumulativePackages.Add(Pair.Key);
		}
	}

	const TSet<FName>& NewRecursiveParentPackages = CumulativePackages.Num() ? CumulativePackages : CumulativeParentPackages;
	for (const FChunkDependencyTreeNode& ChildNode : Node.ChildNodes)
	{
		SubtractParentChunkPackagesFromChildChunks(ChildNode, NewRecursiveParentPackages, OutPackagesMovedBetweenChunks);
	}
}

bool FAssetRegistryGenerator::CheckChunkAssetsAreNotInChild(const FChunkDependencyTreeNode& Node)
{
	for (const FChunkDependencyTreeNode& ChildNode : Node.ChildNodes)
	{
		if (!CheckChunkAssetsAreNotInChild(ChildNode))
		{
			return false;
		}
	}

	if (!(FinalChunkManifests.Num() > Node.ChunkID && FinalChunkManifests[Node.ChunkID]))
	{
		return true;
	}

	FChunkPackageSet& ParentManifest = *FinalChunkManifests[Node.ChunkID];
	for (const FChunkDependencyTreeNode& ChildNode : Node.ChildNodes)
	{
		if (FinalChunkManifests.Num() > ChildNode.ChunkID && FinalChunkManifests[ChildNode.ChunkID])
		{
			FChunkPackageSet& ChildManifest = *FinalChunkManifests[ChildNode.ChunkID];
			for (const TPair<FName,FString>& ParentPair : ParentManifest)
			{
				if (ChildManifest.Find(ParentPair.Key))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FAssetRegistryGenerator::AddPackageToChunk(FChunkPackageSet& ThisPackageSet, FName InPkgName,
	const FString& InSandboxFile, int32 PakchunkIndex, UE::Cook::FCookSandbox& SandboxPlatformFile)
{
	ThisPackageSet.Add(InPkgName, InSandboxFile);
}

FString FAssetRegistryGenerator::GetChunkManifestDirectory(UCookOnTheFlyServer& COTFS) const
{
	return COTFS.GetCookMetadataOutputFolder(TargetPlatform) / TEXT("ChunkManifest");
}

void FAssetRegistryGenerator::FixupPackageDependenciesForChunks(UE::Cook::FCookSandbox& InSandboxFile)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER(FixupPackageDependenciesForChunks);

	// Clear any existing manifests from the final array
	FinalChunkManifests.Empty(ChunkManifests.Num());

	for (int32 PakchunkIndex = 0, MaxPakchunk = ChunkManifests.Num(); PakchunkIndex < MaxPakchunk; ++PakchunkIndex)
	{
		FChunkPackageSet& FinalManifest = *FinalChunkManifests.Emplace_GetRef(new FChunkPackageSet());
		if (!ChunkManifests[PakchunkIndex])
		{
			continue;
		}
		
		for (const TPair<FName,FString>& Pair : *ChunkManifests[PakchunkIndex])
		{
			AddPackageToChunk(FinalManifest, Pair.Key, Pair.Value, PakchunkIndex, InSandboxFile);
		}
	}

	FConfigFile PlatformIniFile;
	FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Engine"), true, *TargetPlatform->IniPlatformName());
	bool bSkipResolveChunkDependencyGraph = false;
	{
		const bool bIsSkipResolveChunkDependencyGraphConfigured = PlatformIniFile.GetBool(TEXT("/Script/CoreUObject.ChunkDependencyInfo"), TEXT("bSkipResolveChunkDependencyGraph"), bSkipResolveChunkDependencyGraph);

		// Handle deprecated INI section since this class has moved from UnrealEd into CoreUObject module
		bool bDeprecatedSkipResolveChunkDependencyGraph = false;
		if (PlatformIniFile.GetBool(TEXT("/Script/UnrealEd.ChunkDependencyInfo"), TEXT("bSkipResolveChunkDependencyGraph"), bDeprecatedSkipResolveChunkDependencyGraph))
		{
			UE_LOGF(LogAssetRegistryGenerator, Warning,
				"Configuration [/Script/UnrealEd.ChunkDependencyInfo]:bSkipResolveChunkDependencyGraph is deprecated since UE 5.8; Use [/Script/CoreUObject.ChunkDependencyInfo]:bSkipResolveChunkDependencyGraph instead!");
			if (bIsSkipResolveChunkDependencyGraphConfigured)
			{
				if (bSkipResolveChunkDependencyGraph != bDeprecatedSkipResolveChunkDependencyGraph)
				{
					UE_LOGF(LogAssetRegistryGenerator, Error,
						"Conflict between deprecated config [/Script/UnrealEd.ChunkDependencyInfo]:bSkipResolveChunkDependencyGraph (%ls) and new config [/Script/CoreUObject.ChunkDependencyInfo]:bSkipResolveChunkDependencyGraph (%ls)",
						bDeprecatedSkipResolveChunkDependencyGraph ? TEXT("true") : TEXT("false"),
						bSkipResolveChunkDependencyGraph ? TEXT("true") : TEXT("false"));
				}
			}
			else
			{
				bSkipResolveChunkDependencyGraph = bDeprecatedSkipResolveChunkDependencyGraph;
			}
		}
	}

	const FChunkDependencyTreeNode* ChunkDepGraph = DependencyInfo.GetOrBuildChunkDependencyGraph(!bSkipResolveChunkDependencyGraph ? HighestChunkId : 0);

	//Once complete, Add any remaining assets (that are not assigned to a chunk) to the first chunk.
	if (FinalChunkManifests.Num() == 0)
	{
		FinalChunkManifests.Emplace(new FChunkPackageSet());
	}
	FChunkPackageSet& Chunk0Manifest = *FinalChunkManifests[0];
	
	// Copy the remaining assets
	FChunkPackageSet RemainingAssets = UnassignedPackageSet; // Loop removes elements from UnassignedPackageSet
	for (const TPair<FName,FString>& Pair : RemainingAssets)
	{
		AddPackageToChunk(Chunk0Manifest, Pair.Key, Pair.Value, 0, InSandboxFile);
	}

	if (!CheckChunkAssetsAreNotInChild(*ChunkDepGraph))
	{
		UE_LOGF(LogAssetRegistryGenerator, Log, "Initial scan of chunks found duplicate assets in graph children");
	}
		
	TArray<TArray<FName>> PackagesRemovedFromChunks;
	PackagesRemovedFromChunks.AddDefaulted(ChunkManifests.Num());

	// Recursively remove child chunk's redundant copies of parent chunk's packages.
	// This has to be done after all remaining assets were added to chunk 0 above, since all chunks
	// are children of chunk 0.
	SubtractParentChunkPackagesFromChildChunks(*ChunkDepGraph, TSet<FName>(), PackagesRemovedFromChunks);

	for (int32 PakchunkIndex = 0, MaxPakchunk = ChunkManifests.Num(); PakchunkIndex < MaxPakchunk; ++PakchunkIndex)
	{
		const int32 ChunkManifestNum = ChunkManifests[PakchunkIndex] ? ChunkManifests[PakchunkIndex]->Num() : 0;
		check(PakchunkIndex < FinalChunkManifests.Num() && FinalChunkManifests[PakchunkIndex]);
		const int32 FinalChunkManifestNum = FinalChunkManifests[PakchunkIndex]->Num();
		if (ChunkManifestNum != 0 || FinalChunkManifestNum != 0)
		{
			UE_LOGF(LogAssetRegistryGenerator, Verbose, "Chunk: %i, Started with %i packages, Final after dependency resolve: %i",
				PakchunkIndex, ChunkManifestNum, FinalChunkManifestNum);
		}
	}
	
	// Fix up the data in the FAssetRegistryState to reflect this chunk layout
	for (int32 PakchunkIndex = 0 ; PakchunkIndex < FinalChunkManifests.Num(); ++PakchunkIndex)
	{
		check(FinalChunkManifests[PakchunkIndex]);
		for (const TPair<FName, FString>& Asset : *FinalChunkManifests[PakchunkIndex])
		{
			State.EnumerateMutableAssetsByPackageName(Asset.Key, [PakchunkIndex](FAssetData* AssetData)
				{
					// Chunk Ids are safe to modify in place
					AssetData->AddChunkID(PakchunkIndex);
					return true;
				});
		}
	}
}

void FAssetRegistryGenerator::FGetShortestReferenceChain::Initialize(const FAssetRegistryGenerator::FChunkPackageSet* SourceSet)
{
	if (bInitialized)
	{
		return;
	}
	bInitialized = true;
	if (!SourceSet)
	{
		return;
	}

	IAssetRegistry& AssetRegistryShadowedVar = IAssetRegistry::GetChecked();
	Vertices.Reset();

	// BFS the entire graph of dependencies outward from the SourceSet to construct the distance graph
	TRingBuffer<FName> BFSQueue;
	for (const TPair<FName, FString>& Pair : *SourceSet)
	{
		BFSQueue.Add(Pair.Key);
		Vertices.FindOrAdd(Pair.Key).SourceDistance = 0;
	}

	TArray<FName> AssetDependencies;
	while (!BFSQueue.IsEmpty())
	{
		FName CurrentVertex = BFSQueue.PopFrontValue();
		int32 NextDistance = Vertices[CurrentVertex].SourceDistance + 1;

		AssetDependencies.Reset();
		AssetRegistryShadowedVar.GetDependencies(CurrentVertex, AssetDependencies);
		for (FName NextVertex : AssetDependencies)
		{
			FVertexData& NextData = Vertices.FindOrAdd(NextVertex);
			if (NextData.SourceDistance > NextDistance)
			{
				NextData.SourceDistance = NextDistance;
				NextData.NextTowardSource = CurrentVertex;
				BFSQueue.Add(NextVertex);
			}
		}
	}
}

FString FAssetRegistryGenerator::FGetShortestReferenceChain::Get(FName TargetPackageName)
{
	FName CurrentVertex = TargetPackageName;
	FVertexData* VertexData = &Vertices.FindOrAdd(CurrentVertex);
	if (VertexData->SourceDistance == NoPath)
	{
		return TEXT("Soft: Unknown reference chain. Soft From Unassigned Package?");
	}
	if (VertexData->SourceDistance == 0)
	{
		return FString(TEXT("Hard"));
	}

	TArray<FName> Chain;
	Chain.Add(TargetPackageName);
	int32 MaxPossibleLength = Vertices.Num();
	for (; VertexData->SourceDistance != 0;)
	{
		CurrentVertex = VertexData->NextTowardSource;
		checkf(!CurrentVertex.IsNone(), TEXT("Distance graph has incomplete path; this should be impossible."));
		Chain.Add(CurrentVertex);
		checkf(Chain.Num() <= MaxPossibleLength, TEXT("Cycle in Distance graph; this should be impossible."));
		VertexData = &Vertices.FindOrAdd(CurrentVertex);
	}
	check(Chain.Num() > 1); // Loop runs at least once

	TStringBuilder<1024> Result;
	Result << TEXT("Soft: ") << Chain[Chain.Num() - 1];
	for (int32 Index = Chain.Num() - 2; Index >= 0; --Index)
	{
		Result << TEXT("->") << Chain[Index];
	}
	return FString(Result);
}

int32 FAssetRegistryGenerator::GetPakchunkIndex(int32 ChunkId) const
{
	const int32* PakChunkId = ChunkIdPakchunkIndexMapping.Find(ChunkId);
	if (PakChunkId)
	{
		check(*PakChunkId >= 0);
		return *PakChunkId;
	}

	return ChunkId;
}

void FAssetRegistryGenerator::UpdateAssetRegistryData(FName PackageName, const UPackage* Package,
	UE::Cook::ECookResult CookResult, FSavePackageResultStruct* SavePackageResult,
	TOptional<TArray<FAssetData>>&& AssetDatasFromSave, TOptional<FAssetPackageData>&& OverrideAssetPackageData,
	TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies, UCookOnTheFlyServer& COTFS)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	UE::Cook::FAssetRegistryPackageMessage Message = RecordUpdateAssetRegistryData(PackageName, TargetPlatform,
		CookResult, SavePackageResult, MoveTemp(AssetDatasFromSave), MoveTemp(OverrideAssetPackageData),
		MoveTemp(OverridePackageDependencies));
	StoreUpdateAssetRegistryData(PackageName, MoveTemp(Message));
}

UE::Cook::FAssetRegistryPackageMessage FAssetRegistryGenerator::RecordUpdateAssetRegistryData(FName PackageName,
	const ITargetPlatform* TargetPlatform, UE::Cook::ECookResult CookResult,
	FSavePackageResultStruct* SavePackageResult,
	TOptional<TArray<FAssetData>>&& AssetDatasFromSave,
	TOptional<FAssetPackageData>&& OverrideAssetPackageData,
	TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies)
{
	UE::Cook::FAssetRegistryPackageMessage Message;

	// When the AssetDatas were not calculated by SavePackage, copy them instead from the on-disk AssetRegistry.
	if (!AssetDatasFromSave)
	{
		AssetDatasFromSave.Emplace();
		IAssetRegistry::GetChecked().GetAssetsByPackageName(PackageName, *AssetDatasFromSave,
			true /* bIncludeOnlyDiskAssets */, false /* SkipARFilteredAssets */);
	}

	bool bSaveSucceeded = CookResult == UE::Cook::ECookResult::Succeeded;
	if (bSaveSucceeded)
	{
		check(SavePackageResult);

		// Set the PackageFlags to the recorded value from SavePackage
		Message.PackageFlags = SavePackageResult->SerializedPackageFlags;
		Message.DiskSize = SavePackageResult->TotalFileSize;
	}
	else
	{
		// Set the package flags to zero to indicate that the package failed to save
		Message.PackageFlags = 0;
		// Set DiskSize (previous value was disksize in the WorkspaceDomain) to a negative number to indicate the
		// cooked file does not exist. The magnitude of the negative value indicates CookResult.
		Message.DiskSize = CookResultToDiskSize(CookResult);
	}

	Message.PackageName = PackageName;
	Message.TargetPlatform = TargetPlatform;
	Message.OverrideAssetPackageData = MoveTemp(OverrideAssetPackageData);
	Message.OverridePackageDependencies = MoveTemp(OverridePackageDependencies);
	Message.AssetDatas = MoveTemp(*AssetDatasFromSave);

	return Message;
}

void FAssetRegistryGenerator::StoreUpdateAssetRegistryData(FName PackageName, UE::Cook::FAssetRegistryPackageMessage&& Message)
{
	// Copy latest data for all Assets in the package into the cooked registry. This should be done even
	// if not successful so that editor-only packages are recorded as well.
	for (FAssetData& AssetData : Message.AssetDatas)
	{
		AssetData.PackageFlags = Message.PackageFlags;
		State.UpdateAssetData(MoveTemp(AssetData), true /* bCreateIfNotExists */);
	}
	bool bCookSucceeded = Message.DiskSize >= 0;
	RemovePackageAssetsThatDoNotExistInCookedPackage(PackageName, Message.AssetDatas, bCookSucceeded);

	// Copy AssetPackageData and Dependencies
	FAssetPackageData* AssetPackageData = GetAssetPackageData(PackageName);
	// Copy the Dependencies and PackageData from the global AssetRegistry
	TOptional<FAssetPackageData> GlobalPackageData = AssetRegistry.GetAssetPackageDataCopy(PackageName);
	if (GlobalPackageData)
	{
		*AssetPackageData = MoveTemp(*GlobalPackageData);
	}
	TArray<FAssetDependency> GlobalDependencies;
	AssetRegistry.GetDependencies(PackageName, GlobalDependencies, UE::AssetRegistry::EDependencyCategory::All);
	State.SetDependencies(FAssetIdentifier(PackageName), GlobalDependencies, UE::AssetRegistry::EDependencyCategory::All);

	// Override the global AssetRegistry's data with the data discovered during save
	if (Message.OverrideAssetPackageData)
	{
		// Generated packages pass in their own AssetPackageData
		*AssetPackageData = MoveTemp(*Message.OverrideAssetPackageData);
	}
	AssetPackageData->DiskSize = Message.DiskSize;
	// AssetPackageData->CookedHash is assigned during CookByTheBookFinished

	if (Message.OverridePackageDependencies)
	{
		// Generated and Generator packages pass in their own dependencies
		SetOverridePackageDependencies(PackageName, *Message.OverridePackageDependencies);
	}
}

void FAssetRegistryGenerator::RemovePackageAssetsThatDoNotExistInCookedPackage(FName PackageName,
	const TArray<FAssetData>& AssetDatasFromSave, bool bCookSucceeded)
{
	TArray<FSoftObjectPath> AssetsToRemove;
	// The usual number of assets is 1. For that case, and all the way up to say 5, it is faster to make
	// n^2 cheap compares than to construct a TSet and make n hash lookups.
	constexpr int32 MaxSizeForLinearCompare = 5;
	if (AssetDatasFromSave.Num() <= MaxSizeForLinearCompare)
	{
		State.EnumerateAssetsByPackageName(PackageName,
			[&AssetsToRemove, &AssetDatasFromSave](const FAssetData* ExistingAsset)
			{
				FSoftObjectPath ExistingObjectPath = ExistingAsset->GetSoftObjectPath();
				if (!AssetDatasFromSave.ContainsByPredicate([&ExistingObjectPath](const FAssetData& AssetFromSave)
					{
						return ExistingObjectPath == AssetFromSave.GetSoftObjectPath();
					}))
				{
					AssetsToRemove.Add(ExistingObjectPath);
				}
				return true;
			});
	}
	else
	{
		TSet<FSoftObjectPath> SetOfAssetsFromSave;
		for (const FAssetData& AssetData : AssetDatasFromSave)
		{
			SetOfAssetsFromSave.Add(AssetData.GetSoftObjectPath());
		}
		State.EnumerateAssetsByPackageName(PackageName,
			[&AssetsToRemove, &SetOfAssetsFromSave](const FAssetData* ExistingAsset)
			{
				FSoftObjectPath ExistingObjectPath = ExistingAsset->GetSoftObjectPath();
				if (!SetOfAssetsFromSave.Contains(ExistingObjectPath))
				{
					AssetsToRemove.Add(ExistingObjectPath);
				}
				return true;
			});
	}
	for (const FSoftObjectPath& AssetToRemove : AssetsToRemove)
	{
		bool bUnusedRemovedAssetData;
		bool bUnusedRemovedPackageData;
		if (bCookSucceeded)
		{
			// Give a log message to diagnose issues in case deleting the Asset is unexpected.
			// But the log fires frequently in some cases of __ExternalObject__ packages that are not saved; suppress
			// it when the package is not saved since deletion of assets in the package is unimportant for the
			// non-runtime packages.
			UE_LOGF(LogAssetRegistryGenerator, Log,
				"Removing Asset %ls from the runtime AssetRegistry; it does not exist in the cooked package.",
				*WriteToString<256>(AssetToRemove));
		}
		State.RemoveAssetData(AssetToRemove, false /* bRemoveDependencyData */,
			bUnusedRemovedAssetData, bUnusedRemovedPackageData);
	}
}

void FAssetRegistryGenerator::SetOverridePackageDependencies(FName PackageName, TConstArrayView<FAssetDependency> OverridePackageDependencies)
{
#if DO_CHECK
	for (FAssetDependency Dependency : OverridePackageDependencies)
	{
		check(Dependency.Category == UE::AssetRegistry::EDependencyCategory::Package);
	}
#endif
	State.SetDependencies(FAssetIdentifier(PackageName), OverridePackageDependencies, UE::AssetRegistry::EDependencyCategory::Package);
}

void FAssetRegistryGenerator::UpdateAssetRegistryData(UE::Cook::FMPCollectorServerMessageContext& Context,
	UE::Cook::FAssetRegistryPackageMessage&& Message, UCookOnTheFlyServer& COTFS)
{
	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	const FName PackageName = Context.GetPackageName();
	check(!PackageName.IsNone());

	StoreUpdateAssetRegistryData(PackageName, MoveTemp(Message));
}

namespace UE::Cook
{

FAssetRegistryReporterRemote::FAssetRegistryReporterRemote(FCookWorkerClient& InClient, const ITargetPlatform* InTargetPlatform)
	: Client(InClient)
	, TargetPlatform(InTargetPlatform)
{
}

void FAssetRegistryReporterRemote::UpdateAssetRegistryData(FName PackageName, const UPackage* Package,
	UE::Cook::ECookResult CookResult, FSavePackageResultStruct* SavePackageResult,
	TOptional<TArray<FAssetData>>&& AssetDatasFromSave, TOptional<FAssetPackageData>&& OverrideAssetPackageData,
	TOptional<TArray<FAssetDependency>>&& OverridePackageDependencies, UCookOnTheFlyServer& COTFS)
{
	FAssetRegistryPackageMessage Message = FAssetRegistryGenerator::RecordUpdateAssetRegistryData(
		PackageName, TargetPlatform, CookResult, SavePackageResult,
		MoveTemp(AssetDatasFromSave), MoveTemp(OverrideAssetPackageData), MoveTemp(OverridePackageDependencies));

	// Send the message to the director
	FCbWriter Writer;
	Writer.BeginObject();
	Message.Write(Writer);
	Writer.EndObject();

	PackageUpdateMessages.Add(PackageName, Writer.Save().AsObject());
}

void FAssetRegistryPackageMessage::Write(FCbWriter& Writer) const
{
	check(!PackageName.IsNone());

	Writer.BeginArray("A");
	for (const FAssetData& AssetData : AssetDatas)
	{
		check(AssetData.PackageName == PackageName && !AssetData.AssetName.IsNone()); // We replicate only regular Assets of the form PackageName.AssetName
		AssetData.NetworkWrite(Writer, false /* bWritePackageName */);

	}
	Writer.EndArray();
	if (OverrideAssetPackageData)
	{
		Writer.SetName("P");
		OverrideAssetPackageData->NetworkWrite(Writer);
	}
	if (OverridePackageDependencies)
	{
		Writer << "D" << *OverridePackageDependencies;
	}
	Writer << "F" << PackageFlags;
	Writer << "S" << DiskSize;
}

bool FAssetRegistryPackageMessage::TryRead(FCbObjectView Object)
{
	check(!PackageName.IsNone());

	LLM_SCOPE_BYTAG(Cooker_GeneratedAssetRegistry);
	FCbFieldView AssetDatasField = Object["A"];
	FCbArrayView AssetDatasArray = AssetDatasField.AsArrayView();
	if (AssetDatasField.HasError())
	{
		return false;
	}
	AssetDatas.Reset(IntCastChecked<int32>(AssetDatasArray.Num()));
	for (FCbFieldView ElementField : AssetDatasArray)
	{
		FAssetData& AssetData = AssetDatas.Emplace_GetRef();
		if (!AssetData.TryNetworkRead(ElementField, false /* bReadPackageName */, PackageName))
		{
			return false;
		}
	}
	OverrideAssetPackageData.Reset();
	FCbFieldView OverrideAssetPackageDataField = Object["P"];
	if (OverrideAssetPackageDataField.HasValue())
	{
		OverrideAssetPackageData.Emplace();
		if (!OverrideAssetPackageData->TryNetworkRead(OverrideAssetPackageDataField))
		{
			return false;
		}
	}

	OverridePackageDependencies.Reset();
	FCbFieldView OverridePackageDependenciesField = Object["D"];
	if (OverridePackageDependenciesField.HasValue())
	{
		OverridePackageDependencies.Emplace();
		if (!LoadFromCompactBinary(OverridePackageDependenciesField, *OverridePackageDependencies))
		{
			return false;
		}
	}

	if (!LoadFromCompactBinary(Object["F"], PackageFlags))
	{
		return false;
	}
	if (!LoadFromCompactBinary(Object["S"], DiskSize))
	{
		return false;
	}
	return true;
}

FGuid FAssetRegistryPackageMessage::MessageType(TEXT("0588DCCEBF1742399EC1E011FC97E4DC"));

FAssetRegistryMPCollector::FAssetRegistryMPCollector(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{
}

void FAssetRegistryMPCollector::ClientTickPackage(FMPCollectorClientTickPackageContext& Context)
{
	bool bLoggedWarning = false;
	for (const FMPCollectorClientTickPackageContext::FPlatformData& ContextPlatformData : Context.GetPlatformDatas())
	{
		if (ContextPlatformData.CookResults == ECookResult::Invalid)
		{
			continue;
		}

		const ITargetPlatform* TargetPlatform = ContextPlatformData.TargetPlatform;
		FPlatformData* PlatformData = COTFS.PlatformManager->GetPlatformData(TargetPlatform);
		FAssetRegistryReporterRemote& RegistryReporter = static_cast<FAssetRegistryReporterRemote&>(*PlatformData->RegistryReporter);
		FCbObject Message;

		if (!RegistryReporter.PackageUpdateMessages.RemoveAndCopyValue(Context.GetPackageName(), Message))
		{
			// For a failed package, UpdateAssetRegistryData might never have been called. Silently skip it and do not send a message.
			if (ContextPlatformData.CookResults == ECookResult::Succeeded)
			{
				// For a successful package, UpdateAssetRegistryData should have been called
				UE_CLOGF(!bLoggedWarning, LogAssetRegistryGenerator, Warning,
					"ClientTickPackage was called for package %ls, platform %ls, but UpdateAssetRegistryData was not called for that package. We will not have up to date AssetDataTags in the generated AssetRegistry.",
					*Context.GetPackageName().ToString(), *ContextPlatformData.TargetPlatform->PlatformName());
				bLoggedWarning = true;
			}
		}
		else
		{
			Context.AddPlatformMessage(TargetPlatform, MoveTemp(Message));
		}
	}
}

void FAssetRegistryMPCollector::ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	FName PackageName = Context.GetPackageName();
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	check(PackageName.IsValid() && TargetPlatform);

	FAssetRegistryPackageMessage ARMessage;
	ARMessage.PackageName = PackageName;
	ARMessage.TargetPlatform = TargetPlatform;
	if (!ARMessage.TryRead(Message))
	{
		UE_LOGF(LogCook, Error, "Corrupt AssetRegistryPackageMessage received from CookWorker %d. It will be ignored.",
			Context.GetProfileId());
	}
	else
	{
		FAssetRegistryGenerator* RegistryGenerator = COTFS.PlatformManager->GetPlatformData(TargetPlatform)->RegistryGenerator.Get();
		check(RegistryGenerator); // The TargetPlatform came from OrderedSessionPlatforms, and the RegistryGenerator should exist for any of those platforms
		RegistryGenerator->UpdateAssetRegistryData(Context, MoveTemp(ARMessage), COTFS);
	}
}

}

bool FAssetRegistryGenerator::UpdateAssetPackageFlags(const FName& PackageName, const uint32 PackageFlags)
{
	return State.UpdateAssetDataPackageFlags(PackageName, PackageFlags);
}

void FAssetRegistryGenerator::InitializeChunkIdPakchunkIndexMapping()
{
	FConfigFile PlatformIniFile;
	FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Game"), true, *TargetPlatform->IniPlatformName());
	TArray<FString> ChunkMapping;
	PlatformIniFile.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("ChunkIdPakchunkIndexMapping"), ChunkMapping);

	FPlatformMisc::ParseChunkIdPakchunkIndexMapping(ChunkMapping, ChunkIdPakchunkIndexMapping);

	// Validate ChunkIdPakchunkIndexMapping
	TArray<int32> AllChunkIDs;
	ChunkIdPakchunkIndexMapping.GetKeys(AllChunkIDs);
	for (int32 ChunkID : AllChunkIDs)
	{
		if(UAssetManager::Get().GetChunkEncryptionKeyGuid(ChunkID).IsValid()
			|| UAssetManager::Get().GetUniqueAssetRegistryName(ChunkID) != NAME_None)
		{
			UE_LOGF(LogAssetRegistryGenerator, Error, "Chunks with encryption key guid or unique assetregistry name (Chunk %d) can not be mapped with ChunkIdPakchunkIndexMapping.  Mapping is removed.", ChunkID);
			ChunkIdPakchunkIndexMapping.Remove(ChunkID);
		}
	}
}
#undef LOCTEXT_NAMESPACE