// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/AssetSizeQueryCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/WildcardString.h"
#include "Templates/Greater.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetSizeQueryCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogAssetSize, Display, All);

UAssetSizeQueryCommandlet::UAssetSizeQueryCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

}

enum class EOutputCSVType : uint8
{
	None,
	Classes,
	Assets
};

int32 UAssetSizeQueryCommandlet::Main(const FString& FullCommandLine)
{
	FString FileName;
	if (FParse::Value(*FullCommandLine, TEXT("AssetRegistry="), FileName) == false)
	{
		UE_LOGF(LogAssetSize, Error, "No AssetRegistry specified.");
		UE_LOGF(LogAssetSize, Display, "");
		UE_LOGF(LogAssetSize, Display, "AssetSizeQueryCommandlet");
		UE_LOGF(LogAssetSize, Display, "");
		UE_LOGF(LogAssetSize, Display, "Used to find asset type size breakdown in a staged/compressed project.");
		UE_LOGF(LogAssetSize, Display, "");
		UE_LOGF(LogAssetSize, Display, "Params:");
		UE_LOGF(LogAssetSize, Display, "    -AssetRegistry=path                     Provides the path to the Development asset registry. This");
		UE_LOGF(LogAssetSize, Display, "                                            asset registry must have staging size metadata via ProjectSettings->");
		UE_LOGF(LogAssetSize, Display, "                                            Packaging->WriteBackMetadataToAssetRegistry.");
		UE_LOGF(LogAssetSize, Display, "    -Filter=wildcard                        (optional) Filters the list of assets using a wildcard match.");
		UE_LOGF(LogAssetSize, Display, "    -FilterClass=class                      (optional) Filters the list of assets to the given class. e.g. Texture2D not /Script/Engine.Texture2D");
		UE_LOGF(LogAssetSize, Display, "    -Show=#                                 (optional) Shows only the top # classes, sorted on size (0 is all, default 10).");
		UE_LOGF(LogAssetSize, Display, "    -CSV=path                               (optional) Output the filtered per class infomation to the given CSV file.");
		UE_LOGF(LogAssetSize, Display, "    -CSVType=(Assets,Classes)               (optional) Specifies whether to write the class summary or all matching assets to the csv file.");
		UE_LOGF(LogAssetSize, Display, "");
		UE_LOGF(LogAssetSize, Display, "Note that a project must be staged in order to determine compressed sizes, and that additionally");
		UE_LOGF(LogAssetSize, Display, "some platforms stage in a platform specific manner that precludes finding compressed sizes.");
		return 1;
	}

	FAssetRegistryState AssetRegistry;
	if (FAssetRegistryState::LoadFromDisk(*FileName, FAssetRegistryLoadOptions(), AssetRegistry) == false)
	{
		UE_LOGF(LogAssetSize, Error, "Failed load asset registry (%ls)", *FileName);		
		return 1;
	}

	UE_LOGF(LogAssetSize, Display, "Using: %ls", *FileName);

	FString AssetFilter;
	FParse::Value(*FullCommandLine, TEXT("Filter="), AssetFilter);

	FString ClassFilter;
	FParse::Value(*FullCommandLine, TEXT("FilterClass="), ClassFilter);
	FName ClassFilterName = ClassFilter.Len() ? FName(ClassFilter) : NAME_None;

	int ShowCount = 10;
	FString ShowCountString;
	if (FParse::Value(*FullCommandLine, TEXT("Show="), ShowCountString))
	{
		ShowCount = FCString::Atoi(*ShowCountString);
		if (ShowCount < 0)
		{
			UE_LOGF(LogAssetSize, Warning, "Invalid 'Show' count specified (%ls), using \"show all\"", *ShowCountString);
			ShowCount = 0;
		}
	}

	FString OutputCSVPath;
	EOutputCSVType OutputCSVType = EOutputCSVType::None;
	if (FParse::Value(*FullCommandLine, TEXT("-CSV="), OutputCSVPath))
	{
		OutputCSVType = EOutputCSVType::Classes;
		FString RawCSVType;
		if (FParse::Value(*FullCommandLine, TEXT("-CSVType="), RawCSVType))
		{
			if (RawCSVType.Compare(TEXT("classes"), ESearchCase::IgnoreCase) == 0)
			{
				OutputCSVType = EOutputCSVType::Classes;
				UE_LOGF(LogAssetSize, Display, "CSV Type: Classes");
			}
			else if (RawCSVType.Compare(TEXT("assets"), ESearchCase::IgnoreCase) == 0)
			{
				OutputCSVType = EOutputCSVType::Assets;
				UE_LOGF(LogAssetSize, Display, "CSV Type: Assets");
			}
			else
			{
				UE_LOGF(LogAssetSize, Error, "Invalid -CSVType: %ls", *RawCSVType);
				return 1;
			}
		}
		else
		{
			UE_LOGF(LogAssetSize, Display, "CSV Type: Default (Classes)");
		}
	}

	uint64 TotalDiskSize = 0;
	const TMap<FName, const FAssetPackageData*>& PackageDatas = AssetRegistry.GetAssetPackageDataMap();
	for (const TPair<FName, const FAssetPackageData*>& Pair : PackageDatas)
	{
		if (Pair.Value->DiskSize >= 0)
		{
			TotalDiskSize += Pair.Value->DiskSize;
		}
	}

	int64 ImportantAssetCount = 0;
	int64 MatchedAssetCount = 0;
	int64 FilteredCompressedSize = 0;
	int64 TotalCompressedSize = 0;

	struct FMatchedAssetInfo
	{
		FSoftObjectPath ObjectPath;
		int64 CompressedSize = 0;
		int64 OptionalSize = 0;
		int64 InstalledSize = 0;
		int64 StreamingSize = 0;
		int64 UncompressedSize = 0;

	};
	TMap<FTopLevelAssetPath /* AssetClass */, int64> FilteredClassCompressedSizes;
	TMap<FTopLevelAssetPath /* AssetClass */, TArray<FMatchedAssetInfo>> FilteredClassMatchedAssets;
	AssetRegistry.EnumerateAllAssets(TSet<FName>(), 
		[&ImportantAssetCount,
		 &MatchedAssetCount,
		 &TotalCompressedSize,
		 &FilteredClassMatchedAssets, 
		 &FilteredCompressedSize, 
		 &FilteredClassCompressedSizes, 
		 &AssetFilter,
		ClassFilterName](const FAssetData& AssetData)
	{
		FString CompressedSize;
		if (AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, CompressedSize) == false ||
			CompressedSize.Len() == 0)
		{
			return true;
		}

		int64 AssetCompressedSize = FCString::Atoi64(*CompressedSize);
		ImportantAssetCount++;
		TotalCompressedSize += AssetCompressedSize;

		bool bMatched = true;
		if (AssetFilter.Len())
		{
			FString ObjectPath = AssetData.GetObjectPathString();
			bMatched = FWildcardString::IsMatchSubstring(*AssetFilter, *ObjectPath, *ObjectPath + ObjectPath.Len(), ESearchCase::IgnoreCase);
		}
		if (bMatched &&
			ClassFilterName != NAME_None)
		{
			bMatched = AssetData.AssetClassPath.GetAssetName() == ClassFilterName;
		}

		if (bMatched)
		{
			MatchedAssetCount++;

			FMatchedAssetInfo& Info = FilteredClassMatchedAssets.FindOrAdd(AssetData.AssetClassPath).AddDefaulted_GetRef();
			Info.ObjectPath = AssetData.GetSoftObjectPath();
			Info.CompressedSize = AssetCompressedSize;

			AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkInstalledSizeFName, Info.InstalledSize);
			AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkOptionalSizeFName, Info.OptionalSize);
			AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkStreamingSizeFName, Info.StreamingSize);
			AssetData.GetTagValue(UE::AssetRegistry::Stage_ChunkSizeFName, Info.UncompressedSize);
			
			FilteredCompressedSize += AssetCompressedSize;
			int64& FilteredClassCompressedSize = FilteredClassCompressedSizes.FindOrAdd(AssetData.AssetClassPath);
			FilteredClassCompressedSize += AssetCompressedSize;
		}

		return true;
	}); // end EnumerateAssets

	if (ImportantAssetCount == 0)
	{
		UE_LOGF(LogAssetSize, Display, "No assets with size information found - staging metadata needs to be added to the asset registry");
		UE_LOGF(LogAssetSize, Display, "via ProjectSettings->Packaging->WriteBackMetadataToAssetRegistry.");
		return 1;
	}

	FilteredClassCompressedSizes.ValueSort(TGreater<int64>());

	
	UE_LOGF(LogAssetSize, Display, "Filter:                                      %ls", AssetFilter.Len() ? *AssetFilter : TEXT("<all>"));
	UE_LOGF(LogAssetSize, Display, "Assets with size information:                %ls", *FText::AsNumber(ImportantAssetCount).ToString());
	UE_LOGF(LogAssetSize, Display, "Filtered to:                                 %ls (%.1f%%)", *FText::AsNumber(MatchedAssetCount).ToString(), 100.0 * MatchedAssetCount / ImportantAssetCount);
	UE_LOGF(LogAssetSize, Display, "Total compressed size (bytes):               %ls", *FText::AsNumber(TotalCompressedSize).ToString());
	UE_LOGF(LogAssetSize, Display, "Filtered compressed size (bytes):            %ls (%.1f%%)", *FText::AsNumber(FilteredCompressedSize).ToString(), 100.0 * FilteredCompressedSize / TotalCompressedSize);
	if (ShowCount)
	{
		UE_LOGF(LogAssetSize, Display, "Top %2ls filtered class sizes:                 bytes          (pct of filtered total)", *FText::AsNumber(ShowCount).ToString());
	}
	else
	{
		UE_LOGF(LogAssetSize, Display, "Filtered class sizes:                         bytes          (pct of filtered total)");
	}
	for (TPair<FTopLevelAssetPath, int64>& ClassSizePair : FilteredClassCompressedSizes)
	{
		UE_LOGF(LogAssetSize, Display, "    %-40ls %-14ls (%.1f%%)", *ClassSizePair.Key.ToString(), *FText::AsNumber(ClassSizePair.Value).ToString(), 100.0 * ClassSizePair.Value / FilteredCompressedSize);
		if (ShowCount)
		{
			ShowCount--;
			if (ShowCount == 0)
			{
				break;
			}
		}
	}

	if (OutputCSVType != EOutputCSVType::None)
	{
		TArray<FString> Lines;

		if (OutputCSVType == EOutputCSVType::Classes)
		{
			Lines.Add(TEXT("AssetClass,AssetCount,TotalCompressedSize"));
			for (TPair<FTopLevelAssetPath, int64>& ClassSizePair : FilteredClassCompressedSizes)
			{
				// we add to both maps at the same time to we know the lookup succeeds.
				Lines.Add(FString::Printf(TEXT("%s,%d,%" INT64_FMT), *ClassSizePair.Key.ToString(), FilteredClassMatchedAssets[ClassSizePair.Key].Num(), ClassSizePair.Value));
			}
		}
		else if (OutputCSVType == EOutputCSVType::Assets)
		{
			Lines.Add(TEXT("AssetName,AssetType,CompressedSize,InstalledSize,OptionalSize,StreamingSize,UncompressedSize"));
			for (TPair<FTopLevelAssetPath, TArray<FMatchedAssetInfo>>& ClassAssetsPair : FilteredClassMatchedAssets)
			{
				// we add to both maps at the same time to we know the lookup succeeds.
				for (const FMatchedAssetInfo& AssetInfo : ClassAssetsPair.Value)
				{
					Lines.Add(FString::Printf(TEXT("%s,%s,%lld,%lld,%lld,%lld,%lld"), *AssetInfo.ObjectPath.ToString(), *ClassAssetsPair.Key.ToString(), AssetInfo.CompressedSize, AssetInfo.InstalledSize, AssetInfo.OptionalSize, AssetInfo.StreamingSize, AssetInfo.UncompressedSize));
				}
			}
		}
		else
		{
			check(0); // added a type ??
		}

		if (FFileHelper::SaveStringArrayToFile(Lines, *OutputCSVPath) == false)
		{
			UE_LOGF(LogAssetSize, Error, "Unable to write CSV file %ls", *OutputCSVPath);
			return 1;
		}
		UE_LOGF(LogAssetSize, Display, "Saved CSV file: %ls", *OutputCSVPath);
	} // end if outputting csv file

	return 0;
}
