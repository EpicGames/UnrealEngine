// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/AudioMixerCommandlet.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundWave.h"
#include "Audio.h"
#include "Audio/AudioTraceUtil.h"
#include "UObject/UObjectHash.h"
#include "UObject/Package.h"
#include "ActiveSound.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMixerCommandlet)

#define ENABLE_AUDIO_MIXER_COMMANDLET (PLATFORM_WINDOWS)

#if ENABLE_AUDIO_MIXER_COMMANDLET

#include "AudioDevice.h"
#include "AudioMixerModule.h"
#include "Components/AudioComponent.h"
#include "Sound/AudioSettings.h"

DEFINE_LOG_CATEGORY_STATIC(AudioMixerCommandlet, Log, All);

/************************************************************************
* UTILITY FUNCTIONS AND CLASSES
************************************************************************/

/** Class which circularly rotates around a an offset with a given angular velocity. */
class FPositionRotator
{
public:
	FPositionRotator(float InRadius, float InCurrentAngle, float InAngularVelocity, const FVector& InOffset = FVector::ZeroVector)
		: Radius(InRadius)
		, CurrentAngle(InCurrentAngle)
		, AngularVelocity(InAngularVelocity)
		, Position(FVector::ZeroVector)
		, Offset(InOffset)
	{
		float PosX = Radius * FMath::Cos(CurrentAngle);
		float PosZ = Radius * FMath::Sin(CurrentAngle);
		Position = Offset + FVector(PosX, 0.0f, PosZ);
	}

	FVector GetPosition() const
	{
		return Position;
	}

	void Update()
	{
		CurrentAngle += AngularVelocity;
		float PosX = Radius * FMath::Cos(CurrentAngle);
		float PosZ = Radius * FMath::Sin(CurrentAngle);
		Position = Offset + FVector(PosX, 0.0f, PosZ);
	}

	void SetAngularVelocity(float InAngularVelocity)
	{
		AngularVelocity = InAngularVelocity;
	}

	float Radius;
	float CurrentAngle;
	float AngularVelocity;
	FVector Position;
	FVector Offset;
};

static USoundWave* LoadSoundWave(const FString& SoundWavePath)
{
	// Load the package
	UPackage* SoundWaveAssetPkg = LoadPackage(nullptr, *SoundWavePath, LOAD_None);

	// Get all the objects associated with this package
	TArray<UObject*> Objects;
	GetObjectsWithOuter(SoundWaveAssetPkg, Objects);

	USoundWave* Sound = nullptr;
	for (UObject* Obj : Objects)
	{
		Sound = Cast<USoundWave>(Obj);
		if (Sound != nullptr)
		{
			break;
		}
	}

	if (!Sound)
	{
		UE_LOGF(AudioMixerCommandlet, Error, "Failed to find a USoundWave for asset path %ls", *SoundWavePath);
	}
	return Sound;
}

static void CreateDefaultSoundSearchPaths(TArray<FString>& OutSearchPaths)
{
	OutSearchPaths.Add(FPaths::EngineContentDir() / TEXT("EditorSounds"));
	OutSearchPaths.Add(FPaths::EngineContentDir() / TEXT("EngineSounds"));
}

template<typename T>
static void LoadEditorAndEngineObjects(TArray<FString>& SearchPaths, TArray<T*>& OutObjects, TArray<FString>* IgnoreList = nullptr)
{
	for (FString& SearchPath : SearchPaths)
	{
		IFileManager& FileManager = IFileManager::Get();
		TArray<FString> EngineContentFiles;
		FileManager.FindFilesRecursive(EngineContentFiles, *SearchPath, TEXT("*.uasset"), true, false);

		for (FString& AssetPath : EngineContentFiles)
		{
			UPackage* Package = LoadPackage(nullptr, *AssetPath, LOAD_None);
			TArray<UObject*> Objects;
			GetObjectsWithOuter(Package, Objects);

			for (UObject* Obj : Objects)
			{
				T* OutObj = Cast<T>(Obj);
				if (OutObj != nullptr)
				{
					bool bIgnored = false;
					if (IgnoreList)
					{
						for (int i = 0; i < IgnoreList->Num(); ++i)
						{
							if (Obj->GetName() == (*IgnoreList)[i])
							{
								bIgnored = true;
								break;
							}
						}
					}

					if (!bIgnored)
					{
						OutObjects.Add(OutObj);
					}
				}
			}
		}
	}

	UE_LOGF(AudioMixerCommandlet, Log, "Loaded %d objects from engine content directory", OutObjects.Num());
}

static void PlayOneShotSound(FAudioDevice* InAudioDevice, const TArray<USoundWave*>& InSounds)
{
	// Randomly pick a sound wave
	USoundWave* SoundWave = nullptr;
	while (true)
	{
		int32 SoundIndex = FMath::RandRange(0, InSounds.Num() - 1);
		SoundWave = InSounds[SoundIndex];
		if (!SoundWave->bLooping)
		{
			break;
		}
	}
	// Create an active sound
	FActiveSound NewActiveSound;
	NewActiveSound.SetSound(SoundWave);

	NewActiveSound.SetVolume(0.25f);
	NewActiveSound.SetPitch(FMath::FRandRange(0.1f, 3.0f));

	NewActiveSound.RequestedStartTime = 0.0f;

	NewActiveSound.bIsUISound = true;
	NewActiveSound.bAllowSpatialization = false;
	NewActiveSound.Priority = 1.0f;

#if UE_AUDIO_PROFILERTRACE_ENABLED
	const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestOneShot;
#else
	const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

	// Add it to the audio device
	InAudioDevice->AddNewActiveSound(NewActiveSound, nullptr, EventLogID);
}

static UAudioComponent* SpawnLoopingSound(FAudioDevice* InAudioDevice, UWorld* World, const TArray<USoundWave*>& InSounds, bool bAllowSpatialization = false, USoundAttenuation* SoundAttenuation = nullptr, float StartTime = 0.0f)
{
	int32 SoundIndex = FMath::RandRange(0, InSounds.Num() - 1);
	USoundWave* SoundWave = InSounds[SoundIndex];

	// Set the sound wave to looping
	SoundWave->bLooping = true;

	FAudioDevice::FCreateComponentParams Params(World);
	Params.AttenuationSettings = SoundAttenuation;

	UAudioComponent* AudioComponent = FAudioDevice::CreateComponent(SoundWave, Params);
	if (AudioComponent)
	{
		AudioComponent->SetVolumeMultiplier(0.5f);
		AudioComponent->SetPitchMultiplier(1.0f);
		AudioComponent->bAllowSpatialization = bAllowSpatialization;
		AudioComponent->bIsUISound = !bAllowSpatialization;
		AudioComponent->bAutoDestroy = true;
	}

	return AudioComponent;
}

/************************************************************************
* FAudioMixerCommand
************************************************************************/

class FAudioMixerCommand
{
public:
	FAudioMixerCommand(const FString& InName, const FString& InDescription = FString(), int32 InNumArgs = 0, const FString& InArgDescription = FString())
		: Name(InName)
		, Description(InDescription)
		, ArgDescription(InArgDescription)
		, NumArgs(InNumArgs)
	{
		Commands.Add(this);
	}

	virtual ~FAudioMixerCommand()
	{
	}

	// Return the name of the test
	const FString& GetName() const { return Name; }

	// Return the description of the test
	const FString& GetDescription() const { return Description; }

	// Return the num args of the test
	int32 GetNumArgs() const { return NumArgs; }

	// Return the description of the test
	const FString& GetArgDescription() const { return ArgDescription; }

	// Run the command
	virtual bool Run(UWorld* World, const TArray<FString>& InArgs) = 0;

	static TArray<FAudioMixerCommand*>& GetCommands() { return Commands; }

protected:
	FString Name;
	FString Description;
	FString ArgDescription;
	int32 NumArgs;

	static TArray<FAudioMixerCommand*> Commands;
};

TArray<FAudioMixerCommand*> FAudioMixerCommand::Commands;

/************************************************************************
* FRunAudioDevice
************************************************************************/

class FRunAudioDevice final : public FAudioMixerCommand
{
public:
	FRunAudioDevice()
		: FAudioMixerCommand(TEXT("RunAudioDevice"), TEXT("Create and run an FAudioDevice object."), 1, TEXT("Number of seconds to run."))
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		// Check if we've been told to run the audio device for a certain amount of time
		float TimeToRunSec = 10.0f;

		if (InArgs.Num() > 0)
		{
			TimeToRunSec = FCString::Atof(*InArgs[0]);
		}

		// Get the audio main device
		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

		bool bSuccess = false;

		if (AudioDevice)
		{
			// Get the quality settings of the audio device (uses game user settings)
			FAudioQualitySettings QualitySettings = AudioDevice->GetQualityLevelSettings();

			// Initialize the audio device
			bSuccess = AudioDevice->Init(AudioDevice.GetDeviceID(), QualitySettings.MaxChannels);

			if (bSuccess)
			{
				// Toggle the audio debug output (sine-wave tones)
				AudioDevice->EnableDebugAudioOutput();
				
				double CurrentTime = AudioDevice->GetAudioTime();
				double StartTime = CurrentTime;
				while (true)
				{

					CurrentTime = AudioDevice->GetAudioTime();
					UE_LOGF(AudioMixerCommandlet, Log, "Current Time: %.2f", CurrentTime);

					if (CurrentTime - StartTime >= TimeToRunSec)
					{
						break;
					}

					FPlatformProcess::Sleep(1);

				}

				// Teardown the audio device
				AudioDevice->Teardown();
			}

		}

		return bSuccess;
	}
};
FRunAudioDevice RunAudioDevice;

/************************************************************************
* FPlaySoundWave2D
************************************************************************/

class FPlaySoundWave2D final : public FAudioMixerCommand
{
public:
	FPlaySoundWave2D()
		: FAudioMixerCommand("PlaySoundWave2D", "Load and play a 2D engine test sound wave")
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		TArray<FString> SearchPaths;
		CreateDefaultSoundSearchPaths(SearchPaths);

		TArray<USoundWave*> SoundWaves;
		TArray<FString> IgnoreList;
		IgnoreList.Add(TEXT("WhiteNoise"));

		LoadEditorAndEngineObjects<USoundWave>(SearchPaths, SoundWaves, &IgnoreList);

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

		bool bSuccess = false;

		float TimeCount = 0.0f;
		const float DeltaTime = 0.033f;

		if (AudioDevice)
		{
			// Get the quality settings of the audio device (uses game user settings)
			FAudioQualitySettings QualitySettings = AudioDevice->GetQualityLevelSettings();

			// Initialize the audio device
			bSuccess = AudioDevice->Init(AudioDevice.GetDeviceID(), QualitySettings.MaxChannels);

			// Wait a few seconds to give the editor a chance to load everything... you get hitches in the beginning otherwise
			FPlatformProcess::Sleep(1);

			int32 SoundCount = 0;
			if (bSuccess)
			{
				double CurrentTime = AudioDevice->GetAudioTime();

				while (true)
				{
					if (TimeCount == 0.0f)
					{
						int32 NumActiveSounds = AudioDevice->GetNumActiveSources();

						PlayOneShotSound(AudioDevice.GetAudioDevice(), SoundWaves);
					}

					CurrentTime = AudioDevice->GetAudioTime();

					// Update the audio device
					AudioDevice->Update(true);

					// Sleep 33 ms
					FPlatformProcess::Sleep(DeltaTime);

					TimeCount += DeltaTime;

					if (TimeCount > 0.25f)
					{
						TimeCount = 0.0f;
					}
				}
			}

		}

		return bSuccess;
	}
};
FPlaySoundWave2D PlaySoundWave2D;

/************************************************************************
* FPlaySoundWaveLooping2D
************************************************************************/

class FPlaySoundWaveLooping2D final : public FAudioMixerCommand
{
public:
	FPlaySoundWaveLooping2D()
		: FAudioMixerCommand("PlaySoundWaveLooping2D", "Load and play a single looping 2D engine test sound wave")
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		// Load a single large seamless loop added to test path

		FString LoopingSoundPath = FPaths::EngineContentDir() / TEXT("EngineSounds") / TEXT("TestSounds") / TEXT("Loops");

		TArray<FString> SearchPaths;
		SearchPaths.Add(LoopingSoundPath);

		TArray<USoundWave*> SoundWaves;
		TArray<FString> IgnoreList;
		IgnoreList.Add(TEXT("WhiteNoise"));

		LoadEditorAndEngineObjects<USoundWave>(SearchPaths, SoundWaves, &IgnoreList);

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

		bool bSuccess = false;
		float TimeCount = 0.0f;
		const float DeltaTime = 0.033f;

		if (AudioDevice)
		{
			// Get the quality settings of the audio device (uses game user settings)
			FAudioQualitySettings QualitySettings = AudioDevice->GetQualityLevelSettings();

			// Initialize the audio device
			bSuccess = AudioDevice->Init(AudioDevice.GetDeviceID(), QualitySettings.MaxChannels);

			// Wait a few seconds to give the editor a chance to load everything... you get hitches in the beginning otherwise
			FPlatformProcess::Sleep(1);

			int32 SoundCount = 0;

			if (bSuccess)
			{
				UAudioComponent* LoopingSound = SpawnLoopingSound(AudioDevice.GetAudioDevice(), World, SoundWaves);

				while (true)
				{
					// Update the audio device
					AudioDevice->Update(true);

					// Sleep 33 ms
					FPlatformProcess::Sleep(DeltaTime);

					TimeCount += DeltaTime;

					if (TimeCount > 0.25f)
					{
						TimeCount = 0.0f;
					}
				}
			}
		}

		return bSuccess;
	}
};
FPlaySoundWaveLooping2D PlaySoundWaveLooping2D;

/************************************************************************
* FPlayRealTimeSoundWaveLooping2D
************************************************************************/

class FPlayRealTimeSoundWaveLooping2D final : public FAudioMixerCommand
{
public:
	FPlayRealTimeSoundWaveLooping2D()
		: FAudioMixerCommand("PlayRealTimeSoundWaveLooping2D", "Load and play a single looping 2D engine test sound wave using real-time decoding.")
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		// Load a single large seamless loop added to test path

		FString LoopingSoundPath = FPaths::EngineContentDir() / TEXT("EngineSounds") / TEXT("TestSounds") / TEXT("Loops");

		TArray<FString> SearchPaths;
		SearchPaths.Add(LoopingSoundPath);

		TArray<USoundWave*> SoundWaves;
		TArray<FString> IgnoreList;
		IgnoreList.Add(TEXT("WhiteNoise"));

		LoadEditorAndEngineObjects<USoundWave>(SearchPaths, SoundWaves, &IgnoreList);

		// Set the looping sound waves sound groups to one that has a 0-second threshold
		for (USoundWave* SoundWave : SoundWaves)
		{
			SoundWave->SoundGroup = ESoundGroup::SOUNDGROUP_Music;
		}

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();

		bool bSuccess = false;
		float TimeCount = 0.0f;
		const float DeltaTime = 0.033f;

		if (AudioDevice)
		{
			// Get the quality settings of the audio device (uses game user settings)
			FAudioQualitySettings QualitySettings = AudioDevice->GetQualityLevelSettings();

			// Initialize the audio device
			bSuccess = AudioDevice->Init(AudioDevice.GetDeviceID(), QualitySettings.MaxChannels);

			// Wait a few seconds to give the editor a chance to load everything... you get hitches in the beginning otherwise
			FPlatformProcess::Sleep(1);

			int32 SoundCount = 0;

			if (bSuccess)
			{
				UAudioComponent* LoopingSound = SpawnLoopingSound(AudioDevice.GetAudioDevice(), World, SoundWaves);
				LoopingSound->Play(0.0f);

				while (true)
				{
					// Update the audio device
					AudioDevice->Update(true);

					// Sleep 33 ms
					FPlatformProcess::Sleep(DeltaTime);

					TimeCount += DeltaTime;

					if (TimeCount > 0.25f)
					{
						TimeCount = 0.0f;
					}
				}
			}
		}

		return bSuccess;
	}
};
FPlayRealTimeSoundWaveLooping2D PlayRealTimeSoundWaveLooping2D;

/************************************************************************
* FPlaySoundWaveLooping2DPitched
************************************************************************/

class FPlaySoundWaveLooping2DPitched final : public FAudioMixerCommand
{
public:
	FPlaySoundWaveLooping2DPitched()
		: FAudioMixerCommand(TEXT("PlaySoundWaveLooping2DPitched"), 
			TEXT("Load and play a single looping 2D engine test sound wave using real-time decoding."), 
			1, 
			TEXT("Number of loops you want to play"))
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		// Load a single large seamless loop added to test path

		int32 NumLoops = 1;
		if (InArgs.Num())
		{
			NumLoops = FMath::Max(FCString::Atoi(*InArgs[0]), 1);
		}

		FString LoopingSoundPath = FPaths::EngineContentDir() / TEXT("EngineSounds") / TEXT("TestSounds") / TEXT("Loops");

		TArray<FString> SearchPaths;
		SearchPaths.Add(LoopingSoundPath);

		TArray<USoundWave*> SoundWaves;
		TArray<FString> IgnoreList;
		IgnoreList.Add(TEXT("WhiteNoise"));

		LoadEditorAndEngineObjects<USoundWave>(SearchPaths, SoundWaves, &IgnoreList);

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
		if (!AudioDevice)
		{
			return false;
		}

		FPlatformProcess::Sleep(1);

		const float DeltaTime = 0.033f;

		TArray<UAudioComponent*> AudioComponents;
		TArray<FDynamicParameter> PitchParams;
		TArray<FDynamicParameter> VolumeParams;
		TArray<float> CurrentTime;
		TArray<float> TargetTime;

		// Get the quality settings of the audio device (uses game user settings)

		int32 SoundCount = 0;
		AudioComponents.Reserve(NumLoops);

		for (int32 i = 0; i < NumLoops; ++i)
		{
			UAudioComponent* LoopingSound = SpawnLoopingSound(AudioDevice.GetAudioDevice(), World, SoundWaves);
			if (!LoopingSound)
			{
				return false;
			}

			AudioComponents.Add(LoopingSound);
			PitchParams.Add(FDynamicParameter(FMath::FRandRange(0.1f, 4.0f)));
			VolumeParams.Add(FDynamicParameter(0.0f));

			float NewTargetTime = FMath::FRandRange(0.5f, 3.0f);
			PitchParams[i].Set(FMath::FRandRange(0.1f, 4.0f), NewTargetTime);
			VolumeParams[i].Set(FMath::FRandRange(0.1f, 1.0f), NewTargetTime);

			CurrentTime.Add(0.0f);
			TargetTime.Add(NewTargetTime);
		}

		while (true)
		{
			// Update the audio device
			AudioDevice->Update(true);

			for (int32 i = 0; i < NumLoops; ++i)
			{
				if (!AudioComponents[i]->IsActive())
				{
					AudioComponents[i]->Play(0.0f);
				}

				AudioComponents[i]->SetPitchMultiplier(PitchParams[i].GetValue());
				AudioComponents[i]->SetVolumeMultiplier(VolumeParams[i].GetValue());
				PitchParams[i].Update(DeltaTime);
				VolumeParams[i].Update(DeltaTime);
				CurrentTime[i] += DeltaTime;
				if (CurrentTime[i] >= TargetTime[i])
				{
					CurrentTime[i] = 0.0f;
					float NewTargetTime = FMath::FRandRange(2.0f, 3.0f);
					PitchParams[i].Set(FMath::FRandRange(0.1f, 4.0f), NewTargetTime);
					VolumeParams[i].Set(FMath::FRandRange(0.1f, 1.0f), NewTargetTime);
					TargetTime[i] = NewTargetTime;
				}
			}

			// Sleep 33 ms
			FPlatformProcess::Sleep(DeltaTime);
		}
	}
};
FPlaySoundWaveLooping2DPitched PlaySoundWaveLooping2DPitched;


/************************************************************************
* FPlaySoundWaveLooping2DPitched
************************************************************************/

class FPlaySoundWaveLooping3DPitched final : public FAudioMixerCommand
{
public:

	FPlaySoundWaveLooping3DPitched()
		: FAudioMixerCommand("PlaySoundWaveLooping3DPitched",
			"Load and play a single looping 3D engine test sound wave using real-time decoding.",
			1,
			"Number of loops you want to play")
	{}

	bool Run(UWorld* World, const TArray<FString>& InArgs) override
	{
		// Load a single large seamless loop added to test path

		int32 NumLoops = 1;
		if (InArgs.Num())
		{
			NumLoops = FMath::Max(FCString::Atoi(*InArgs[0]), 1);
		}

		FString LoopingSoundPath = FPaths::EngineContentDir() / TEXT("EngineSounds") / TEXT("TestSounds") / TEXT("Loops") / TEXT("Mono");

		TArray<FString> SearchPaths;
		SearchPaths.Add(LoopingSoundPath);

		TArray<USoundWave*> SoundWaves;
		TArray<FString> IgnoreList;
		IgnoreList.Add(TEXT("WhiteNoise"));

		LoadEditorAndEngineObjects<USoundWave>(SearchPaths, SoundWaves, &IgnoreList);

		FString AttenuationSearchPath = FPaths::EngineContentDir() / TEXT("EngineSounds") / TEXT("TestSounds") / TEXT("Attenuation");

		TArray<USoundAttenuation*> SoundAttenuations;
		SearchPaths.Reset();
		SearchPaths.Add(AttenuationSearchPath);

		LoadEditorAndEngineObjects<USoundAttenuation>(SearchPaths, SoundAttenuations);

		FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
		if (!AudioDevice)
		{
			return false;
		}

		FPlatformProcess::Sleep(1);

		const float DeltaTime = 0.033f;

		TArray<UAudioComponent*> AudioComponents;
		TArray<FDynamicParameter> PitchParams;
		TArray<FDynamicParameter> VolumeParams;
		TArray<float> CurrentTime;
		TArray<float> TargetTime;
		TArray<FPositionRotator> Rotators;

		// Get the quality settings of the audio device (uses game user settings)

		int32 SoundCount = 0;

		for (int32 i = 0; i < NumLoops; ++i)
		{
			int32 AttenuationIndex = FMath::RandRange(0, SoundAttenuations.Num() - 1);
			USoundAttenuation* Attenuation = SoundAttenuations[AttenuationIndex];

			UAudioComponent* LoopingSound = SpawnLoopingSound(AudioDevice.GetAudioDevice(), World, SoundWaves, true, Attenuation);
			if (!LoopingSound)
			{
				return false;
			}

			AudioComponents.Add(LoopingSound);
			PitchParams.Add(FDynamicParameter(FMath::FRandRange(0.1f, 4.0f)));
			VolumeParams.Add(FDynamicParameter(0.0f));
			Rotators.Add(FPositionRotator(FMath::FRandRange(50.0f, 1000.0f), FMath::FRandRange(0.0f, 2.0f*PI), FMath::FRandRange(-0.1f, 0.1f)));

			float NewTargetTime = FMath::FRandRange(0.5f, 3.0f);
			PitchParams[i].Set(FMath::FRandRange(0.1f, 4.0f), NewTargetTime);
			VolumeParams[i].Set(FMath::FRandRange(0.1f, 1.0f), NewTargetTime);

			CurrentTime.Add(0.0f);
			TargetTime.Add(NewTargetTime);
		}

		while (true)
		{
			// Update the audio device
			AudioDevice->Update(true);

			for (int32 i = 0; i < NumLoops; ++i)
			{
				// Update the position
				FPositionRotator& Rotator = Rotators[i];
				Rotator.Update();
				FVector Position = Rotator.GetPosition();

				UE_LOGF(LogTemp, Log, "Position - X: %.2f, Y: %.2f, Z: %.2f", Position.X, Position.Y, Position.Z);

				AudioComponents[i]->SetWorldLocationAndRotation(Position, FRotator::ZeroRotator);

				AudioComponents[i]->SetPitchMultiplier(PitchParams[i].GetValue());
				AudioComponents[i]->SetVolumeMultiplier(VolumeParams[i].GetValue());

				if (!AudioComponents[i]->IsActive())
				{
					AudioComponents[i]->Play(0.0f);
				}

				PitchParams[i].Update(DeltaTime);
				VolumeParams[i].Update(DeltaTime);
				CurrentTime[i] += DeltaTime;
				if (CurrentTime[i] >= TargetTime[i])
				{
					CurrentTime[i] = 0.0f;
					float NewTargetTime = FMath::FRandRange(2.0f, 3.0f);
					PitchParams[i].Set(FMath::FRandRange(0.1f, 4.0f), NewTargetTime);
					VolumeParams[i].Set(FMath::FRandRange(0.1f, 1.0f), NewTargetTime);
					TargetTime[i] = NewTargetTime;
				}
			}

			// Sleep 33 ms
			FPlatformProcess::Sleep(DeltaTime);
		}
	}
};
FPlaySoundWaveLooping3DPitched PlaySoundWaveLooping3DPitched;


#endif // ENABLE_AUDIO_MIXER_COMMANDLET

/************************************************************************
* UAudioMixerCommandlet
************************************************************************/

UAudioMixerCommandlet::UAudioMixerCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAudioMixerCommandlet::PrintUsage() const
{
#if ENABLE_AUDIO_MIXER_COMMANDLET
	UE_LOGF(AudioMixerCommandlet, Display, "AudioMixerCommandlet Usage: {Editor}.exe UnrealEd.AudioMixerCommandlet {CommandName} {Args}");
	UE_LOGF(AudioMixerCommandlet, Display, "Possible commands:\n");

	UE_LOGF(AudioMixerCommandlet, Display, "Command Name, Command Description, Number of Arguments, Argument Description");

	TArray<FAudioMixerCommand*>& Commands = FAudioMixerCommand::GetCommands();

	for (int32 i = 0; i < Commands.Num(); ++i)
	{
		const FAudioMixerCommand* MixerCommand = Commands[i];
		UE_LOGF(AudioMixerCommandlet, Display, "%ls, %ls, %d, %ls", *MixerCommand->GetName(), *MixerCommand->GetDescription(), MixerCommand->GetNumArgs(), *MixerCommand->GetArgDescription());
	}
#endif
}


int32 UAudioMixerCommandlet::Main(const FString& InParams)
{
#if ENABLE_AUDIO_MIXER_COMMANDLET
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*InParams, Tokens, Switches);

	if (Tokens.Num() < 2)
	{
		PrintUsage();
		return 0;
	}

	UWorld* World = UWorld::CreateWorld(EWorldType::Game, true);
	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(EWorldType::Game);
	WorldContext.SetCurrentWorld(World);

	FURL URL;
	World->InitializeActorsForPlay(URL);
	World->BeginPlay();

	// Get command name
	FString CommandName = Tokens[1];

	bool bFoundTest = false;
	TArray<FAudioMixerCommand*>& Commands = FAudioMixerCommand::GetCommands();

	for (int32 i = 0; i < Commands.Num(); ++i)
	{
		if (Commands[i]->GetName() == CommandName)
		{
			bFoundTest = true;
			TArray<FString> Args;

			if (Commands[i]->GetNumArgs() > 0)
			{
				for (int32 j = 2; j < Tokens.Num(); ++j)
				{
					Args.Add(Tokens[j]);
				}
			}

			bool bSuccess = Commands[i]->Run(World, Args);
			UE_LOGF(AudioMixerCommandlet, Display, "Command %ls %ls.", *Commands[i]->GetName(), bSuccess ? TEXT("succeeded") : TEXT("failed"));

			break;
		}
	}

	GEngine->DestroyWorldContext(World);
	World->DestroyWorld(true);

	if (!bFoundTest)
	{
		UE_LOGF(AudioMixerCommandlet, Display, "Unknown test '%ls'. Exiting.", *CommandName);
		return 0;
	}

#endif // ENABLE_AUDIO_MIXER_COMMANDLET

	return 0;
}
