// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BaseIteratePackageCommandlet.cpp: Commandlet provides basic package iteration functionaliy for derived commandlets
=============================================================================*/
#include "Commandlets/BaseIteratePackagesCommandlet.h"

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/LinkerLoad.h"
#include "UObject/MetaData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Misc/EngineVersion.h"
#include "Misc/RedirectCollector.h"
#include "Engine/EngineTypes.h"
#include "Materials/Material.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "Engine/MapBuildDataRegistry.h"
#include "EngineGlobals.h"
#include "GameFramework/WorldSettings.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "FileHelpers.h"
#include "PlatformInfo.h"
#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CommandletSourceControlUtils.h"
#include "AssetCompilingManager.h"
#include "PackageHelperFunctions.h"
#include "PackageTools.h"
#include "StaticMeshCompiler.h"
#include "String/ParseTokens.h"
#include "UObject/PackageTrailer.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseIteratePackagesCommandlet)

DEFINE_LOG_CATEGORY(LogIteratePackagesCommandlet);

#include "AssetRegistry/AssetRegistryModule.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Engine/LevelStreaming.h"
#include "EditorBuildUtils.h"

// For preloading FFindInBlueprintSearchManager
#include "FindInBlueprintManager.h"

#include "ShaderCompiler.h"

// world partition includes
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "UObject/GCObjectScopeGuard.h"

/**-----------------------------------------------------------------------------
 *	UBaseIteratePackagesCommandlet commandlet.
 *
 * This commandlet exposes some functionality for iterating packages 
 *
 *
----------------------------------------------------------------------------**/

#define CURRENT_PACKAGE_VERSION 0
#define IGNORE_PACKAGE_VERSION INDEX_NONE

UBaseIteratePackagesCommandlet::UBaseIteratePackagesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bForceUATEnvironmentVariableSet(false)
{
}

int32 UBaseIteratePackagesCommandlet::InitializeParameters( const TArray<FString>& Tokens, TArray<FString>& PackageNames )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::InitializeParameters);

	Verbosity = VERY_VERBOSE;

	TArray<FString> Unused;
	bool bExplicitPackages = false;

	// Check to see if we have an explicit list of packages
	for( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
	{
		FString Package;
		FString PackageFolder;
		FString Maps;
		FString File;
		const FString& CurrentSwitch = Switches[ SwitchIdx ];
		if( FParse::Value( *CurrentSwitch, TEXT( "PACKAGE="), Package ) )
		{
			FString PackageFile;
			if (FPackageName::SearchForPackageOnDisk(Package, NULL, &PackageFile))
			{
				PackageNames.Add( *PackageFile );
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to find the package given by the cmdline: PACKAGE=%ls", *Package);
			}
			
			bExplicitPackages = true;
		}
		else if( FParse::Value( *CurrentSwitch, TEXT( "PACKAGEFOLDER="), PackageFolder ) )
		{
			TArray<FString> FilesInPackageFolder;
			FPackageName::FindPackagesInDirectory(FilesInPackageFolder, PackageFolder);
			for( int32 FileIndex = 0; FileIndex < FilesInPackageFolder.Num(); FileIndex++ )
			{
				FString PackageFile(FilesInPackageFolder[FileIndex]);
				FPaths::MakeStandardFilename(PackageFile);
				PackageNames.Add( *PackageFile );
			}
			bExplicitPackages = true;
		}
		else if (FParse::Value(*CurrentSwitch, TEXT("MAP="), Maps))
		{
			// Allow support for -MAP=Value1+Value2+Value3
			for (int32 PlusIdx = Maps.Find(TEXT("+"), ESearchCase::CaseSensitive); PlusIdx != INDEX_NONE; PlusIdx = Maps.Find(TEXT("+"), ESearchCase::CaseSensitive))
			{
				const FString NextMap = Maps.Left(PlusIdx);
				if (NextMap.Len() > 0)
				{
					FString MapFile;
					FPackageName::SearchForPackageOnDisk(NextMap, NULL, &MapFile);
					PackageNames.Add(*MapFile);
					bExplicitPackages = true;
				}

				Maps.RightInline(Maps.Len() - (PlusIdx + 1), EAllowShrinking::No);
			}
			FString MapFile;
			FPackageName::SearchForPackageOnDisk(Maps, NULL, &MapFile);
			PackageNames.Add(*MapFile);
			bExplicitPackages = true;
		}
		else if (FParse::Value(*CurrentSwitch, TEXT("FILE="), File))
		{
			FString Text;
			if (FFileHelper::LoadFileToString(Text, *File))
			{
				TArray<FString> Lines;

				// Remove all carriage return characters.
				Text.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
				// Read all lines
				Text.ParseIntoArray(Lines, TEXT("\n"), true);

				for (auto Line : Lines)
				{
					FString PackageFile;
					if (FPackageName::SearchForPackageOnDisk(Line, NULL, &PackageFile))
					{
						PackageNames.AddUnique(*PackageFile);
					}
					else
					{
						UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to find package %ls", *Line);
					}
				}

				bExplicitPackages = true;
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Loaded %d Packages from %ls", PackageNames.Num(), *File);
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to load file %ls", *File);
			}
		}
	}

	// Check for numeric settings
	for (const FString& CurrentSwitch : Switches)
	{
		if (FParse::Value(*CurrentSwitch, TEXT("GCFREQ="), GarbageCollectionFrequency))
		{
			UE_LOGF(LogIteratePackagesCommandlet, Display, "Setting garbage collection to happen every %d packages.", GarbageCollectionFrequency);
		}
	}

	InitializePackageNames(Tokens, PackageNames, bExplicitPackages);

	// ... if not, load in all packages
	if( !bExplicitPackages )
	{
		UE_LOGF( LogIteratePackagesCommandlet, Display, "No maps found to save, checking Project Settings for Directory or Asset Path(s)" );

		uint8 PackageFilter = NORMALIZE_DefaultFlags;
		if ( Switches.Contains(TEXT("SKIPMAPS")) )
		{
			PackageFilter |= NORMALIZE_ExcludeMapPackages;
		}
		else if ( Switches.Contains(TEXT("MAPSONLY")) )
		{
			PackageFilter |= NORMALIZE_ExcludeContentPackages;
		}

		if (Switches.Contains(TEXT("PROJECTONLY")))
		{
			PackageFilter |= NORMALIZE_ExcludeEnginePackages;
		}

		if ( Switches.Contains(TEXT("SkipDeveloperFolders")) || Switches.Contains(TEXT("NODEV")) )
		{
			PackageFilter |= NORMALIZE_ExcludeDeveloperPackages;
		}
		else if ( Switches.Contains(TEXT("OnlyDeveloperFolders")) )
		{
			PackageFilter |= NORMALIZE_ExcludeNonDeveloperPackages;
		}

		const FString AssetSearch = TEXT("*") + FPackageName::GetAssetPackageExtension();
		const FString MapSearch = TEXT("*") + FPackageName::GetMapPackageExtension();

		bool bAnyFound = NormalizePackageNames(Unused, PackageNames, AssetSearch, PackageFilter);
		bAnyFound |= NormalizePackageNames(Unused, PackageNames, MapSearch, PackageFilter);
		
		if (!bAnyFound)
		{
			return 1;
		}
	}

	// Check for a max package limit
	MaxPackagesToResave = -1;
	for ( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
	{
		const FString& CurrentSwitch = Switches[SwitchIdx];
		if( FParse::Value(*CurrentSwitch,TEXT("MAXPACKAGESTORESAVE="),MaxPackagesToResave))
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Only resaving a maximum of %d packages.", MaxPackagesToResave );
			break;
		}
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	

	// ResaveOnDemand is a generic way for systems to specify during load that the commandlet should resave a package
	ResaveOnDemandSystems = ParseResaveOnDemandSystems();
	if (!ResaveOnDemandSystems.IsEmpty())
	{
		TStringBuilder<64> SystemNames;
		for (FName SystemName : ResaveOnDemandSystems)
		{
			SystemNames << SystemName << TEXT(",");
		}
		SystemNames.RemoveSuffix(1); // Remove the trailing ,
		UE_LOGF(LogIteratePackagesCommandlet, Display, "ResaveOnDemand=%ls. Only saving packages that are reported via UE::SavePackageUtilities::OnAddResaveOnDemandPackage.",
			*SystemNames);
		bResaveOnDemand = true;
		UE::SavePackageUtilities::OnAddResaveOnDemandPackage.BindUObject(this, &UBaseIteratePackagesCommandlet::OnAddResaveOnDemandPackage);
	}

	// This option allows the dependency graph and soft object path redirect map to be populated. This is useful if you want soft object references to redirectors to be followed to the destination asset at save time.
	const bool bSearchAllAssets = Switches.Contains(TEXT("SearchAllAssets"));

	// Check for the min and max versions
	MinResaveUEVersion = IGNORE_PACKAGE_VERSION;
	MaxResaveUEVersion = IGNORE_PACKAGE_VERSION;
	MaxResaveLicenseeUEVersion = IGNORE_PACKAGE_VERSION;
	if ( Switches.Contains(TEXT("CHECKLICENSEEVER")) )
	{
		// Limits resaving to packages with this licensee package version or lower.
		MaxResaveLicenseeUEVersion = FMath::Max<int32>(GPackageFileLicenseeUEVersion - 1, 0);
	}
	if ( Switches.Contains(TEXT("CHECKUE4VER")) )
	{
		// Limits resaving to packages with this package version or lower.
		MaxResaveUEVersion = FMath::Max<int32>(GPackageFileUEVersion.ToValue() - 1, 0);
	}
	else if ( Switches.Contains(TEXT("RESAVEDEPRECATED")) )
	{
		// Limits resaving to packages with this package version or lower.
		MaxResaveUEVersion = FMath::Max<int32>(VER_UE4_DEPRECATED_PACKAGE - 1, 0);
	}
	else
	{
		// determine if the resave operation should be constrained to certain package versions
		for ( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
		{
			const FString& CurrentSwitch = Switches[SwitchIdx];
			if ( MinResaveUEVersion == IGNORE_PACKAGE_VERSION && FParse::Value(*CurrentSwitch,TEXT("MINVER="),MinResaveUEVersion) )
			{
				if ( MinResaveUEVersion == CURRENT_PACKAGE_VERSION )
				{
					MinResaveUEVersion = GPackageFileUEVersion.ToValue();
				}
			}

			if ( MaxResaveUEVersion == IGNORE_PACKAGE_VERSION && FParse::Value(*CurrentSwitch,TEXT("MAXVER="),MaxResaveUEVersion) )
			{
				if ( MaxResaveUEVersion == CURRENT_PACKAGE_VERSION )
				{
					MaxResaveUEVersion = GPackageFileUEVersion.ToValue();
				}
			}
		}
	}

	FString ClassList;
	for (const FString& CurrentSwitch : Switches)
	{
		if ( FParse::Value(*CurrentSwitch, TEXT("RESAVECLASS="), ClassList, false) )
		{
			TArray<FString> ClassNames;
			ClassList.ParseIntoArray(ClassNames, TEXT(","), true);
			for (FString& ClassName : ClassNames)
			{
				if (FPackageName::IsShortPackageName(ClassName))
				{
					UE_LOGF(LogIteratePackagesCommandlet, Warning, "RESAVECLASS param requires class path names. Short name provided: %ls.", *ClassName);
					FTopLevelAssetPath ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ClassName, ELogVerbosity::Warning, TEXT("ResavePackages"));
					if (!ClassPathName.IsNull())
					{
						ClassName = ClassPathName.ToString();
					}
				}
				ResaveClasses.AddUnique(ClassName);
			}

			break;
		}
	}

	/** determine if we should check subclasses of ResaveClasses */
	bool bIncludeChildClasses = Switches.Contains(TEXT("IncludeChildClasses"));
	if (bIncludeChildClasses && ResaveClasses.Num() == 0)
	{
		// Sanity check fail
		UE_LOGF(LogIteratePackagesCommandlet, Error, "AllowSubclasses param requires ResaveClass param.");
		return 1;
	}

	if (bIncludeChildClasses)
	{
		// Can't use ranged for here because the array grows inside of this loop.
		// Also, no need to iterate over the newly added objects as we know
		// we have found all of their subclasses too (IsChildOf guarantees that).
		const int32 NumResaveClasses = ResaveClasses.Num();
		for (int32 ClassIndex = 0; ClassIndex < NumResaveClasses; ++ClassIndex)
		{
			// Find the class object and then all derived classes
			UClass* ResaveClass = UClass::TryFindTypeSlow<UClass>(ResaveClasses[ClassIndex]);
			if (ResaveClass)
			{
				for (TObjectIterator<UClass> It; It; ++It)
				{
					UClass* MaybeChildClass = *It;
					if (MaybeChildClass->IsChildOf(ResaveClass))
					{
						ResaveClasses.AddUnique(MaybeChildClass->GetPathName());
					}
				}
			}
		}
	}

	return 0;
}

void UBaseIteratePackagesCommandlet::ParseSourceControlOptions(const TArray<FString>& Tokens)
{
	if (SourceControlQueue == nullptr)
	{
		return; // No point parsing the options if we don't have anything enabled
	}

	int32 QueuedPackageFlushLimit = INDEX_NONE;
	int64 QueueFileSizeFlushLimit = INDEX_NONE;

	for (const FString& CurrentSwitch : Tokens)
	{
		if (FParse::Value(*CurrentSwitch, TEXT("BatchPackageLimit="), QueuedPackageFlushLimit))
		{
			if (QueuedPackageFlushLimit >= 0)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Setting revision control batches to be limited to %d package(s) at a time.", QueuedPackageFlushLimit);
				SourceControlQueue->SetMaxNumQueuedPackages(QueuedPackageFlushLimit);
			}
			else
			{
				// Negative values mean we will not flush the source control batch based on the number of packages
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Setting revision control batches to have no package limit!");
			}
		}
		else  if (FParse::Value(*CurrentSwitch, TEXT("BatchFileSizeLimit="), QueueFileSizeFlushLimit))
		{
			const int64 SizeLimit = TNumericLimits<int64>::Max() / (1024 * 1024);
			if (QueueFileSizeFlushLimit > SizeLimit)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "-BatchFileSizeLimit=%lld is too large! The max value allowed is %lld, clamping...", QueueFileSizeFlushLimit, SizeLimit);
				QueueFileSizeFlushLimit = SizeLimit;
			}

			if (QueueFileSizeFlushLimit >= 0)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Setting revision control batches to be limited to %lld MB.", QueueFileSizeFlushLimit);

				SourceControlQueue->SetMaxTemporaryFileTotalSize(QueueFileSizeFlushLimit);
			}
			else
			{
				// Negative values mean we will not flush the source control batch based on the disk space taken by temp files
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Setting revision control batches to have no disk space limit!");
			}
		}
	}
}

void UBaseIteratePackagesCommandlet::OnAddResaveOnDemandPackage(FName SystemName, FName PackageName)
{
	FScopeLock ScopeLock(&ResaveOnDemandPackagesLock);
	if (ResaveOnDemandSystems.Contains(SystemName))
	{
		ResaveOnDemandPackages.Add(PackageName);
	}
}

void UBaseIteratePackagesCommandlet::SavePackages(const TArray<UPackage*>& PackagesToSave)
{
	for (UPackage* PackageToSave : PackagesToSave)
	{
		const FString LongPackageName = PackageToSave->GetName();
		FString Extension = FPackageName::GetAssetPackageExtension();
		if (PackageToSave->ContainsMap())
		{
			Extension = FPackageName::GetMapPackageExtension();
		}
		FString FileName;
		if (FPackageName::TryConvertLongPackageNameToFilename(LongPackageName, FileName, Extension))
		{
			ESaveFlags SaveFlags = SAVE_None;
			SavePackageHelper(PackageToSave, FileName, RF_Standalone, GWarn, SaveFlags);
		}
		else
		{
			UE_LOGF(LogIteratePackagesCommandlet, Error, "Unable to save package %ls, couldn't convert package name to filename", *LongPackageName);
		}
	}
}

TSet<FName> UBaseIteratePackagesCommandlet::ParseResaveOnDemandSystems()
{
	using namespace UE::String;
	TSet<FName> SystemNames;

	const TCHAR* CommandLineStream = FCommandLine::Get();
	for (;;)
	{
		FString Token = FParse::Token(CommandLineStream, false /* bUseEscape */);
		if (Token.IsEmpty())
		{
			break;
		}
		FString TokenSystemNames;
		if (FParse::Value(*Token, TEXT("-resaveondemand="), TokenSystemNames))
		{
			UE::String::ParseTokensMultiple(TokenSystemNames, TConstArrayView<TCHAR>({ '+', ',' }),
				[&SystemNames](FStringView TokenSystemName)
				{
					SystemNames.Add(FName(TokenSystemName));
				}, EParseTokensOptions::Trim | EParseTokensOptions::SkipEmpty);
		}
	}
	return SystemNames;
}


bool UBaseIteratePackagesCommandlet::ShouldSkipPackage(const FString& Filename)
{
	return false;
}

void UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage);

	// Check to see if a derived commandlet wants to skip this package for one reason or another
	if (ShouldSkipPackage(Filename))
	{
		return;
	}

	// Skip the package if it doesn't have a required substring match
	if (PackageSubstring.Len() && !Filename.Contains(PackageSubstring) )
	{
		VerboseMessage(FString::Printf(TEXT("Skipping %s"), *Filename));
		return;
	}

	FPackagePath PackagePath = FPackagePath::FromLocalPath(Filename);
	// Convert this path to mounted before querying the asset registry or CollectionFilter because they use mounted paths 
	PackagePath.TryConvertToMounted();
	if (CollectionFilter.Num())
	{
		FString PackageNameToCreate = PackagePath.GetPackageNameOrFallback();
		FName PackageName(*PackageNameToCreate);
		if (!CollectionFilter.Contains(PackageName))
		{
			return;
		}
	}

	if (bOnlyMaterials)
	{
		FString PackageNameToCreate = PackagePath.GetPackageNameOrFallback();
		FName PackageFNameToCreate(*PackageNameToCreate);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		TArray<FAssetData> PackageAssetData;
		if (!AssetRegistry.GetAssetsByPackageName(PackageFNameToCreate, PackageAssetData, true))
		{
			return;
		}

		bool bAnyMaterialsPresent = false;
		for (const FAssetData& AssetData : PackageAssetData)
		{
			UClass* AssetClass = AssetData.GetClass();
			if (AssetClass && AssetClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				bAnyMaterialsPresent = true;
				break;
			}
		}

		if (!bAnyMaterialsPresent)
		{
			return;
		}
	}

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

	if ( bIsReadOnly && !bVerifyContent && !bAutoCheckOut )
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Skipping read-only file %ls", *Filename);
		}
	}
	else
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogIteratePackagesCommandlet, Display, "Loading %ls", *Filename);
		}

		static int32 LastErrorCount = 0;

		int32 NumErrorsFromLoading = GWarn->GetNumErrors();
		if (NumErrorsFromLoading > LastErrorCount)
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "%d total errors encountered during loading", NumErrorsFromLoading);
		}
		LastErrorCount = NumErrorsFromLoading;

		// Get the package linker.
		VerboseMessage(TEXT("Pre GetPackageLinker"));

		bool bSavePackage = true;
		
		if (bSavePackage)
		{
			PackagesConsideredForResave++;

			// Only rebuild static meshes on load for the to be saved package.
			extern ENGINE_API FName GStaticMeshPackageNameToRebuild;
			FName PackageFName = PackagePath.GetPackageFName();
			check(!PackageFName.IsNone());
			GStaticMeshPackageNameToRebuild = PackageFName;

			// Assert if package couldn't be opened so we have no chance of messing up saving later packages.
			UPackage* Package = nullptr;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::LoadPackage);
				Package = LoadWorldPackageForEditor(PackagePath.GetPackageName());

				// Package = LoadPackage(NULL, *Filename, 0);
			}

			if (Package == NULL)
			{
				if (bCanIgnoreFails == true)
				{
					return;
				}
				else
				{
					check(Package);
				}
			}

			FLinkerLoad* Linker = Package->GetLinker();
			if(Linker)
			{
				PerformPreloadOperations(Linker, bSavePackage); 
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Warning, "Package %ls had no linker. Skipping PreloadOperations", *PackagePath.GetPackageName());
			}
			VerboseMessage(FString::Printf(TEXT("Post PerformPreloadOperations, Resave? %d"), bSavePackage));

			VerboseMessage(TEXT("Post LoadPackage"));

			// if we are only saving dirty packages and the package is not dirty, then we do not want to save the package (remember the default behavior is to ALWAYS save the package)
			if( ( bOnlySaveDirtyPackages == true ) && ( Package->IsDirty() == false ) )
			{
				bSavePackage = false;
			}

			// here we want to check and see if we have any loading warnings
			// if we do then we want to resave this package
			if( !bSavePackage && FParse::Param(FCommandLine::Get(),TEXT("SavePackagesThatHaveFailedLoads")) == true )
			{
				//UE_LOGF(LogIteratePackagesCommandlet, Warning, "NumErrorsFromLoading: %d GWarn->Errors num: %d", NumErrorsFromLoading, GWarn->GetNumErrors() );

				if( NumErrorsFromLoading != GWarn->GetNumErrors() )
				{
					bSavePackage = true;
				}
			}

			UWorld* World = UWorld::FindWorldInPackage(Package);
			if (World)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::PerformAdditionalOperations::World);
				PerformAdditionalOperations(World, bSavePackage);
			}
			
			// hook to allow performing additional checks without lumping everything into this one function
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::PerformAdditionalOperations::Package);
				PerformAdditionalOperations(Package,bSavePackage);
			}
			if (bUseWorldPartitionBuilder && World && UWorld::IsPartitionedWorld(World))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::ProcessWorldPartition);
				// Load configuration file
				FString WorldConfigFilename = FPackageName::LongPackageNameToFilename(World->GetPackage()->GetName(), TEXT(".ini"));
				if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*WorldConfigFilename))
				{
					LoadConfig(GetClass(), *WorldConfigFilename);
				}


				// process all the objects loaded with the base package.
				TArray<UObject*> PackageObjects;
				GetObjectsWithOuter(Package, PackageObjects);
				for (UObject* Object : PackageObjects)
				{
					PerformAdditionalOperations(Object, bSavePackage);
				}

				FScopedEditorWorld ScopeEditorWorld(World, WorldInitialisationValues);

				/*TUniquePtr<FLoaderAdapterShape> LoaderAdapterShape;
				FBox Bounds = FBox(FVector(-HALF_WORLD_MAX, -HALF_WORLD_MAX, -HALF_WORLD_MAX), FVector(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX));
				LoaderAdapterShape = MakeUnique<FLoaderAdapterShape>(World, Bounds, TEXT("Loaded Region"));
				LoaderAdapterShape->Load();*/

				TArray<UPackage*> PackagesToSave;

				FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;

				ForEachActorWithLoadingParams.ActorClasses = { AActor::StaticClass()};

				ForEachActorWithLoadingParams.OnPreGarbageCollect = [&PackagesToSave, this]()
					{
						SavePackages(PackagesToSave);
						//UWorldPartitionBuilder::SavePackages(PackagesToSave, PackageHelper, true);
						PackagesToSave.Empty();
					};

				ForEachActorWithLoadingParams.FilterActorDesc = [this](const FWorldPartitionActorDesc* ActorDesc) -> bool{ return this->FilterActorDesc(ActorDesc); };

				UWorldPartition* WorldPartition = World->GetWorldPartition();

				FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [&PackagesToSave, this](const FWorldPartitionActorDescInstance* ActorDescInstance)
					{
						AActor* Actor = ActorDescInstance->GetActor();

						if (!Actor)
						{
							WorldBuilderFailedLoadingActor(ActorDescInstance);
							return true;
						}

						bool bSavePackage = false;
						PerformWorldBuilderAdditionalOperations(Actor, bSavePackage);

						UPackage* Package = Actor->GetExternalPackage();
						check(Package);

						TArray<UObject*> DependantObjects;
						ForEachObjectWithPackage(Package, [this, &bSavePackage, &DependantObjects](UObject* Object)
							{
								if (!IsValid(Object))
								{
									return true;
								}
								DependantObjects.Add(Object);
								return true;
							}, EGetObjectsFlags::IncludeNestedObjects);

						for (UObject* Object : DependantObjects)
						{
							PerformWorldBuilderAdditionalOperations(Object, bSavePackage);
						}

						if (bSavePackage)
						{
							PackagesToSave.Add(Package);
							return true;
						}
						return true;
					}, ForEachActorWithLoadingParams);

				SavePackages(PackagesToSave);
				PackagesToSave.Empty();
			}
			else
			{
				VerboseMessage(TEXT("Post PerformAdditionalOperations"));

				// Check for any special per object operations
				TArray<UObject*> PackageObjects;
				ForEachObjectWithOuter(Package, [this, &PackageObjects](UObject* Object)
					{
						if (!IsValid(Object))
						{
							return;
						}
						PackageObjects.Add(Object);
					});
				for(UObject* Object : PackageObjects)
				{
					PerformAdditionalOperations(Object, bSavePackage);
				}
			}

			PostPerformAdditionalOperations(Package);

			VerboseMessage(TEXT("Post PerformAdditionalOperations Loop"));

			if (bStripEditorOnlyContent)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Log, "Removing editor only data");
				Package->SetPackageFlags(PKG_FilterEditorOnly);
			}

			if (bResaveOnDemand && bSavePackage)
			{
				// Finish all async loading for the package, in case the systems marking it for load do so only from the async loading
				FAssetCompilingManager::Get().FinishAllCompilation();
				FlushAsyncLoading();
				// Skip saving the package if no systems requested it
				FScopeLock ScopeLock(&ResaveOnDemandPackagesLock);
				bSavePackage = ResaveOnDemandPackages.Contains(Package->GetFName());
			}

			if (bSavePackage == true)
			{
				// Check to see if this package contains only metadata, and if so delete the package instead of resaving it

				TArray<UObject *> ObjectsInOuter;
				GetObjectsWithOuter(Package, ObjectsInOuter);
				const bool bIsEmpty = ObjectsInOuter.Num() == 0;

				if (bIsEmpty)
				{
					bSavePackage = false;
					Package = nullptr;
					
					UE_LOGF(LogIteratePackagesCommandlet, Display, "Package %ls is empty and will be deleted", *Filename);

					DeleteOnePackage(Filename);
				}
			}

			// Now based on the computation above we will see if we should actually attempt
			// to save this package
			if (bSavePackage == true)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::SavePackage);
				if( bIsReadOnly == true && bVerifyContent == true && bAutoCheckOut == false )
				{
					UE_LOGF(LogIteratePackagesCommandlet, Warning, "Package [%ls] is read-only but needs to be resaved (UE Version: %i, Licensee Version: %i  Current UE Version: %i, Current Licensee Version: %i)",
						*Filename, Linker->Summary.GetFileVersionUE().ToValue(), Linker->Summary.GetFileVersionLicenseeUE(), GPackageFileUEVersion.ToValue(), VER_LATEST_ENGINE_LICENSEEUE4 );
					if( SavePackageHelper(Package, FString(TEXT("Temp.temp"))) )
					{
						UE_LOGF(LogIteratePackagesCommandlet, Warning, "Correctly saved:  [Temp.temp]." );
					}
				}
				else
				{
					// Check to see if we need to check this package out (but do not check out here if SourceControlQueue is enabled)
					const bool bAttemptCheckoutNow = bAutoCheckOut && SourceControlQueue == nullptr;
					if (bAttemptCheckoutNow)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::AutoCheckOut);
						if( bIsReadOnly )
						{
							VerboseMessage(TEXT("Pre ForceGetStatus1"));
							ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
							FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState( Package, EStateCacheUsage::ForceUpdate );
							if(SourceControlState.IsValid())
							{
								FString OtherCheckedOutUser;
								if( SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser) )
								{
									UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] Overwriting package %ls already checked out by someone else (%ls), will not submit", *Filename, *OtherCheckedOutUser);
								}
								else if( !SourceControlState->IsCurrent() )
								{
									UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] Overwriting package %ls (not at head), will not submit", *Filename);
								}
								else
								{
									VerboseMessage(TEXT("Pre CheckOut"));

									SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), Package);

									VerboseMessage(TEXT("Post CheckOut"));

									FilesToSubmit.AddUnique(*Filename);
								}
							}
							VerboseMessage(TEXT("Post ForceGetStatus2"));
						}
					}

					// Update the readonly state now that source control has had a chance to run
					bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

					// If we tried to check out the file but it is still read only then we failed and 
					// need to emit an error and go to the next package			
					if (bAttemptCheckoutNow && bIsReadOnly)
					{
						if (bSkipCheckedOutFiles)
						{
							UE_LOGF(LogIteratePackagesCommandlet, Warning, "Skipping Package: %ls (unable to check out)", *Filename);
						}
						else
						{
							UE_LOGF(LogIteratePackagesCommandlet, Error, "Unable to check out the Package: %ls", *Filename);
						}
						
						return;
					}

					if (Verbosity != ONLY_ERRORS)
					{
						UE_LOGF(LogIteratePackagesCommandlet, Display, "Resaving package [%ls] (UE Version: %i, Licensee Version: %i  Saved UE Version: %i, Saved Licensee Version: %i)",
							*Filename,Linker->Summary.GetFileVersionUE().ToValue(), Linker->Summary.GetFileVersionLicenseeUE(), GPackageFileUEVersion.ToValue(), VER_LATEST_ENGINE_LICENSEEUE4 );
					}

					ESaveFlags SaveFlags = SAVE_None;
					
					if (bIsReadOnly == false || SourceControlQueue == nullptr)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::LoadAndSaveOnePackage::SavePackage);

						if (SavePackageHelper(Package, Filename, RF_Standalone, GWarn, SaveFlags))
						{
							PackagesResaved++;
							if (Verbosity == VERY_VERBOSE)
							{
								UE_LOGF(LogIteratePackagesCommandlet, Display, "Correctly saved:  [%ls].", *Filename);
							}
						}
					}
					else
					{
						// The target file is still read only and we have SourceControlQueue enabled so we
						// need to save to a temporary file first, and then queue the result.
						const FString TempFilename = CreateTempFilename();

						if (SavePackageHelper(Package, TempFilename, RF_Standalone, GWarn, SaveFlags))
						{
							SourceControlQueue->QueueCheckoutAndReplaceOperation(Filename, TempFilename, Package);
						}
					}
				}
			}
		}
		static int32 Counter = 0;

		if (!GarbageCollectionFrequency || Counter++ % GarbageCollectionFrequency == 0)
		{
			if (FApp::CanEverRender() || bForceFinishAllCompilationBeforeGC)
			{
				FAssetCompilingManager::Get().FinishAllCompilation();
			}

			if (GarbageCollectionFrequency > 1)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "GC");
			}
			VerboseMessage(TEXT("Pre CollectGarbage"));

			CollectGarbage(RF_NoFlags);

			VerboseMessage(TEXT("Post CollectGarbage"));
		}
	}
}


void UBaseIteratePackagesCommandlet::DeleteOnePackage(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::DeleteOnePackage);

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

	if (bVerifyContent)
	{
		return;
	}

	if (bIsReadOnly && !bAutoCheckOut)
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Skipping read-only file %ls", *Filename);
		}
		return;
	}

	if (SourceControlQueue != nullptr)
	{
		// All files (read only and non read only) need to query their source control status so
		// they should all be queued if avaliable.
		SourceControlQueue->QueueDeleteOperation(Filename);
		return;
	}

	FString PackageName;
	FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName);

	UPackage* Package = FindPackage(nullptr, *PackageName);

	if (Package)
	{
		// Unload package so we can delete it
		TArray<UPackage *> PackagesToDelete;
		PackagesToDelete.Add(Package);
		UPackageTools::UnloadPackages(PackagesToDelete);
		PackagesToDelete.Empty();
		Package = nullptr;
	}

	FString PackageFilename = SourceControlHelpers::PackageFilename(Filename);
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(PackageFilename, EStateCacheUsage::ForceUpdate);

	if (SourceControlState.IsValid() && (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded()))
	{
		UE_LOGF(LogIteratePackagesCommandlet, Display, "Revert '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), PackageFilename);

		UE_LOGF(LogIteratePackagesCommandlet, Display, "Deleting '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), PackageFilename);

		PackagesDeleted++;

		FilesToSubmit.AddUnique(Filename);
	}
	else if (SourceControlState.IsValid() && SourceControlState->CanCheckout())
	{
		UE_LOGF(LogIteratePackagesCommandlet, Display, "Deleting '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), PackageFilename);

		PackagesDeleted++;

		FilesToSubmit.AddUnique(Filename);
	}
	else if (SourceControlState.IsValid() && SourceControlState->IsCheckedOutOther())
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning, "Couldn't delete '%ls' from revision control, someone has it checked out, skipping...", *Filename);
	}
	else if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning, "'%ls' is not in revision control, attempting to delete from disk...", *Filename);
		if (IFileManager::Get().Delete(*Filename, false, true) == true)
		{
			PackagesDeleted++;
		}
		else
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "  ... failed to delete from disk.");
		}
	}
	else
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning, "'%ls' is in an unknown revision control state, attempting to delete from disk...", *Filename);
		if (IFileManager::Get().Delete(*Filename, false, true)== true)
		{
			PackagesDeleted++;
		}
		else
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "  ... failed to delete from disk.");
		}
	}
}

int32 UBaseIteratePackagesCommandlet::Main( const FString& Params )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet);

	const TCHAR* Parms = *Params;
	TArray<FString> Tokens;
	ParseCommandLine(Parms, Tokens, Switches);

	// Ensure source control is initialized and shut down properly
	FScopedSourceControl SourceControl;

	// skip the assert when a package can not be opened
	bCanIgnoreFails = Switches.Contains(TEXT("SKIPFAILS"));
	/** load all packages, and display warnings for those packages which would have been resaved but were read-only */
	bVerifyContent = Switches.Contains(TEXT("VERIFY"));
	/** if we should only save dirty packages **/
	bOnlySaveDirtyPackages = Switches.Contains(TEXT("OnlySaveDirtyPackages"));
	/** if we should auto checkout packages that need to be saved**/
	bAutoCheckOut = Switches.Contains(TEXT("AutoCheckOutPackages")) || Switches.Contains(TEXT("AutoCheckOut"));
	/** if we should batch source control operations*/
	bBatchSourceControl = Switches.Contains(TEXT("BatchSourceControl"));
	/** if we should simply skip checked out files rather than error-ing out */
	bSkipCheckedOutFiles = Switches.Contains(TEXT("SkipCheckedOutPackages"));
	/** if we should auto checkin packages that were checked out**/
	bAutoCheckIn = bAutoCheckOut && (Switches.Contains(TEXT("AutoCheckIn")) || Switches.Contains(TEXT("AutoSubmit")));
	/** determine if we can skip the version changelist check */
	bIgnoreChangelist = Switches.Contains(TEXT("IgnoreChangelist"));
	/** whether we should only save packages with changelist zero */
	bOnlyUnversioned = Switches.Contains(TEXT("OnlyUnversioned"));
	/** whether we should only save packages saved by licensees */
	bOnlyLicenseed = Switches.Contains(TEXT("OnlyLicenseed"));
	/** whether we should only save packages containing virtualized bulkdata payloads */
	bOnlyVirtualized = Switches.Contains(TEXT("OnlyVirtualized"));
	/** whether we should only save packages containing FPayloadTrailers */
	bOnlyPayloadTrailers = Switches.Contains(TEXT("OnlyPayloadTrailers"));
	/** only process packages containing materials */
	bOnlyMaterials = Switches.Contains(TEXT("onlymaterials"));

	// Start an async scan of the asset registry. The asset registry being loaded is required to set EPackageLocationFilter::FileSystem to support loading uncooked.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(false);

	bForceFinishAllCompilationBeforeGC = false;
	bRequireAssetRegistryLoadedBeforeIteration = true;

	bUseWorldPartitionBuilder = false;
	WorldInitialisationValues.RequiresHitProxies(false);
	WorldInitialisationValues.ShouldSimulatePhysics(false);
	WorldInitialisationValues.EnableTraceCollision(false);
	WorldInitialisationValues.CreateNavigation(false);
	WorldInitialisationValues.CreateAISystem(false);
	WorldInitialisationValues.AllowAudioPlayback(false);
	WorldInitialisationValues.CreatePhysicsScene(true);

	// UE_DEPRECATED(5.6)
	if (Switches.Contains(TEXT("KeepPackageGUIDOnSave")))
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning,
			"KeepPackageGUIDOnSave is no longer supported, because package save no longer records arbitrary guids but instead records a hash of the package's bytes - PackageSavedHash.");
	}
	
	/** check for filtering packages by collection. **/
	FString FilterByCollection;
	FParse::Value(*Params, TEXT("FilterByCollection="), FilterByCollection);

	CollectionFilter = TSet<FName>();
	if (!FilterByCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		TArray<FSoftObjectPath> CollectionAssets;
		if (!CollectionManager.TryParseCollectionPath(FilterByCollection, &CollectionContainer, &CollectionName, &ShareType) ||
			!CollectionContainer->GetAssetsInCollection(CollectionName, ShareType, CollectionAssets))
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Could not get assets in collection '%ls'. Skipping filter.", *FilterByCollection);
		}
		else
		{
			//insert all of the collection names into the set for fast filter checks
			for (const FSoftObjectPath& AssetPath : CollectionAssets)
			{
				CollectionFilter.Add(AssetPath.GetLongPackageFName());
			}
		}
	}


	TArray<FString> PackageNames;
	int32 ResultCode = InitializeParameters(Tokens, PackageNames);
	if ( ResultCode != 0 )
	{
		return ResultCode;
	}


	int32 GCIndex = 0;
	PackagesConsideredForResave = 0;
	PackagesResaved = 0;
	PackagesDeleted = 0;
	TotalPackagesForResave = PackageNames.Num();

	// allow for an option to restart at a given package name (in case it dies during a run, etc)
	bool bCanProcessPackage = true;
	FString FirstPackageToProcess;
	if (FParse::Value(*Params, TEXT("FirstPackage="), FirstPackageToProcess))
	{
		bCanProcessPackage = false;
	}
	FParse::Value(*Params, TEXT("PackageSubString="), PackageSubstring);
	if (PackageSubstring.Len())
	{
		UE_LOGF(LogIteratePackagesCommandlet, Display, "Restricted to packages containing %ls", *PackageSubstring );
	}

	// Avoid crash saving blueprint
	FFindInBlueprintSearchManager::Get();

	// Tick shader compiler if we are rendering
	if(IsAllowCommandletRendering() && GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(true, false);
	}

	if (bBatchSourceControl)
	{
		// Convert the commandlets verbosity to that of FQueuedSourceControlOperations
		FQueuedSourceControlOperations::EVerbosity LogVerbosity = FQueuedSourceControlOperations::EVerbosity::All;
		switch (Verbosity)
		{
		case VERY_VERBOSE:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::All;
			break;
		case INFORMATIVE:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::Info;
			break;
		case ONLY_ERRORS:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::ErrorsOnly;
			break;
		default:
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Unknown verbosity to pass to FQueuedSourceControlOperations!");
		}

		SourceControlQueue = MakePimpl<FQueuedSourceControlOperations>(LogVerbosity);	
		ParseSourceControlOptions(Switches);
	}

	// Make sure any remaining temp files from previous runs are removed
	CleanTempFiles();

	if(bRequireAssetRegistryLoadedBeforeIteration)
	{
		// Ensure the asset registry has loaded, before iterating.
		TRACE_CPUPROFILER_EVENT_SCOPE(AssetRegistry.WaitForCompletion);
		AssetRegistry.WaitForCompletion();
	}
	
	// Iterate over all packages.
	for( int32 PackageIndex = 0; PackageIndex < PackageNames.Num(); PackageIndex++ )
	{
		// Make sure we don't rebuild SMs that we're not going to save.
		extern ENGINE_API FName GStaticMeshPackageNameToRebuild;
		GStaticMeshPackageNameToRebuild = NAME_None;

		const FString& Filename = PackageNames[PackageIndex];

		// skip over packages before the first one allowed, if it was specified
		if (!bCanProcessPackage)
		{
			if (FPackageName::FilenameToLongPackageName(Filename) == FirstPackageToProcess)
			{
				bCanProcessPackage = true;
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "Skipping %ls", *Filename);
				continue;
			}
		}

		// Load and save this package
		LoadAndSaveOnePackage(Filename);

		// Check if we need to flush any source control operations yet
		if (SourceControlQueue != nullptr)
		{
			SourceControlQueue->FlushPendingOperations(false);
		}

		// Tick shader compiler if we are rendering
		if(IsAllowCommandletRendering() && GShaderCompilingManager)
		{
			GShaderCompilingManager->ProcessAsyncResults(true, false);
		}

		// Break out if we've resaved enough packages
		if( MaxPackagesToResave > -1 && PackagesResaved >= MaxPackagesToResave )
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Attempting to resave more than MaxPackagesToResave; exiting" );
			break;
		}
	}
	
	if (SourceControlQueue != nullptr)
	{
		// Flush any remaining source control operations
		SourceControlQueue->FlushPendingOperations(true);

		// Fix up the stats
		PackagesDeleted += SourceControlQueue->GetNumDeletedFiles();
		PackagesResaved += SourceControlQueue->GetNumReplacedFiles();

		// Add files that the source control queue modified to the list of files that 
		// will need to be submitted to source control
		FilesToSubmit.Append(SourceControlQueue->GetModifiedFiles());

		SourceControlQueue.Reset();
	}

	// Force a directory watcher and asset registry tick
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	DirectoryWatcherModule.Get()->Tick(-1.0f);

	AssetRegistry.Tick(-1.0f);

	PostProcessPackages();

	// Submit the results to source control
	CheckInFiles(FilesToSubmit, GetChangelistDescription());

	if (bForceUATEnvironmentVariableSet)
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("uebp_UATMutexNoWait"), TEXT("0"));		
	}

	UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %d/%d packages were considered for resaving", PackagesConsideredForResave, TotalPackagesForResave);
	UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %d/%d packages were resaved", PackagesResaved, PackagesConsideredForResave);
	UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %d/%d packages were deleted", PackagesDeleted, PackagesConsideredForResave);


	return 0;
}

FText UBaseIteratePackagesCommandlet::GetChangelistDescription() const
{
	FTextBuilder ChangelistDescription;
	ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionResavePackages", "Resave Packages"));
	return ChangelistDescription.ToText();
}


void UBaseIteratePackagesCommandlet::PerformPreloadOperations( FLinkerLoad* PackageLinker, bool& bSavePackage )
{
	const int32 UEPackageVersion = PackageLinker->Summary.GetFileVersionUE().ToValue();
	const int32 LicenseeUEPackageVersion = PackageLinker->Summary.GetFileVersionLicenseeUE();

	// validate that this package meets the minimum requirement
	if (MinResaveUEVersion != IGNORE_PACKAGE_VERSION && UEPackageVersion < MinResaveUEVersion)
	{
		bSavePackage = false;
		return;
	}

	// Check if this package meets the maximum requirements.
	const bool bNoLimitation = MaxResaveUEVersion == IGNORE_PACKAGE_VERSION && MaxResaveLicenseeUEVersion == IGNORE_PACKAGE_VERSION;
	const bool bAllowResave = bNoLimitation ||
						 (MaxResaveUEVersion != IGNORE_PACKAGE_VERSION && UEPackageVersion <= MaxResaveUEVersion) ||
						 (MaxResaveLicenseeUEVersion != IGNORE_PACKAGE_VERSION && LicenseeUEPackageVersion <= MaxResaveLicenseeUEVersion);
	// If not, don't resave it.
	if (!bAllowResave)
	{
		bSavePackage = false;
		return;
	}

	// If the package was saved with a higher engine version do not try to resave it. This also addresses problem with people 
	// building editor locally and resaving content with a 0 CL version (e.g. BUILD_FROM_CL == 0)
	if (!bIgnoreChangelist && PackageLinker->Summary.SavedByEngineVersion.GetChangelist() > FEngineVersion::Current().GetChangelist())
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning, "Skipping resave of %ls due to engine version mismatch (Package:%d, Editor:%d) ", 
			*PackageLinker->GetArchiveName(),
			PackageLinker->Summary.SavedByEngineVersion.GetChangelist(), 
			FEngineVersion::Current().GetChangelist());
		bSavePackage = false;
		return;
	}

	// Check if the changelist number is zero
	if (bOnlyUnversioned && PackageLinker->Summary.SavedByEngineVersion.GetChangelist() != 0)
	{
		bSavePackage = false;
		return;
	}

	// Check if the package was saved by licensees
	if (bOnlyLicenseed)
	{
		if (!PackageLinker->Summary.SavedByEngineVersion.IsLicenseeVersion())
		{
			bSavePackage = false;
			return;
		}
		else
		{
			UE_LOGF(LogIteratePackagesCommandlet, Display, "Resaving %ls that contains licensee version (%ls)",
				*PackageLinker->GetArchiveName(),
				*PackageLinker->Summary.SavedByEngineVersion.ToString());
		}
	}

	// Check if the package contains virtualized bulkdata payloads
	if (bOnlyVirtualized)
	{
		const UE::FPackageTrailer* Trailer = PackageLinker->GetPackageTrailer();
		if (Trailer == nullptr || Trailer->GetNumPayloads(UE::EPayloadStorageType::Virtualized) == 0)
		{
			bSavePackage = false;
			return;
		}
	}

	// Check if the package contains a FPackageTrailer or not
	if (bOnlyPayloadTrailers)
	{
		const UE::FPackageTrailer* Trailer = PackageLinker->GetPackageTrailer();
		if (Trailer == nullptr)
		{
			bSavePackage = false;
			return;
		}
	}

	// Check if the package contains any instances of the class that needs to be resaved.
	if (ResaveClasses.Num() > 0)
	{
		bSavePackage = false;
		for (int32 ExportIndex = 0; !bSavePackage && ExportIndex < PackageLinker->ExportMap.Num(); ExportIndex++)
		{
			FTopLevelAssetPath ExportClassPathName(PackageLinker->GetExportClassPackage(ExportIndex), PackageLinker->GetExportClassName(ExportIndex));
			if (ResaveClasses.Contains(ExportClassPathName.ToString()))
			{
				bSavePackage = true;
				break;
			}
		}
	}
}

bool UBaseIteratePackagesCommandlet::CheckoutFile(const FString& Filename, bool bAddFile, bool bIgnoreAlreadyCheckedOut)
{
	if (!bAutoCheckOut)
	{
		return true;
	}

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);
	if (!bIsReadOnly && !bAddFile)
	{
		return true;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	if (SourceControlState.IsValid())
	{
		// Already checked out/added this file
		if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
		{
			return true;
		}
		else if (!SourceControlState->IsSourceControlled())
		{
			if ( bAddFile )
			{
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *Filename) == ECommandResult::Succeeded)
				{
					UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %ls successfully added", *Filename);
					return true;
				}
				else
				{
					if (!bIgnoreAlreadyCheckedOut)
					{
						UE_LOGF(LogIteratePackagesCommandlet, Error, "[REPORT] %ls could not be added!", *Filename);
					}
					else
					{
						UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] %ls could not be added!", *Filename);
					}
				}
			}
		}
		else if (!SourceControlState->IsCurrent())
		{
			if (!bIgnoreAlreadyCheckedOut)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "[REPORT] %ls is not synced to head, can not submit", *Filename);
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] %ls is not synced to head, can not submit", *Filename);
			}
		}
		else if (!SourceControlState->CanCheckout())
		{
			FString CurrentlyCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&CurrentlyCheckedOutUser))
			{
				if (!bIgnoreAlreadyCheckedOut)
				{
					UE_LOGF(LogIteratePackagesCommandlet, Error, "[REPORT] %ls level is already checked out by someone else (%ls), can not submit!", *Filename, *CurrentlyCheckedOutUser);
				}
				else
				{
					UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] %ls level is already checked out by someone else (%ls), can not submit!", *Filename, *CurrentlyCheckedOutUser);
				}
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "[REPORT] Unable to checkout %ls, can not submit", *Filename);
			}
		}
		else 
		{
			if (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), *Filename) == ECommandResult::Succeeded)
			{
				UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %ls Checked out successfully", *Filename);
				return true;
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] %ls could not be checked out!", *Filename);
			}
		}
	}
	return false;
}

bool UBaseIteratePackagesCommandlet::RevertFile(const FString& Filename)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	bool bSuccesfullyReverted = false;

	if (SourceControlState.IsValid())
	{
		if (SourceControlState->CanRevert() && (SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), *Filename) == ECommandResult::Succeeded))
		{
			bSuccesfullyReverted = true;
			UE_LOGF(LogIteratePackagesCommandlet, Display, "[REPORT] %ls Reverted successfully", *Filename);			
		}
		else
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "[REPORT] %ls could not be reverted!", *Filename);
		}
	}

	return bSuccesfullyReverted;
}

bool UBaseIteratePackagesCommandlet::CanCheckoutFile(const FString& Filename, FString& CheckedOutUser)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	bool bCanCheckout = true;
	if (SourceControlState.IsValid())
	{
		if (!SourceControlState->CanCheckout())
		{
			if (!SourceControlState->IsCheckedOutOther(&CheckedOutUser))
			{
				CheckedOutUser = "";
			}

			bCanCheckout = false;
		}
	}

	return bCanCheckout;
}

void UBaseIteratePackagesCommandlet::CheckoutAndSavePackage(UPackage* Package, TArray<FString>& SublevelFilenames, bool bIgnoreAlreadyCheckedOut)
{
	check(Package);

	FString PackageFilename;
	if (FPackageName::TryConvertLongPackageNameToFilename(Package->GetName(), PackageFilename, Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension()))
	{
		if (IFileManager::Get().FileExists(*PackageFilename))
		{
			if (CheckoutFile(PackageFilename, true, bIgnoreAlreadyCheckedOut))
			{
				SublevelFilenames.Add(PackageFilename);
				if (!SavePackageHelper(Package, PackageFilename))
				{
					UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to save existing package %ls", *PackageFilename);
				}
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to check out existing package %ls", *PackageFilename);
			}
		}
		else
		{
			if (SavePackageHelper(Package, PackageFilename))
			{
				if (CheckoutFile(PackageFilename, true, bIgnoreAlreadyCheckedOut))
				{
					SublevelFilenames.Add(PackageFilename);
				}
				else
				{
					UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to check out new package %ls", *PackageFilename);
				}
			}
			else
			{
				UE_LOGF(LogIteratePackagesCommandlet, Error, "Failed to save new package %ls", *PackageFilename);
			}
		}
	}
}

void UBaseIteratePackagesCommandlet::CheckInFiles(const TArray<FString>& InFilesToSubmit, const FText& InDescription) const
{
	if (!bAutoCheckIn)
	{
		return;
	}

	// Check in all changed files
	if (InFilesToSubmit.Num() > 0)
	{
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
		CheckInOperation->SetDescription(InDescription);

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		SourceControlProvider.Execute(CheckInOperation, SourceControlHelpers::PackageFilenames(InFilesToSubmit));
	}
}


bool UBaseIteratePackagesCommandlet::CleanClassesFromContentPackages( UPackage* Package )
{
	check(Package);
	bool bResult = false;

	for ( TObjectIterator<UClass> It; It; ++It )
	{
		if ( It->IsIn(Package) )
		{
			UE_LOGF(LogIteratePackagesCommandlet, Warning, "Removing class '%ls' from package [%ls]", *It->GetPathName(), *Package->GetName());

			// mark the class as transient so that it won't be saved into the package
			It->SetFlags(RF_Transient);

			// clear the standalone flag just to be sure :)
			It->ClearFlags(RF_Standalone);
			bResult = true;
		}
	}

	return bResult;
}

void UBaseIteratePackagesCommandlet::VerboseMessage(const FString& Message)
{
	if (Verbosity == VERY_VERBOSE)
	{
		UE_LOGF(LogIteratePackagesCommandlet, Verbose, "%ls", *Message);
	}
}

FString UBaseIteratePackagesCommandlet::CreateTempFilename()
{
	return FPaths::CreateTempFilename(*GetTempFilesDirectory());
}

void UBaseIteratePackagesCommandlet::CleanTempFiles()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBaseIteratePackagesCommandlet::CleanTempFiles);
	const FString DirPath = GetTempFilesDirectory();

	UE_LOGF(LogIteratePackagesCommandlet, Display, "Cleaning temp file directory");

	if (!IFileManager::Get().DeleteDirectory(*DirPath, false, true))
	{
		UE_LOGF(LogIteratePackagesCommandlet, Warning, "Failed to clean temp file directory:  %ls", *DirPath);
	}
}

FString UBaseIteratePackagesCommandlet::GetTempFilesDirectory()
{
	return FPaths::ProjectSavedDir() / TEXT("Temp/ResavePackages");
}
