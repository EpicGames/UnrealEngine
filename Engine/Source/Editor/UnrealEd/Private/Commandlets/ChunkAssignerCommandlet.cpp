// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ChunkAssignerCommandlet.cpp: used to update cook output with chunk assignment.
	Updates asset registry
	Updates loc settings.
=============================================================================*/

#include "Commandlets/ChunkAssignerCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Async/ParallelFor.h"
#include "CookMetadata.h"
#include "HAL/FileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "IO/IoHash.h"
#include "Memory/SharedBuffer.h"
#include "Misc/App.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "LocalizationChunkDataGenerator.h"
#include "PipelineCacheUtilities.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CompactBinary.h"
#include "Settings/ProjectPackagingSettings.h"
#include "ZenStoreHttpClient.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ChunkAssignerCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogUnrealChunkAssigner, Log, All);

// Read the "ChunkAssignment" op out of the cook-platform's Zen oplog and resolve each
// per-file IoChunkId back to a package name using the same oplog's packagedata / bulkdata
// entries. The op writer (ChunkAssignmentJsonWriter::WriteOp) emits the IoChunkId per
// file plus optional pkg/path overrides for loose files, so we walk the oplog once to
// build a chunkid -> package-name map and use the inline pkg field when present.
//
// On-wire shape (see ChunkAssignmentJsonWriter.cs):
//   Op entry: { key: "ChunkAssignment", value: <BinaryAttachment rawHash> }
//   Attachment payload: { chunks: [ { id, name?, encguid?, files: [ { id?, pkg?, path? } ] } ] }
//
// Returns chunk-name (e.g. "pakchunk0") -> package names. Empty on any failure.
static TMap<FString, TArray<FName>> ReadChunkAssignmentsFromOplog(const FString& ProjectId, const FString& OplogId)
{
	TMap<FString, TArray<FName>> Result;

	if (ProjectId.IsEmpty() || OplogId.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"ReadChunkAssignmentsFromOplog: ProjectId/OplogId both required (got '%ls'/'%ls').",
			*ProjectId, *OplogId);
		return Result;
	}

	UE::FZenStoreHttpClient ZenStoreClient;
	ZenStoreClient.InitializeReadOnly(ProjectId, OplogId);
	if (!ZenStoreClient.IsConnected())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"ReadChunkAssignmentsFromOplog: failed to connect to Zen for %ls.%ls",
			*ProjectId, *OplogId);
		return Result;
	}

	// Fetch the full oplog so we can both (a) locate the ChunkAssignment entry and
	// (b) build an IoChunkId -> package-name lookup from every package entry.
	TIoStatusOr<FCbObject> OplogStatus = ZenStoreClient.GetOplog().Get();
	if (!OplogStatus.IsOk())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"ReadChunkAssignmentsFromOplog: failed to fetch oplog from %ls.%ls",
			*ProjectId, *OplogId);
		return Result;
	}
	FCbObject OplogObject = OplogStatus.ConsumeValueOrDie();

	TMap<FIoHash, FName> ChunkIdToPackageName;
	ChunkIdToPackageName.Reserve(OplogObject["entries"].AsArrayView().Num() * 2 /*packagedata+bulkdata*/);
	FIoHash ChunkAssignmentAttachmentHash;
	bool bFoundChunkAssignment = false;

	for (FCbFieldView EntryField : OplogObject["entries"].AsArrayView())
	{
		FCbObjectView EntryObj = EntryField.AsObjectView();

		// Find the ChunkAssignment op while we are walking the entries.
		if (!bFoundChunkAssignment && EntryObj["key"].AsString() == UTF8TEXTVIEW("ChunkAssignment"))
		{
			FCbFieldView ValueField = EntryObj["value"];
			if (ValueField.IsBinaryAttachment() || ValueField.IsHash())
			{
				ChunkAssignmentAttachmentHash = ValueField.AsHash();
				bFoundChunkAssignment = true;
			}
		}

		// Build the IoChunkId -> package name map from package entries. The "data" field on
		// each packagedata / bulkdata sub-object is the IoChunkId (IoHash) we want.
		FCbObjectView StoreEntry = EntryObj["packagestoreentry"].AsObjectView();
		FUtf8StringView PackageNameView = StoreEntry["packagename"].AsString();
		if (PackageNameView.IsEmpty())
		{
			continue;
		}
		const FName PackageName(PackageNameView);

		auto AddChunkIds = [&ChunkIdToPackageName, &PackageName, &EntryObj](const char* FieldName)
		{
			for (FCbFieldView FileField : EntryObj[FieldName].AsArrayView())
			{
				FCbObjectView FileObj = FileField.AsObjectView();
				FCbFieldView DataField = FileObj["data"];
				if (!DataField.HasValue())
				{
					continue;
				}
				const FIoHash ChunkId = DataField.AsHash();
				if (ChunkId.IsZero())
				{
					continue;
				}
				ChunkIdToPackageName.Emplace(ChunkId, PackageName);
			}
		};
		AddChunkIds("packagedata");
		AddChunkIds("bulkdata");
	}

	if (!bFoundChunkAssignment)
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"ReadChunkAssignmentsFromOplog: 'ChunkAssignment' op not found in %ls.%ls",
			*ProjectId, *OplogId);
		return Result;
	}

	// ReadChunk transparently downloads the compressed buffer and decompresses it, so the
	// returned FIoBuffer already contains the raw CbObject payload bytes.
	TIoStatusOr<FIoBuffer> ChunkResult = ZenStoreClient.ReadChunk(ChunkAssignmentAttachmentHash);
	if (!ChunkResult.IsOk())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"ReadChunkAssignmentsFromOplog: failed to read attachment %ls from %ls.%ls",
			*LexToString(ChunkAssignmentAttachmentHash), *ProjectId, *OplogId);
		return Result;
	}

	FIoBuffer PayloadBuffer = ChunkResult.ConsumeValueOrDie();
	FCbObject Payload(FSharedBuffer::MakeView(PayloadBuffer.GetData(), PayloadBuffer.GetSize()));

	int32 UnresolvedFiles = 0;
	for (FCbFieldView ChunkField : Payload["chunks"].AsArrayView())
	{
		FCbObjectView ChunkObj = ChunkField.AsObjectView();
		FString ChunkName(ChunkObj["name"].AsString());
		if (ChunkName.IsEmpty())
		{
			// Fall back to "pakchunk<id>" when only the numeric chunk id was emitted.
			const int32 ChunkIdNum = (int32)ChunkObj["id"].AsInt64(INT64_MIN);
			if (ChunkIdNum == INT64_MIN)
			{
				continue;
			}
			ChunkName = FString::Printf(TEXT("pakchunk%d"), ChunkIdNum);
		}

		TArray<FName>& Packages = Result.FindOrAdd(ChunkName);
		for (FCbFieldView FileField : ChunkObj["files"].AsArrayView())
		{
			FCbObjectView FileObj = FileField.AsObjectView();

			// Prefer the explicit pkg field when present (loose files / overrides).
			FUtf8StringView PkgView = FileObj["pkg"].AsString();
			if (!PkgView.IsEmpty())
			{
				Packages.Add(FName(PkgView));
				continue;
			}

			// Otherwise resolve via IoChunkId.
			FCbFieldView IdField = FileObj["id"];
			if (!IdField.HasValue())
			{
				++UnresolvedFiles;
				continue;
			}
			const FIoHash ChunkId = IdField.AsHash();
			if (const FName* PackageNamePtr = ChunkIdToPackageName.Find(ChunkId))
			{
				Packages.Add(*PackageNamePtr);
			}
			else
			{
				++UnresolvedFiles;
			}
		}
	}

	if (UnresolvedFiles > 0)
	{
		UE_LOGF(LogUnrealChunkAssigner, Warning,
			"ReadChunkAssignmentsFromOplog: %d file entry(s) in 'ChunkAssignment' could not be resolved against packagedata/bulkdata in %ls.%ls",
			UnresolvedFiles, *ProjectId, *OplogId);
	}

	UE_LOGF(LogUnrealChunkAssigner, Display,
		"ReadChunkAssignmentsFromOplog: read %d chunk(s) from oplog %ls.%ls",
		Result.Num(), *ProjectId, *OplogId);
	return Result;
}

// Inverts ReadChunkAssignmentsFromManifestFiles output from chunk→packages into package→sorted ChunkIds.
static TMap<FName, TArray<int32>> BuildPackageToChunkMap(const TMap<FString, TArray<FName>>& ChunkPackages)
{
	TMap<FName, TArray<int32>> Result;
	for (const TPair<FString, TArray<FName>>& Entry : ChunkPackages)
	{
		const int32 ChunkId = FCString::Atoi(*Entry.Key.Mid(8)); // strip "pakchunk" prefix
		for (const FName& PackageName : Entry.Value)
		{
			Result.FindOrAdd(PackageName).AddUnique(ChunkId);
		}
	}
	for (TPair<FName, TArray<int32>>& Entry : Result)
	{
		Entry.Value.Sort();
	}
	return Result;
}

static int32 UpdateAssetRegistry(
	const FString& InputPath,
	const FString& OutputPath,
	const bool bIsDevelopmentAR,
	const FAssetRegistrySerializationOptions& SaveOptions,
	const TMap<FName, TArray<int32>>& PackageChunks)
{
	FAssetRegistryState State;
	FAssetRegistrySerializationOptions LoadSerializationOptions(bIsDevelopmentAR ? UE::AssetRegistry::ESerializationTarget::ForDevelopment : UE::AssetRegistry::ESerializationTarget::ForGame);
	const FAssetRegistryLoadOptions LoadOptions(LoadSerializationOptions);
	if (!FAssetRegistryState::LoadFromDisk(*InputPath, LoadOptions, State))
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "Failed to load asset registry: %ls", *InputPath);
		return 1;
	}

	// Collect updates before applying them to avoid modifying the state during enumeration.
	TArray<TPair<const FAssetData*, FAssetData>> PendingUpdates;
	State.EnumerateAllAssets([&](const FAssetData& AssetData)
		{
			if (const TArray<int32>* NewChunks = PackageChunks.Find(AssetData.PackageName))
			{
				FAssetData NewData = AssetData;
				NewData.SetChunkIDs(TConstArrayView<int32>(*NewChunks));
				PendingUpdates.Add({ &AssetData, MoveTemp(NewData) });
			}
		});
	for (TPair<const FAssetData*, FAssetData>& Updated : PendingUpdates)
	{
		State.UpdateAssetData(const_cast<FAssetData*>(Updated.Key), MoveTemp(Updated.Value));
	}

	bool bSuccessWrite = false;
	if (bIsDevelopmentAR)
	{
		FBufferArchive64 SerializedAssetRegistry;
		bSuccessWrite = State.Save(SerializedAssetRegistry, SaveOptions);
		UE::Tasks::TTask<uint64> HashTask = UE::Tasks::Launch(TEXT("HashDevelopmentAssetRegistry"),
			[&SerializedAssetRegistry]()
			{
				FMemoryView ToHash = MakeMemoryView(SerializedAssetRegistry);
				return UE::Cook::FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(ToHash);
			});
		bSuccessWrite &= FFileHelper::SaveArrayToFile(SerializedAssetRegistry, *OutputPath);
		uint64 DevArXxHash = HashTask.GetResult();

		// Update Cooked Metadata information.
		const FString CookedMetaPath = OutputPath.Replace(TEXT("DevelopmentAssetRegistry.bin"), TEXT("CookMetadata.ucookmeta"));
		if (FPaths::FileExists(CookedMetaPath))
		{
			UE::Cook::FCookMetadataState MetadataState;
			MetadataState.ReadFromFile(CookedMetaPath);
			MetadataState.SetAssociatedDevelopmentAssetRegistryHash(DevArXxHash);
			MetadataState.SaveToFile(CookedMetaPath);
		}
	}
	else
	{
		FArrayWriter SerializedAssetRegistry;
		SerializedAssetRegistry.SetFilterEditorOnly(true);
		bSuccessWrite = State.Save(SerializedAssetRegistry, SaveOptions);
		bSuccessWrite &= FFileHelper::SaveArrayToFile(SerializedAssetRegistry, *OutputPath);
	}
	if (!bSuccessWrite)
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "Failed to write asset registry: %ls", *OutputPath);
		return 1;
	}

	UE_LOGF(LogUnrealChunkAssigner, Display, "Updated asset registry: %ls", *OutputPath);
	return 0;
}

static int32 GenerateLocalizationByChunk(const FString& CookOutputDir, const ITargetPlatform* TargetPlatform, const TMap<FString, TArray<FName>>& ChunkPackages)
{
	GetMutableDefault<UProjectPackagingSettings>()->ReloadConfig();
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	const int32 CatchAllChunkId = PackagingSettings->LocalizationTargetCatchAllChunkId;

	// Build the list of localization targets to chunk, applying the same blocklist filtering as the cook
	TArray<FString> LocalizationTargetsToChunk = PackagingSettings->LocalizationTargetsToChunk;
	{
		TArray<FString> BlocklistLocalizationTargets;
		GConfig->GetArray(TEXT("Staging"), TEXT("DisallowedLocalizationTargets"), BlocklistLocalizationTargets, GGameIni);
		if (BlocklistLocalizationTargets.Num() > 0)
		{
			LocalizationTargetsToChunk.RemoveAll([&BlocklistLocalizationTargets](const FString& InLocalizationTarget)
			{
				return BlocklistLocalizationTargets.Contains(InLocalizationTarget);
			});
		}
	}
	if (LocalizationTargetsToChunk.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Display, "No localization targets to chunk; skipping GenerateLocalizationByChunk");
		return 0;
	}

	// Discover which cultures to process. Allow explicit override via -COOKCULTURES=en+fr+de+...
	TArray<FString> CookCultures;
	FString CulturesOverride;
	if (FParse::Value(FCommandLine::Get(), TEXT("COOKCULTURES="), CulturesOverride))
	{
		CulturesOverride.ParseIntoArray(CookCultures, TEXT("+"), true);
	}
	else
	{
		CookCultures = PackagingSettings->CulturesToStage;
	}
	TArray<FString> AllCulturesToCook(CookCultures);
	for (const FString& CultureName : CookCultures)
	{
		const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(CultureName);
		for (const FString& PrioritizedCultureName : PrioritizedCultureNames)
		{
			AllCulturesToCook.AddUnique(PrioritizedCultureName);
		}
	}
	AllCulturesToCook.Sort();
	if (AllCulturesToCook.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Warning, "No cultures discovered for localization chunking; skipping");
		return 0;
	}

	const FString LocalizationContentRoot  = FPaths::Combine(CookOutputDir, FApp::GetProjectName(), TEXT("Content"),  TEXT("Localization"));
	const FString LocalizationMetadataRoot = FPaths::Combine(CookOutputDir, FApp::GetProjectName(), TEXT("Metadata"), TEXT("Localization"));

	FLocalizationChunkDataGenerator Generator(CatchAllChunkId, MoveTemp(LocalizationTargetsToChunk), MoveTemp(AllCulturesToCook));
	for (const TPair<FString, TArray<FName>>& Entry : ChunkPackages)
	{
		const int32 ChunkId = FCString::Atoi(*Entry.Key.Mid(8)); // strip "pakchunk"
		const TSet<FName> PackageSet(Entry.Value);
		TArray<FString> OutFiles;
		Generator.GenerateChunkDataFilesForPaths(ChunkId, PackageSet, TargetPlatform, LocalizationContentRoot, LocalizationMetadataRoot, OutFiles);
	}

	UE_LOGF(LogUnrealChunkAssigner, Display, "GenerateLocalizationByChunk complete: %d chunks processed", ChunkPackages.Num());
	return 0;
}

int32 UChunkAssignerCommandlet::Main(const FString& CmdLineParams)
{
	FString CookDir;
	if (!FParse::Value(FCommandLine::Get(), TEXT("CookDir="), CookDir))
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "-CookDir is required");
		return 1;
	}

	FString PlatformName;
	if (!FParse::Value(FCommandLine::Get(), TEXT("Platform="), PlatformName))
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "-Platform is required");
		return 1;
	}

	// Resolve ITargetPlatform
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	if (!TPM)
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "ITargetPlatformManagerModule not available");
		return 1;
	}
	const ITargetPlatform* TargetPlatform = TPM->FindTargetPlatform(PlatformName);
	if (!TargetPlatform)
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "Unknown platform: %ls", *PlatformName);
		return 1;
	}

	// OplogChunkAssigner that produced the op supplies the
	// Zen project / oplog IDs on the command line.
	FString ProjectId;
	FString OplogId;
	if (!FParse::Value(FCommandLine::Get(), TEXT("ProjectId="), ProjectId) || ProjectId.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "-ProjectId is required (Zen project id for the cook platform)");
		return 1;
	}
	if (!FParse::Value(FCommandLine::Get(), TEXT("OplogId="), OplogId) || OplogId.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error, "-OplogId is required (Zen oplog id for the cook platform)");
		return 1;
	}

	TMap<FString, TArray<FName>> ChunkPackages = ReadChunkAssignmentsFromOplog(ProjectId, OplogId);
	if (ChunkPackages.IsEmpty())
	{
		UE_LOGF(LogUnrealChunkAssigner, Error,
			"No chunk assignments found in oplog %ls.%ls", *ProjectId, *OplogId);
		return 0;
	}

	const TMap<FName, TArray<int32>> PackageChunks = BuildPackageToChunkMap(ChunkPackages);

	FString AssetRegistryFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("AssetRegistryFile="), AssetRegistryFile))
	{
		FString OutputFile = AssetRegistryFile;
		FParse::Value(FCommandLine::Get(), TEXT("AssetRegistryOutputFile="), OutputFile);

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		FAssetRegistrySerializationOptions SaveOptions;
		AssetRegistry.InitializeSerializationOptions(SaveOptions, TargetPlatform, UE::AssetRegistry::ESerializationTarget::ForGame);
		SaveOptions.bSerializeAssetRegistry = true;

		if (int32 Ret = UpdateAssetRegistry(AssetRegistryFile, OutputFile, false, SaveOptions, PackageChunks))
		{
			return Ret;
		}
	}

	FString DevAssetRegistryFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("DevelopmentAssetRegistryFile="), DevAssetRegistryFile))
	{
		FString OutputFile = DevAssetRegistryFile;
		FParse::Value(FCommandLine::Get(), TEXT("DevelopmentAssetRegistryOutputFile="), OutputFile);

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		FAssetRegistrySerializationOptions DevSaveOptions;
		AssetRegistry.InitializeSerializationOptions(DevSaveOptions, TargetPlatform, UE::AssetRegistry::ESerializationTarget::ForDevelopment);
		DevSaveOptions.bKeepDevelopmentAssetRegistryTags = true;

		if (int32 Ret = UpdateAssetRegistry(DevAssetRegistryFile, OutputFile, true, DevSaveOptions, PackageChunks))
		{
			return Ret;
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("GenerateLocalization")))
	{
		if (int32 Ret = GenerateLocalizationByChunk(CookDir, TargetPlatform, ChunkPackages))
		{
			return Ret;
		}
	}

	return 0;
}
