// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CommandletSourceControlUtils.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Logging/LogCategory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "PackageTools.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SourceControlOperations.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogSourceControlUtils);

FQueuedSourceControlOperations::FQueuedSourceControlOperations(EVerbosity InVerbosity)
	: ReplacementFilesTotalSize(0)
	, TotalFilesDeleted(0)
	, TotalFilesCheckedOut(0)
	, TotalFilesReplaced(0)
	, Verbosity(InVerbosity)
{
	SetMaxNumQueuedPackages(1000);			// Default to 1k packages to prevent the batch size from getting too big
	SetMaxTemporaryFileTotalSize(5 * 1024);	// Default to 5GB of replacement files on disk to prevent disk bloat
}

FQueuedSourceControlOperations::~FQueuedSourceControlOperations()
{
	checkf(PendingCheckoutFiles.Num() == 0, TEXT("Not all tmp files were flushed!"));
	checkf(PendingDeleteFiles.Num() == 0, TEXT("Not all pending deletes were flushed!"));
}

void FQueuedSourceControlOperations::QueueDeleteOperation(const FString& FileToDelete)
{
	PendingDeleteFiles.Add(FileToDelete);

	if (Verbosity < EVerbosity::ErrorsOnly)
	{
		UE_LOGF(LogSourceControlUtils, Display, "Queuing for deletion '%ls' [CurrentlyQueued %d]...", *FileToDelete, PendingDeleteFiles.Num());
	}
}

void FQueuedSourceControlOperations::QueueCheckoutOperation(const FString& FileToCheckout, UPackage* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedSourceControlOperations::QueueCheckoutOperation);

	PendingCheckoutFiles.Emplace(Package, FileToCheckout, FString());

	if (Verbosity == EVerbosity::All)
	{
		const float SizeInMB = ReplacementFilesTotalSize / (1024.0f * 1024.0f);
		UE_LOGF(LogSourceControlUtils, Display, "Queuing for checkout '%ls' [CurrentlyQueued %d Size On Disk %.2f MB]...", *FileToCheckout, PendingCheckoutFiles.Num(), SizeInMB);
	}
}

void FQueuedSourceControlOperations::QueueCheckoutAndReplaceOperation(const FString& FileToCheckout, const FString& ReplacementFile, UPackage* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedSourceControlOperations::QueueCheckoutAndReplaceOperation);

	PendingCheckoutFiles.Emplace(Package, FileToCheckout, ReplacementFile);

	const int64 FileSizeBytes = IFileManager::Get().FileSize(*ReplacementFile);
	if (FileSizeBytes >= 0)
	{
		ReplacementFilesTotalSize += FileSizeBytes;
	}
	else
	{
		// This shouldn't be possible, if the original file doesn't exist the commandlet should not have
		// managed to get this far, but better safe than sorry.
		UE_LOGF(LogSourceControlUtils, Error, "Was unable to find the size for the tmp file (%ls) for %ls!", *ReplacementFile, *FileToCheckout);
	}

	if (Verbosity == EVerbosity::All)
	{
		const float SizeInMB = ReplacementFilesTotalSize / (1024.0f * 1024.0f);
		UE_LOGF(LogSourceControlUtils, Display, "Queued for checkout '%ls' [CurrentlyQueued %d Size On Disk %.2f MB]...", *FileToCheckout, PendingCheckoutFiles.Num(), SizeInMB);
	}
}

bool FQueuedSourceControlOperations::HasPendingOperations() const
{
	return PendingDeleteFiles.Num() != 0 || PendingCheckoutFiles.Num() != 0;
}

void FQueuedSourceControlOperations::FlushPendingOperations(bool bForceAll)
{
	FlushDeleteOperations(bForceAll);
	FlushCheckoutOperations(bForceAll);
}

void FQueuedSourceControlOperations::FlushDeleteOperations(bool bForceAll)
{
	// Early out if we have no entries
	if (PendingDeleteFiles.Num() == 0)
	{
		return;
	}

	const bool bIsBatchPackageLimitReached = (QueuedPackageFlushLimit >= 0 && PendingDeleteFiles.Num() >= QueuedPackageFlushLimit);

	// We only continue if bForceAll is true or we have more queued deletes than the flush limit
	if (bIsBatchPackageLimitReached == false && bForceAll == false)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedSourceControlOperations::FlushDeleteOperations);

	if (Verbosity < EVerbosity::ErrorsOnly)
	{
		UE_LOGF(LogSourceControlUtils, Display, "Attempting to delete %d packages", PendingDeleteFiles.Num());
	}

	IFileManager& FileManager = IFileManager::Get();

	// First we need to unload any of the packages that we want to delete
	UnloadPackages(PendingDeleteFiles);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	TArray<FSourceControlStateRef> FileStates;
	const ECommandResult::Type GetStateResult = SourceControlProvider.GetState(PendingDeleteFiles, FileStates, EStateCacheUsage::ForceUpdate);
	if (GetStateResult == ECommandResult::Succeeded)
	{
		TArray<FString> FilesToRevert;
		TArray<FString> FilesToDelete;

		for (int32 Index = 0; Index < PendingDeleteFiles.Num(); ++Index)
		{
			const FSourceControlStateRef& FileState = FileStates[Index];
			const FString& Filename = PendingDeleteFiles[Index];

			if (FileState->IsCheckedOut() || FileState->IsAdded())
			{
				FilesToRevert.Add(Filename);
			}
			else if (FileState->CanCheckout())
			{
				FilesToDelete.Add(Filename);
			}
			else if (FileState->IsCheckedOutOther())
			{
				UE_LOGF(LogSourceControlUtils, Warning, "Couldn't delete '%ls' from revision control, someone has it checked out, skipping...", *Filename);
			}
			else if (FileState->IsSourceControlled() == false)
			{
				UE_LOGF(LogSourceControlUtils, Warning, "'%ls' is not in revision control, attempting to delete from disk...", *Filename);
				if (FileManager.Delete(*Filename, false, true) == true)
				{
					TotalFilesDeleted++;
				}
				else
				{
					UE_LOGF(LogSourceControlUtils, Warning, "  ... failed to delete from disk.");
				}
			}
			else
			{
				UE_LOGF(LogSourceControlUtils, Warning, "'%ls' is in an unknown revision control state, attempting to delete from disk...", *Filename);
				if (FileManager.Delete(*Filename, false, true) == true)
				{
					TotalFilesDeleted++;
				}
				else
				{
					UE_LOGF(LogSourceControlUtils, Warning, "  ... failed to delete from disk.");
				}
			}
		}

		DeleteFilesFromSourceControl(FilesToRevert, true);
		DeleteFilesFromSourceControl(FilesToDelete, false);
	}
	else
	{
		for (const FString& Filename : PendingDeleteFiles)
		{
			UE_LOGF(LogSourceControlUtils, Warning, "'%ls' is in an unknown revision control state, attempting to delete from disk...", *Filename);
			if (!FileManager.Delete(*Filename, false, true))
			{
				UE_LOGF(LogSourceControlUtils, Warning, "  ... failed to delete from disk.");
			}
		}
	}

	// Clear the pending files now we are done
	PendingDeleteFiles.Empty();
}

void FQueuedSourceControlOperations::FlushCheckoutOperations(bool bForceAll)
{
	// Early out if we have no entries
	if (PendingCheckoutFiles.Num() == 0)
	{
		return;
	}

	const bool bIsPackageLimitReached = QueuedPackageFlushLimit >= 0 && PendingCheckoutFiles.Num() >= QueuedPackageFlushLimit;
	const bool bIsFileSizeLimitReached = QueueFileSizeFlushLimit >= 0 && ReplacementFilesTotalSize >= QueueFileSizeFlushLimit;

	// We only continue if bForceAll is true or we have more queued deletes than the flush limit
	// or we have more tmp data on disk than the size limit
	if (bIsPackageLimitReached == false && bIsFileSizeLimitReached == false && bForceAll == false)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedSourceControlOperations::FlushCheckoutOperations);

	if (Verbosity < EVerbosity::ErrorsOnly)
	{
		const float SizeInMB = ReplacementFilesTotalSize / (1024.0f * 1024.0f);
		UE_LOGF(LogSourceControlUtils, Display, "Attempting to checkout %d packages with %.2f MB of replacement files", PendingCheckoutFiles.Num(), SizeInMB);
	}

	IFileManager& FileManager = IFileManager::Get();

	TArray<FString> FilesToGetState;
	FilesToGetState.Reserve(PendingCheckoutFiles.Num());

	for (const FileCheckoutOperation& PendingFile : PendingCheckoutFiles)
	{
		FilesToGetState.Add(PendingFile.FileToCheckout);
	}

	VerboseMessage(TEXT("Pre ForceGetStatus1"));
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	TArray<FSourceControlStateRef> FileStates;
	const ECommandResult::Type GetStateResult = SourceControlProvider.GetState(FilesToGetState, FileStates, EStateCacheUsage::ForceUpdate);
	if (GetStateResult == ECommandResult::Succeeded)
	{
		TArray<FString> FilesToCheckOut;
		FilesToCheckOut.Reserve(FilesToGetState.Num());

		for (int32 FileIndex = 0; FileIndex < FilesToGetState.Num(); FileIndex++)
		{
			const FSourceControlStateRef& SourceControlState = FileStates[FileIndex];
			const FString& Filename = FilesToGetState[FileIndex];

			checkSlow(PendingCheckoutFiles[FileIndex].FileToCheckout == Filename);

			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				UE_LOGF(LogSourceControlUtils, Warning, "[REPORT] Overwriting package %ls already checked out by someone else (%ls), will not submit", *Filename, *OtherCheckedOutUser);
				FileManager.Delete(*PendingCheckoutFiles[FileIndex].ReplacementFile);
			}
			else if (!SourceControlState->IsCurrent())
			{
				UE_LOGF(LogSourceControlUtils, Warning, "[REPORT] Overwriting package %ls (not at head), will not submit", *Filename);
				FileManager.Delete(*PendingCheckoutFiles[FileIndex].ReplacementFile);
			}
			else
			{
				FilesToCheckOut.Add(Filename);
			}
		}

		if (FilesToCheckOut.Num() > 0)
		{
			VerboseMessage(TEXT("Pre CheckOut"));
			ECommandResult::Type CheckoutResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToCheckOut);
			VerboseMessage(TEXT("Post CheckOut"));

			if (CheckoutResult == ECommandResult::Succeeded)
			{
				if (Verbosity < EVerbosity::ErrorsOnly)
				{
					UE_LOGF(LogSourceControlUtils, Display, "Successfully checked out %d files from revision control", FilesToCheckOut.Num());
				}

				TotalFilesCheckedOut += FilesToCheckOut.Num();

				for (const FString& Filename : FilesToCheckOut)
				{
					ModifiedFiles.AddUnique(*Filename);
				}
			}
			else
			{
				UE_LOGF(LogSourceControlUtils, Error, "Failed to checkout %d files from revision control", FilesToCheckOut.Num());
			}
		}

		// Now move the temp files to their correct location
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FQueuedSourceControlOperations::MoveFiles);

			VerboseMessage(TEXT("Pre MoveTmpFiles"));

			uint32 NumPackagesMoved = 0;

			FString PackageName;

			for (const FileCheckoutOperation& PendingFile : PendingCheckoutFiles)
			{
				// Skip if we don't have a replacement file
				if (PendingFile.ReplacementFile.Len() <= 0)
				{
					continue;
				}

				// Try to find the package for each file before we overwrite it so that we can
				// call ::ResetLoadersForSave incase we have any open file handles on it.
				UPackage* Package = PendingFile.Package.Get();
				if (Package == nullptr)
				{
					// Either we were never given a package or the pointer is now invalid, so we need to
					// check via the filename.
					if (FPackageName::TryConvertFilenameToLongPackageName(PendingFile.FileToCheckout, PackageName))
					{
						Package = FindPackage(nullptr, *PackageName);
					}
				}

				ResetLoadersForSave(Package, *PendingFile.FileToCheckout);

				VerboseMessage(FString::Printf(TEXT("Moving %s to %s"), *PendingFile.ReplacementFile, *PendingFile.FileToCheckout));

				if (!FileManager.Move(*PendingFile.FileToCheckout, *PendingFile.ReplacementFile))
				{
					UE_LOGF(LogSourceControlUtils, Error, "[REPORT] Failed to move %ls to %ls!", *PendingFile.ReplacementFile, *PendingFile.FileToCheckout);
					FileManager.Delete(*PendingFile.ReplacementFile);
				}
				else
				{
					TotalFilesReplaced++;
					NumPackagesMoved++;
				}
			}
			VerboseMessage(TEXT("Post MoveTmpFiles"));

			if (Verbosity < EVerbosity::ErrorsOnly)
			{
				UE_LOGF(LogSourceControlUtils, Display, "Successfully flushed %d temp files to the correct location", NumPackagesMoved);
			}
		}
	}
	else
	{
		UE_LOGF(LogSourceControlUtils, Error, "[REPORT] Failed to get revision control status for files!");

		// Only delete the tmp files we are referencing rather than call ::CleanTempFiles
		for (const FileCheckoutOperation& PendingFile : PendingCheckoutFiles)
		{
			FileManager.Delete(*PendingFile.ReplacementFile);
		}
	}

	// Clear the pending files now we are done
	PendingCheckoutFiles.Empty();
	ReplacementFilesTotalSize = 0;

	VerboseMessage(TEXT("Post ForceGetStatus2"));
}

void FQueuedSourceControlOperations::UnloadPackages(const TArray<FString> PackageNames)
{
	FString PackageName;
	TArray<UPackage*> PackagesToUnload;

	// Build an array of packages to unload
	for (const FString& Filename : PackageNames)
	{
		if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
		{
			UPackage* Package = FindPackage(nullptr, *PackageName);

			if (Package != nullptr)
			{
				PackagesToUnload.Add(Package);
			}
		}
	}

	UPackageTools::UnloadPackages(PackagesToUnload);
}

void FQueuedSourceControlOperations::DeleteFilesFromSourceControl(const TArray<FString>& FilesToDelete, bool bShouldRevert)
{
	// Now revert and delete any files
	if (FilesToDelete.Num() > 0)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		if (Verbosity == EVerbosity::All)
		{
			const TCHAR* WorkType = bShouldRevert ? TEXT("Reverting then deleting") : TEXT("Deleting");

			UE_LOGF(LogSourceControlUtils, Display, "%ls '%d' files from revision control:", WorkType, FilesToDelete.Num());

			for (const FString& Filename : FilesToDelete)
			{
				UE_LOGF(LogSourceControlUtils, Display, "\t%ls", *Filename);
			}
		}

		if (bShouldRevert)
		{
			const ECommandResult::Type RevertResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), FilesToDelete);
			if (RevertResult != ECommandResult::Succeeded)
			{
				UE_LOGF(LogSourceControlUtils, Error, "Failed to revert the files from revision control!");
				return;
			}
		}

		const ECommandResult::Type DeleteResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), FilesToDelete);
		if (DeleteResult != ECommandResult::Succeeded)
		{
			UE_LOGF(LogSourceControlUtils, Error, "Failed to delete the files from revision control!");
			return;
		}

		TArray<FSourceControlStateRef> PostDeletedFileStates;
		SourceControlProvider.GetState(FilesToDelete, PostDeletedFileStates, EStateCacheUsage::Use);

		int32 SuccessfulDeletes = 0;
		for (int32 Index = 0; Index < FilesToDelete.Num(); ++Index)
		{
			if (PostDeletedFileStates[Index]->IsDeleted())
			{
				SuccessfulDeletes++;

				for (const FString& Filename : FilesToDelete)
				{
					ModifiedFiles.AddUnique(Filename);
				}
			}
		}

		TotalFilesDeleted += SuccessfulDeletes;

		if (Verbosity == EVerbosity::All)
		{
			UE_LOGF(LogSourceControlUtils, Display, "Successfully deleted '%d' files from revision control", SuccessfulDeletes);
		}
	}
}

void FQueuedSourceControlOperations::VerboseMessage(const FString& Message)
{
	if (Verbosity == EVerbosity::All)
	{
		UE_LOGF(LogSourceControlUtils, Verbose, "%ls", *Message);
	}
}