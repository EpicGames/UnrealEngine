// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSourceControlUtils, Log, All);

/**
* This class is a utility for UResavePackagesCommandlet to make it easier to batch up source control
* requests as it is significantly more efficient to perform the same operation on a large number of
* files at once rather than send a single request per file.
*
* The size of the batches are controlled by either the number of packages stored or the amount of temporary disk
* space that the queued requests are taking up, which ever limit is reached first.
*/
class FQueuedSourceControlOperations
{
public:
	/** Controls the verbosity of logging */
	enum class EVerbosity
	{
		/** Log all messages including detailed messages */
		All = 0,
		/** Do not log overly verbose messages */
		Info,
		/** Log only errors */
		ErrorsOnly
	};

	/**
	 * Constructor
	 *
	 * @param InVerbosity The verbosity level to be used when logging
	 */
	FQueuedSourceControlOperations(EVerbosity InVerbosity = EVerbosity::All);

	/** 
	 * Destructor
	 */
	~FQueuedSourceControlOperations();

	/**
	* Sets the maximum number of packages to be queued for each operation before we actually
	* flush the requests in a single batch.
	*
	* @param	MaxNumPackages	The number of packages that will be queued for each operation.
	*/
	void SetMaxNumQueuedPackages(int32 MaxNumPackages) 
	{ 
		QueuedPackageFlushLimit = MaxNumPackages;  
	}

	/**
	* Sets the maximum size (in MB) of the temporary files on disk for each operation before 
	* we actually flush the requests in in a single batch.
	*
	* @param	MaxTotalFileSizeInMB	The maximum size for each operation type.
	*/
	void SetMaxTemporaryFileTotalSize(int64 MaxTotalFileSizeInMB)
	{ 
		// Stored as bytes!
		QueueFileSizeFlushLimit = MaxTotalFileSizeInMB * (1024 * 1024);
	}

	/**
	* Queue a file to be deleted at some point in the future. Any required source control operations to enable
	* the file to be deleted will be handled internally.
	*
	* @param FileToDelete	The path of the file to be deleted
	*/
	void QueueDeleteOperation(const FString& FileToDelete);

	/**
	* Add a file to a list of files to be checked out from source control at a point in the future.
	*
	* In order to speed up processing a pointer to the loaded UPackage for the file can be provided, if it
	* is not then the eventual checkout operation will need to try finding the UPackage via the filename which
	* will be slower.
	*
	* @param	FileToCheckout	The path of the file to check out in source control
	* @param	Package			A pointer to the loaded UPackage for the file (optional)
	*/
	void QueueCheckoutOperation(const FString& FileToCheckout, UPackage* Package = nullptr);

	/**
	* Add a file to a list of files to be checked out from source control at a point in the future and to be replaced
	* with another file once the checkout has been completed.
	*
	* In order to speed up processing a pointer to the loaded UPackage for the file can be provided, if it 
	* is not then the eventual checkout operation will need to try finding the UPackage via the filename which 
	* will be slower.
	*
	* @param	FileToCheckout	The path of the file to check out in source control
	* @param	ReplacementFile The path of a temporary file to be written over 'FileToCheckout' once it is
	*							checked out. NOTE that the file will be deleted on either success or failure!
	* @param	Package			A pointer to the loaded UPackage for the file (optional)
	*/
	void QueueCheckoutAndReplaceOperation(const FString& FileToCheckout, const FString& ReplacementFile, UPackage* Package = nullptr);

	/**
	* Query if there are still packages queued for a source control operation or not.
	*
	* @return True if there are packages pending and false if there are none.
	*/
	bool HasPendingOperations() const;

	/**
	* Checks if any of the operation queues has reached the point where they should 
	* process or not. This should be called fairly regularly. 
	*
	* @param bForceAll	When true *ALL* pending source control operation queues will be  
	*					fully processed, no matter how few packages are waiting.
	*/
	void FlushPendingOperations(bool bForceAll);

	/**
	* Query the number of files that have been deleted so far.
	*/
	int32 GetNumDeletedFiles() const
	{
		return TotalFilesDeleted; 
	}

	/**
	* Query the number of files that have been checked our from source control so far.
	*/
	int32 GetNumCheckedOutFiles() const 
	{ 
		return TotalFilesCheckedOut;  
	}

	/**
	* Query the number of files that have been replaced by temporary files so far.
	*/
	int32 GetNumReplacedFiles() const 
	{ 
		return TotalFilesReplaced; 
	}

	/**
	* Returns a list of all files that have been modified.
	*/
	const TArray<FString>& GetModifiedFiles() const 
	{ 
		return ModifiedFiles; 
	}

private:

	void FlushDeleteOperations(bool bForceAll);
	void FlushCheckoutOperations(bool bForceAll);

	void UnloadPackages(const TArray<FString> PackageNames);
	void DeleteFilesFromSourceControl(const TArray<FString>& FilesToDelete, bool bShouldRevert);

	void VerboseMessage(const FString& Message);

	struct FileCheckoutOperation
	{
		FileCheckoutOperation(UPackage* InPackage, const FString& InFileToCheckout, const FString& InReplacementFile)
			: Package(InPackage)
			, FileToCheckout(InFileToCheckout)
			, ReplacementFile(InReplacementFile)
		{}

		TWeakObjectPtr<UPackage>	Package;

		FString						FileToCheckout;
		FString						ReplacementFile;

	};

	/** Queue for delete operations */
	TArray<FString> PendingDeleteFiles;
	/** Queue for checkout operations */
	TArray<FileCheckoutOperation> PendingCheckoutFiles;

	/** Total size of temporary replacement files for the checkout queue */
	int64 ReplacementFilesTotalSize;

	/** The total number of files in a queue before it will be flushed */
	int32 QueuedPackageFlushLimit;
	/** The total size of temporary replacement files before the queue is flushed */
	int64 QueueFileSizeFlushLimit;

	/** The number of files successfully deleted from source control */
	int32 TotalFilesDeleted;
	/** The number of files successfully checked out from source control */
	int32 TotalFilesCheckedOut;
	/** The number of files successfully replaced after being checked out from source control */
	int32 TotalFilesReplaced;

	/** A list of files that have been modified in source control */
	TArray<FString> ModifiedFiles;

	/** Controls which log messages are actually sent */
	EVerbosity Verbosity;
};