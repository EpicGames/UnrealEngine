// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CompileAllBlueprintsCommandlet.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Kismet2/CompilerResultsLog.h"
#include "EngineUtils.h"
#include "ISourceControlModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "Kismet2/KismetEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompileAllBlueprintsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCompileAllBlueprintsCommandlet, Log, All);

#define KISMET_COMPILER_MODULENAME "KismetCompiler"

UCompileAllBlueprintsCommandlet::UCompileAllBlueprintsCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bResultsOnly = false;
	bCookedOnly = false;
	bDirtyOnly = false;
	bSimpleAssetList = false;
	BlueprintBaseClassName = UBlueprint::StaticClass()->GetClassPathName();

	TotalNumFailedLoads = 0;
	TotalNumFatalIssues = 0;
	TotalNumWarnings = 0;
}

int32 UCompileAllBlueprintsCommandlet::Main(const FString& Params)
{
	InitCommandLine(Params);
	InitKismetBlueprintCompiler();

	BuildBlueprintAssetList();
	BuildBlueprints();
	
	LogResults();

	return (TotalNumFatalIssues + TotalNumFailedLoads);
}

void UCompileAllBlueprintsCommandlet::InitCommandLine(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> SwitchParams;
	ParseCommandLine(*Params, Tokens, Switches, SwitchParams);

	bResultsOnly = Switches.Contains(TEXT("ShowResultsOnly"));
	bDirtyOnly = Switches.Contains(TEXT("DirtyOnly"));
	bCookedOnly = Switches.Contains(TEXT("CookedOnly"));
	bSimpleAssetList = Switches.Contains(TEXT("SimpleAssetList"));
	
	RequireAssetTags.Empty();
	if (SwitchParams.Contains(TEXT("RequireTags")))
	{
		const FString& FullTagInfo = SwitchParams[TEXT("RequireTags")];
		ParseTagPairs(FullTagInfo, RequireAssetTags);
	}

	ExcludeAssetTags.Empty();
	if (SwitchParams.Contains(TEXT("ExcludeTags")))
	{
		const FString& FullTagInfo = SwitchParams[TEXT("ExcludeTags")];
		ParseTagPairs(FullTagInfo, ExcludeAssetTags);
	}

	IgnoreFolders.Empty();
	if (SwitchParams.Contains(TEXT("IgnoreFolder")))
	{
		const FString& AllIgnoreFolders = SwitchParams[TEXT("IgnoreFolder")];
		ParseIgnoreFolders(AllIgnoreFolders);
	}

	FileAllowList.Empty();
	if (SwitchParams.Contains(TEXT("AllowListFile")))
	{
		const FString& ListFullPath = SwitchParams[TEXT("AllowListFile")];
		ParseAllowList(ListFullPath);
	}

	if (SwitchParams.Contains(TEXT("BlueprintBaseClass")))
	{
		FString BlueprintBaseClassParam = *SwitchParams[TEXT("BlueprintBaseClass")];
		BlueprintBaseClassName = UClass::TryConvertShortTypeNameToPathName<UClass>(BlueprintBaseClassParam, ELogVerbosity::Warning, TEXT("UCompileAllBlueprintsCommandlet::InitCommandLine"));
		UE_CLOGF(BlueprintBaseClassName.IsNull(), LogCompileAllBlueprintsCommandlet, Error, "Failed to convert short class name -BlueprintBaseClass=\"%ls\" to path name", *BlueprintBaseClassParam);
	}
}

void UCompileAllBlueprintsCommandlet::ParseTagPairs(const FString& FullTagString, TArray<TPair<FString, TArray<FString>>>& OutputAssetTags)
{
	TArray<FString> AllTagPairs;
	FullTagString.ParseIntoArray(AllTagPairs, TEXT(";"));

	//Break All Tag Pairs into individual tags and values
	for (const FString& StringTagPair : AllTagPairs)
	{
		TArray<FString> ParsedTagPairs;
		StringTagPair.ParseIntoArray(ParsedTagPairs, TEXT(","));

		if (ParsedTagPairs.Num() > 0)
		{
			TArray<FString> TagValues;

			//Start AssetTagIndex at 1, as the first one is the key. We will be using index 0 in all adds as the key
			for (int AssetTagIndex = 1; AssetTagIndex < ParsedTagPairs.Num(); ++AssetTagIndex)
			{
				TagValues.Add(ParsedTagPairs[AssetTagIndex]);
			}

			TPair<FString, TArray<FString>> NewPair(ParsedTagPairs[0], TagValues);
			OutputAssetTags.Add(NewPair);
		}
	}
}

void UCompileAllBlueprintsCommandlet::ParseIgnoreFolders(const FString& FullIgnoreFolderString)
{
	TArray<FString> ParsedIgnoreFolders;
	FullIgnoreFolderString.ParseIntoArray(ParsedIgnoreFolders, TEXT(","));

	for (const FString& IgnoreFolder : ParsedIgnoreFolders)
	{
		IgnoreFolders.Add(IgnoreFolder.TrimQuotes());
	}
}

void UCompileAllBlueprintsCommandlet::ParseAllowList(const FString& ListFilePath)
{
	const FString FilePath = FPaths::ProjectDir() + ListFilePath;
	if (!FFileHelper::LoadANSITextFileToStrings(*FilePath, &IFileManager::Get(), FileAllowList))
	{
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Error, "Failed to Load AllowList File! : %ls", *FilePath);
	}
}

void UCompileAllBlueprintsCommandlet::BuildBlueprints()
{
	double LastGCTime = FPlatformTime::Seconds();

	for (FAssetData const& Asset : BlueprintAssetList)
	{
		if (ShouldBuildAsset(Asset))
		{
			FString const AssetPath = Asset.GetObjectPathString();
			UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Loading and Compiling: '%ls'...", *AssetPath);

			//Load with LOAD_NoWarn and LOAD_DisableCompileOnLoad as we are covering those explicitly with CompileBlueprint errors.
			UBlueprint* LoadedBlueprint = Cast<UBlueprint>(StaticLoadObject(Asset.GetClass(), /*Outer =*/nullptr, *AssetPath,nullptr, LOAD_NoWarn | LOAD_DisableCompileOnLoad));
			if (LoadedBlueprint == nullptr)
			{
				++TotalNumFailedLoads;
				UE_LOGF(LogCompileAllBlueprintsCommandlet, Error, "Failed to Load : '%ls'.", *AssetPath);
				continue;
			}
			else
			{
				CompileBlueprint(LoadedBlueprint);
			}

			const double TimeNow = FPlatformTime::Seconds();

			if (TimeNow - LastGCTime >= 10.0)
			{
				GEngine->TrimMemory();
				LastGCTime = TimeNow;
			}
		}
	}
}

void UCompileAllBlueprintsCommandlet::BuildBlueprintAssetList()
{
	BlueprintAssetList.Empty();

	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Loading Asset Registry...");
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
	AssetRegistryModule.Get().SearchAllAssets(/*bSynchronousSearch =*/true);
	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Finished Loading Asset Registry.");
	
	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Gathering All Blueprints From Asset Registry...");
	AssetRegistryModule.Get().GetAssetsByClass(BlueprintBaseClassName, BlueprintAssetList, true);
}

bool UCompileAllBlueprintsCommandlet::ShouldBuildAsset(FAssetData const& Asset) const
{
	bool bShouldBuild = true;

	if (bCookedOnly && Asset.GetClass() && !Asset.GetClass()->bCooked)
	{
		FString const AssetPath = Asset.GetObjectPathString();
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As is not cooked", *AssetPath);
		bShouldBuild = false;
	}

	if (IgnoreFolders.Num() > 0)
	{
		for (const FString& IgnoreFolder : IgnoreFolders)
		{
			if (Asset.GetObjectPathString().StartsWith(IgnoreFolder))
			{
				FString const AssetPath = Asset.GetObjectPathString();
				UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As Object is in an Ignored Folder", *AssetPath);
				bShouldBuild = false;
			}
		}
	}

	if ((ExcludeAssetTags.Num() > 0) && (CheckHasTagInList(Asset, ExcludeAssetTags)))
	{
		FString const AssetPath = Asset.GetObjectPathString();
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As has an excluded tag", *AssetPath);
		bShouldBuild = false;
	}

	if ((RequireAssetTags.Num() > 0) && (!CheckHasTagInList(Asset, RequireAssetTags)))
	{
		FString const AssetPath = Asset.GetObjectPathString();
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As the asset is missing a required tag", *AssetPath);
		bShouldBuild = false;
	}

	if ((FileAllowList.Num() > 0) && (!IsAssetAllowed(Asset)))
	{
		FString const AssetPath = Asset.GetObjectPathString();
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As the asset is not part of the allow list", *AssetPath);
		bShouldBuild = false;
	}

	if (bDirtyOnly)
	{
		const UPackage* AssetPackage = Asset.GetPackage();
		if ((AssetPackage == nullptr) || !AssetPackage->IsDirty())
		{
			FString const AssetPath = Asset.GetObjectPathString();
			UE_LOGF(LogCompileAllBlueprintsCommandlet, Verbose, "Skipping Building %ls: As Package is not dirty", *AssetPath);
			bShouldBuild = false;
		}
	}

	return bShouldBuild;
}

bool UCompileAllBlueprintsCommandlet::CheckHasTagInList(FAssetData const& Asset, const TArray<TPair<FString, TArray<FString>>>& TagCollectionToCheck) const 
{
	bool bContainedTag = false;

	for (const TPair<FString, TArray<FString>>& SingleTagAndValues : TagCollectionToCheck)
	{
		if (Asset.TagsAndValues.Contains(FName(*SingleTagAndValues.Key)))
		{
			const TArray<FString>& TagValuesToCheck = SingleTagAndValues.Value;
			if (TagValuesToCheck.Num() > 0)
			{
				for (const FString& IndividualValueToCheck : TagValuesToCheck)
				{
					
					if (Asset.TagsAndValues.ContainsKeyValue(FName(*SingleTagAndValues.Key), IndividualValueToCheck))
					{
						bContainedTag = true;
						break;
					}
				}
			}
			//if we don't have any values to check, just return true as the tag was included
			else
			{
				bContainedTag = true;
				break;
			}
		}
	}

	return bContainedTag;
}

bool UCompileAllBlueprintsCommandlet::IsAssetAllowed(FAssetData const& Asset) const
{
	bool bIsInList = false;

	const FString& AssetFilePath = Asset.GetObjectPathString();
	for (const FString& Entry : FileAllowList)
	{
		if (AssetFilePath == Entry)
		{
			bIsInList = true;
			break;
		}
	}

	return bIsInList;
}

void UCompileAllBlueprintsCommandlet::CompileBlueprint(UBlueprint* Blueprint)
{
	if (KismetBlueprintCompilerModule && Blueprint)
	{
		//Have to create a new MessageLog for each asset as the warning / error counts are cumulative
		FCompilerResultsLog MessageLog;
		//Need to prevent the Compiler Results Log from automatically outputting results if verbosity is too low
		if (bResultsOnly)
		{
			MessageLog.bSilentMode = true;
		}
		else
		{
			MessageLog.bAnnotateMentionedNodes = true;
		}
		MessageLog.SetSourcePath(Blueprint->GetPathName());
		MessageLog.BeginEvent(TEXT("Compile"));

		FKismetEditorUtilities::CompileBlueprint(Blueprint, EBlueprintCompileOptions::SkipGarbageCollection, &MessageLog);

		MessageLog.EndEvent();

		if ((MessageLog.NumErrors + MessageLog.NumWarnings) > 0)
		{
			AssetsWithErrorsOrWarnings.Add(Blueprint->GetPathName());
		
			TotalNumFatalIssues += MessageLog.NumErrors;
			TotalNumWarnings += MessageLog.NumWarnings;
		}
		
		for (TSharedRef<class FTokenizedMessage>& Message : MessageLog.Messages)
		{
			UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "%ls", *Message->ToText().ToString());
		}
	}
}

void UCompileAllBlueprintsCommandlet::InitKismetBlueprintCompiler()
{
	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Loading Kismit Blueprint Compiler...");
	//Get Kismet Compiler Setup. Static so that the expensive stuff only happens once per run.
	KismetBlueprintCompilerModule = &FModuleManager::LoadModuleChecked<IKismetCompilerInterface>(TEXT(KISMET_COMPILER_MODULENAME));
	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "Finished Loading Kismit Blueprint Compiler...");
}

void UCompileAllBlueprintsCommandlet::LogResults()
{
	//results output
	UE_LOGF(LogCompileAllBlueprintsCommandlet, Display, "\n\n\n===================================================================================\nCompiling Completed with %d errors and %d warnings and %d blueprints that failed to load.\n===================================================================================\n\n\n", TotalNumFatalIssues, TotalNumWarnings, TotalNumFailedLoads);

	//Assets with problems listing
	if (bSimpleAssetList && (AssetsWithErrorsOrWarnings.Num() > 0))
	{
		UE_LOGF(LogCompileAllBlueprintsCommandlet, Warning, "\n===================================================================================\nAssets With Errors or Warnings:\n===================================================================================\n");

		for (const FString& Asset : AssetsWithErrorsOrWarnings)
		{
			UE_LOGF(LogCompileAllBlueprintsCommandlet, Warning, "%ls", *Asset);
		}

		UE_LOGF(LogCompileAllBlueprintsCommandlet, Warning, "\n===================================================================================\nEnd of Asset List\n===================================================================================\n");
	}
}
