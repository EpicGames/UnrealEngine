// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CompileShadersTestBedCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "GlobalShader.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompileShadersTestBedCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCompileShadersTestBedCommandlet, Log, All);

UCompileShadersTestBedCommandlet::UCompileShadersTestBedCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UCompileShadersTestBedCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCompileShadersTestBedCommandlet::Main);

	StaticExec(nullptr, TEXT("log LogMaterial Verbose"));

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "CompileShadersTestBed");
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "This commandlet compiles global and default material shaders.  Used to profile and test shader compilation.");
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -collection=<name>                (You can also specify a collection of assets to narrow down the results e.g. if you maintain a collection that represents the actually used in-game assets).");
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -materials=<path1>+<path2>        (You can also specify a list of material asset paths separated by a '+' to narrow down the results.)");
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -all                              (You can specify -all to compile all global/default shaders as well as shaders for all materials/material instances found in a project.)")
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -ExcludeGlobalShaders             (Skip the compilation of global shaders.)")
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -ExcludeDefaultMaterials          (Skip the compilation of default material shaders.)")
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, " Optional: -ExcludeMaterials                 (Skip the compilation of non default material shaders.)")
		return 0;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.SearchAllAssets(true);

	// Optional list of materials to compile.
	TArray<FAssetData> MaterialList;

	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	// Find all materials/material instances if -all is specified
	if (Switches.Contains(TEXT("all")))
	{
		Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
		Filter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());
		AssetRegistry.GetAssets(Filter, MaterialList);
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Found %d materials/material instances in project.", MaterialList.Num());
	}
	else // otherwise parse -collection and -materials arguments
	{
		// Parse collection
		FString Collection;
		if (FParse::Value(*Params, TEXT("collection="), Collection, true))
		{
			if (!Collection.IsEmpty())
			{
				// Get the list of materials from a collection
				Filter.PackagePaths.Add(FName(TEXT("/Game")));
				Filter.bRecursivePaths = true;
				Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
				Filter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());

				ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
				TSharedPtr<ICollectionContainer> CollectionContainer;
				FName CollectionName;
				ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
				if (CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType))
				{
					CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
				}

				AssetRegistry.GetAssets(Filter, MaterialList);
				UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Found %d materials/material instances from collection %ls.", MaterialList.Num(), *Collection);
			}
		}
		// Process -materials= switches separated by a '+'
		TArray<FString> CmdLineMaterialEntries;
		const TCHAR* MaterialsSwitchName = TEXT("Materials");
		if (const FString* MaterialsSwitches = ParamVals.Find(MaterialsSwitchName))
		{
			MaterialsSwitches->ParseIntoArray(CmdLineMaterialEntries, TEXT("+"));
		}

		if (CmdLineMaterialEntries.Num())
		{
			// re-use the filter and only filter based on the passed in objects.
			Filter.ClassPaths.Empty();
			Filter.SoftObjectPaths.Empty();
			int MaterialsNumBefore = MaterialList.Num();
			for (const FString& MaterialPathString : CmdLineMaterialEntries)
			{
				const FSoftObjectPath MaterialPath(MaterialPathString);
				if (!Filter.SoftObjectPaths.Contains(MaterialPath))
				{
					Filter.SoftObjectPaths.Add(MaterialPath);
				}
			}

			AssetRegistry.GetAssets(Filter, MaterialList);
			UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Found %d/%d requested materials/material instances.", MaterialList.Num() - MaterialsNumBefore, Filter.SoftObjectPaths.Num());
			if ((MaterialList.Num() - MaterialsNumBefore) != CmdLineMaterialEntries.Num())
			{
				UE_LOGF(LogCompileShadersTestBedCommandlet, Error, "Some materials/material instances were not found.");
			}
		}
	}

	static constexpr bool bLimitExecutationTime = false;

	// For all active platforms
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

	UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Begin Compiling Shaders");

	for (ITargetPlatform* Platform : Platforms)
	{
		UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Compiling shaders for %ls...", *Platform->PlatformName());

		// Compile default materials
		if (!Switches.Contains(TEXT("ExcludeDefaultMaterials")))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DefaultMaterials);

			UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Compile default materials");

			for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
			{
				UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(static_cast<EMaterialDomain>(Domain));

				DefaultMaterial->BeginCacheForCookedPlatformData(Platform);
				while (!DefaultMaterial->IsCachedCookedPlatformDataLoaded(Platform))
				{
					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);
				}
				DefaultMaterial->ClearCachedCookedPlatformData(Platform);
			}
		}

		// Compile global shaders
		if (!Switches.Contains(TEXT("ExcludeGlobalShaders")))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GlobalShaders);

			TArray<FName> DesiredShaderFormats;
			Platform->GetAllTargetedShaderFormats(DesiredShaderFormats);

			if (!DesiredShaderFormats.IsEmpty())
			{
				TStringBuilder<4086> ShaderFormatListString;
				for (const FName& ShaderFormat : DesiredShaderFormats)
				{
					if (ShaderFormatListString.Len() > 0)
					{
						ShaderFormatListString.Append(TEXT(", "));
					}
					ShaderFormatListString.Append(*ShaderFormat.ToString());
				}
				UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Compile global shaders for %ls", *ShaderFormatListString);

				for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
				{
					const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
					CompileGlobalShaderMap(ShaderPlatform, Platform, false);
					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, true /* bBlockOnGlobalShaderCompilation */);
					check(GGlobalShaderMap[ShaderPlatform]->IsComplete(Platform));
				}
			}
			else
			{
				UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "No target shader formats available to compile global shaders");
			}
		}

		TSet<UMaterialInterface*> MaterialsToCompile;

		// Begin Material Compiles
		if (!Switches.Contains(TEXT("ExcludeMaterials")))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BeginCacheForCookedPlatformData);

			// Sort the material lists by name so the order is stable.
			Algo::SortBy(MaterialList, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath(); }, [](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.LexicalLess(B); });

			UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Begin Cache For Cooked PlatformData");

			for (const FAssetData& AssetData : MaterialList)
			{
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
				{
					UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "BeginCache for %ls", *MaterialInterface->GetFullName());
					MaterialInterface->BeginCacheForCookedPlatformData(Platform);
					// need to call this once for all objects before any calls to ProcessAsyncResults as otherwise we'll potentially upload
					// incremental/incomplete shadermaps to DDC (as this function actually triggers compilation, some compiles for a particular
					// material may finish before we've even started others - if we call ProcessAsyncResults in that case the associated shader
					// maps will think they are "finished" due to having no outstanding dependencies).
					if (!MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						MaterialsToCompile.Add(MaterialInterface);
					}
				}
			}
		}

		int32 PreviousOutstandingJobs = 0;

		constexpr int32 MaxOutstandingJobs = 20000; // Having a max is a way to try to reduce memory usage.. otherwise outstanding jobs can reach 100k+ and use up 300gb committed memory

		// Submit all the jobs.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SubmitJobs);

			UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Submit Jobs");

			while (MaterialsToCompile.Num())
			{
				for (auto It = MaterialsToCompile.CreateIterator(); It; ++It)
				{
					UMaterialInterface* MaterialInterface = *It;
					if (MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						It.RemoveCurrent();
						UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Finished cache for %ls.", *MaterialInterface->GetFullName());
						UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Materials remaining: %d", MaterialsToCompile.Num());
					}

					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

					while (true)
					{
						const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
						if (CurrentOutstandingJobs != PreviousOutstandingJobs)
						{
							UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
							PreviousOutstandingJobs = CurrentOutstandingJobs;
						}

						// Flush rendering commands to release any RHI resources (shaders and shader maps).
						// Delete any FPendingCleanupObjects (shader maps).
						FlushRenderingCommands();

						if (CurrentOutstandingJobs < MaxOutstandingJobs)
						{
							break;
						}
						FPlatformProcess::Sleep(1);
					}
				}
			}
		}

		// Process the shader maps and save to the DDC.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessShaderCompileResults);

			UE_LOGF(LogCompileShadersTestBedCommandlet, Log, "ProcessAsyncResults");

			while (GShaderCompilingManager->IsCompiling())
			{
				GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

				while (true)
				{
					const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
					if (CurrentOutstandingJobs != PreviousOutstandingJobs)
					{
						UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
						PreviousOutstandingJobs = CurrentOutstandingJobs;
					}

					// Flush rendering commands to release any RHI resources (shaders and shader maps).
					// Delete any FPendingCleanupObjects (shader maps).
					FlushRenderingCommands();

					if (CurrentOutstandingJobs < MaxOutstandingJobs)
					{
						break;
					}
					FPlatformProcess::Sleep(1);
				}
			}
		}

		// Perform cleanup and clear cached data for cooking.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ClearCachedCookedPlatformData);

			UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "Clear Cached Cooked Platform Data");

			for (const FAssetData& AssetData : MaterialList)
			{
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
				{
					MaterialInterface->ClearAllCachedCookedPlatformData();
				}
			}
		}
	}

	UE_LOGF(LogCompileShadersTestBedCommandlet, Display, "End compiling shaders");

	GShaderCompilingManager->PrintStats();

	return 0;
}
