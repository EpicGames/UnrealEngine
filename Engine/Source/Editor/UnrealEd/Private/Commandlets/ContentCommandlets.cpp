// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UCContentCommandlets.cpp: Various commmandlets.
=============================================================================*/

#include "Algo/RemoveIf.h"
#include "AssetCompilingManager.h"
#include "AssetRegistry/AssetData.h"
#include "CollectionManagerModule.h"
#include "CommandletSourceControlUtils.h"
#include "Commandlets/LandscapeGrassTypeCommandlet.h"
#include "Commandlets/ListMaterialsUsedWithMeshEmittersCommandlet.h"
#include "Commandlets/ListStaticMeshesImportedFromSpeedTreesCommandlet.h"
#include "Commandlets/ResavePackagesCommandlet.h"
#include "Commandlets/StaticMeshMinLodCommandlet.h"
#include "Commandlets/WrangleContentCommandlet.h"
#include "CoreMinimal.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "Engine/Brush.h"
#include "Engine/EngineTypes.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/StaticMesh.h"
#include "EngineGlobals.h"
#include "FileHelpers.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/FileManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "LandscapeGrassType.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Materials/Material.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/RedirectCollector.h"
#include "Modules/ModuleManager.h"
#include "PackageHelperFunctions.h"
#include "PackageTools.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleSystem.h"
#include "PlatformInfo.h"
#include "Serialization/BulkDataRegistry.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "StaticMeshCompiler.h"
#include "String/ParseTokens.h"
#include "UObject/Class.h"
#include "UObject/LinkerLoad.h"
#include "UObject/MetaData.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageTrailer.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "Virtualization/VirtualizationSystem.h"

DEFINE_LOG_CATEGORY(LogContentCommandlet);

#include "AssetRegistry/AssetRegistryModule.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Particles/Material/ParticleModuleMeshMaterial.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/TypeData/ParticleModuleTypeDataMesh.h"
#include "Engine/LevelStreaming.h"
#include "EditorBuildUtils.h"
#include "ExternalPackageHelper.h"

// for UResavePackagesCommandlet::PerformAdditionalOperations building lighting code
#include "LightingBuildOptions.h"

// for UResavePackagesCommandlet::PerformAdditionalOperations building navigation data
#include "EngineUtils.h"
#include "NavigationData.h"
#include "AI/NavigationSystemBase.h"

// For preloading FFindInBlueprintSearchManager
#include "FindInBlueprintManager.h"
#include "IHierarchicalLODUtilities.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "HierarchicalLOD.h"
#include "HierarchicalLODProxyProcessor.h"
#include "HLOD/HLODEngineSubsystem.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/ThreadManager.h"
#include "ShaderCompiler.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "UObject/UObjectThreadContext.h"
#include "Engine/LODActor.h"
#include "PerQualityLevelProperties.h"
#include "Misc/RedirectCollector.h"


/**-----------------------------------------------------------------------------
 *	UResavePackages commandlet.
 *
 * This commandlet is meant to resave packages as a default.  We are able to pass in
 * flags to determine which conditions we do NOT want to resave packages. (e.g. not dirty
 * or not older than some version)
 *
 *
----------------------------------------------------------------------------**/

#define CURRENT_PACKAGE_VERSION 0
#define IGNORE_PACKAGE_VERSION INDEX_NONE

UResavePackagesCommandlet::UResavePackagesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bForceUATEnvironmentVariableSet(false)
{

}

int32 UResavePackagesCommandlet::InitializeResaveParameters( const TArray<FString>& Tokens, TArray<FString>& PackageNames )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::InitializeResaveParameters);

	Verbosity = VERY_VERBOSE;

	TArray<FString> Unused;
	bool bExplicitPackages = false;

	// Check to see if we have an explicit list of packages
	for( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
	{
		FString Package;
		FString PackageFolderArg;
		FString Maps;
		FString File;
		FString CollectionArg;
		const FString& CurrentSwitch = Switches[ SwitchIdx ];
		if( FParse::Value( *CurrentSwitch, TEXT( "PACKAGE="), Package ) )
		{
			FString PackageFile;
			if (FPackageName::SearchForPackageOnDisk(Package, NULL, &PackageFile))
			{
				PackageNames.Add( *PackageFile );
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Error, "Failed to find the package given by the cmdline: PACKAGE=%ls", *Package);
			}
			
			bExplicitPackages = true;
		}
		else if( FParse::Value( *CurrentSwitch, TEXT( "PACKAGEFOLDER="), PackageFolderArg) )
		{
			// Allow support for -PACKAGEFOLDER=Value1+Value2+Value3
			TArray<FString> PackageFolders;
			PackageFolderArg.ParseIntoArray(PackageFolders, TEXT("+"));
			for (FString& PackageFolder : PackageFolders)
			{
				FPaths::NormalizeDirectoryName(PackageFolder);

				TArray<FString> FilesInPackageFolder;
				FPackageName::FindPackagesInDirectory(FilesInPackageFolder, PackageFolder);
				for( int32 FileIndex = 0; FileIndex < FilesInPackageFolder.Num(); FileIndex++ )
				{
					FString PackageFile(FilesInPackageFolder[FileIndex]);
					FPaths::MakeStandardFilename(PackageFile);
					PackageNames.Add( *PackageFile );
				}
			}

			UE_CLOGF(PackageNames.IsEmpty(), LogContentCommandlet, Warning, "Failed to find any packages in folder: '%ls'", *PackageFolderArg);
			bExplicitPackages = true;
		}
		else if (FParse::Value(*CurrentSwitch, TEXT("MAP="), Maps))
		{
			// Allow support for -MAP=Value1+Value2+Value3
			for (int32 PlusIdx = Maps.Find(TEXT("+"), ESearchCase::CaseSensitive); PlusIdx != INDEX_NONE; PlusIdx = Maps.Find(TEXT("+"), ESearchCase::CaseSensitive))
			{
				const FString NextMap = Maps.Left(PlusIdx);
				if (NextMap.Len() > 0)
				{
					FString MapFile;
					FPackageName::SearchForPackageOnDisk(NextMap, NULL, &MapFile);
					PackageNames.Add(*MapFile);
					bExplicitPackages = true;
				}

				Maps.RightInline(Maps.Len() - (PlusIdx + 1), EAllowShrinking::No);
			}
			FString MapFile;
			FPackageName::SearchForPackageOnDisk(Maps, NULL, &MapFile);
			PackageNames.Add(*MapFile);
			bExplicitPackages = true;
		}
		else if (FParse::Value(*CurrentSwitch, TEXT("FILE="), File))
		{
			FString Text;
			if (FFileHelper::LoadFileToString(Text, *File))
			{
				TArray<FString> Lines;

				// Remove all carriage return characters.
				Text.ReplaceInline(TEXT("\r"), TEXT(""), ESearchCase::CaseSensitive);
				// Read all lines
				Text.ParseIntoArray(Lines, TEXT("\n"), true);

				for (auto Line : Lines)
				{
					FString PackageFile;
					if (FPackageName::SearchForPackageOnDisk(Line, NULL, &PackageFile))
					{
						PackageNames.AddUnique(*PackageFile);
					}
					else
					{
						UE_LOGF(LogContentCommandlet, Error, "Failed to find package %ls", *Line);
					}
				}

				bExplicitPackages = true;
				UE_LOGF(LogContentCommandlet, Display, "Loaded %d Packages from %ls", PackageNames.Num(), *File);
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Error, "Failed to load file %ls", *File);
			}
		}
		else if (FParse::Value(*CurrentSwitch, TEXT("COLLECTION="), CollectionArg))
		{
			const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
			TArray<FSoftObjectPath> CollectionObjects;

			TArray<FString> Collections;
			CollectionArg.ParseIntoArray(Collections, TEXT("+"));
			for (const FString& Collection : Collections)
			{
				if (!CollectionContainer->CollectionExists(*Collection, ECollectionShareType::CST_All))
				{
					UE_LOGF(LogContentCommandlet, Error, "Unknown source collection '%ls'", *Collection);
					continue;
				}

				CollectionObjects.Reset();
				if (!CollectionContainer->GetAssetsInCollection(*Collection, ECollectionShareType::CST_All, CollectionObjects))
				{
					UE_LOGF(LogContentCommandlet, Error, "Failed to get assets in collection '%ls'", *Collection);
					continue;
				}

				for (const FSoftObjectPath& CollectionObject : CollectionObjects)
				{
					PackageNames.Add(CollectionObject.GetLongPackageName());
					bExplicitPackages = true;
				}
			}
		}
	}

	// Check for numeric settings
	for (const FString& CurrentSwitch : Switches)
	{
		if (FParse::Value(*CurrentSwitch, TEXT("GCFREQ="), GarbageCollectionFrequency))
		{
			UE_LOGF(LogContentCommandlet, Display, "Setting garbage collection to happen every %d packages.", GarbageCollectionFrequency);
		}
	}

	if ((bShouldBuildLighting || bShouldBuildReflectionCaptures) && !bExplicitPackages)
	{
		UE_LOGF(LogContentCommandlet, Display, "No maps found to save when building lighting, checking CommandletSettings:ResavePackages in EditorIni");
		// if we haven't specified any maps and we are building lighting check if there are packages setup in the ini file to build
		TArray<FString> ResavePackages;
		GConfig->GetArray(TEXT("CommandletSettings"), TEXT("ResavePackages"), ResavePackages, GEditorIni);
		for ( const auto& ResavePackage : ResavePackages )
		{
			FString PackageFile;
			FPackageName::SearchForPackageOnDisk(ResavePackage, NULL, &PackageFile);
			UE_LOGF(LogContentCommandlet, Display, "Rebuilding lighting for package %ls", *PackageFile);
			PackageNames.Add(*PackageFile);
			bExplicitPackages = true;
		}
	}

	if (bShouldBuildHLOD && !bExplicitPackages)
	{
		bool bWaitingForMapToSkipTo = !HLODSkipToMap.IsEmpty();

		const UHierarchicalLODSettings* Settings = GetDefault<UHierarchicalLODSettings>();
		for (const FDirectoryPath& Path : Settings->DirectoriesForHLODCommandlet)
		{
			TArray<FString> FilesInPackageFolder;			
			FPackageName::FindPackagesInDirectory(FilesInPackageFolder, FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir(), Path.Path));
			for (int32 FileIndex = 0; FileIndex < FilesInPackageFolder.Num(); FileIndex++)
			{
				FString PackageFile(FilesInPackageFolder[FileIndex]);
				FPaths::MakeStandardFilename(PackageFile);

				if( bWaitingForMapToSkipTo )
				{
					if( FPaths::GetBaseFilename( PackageFile ) == FPaths::GetBaseFilename( HLODSkipToMap ) )
					{
						bWaitingForMapToSkipTo = false;
					}
				}

				if( !bWaitingForMapToSkipTo )
				{
					PackageNames.AddUnique(*PackageFile);
					bExplicitPackages = true;
				}
			}
		}

		for (const FFilePath& FilePath : Settings->MapsToBuild)
		{
			FString Path = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir(), FilePath.FilePath);
			FString OutPath;
			if (FPackageName::DoesPackageExist(Path, &OutPath))
			{				
				PackageNames.AddUnique(*OutPath);
				bExplicitPackages = true;
			}
		}		
	}

	// ... if not, load in all packages
	if( !bExplicitPackages || Switches.Contains(TEXT("SaveAll")))
	{
		UE_CLOGF( bShouldBuildHLOD, LogContentCommandlet, Display, "No maps found to save when building HLODs, checking Project Settings for Directory or Asset Path(s)" );

		uint8 PackageFilter = NORMALIZE_DefaultFlags;
		if ( Switches.Contains(TEXT("SKIPMAPS")) )
		{
			PackageFilter |= NORMALIZE_ExcludeMapPackages;
		}
		else if ( Switches.Contains(TEXT("MAPSONLY")) )
		{
			PackageFilter |= NORMALIZE_ExcludeContentPackages;
		}

		if (Switches.Contains(TEXT("PROJECTONLY")))
		{
			PackageFilter |= NORMALIZE_ExcludeEnginePackages;
		}

		if ( Switches.Contains(TEXT("SkipDeveloperFolders")) || Switches.Contains(TEXT("NODEV")) )
		{
			PackageFilter |= NORMALIZE_ExcludeDeveloperPackages;
		}
		else if ( Switches.Contains(TEXT("OnlyDeveloperFolders")) )
		{
			PackageFilter |= NORMALIZE_ExcludeNonDeveloperPackages;
		}

		const FString AssetSearch = TEXT("*") + FPackageName::GetAssetPackageExtension();
		const FString MapSearch = TEXT("*") + FPackageName::GetMapPackageExtension();

		bool bAnyFound = NormalizePackageNames(Unused, PackageNames, AssetSearch, PackageFilter);
		bAnyFound |= NormalizePackageNames(Unused, PackageNames, MapSearch, PackageFilter);
		
		if (!bAnyFound)
		{
			return 1;
		}
	}

	// Filtering out external actors/objects here avoids any issues that could happen with loading and saving them individually. They are instead handled within the context of their owning level in UResavePackagesCommandlet::PerformAdditionalOperations. For specifically resaving a World Partition map and its actors, see WorldPartitionResaveActorsBuilder.
	PackageNames.SetNum(Algo::RemoveIf(PackageNames, [](const FString& PackageName)
	{
		return PackageName.Contains(FPackagePath::GetExternalActorsFolderName()) || PackageName.Contains(FPackagePath::GetExternalObjectsFolderName());
	}));

	// Check for a max package limit
	MaxPackagesToResave = -1;
	for ( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
	{
		const FString& CurrentSwitch = Switches[SwitchIdx];
		if( FParse::Value(*CurrentSwitch,TEXT("MAXPACKAGESTORESAVE="),MaxPackagesToResave))
		{
			UE_LOGF(LogContentCommandlet, Warning, "Only resaving a maximum of %d packages.", MaxPackagesToResave );
			break;
		}
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// This option works if a single package is specified, it will resave all packages that reference it, and all packages that it references
	const bool bResaveDirectRefsAndDeps = Switches.Contains(TEXT("ResaveDirectRefsAndDeps"));

	// This option will filter the package list and only save packages that are redirectors, or that reference redirectors
	const bool bFixupRedirects = (Switches.Contains(TEXT("FixupRedirects")) || Switches.Contains(TEXT("FixupRedirectors")));

	// ResaveOnDemand is a generic way for systems to specify during load that the commandlet should resave a package
	ResaveOnDemandSystems = ParseResaveOnDemandSystems();
	if (!ResaveOnDemandSystems.IsEmpty())
	{
		TStringBuilder<64> SystemNames;
		for (FName SystemName : ResaveOnDemandSystems)
		{
			SystemNames << SystemName << TEXT(",");
		}
		SystemNames.RemoveSuffix(1); // Remove the trailing ,
		UE_LOGF(LogContentCommandlet, Display, "ResaveOnDemand=%ls. Only saving packages that are reported via UE::SavePackageUtilities::OnAddResaveOnDemandPackage.",
			*SystemNames);
		bResaveOnDemand = true;
		UE::SavePackageUtilities::OnAddResaveOnDemandPackage.BindUObject(this, &UResavePackagesCommandlet::OnAddResaveOnDemandPackage);

		if (!IBulkDataRegistry::IsEnabled() && ResaveOnDemandSystems.Contains(FName("bulkdataduplicates")))
		{
			UE_LOGF(LogContentCommandlet, Error, "The BulkDataRegistry must be enabled to use -ResaveOnDemand=BulkDataDuplicates! Try using the -AllowCommandletBulkDataRegistry cmdline switch.");
			return 1;
		}
	}

	// This option allows the dependency graph and soft object path redirect map to be populated. This is useful if you want soft object references to redirectors to be followed to the destination asset at save time.
	const bool bSearchAllAssets = Switches.Contains(TEXT("SearchAllAssets"));

	if (bResaveDirectRefsAndDeps || bFixupRedirects || bOnlyMaterials || bSearchAllAssets)
	{
		AssetRegistry.SearchAllAssets(true);

		// Force directory watcher tick to register paths
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		DirectoryWatcherModule.Get()->Tick(-1.0f);
	}

	if (bExplicitPackages && PackageNames.Num() == 1 && bResaveDirectRefsAndDeps)
	{
		FName PackageName = FName(*FPackageName::FilenameToLongPackageName(PackageNames[0]));

		TArray<FName> Referencers;
		AssetRegistry.GetReferencers(PackageName, Referencers);
		TArray<FName> Dependencies;
		AssetRegistry.GetDependencies(PackageName, Dependencies);

		for (FName Ref : Referencers)
		{
			FString File;
			FPackageName::SearchForPackageOnDisk(*Ref.ToString(), NULL, &File);
			PackageNames.Add(File);
		}
		for (FName Dep : Dependencies)
		{
			FString File;
			FPackageName::SearchForPackageOnDisk(*Dep.ToString(), NULL, &File);
			PackageNames.Add(File);
		}
	}
	else if (bFixupRedirects)
	{
		// Look for all packages containing redirects, and their referencers
		TArray<FAssetData> RedirectAssets;
		TSet<FString> RedirectPackages;
		TSet<FString> ReferencerPackages;

		AssetRegistry.GetAssetsByClass(UObjectRedirector::StaticClass()->GetClassPathName(), RedirectAssets);

		for (const FAssetData& AssetData : RedirectAssets)
		{
			bool bIsAlreadyInSet = false;
			FString RedirectFile;
			FPackageName::SearchForPackageOnDisk(*AssetData.PackageName.ToString(), nullptr, &RedirectFile);

			RedirectPackages.Add(RedirectFile, &bIsAlreadyInSet);

			if (!bIsAlreadyInSet)
			{
				TArray<FName> Referencers;
				AssetRegistry.GetReferencers(AssetData.PackageName, Referencers);

				for (FName Referencer : Referencers)
				{
					FString ReferencerFile;
					FPackageName::SearchForPackageOnDisk(*Referencer.ToString(), nullptr, &ReferencerFile);

					ReferencerPackages.Add(ReferencerFile);
				}
			}
		}

		// Filter packagenames list to packages that are pointing to redirectors, it will probably be much smaller
		TArray<FString> OldArray = PackageNames;
		PackageNames.Reset();
		for (FString& PackageName : OldArray)
		{
			if (RedirectPackages.Contains(PackageName))
			{
				RedirectorsToFixup.Add(PackageName);
			}

			for (const FString& ReferencerPackage : ReferencerPackages)
			{
				if (ReferencerPackage == PackageName)
				{
					PackageNames.Add(PackageName);
				}
				else if (ReferencerPackage.Contains(FPackagePath::GetExternalActorsFolderName()) || ReferencerPackage.Contains(FPackagePath::GetExternalObjectsFolderName()))
				{
					FString CleanPackageName = PackageName;
					if (CleanPackageName.RemoveFromEnd(TEXT(".umap")))
					{
						FString WorldReferencerPackage = 
							ReferencerPackage
								.Replace(FPackagePath::GetExternalActorsFolderName(), TEXT(""))
								.Replace(FPackagePath::GetExternalObjectsFolderName(), TEXT(""))
								.Replace(TEXT("//"), TEXT("/"));
						WorldReferencerPackage.LeftInline(CleanPackageName.Len());
						if (WorldReferencerPackage == CleanPackageName)
						{
							PackageNames.Add(ReferencerPackage);
						}
					}
				}
			}
		}
	}

	// Check for the min and max versions
	MinResaveUEVersion = IGNORE_PACKAGE_VERSION;
	MaxResaveUEVersion = IGNORE_PACKAGE_VERSION;
	MaxResaveLicenseeUEVersion = IGNORE_PACKAGE_VERSION;
	if ( Switches.Contains(TEXT("CHECKLICENSEEVER")) )
	{
		// Limits resaving to packages with this licensee package version or lower.
		MaxResaveLicenseeUEVersion = FMath::Max<int32>(GPackageFileLicenseeUEVersion - 1, 0);
	}
	if ( Switches.Contains(TEXT("CHECKUE4VER")) )
	{
		// Limits resaving to packages with this package version or lower.
		MaxResaveUEVersion = FMath::Max<int32>(GPackageFileUEVersion.ToValue() - 1, 0);
	}
	else if ( Switches.Contains(TEXT("RESAVEDEPRECATED")) )
	{
		// Limits resaving to packages with this package version or lower.
		MaxResaveUEVersion = FMath::Max<int32>(VER_UE4_DEPRECATED_PACKAGE - 1, 0);
	}
	else
	{
		// determine if the resave operation should be constrained to certain package versions
		for ( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++ )
		{
			const FString& CurrentSwitch = Switches[SwitchIdx];
			if ( MinResaveUEVersion == IGNORE_PACKAGE_VERSION && FParse::Value(*CurrentSwitch,TEXT("MINVER="),MinResaveUEVersion) )
			{
				if ( MinResaveUEVersion == CURRENT_PACKAGE_VERSION )
				{
					MinResaveUEVersion = GPackageFileUEVersion.ToValue();
				}
			}

			if ( MaxResaveUEVersion == IGNORE_PACKAGE_VERSION && FParse::Value(*CurrentSwitch,TEXT("MAXVER="),MaxResaveUEVersion) )
			{
				if ( MaxResaveUEVersion == CURRENT_PACKAGE_VERSION )
				{
					MaxResaveUEVersion = GPackageFileUEVersion.ToValue();
				}
			}
		}
	}

	FString ClassList;
	for (const FString& CurrentSwitch : Switches)
	{
		if ( FParse::Value(*CurrentSwitch, TEXT("RESAVECLASS="), ClassList, false) )
		{
			TArray<FString> ClassNames;
			ClassList.ParseIntoArray(ClassNames, TEXT(","), true);
			for (FString& ClassName : ClassNames)
			{
				if (FPackageName::IsShortPackageName(ClassName))
				{
					UE_LOGF(LogContentCommandlet, Warning, "RESAVECLASS param requires class path names. Short name provided: %ls.", *ClassName);
					FTopLevelAssetPath ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ClassName, ELogVerbosity::Warning, TEXT("ResavePackages"));
					if (!ClassPathName.IsNull())
					{
						ClassName = ClassPathName.ToString();
					}
				}
				ResaveClasses.AddUnique(ClassName);
			}

			break;
		}
	}

	/** determine if we should check subclasses of ResaveClasses */
	bool bIncludeChildClasses = Switches.Contains(TEXT("IncludeChildClasses"));
	if (bIncludeChildClasses && ResaveClasses.Num() == 0)
	{
		// Sanity check fail
		UE_LOGF(LogContentCommandlet, Error, "AllowSubclasses param requires ResaveClass param.");
		return 1;
	}

	if (bIncludeChildClasses)
	{
		// Can't use ranged for here because the array grows inside of this loop.
		// Also, no need to iterate over the newly added objects as we know
		// we have found all of their subclasses too (IsChildOf guarantees that).
		const int32 NumResaveClasses = ResaveClasses.Num();
		for (int32 ClassIndex = 0; ClassIndex < NumResaveClasses; ++ClassIndex)
		{
			// Find the class object and then all derived classes
			UClass* ResaveClass = UClass::TryFindTypeSlow<UClass>(ResaveClasses[ClassIndex]);
			if (ResaveClass)
			{
				for (TObjectIterator<UClass> It; It; ++It)
				{
					UClass* MaybeChildClass = *It;
					if (MaybeChildClass->IsChildOf(ResaveClass))
					{
						ResaveClasses.AddUnique(MaybeChildClass->GetPathName());
					}
				}
			}
		}
	}

	return 0;
}

void UResavePackagesCommandlet::ParseSourceControlOptions(const TArray<FString>& Tokens)
{
	if (SourceControlQueue == nullptr)
	{
		return; // No point parsing the options if we don't have anything enabled
	}

	int32 QueuedPackageFlushLimit = INDEX_NONE;
	int64 QueueFileSizeFlushLimit = INDEX_NONE;

	for (const FString& CurrentSwitch : Tokens)
	{
		if (FParse::Value(*CurrentSwitch, TEXT("BatchPackageLimit="), QueuedPackageFlushLimit))
		{
			if (QueuedPackageFlushLimit >= 0)
			{
				UE_LOGF(LogContentCommandlet, Display, "Setting revision control batches to be limited to %d package(s) at a time.", QueuedPackageFlushLimit);
				SourceControlQueue->SetMaxNumQueuedPackages(QueuedPackageFlushLimit);
			}
			else
			{
				// Negative values mean we will not flush the source control batch based on the number of packages
				UE_LOGF(LogContentCommandlet, Display, "Setting revision control batches to have no package limit!");
			}
		}
		else  if (FParse::Value(*CurrentSwitch, TEXT("BatchFileSizeLimit="), QueueFileSizeFlushLimit))
		{
			const int64 SizeLimit = TNumericLimits<int64>::Max() / (1024 * 1024);
			if (QueueFileSizeFlushLimit > SizeLimit)
			{
				UE_LOGF(LogContentCommandlet, Display, "-BatchFileSizeLimit=%lld is too large! The max value allowed is %lld, clamping...", QueueFileSizeFlushLimit, SizeLimit);
				QueueFileSizeFlushLimit = SizeLimit;
			}

			if (QueueFileSizeFlushLimit >= 0)
			{
				UE_LOGF(LogContentCommandlet, Display, "Setting revision control batches to be limited to %lld MB.", QueueFileSizeFlushLimit);

				SourceControlQueue->SetMaxTemporaryFileTotalSize(QueueFileSizeFlushLimit);
			}
			else
			{
				// Negative values mean we will not flush the source control batch based on the disk space taken by temp files
				UE_LOGF(LogContentCommandlet, Display, "Setting revision control batches to have no disk space limit!");
			}
		}
	}
}

void UResavePackagesCommandlet::OnAddResaveOnDemandPackage(FName SystemName, FName PackageName)
{
	FScopeLock ScopeLock(&ResaveOnDemandPackagesLock);
	if (ResaveOnDemandSystems.Contains(SystemName))
	{
		ResaveOnDemandPackages.Add(PackageName);
	}
}


TSet<FName> UResavePackagesCommandlet::ParseResaveOnDemandSystems()
{
	using namespace UE::String;
	TSet<FName> SystemNames;

	const TCHAR* CommandLineStream = FCommandLine::Get();
	for (;;)
	{
		FString Token = FParse::Token(CommandLineStream, false /* bUseEscape */);
		if (Token.IsEmpty())
		{
			break;
		}
		FString TokenSystemNames;
		if (FParse::Value(*Token, TEXT("-resaveondemand="), TokenSystemNames))
		{
			UE::String::ParseTokensMultiple(TokenSystemNames, TConstArrayView<TCHAR>({ '+', ',' }),
				[&SystemNames](FStringView TokenSystemName)
				{
					SystemNames.Add(FName(TokenSystemName));
				}, EParseTokensOptions::Trim | EParseTokensOptions::SkipEmpty);
		}
	}
	return SystemNames;
}


bool UResavePackagesCommandlet::ShouldSkipPackage(const FString& Filename)
{
	return false;
}

void UResavePackagesCommandlet::LoadAndSaveOnePackage(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage);

	// Check to see if a derived commandlet wants to skip this package for one reason or another
	if (ShouldSkipPackage(Filename))
	{
		return;
	}

	// Skip the package if it doesn't have a required substring match
	if (PackageSubstring.Len() && !Filename.Contains(PackageSubstring) )
	{
		VerboseMessage(FString::Printf(TEXT("Skipping %s"), *Filename));
		return;
	}

	FPackagePath PackagePath = FPackagePath::FromLocalPath(Filename);
	// Convert this path to mounted before querying the asset registry or CollectionFilter because they use mounted paths 
	PackagePath.TryConvertToMounted();
	if (CollectionFilter.Num())
	{
		FString PackageNameToCreate = PackagePath.GetPackageNameOrFallback();
		FName PackageName(*PackageNameToCreate);
		if (!CollectionFilter.Contains(PackageName))
		{
			return;
		}
	}

	if (bOnlyMaterials)
	{
		FString PackageNameToCreate = PackagePath.GetPackageNameOrFallback();
		FName PackageFNameToCreate(*PackageNameToCreate);
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		TArray<FAssetData> PackageAssetData;
		if (!AssetRegistry.GetAssetsByPackageName(PackageFNameToCreate, PackageAssetData, true))
		{
			return;
		}

		bool bAnyMaterialsPresent = false;
		for (const FAssetData& AssetData : PackageAssetData)
		{
			UClass* AssetClass = AssetData.GetClass();
			if (AssetClass && AssetClass->IsChildOf(UMaterialInterface::StaticClass()))
			{
				bAnyMaterialsPresent = true;
				break;
			}
		}

		if (!bAnyMaterialsPresent)
		{
			return;
		}
	}

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

	if ( bIsReadOnly && !bVerifyContent && !bAutoCheckOut )
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogContentCommandlet, Warning, "Skipping read-only file %ls", *Filename);
		}
	}
	else
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogContentCommandlet, Display, "Loading %ls", *Filename);
		}

		static int32 LastErrorCount = 0;

		int32 NumErrorsFromLoading = GWarn->GetNumErrors();
		if (NumErrorsFromLoading > LastErrorCount)
		{
			UE_LOGF(LogContentCommandlet, Warning, "%d total errors encountered during loading", NumErrorsFromLoading);
		}
		LastErrorCount = NumErrorsFromLoading;

		// Get the package linker.
		VerboseMessage(TEXT("Pre GetPackageLinker"));

		// Load the package before accessing its linker to prevent interference with async loads in case
		// that package was already queued and being loaded asynchronously.
		LoadPackage(nullptr, PackagePath, LOAD_NoVerify);

		FLinkerLoad* Linker = GetPackageLinker(nullptr, PackagePath, LOAD_NoVerify, nullptr);

		// Bail early if we don't have a valid linker (package was out of date, etc)
		if( !Linker )
		{
			VerboseMessage(TEXT("Aborting...package could not be loaded"));
			CollectGarbage(RF_NoFlags);
			return;
		}

		VerboseMessage(TEXT("Post GetPackageLinker"));

		bool bSavePackage = true;
		PerformPreloadOperations(Linker, bSavePackage);

		VerboseMessage(FString::Printf(TEXT("Post PerformPreloadOperations, Resave? %d"), bSavePackage));
		
		if (bSavePackage)
		{
			PackagesConsideredForResave++;

			// Only rebuild static meshes on load for the to be saved package.
			extern ENGINE_API FName GStaticMeshPackageNameToRebuild;
			FName PackageFName = PackagePath.GetPackageFName();
			check(!PackageFName.IsNone());
			GStaticMeshPackageNameToRebuild = PackageFName;

			// Assert if package couldn't be opened so we have no chance of messing up saving later packages.
			UPackage* Package = nullptr;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage::LoadPackage);
				Package = LoadPackage(NULL, *Filename, 0);
			}

			if (Package == NULL)
			{
				if (bCanIgnoreFails == true)
				{
					return;
				}
				else
				{
					check(Package);
				}
			}

			VerboseMessage(TEXT("Post LoadPackage"));

			// if we are only saving dirty packages and the package is not dirty, then we do not want to save the package (remember the default behavior is to ALWAYS save the package)
			if( ( bOnlySaveDirtyPackages == true ) && ( Package->IsDirty() == false ) )
			{
				bSavePackage = false;
			}

			// here we want to check and see if we have any loading warnings
			// if we do then we want to resave this package
			if( !bSavePackage && FParse::Param(FCommandLine::Get(),TEXT("SavePackagesThatHaveFailedLoads")) == true )
			{
				//UE_LOGF(LogContentCommandlet, Warning, "NumErrorsFromLoading: %d GWarn->Errors num: %d", NumErrorsFromLoading, GWarn->GetNumErrors() );

				if( NumErrorsFromLoading != GWarn->GetNumErrors() )
				{
					bSavePackage = true;
				}
			}

			{
				UWorld* World = UWorld::FindWorldInPackage(Package);
				if (World)
				{
					PerformAdditionalOperations(World, bSavePackage);
				}
			}

			// hook to allow performing additional checks without lumping everything into this one function
			PerformAdditionalOperations(Package,bSavePackage);

			VerboseMessage(TEXT("Post PerformAdditionalOperations"));

			// Check for any special per object operations
			ForEachObjectWithOuter(Package, [this, &bSavePackage](UObject* Object)
			{
				PerformAdditionalOperations(Object, bSavePackage);
			});
			
			VerboseMessage(TEXT("Post PerformAdditionalOperations Loop"));

			if (bStripEditorOnlyContent)
			{
				UE_LOGF(LogContentCommandlet, Log, "Removing editor only data");
				Package->SetPackageFlags(PKG_FilterEditorOnly);
			}

			if (bResaveOnDemand && bSavePackage)
			{
				// Finish all async loading for the package, in case the systems marking it for load do so only from the async loading
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage::ResaveOnDemandFlush);
					FAssetCompilingManager::Get().FinishAllCompilation();
					FlushAsyncLoading();
				}
				// Skip saving the package if no systems requested it
				FScopeLock ScopeLock(&ResaveOnDemandPackagesLock);
				bSavePackage = ResaveOnDemandPackages.Contains(Package->GetFName());
			}

			if (bSavePackage == true)
			{
				TArray<UObject*> ObjectsInOuter;
				GetObjectsWithOuter(Package, ObjectsInOuter);

				{
					// Added to guard against FBulkData being used in an async task while ::ResetLoaders is called later on this thread.
					// Can be removed when bulkdata attachment issues are sorted.
					// If this shows up as a perf issue we could consider only waiting if the package's FLinkerLoad has attached
					// FBulkData.
					// NOTE: This call will wait for any async job related to one of the UObjects in the package, the ::FinishAllCompilation
					// call above (if bResaveOnDemand is true) will wait for all sync compilation jobs to complete.
					TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage::FinishCompilationForObjects);
					FAssetCompilingManager::Get().FinishCompilationForObjects(ObjectsInOuter);
				}

				if (ObjectsInOuter.IsEmpty())
				{
					bSavePackage = false;
					Package = nullptr;
					
					UE_LOGF(LogContentCommandlet, Display, "Package %ls is empty and will be deleted", *Filename);

					DeleteOnePackage(Filename);
				}
			}

			// Now based on the computation above we will see if we should actually attempt
			// to save this package
			if (bSavePackage == true)
			{
				if( bIsReadOnly == true && bVerifyContent == true && bAutoCheckOut == false )
				{
					UE_LOGF(LogContentCommandlet, Warning, "Package [%ls] is read-only but needs to be resaved (UE Version: %i, Licensee Version: %i  Current UE Version: %i, Current Licensee Version: %i)",
						*Filename, Linker->Summary.GetFileVersionUE().ToValue(), Linker->Summary.GetFileVersionLicenseeUE(), GPackageFileUEVersion.ToValue(), VER_LATEST_ENGINE_LICENSEEUE4 );
					if( SavePackageHelper(Package, FString(TEXT("Temp.temp"))) )
					{
						UE_LOGF(LogContentCommandlet, Warning, "Correctly saved:  [Temp.temp]." );
					}
				}
				else
				{
					// Check to see if we need to check this package out (but do not check out here if SourceControlQueue is enabled)
					const bool bAttemptCheckoutNow = bAutoCheckOut && SourceControlQueue == nullptr;
					if (bAttemptCheckoutNow)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage::AutoCheckOut);
						if( bIsReadOnly )
						{
							VerboseMessage(TEXT("Pre ForceGetStatus1"));
							ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
							FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState( Package, EStateCacheUsage::ForceUpdate );
							if(SourceControlState.IsValid())
							{
								FString OtherCheckedOutUser;
								if( SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser) )
								{
									UE_LOGF(LogContentCommandlet, Warning, "[REPORT] Overwriting package %ls already checked out by someone else (%ls), will not submit", *Filename, *OtherCheckedOutUser);
								}
								else if( !SourceControlState->IsCurrent() )
								{
									UE_LOGF(LogContentCommandlet, Warning, "[REPORT] Overwriting package %ls (not at head), will not submit", *Filename);
								}
								else
								{
									VerboseMessage(TEXT("Pre CheckOut"));

									SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), Package);

									VerboseMessage(TEXT("Post CheckOut"));

									FilesToSubmit.AddUnique(*Filename);
								}
							}
							VerboseMessage(TEXT("Post ForceGetStatus2"));
						}
					}
					else if (!ISourceControlModule::Get().GetProvider().UsesCheckout())
					{
						// Handle git which does not set files readonly and does not use checkout
						FilesToSubmit.AddUnique(*Filename);
					}

					// Update the readonly state now that source control has had a chance to run
					bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

					// If we tried to check out the file but it is still read only then we failed and 
					// need to emit an error and go to the next package			
					if (bAttemptCheckoutNow && bIsReadOnly)
					{
						if (bSkipCheckedOutFiles)
						{
							UE_LOGF(LogContentCommandlet, Warning, "Skipping Package: %ls (unable to check out)", *Filename);
						}
						else
						{
							UE_LOGF(LogContentCommandlet, Error, "Unable to check out the Package: %ls", *Filename);
						}
						
						return;
					}

					if (Verbosity != ONLY_ERRORS)
					{
						UE_LOGF(LogContentCommandlet, Display, "Resaving package [%ls] (UE Version: %i, Licensee Version: %i  Saved UE Version: %i, Saved Licensee Version: %i)",
							*Filename,Linker->Summary.GetFileVersionUE().ToValue(), Linker->Summary.GetFileVersionLicenseeUE(), GPackageFileUEVersion.ToValue(), VER_LATEST_ENGINE_LICENSEEUE4 );
					}

					uint32 SaveFlags = ESaveFlags::SAVE_None;
					
					if (bIsReadOnly == false || SourceControlQueue == nullptr)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::LoadAndSaveOnePackage::SavePackage);

						if (SavePackageHelper(Package, Filename, RF_Standalone, GWarn, (ESaveFlags)SaveFlags))
						{
							PackagesResaved++;
							if (Verbosity == VERY_VERBOSE)
							{
								UE_LOGF(LogContentCommandlet, Display, "Correctly saved:  [%ls].", *Filename);
							}
						}
					}
					else
					{
						// The target file is still read only and we have SourceControlQueue enabled so we
						// need to save to a temporary file first, and then queue the result.
						const FString TempFilename = CreateTempFilename();

						// Prevent the persistent GUID of the package being regenerated as we are saving out to a different package path.
						// We want the GUID to remain the same as we will eventually replace the source package with this temp one.
						SaveFlags |= ESaveFlags::SAVE_KeepPersistentGUID;

						if (SavePackageHelper(Package, TempFilename, RF_Standalone, GWarn, (ESaveFlags)SaveFlags))
						{
							SourceControlQueue->QueueCheckoutAndReplaceOperation(Filename, TempFilename, Package);
						}
					}
				}
			}
		}

		static int32 Counter = 0;

		if (!GarbageCollectionFrequency || Counter++ % GarbageCollectionFrequency == 0)
		{
			if (FApp::CanEverRender())
			{
				FAssetCompilingManager::Get().FinishAllCompilation();
			}

			if (IBulkDataRegistry::IsEnabled())
			{
				FTickableEditorObject::TickObjects(0.0f);
			}

			if (GarbageCollectionFrequency > 1)
			{
				UE_LOGF(LogContentCommandlet, Display, "GC");
			}
			VerboseMessage(TEXT("Pre CollectGarbage"));

			CollectGarbage(RF_NoFlags);

			VerboseMessage(TEXT("Post CollectGarbage"));
		}
	}
}

void UResavePackagesCommandlet::DeleteOnePackage(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::DeleteOnePackage);

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);

	if (bVerifyContent)
	{
		return;
	}

	if (bIsReadOnly && !bAutoCheckOut)
	{
		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogContentCommandlet, Warning, "Skipping read-only file %ls", *Filename);
		}
		return;
	}

	if (SourceControlQueue != nullptr)
	{
		// All files (read only and non read only) need to query their source control status so
		// they should all be queued if avaliable.
		SourceControlQueue->QueueDeleteOperation(Filename);
		return;
	}

	FString PackageName;
	FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName);

	UPackage* Package = FindPackage(nullptr, *PackageName);

	if (Package)
	{
		// Unload package so we can delete it
		TArray<UPackage *> PackagesToDelete;
		PackagesToDelete.Add(Package);
		UPackageTools::UnloadPackages(PackagesToDelete);
		PackagesToDelete.Empty();
		Package = nullptr;
	}

	FString PackageFilename = SourceControlHelpers::PackageFilename(Filename);
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(PackageFilename, EStateCacheUsage::ForceUpdate);

	if (SourceControlState.IsValid() && (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded()))
	{
		UE_LOGF(LogContentCommandlet, Display, "Revert '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), PackageFilename);

		UE_LOGF(LogContentCommandlet, Display, "Deleting '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), PackageFilename);

		PackagesDeleted++;

		FilesToSubmit.AddUnique(Filename);
	}
	else if (SourceControlState.IsValid() && SourceControlState->CanCheckout())
	{
		UE_LOGF(LogContentCommandlet, Display, "Deleting '%ls' from revision control...", *Filename);
		SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), PackageFilename);

		PackagesDeleted++;

		FilesToSubmit.AddUnique(Filename);
	}
	else if (SourceControlState.IsValid() && SourceControlState->IsCheckedOutOther())
	{
		UE_LOGF(LogContentCommandlet, Warning, "Couldn't delete '%ls' from revision control, someone has it checked out, skipping...", *Filename);
	}
	else if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
	{
		UE_LOGF(LogContentCommandlet, Warning, "'%ls' is not in revision control, attempting to delete from disk...", *Filename);
		if (IFileManager::Get().Delete(*Filename, false, true) == true)
		{
			PackagesDeleted++;
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Warning, "  ... failed to delete from disk.");
		}
	}
	else
	{
		UE_LOGF(LogContentCommandlet, Warning, "'%ls' is in an unknown revision control state, attempting to delete from disk...", *Filename);
		if (IFileManager::Get().Delete(*Filename, false, true)== true)
		{
			PackagesDeleted++;
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Warning, "  ... failed to delete from disk.");
		}
	}
}

int32 UResavePackagesCommandlet::Main( const FString& Params )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet);

	const TCHAR* Parms = *Params;
	TArray<FString> Tokens;
	ParseCommandLine(Parms, Tokens, Switches);

	// Ensure source control is initialized and shut down properly
	FScopedSourceControl SourceControl;

	// We load the default editor modules to ensure UPackage::OnMetaDataTransferRequestedDelegate is bound and handled
	// By default FMetaData gets thrown out, the callback re-adds it to the new packages.
	// One such example is the UserAssetTagsEditor, which needs to be loaded so the tags aren't lost during Resave.
	if (!GEditor->IsInitialized())
	{
		GEditor->LoadDefaultEditorModules();
	}
	
	// strip editor only content
	bStripEditorOnlyContent = Switches.Contains(TEXT("STRIPEDITORONLY"));
	// skip the assert when a package can not be opened
	bCanIgnoreFails = Switches.Contains(TEXT("SKIPFAILS"));
	/** load all packages, and display warnings for those packages which would have been resaved but were read-only */
	bVerifyContent = Switches.Contains(TEXT("VERIFY"));
	/** if we should only save dirty packages **/
	bOnlySaveDirtyPackages = Switches.Contains(TEXT("OnlySaveDirtyPackages"));
	/** if we should auto checkout packages that need to be saved**/
	bAutoCheckOut = Switches.Contains(TEXT("AutoCheckOutPackages")) || Switches.Contains(TEXT("AutoCheckOut"));
	/** if we should batch source control operations*/
	bBatchSourceControl = Switches.Contains(TEXT("BatchSourceControl"));
	/** if we should simply skip checked out files rather than error-ing out */
	bSkipCheckedOutFiles = Switches.Contains(TEXT("SkipCheckedOutPackages"));
	/** if we should auto checkin packages that were checked out**/
	bAutoCheckIn = bAutoCheckOut && (Switches.Contains(TEXT("AutoCheckIn")) || Switches.Contains(TEXT("AutoSubmit")));
	if (!ISourceControlModule::Get().GetProvider().UsesCheckout())
	{
		// Handle git which does not set files readonly and does not use checkout
		bAutoCheckIn = Switches.Contains(TEXT("AutoCheckIn")) || Switches.Contains(TEXT("AutoSubmit"));
	}
	/** if we should skip trying to virtualize packages being checked in **/
	bSkipVirtualization = Switches.Contains(TEXT("SkipVirtualization"));
	/** determine if we are building lighting for the map packages on the pass. **/
	bShouldBuildLighting = Switches.Contains(TEXT("buildlighting"));
	/** determine if we are building reflection captures for the map packages on the pass. **/
	bShouldBuildReflectionCaptures = Switches.Contains(TEXT("buildreflectioncaptures"));	/** rebuilds texture streaming data for all packages, rather than just maps. **/
	bShouldBuildTextureStreamingForAll = Switches.Contains(TEXT("buildtexturestreamingforall"));
	/** determine if we are building texture streaming data for the map packages on the pass. **/
	bShouldBuildTextureStreaming = bShouldBuildTextureStreamingForAll || Switches.Contains(TEXT("buildtexturestreaming"));
	/** determine if we can skip the version changelist check */
	bIgnoreChangelist = Switches.Contains(TEXT("IgnoreChangelist"));
	/** whether we should only save packages with changelist zero */
	bOnlyUnversioned = Switches.Contains(TEXT("OnlyUnversioned"));
	/** whether we should only save packages saved by licensees */
	bOnlyLicenseed = Switches.Contains(TEXT("OnlyLicenseed"));
	/** whether we should only save packages containing virtualized payloads */
	bOnlyVirtualized = Switches.Contains(TEXT("OnlyVirtualized"));
	/** whether we should skip saving packages that already contain virtualized payloads */
	bSkipVirtualized = Switches.Contains(TEXT("SkipVirtualized"));
	/** whether we should only save packages containing FPayloadTrailers */
	bOnlyPayloadTrailers = Switches.Contains(TEXT("OnlyPayloadTrailers"));
	/** whether we should skip saving packages that already have a payload trailer */
	bSkipPayloadTrailers = Switches.Contains(TEXT("SkipPayloadTrailers"));
	/** only process packages containing materials */
	bOnlyMaterials = Switches.Contains(TEXT("onlymaterials"));
	
	/** force the materials auto usage flag */
	{
		int32 ForceAutoUsageFlagValue = 0;
		if (FParse::Value(FCommandLine::Get(), TEXT("ForceMaterialAutoUsageFlag="), ForceAutoUsageFlagValue))
		{
			if (bOnlyMaterials)
			{
				ForceMaterialAutoUsageFlag = (ForceAutoUsageFlagValue != 0);
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Warning,
					"ForceMaterialAutoUsageFlag should only be used with -onlymaterials. Ignoring.");
			}
		}

	}
	/** determine if we are building navigation data for the map packages on the pass. **/
	bShouldBuildNavigationData = Switches.Contains(TEXT("BuildNavigationData"));

	// UE_DEPRECATED(5.6)
	bool bKeepPackageGUIDOnSave = FParse::Param(FCommandLine::Get(), TEXT("KeepPackageGUIDOnSave"));
	if (bKeepPackageGUIDOnSave)
	{
		UE_LOGF(LogContentCommandlet, Warning,
			"KeepPackageGUIDOnSave is no longer supported, because package save no longer records arbitrary guids but instead records a hash of the package's bytes - PackageSavedHash.");
	}

	/** check for filtering packages by collection. **/
	FString FilterByCollection;
	FParse::Value(*Params, TEXT("FilterByCollection="), FilterByCollection);

	CollectionFilter = TSet<FName>();
	if (!FilterByCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		TArray<FSoftObjectPath> CollectionAssets;
		if (!CollectionManager.TryParseCollectionPath(FilterByCollection, &CollectionContainer, &CollectionName, &ShareType) ||
			!CollectionContainer->GetAssetsInCollection(CollectionName, ShareType, CollectionAssets))
		{
			UE_LOGF(LogContentCommandlet, Warning, "Could not get assets in collection '%ls'. Skipping filter.", *FilterByCollection);
		}
		else
		{
			//insert all of the collection names into the set for fast filter checks
			for (const FSoftObjectPath& AssetPath: CollectionAssets)
			{
				CollectionFilter.Add(AssetPath.GetLongPackageFName());
			}
		}
	}

	/** determine if we are building HLODs for the map packages on the pass. **/
	bShouldBuildHLOD = Switches.Contains(TEXT("BuildHLOD"));
	if (bShouldBuildHLOD)
	{
		FString HLODOptions;
		FParse::Value(*Params, TEXT("BuildOptions="), HLODOptions);
		bGenerateClusters = HLODOptions.Contains(TEXT("Clusters"));
		bGenerateMeshProxies = HLODOptions.Contains(TEXT("Proxies"));
		bForceClusterGeneration = HLODOptions.Contains(TEXT("ForceClusters"));
		bForceProxyGeneration = HLODOptions.Contains(TEXT("ForceProxies"));
		bForceSingleClusterForLevel = HLODOptions.Contains(TEXT("ForceSingleCluster"));
		bSkipSubLevels = HLODOptions.Contains(TEXT("SkipSubLevels"));
		bHLODMapCleanup = HLODOptions.Contains(TEXT("MapCleanup"));

		ForceHLODSetupAsset = FString();
		FParse::Value(*Params, TEXT("ForceHLODSetupAsset="), ForceHLODSetupAsset);
		
		HLODSkipToMap = FString();
		FParse::Value(*Params, TEXT("SkipToMap="), HLODSkipToMap);

		UE_LOGF(LogContentCommandlet, Display, "Rebuilding HLODs... Options are:");
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] Clusters", bGenerateClusters ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] Proxies", bGenerateMeshProxies ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] ForceClusters", bForceClusterGeneration ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] ForceProxies", bForceProxyGeneration ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] SkipSubLevels", bSkipSubLevels ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] ForceSingleCluster", bForceSingleClusterForLevel ? TEXT("X") : TEXT(" "));
		UE_LOGF(LogContentCommandlet, Display, "  [%ls] Map Cleanup", bHLODMapCleanup ? TEXT("X") : TEXT(" "));

		// Allow multiple instances when building HLODs
		FString MutexVariableValue = FPlatformMisc::GetEnvironmentVariable(TEXT("uebp_UATMutexNoWait"));
		if (MutexVariableValue != TEXT("1"))
		{
			FPlatformMisc::SetEnvironmentVar(TEXT("uebp_UATMutexNoWait"), TEXT("1"));
			bForceUATEnvironmentVariableSet = true;
		}

		if (bHLODMapCleanup)
		{
			GEngine->GetEngineSubsystem<UHLODEngineSubsystem>()->DisableHLODCleanupOnLoad(true);
		}
	}
		
	if (bShouldBuildLighting || bShouldBuildHLOD || bShouldBuildReflectionCaptures)
	{
		check( Switches.Contains(TEXT("AllowCommandletRendering")) );
		GarbageCollectionFrequency = 1;
	}

	// Default build on production
	LightingBuildQuality = Quality_Production;
	FString QualityStr;
	FParse::Value(*Params, TEXT("Quality="), QualityStr);
	if (QualityStr.Len())
	{
		if (QualityStr.Equals(TEXT("Preview"), ESearchCase::IgnoreCase))
		{
			LightingBuildQuality = Quality_Preview;
		}
		else if (QualityStr.Equals(TEXT("Medium"), ESearchCase::IgnoreCase))
		{
			LightingBuildQuality = Quality_Medium;
		}
		else if (QualityStr.Equals(TEXT("High"), ESearchCase::IgnoreCase))
		{
			LightingBuildQuality = Quality_High;
		}
		else if (QualityStr.Equals(TEXT("Production"), ESearchCase::IgnoreCase))
		{
			LightingBuildQuality = Quality_Production;
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Fatal, "Unknown Quality(must be Preview/Medium/High/Production): %ls", *QualityStr );
		}
		UE_LOGF(LogContentCommandlet, Display, "Lighing Build Quality is %ls", *QualityStr);
	}

	TArray<FString> PackageNames;
	int32 ResultCode = InitializeResaveParameters(Tokens, PackageNames);
	if ( ResultCode != 0 )
	{
		return ResultCode;
	}

	// Retrieve list of all packages in .ini paths.
	if(!PackageNames.Num() && !RedirectorsToFixup.Num())
	{
		return 0;
	}

	int32 GCIndex = 0;
	PackagesConsideredForResave = 0;
	PackagesResaved = 0;
	PackagesDeleted = 0;
	TotalPackagesForResave = PackageNames.Num();

	// allow for an option to restart at a given package name (in case it dies during a run, etc)
	bool bCanProcessPackage = true;
	FString FirstPackageToProcess;
	if (FParse::Value(*Params, TEXT("FirstPackage="), FirstPackageToProcess))
	{
		bCanProcessPackage = false;
	}
	FParse::Value(*Params, TEXT("PackageSubString="), PackageSubstring);
	if (PackageSubstring.Len())
	{
		UE_LOGF(LogContentCommandlet, Display, "Restricted to packages containing %ls", *PackageSubstring );
	}

	// Avoid crash saving blueprint
	FFindInBlueprintSearchManager::Get();

	// Tick shader compiler if we are rendering
	if(IsAllowCommandletRendering() && GShaderCompilingManager)
	{
		GShaderCompilingManager->ProcessAsyncResults(true, false);
	}

	if (bBatchSourceControl)
	{
		// Convert the commandlets verbosity to that of FQueuedSourceControlOperations
		FQueuedSourceControlOperations::EVerbosity LogVerbosity = FQueuedSourceControlOperations::EVerbosity::All;
		switch (Verbosity)
		{
		case VERY_VERBOSE:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::All;
			break;
		case INFORMATIVE:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::Info;
			break;
		case ONLY_ERRORS:
			LogVerbosity = FQueuedSourceControlOperations::EVerbosity::ErrorsOnly;
			break;
		default:
			UE_LOGF(LogContentCommandlet, Warning, "Unknown verbosity to pass to FQueuedSourceControlOperations!");
		}

		SourceControlQueue = MakePimpl<FQueuedSourceControlOperations>(LogVerbosity);	
		ParseSourceControlOptions(Switches);
	}

	// Make sure any remaining temp files from previous runs are removed
	CleanTempFiles();

	//Flush Async loading before running the commandlet to ensure packages we want to save that are being loaded asynchronously are loaded before we begin
	UE_LOGF(LogContentCommandlet, Display, "Flushing Async Loading");
	FlushAsyncLoading();

	// Iterate over all packages.
	for( int32 PackageIndex = 0; PackageIndex < PackageNames.Num(); PackageIndex++ )
	{
		// Make sure we don't rebuild SMs that we're not going to save.
		extern ENGINE_API FName GStaticMeshPackageNameToRebuild;
		GStaticMeshPackageNameToRebuild = NAME_None;

		const FString& Filename = PackageNames[PackageIndex];

		// skip over packages before the first one allowed, if it was specified
		if (!bCanProcessPackage)
		{
			if (FPackageName::FilenameToLongPackageName(Filename) == FirstPackageToProcess)
			{
				bCanProcessPackage = true;
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Display, "Skipping %ls", *Filename);
				continue;
			}
		}

		// Load and save this package
		LoadAndSaveOnePackage(Filename);

		// Check if we need to flush any source control operations yet
		if (SourceControlQueue != nullptr)
		{
			SourceControlQueue->FlushPendingOperations(false);
		}

		// Tick shader compiler if we are rendering
		if(IsAllowCommandletRendering() && GShaderCompilingManager)
		{
			GShaderCompilingManager->ProcessAsyncResults(true, false);
		}

		// Break out if we've resaved enough packages
		if( MaxPackagesToResave > -1 && PackagesResaved >= MaxPackagesToResave )
		{
			UE_LOGF(LogContentCommandlet, Warning, "Attempting to resave more than MaxPackagesToResave; exiting" );
			break;
		}
	}
	
	if (SourceControlQueue != nullptr)
	{
		// Flush any remaining source control operations
		SourceControlQueue->FlushPendingOperations(true);

		// Fix up the stats
		PackagesDeleted += SourceControlQueue->GetNumDeletedFiles();
		PackagesResaved += SourceControlQueue->GetNumReplacedFiles();

		// Add files that the source control queue modified to the list of files that 
		// will need to be submitted to source control
		FilesToSubmit.Append(SourceControlQueue->GetModifiedFiles());

		SourceControlQueue.Reset();
	}

	// Force a directory watcher and asset registry tick
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	DirectoryWatcherModule.Get()->Tick(-1.0f);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.Tick(-1.0f);

	int32 DeleteMinimumAgeDays = 0;
	if (FParse::Value(*Params, TEXT("DeleteMinimumAgeDays="), DeleteMinimumAgeDays))
	{
		DeleteMinimumAgeDays = FMath::Max(DeleteMinimumAgeDays, 0);
	}
	FDateTime Now = FDateTime::Now();

	// Delete unreferenced redirector packages
	for (int32 PackageIndex = 0; PackageIndex < RedirectorsToFixup.Num(); PackageIndex++)
	{
		const FString& Filename = RedirectorsToFixup[PackageIndex];

		FName PackageName = FName(*FPackageName::FilenameToLongPackageName(Filename));

		// Load and save this package
		TArray<FName> Referencers;

		AssetRegistry.GetReferencers(PackageName, Referencers);

		if (Referencers.Num() > 0)
		{
			if (Verbosity != ONLY_ERRORS)
			{
				UE_LOGF(LogContentCommandlet, Display, "Can't delete redirector [%ls], unsaved packages reference it", *Filename);
			}
			continue;
		}

		if (DeleteMinimumAgeDays > 0)
		{
			FDateTime FileModificationTime = IFileManager::Get().GetTimeStamp(*Filename);
			int32 DaysSinceModification = (Now - FileModificationTime).GetDays();
			if (DaysSinceModification < DeleteMinimumAgeDays)
			{
				if (Verbosity != ONLY_ERRORS)
				{
					UE_LOGF(LogContentCommandlet, Display,
						"Not deleting unreferenced redirector [%ls] (age threshold set to %d days, redirector was modified %d days ago)",
						*Filename, DeleteMinimumAgeDays, DaysSinceModification);
				}
				continue;
			}
		}

		if (Verbosity != ONLY_ERRORS)
		{
			UE_LOGF(LogContentCommandlet, Display, "Deleting unreferenced redirector [%ls]", *Filename);
		}

		DeleteOnePackage(Filename);
	}

	// Submit the results to source control
	CheckInFiles(FilesToSubmit, GetChangelistDescription());

	if (bForceUATEnvironmentVariableSet)
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("uebp_UATMutexNoWait"), TEXT("0"));		
	}

	UE_LOGF(LogContentCommandlet, Display, "[REPORT] %d/%d packages were considered for resaving", PackagesConsideredForResave, TotalPackagesForResave);
	UE_LOGF(LogContentCommandlet, Display, "[REPORT] %d/%d packages were resaved", PackagesResaved, PackagesConsideredForResave);
	UE_LOGF(LogContentCommandlet, Display, "[REPORT] %d/%d packages were deleted", PackagesDeleted, PackagesConsideredForResave);


	return 0;
}

FText UResavePackagesCommandlet::GetChangelistDescription() const
{
	FTextBuilder ChangelistDescription;

	if (bShouldBuildLighting)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionBuildLighting", "Rebuild lightmaps."));
	}

	if (bShouldBuildTextureStreaming)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionBuildTextureStreaming", "Rebuild texture streaming."));
	}

	if (bShouldBuildReflectionCaptures)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionBuildReflectionCaptures", "Rebuild reflection captures."));
	}

	if (RedirectorsToFixup.Num() > 0)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionRedirectors", "Fixing Redirectors"));
	}

	if (bShouldBuildHLOD)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionHLODs", "Rebuilding HLODs"));
	}

	if (bShouldBuildNavigationData)
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescriptionBuildNavigationData", "Rebuilding Navigation data."));
	}
	
	if (ChangelistDescription.IsEmpty())
	{
		ChangelistDescription.AppendLine(NSLOCTEXT("ContentCmdlets", "ChangelistDescription", "Resave Deprecated Packages"));
	}

	return ChangelistDescription.ToText();
}


void UResavePackagesCommandlet::PerformPreloadOperations( FLinkerLoad* PackageLinker, bool& bSavePackage )
{
	const int32 UEPackageVersion = PackageLinker->Summary.GetFileVersionUE().ToValue();
	const int32 LicenseeUEPackageVersion = PackageLinker->Summary.GetFileVersionLicenseeUE();

	// validate that this package meets the minimum requirement
	if (MinResaveUEVersion != IGNORE_PACKAGE_VERSION && UEPackageVersion < MinResaveUEVersion)
	{
		bSavePackage = false;
		return;
	}

	// Check if this package meets the maximum requirements.
	const bool bNoLimitation = MaxResaveUEVersion == IGNORE_PACKAGE_VERSION && MaxResaveLicenseeUEVersion == IGNORE_PACKAGE_VERSION;
	const bool bAllowResave = bNoLimitation ||
						 (MaxResaveUEVersion != IGNORE_PACKAGE_VERSION && UEPackageVersion <= MaxResaveUEVersion) ||
						 (MaxResaveLicenseeUEVersion != IGNORE_PACKAGE_VERSION && LicenseeUEPackageVersion <= MaxResaveLicenseeUEVersion);
	// If not, don't resave it.
	if (!bAllowResave)
	{
		bSavePackage = false;
		return;
	}

	// If the package was saved with a higher engine version do not try to resave it. This also addresses problem with people 
	// building editor locally and resaving content with a 0 CL version (e.g. BUILD_FROM_CL == 0)
	if (!bIgnoreChangelist && PackageLinker->Summary.SavedByEngineVersion.GetChangelist() > FEngineVersion::Current().GetChangelist())
	{
		UE_LOGF(LogContentCommandlet, Warning, "Skipping resave of %ls due to engine version mismatch (Package:%d, Editor:%d) ", 
			*PackageLinker->GetArchiveName(),
			PackageLinker->Summary.SavedByEngineVersion.GetChangelist(), 
			FEngineVersion::Current().GetChangelist());
		bSavePackage = false;
		return;
	}

	// Check if the changelist number is zero
	if (bOnlyUnversioned && PackageLinker->Summary.SavedByEngineVersion.GetChangelist() != 0)
	{
		bSavePackage = false;
		return;
	}

	// Check if the package was saved by licensees
	if (bOnlyLicenseed)
	{
		if (!PackageLinker->Summary.SavedByEngineVersion.IsLicenseeVersion())
		{
			bSavePackage = false;
			return;
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Display, "Resaving %ls that contains licensee version (%ls)",
				*PackageLinker->GetArchiveName(),
				*PackageLinker->Summary.SavedByEngineVersion.ToString());
		}
	}

	if (bOnlyVirtualized)
	{
		const UE::FPackageTrailer* Trailer = PackageLinker->GetPackageTrailer();
		if (Trailer == nullptr || Trailer->GetNumPayloads(UE::EPayloadStorageType::Virtualized) == 0)
		{
			bSavePackage = false;
			return;
		}
	}

	if (bSkipVirtualized)
	{
		const UE::FPackageTrailer* Trailer = PackageLinker->GetPackageTrailer();
		if (Trailer != nullptr && Trailer->GetNumPayloads(UE::EPayloadStorageType::Virtualized) > 0)
		{
			bSavePackage = false;
			return;
		}
	}

	if (bOnlyPayloadTrailers && PackageLinker->GetPackageTrailer() == nullptr)
	{
		bSavePackage = false;
		return;
	}

	if (bSkipPayloadTrailers && PackageLinker->GetPackageTrailer() != nullptr)
	{
		bSavePackage = false;
		return;
	}

	// Check if the package contains any instances of the class that needs to be resaved.
	if (ResaveClasses.Num() > 0)
	{
		bSavePackage = false;
		for (int32 ExportIndex = 0; !bSavePackage && ExportIndex < PackageLinker->ExportMap.Num(); ExportIndex++)
		{
			FTopLevelAssetPath ExportClassPathName(PackageLinker->GetExportClassPackage(ExportIndex), PackageLinker->GetExportClassName(ExportIndex));
			if (ResaveClasses.Contains(ExportClassPathName.ToString()))
			{
				bSavePackage = true;
				break;
			}
		}
	}
}

bool UResavePackagesCommandlet::CheckoutFile(const FString& Filename, bool bAddFile, bool bIgnoreAlreadyCheckedOut)
{
	if (!bAutoCheckOut)
	{
		return true;
	}

	bool bIsReadOnly = IFileManager::Get().IsReadOnly(*Filename);
	if (!bIsReadOnly && !bAddFile)
	{
		return true;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	if (SourceControlState.IsValid())
	{
		// Already checked out/added this file
		if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
		{
			return true;
		}
		else if (!SourceControlState->IsSourceControlled())
		{
			if ( bAddFile )
			{
				if (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), *Filename) == ECommandResult::Succeeded)
				{
					UE_LOGF(LogContentCommandlet, Display, "[REPORT] %ls successfully added", *Filename);
					return true;
				}
				else
				{
					if (!bIgnoreAlreadyCheckedOut)
					{
						UE_LOGF(LogContentCommandlet, Error, "[REPORT] %ls could not be added!", *Filename);
					}
					else
					{
						UE_LOGF(LogContentCommandlet, Warning, "[REPORT] %ls could not be added!", *Filename);
					}
				}
			}
		}
		else if (!SourceControlState->IsCurrent())
		{
			if (!bIgnoreAlreadyCheckedOut)
			{
				UE_LOGF(LogContentCommandlet, Error, "[REPORT] %ls is not synced to head, can not submit", *Filename);
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Warning, "[REPORT] %ls is not synced to head, can not submit", *Filename);
			}
		}
		else if (!SourceControlState->CanCheckout())
		{
			FString CurrentlyCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&CurrentlyCheckedOutUser))
			{
				if (!bIgnoreAlreadyCheckedOut)
				{
					UE_LOGF(LogContentCommandlet, Error, "[REPORT] %ls level is already checked out by someone else (%ls), can not submit!", *Filename, *CurrentlyCheckedOutUser);
				}
				else
				{
					UE_LOGF(LogContentCommandlet, Warning, "[REPORT] %ls level is already checked out by someone else (%ls), can not submit!", *Filename, *CurrentlyCheckedOutUser);
				}
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Error, "[REPORT] Unable to checkout %ls, can not submit", *Filename);
			}
		}
		else 
		{
			if (SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), *Filename) == ECommandResult::Succeeded)
			{
				UE_LOGF(LogContentCommandlet, Display, "[REPORT] %ls Checked out successfully", *Filename);
				return true;
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Warning, "[REPORT] %ls could not be checked out!", *Filename);
			}
		}
	}
	return false;
}

bool UResavePackagesCommandlet::RevertFile(const FString& Filename)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	bool bSuccesfullyReverted = false;

	if (SourceControlState.IsValid())
	{
		if (SourceControlState->CanRevert() && (SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), *Filename) == ECommandResult::Succeeded))
		{
			bSuccesfullyReverted = true;
			UE_LOGF(LogContentCommandlet, Display, "[REPORT] %ls Reverted successfully", *Filename);			
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Warning, "[REPORT] %ls could not be reverted!", *Filename);
		}
	}

	return bSuccesfullyReverted;
}

bool UResavePackagesCommandlet::CanCheckoutFile(const FString& Filename, FString& CheckedOutUser)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*Filename, EStateCacheUsage::ForceUpdate);
	bool bCanCheckout = true;
	if (SourceControlState.IsValid())
	{
		if (!SourceControlState->CanCheckout())
		{
			if (!SourceControlState->IsCheckedOutOther(&CheckedOutUser))
			{
				CheckedOutUser = "";
			}

			bCanCheckout = false;
		}
	}

	return bCanCheckout;
}

void UResavePackagesCommandlet::CheckoutAndSavePackage(UPackage* Package, TArray<FString>& SublevelFilenames, bool bIgnoreAlreadyCheckedOut)
{
	check(Package);

	FString PackageFilename;
	if (FPackageName::TryConvertLongPackageNameToFilename(Package->GetName(), PackageFilename, Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension()))
	{
		if (IFileManager::Get().FileExists(*PackageFilename))
		{
			if (CheckoutFile(PackageFilename, true, bIgnoreAlreadyCheckedOut))
			{
				SublevelFilenames.Add(PackageFilename);
				if (!SavePackageHelper(Package, PackageFilename))
				{
					UE_LOGF(LogContentCommandlet, Error, "Failed to save existing package %ls", *PackageFilename);
				}
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Error, "Failed to check out existing package %ls", *PackageFilename);
			}
		}
		else
		{
			if (SavePackageHelper(Package, PackageFilename))
			{
				if (CheckoutFile(PackageFilename, true, bIgnoreAlreadyCheckedOut))
				{
					SublevelFilenames.Add(PackageFilename);
				}
				else
				{
					UE_LOGF(LogContentCommandlet, Error, "Failed to check out new package %ls", *PackageFilename);
				}
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Error, "Failed to save new package %ls", *PackageFilename);
			}
		}
	}
}

void UResavePackagesCommandlet::CheckInFiles(const TArray<FString>& InFilesToSubmit, const FText& InDescription) const
{
	if (!bAutoCheckIn || InFilesToSubmit.IsEmpty())
	{
		return;
	}

	FText FinalDescription = InDescription;
	if (!bSkipVirtualization)
	{
		if (!TryVirtualization(InFilesToSubmit, FinalDescription))
		{
			UE_LOGF(LogContentCommandlet, Error, "Files will not be checked in due to virtualization failure!");
			return;
		}
	}
	else
	{
		UE_LOGF(LogContentCommandlet, Display, "Skipping virtualization due to the cmdline");
	}

	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
	CheckInOperation->SetDescription(FinalDescription);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	SourceControlProvider.Execute(CheckInOperation, SourceControlHelpers::PackageFilenames(InFilesToSubmit));
}

bool UResavePackagesCommandlet::TryVirtualization(const TArray<FString>& FilesToSubmit, FText& InOutDescription)
{
	using namespace UE::Virtualization;

	IVirtualizationSystem& System = IVirtualizationSystem::Get();
	if (!System.IsEnabled())
	{
		return true;
	}

	EVirtualizationOptions VirtualizationOptions = EVirtualizationOptions::None;

	FVirtualizationResult Result = System.TryVirtualizePackages(FilesToSubmit, VirtualizationOptions);
	if (Result.WasSuccessful())
	{
		FTextBuilder NewDescription;
		NewDescription.AppendLine(InOutDescription);

		for (const FText& Line : Result.DescriptionTags)
		{
			NewDescription.AppendLine(Line);
		}

		InOutDescription = NewDescription.ToText();

		return true;
	}
	else if (System.AllowSubmitIfVirtualizationFailed())
	{
		for (const FText& Error : Result.Errors)
		{
			UE_LOGF(LogContentCommandlet, Warning, "%ls", *Error.ToString());
		}

		// Even though the virtualization process had problems we should continue submitting
		return true;
	}
	else
	{
		for (const FText& Error : Result.Errors)
		{
			UE_LOGF(LogContentCommandlet, Error, "%ls", *Error.ToString());
		}

		UE_LOGF(LogContentCommandlet, Error, "Failed to virtualize the files being submitted!");

		return false;
	}
}

void UResavePackagesCommandlet::PerformAdditionalOperations(class UWorld* World, bool& bSavePackage)
{
	check(World);

	TArray<TWeakObjectPtr<ULevel>> LevelsToRebuild;
	ABrush::NeedsRebuild(&LevelsToRebuild);
	for (const TWeakObjectPtr<ULevel>& Level : LevelsToRebuild)
	{
		if (Level.IsValid())
		{
			GEditor->RebuildLevel(*Level.Get());
		}
	}
	ABrush::OnRebuildDone();

	bool bRevertCheckedOutFilesIfNotSaving = true;

	const bool bAllowExternalPackageResaves = !Switches.Contains(TEXT("FixupRedirects")) && !Switches.Contains(TEXT("FixupRedirectors")) && !Switches.Contains(TEXT("NoExternalPackages"));
	const bool bShouldBuildTextureStreamingForWorld = bShouldBuildTextureStreaming && !bShouldBuildTextureStreamingForAll;
	const bool bBuildingNonHLODData = (bShouldBuildLighting || bShouldBuildTextureStreamingForWorld || bShouldBuildReflectionCaptures);

	// indicates if world and level packages should be checked out only if dirty after building data
	const bool bShouldCheckoutDirtyPackageOnly = (bShouldBuildHLOD || bShouldBuildNavigationData) && !bBuildingNonHLODData;

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	const bool bResaveWorldPartitionExternalPackages = !!WorldPartition;
	const int32 DefaultExternalActorGCFreq = 2048;

	// Load and Save Level's external packages
 	if (!bResaveWorldPartitionExternalPackages && bAllowExternalPackageResaves)
	{
		// Use a default GC frequency for external actors if GarbageCollectionFrequency is 0.
		TGuardValue<int32> ScopedGCFreq(GarbageCollectionFrequency, GarbageCollectionFrequency ? GarbageCollectionFrequency : DefaultExternalActorGCFreq);

		World->AddToRoot();
		for (UPackage* Package : World->PersistentLevel->GetPackage()->GetExternalPackages())
		{
			++TotalPackagesForResave;
			const FString PackageFilename = Package->GetLoadedPath().GetLocalFullPath();
			check(FLinkerLoad::FindExistingLinkerForPackage(Package));
			LoadAndSaveOnePackage(PackageFilename);
		}
		World->RemoveFromRoot();
	}

	if (!bBuildingNonHLODData && !bShouldBuildHLOD && !bShouldBuildNavigationData && !bResaveWorldPartitionExternalPackages)
	{
		return;
	}

	// Setup the world.
	UWorld::InitializationValues IVS;
	IVS.RequiresHitProxies(false);
	IVS.ShouldSimulatePhysics(false);
	IVS.EnableTraceCollision(false);
	IVS.CreateNavigation(bShouldBuildNavigationData);
	IVS.CreateAISystem(false);
	IVS.AllowAudioPlayback(false);
	IVS.CreatePhysicsScene(true);
	FScopedEditorWorld EditorWorld(World, IVS);

	// Load and Save world partition actor packages
	if (bResaveWorldPartitionExternalPackages && !bShouldBuildNavigationData)
	{
		// Use a default GC frequency for external actors if GarbageCollectionFrequency is 0.
		TGuardValue<int32> ScopedGCFreq(GarbageCollectionFrequency, GarbageCollectionFrequency ? GarbageCollectionFrequency : DefaultExternalActorGCFreq);

		if (bAllowExternalPackageResaves)
		{
			auto ResaveExternalPackage = [this](const UPackage* Package)
			{
				++TotalPackagesForResave;
				if (Package == nullptr)
				{
					check(bCanIgnoreFails);
					return;
				}
				const FString PackageFilename = Package->GetLoadedPath().GetLocalFullPath();
				check(FLinkerLoad::FindExistingLinkerForPackage(Package));
				LoadAndSaveOnePackage(PackageFilename);
			};

			// Resave all external actors packages
			FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [this, WorldPartition, ResaveExternalPackage](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				// Load & Register World Partition Actor
				FWorldPartitionReference LoadedActor(WorldPartition, ActorDescInstance->GetGuid());
				AActor* Actor = LoadedActor.GetActor();
				UPackage* Package = Actor ? Actor->GetExternalPackage() : nullptr;
				ResaveExternalPackage(Package);
				return true;
			});

			// Resave all external objects packages
			FExternalPackageHelper::LoadObjectsFromExternalPackages<UObject>(World, [this, ResaveExternalPackage](UObject* ExternalObject)
			{
				ResaveExternalPackage(ExternalObject->GetPackage());
			});
		}
	}

	if (bBuildingNonHLODData || bShouldBuildHLOD || bShouldBuildNavigationData)
	{
		bool bShouldProceedWithRebuild = true;

		TArray<FString> CheckedOutPackagesFilenames;
		auto CheckOutLevelFile = [this,&bShouldProceedWithRebuild, &CheckedOutPackagesFilenames](ULevel* InLevel)
		{
			if (InLevel && InLevel->MapBuildData)
			{
				UPackage* MapBuildDataPackage = InLevel->MapBuildData->GetOutermost();
				if (MapBuildDataPackage != InLevel->GetOutermost())
				{
					FString MapBuildDataPackageName;
					if (FPackageName::DoesPackageExist(MapBuildDataPackage->GetName(), &MapBuildDataPackageName))
					{
						if (CheckoutFile(MapBuildDataPackageName))
						{
							CheckedOutPackagesFilenames.Add(MapBuildDataPackageName);
						}
						else
						{
							bShouldProceedWithRebuild = false;
						}
					}
					else
					{
						bShouldProceedWithRebuild = false;
					}
				}
			}
		};

		FString WorldPackageName;
		if (FPackageName::DoesPackageExist(World->GetOutermost()->GetName(), &WorldPackageName))
		{
			if (!bShouldCheckoutDirtyPackageOnly)
			{
				// if we can't check out the main map or it's not up to date then we can't do the lighting rebuild at all!
				if (CheckoutFile(WorldPackageName))
				{
					CheckedOutPackagesFilenames.Add(WorldPackageName);

					CheckOutLevelFile(World->PersistentLevel);
				}
				else
				{
					bShouldProceedWithRebuild = false;
				}
			}
		}
		else
		{
			bShouldProceedWithRebuild = false;
		}

		if (bShouldProceedWithRebuild)
		{
			World->LoadSecondaryLevels(true, NULL);

			TArray<ULevelStreaming*> StreamingLevels = World->GetStreamingLevels();
			for (ULevelStreaming* StreamingLevel : StreamingLevels)
			{
				bool bShouldBeLoaded = true;

				if (!bShouldCheckoutDirtyPackageOnly)
				{
					CheckOutLevelFile(StreamingLevel->GetLoadedLevel());
				}

				FString StreamingLevelPackageFilename;
				const FString StreamingLevelWorldAssetPackageName = StreamingLevel->GetWorldAssetPackageName();
				if (FPackageName::DoesPackageExist(StreamingLevelWorldAssetPackageName, &StreamingLevelPackageFilename))
				{
					if (!bShouldCheckoutDirtyPackageOnly)
					{
						// check to see if we need to check this package out
						if (CheckoutFile(StreamingLevelPackageFilename))
						{
							CheckedOutPackagesFilenames.Add(StreamingLevelPackageFilename);
						}
						else
						{
							UE_LOGF(LogContentCommandlet, Error, "[REPORT] %ls is currently already checked out, cannot continue resaving", *StreamingLevelPackageFilename);
							bShouldProceedWithRebuild = false;
							bShouldBeLoaded = false;
							break;
						}
					}
				}

				if (!bShouldBeLoaded)
				{
					World->RemoveStreamingLevel(StreamingLevel);
				}
				
				StreamingLevel->SetShouldBeVisible(bShouldBeLoaded);
				StreamingLevel->SetShouldBeLoaded(bShouldBeLoaded);
			}
		}

	
		// If nothing came up that stops us from continuing, then start building lightmass
		if (bShouldProceedWithRebuild)
		{
			World->FlushLevelStreaming(EFlushLevelStreamingType::Full);

			// If we are (minimally) rebuilding using only dirty packages, set the visible streamed-in levels packages to clean (as FlushLevelStreaming will dirty their packages in this commandlet context)
			if (bShouldCheckoutDirtyPackageOnly)
			{
				for (const ULevel* Level : World->GetLevels())
				{
					if (Level->bIsVisible)
					{
						Level->GetOutermost()->SetDirtyFlag(false);
					}
				}
			}

			// We need any deferred commands added when loading to be executed before we start building lighting.
			GEngine->TickDeferredCommands();

			if (bShouldBuildTextureStreamingForWorld)
			{
				FEditorBuildUtils::EditorBuildTextureStreaming(World);
			}

			if (bShouldBuildHLOD)
			{
				UE_LOGF( LogContentCommandlet, Display, "Generating HLOD data for %ls", *World->GetOutermost()->GetName() );

				if( !ForceHLODSetupAsset.IsEmpty() )
				{
					TSubclassOf<UHierarchicalLODSetup> NewHLODSetupAsset = LoadClass<UHierarchicalLODSetup>( NULL, *ForceHLODSetupAsset, NULL, LOAD_None, NULL );
					if( NewHLODSetupAsset != nullptr )
					{
						World->GetWorldSettings()->HLODSetupAsset = NewHLODSetupAsset;
					}
					else
					{
						UE_LOGF( LogContentCommandlet, Fatal, "Could not find HLOD Setup Asset specified with the -ForceHLODSetupAsset option: %ls", *ForceHLODSetupAsset );
					}
				}

				// Use a single cluster for all actors in the level if we were asked to
				if( bForceSingleClusterForLevel )
				{
					World->GetWorldSettings()->bGenerateSingleClusterForLevel = true;
				}

				bool bHLODLeaveMapUntouched = GetDefault<UHierarchicalLODSettings>()->bSaveLODActorsToHLODPackages && !bHLODMapCleanup;

				// Maintain a list of packages that needs to be saved after cluster rebuilding.
				TSet<UPackage*> PackagesToSave;

				if (bHLODMapCleanup)
				{
					bool bPerformedCleanup = GEngine->GetEngineSubsystem<UHLODEngineSubsystem>()->CleanupHLODs(World);
					if (bPerformedCleanup)
					{
						PackagesToSave.Add(World->GetOutermost());
					}
				}

				FHierarchicalLODBuilder Builder(World, bSkipSubLevels);

				if (bForceClusterGeneration)
				{
					Builder.ClearHLODs();
					Builder.PreviewBuild();
				}
				else if (bGenerateClusters)
				{
					Builder.PreviewBuild();
				}

				if (bGenerateMeshProxies || bForceProxyGeneration)
				{
					Builder.BuildMeshesForLODActors(bForceProxyGeneration);
				}

				FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>(FName("HierarchicalLODUtilities"));
				FHierarchicalLODProxyProcessor* Processor = Module.GetProxyProcessor();

				while (Processor->IsProxyGenerationRunning())
				{
					FTSTicker::GetCoreTicker().Tick(static_cast<float>(FApp::GetDeltaTime()));
					FThreadManager::Get().Tick();
					FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
					FPlatformProcess::Sleep(0.1f);
				}

				// Flush shader compiler manager so we end up writing out any shaders we generated
				if(GShaderCompilingManager)
				{
					GShaderCompilingManager->ProcessAsyncResults(false, false);
				}

				// Get the list of packages needs to be saved after proxy mesh generation.
				for(ULevel* Level : World->GetLevels())
				{
					if(Level->bIsVisible)
					{
						Builder.GetMeshesPackagesToSave(Level, PackagesToSave);
					}
				}

				// Checkout and save each dirty package
				if (!bVerifyContent)
				{
					for (UPackage* Package : PackagesToSave)
					{
						if (Package->IsDirty())
						{
							CheckoutAndSavePackage(Package, CheckedOutPackagesFilenames, bSkipCheckedOutFiles);
						}
					}
				}

				// If the only operation performed by this commandlet is to update HLOD proxy packages,
				// avoid saving the level files.
				if (bHLODLeaveMapUntouched && !bBuildingNonHLODData && !bShouldBuildNavigationData)
				{
					bRevertCheckedOutFilesIfNotSaving  = false;
					bShouldProceedWithRebuild = false;
				}
			}

			if (bShouldBuildNavigationData)
			{
				// Make sure static meshes have compiled before generating navigation data
				FStaticMeshCompilingManager::Get().FinishAllCompilation();
				
				// Make sure navigation is added and initialized in EditorMode
				FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorMode);

				// Invoke navigation data generator
				UE_LOGF(LogContentCommandlet, Display, "Building navigation data for %ls", *World->GetOutermost()->GetName());
				FNavigationSystem::Build(*World);

				// Checkout and save each dirty package
				for (TActorIterator<ANavigationData> It(World); It; ++It)
				{
					UPackage* Package = It->GetOutermost();
					if (Package != nullptr && Package->IsDirty() && !Package->HasAnyFlags(RF_Transient))
					{
						CheckoutAndSavePackage(Package, CheckedOutPackagesFilenames, bSkipCheckedOutFiles);
					}
				}
			}

			if (bShouldBuildLighting)
			{
				FLightingBuildOptions LightingOptions;
 				LightingOptions.QualityLevel = LightingBuildQuality;

				auto BuildFailedDelegate = [&bShouldProceedWithRebuild,&World]() {
					UE_LOGF(LogContentCommandlet, Error, "[REPORT] Failed building lighting for %ls", *World->GetOutermost()->GetName());
					bShouldProceedWithRebuild = false;
				};

				FDelegateHandle BuildFailedDelegateHandle = FEditorDelegates::OnLightingBuildFailed.AddLambda(BuildFailedDelegate);

				GEditor->BuildLighting(LightingOptions);
				while (GEditor->IsLightingBuildCurrentlyRunning())
				{
					GEditor->UpdateBuildLighting();
				}

				if (bShouldBuildReflectionCaptures)
				{
					GEditor->BuildReflectionCaptures();
				}

				FEditorDelegates::OnLightingBuildFailed.Remove(BuildFailedDelegateHandle);
			}

			// If everything is a success, resave the levels.
			if (bShouldProceedWithRebuild)
			{
				auto SaveMapBuildData = [this, &CheckedOutPackagesFilenames](ULevel* InLevel)
				{
					if (InLevel && InLevel->MapBuildData && (bShouldBuildLighting || bShouldBuildHLOD || bShouldBuildReflectionCaptures))
					{
						UPackage* MapBuildDataPackage = InLevel->MapBuildData->GetOutermost();
						if (MapBuildDataPackage != InLevel->GetOutermost() && MapBuildDataPackage->IsDirty())
						{
							CheckoutAndSavePackage(MapBuildDataPackage, CheckedOutPackagesFilenames, bSkipCheckedOutFiles);
						}
					}
				};

				SaveMapBuildData(World->PersistentLevel);

				for (ULevelStreaming* NextStreamingLevel : World->GetStreamingLevels())
				{
					FString StreamingLevelPackageFilename;
					const FString StreamingLevelWorldAssetPackageName = NextStreamingLevel->GetWorldAssetPackageName();
					if (FPackageName::DoesPackageExist(StreamingLevelWorldAssetPackageName, &StreamingLevelPackageFilename) && CheckedOutPackagesFilenames.Contains(StreamingLevelPackageFilename))
					{
						UPackage* SubLevelPackage = NextStreamingLevel->GetLoadedLevel()->GetOutermost();
						bool bSaveSubLevelPackage = true;
						if (bShouldCheckoutDirtyPackageOnly)
						{
							bSaveSubLevelPackage = SubLevelPackage->IsDirty();
						}

						if (bSaveSubLevelPackage)
						{
							CheckoutAndSavePackage(SubLevelPackage, CheckedOutPackagesFilenames, bSkipCheckedOutFiles);
						}

						SaveMapBuildData(NextStreamingLevel->GetLoadedLevel());
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Error, "[REPORT] Failed to complete steps necessary to perform build for %ls", *World->GetName());
		}

		if (!bShouldProceedWithRebuild || !bSavePackage)
		{
			// don't save our parent package
			bSavePackage = false;
			
			if (bRevertCheckedOutFilesIfNotSaving)
			{
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			
				// revert all our packages
				for (const auto& CheckedOutPackageFilename : CheckedOutPackagesFilenames)
				{
					SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), *CheckedOutPackageFilename);
				}
			}
		}
		else
		{
			for(const auto& CheckedOutPackageFilename : CheckedOutPackagesFilenames)
			{
				FilesToSubmit.AddUnique(CheckedOutPackageFilename);
			}

			if (bShouldCheckoutDirtyPackageOnly)
			{
				bSavePackage = World->GetOutermost()->IsDirty();
				if (bSavePackage && bSkipCheckedOutFiles)
				{
					FString OutUser;
					bSavePackage = CanCheckoutFile(WorldPackageName, OutUser);
					if (!bSavePackage)
					{
						if (OutUser.Len())
						{
							UE_LOGF(LogContentCommandlet, Warning, "[REPORT] Skipping %ls as it is checked out by %ls", *WorldPackageName, *OutUser);
						}
						else
						{
							UE_LOGF(LogContentCommandlet, Warning, "[REPORT] Skipping %ls as it could not be checked out (not at head revision ?)", *WorldPackageName);
						}
					}
				}
			}
		}
	}
}

void UResavePackagesCommandlet::PerformAdditionalOperations( class UObject* Object, bool& bSavePackage )
{
	if (bOnlyMaterials && ForceMaterialAutoUsageFlag.IsSet())
	{
		if (UMaterial* Material = Cast<UMaterial>(Object); 
			(Material != nullptr) && (Material->bAutomaticallySetUsageInEditor != *ForceMaterialAutoUsageFlag))
		{
			Material->MarkPackageDirty();
			Material->bAutomaticallySetUsageInEditor = *ForceMaterialAutoUsageFlag;
			bSavePackage = true;
		}
	}
}


void UResavePackagesCommandlet::PerformAdditionalOperations( UPackage* Package, bool& bSavePackage )
{
	check(Package);
	bool bShouldSavePackage = false;
	
	if( ( FParse::Param(FCommandLine::Get(), TEXT("CLEANCLASSES")) == true ) && ( CleanClassesFromContentPackages(Package) == true ) )
	{
		bShouldSavePackage = true;
	}

	if (bShouldBuildTextureStreamingForAll)
	{
		if (FEditorBuildUtils::EditorBuildMaterialTextureStreamingData(Package))
		{
			bSavePackage = true;
		}
	}

	// add additional operations here

	bSavePackage = bSavePackage || bShouldSavePackage;
}


bool UResavePackagesCommandlet::CleanClassesFromContentPackages( UPackage* Package )
{
	check(Package);
	bool bResult = false;

	for ( TObjectIterator<UClass> It; It; ++It )
	{
		if ( It->IsIn(Package) )
		{
			UE_LOGF(LogContentCommandlet, Warning, "Removing class '%ls' from package [%ls]", *It->GetPathName(), *Package->GetName());

			// mark the class as transient so that it won't be saved into the package
			It->SetFlags(RF_Transient);

			// clear the standalone flag just to be sure :)
			It->ClearFlags(RF_Standalone);
			bResult = true;
		}
	}

	return bResult;
}

void UResavePackagesCommandlet::VerboseMessage(const FString& Message)
{
	if (Verbosity == VERY_VERBOSE)
	{
		UE_LOGF(LogContentCommandlet, Verbose, "%ls", *Message);
	}
}

FString UResavePackagesCommandlet::CreateTempFilename()
{
	return FPaths::CreateTempFilename(*GetTempFilesDirectory());
}

void UResavePackagesCommandlet::CleanTempFiles()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UResavePackagesCommandlet::CleanTempFiles);
	const FString DirPath = GetTempFilesDirectory();

	UE_LOGF(LogContentCommandlet, Display, "Cleaning temp file directory");

	if (!IFileManager::Get().DeleteDirectory(*DirPath, false, true))
	{
		UE_LOGF(LogContentCommandlet, Warning, "Failed to clean temp file directory:  %ls", *DirPath);
	}
}

FString UResavePackagesCommandlet::GetTempFilesDirectory()
{
	return FPaths::ProjectSavedDir() / TEXT("Temp/ResavePackages");
}

/*-----------------------------------------------------------------------------
	UWrangleContent.
-----------------------------------------------------------------------------*/

/** 
 * Helper struct to store information about a unreferenced object
 */
struct FUnreferencedObject
{
	/** Name of package this object resides in */
	FString PackageName;
	/** Full name of object */
	FString ObjectName;
	/** Size on disk as recorded in FObjectExport */
	int64 SerialSize;

	/**
	 * Constructor for easy creation in a TArray
	 */
	FUnreferencedObject(const FString& InPackageName, const FString& InObjectName, int32 InSerialSize)
	: PackageName(InPackageName)
	, ObjectName(InObjectName)
	, SerialSize(InSerialSize)
	{
	}
};

/**
 * Helper struct to store information about referenced objects insde
 * a package. Stored in TMap<> by package name, so this doesn't need
 * to store the package name 
 */
struct FPackageObjects
{
	/** All objected referenced in this package, and their class */
	TMap<FString, UClass*> ReferencedObjects;

	/** Was this package a fully loaded package, and saved right after being loaded? */
	bool bIsFullyLoadedPackage;

	FPackageObjects()
	: bIsFullyLoadedPackage(false)
	{
	}

};
	
FArchive& operator<<(FArchive& Ar, FPackageObjects& PackageObjects)
{
	Ar << PackageObjects.bIsFullyLoadedPackage;

	if (Ar.IsLoading())
	{
		int32 NumObjects = 0;
		FString ObjectName;
		FString ClassName;

		Ar << NumObjects;
		for (int32 ObjIndex = 0; ObjIndex < NumObjects; ObjIndex++)
		{
			Ar << ObjectName << ClassName;
			UClass* Class = StaticLoadClass(UObject::StaticClass(), NULL, *ClassName, NULL, LOAD_None, NULL);
			PackageObjects.ReferencedObjects.Add(*ObjectName, Class);
		}
	}
	else if (Ar.IsSaving())
	{
		int32 NumObjects = PackageObjects.ReferencedObjects.Num();
		Ar << NumObjects;
		for (TMap<FString, UClass*>::TIterator It(PackageObjects.ReferencedObjects); It; ++It)
		{
			FString ObjectName, ClassName;
			ObjectName = It.Key();
			ClassName = It.Value()->GetPathName();

			Ar << ObjectName << ClassName;
		}
		
	}

	return Ar;
}

/**
 * Stores the fact that an object (given just a name) was referenced
 *
 * @param PackageName Name of the package the object lives in
 * @param ObjectName FullName of the object
 * @param ObjectClass Class of the object
 * @param ObjectRefs Map to store the object information in
 * @param bIsFullLoadedPackage true if the packge this object is in was fully loaded
 */
void ReferenceObjectInner(const FString& PackageName, const FString& ObjectName, UClass* ObjectClass, TMap<FString, FPackageObjects>& ObjectRefs, bool bIsFullyLoadedPackage)
{
	// look for an existing FPackageObjects
	FPackageObjects* PackageObjs = ObjectRefs.Find(*PackageName);
	// if it wasn't found make a new entry in the map
	if (PackageObjs == NULL)
	{
		PackageObjs = &ObjectRefs.Add(*PackageName, FPackageObjects());
	}

	// if either the package was already marked as fully loaded or it now is fully loaded, then
	// it will be fully loaded
	PackageObjs->bIsFullyLoadedPackage = PackageObjs->bIsFullyLoadedPackage || bIsFullyLoadedPackage;

	// add this referenced object to the map
	PackageObjs->ReferencedObjects.Add(*ObjectName, ObjectClass);

	// make sure the class is in the root set so it doesn't get GC'd, making the pointer we cached invalid
	ObjectClass->AddToRoot();
}

/**
 * Stores the fact that an object was referenced
 *
 * @param Object The object that was referenced
 * @param ObjectRefs Map to store the object information in
 * @param bIsFullLoadedPackage true if the package this object is in was fully loaded
 */
void ReferenceObject(UObject* Object, TMap<FString, FPackageObjects>& ObjectRefs, bool bIsFullyLoadedPackage)
{
	FString PackageName = Object->GetOutermost()->GetName();

	// find the outermost non-upackage object, as it will be loaded later with all its subobjects
	while (Object->GetOuter() && Object->GetOuter()->GetClass() != UPackage::StaticClass())
	{
		Object = Object->GetOuter();
	}

	// make sure this object is valid (it's not in a script or native-only package)
	// An invalid writable outer name indicates the package name is in a temp or script path, or is using a short package name
	const bool bValidWritableOuterName = FPackageName::IsValidLongPackageName(Object->GetOutermost()->GetName());
	bool bIsValid = true;
	// can't be in a script packge or be a field/template in a native package, or a top level pacakge, or in the transient package
	if (!bValidWritableOuterName ||
		Object->GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript) ||
		Object->IsA(UField::StaticClass()) ||
		Object->IsTemplate(RF_ClassDefaultObject) ||
		Object->GetOuter() == NULL ||
		Object->IsIn(GetTransientPackage()))
	{
		bIsValid = false;
	}

	if (bIsValid)
	{
		// save the reference
		ReferenceObjectInner(PackageName, Object->GetFullName(), Object->GetClass(), ObjectRefs, bIsFullyLoadedPackage);

		//@todo-packageloc Add reference to localized packages.
	}
}

/**
 * Take a package pathname and return a path for where to save the cutdown
 * version of the package. Will create the directory if needed.
 *
 * @param Filename Path to a package file
 * @param CutdownDirectoryName Name of the directory to put this package into
 *
 * @return Location to save the cutdown package
 */
FString MakeCutdownFilename(const FString& Filename, const TCHAR* CutdownDirectoryName=TEXT("CutdownPackages"))
{
	// replace the .. with ..\GAMENAME\CutdownContent
	FString CutdownDirectory = FPaths::GetPath(Filename);
	if ( CutdownDirectory.Contains(FPaths::ProjectDir()) )
	{
		// Content from the game directory may not be relative to the engine folder
		CutdownDirectory = CutdownDirectory.Replace(*FPaths::ProjectDir(), *FString::Printf(TEXT("%s%s/Game/"), *FPaths::ProjectSavedDir(), CutdownDirectoryName));
	}
	else
	{
		CutdownDirectory = CutdownDirectory.Replace(TEXT("../../../"), *FString::Printf(TEXT("%s%s/"), *FPaths::ProjectSavedDir(), CutdownDirectoryName));
	}

	// make sure it exists
	IFileManager::Get().MakeDirectory(*CutdownDirectory, true);

	// return the full pathname
	return CutdownDirectory / FPaths::GetCleanFilename(Filename);
}

UWrangleContentCommandlet::UWrangleContentCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LogToConsole = false;
}

int32 UWrangleContentCommandlet::Main( const FString& Params )
{
	// overall commandlet control options
	bool bShouldRestoreFromPreviousRun = FParse::Param(*Params, TEXT("restore"));
	bool bShouldSavePackages = !FParse::Param(*Params, TEXT("nosave"));
	bool bShouldSaveUnreferencedContent = !FParse::Param(*Params, TEXT("nosaveunreferenced"));
	bool bShouldDumpUnreferencedContent = FParse::Param(*Params, TEXT("reportunreferenced"));
	bool bShouldCleanOldDirectories = !FParse::Param(*Params, TEXT("noclean"));
	bool bShouldSkipMissingClasses = FParse::Param(*Params, TEXT("skipMissingClasses"));

	// what per-object stripping to perform
	bool bShouldStripLargeEditorData = FParse::Param(*Params, TEXT("striplargeeditordata"));
	bool bShouldStripMips = FParse::Param(*Params, TEXT("stripmips"));

	// package loading options
	bool bShouldLoadAllMaps = FParse::Param(*Params, TEXT("allmaps"));
	
	// if no platforms specified, keep them all
	UE_LOGF(LogContentCommandlet, Warning, "Keeping platform-specific data for ALL platforms");

	FString SectionStr;
	FParse::Value( *Params, TEXT( "SECTION=" ), SectionStr );

	// store all referenced objects
	TMap<FString, FPackageObjects> AllReferencedPublicObjects;
	TSet<FString> ExternalPackages;

	if (bShouldRestoreFromPreviousRun)
	{
		FArchive* Ar = IFileManager::Get().CreateFileReader(*(FPaths::ProjectDir() + TEXT("Wrangle.bin")));
		if( Ar != NULL )
		{
			*Ar << AllReferencedPublicObjects;
			delete Ar;
		}
		else
		{
			UE_LOGF(LogContentCommandlet, Warning, "Could not read in Wrangle.bin so not restoring and doing a full wrangle" );
		}
	}
	else
	{
		// make name for our ini file to control loading
		FString WrangleContentIniName =	FPaths::SourceConfigDir() + TEXT("WrangleContent.ini");

		// figure out which section to use to get the packages to fully load
		FString PackagesToFullyLoadSectionName = TEXT("WrangleContent.PackagesToFullyLoad");
		FString CollectionsToFullyLoadSectionName = TEXT("WrangleContent.CollectionsToFullyLoad");

		if (SectionStr.Len() > 0)
		{
			PackagesToFullyLoadSectionName = FString::Printf(TEXT("WrangleContent.%sPackagesToFullyLoad"), *SectionStr);
			CollectionsToFullyLoadSectionName = FString::Printf(TEXT("WrangleContent.%sCollectionsToFullyLoad"), *SectionStr);
		}

		// get a list of packages to load
		const FConfigSection* PackagesToFullyLoadSection = GConfig->GetSection(*PackagesToFullyLoadSectionName, 0, *WrangleContentIniName);
		const FConfigSection* StartupPackages = GConfig->GetSection(TEXT("/Script/Engine.StartupPackages"), 0, GEngineIni);
		const FConfigSection* CollectionsToFullyLoadSection = GConfig->GetSection(*CollectionsToFullyLoadSectionName, 0, *WrangleContentIniName);

		// we expect either the .ini to exist, or -allmaps to be specified
		if (!PackagesToFullyLoadSection && !bShouldLoadAllMaps && !CollectionsToFullyLoadSection)
		{
			UE_LOGF(LogContentCommandlet, Error, "This commandlet needs a WrangleContent.ini in the Config directory with at least a [WrangleContent.PackagesToFullyLoad] section or a [WragnelContent.CollectionsToFullyLoad] section.");
			return 1;
		}

		if (bShouldCleanOldDirectories)
		{
			IFileManager::Get().DeleteDirectory(*FString::Printf(TEXT("%sCutdownPackages"), *FPaths::ProjectSavedDir()), false, true);
			IFileManager::Get().DeleteDirectory(*FString::Printf(TEXT("%sNFSContent"), *FPaths::ProjectSavedDir()), false, true);
		}

		// copy the packages to load, since we are modifying it
		FConfigSectionMap PackagesToFullyLoad;
		if (PackagesToFullyLoadSection)
		{
			PackagesToFullyLoad = *PackagesToFullyLoadSection;
		}
		
		if (bShouldLoadAllMaps)
		{
			TArray<FString> AllPackageFilenames;
			FEditorFileUtils::FindAllPackageFiles(AllPackageFilenames);
			for (int32 PackageIndex = 0; PackageIndex < AllPackageFilenames.Num(); PackageIndex++)
			{
				const FString& Filename = AllPackageFilenames[PackageIndex];
				if (FPaths::GetExtension(Filename,true) == FPackageName::GetMapPackageExtension() )
				{
					PackagesToFullyLoad.Add(TEXT("Package"), FPackageName::FilenameToLongPackageName(Filename));
				}
			}
		}

		// read in the per-map packages to cook
		TMap<FString, TArray<FString> > PerMapCookPackages;
		GConfig->Parse1ToNSectionOfStrings(TEXT("/Script/Engine.PackagesToForceCookPerMap"), TEXT("Map"), TEXT("Package"), PerMapCookPackages, GEngineIni);

		// gather any per map packages for cooking
		TArray<FString> PerMapPackagesToLoad;
		for (FConfigSectionMap::TIterator PackageIt(PackagesToFullyLoad); PackageIt; ++PackageIt)
		{
			// add dependencies for the per-map packages for this map (if any)
			TArray<FString>* Packages = PerMapCookPackages.Find(PackageIt.Value().GetValue());
			if (Packages != NULL)
			{
				for (int32 PackageIndex = 0; PackageIndex < Packages->Num(); PackageIndex++)
				{
					PerMapPackagesToLoad.Add(*(*Packages)[PackageIndex]);
				}
			}
		}

		// now add them to the list of all packages to load
		for (int32 PackageIndex = 0; PackageIndex < PerMapPackagesToLoad.Num(); PackageIndex++)
		{
			PackagesToFullyLoad.Add(TEXT("Package"), *PerMapPackagesToLoad[PackageIndex]);
		}

		// all currently loaded public objects were referenced by script code, so mark it as referenced
		for(FThreadSafeObjectIterator ObjectIt;ObjectIt;++ObjectIt)
		{
			UObject* Object = *ObjectIt;

			// record all public referenced objects
//			if (Object->HasAnyFlags(RF_Public))
			{
				ReferenceObject(Object, AllReferencedPublicObjects, false);
			}
		}

		if (CollectionsToFullyLoadSection)
		{
			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
			TArray<FSoftObjectPath> CollectionAssets;
			
			for (FConfigSectionMap::TConstIterator CollectionIt(*CollectionsToFullyLoadSection); CollectionIt; ++CollectionIt)
			{
				CollectionAssets.Reset();
				const FString& Collection = CollectionIt.Value().GetValue();

				TSharedPtr<ICollectionContainer> CollectionContainer;
				FName CollectionName;
				ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
				if (!CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType) ||
					!CollectionContainer->GetAssetsInCollection(CollectionName, ShareType, CollectionAssets))
				{
					UE_LOGF(LogContentCommandlet, Warning, "Could not get assets in collection '%ls'. Skipping filter.", *Collection);
				}
				else
				{
					//insert all of the collection names into the set for fast filter checks
					for (const FSoftObjectPath& AssetPath : CollectionAssets)
					{
						PackagesToFullyLoad.Add(TEXT("Package"), AssetPath.GetLongPackageName());
					}
				}
			}
		}
		
		// go over all the packages that we want to fully load
		for (FConfigSectionMap::TIterator PackageIt(PackagesToFullyLoad); PackageIt; ++PackageIt)
		{
			// there may be multiple sublevels to load if this package is a persistent level with sublevels
			TArray<FString> PackagesToLoad;
			// start off just loading this package (more may be added in the loop)
			PackagesToLoad.Add(*PackageIt.Value().GetValue());

			TArray<UObject*> ObjectsAddedToRoot;
			TSet<UObject*> AlreadyVisited;
			for (int32 PackageIndex = 0; PackageIndex < PackagesToLoad.Num(); PackageIndex++)
			{
				// save a copy of the packagename (not a reference in case the PackgesToLoad array gets realloced)
				FString PackageName = PackagesToLoad[PackageIndex];
				const bool bIsExternalPackage = ExternalPackages.Contains(PackageName);
				const bool bForceSkipSavePackage = bIsExternalPackage;
				// Avoid GC'ing between each external package
				const int32 GarbageCollectionFrequency = bIsExternalPackage ? 2048 : 0;

				FPackagePath PackagePath;
				if (!FPackagePath::TryFromMountedName(PackageName, PackagePath) ||
					!FPackageName::DoesPackageExist(PackagePath, &PackagePath))
				{
					continue;
				}

				FString PackageFilename = PackagePath.GetLocalFullPath();
				SET_WARN_COLOR(COLOR_WHITE);
				UE_LOGF(LogContentCommandlet, Warning, "Fully loading %ls... [%d/%d]", *PackageFilename, PackageIndex + 1, PackagesToLoad.Num());
				CLEAR_WARN_COLOR();

	// @todo josh: track redirects in this package and then save the package instead of copy it if there were redirects
	// or make sure that the following redirects marks the package dirty (which maybe it shouldn't do in the editor?)

				// load the package fully
				UPackage* Package = LoadPackage(nullptr, PackagePath, LOAD_None);

				// Now that we've loaded the package, go ahead and also load any soft paths.
				// This should help capture more references.
				GRedirectCollector.ResolveAllSoftObjectPaths(NAME_None);
				
				FLinkerLoad* Linker = GetPackageLinker(nullptr, PackagePath, LOAD_Quiet | LOAD_NoWarn | LOAD_NoVerify, nullptr);

				UWorld* World = UWorld::FindWorldInPackage(Package);
				if (World && World->IsPartitionedWorld())
				{
					TArray<FString> ExternalActorPackages = World->PersistentLevel->GetOnDiskExternalActorPackages();
					if (ExternalActorPackages.Num())
					{
						PackagesToLoad.Append(ExternalActorPackages);
						// Exclude external actors packages from save
						ExternalPackages.Append(ExternalActorPackages);
					
						// Keep partition world around to avoid reloading it for every loaded external actor
						World->AddToRoot();
						ObjectsAddedToRoot.Add(World);
					}
				}

				// look for special package types
				bool bIsMap = Linker->ContainsMap();
				bool bIsScriptPackage = Linker->ContainsCode();

				// collect all public objects loaded
				for(FThreadSafeObjectIterator ObjectIt; ObjectIt; ++ObjectIt)
				{
					UObject* Object = *ObjectIt;
					
					// Because we don't GC between each load of external packages, we want to skip already visited objects
					bool bAlreadyVisited = false;
					AlreadyVisited.Add(Object, &bAlreadyVisited);
					if (bAlreadyVisited)
					{
						continue;
					}

					// Add any levels referenced by Level Instance actors to the list of levels to load
					ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Object);
					if (LevelInstance && !CastChecked<AActor>(LevelInstance)->IsTemplate())
					{
						FString LevelPackageName = LevelInstance->GetWorldAssetPackage();
						if (!LevelPackageName.IsEmpty() && PackagesToFullyLoad.FindKey(LevelPackageName) == nullptr)
						{
							PackagesToLoad.AddUnique(LevelPackageName);
						}
					}

					// record all public referenced objects (skipping over top level packages)
					if (/*Object->HasAnyFlags(RF_Public) &&*/ Object->GetOuter() != NULL)
					{
						// is this public object in a fully loaded package?
						bool bIsObjectInFullyLoadedPackage = Object->IsIn(Package);

						if (bIsMap && bIsObjectInFullyLoadedPackage && Object->HasAnyFlags(RF_Public))
						{
							UE_LOGF(LogContentCommandlet, Warning, "Clearing public flag on map object %ls", *Object->GetFullName());
							Object->ClearFlags(RF_Public);
							// mark that we need to save the package since we modified it (instead of copying it)
							Object->MarkPackageDirty();
						}
						else
						{
							// record that this object was referenced
							ReferenceObject(Object, AllReferencedPublicObjects, bIsObjectInFullyLoadedPackage);
						}
					}
				}

				// add any sublevels of this world to the list of levels to load
				for (TObjectIterator<UWorld> WorldIt; WorldIt; ++WorldIt)
				{
					// iterate over streaming level objects loading the levels.
					for (ULevelStreaming* StreamingLevel : (*WorldIt)->GetStreamingLevels())
					{
						if (StreamingLevel)
						{
							FString SubLevelName = StreamingLevel->GetWorldAssetPackageName();
							// add this sublevel's package to the list of packages to load if it's not already in the master list of packages
							if (PackagesToFullyLoad.FindKey(SubLevelName) == NULL)
							{
								PackagesToLoad.AddUnique(SubLevelName);
							}
						}
					}
				}

				// save/copy the package if desired, and only if it's not a script package (script code is
				// not cutdown, so we always use original script code)
				if (bShouldSavePackages && !bIsScriptPackage && !bForceSkipSavePackage)
				{
					// make the name of the location to put the package
					FString CutdownPackageName = MakeCutdownFilename(PackageFilename);
						
					// if the package was modified by loading it, then we should save the package
					if (Package->IsDirty())
					{
						// save the fully load packages
						UE_LOGF(LogContentCommandlet, Warning, "Saving fully loaded package %ls...", *CutdownPackageName);
						if (!SavePackageHelper(Package, CutdownPackageName))
						{
							UE_LOGF(LogContentCommandlet, Error, "Failed to save package %ls...", *CutdownPackageName);
						}
					}
					else
					{
						UE_LOGF(LogContentCommandlet, Warning, "Copying fully loaded package %ls...", *CutdownPackageName);
						// copy the unmodified file (faster than saving) (0 is success)
						if (IFileManager::Get().Copy(*CutdownPackageName, *PackageFilename) != 0)
						{
							UE_LOGF(LogContentCommandlet, Error, "Failed to copy package to %ls...", *CutdownPackageName);
						}
					}
				}

				static int32 Counter = 0;
				if (!GarbageCollectionFrequency || (Counter++ % GarbageCollectionFrequency) == 0)
				{
					AlreadyVisited.Empty();
					CollectGarbage(RF_NoFlags);
				}
			}

			// Get rid of rooted objects if any
			if (ObjectsAddedToRoot.Num())
			{
				for (UObject* Object : ObjectsAddedToRoot)
				{
					Object->RemoveFromRoot();
				}
				CollectGarbage(RF_NoFlags);
			}
		}

		// save out the referenced objects so we can restore
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*(FPaths::ProjectDir() + TEXT("Wrangle.bin")));
		*Ar << AllReferencedPublicObjects;
		delete Ar;
	}

	// list of all objects that aren't needed
	TArray<FUnreferencedObject> UnnecessaryPublicObjects;
	TMap<FString, FPackageObjects> UnnecessaryObjectsByPackage;
	TMap<FString, bool> UnnecessaryObjects;
	TArray<FString> UnnecessaryPackages;

	// now go over all packages, quickly, looking for public objects NOT in the AllNeeded array
	TArray<FString> AllPackages;
	FEditorFileUtils::FindAllPackageFiles(AllPackages);

	if (bShouldDumpUnreferencedContent || bShouldSaveUnreferencedContent)
	{
		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogContentCommandlet, Warning, "");
		UE_LOGF(LogContentCommandlet, Warning, "Looking for unreferenced objects:");
		CLEAR_WARN_COLOR();

		// Iterate over all files doing stuff.
		for (int32 PackageIndex = 0; PackageIndex < AllPackages.Num(); PackageIndex++)
		{
			FPackagePath PackagePath;
			if (!FPackagePath::TryFromMountedName(AllPackages[PackageIndex], PackagePath))
			{
				UE_LOGF(LogContentCommandlet, Warning, "Skipping unmounted path %ls...", *AllPackages[PackageIndex]);
				continue;
			}
			FString PackageName = PackagePath.GetPackageName();

			// the list of objects in this package
			FPackageObjects* PackageObjs = NULL;

			// this will be set to true if every object in the package is unnecessary
			bool bAreAllObjectsUnnecessary = false;

			if (PackagePath.GetHeaderExtension() == EPackageExtension::Map)
			{
				UE_LOGF(LogContentCommandlet, Warning, "Skipping map %ls...", *PackagePath.GetDebugName());
				continue;
			}
			else
			{
				// get the objects referenced by this package
				PackageObjs = AllReferencedPublicObjects.Find(*PackageName);

				// if the were no objects referenced in this package, we can just skip it, 
				// and mark the whole package as unreferenced
				if (PackageObjs == NULL)
				{
					UE_LOGF(LogContentCommandlet, Warning, "No objects in %ls were referenced...", *PackagePath.GetDebugName());
					UnnecessaryPublicObjects.Emplace(PackageName,
						TEXT("ENTIRE PACKAGE"), IPackageResourceManager::Get().FileSize(PackagePath));

					// all objects in this package are unnecessary
					bAreAllObjectsUnnecessary = true;
				}
				else if (PackageObjs->bIsFullyLoadedPackage)
				{
					UE_LOGF(LogContentCommandlet, Warning, "Skipping fully loaded package %ls...", *PackagePath.GetDebugName());
					continue;
				}
				else
				{
					UE_LOGF(LogContentCommandlet, Warning, "Scanning %ls...", *PackagePath.GetDebugName());
				}
			}

			// Load the package before accessing its linker to prevent interference with async loads in case
			// that package was already queued and being loaded asynchronously.
			LoadPackage(nullptr, PackagePath, LOAD_Quiet | LOAD_NoWarn | LOAD_NoVerify);

			FLinkerLoad* Linker = GetPackageLinker(nullptr, PackagePath, LOAD_Quiet | LOAD_NoWarn | LOAD_NoVerify, nullptr);

			// go through the exports in the package, looking for public objects
			for (int32 ExportIndex = 0; ExportIndex < Linker->ExportMap.Num(); ExportIndex++)
			{
				FObjectExport& Export = Linker->ExportMap[ExportIndex];
				FString ExportName = Linker->GetExportFullName(ExportIndex);

				// some packages may have brokenness in them so we want to just continue so we can wrangle
				if( Export.ObjectName == NAME_None )
				{
					UE_LOGF(LogContentCommandlet, Warning, "    Export.ObjectName == NAME_None  for Package: %ls ", *PackagePath.GetDebugName());
					continue;
				}

				// make sure its outer is a package, and this isn't a package
				if (Linker->GetExportClassName(ExportIndex) == NAME_Package || 
					(!Export.OuterIndex.IsNull() && Linker->GetExportClassName(Export.OuterIndex) != NAME_Package))
				{
					continue;
				}

				// was it not already referenced?
				// NULL means it wasn't in the reffed public objects map for the package
				if (bAreAllObjectsUnnecessary || PackageObjs->ReferencedObjects.Find(ExportName) == NULL)
				{
					// is it public?
					if ((Export.ObjectFlags & RF_Public) != 0 && !bAreAllObjectsUnnecessary)
					{
						// if so, then add it to list of unused pcreateexportublic items
						UnnecessaryPublicObjects.Emplace(PackageName, ExportName, Export.SerialSize);
					}

					// look for existing entry
					FPackageObjects* ObjectsInPackage = UnnecessaryObjectsByPackage.Find(PackagePath.GetLocalBaseFilenameWithPath());
					// if not found, make a new one
					if (ObjectsInPackage == NULL)
					{
						ObjectsInPackage = &UnnecessaryObjectsByPackage.Add(PackagePath.GetLocalBaseFilenameWithPath(), FPackageObjects());
					}

					// get object's class
					FString ClassName;
					if(Export.ClassIndex.IsImport())
					{
						ClassName = Linker->GetImportPathName(Export.ClassIndex);
					}
					else
					{
						ClassName = Linker->GetExportPathName(Export.ClassIndex);
					}
					UClass* Class = StaticLoadClass(UObject::StaticClass(), NULL, *ClassName, NULL, LOAD_None, NULL);
					// When wrangling content, you often are loading packages that have not been saved in ages and have a reference to a class
					// that no longer exists.  Instead of asserting, we will just continue

					if ( !Class )
					{
						UE_LOGF(LogContentCommandlet, Warning, "Missing class %ls", *ClassName);
						check(bShouldSkipMissingClasses);
						continue;
					}

					// make sure it doesn't get GC'd
					Class->AddToRoot();
				
					// add this referenced object to the map
					ObjectsInPackage->ReferencedObjects.Add(*ExportName, Class);

					// add this to the map of all unnecessary objects
					UnnecessaryObjects.Add(*ExportName, true);
				}
			}

			// collect garbage every 20 packages (we aren't fully loading, so it doesn't need to be often)
			if ((PackageIndex % 20) == 0)
			{
				CollectGarbage(RF_NoFlags);
			}
		}
	}

	if (bShouldSavePackages)
	{
		int32 NumPackages = AllReferencedPublicObjects.Num();

		// go through all packages, and save out referenced objects
		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogContentCommandlet, Warning, "");
		UE_LOGF(LogContentCommandlet, Warning, "Saving referenced objects in %d Packages:", NumPackages);
		CLEAR_WARN_COLOR();
		int32 PackageIndex = 0;
		for (TMap<FString, FPackageObjects>::TIterator It(AllReferencedPublicObjects); It; ++It, PackageIndex++ )
		{
			const FString& PackageName = It.Key();

			// Skip save of external packages
			const bool bForceSkipSavePackage = ExternalPackages.Contains(PackageName);
			if (bForceSkipSavePackage)
			{
				continue;
			}

			// if the package was a fully loaded package, than we already saved it
			if (It.Value().bIsFullyLoadedPackage)
			{
				continue;
			}

			// package for all loaded objects
			UPackage* Package = NULL;
			
			// fully load all the referenced objects in the package
			for (TMap<FString, UClass*>::TIterator It2(It.Value().ReferencedObjects); It2; ++It2)
			{
				// get the full object name
				FString ObjectPathName = It2.Key();

				// skip over the class portion (the It2.Value() has the class pointer already)
				int32 Space = ObjectPathName.Find(TEXT(" "), ESearchCase::CaseSensitive);
				check(Space);

				// get everything after the space
				ObjectPathName.RightInline(ObjectPathName.Len() - (Space + 1), EAllowShrinking::No);

				// load the referenced object

				UObject* Object = StaticLoadObject(It2.Value(), NULL, *ObjectPathName, NULL, LOAD_NoWarn, NULL);

				// the object may not exist, because of attempting to load localized content
				if (Object)
				{
					check(Object->GetPathName() == ObjectPathName);

					// set the package if needed
					if (Package == NULL)
					{
						Package = Object->GetOutermost();
					}
					else
					{
						// make sure all packages are the same
						check(Package == Object->GetOutermost());
					}
				}
			}

			// make sure we found some objects in here
			// Don't worry about script packages
			if (Package)
			{
				// mark this package as fully loaded so it can be saved, even though we didn't fully load it
				// (which is the point of this commandlet)
				Package->MarkAsFullyLoaded();

				// get original path of package
				FString OriginalPackageFilename;

				//UE_LOGF(LogContentCommandlet, Warning, "*It.Key(): %ls", *It.Key() );

				// we need to be able to find the original package
				if( FPackageName::DoesPackageExist(PackageName, &OriginalPackageFilename) == false )
				{
					UE_LOGF(LogContentCommandlet, Fatal, "Could not find file in file cache: %ls", *PackageName);
				}

				// any maps need to be fully referenced
				check( FPaths::GetExtension(OriginalPackageFilename, true) != FPackageName::GetMapPackageExtension() );

				// make the filename for the output package
				FString CutdownPackageName = MakeCutdownFilename(OriginalPackageFilename);

				UE_LOGF(LogContentCommandlet, Warning, "Saving %ls... [%d/%d]", *CutdownPackageName, PackageIndex + 1, NumPackages);

				// save the package now that all needed objects in it are loaded.
				// At this point, any object still around should be saved so we pass all flags so all objects are saved
				SavePackageHelper(Package, *CutdownPackageName, RF_AllFlags, GWarn, SAVE_CutdownPackage);

				// close up this package
				CollectGarbage(RF_NoFlags);
			}
		}
	}

	if (bShouldDumpUnreferencedContent)
	{
		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogContentCommandlet, Warning, "");
		UE_LOGF(LogContentCommandlet, Warning, "Unreferenced Public Objects:");
		CLEAR_WARN_COLOR();

		// create a .csv
		FString CSVFilename = FString::Printf(TEXT("%sUnreferencedObjects-%s.csv"), *FPaths::ProjectLogDir(), *FDateTime::Now().ToString());
		FArchive* CSVFile = IFileManager::Get().CreateFileWriter(*CSVFilename);

		if (!CSVFile)
		{
			UE_LOGF(LogContentCommandlet, Error, "Failed to open output file %ls", *CSVFilename);
		}

		for (int32 ObjectIndex = 0; ObjectIndex < UnnecessaryPublicObjects.Num(); ObjectIndex++)
		{
			FUnreferencedObject& Object = UnnecessaryPublicObjects[ObjectIndex];
			UE_LOGF(LogContentCommandlet, Warning, "%ls", *Object.ObjectName);

			// dump out a line to the .csv file
			// @todo: sort by size to Excel's 65536 limit gets the biggest objects
			FString CSVLine = FString::Printf(TEXT("%s,%s,%" INT64_FMT "%s"), *Object.PackageName, *Object.ObjectName, Object.SerialSize, LINE_TERMINATOR);
			CSVFile->Serialize(TCHAR_TO_ANSI(*CSVLine), CSVLine.Len());
		}
	}

	// load every unnecessary object by package, rename it and any unnecessary objects if uses, to the 
	// an unnecessary package, and save it
	if (bShouldSaveUnreferencedContent)
	{
		int32 NumPackages = UnnecessaryObjectsByPackage.Num();
		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogContentCommandlet, Warning, "");
		UE_LOGF(LogContentCommandlet, Warning, "Saving unreferenced objects [%d packages]:", NumPackages);
		CLEAR_WARN_COLOR();

		// go through each package that has unnecessary objects in it
		int32 PackageIndex = 0;
		for (TMap<FString, FPackageObjects>::TIterator PackageIt(UnnecessaryObjectsByPackage); PackageIt; ++PackageIt, PackageIndex++)
		{
			const FString& PackageWithUnnecessaryObjects = PackageIt.Key();
			UE_LOGF(LogContentCommandlet, Warning, "Processing %ls", *PackageWithUnnecessaryObjects);

			UPackage* FullyLoadedPackage = NULL;
			// fully load unnecessary packages with no objects, 
			if (PackageIt.Value().ReferencedObjects.Num() == 0)
			{
				// just load it, and don't need a reference to it
				FullyLoadedPackage = LoadPackage(NULL, FPackagePath::FromLocalPath(PackageWithUnnecessaryObjects), LOAD_None);
			}
			else
			{
				// load every unnecessary object in this package
				for (TMap<FString, UClass*>::TIterator ObjectIt(PackageIt.Value().ReferencedObjects); ObjectIt; ++ObjectIt)
				{
					// get the full object name
					FString ObjectPathName = ObjectIt.Key();

					// skip over the class portion (the It2.Value() has the class pointer already)
					int32 Space = ObjectPathName.Find(TEXT(" "), ESearchCase::CaseSensitive);
					check(Space > 0);

					// get everything after the space
					ObjectPathName.RightInline(ObjectPathName.Len() - (Space + 1), EAllowShrinking::No);

					// load the unnecessary object
					UObject* Object = StaticLoadObject(ObjectIt.Value(), NULL, *ObjectPathName, NULL, LOAD_NoWarn, NULL);
					
					// this object should exist since it was gotten from a linker
					if (!Object)
					{
						UE_LOGF(LogContentCommandlet, Error, "Failed to load object %ls, it will be deleted permanently!", *ObjectPathName);
					}
				}
			}

			// now find all loaded objects (in any package) that are in marked as unnecessary,
			// and rename them to their destination
			for (TObjectIterator<UObject> It; It; ++It)
			{
				// if was unnecessary...
				if (UnnecessaryObjects.Find(*It->GetFullName()))
				{
					UObject* UnnecessaryObject = *It;
					// ... then rename it (its outer needs to be a package, everything else will have to be
					// moved by its outer getting moved)
					if (!UnnecessaryObject->IsA(UPackage::StaticClass()) &&
						UnnecessaryObject->GetOuter() &&
						UnnecessaryObject->GetOuter()->IsA(UPackage::StaticClass()))
					{
						FString PackageFullName = UnnecessaryObject->GetOuter()->GetPathName();
						FString Path = FPackageName::GetLongPackagePath(PackageFullName);
						FString AssetName = FPackageName::GetLongPackageAssetName(PackageFullName);
						if (AssetName.Left(4) != TEXT("NFS_"))
						{
							FString NewPackageName = FString::Printf(TEXT("%s/NFS_%s"), *Path, *AssetName);
							UPackage* NewPackage = CreatePackage(*NewPackageName);
							//UE_LOGF(LogContentCommandlet, Warning, "Renaming object from %ls to %ls.%ls", *UnnecessaryObject->GetPathName(), *NewPackage->GetPathName(), *UnnecessaryObject->GetName());

							// move the object if we can. IF the rename fails, then the object was already renamed to this spot, but not GC'd.
							// that's okay.
							if (UnnecessaryObject->Rename(*UnnecessaryObject->GetName(), NewPackage, REN_Test))
							{
								UnnecessaryObject->Rename(*UnnecessaryObject->GetName(), NewPackage, REN_None);
							}
						}
					}
				}
			}

			// find the one we moved this packages objects to
			FPackagePath PackagePath = FPackagePath::FromLocalPath(PackageWithUnnecessaryObjects);
			FString PackageFilePath = PackagePath.GetLocalFullPath();
			FString PackageName = PackagePath.GetPackageName();
			FString FindPackageName = FString::Printf(TEXT("%s/NFS_%s"), *FPackageName::GetLongPackagePath(PackageName), *FPackageName::GetLongPackageAssetName(PackageName));
			// convert the new name to a a NFS directory directory
			FString MovedFilename = MakeCutdownFilename(FString::Printf(TEXT("%s/NFS_%s"), *FPaths::GetPath(PackageFilePath), *FPaths::GetCleanFilename(PackageFilePath)), TEXT("NFSContent"));

			// finally save it out
			UPackage* MovedPackage = FindPackage(NULL, *FindPackageName);
			if (ensure(MovedPackage))
			{
				UE_LOGF(LogContentCommandlet, Warning, "Saving package %ls [%d/%d]", *MovedFilename, PackageIndex, NumPackages);
				SavePackageHelper(MovedPackage, *MovedFilename);
			}
			else
			{
				UE_LOGF(LogContentCommandlet, Warning, "Moved package %ls not found.", *FindPackageName);
				UE_LOGF(LogContentCommandlet, Error, "Can't save package %ls [%d/%d]", *MovedFilename, PackageIndex, NumPackages);
			}
			CollectGarbage(RF_NoFlags);
		}
	}

	return 0;
}

/* ==========================================================================================================
	UListMaterialsUsedWithMeshEmittersCommandlet
========================================================================================================== */

UListMaterialsUsedWithMeshEmittersCommandlet::UListMaterialsUsedWithMeshEmittersCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UListMaterialsUsedWithMeshEmittersCommandlet::ProcessParticleSystem( UParticleSystem* ParticleSystem , TArray<FString> &OutMaterials)
{
	for (int32 EmitterIndex = 0; EmitterIndex < ParticleSystem->Emitters.Num(); EmitterIndex++)
	{
		UParticleEmitter *Emitter = ParticleSystem->Emitters[EmitterIndex];
		if (Emitter && Emitter->LODLevels.Num() > 0)
		{
			UParticleLODLevel* LODLevel = Emitter->LODLevels[0];
			// Only process mesh emitters
			if (LODLevel && 
				LODLevel->TypeDataModule && 
				LODLevel->TypeDataModule->IsA( UParticleModuleTypeDataMesh::StaticClass())) // The type data module is mesh type data
			{

				// Attempt to find MeshMaterial module on emitter.
				UParticleModuleTypeDataMesh* MeshTypeData = Cast<UParticleModuleTypeDataMesh>(LODLevel->TypeDataModule);
				bool bFoundMaterials = false;
				for( int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ++ModuleIdx )
				{
					if(LODLevel->Modules[ModuleIdx]->IsA(UParticleModuleMeshMaterial::StaticClass()))
					{
						UParticleModuleMeshMaterial* MaterialModule = Cast<UParticleModuleMeshMaterial>( LODLevel->Modules[ ModuleIdx ] );
						for(int32 MatIdx = 0; MatIdx < MaterialModule->MeshMaterials.Num(); MatIdx++ )
						{
							if(MaterialModule->MeshMaterials[MatIdx])
							{
								bFoundMaterials = true;
								if(!MaterialModule->MeshMaterials[MatIdx]->GetUsageByFlag(MATUSAGE_MeshParticles))
								{
									OutMaterials.AddUnique(MaterialModule->MeshMaterials[MatIdx]->GetPathName());
								}
							}
						}
					}
				}

				// Check override material only if we've not found materials on a MeshMaterial module within the emitter
				if(!bFoundMaterials && MeshTypeData->bOverrideMaterial)
				{
					UMaterialInterface* OverrideMaterial = LODLevel->RequiredModule->Material;
					if(OverrideMaterial && !OverrideMaterial->GetUsageByFlag(MATUSAGE_MeshParticles))
					{
						OutMaterials.AddUnique(OverrideMaterial->GetPathName());
					}
				}

				// Find materials on the static mesh
				else if (!bFoundMaterials)
				{
					if (MeshTypeData->Mesh)
					{
						for (int32 MaterialIdx = 0; MaterialIdx < MeshTypeData->Mesh->GetStaticMaterials().Num(); MaterialIdx++)
						{
							if(UMaterialInterface* Mat = MeshTypeData->Mesh->GetStaticMaterials()[MaterialIdx].MaterialInterface)
							{
								if(!Mat->GetUsageByFlag(MATUSAGE_MeshParticles))
								{
									OutMaterials.AddUnique(Mat->GetPathName());
								}
							}							
						}
					}
				}
			}
		}
	}
}

int32 UListMaterialsUsedWithMeshEmittersCommandlet::Main( const FString& Params )
{

	TArray<FString> FilesInPath;
	FEditorFileUtils::FindAllPackageFiles(FilesInPath);

	if( FilesInPath.Num() == 0 )
	{
		UE_LOGF(LogContentCommandlet, Warning,"No packages found");
		return 1;
	}

	TArray<FString> MaterialList;
	int32 GCIndex = 0;
	int32 TotalPackagesChecked = 0;

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogContentCommandlet, Display, "Searching Asset Registry for particle systems");
	AssetRegistryModule.Get().SearchAllAssets(true);

	// Retrieve list of all assets, used to find unreferenced ones.
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssetsByClass(UParticleSystem::StaticClass()->GetClassPathName(), AssetList, true);

	for(int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx )
	{
		const FString Filename = AssetList[AssetIdx].GetObjectPathString();

		UE_LOGF(LogContentCommandlet, Display, "Processing particle system (%i/%i):  %ls ", AssetIdx, AssetList.Num(), *Filename );

		UPackage* Package = LoadPackage( NULL, *Filename, LOAD_Quiet );
		if ( Package == NULL )
		{
			UE_LOGF(LogContentCommandlet, Error, "Error loading %ls!", *Filename );
			continue;
		}

		TotalPackagesChecked++;
		for (TObjectIterator<UParticleSystem> It; It; ++It )
		{
			UParticleSystem* ParticleSys = *It;
			if (ParticleSys->IsIn(Package) && !ParticleSys->IsTemplate())
			{
				// For any mesh emitters we append to MaterialList any materials that are referenced and don't have bUsedWithMeshParticles set.
				ProcessParticleSystem(ParticleSys, MaterialList);
			}
		}

		// Collect garbage every 10 packages instead of every package makes the commandlet run much faster
		if( (++GCIndex % 10) == 0 )
		{
			CollectGarbage(RF_NoFlags);
		}
	}

	if(MaterialList.Num() > 0)
	{
		// Now, dump out the list of materials that require updating.
		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
		UE_LOGF(LogContentCommandlet, Display, "The following materials require bUsedWithMeshParticles to be enabled:");
		for(int32 Index = 0; Index < MaterialList.Num(); ++Index)
		{
			UE_LOGF(LogContentCommandlet, Error, "%ls", *MaterialList[Index] );
		}
		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
	}
	else
	{
		UE_LOGF(LogContentCommandlet, Display, "No materials require updating!");
	}
	return 0;
}


/* ==========================================================================================================
	UListStaticMeshesImportedFromSpeedTreesCommandlet
========================================================================================================== */

UListStaticMeshesImportedFromSpeedTreesCommandlet::UListStaticMeshesImportedFromSpeedTreesCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

int32 UListStaticMeshesImportedFromSpeedTreesCommandlet::Main(const FString& Params)
{

	TArray<FString> FilesInPath;
	FEditorFileUtils::FindAllPackageFiles(FilesInPath);

	if (FilesInPath.Num() == 0)
	{
		UE_LOGF(LogContentCommandlet, Warning, "No packages found");
		return 1;
	}

	TArray<FString> StaticMeshList;
	int32 GCIndex = 0;
	int32 TotalPackagesChecked = 0;

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogContentCommandlet, Display, "Searching Asset Registry for static mesh ");
	AssetRegistryModule.Get().SearchAllAssets(true);

	// Retrieve list of all assets, used to find unreferenced ones.
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssetsByClass(UStaticMesh::StaticClass()->GetClassPathName(), AssetList, true);

	for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
	{
		const FString Filename = AssetList[AssetIdx].GetObjectPathString();

		UE_LOGF(LogContentCommandlet, Display, "Processing static mesh (%i/%i):  %ls ", AssetIdx, AssetList.Num(), *Filename);

		UPackage* Package = LoadPackage(NULL, *Filename, LOAD_Quiet);
		if (Package == NULL)
		{
			UE_LOGF(LogContentCommandlet, Error, "Error loading %ls!", *Filename);
			continue;
		}

		TotalPackagesChecked++;
		for (TObjectIterator<UStaticMesh> It; It; ++It)
		{
			UStaticMesh* StaticMesh = *It;
			if (StaticMesh->IsIn(Package) && !StaticMesh->IsTemplate())
			{
				// If the mesh was imported from a speedtree, we append the static mesh name to the list.
				if (StaticMesh->SpeedTreeWind.IsValid())
				{
					StaticMeshList.Add(StaticMesh->GetPathName());
				}
			}
		}

		// Collect garbage every 10 packages instead of every package makes the commandlet run much faster
		if ((++GCIndex % 10) == 0)
		{
			CollectGarbage(RF_NoFlags);
		}
	}

	if (StaticMeshList.Num() > 0)
	{
		// Now, dump out the list of materials that require updating.
		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
		UE_LOGF(LogContentCommandlet, Display, "The following static meshes were imported from SpeedTrees:");
		for (int32 Index = 0; Index < StaticMeshList.Num(); ++Index)
		{
			UE_LOGF(LogContentCommandlet, Error, "%ls", *StaticMeshList[Index]);
		}
		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
	}
	else
	{
		UE_LOGF(LogContentCommandlet, Display, "No static meshes were imported from speedtrees in this project.");
	}
	return 0;
}

/* ==========================================================================================================
	UStaticMeshMinLodCommandlet
========================================================================================================== */

UStaticMeshMinLodCommandlet::UStaticMeshMinLodCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UStaticMeshMinLodCommandlet::Main(const FString& Params)
{
	TArray<FString> FilesInPath;
	FEditorFileUtils::FindAllPackageFiles(FilesInPath);
	if (FilesInPath.Num() == 0)
	{
		UE_LOGF(LogContentCommandlet, Warning, "No packages found");
		return 1;
	}

	// parse the mapping from PerPlatform to PerQualityLevel
	// ex:-mapping=Mobile:Low,Switch:Medium,Desktop:High,PS4:High,XboxOne:High,PS5:Epic,XSX:Epic;console:medium;high;epic,Desktop:high
	TMultiMap<FName, FName> PerPlatformToQualityLevel;
	FString MappingStr;
	if (FParse::Value(*Params, TEXT("-mapping="), MappingStr, false))
	{
		TArray<FString> Mappings;
		MappingStr.ParseIntoArray(Mappings, TEXT(","), true);
		for (FString& PlatformToQualityLevel : Mappings)
		{
			TArray<FString> Entries;
			PlatformToQualityLevel.ParseIntoArray(Entries, TEXT(":"), false);

			if (Entries.Num() != 2)
			{
				UE_LOGF(LogContentCommandlet, Error, "Error bad -mapping argument: %ls", *MappingStr);
				return 1;
			}

			TArray<FString> Values;
			Entries[1].ParseIntoArray(Values, TEXT(";"), false);

			for (const FString& Value : Values)
			{
				PerPlatformToQualityLevel.AddUnique(FName(*Entries[0]), FName(*Value));
			}
		}
	}

	bool bConvertToQualityLevel = PerPlatformToQualityLevel.Num() > 0;
	bool bNoSourceControl = FParse::Param(*Params, TEXT("nosourcecontrol"));
	bool bGenerateCollections = FParse::Param(*Params, TEXT("collections"));
	ISourceControlProvider* SourceControlProvider = bNoSourceControl ? nullptr : &ISourceControlModule::Get().GetProvider();
	const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

	struct OverrideGroupInfo
	{
		int32 Count;
		TArray<FString> Names;
		TArray<FString> Paths;
	};

	TMap<FName, OverrideGroupInfo> MinLodStats;
	TArray<FString> StaticMeshList;
	int32 TotalPackagesChecked = 0;
	int32 TotalStaticMeshesChecked = 0;
	int32 GCIndex = 0;

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogContentCommandlet, Display, "Searching Asset Registry for static mesh ");
	AssetRegistryModule.Get().SearchAllAssets(true);

	// Retrieve list of all assets, used to find unreferenced ones.
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssetsByClass(UStaticMesh::StaticClass()->GetClassPathName(), AssetList, true);
	TArray<UPackage*> PackagesToSave;

	for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
	{
		bool SavePackage = false;
		const FString Filename = AssetList[AssetIdx].GetObjectPathString();
		UE_LOGF(LogContentCommandlet, Display, "Processing static mesh (%i/%i):  %ls ", AssetIdx, AssetList.Num(), *Filename);

		UPackage* Package = LoadPackage(NULL, *Filename, LOAD_Quiet);
		if (Package == NULL)
		{
			UE_LOGF(LogContentCommandlet, Error, "Error loading %ls!", *Filename);
			continue;
		}

		TotalPackagesChecked++;

		for (TObjectIterator<UStaticMesh> It; It; ++It)
		{
			UStaticMesh* StaticMesh = *It;
			if (StaticMesh->IsIn(Package) && !StaticMesh->IsTemplate() && StaticMesh->GetMinLOD().PerPlatform.Num() > 0)
			{
				if (bConvertToQualityLevel)
				{
					//Convert default value
					FPerPlatformInt PerPlatformMinLOD = StaticMesh->GetMinLOD();
					FPerQualityLevelInt QualityLevelMinLOD;
					QualityLevelMinLOD.PerQuality.Empty();
					QualityLevelMinLOD.Default = PerPlatformMinLOD.Default;
					
					// convert each platform overrides
					for (const TPair<FName, int32>& Pair : StaticMesh->GetMinLOD().PerPlatform)
					{
						// get all quality levels associated with the PerPlatform override (PS4:high, Console:medium;high;Epic) ...
						TArray<FName> QLNames;
						PerPlatformToQualityLevel.MultiFind(Pair.Key, QLNames);

						for (const FName& QLName : QLNames)
						{
							int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QLName);
							if (QLKey != INDEX_NONE)
							{
								int32* Value = QualityLevelMinLOD.PerQuality.Find(QLKey);

								// if the quality level already as a value, only change it if the value is lower
								// this can happen if two mapping key as the same quality level but different min lod idx value
								// ex: Desktop=2, PS4=1
								// If Desktop and Console also is mapping to high, we take the smallest min lod idx
								if (Value != nullptr && Pair.Value < *Value)
								{
									// only change the override if its a smaller minLod
									*Value = Pair.Value;
								}
								else
								{
									QualityLevelMinLOD.PerQuality.Add(QLKey, Pair.Value);
								}
							}
						}
					}
					StaticMesh->SetQualityLevelMinLOD(MoveTemp(QualityLevelMinLOD));
				}

				//generate a unique group name to collect stats
				TArray<FName> PerPlatformNames;
				StaticMesh->GetMinLOD().PerPlatform.GenerateKeyArray(PerPlatformNames);

				// sort platform names so that (Switch, PS4) and (PS4, Switch) produce the same key
				PerPlatformNames.Sort(FNameLexicalLess());

				FString FinalCategoryName;
				for (const FName& SortedPlatformName : PerPlatformNames)
				{
					FinalCategoryName += SortedPlatformName.ToString() + TEXT(" ");
				}

				// save the override info
				OverrideGroupInfo& Info = MinLodStats.FindOrAdd(FName(*FinalCategoryName));
				Info.Count++;
				Info.Names.Add(StaticMesh->GetPathName());
				Info.Paths.Add(Filename);

				if (!SavePackage)
				{
					PackagesToSave.AddUnique(Package);
					SavePackage = true;
				}
				
				TotalStaticMeshesChecked++;
			}
		}
	}

	// output stats to the log
	// generate collection uasset
	int32 TotalOverrideCount = 0;
	for (const TPair<FName, OverrideGroupInfo>& Pair : MinLodStats)
	{
		const OverrideGroupInfo& Info = Pair.Value;

		if (bGenerateCollections)
		{
			if (!CollectionContainer->CollectionExists(Pair.Key, ECollectionShareType::CST_Local))
			{
				CollectionContainer->CreateCollection(Pair.Key, ECollectionShareType::CST_Local, ECollectionStorageMode::Static);
			}
		}

		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
		UE_LOGF(LogContentCommandlet, Display, "Mask: %ls", *Pair.Key.ToString());
		UE_LOGF(LogContentCommandlet, Display, "Nb overrides: %d", Info.Count);

		for (int i = 0; i < Info.Names.Num(); i++)
		{
			UE_LOGF(LogContentCommandlet, Display, "	%ls", *Info.Names[i]);

			if (bGenerateCollections)
			{
				CollectionContainer->AddToCollection(Pair.Key, ECollectionShareType::CST_Local, FSoftObjectPath(Info.Paths[i]));
			}
		}

		if (bGenerateCollections)
		{
			CollectionContainer->SaveCollection(Pair.Key, ECollectionShareType::CST_Local);
		}

		UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
		TotalOverrideCount += Info.Count;
	}

	UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");
	UE_LOGF(LogContentCommandlet, Display, "Total overrides: %d", TotalOverrideCount);
	UE_LOGF(LogContentCommandlet, Display, "Total ovestatic meshesrrides: %d", TotalStaticMeshesChecked);
	UE_LOGF(LogContentCommandlet, Display, "-------------------------------------------------------------------");

	// save quality level modifications 
	if (bConvertToQualityLevel)
	{
		if (SourceControlProvider)
		{
			FEditorFileUtils::CheckoutPackages(PackagesToSave, nullptr, false);
		}
		else
		{
			for (UPackage* Package : PackagesToSave)
			{
				FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
				if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename))
				{
					if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackageFilename, false))
					{
						UE_LOGF(LogContentCommandlet, Error, "Error setting %ls writable", *PackageFilename);
						return 1;
					}
				}
			}
		}
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false, nullptr, true, false);
	}
	
	return 0;
}


/* ==========================================================================================================
	ULandscapeGrassTypeCommandlet
========================================================================================================== */

ULandscapeGrassTypeCommandlet::ULandscapeGrassTypeCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 ULandscapeGrassTypeCommandlet::Main(const FString& Params)
{
	TArray<FString> FilesInPath;
	FEditorFileUtils::FindAllPackageFiles(FilesInPath);
	if (FilesInPath.Num() == 0)
	{
		UE_LOGF(LogContentCommandlet, Warning, "No packages found");
		return 1;
	}

	// parse the mapping from PerPlatform to PerQualityLevel
	// ex:-mapping=Mobile:Low,Switch:Medium,Desktop:High,PS4:High,XboxOne:High,PS5:Epic,XSX:Epic,console:medium;high;epic,Desktop:high
	TMultiMap<FName, FName> PerPlatformToQualityLevel;
	FString MappingStr;
	if (FParse::Value(*Params, TEXT("-mapping="), MappingStr, false))
	{
		TArray<FString> Mappings;
		MappingStr.ParseIntoArray(Mappings, TEXT(","), true);
		for (FString& PlatformToQualityLevel : Mappings)
		{
			TArray<FString> Entries;
			PlatformToQualityLevel.ParseIntoArray(Entries, TEXT(":"), false);

			if (Entries.Num() != 2)
			{
				UE_LOGF(LogContentCommandlet, Error, "Error bad -mapping argument: %ls", *MappingStr);
				return 1;
			}

			TArray<FString> Values;
			Entries[1].ParseIntoArray(Values, TEXT(";"), false);

			for (const FString& Value : Values)
			{
				PerPlatformToQualityLevel.AddUnique(FName(*Entries[0]), FName(*Value));
			}
		}
	}

	if (PerPlatformToQualityLevel.Num() == 0)
	{
		UE_LOGF(LogContentCommandlet, Warning, "No Mapping rules found");
		return 1;
	}

	bool bNoSourceControl = FParse::Param(*Params, TEXT("nosourcecontrol"));
	ISourceControlProvider* SourceControlProvider = bNoSourceControl ? nullptr : &ISourceControlModule::Get().GetProvider();

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogContentCommandlet, Display, "Searching Asset Registry for static mesh ");
	AssetRegistryModule.Get().SearchAllAssets(true);

	// Retrieve list of all assets, used to find unreferenced ones.
	TArray<FAssetData> AssetList;
	int32 TotalPackagesChecked = 0;
	AssetRegistryModule.Get().GetAssetsByClass(ULandscapeGrassType::StaticClass()->GetClassPathName(), AssetList, true);
	TArray<UPackage*> PackagesToSave;

	for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
	{
		bool SavePackage = false;
		const FString Filename = AssetList[AssetIdx].GetObjectPathString();
		UE_LOGF(LogContentCommandlet, Display, "Processing static mesh (%i/%i):  %ls ", AssetIdx, AssetList.Num(), *Filename);

		UPackage* Package = LoadPackage(NULL, *Filename, LOAD_Quiet);
		if (Package == NULL)
		{
			UE_LOGF(LogContentCommandlet, Error, "Error loading %ls!", *Filename);
			continue;
		}

		TotalPackagesChecked++;

		for (TObjectIterator<ULandscapeGrassType> It; It; ++It)
		{
			ULandscapeGrassType* GrassType = *It;
			if (GrassType->IsIn(Package) && !GrassType->IsTemplate())
			{
				for (FGrassVariety& GrassVariety : GrassType->GrassVarieties)
				{
					GrassVariety.GrassDensityQuality.ConvertQualityLevelData(GrassVariety.GrassDensity.PerPlatform, PerPlatformToQualityLevel, GrassVariety.GrassDensity.Default);
					GrassVariety.StartCullDistanceQuality.ConvertQualityLevelData(GrassVariety.StartCullDistance.PerPlatform, PerPlatformToQualityLevel, GrassVariety.StartCullDistance.Default);
					GrassVariety.EndCullDistanceQuality.ConvertQualityLevelData(GrassVariety.EndCullDistance.PerPlatform, PerPlatformToQualityLevel, GrassVariety.EndCullDistance.Default);
				}
			}
		}

		if (!SavePackage)
		{
			PackagesToSave.AddUnique(Package);
			SavePackage = true;
		}
	}

	// save quality level modifications 
	if (SourceControlProvider)
	{
		FEditorFileUtils::CheckoutPackages(PackagesToSave, nullptr, false);
	}
	else
	{
		for (UPackage* Package : PackagesToSave)
		{
			FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
			if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename))
			{
				if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackageFilename, false))
				{
					UE_LOGF(LogContentCommandlet, Error, "Error setting %ls writable", *PackageFilename);
					return 1;
				}
			}
		}
	}
	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false, nullptr, true, false);
	
	return 0;
}
