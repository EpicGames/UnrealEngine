// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 ConvertLevelsToExternalActorsCommandlet: Commandlet used to convert levels uses external actors in batch
=============================================================================*/

#include "Commandlets/ConvertLevelsToExternalActorsCommandlet.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Editor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/LevelStreaming.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectHash.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageHelperFunctions.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConvertLevelsToExternalActorsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogConvertLevelsToExternalActorsCommandlet, All, All);

UConvertLevelsToExternalActorsCommandlet::UConvertLevelsToExternalActorsCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

ULevel* UConvertLevelsToExternalActorsCommandlet::LoadLevel(const FString& LevelToLoad) const
{
	SET_WARN_COLOR(COLOR_WHITE);
	UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Loading level %ls.", *LevelToLoad);
	CLEAR_WARN_COLOR();

	FString MapLoadCommand = FString::Printf(TEXT("MAP LOAD FILE=%s TEMPLATE=0 SHOWPROGRESS=0 FEATURELEVEL=3"), *LevelToLoad);
	GEditor->Exec(nullptr, *MapLoadCommand, *GError);
	FlushAsyncLoading();

	UPackage* MapPackage = FindPackage(nullptr, *LevelToLoad);
	UWorld* World = MapPackage ? UWorld::FindWorldInPackage(MapPackage) : nullptr;
	return World ? World->PersistentLevel : nullptr;
}

void UConvertLevelsToExternalActorsCommandlet::GetSubLevelsToConvert(ULevel* MainLevel, TSet<ULevel*>& SubLevels, bool bRecursive)
{
	UWorld* World = MainLevel->GetTypedOuter<UWorld>();
	for(ULevelStreaming* StreamingLevel: World->GetStreamingLevels())
	{
		if (ULevel* SubLevel = StreamingLevel->GetLoadedLevel())
		{
			SubLevels.Add(SubLevel);
			if (bRecursive)
			{
				// Recursively obtain sub levels to convert
				GetSubLevelsToConvert(SubLevel, SubLevels, bRecursive);
			}
		}
	}
}

bool UConvertLevelsToExternalActorsCommandlet::CheckExternalActors(const FString& Level, bool bRepair)
{
	// Gather duplicated actor files.
	TMultiMap<FSoftObjectPath, FName> DuplicatedActorFiles;
	{
		TMap<FSoftObjectPath, FName> ActorFiles;

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		// Let the asset registry process any pending operations before continuing
		AssetRegistry.Tick(-1);

		FDelegateHandle AddedCheckHandle = AssetRegistry.OnAssetAdded().AddLambda([&ActorFiles](const FAssetData& AssetData)
		{
			check(!ActorFiles.Contains(AssetData.GetSoftObjectPath()));
			ActorFiles.Add(AssetData.GetSoftObjectPath(), AssetData.PackageName);
		});

		FDelegateHandle UpdatedCheckHandle = AssetRegistry.OnAssetUpdated().AddLambda([&ActorFiles, &DuplicatedActorFiles](const FAssetData& AssetData)
		{
			FName ExistingPackageName;
			if (ActorFiles.RemoveAndCopyValue(AssetData.GetSoftObjectPath(), ExistingPackageName))
			{
				DuplicatedActorFiles.Add(AssetData.GetSoftObjectPath(), ExistingPackageName);
			}

			DuplicatedActorFiles.Add(AssetData.GetSoftObjectPath(), AssetData.PackageName);
		});

		AssetRegistry.ScanPathsSynchronous(ULevel::GetExternalObjectsPaths(Level), /*bForceRescan*/true, /*bIgnoreDenyListScanFilters*/true);

		AssetRegistry.OnAssetAdded().Remove(AddedCheckHandle);
		AssetRegistry.OnAssetUpdated().Remove(UpdatedCheckHandle);
	}

	if (DuplicatedActorFiles.Num())
	{
		// Gather unique keys from the duplicated map.
		// Note: TMultiMap::GenerateKeyArray will return duplicated keys, clean that.
		TArray<FSoftObjectPath> DuplicatedActorFilesKeys;
		DuplicatedActorFiles.GenerateKeyArray(DuplicatedActorFilesKeys);
		DuplicatedActorFilesKeys.Sort([](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.FastLess(B); });
		int32 EndIndex = Algo::Unique(DuplicatedActorFilesKeys);
		DuplicatedActorFilesKeys.RemoveAt(EndIndex, DuplicatedActorFilesKeys.Num() - EndIndex);

		// Report or delete duplicated entries, keeping the latest one
		for (const FSoftObjectPath& DuplicatedActorFileKey : DuplicatedActorFilesKeys)
		{
			TArray<FName> DuplicatedActorFilesPaths;
			DuplicatedActorFiles.MultiFind(DuplicatedActorFileKey, DuplicatedActorFilesPaths);
			check(DuplicatedActorFilesPaths.Num() > 1);

			FDateTime MostRecentStamp;
			int32 MostRecentIndex = INDEX_NONE;

			for (int32 i=0; i<DuplicatedActorFilesPaths.Num(); i++)
			{
				FString Filename = FPackageName::LongPackageNameToFilename(DuplicatedActorFilesPaths[i].ToString(), FPackageName::GetAssetPackageExtension());
				FDateTime FileTimeStamp = IFileManager::Get().GetTimeStamp(*Filename);

				if ((MostRecentIndex == INDEX_NONE) || (FileTimeStamp > MostRecentStamp))
				{
					MostRecentIndex = i;
					MostRecentStamp = FileTimeStamp;
				}
			}

			check(MostRecentIndex != INDEX_NONE);

			for (int32 i=0; i<DuplicatedActorFilesPaths.Num(); i++)
			{
				if (i != MostRecentIndex)
				{
					FString Filename = FPackageName::LongPackageNameToFilename(DuplicatedActorFilesPaths[i].ToString(), FPackageName::GetAssetPackageExtension());

					UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Warning, "Found duplicated actor file %ls", *Filename);

					if (bRepair)
					{
						DeleteFile(Filename);
					}
				}
			}
		}

		return bRepair;
	}

	return true;
}

bool UConvertLevelsToExternalActorsCommandlet::AddPackageToSourceControl(UPackage* Package)
{
	if (UseSourceControl())
	{
		FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
		FSourceControlStatePtr SourceControlState = GetSourceControlProvider().GetState(PackageFilename, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
		{
			UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Adding package %ls to revision control", *PackageFilename);
			if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FMarkForAdd>(), Package) != ECommandResult::Succeeded)
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error adding %ls to revision control.", *PackageFilename);
				return false;
			}
		}
	}

	return true;
}

bool UConvertLevelsToExternalActorsCommandlet::SavePackage(UPackage* Package)
{
	// Use GEditor save as it does some UWorld specific shenanigans such as handle level offsets
	FString PackageFileName = SourceControlHelpers::PackageFilename(Package);
	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	FSavePackageResultStruct SaveResult = GEditor->Save(Package, nullptr, *PackageFileName, SaveArgs);

	if (SaveResult.Result != ESavePackageResult::Success)
	{
		UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error saving %ls", *PackageFileName);
		return false;
	}

	return true;
}

bool UConvertLevelsToExternalActorsCommandlet::CheckoutPackage(UPackage* Package)
{
	if (UseSourceControl())
	{
		FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
		FSourceControlStatePtr SourceControlState = GetSourceControlProvider().GetState(PackageFilename, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid())
		{
			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Overwriting package %ls already checked out by %ls, will not submit", *PackageFilename, *OtherCheckedOutUser);
				return false;
			}
			else if (!SourceControlState->IsCurrent())
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Overwriting package %ls (not at head revision), will not submit", *PackageFilename);
				return false;
			}
			else if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Skipping package %ls (already checked out)", *PackageFilename);
				return true;
			}
			else if (SourceControlState->IsSourceControlled())
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Checking out package %ls from revision control", *PackageFilename);
				return GetSourceControlProvider().Execute(ISourceControlOperation::Create<FCheckOut>(), Package) == ECommandResult::Succeeded;
			}
		}
	}
	else
	{
		FString PackageFilename = SourceControlHelpers::PackageFilename(Package);
		if (IPlatformFile::GetPlatformPhysical().FileExists(*PackageFilename))
		{
			if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*PackageFilename, false))
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error setting %ls writable", *PackageFilename);
				return false;
			}
		}
	}

	return true;
}

bool UConvertLevelsToExternalActorsCommandlet::DeleteFile(const FString& Filename)
{
	if (!UseSourceControl())
	{
		if (!IPlatformFile::GetPlatformPhysical().SetReadOnly(*Filename, false) ||
			!IPlatformFile::GetPlatformPhysical().DeleteFile(*Filename))
		{
			UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error deleting %ls", *Filename);
			return false;
		}
	}
	else 
	{
		FSourceControlStatePtr SourceControlState = GetSourceControlProvider().GetState(Filename, EStateCacheUsage::ForceUpdate);

		if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
		{
			FString OtherCheckedOutUser;
			if (SourceControlState->IsCheckedOutOther(&OtherCheckedOutUser))
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Overwriting package %ls already checked out by %ls, will not submit", *Filename, *OtherCheckedOutUser);
				return false;
			}
			else if (!SourceControlState->IsCurrent())
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Overwriting package %ls (not at head revision), will not submit", *Filename);
				return false;
			}
			else if (SourceControlState->IsAdded())
			{
				if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FRevert>(), Filename) != ECommandResult::Succeeded)
				{
					UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error reverting package %ls from revision control", *Filename);
					return false;
				}
			}
			else
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Deleting package %ls from revision control", *Filename);

				if (SourceControlState->IsCheckedOut())
				{
					if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FRevert>(), Filename) != ECommandResult::Succeeded)
					{
						UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error reverting package %ls from revision control", *Filename);
						return false;
					}
				}

				if (GetSourceControlProvider().Execute(ISourceControlOperation::Create<FDelete>(), Filename) != ECommandResult::Succeeded)
				{
					UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error deleting package %ls from revision control", *Filename);
					return false;
				}
			}
		}
		else
		{
			if (!IFileManager::Get().Delete(*Filename, false, true))
			{
				UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Error deleting package %ls locally", *Filename);
				return false;
			}
		}
	}

	return true;
}

int32 UConvertLevelsToExternalActorsCommandlet::Main(const FString& Params)
{
	FAutoScopedDurationTimer ConversionTimer;

	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	// Need at least the level to convert
	if (Tokens.Num() < 1)
	{
		UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "ConvertLevelToExternalActors bad parameters");
		return 1;
	}

	bool bNoSourceControl = Switches.Contains(TEXT("nosourcecontrol"));
	bool bConvertSubLevel = Switches.Contains(TEXT("convertsublevels"));
	bool bRecursiveSubLevel = Switches.Contains(TEXT("recursive"));
	bool bConvertToExternal = !Switches.Contains(TEXT("internal"));
	bool bRepairActorFiles = Switches.Contains(TEXT("repair"));

	FScopedSourceControl SourceControl;
	SourceControlProvider = bNoSourceControl ? nullptr : &ISourceControlModule::Get().GetProvider();

	// This will convert incomplete package name to a fully qualifed path
	if (!FPackageName::SearchForPackageOnDisk(Tokens[0], &Tokens[0]))
	{
		UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Unknown level '%ls'", *Tokens[0]);
		return 1;
	}

	// Check external actors consistency for this level
	if (!CheckExternalActors(Tokens[0], bRepairActorFiles))
	{
		UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "External actor files inconsistency");
		return 1;
	}

	// Load persistent level
	ULevel* MainLevel = LoadLevel(Tokens[0]);
	if (!MainLevel)
	{
		UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Unable to load level '%ls'", *Tokens[0]);
		return 1;
	}

	UWorld* MainWorld = MainLevel->GetWorld();
	UPackage* MainPackage = MainLevel->GetPackage();

	TSet<ULevel*> LevelsToConvert;
	LevelsToConvert.Add(MainLevel);
	if (bConvertSubLevel)
	{
		GetSubLevelsToConvert(MainLevel, LevelsToConvert, bRecursiveSubLevel);
	}

	bool bNeedsResaveSubLevels = false;
	for(ULevel* Level : LevelsToConvert)
	{
		if (!Level->bContainsStableActorGUIDs)
		{
			bNeedsResaveSubLevels |= true;
			UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Error, "Unable to convert level '%ls' with non-stable actor GUIDs. Resave the level before converting.", *Level->GetPackage()->GetName());
		}
	}

	if (bNeedsResaveSubLevels)
	{
		return 1;
	}
	
	TArray<UPackage*> PackagesToSave;
	for(ULevel* Level : LevelsToConvert)
	{
		Level->ConvertAllActorsToPackaging(bConvertToExternal);
		UPackage* LevelPackage = Level->GetPackage();
		PackagesToSave.Add(LevelPackage);

		for (UPackage* ExternalPackage : Level->GetLoadedExternalObjectPackages())
		{
			if (!UPackage::IsEmptyPackage(ExternalPackage))
			{
				PackagesToSave.Add(ExternalPackage);
			}
		}
	}

	for (UPackage* PackageToSave : PackagesToSave)
	{
		if(!CheckoutPackage(PackageToSave))
		{
			return 1;
		}
	}

	// Save packages
	UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Saving %d packages.", PackagesToSave.Num());
	for (UPackage* PackageToSave : PackagesToSave)
	{
		if (!SavePackage(PackageToSave))
		{
			return 1;
		}
	}

	// Add new packages to source control
	for (UPackage* PackageToSave : PackagesToSave)
	{
		if(!AddPackageToSourceControl(PackageToSave))
		{
			return 1;
		}
	}

	UE_LOGF(LogConvertLevelsToExternalActorsCommandlet, Log, "Conversion took %.2f seconds", ConversionTimer.GetTime());
	return 0;
}
