// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CookCommandlet.cpp: Commandlet for cooking content
=============================================================================*/

#include "Commandlets/CookCommandlet.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/TaskGraphInterfaces.h"
#include "AutoRTFM.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookTypes.h"
#include "CookerSettings.h"
#include "Editor.h"
#include "EngineGlobals.h"
#include "GameDelegates.h"
#include "GlobalShader.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/MemoryBase.h"
#include "HAL/MemoryMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "INetworkFileSystemModule.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/RedirectCollector.h"
#include "Modules/ModuleManager.h"
#include "PackageHelperFunctions.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/ArrayWriter.h"
#include "Settings/ProjectPackagingSettings.h"
#include "ShaderCompiler.h"
#include "Stats/StatsMisc.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookCommandlet)
DEFINE_LOG_CATEGORY_STATIC(LogCookCommandlet, Log, All);

namespace UE::Cook
{

struct FScopeRootObject
{
	UObject* Object;
	FScopeRootObject(UObject* InObject) : Object(InObject)
	{
		Object->AddToRoot();
	}

	~FScopeRootObject()
	{
		Object->RemoveFromRoot();
	}
};

bool bAllowContentValidation = true;
FAutoConsoleVariableRef AllowContentValidationCVar(
	TEXT("Cook.AllowContentValidation"),
	bAllowContentValidation,
	TEXT("True to allow content validation to run during cook (if requested), or false to disable it."));

}

UCookCommandlet::UCookCommandlet( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{

	LogToConsole = false;
}

bool UCookCommandlet::CookOnTheFly( FGuid InstanceId, int32 Port, int32 Timeout, bool bForceClose, const TArray<ITargetPlatform*>& TargetPlatforms)
{
	UCookOnTheFlyServer *CookOnTheFlyServer = NewObject<UCookOnTheFlyServer>();

	// make sure that the cookonthefly server doesn't get cleaned up while we are garbage collecting below :)
	UE::Cook::FScopeRootObject S(CookOnTheFlyServer);

	UCookerSettings const* CookerSettings = GetDefault<UCookerSettings>();
	ECookInitializationFlags LegacyIterativeFlags = ECookInitializationFlags::LegacyIterative;

	ECookInitializationFlags CookFlags = ECookInitializationFlags::None;
	CookFlags |= bLegacyIterativeCooking ? LegacyIterativeFlags : ECookInitializationFlags::None;
	CookFlags |= bSkipEditorContent ? ECookInitializationFlags::SkipEditorContent : ECookInitializationFlags::None;
	CookFlags |= bUnversioned ? ECookInitializationFlags::Unversioned : ECookInitializationFlags::None;
	CookFlags |= bCookEditorOptional ? ECookInitializationFlags::CookEditorOptional : ECookInitializationFlags::None;
	CookFlags |= bIgnoreIniSettingsOutOfDate || CookerSettings->bIgnoreIniSettingsOutOfDateForIteration ? ECookInitializationFlags::IgnoreIniSettingsOutOfDate : ECookInitializationFlags::None;
	CookOnTheFlyServer->Initialize( ECookMode::CookOnTheFly, CookFlags );

	UCookOnTheFlyServer::FCookOnTheFlyStartupOptions CookOnTheFlyStartupOptions;
	CookOnTheFlyStartupOptions.Port = Port;
	CookOnTheFlyStartupOptions.bZenStore = Switches.Contains(TEXT("ZenStore"));
	CookOnTheFlyStartupOptions.bPlatformProtocol = Switches.Contains(TEXT("PlatformProtocol"));
	CookOnTheFlyStartupOptions.TargetPlatforms = TargetPlatforms;

	if (CookOnTheFlyServer->StartCookOnTheFly(MoveTemp(CookOnTheFlyStartupOptions)) == false)
	{
		return false;
	}

	if ( InstanceId.IsValid() )
	{
		if ( CookOnTheFlyServer->BroadcastFileserverPresence(InstanceId) == false )
		{
			return false;
		}
	}

	FDateTime LastConnectionTime = FDateTime::UtcNow();
	bool bHadConnection = false;

	while (!IsEngineExitRequested())
	{
		uint32 TickResults = CookOnTheFlyServer->TickCookOnTheFly(/*TimeSlice =*/MAX_flt,
			ShowProgress ? ECookTickFlags::None : ECookTickFlags::HideProgressDisplay);
		ConditionalCollectGarbage(TickResults, *CookOnTheFlyServer);

		if (!CookOnTheFlyServer->HasRemainingWork() && !IsEngineExitRequested())
		{
			// handle server timeout
			if (InstanceId.IsValid() || bForceClose)
			{
				if (CookOnTheFlyServer->NumConnections() > 0)
				{
					bHadConnection = true;
					LastConnectionTime = FDateTime::UtcNow();
				}

				if ((FDateTime::UtcNow() - LastConnectionTime) > FTimespan::FromSeconds(Timeout))
				{
					uint32 Result = FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "FileServerIdle", "The file server did not receive any connections in the past 3 minutes. Would you like to shut it down?"));

					if (Result == EAppReturnType::No && !bForceClose)
					{
						LastConnectionTime = FDateTime::UtcNow();
					}
					else
					{
						RequestEngineExit(TEXT("Cook file server idle"));
					}
				}
				else if (bHadConnection && (CookOnTheFlyServer->NumConnections() == 0) && bForceClose) // immediately shut down if we previously had a connection and now do not
				{
					RequestEngineExit(TEXT("Cook file server lost last connection"));
				}
			}

			CookOnTheFlyServer->WaitForRequests(100 /* timeoutMs */);
		}
	}

	CookOnTheFlyServer->ShutdownCookOnTheFly();
	return true;
}

/* UCommandlet interface
 *****************************************************************************/

int32 UCookCommandlet::Main(const FString& CmdLineParams)
{
#if WITH_VERSE_VM
	AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::EAutoRTFMEnabledState::AutoRTFM_Enabled);
#else
	AutoRTFM::ForTheRuntime::SetAutoRTFMRuntime(AutoRTFM::ForTheRuntime::EAutoRTFMEnabledState::AutoRTFM_Disabled);
#endif

	COOK_STAT(DetailedCookStats::CookStartTime = FPlatformTime::Seconds());
	Params = CmdLineParams;
	ParseCommandLine(*Params, Tokens, Switches);

	bCookOnTheFly = Switches.Contains(TEXT("COOKONTHEFLY"));   // Prototype cook-on-the-fly server
	bCookAll = Switches.Contains(TEXT("COOKALL"));   // Cook everything
	bUnversioned = Switches.Contains(TEXT("UNVERSIONED"));   // Save all cooked packages without versions. These are then assumed to be current version on load. This is dangerous but results in smaller patch sizes.
	bCookEditorOptional = Switches.Contains(TEXT("EDITOROPTIONAL")); // Produce the optional editor package data alongside the cooked data.
	bGenerateStreamingInstallManifests = Switches.Contains(TEXT("MANIFESTS"));   // Generate manifests for building streaming install packages
	bLegacyIterativeCooking = Switches.Contains(TEXT("ITERATE")) || Switches.Contains(TEXT("ITERATIVE")) || Switches.Contains(TEXT("LEGACYITERATIVE"));
	bSkipEditorContent = Switches.Contains(TEXT("SKIPEDITORCONTENT")); // This won't save out any packages in Engine/Content/Editor*
	bErrorOnEngineContentUse = Switches.Contains(TEXT("ERRORONENGINECONTENTUSE"));
	bCookSinglePackage = Switches.Contains(TEXT("cooksinglepackagenorefs"));
	bKeepSinglePackageRefs = Switches.Contains(TEXT("cooksinglepackage")); // This is a legacy parameter; it's a minor misnomer since singlepackage implies norefs, but we want to avoiding changing the behavior
	bCookSinglePackage = bCookSinglePackage || bKeepSinglePackageRefs;
	bVerboseCookerWarnings = Switches.Contains(TEXT("verbosecookerwarnings"));
	bPartialGC = Switches.Contains(TEXT("Partialgc"));
	ShowErrorCount = !Switches.Contains(TEXT("DIFFONLY")) && !Switches.Contains(TEXT("NoErrorSummary"));
	ShowProgress = !Switches.Contains(TEXT("DIFFONLY"));
	bIgnoreIniSettingsOutOfDate = Switches.Contains(TEXT("IgnoreIniSettingsOutOfDate"));
	bFastCook = Switches.Contains(TEXT("FastCook"));

	COOK_STAT(DetailedCookStats::IsCookAll = bCookAll);
	COOK_STAT(DetailedCookStats::IsCookOnTheFly = bCookOnTheFly);
	COOK_STAT(DetailedCookStats::IsIterativeCook = bLegacyIterativeCooking);
	COOK_STAT(DetailedCookStats::IsFastCook = bFastCook);
	COOK_STAT(DetailedCookStats::IsUnversioned = bUnversioned);

	COOK_STAT(DetailedCookStats::CookProject = FApp::GetProjectName());

	COOK_STAT(FParse::Value(*Params, TEXT("CookCultures="), DetailedCookStats::CookCultures));
	COOK_STAT(FParse::Value(*Params, TEXT("CookLabel="), DetailedCookStats::CookLabel));

	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	if ( bCookOnTheFly )
	{
		// In cook on the fly, if the user did not provide a targetplatform on the commandline, then we do not intialize any platforms up front; we wait for the first connection.
		// TPM.GetActiveTargetPlatforms defaults to the currently running platform (e.g. Windows, with editor) in the no-target case, so we need to only call GetActiveTargetPlatforms
		// if targetplatform was on the commandline
		FString Unused;
		TArray<ITargetPlatform*> TargetPlatforms;
		if (FParse::Value(FCommandLine::Get(), TEXT("TARGETPLATFORM="), Unused))
		{
			TargetPlatforms = TPM.GetActiveTargetPlatforms();
		}

		// parse instance identifier
		FString InstanceIdString;
		bool bForceClose = Switches.Contains(TEXT("FORCECLOSE"));

		int32 Port = UCookOnTheFlyServer::FCookOnTheFlyStartupOptions::DefaultPort;
		if (!FParse::Value(*Params, TEXT("port="), Port))
		{
			Port = UCookOnTheFlyServer::FCookOnTheFlyStartupOptions::DefaultPort;
		}

		FGuid InstanceId;
		if (FParse::Value(*Params, TEXT("InstanceId="), InstanceIdString))
		{
			if (FGuid::Parse(InstanceIdString, InstanceId) && InstanceId.IsValid())
			{
				Port = UCookOnTheFlyServer::FCookOnTheFlyStartupOptions::AnyPort;
			}
			else
			{
				UE_LOGF(LogCookCommandlet, Warning, "Invalid InstanceId on command line: %ls", *InstanceIdString);
			}
		}

		int32 Timeout = 180;
		if (!FParse::Value(*Params, TEXT("timeout="), Timeout))
		{
			Timeout = 180;
		}

		if (Switches.Contains(TEXT("ODSC")))
		{
			// ODSC piggybacks on the cook commandlet but does not cook any packages. Turn -legacyiterative on to prevent
			// an unnecessary clearing of the cook results.
			bLegacyIterativeCooking = true;
		}

		CookOnTheFly( InstanceId, Port, Timeout, bForceClose, TargetPlatforms);
	}
	else if (Switches.Contains(TEXT("COOKWORKER")))
	{
		CookAsCookWorker();
	}
	else
	{
		CookByTheBook(TPM.GetActiveTargetPlatforms());
	}
	return 0;
}

bool UCookCommandlet::CookByTheBook(const TArray<ITargetPlatform*>& Platforms)
{
#if OUTPUT_COOKTIMING
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(CookByTheBook, CookChannel);
#endif // OUTPUT_COOKTIMING

	UCookOnTheFlyServer* CookOnTheFlyServer = NewObject<UCookOnTheFlyServer>();
	// make sure that the cookonthefly server doesn't get cleaned up while we are garbage collecting below :)
	UE::Cook::FScopeRootObject S(CookOnTheFlyServer);

	UCookerSettings const* CookerSettings = GetDefault<UCookerSettings>();
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	ECookInitializationFlags LegacyIterativeFlags = ECookInitializationFlags::LegacyIterative;

	if (Switches.Contains(TEXT("IterateSharedCookedbuild"))
		|| Switches.Contains(TEXT("LegacyIterativeSharedCookedbuild")))
	{
		// Add shared build flag to method flag, and enable legacyiterative
		LegacyIterativeFlags |= ECookInitializationFlags::LegacyIterativeSharedBuild;

		bLegacyIterativeCooking = true;
	}
	if (Switches.Contains(TEXT("ODSC")))
	{
		// ODSC piggybacks on the cook commandlet but does not cook any packages. Turn -legacyiterative on to prevent
		// an unnecessary clearing of the cook results.
		bLegacyIterativeCooking = true;
	}

	ECookInitializationFlags CookFlags = ECookInitializationFlags::IncludeServerMaps;
	CookFlags |= bLegacyIterativeCooking ? LegacyIterativeFlags : ECookInitializationFlags::None;
	CookFlags |= bSkipEditorContent ? ECookInitializationFlags::SkipEditorContent : ECookInitializationFlags::None;
	CookFlags |= bUnversioned ? ECookInitializationFlags::Unversioned : ECookInitializationFlags::None;
	CookFlags |= bCookEditorOptional ? ECookInitializationFlags::CookEditorOptional : ECookInitializationFlags::None;
	CookFlags |= bVerboseCookerWarnings ? ECookInitializationFlags::OutputVerboseCookerWarnings : ECookInitializationFlags::None;
	CookFlags |= bPartialGC ? ECookInitializationFlags::EnablePartialGC : ECookInitializationFlags::None;
	CookFlags |= Switches.Contains(TEXT("TestCook")) ? ECookInitializationFlags::TestCook : ECookInitializationFlags::None;
	CookFlags |= Switches.Contains(TEXT("LogDebugInfo")) ? ECookInitializationFlags::LogDebugInfo : ECookInitializationFlags::None;
	CookFlags |= bIgnoreIniSettingsOutOfDate || CookerSettings->bIgnoreIniSettingsOutOfDateForIteration ? ECookInitializationFlags::IgnoreIniSettingsOutOfDate : ECookInitializationFlags::None;
	CookFlags |= Switches.Contains(TEXT("IgnoreScriptPackagesOutOfDate")) || CookerSettings->bIgnoreScriptPackagesOutOfDateForIteration ? ECookInitializationFlags::IgnoreScriptPackagesOutOfDate : ECookInitializationFlags::None;

	//////////////////////////////////////////////////////////////////////////
	// parse commandline options

	FString DLCName;
	FParse::Value(*Params, TEXT("DLCNAME="), DLCName);

	FString BasedOnReleaseVersion;
	FParse::Value(*Params, TEXT("BasedOnReleaseVersion="), BasedOnReleaseVersion);

	FString CreateReleaseVersion;
	FParse::Value(*Params, TEXT("CreateReleaseVersion="), CreateReleaseVersion);

	FString OutputDirectoryOverride;
	FParse::Value(*Params, TEXT("OutputDir="), OutputDirectoryOverride);

	TArray<FString> CmdLineMapEntries;
	TArray<FString> CmdLineDirEntries;
	TArray<FString> CmdLineCultEntries;
	TArray<FString> CmdLineNeverCookDirEntries;
	for (int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); SwitchIdx++)
	{
		const FString& Switch = Switches[SwitchIdx];

		auto GetSwitchValueElements = [&Switch](const FString SwitchKey) -> TArray<FString>
		{
			TArray<FString> ValueElements;
			if (Switch.StartsWith(SwitchKey + TEXT("=")) == true)
			{
				FString ValuesList = Switch.Right(Switch.Len() - (SwitchKey + TEXT("=")).Len());

				// Allow support for -KEY=Value1+Value2+Value3 as well as -KEY=Value1 -KEY=Value2
				for (int32 PlusIdx = ValuesList.Find(TEXT("+"), ESearchCase::CaseSensitive); PlusIdx != INDEX_NONE; PlusIdx = ValuesList.Find(TEXT("+"), ESearchCase::CaseSensitive))
				{
					const FString ValueElement = ValuesList.Left(PlusIdx);
					ValueElements.Add(ValueElement);

					ValuesList.RightInline(ValuesList.Len() - (PlusIdx + 1), EAllowShrinking::No);
				}
				ValueElements.Add(ValuesList);
			}
			return ValueElements;
		};

		// Check for -MAP=<name of map> entries
		CmdLineMapEntries += GetSwitchValueElements(TEXT("MAP"));
		CmdLineMapEntries += GetSwitchValueElements(TEXT("PACKAGE"));

		// Check for -COOKDIR=<path to directory> entries
		const FString CookDirPrefix = TEXT("COOKDIR=");
		if (Switch.StartsWith(CookDirPrefix))
		{
			FString Entry = Switch.Mid(CookDirPrefix.Len()).TrimQuotes();
			FPaths::NormalizeDirectoryName(Entry);
			CmdLineDirEntries.Add(Entry);
		}

		// Check for -NEVERCOOKDIR=<path to directory> entries
		for (FString& NeverCookDir : GetSwitchValueElements(TEXT("NEVERCOOKDIR")))
		{
			FPaths::NormalizeDirectoryName(NeverCookDir);
			CmdLineNeverCookDirEntries.Add(MoveTemp(NeverCookDir));
		}

		// Check for -COOKCULTURES=<culture name> entries
		CmdLineCultEntries += GetSwitchValueElements(TEXT("COOKCULTURES"));
	}

	CookOnTheFlyServer->Initialize(ECookMode::CookByTheBook, CookFlags, OutputDirectoryOverride);

	TArray<FString> MapIniSections;
	FString SectionStr;
	if (FParse::Value(*Params, TEXT("MAPINISECTION="), SectionStr))
	{
		if (SectionStr.Contains(TEXT("+")))
		{
			TArray<FString> Sections;
			SectionStr.ParseIntoArray(Sections, TEXT("+"), true);
			for (int32 Index = 0; Index < Sections.Num(); Index++)
			{
				MapIniSections.Add(Sections[Index]);
			}
		}
		else
		{
			MapIniSections.Add(SectionStr);
		}
	}

	// Set the list of cultures to cook as those on the commandline, if specified.
	// Otherwise, use the project packaging settings.
	TArray<FString> CookCultures;
	if (Switches.ContainsByPredicate([](const FString& Switch) -> bool
		{
			return Switch.StartsWith("COOKCULTURES=");
		}))
	{
		CookCultures = CmdLineCultEntries;
	}
	else
	{
		CookCultures = PackagingSettings->CulturesToStage;
	}

	const bool bUseZenStore =
		!Switches.Contains(TEXT("SkipZenStore")) &&
		(Switches.Contains(TEXT("ZenStore")) || PackagingSettings->GetUseZenStoreEffective());

	//////////////////////////////////////////////////////////////////////////
	// start cook by the book
	ECookByTheBookOptions CookOptions = ECookByTheBookOptions::None;

	CookOptions |= bCookAll ? ECookByTheBookOptions::CookAll : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("MAPSONLY")) ? ECookByTheBookOptions::MapsOnly : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("NODEV")) ? ECookByTheBookOptions::NoDevContent : ECookByTheBookOptions::None;
	if (Switches.Contains(TEXT("FullLoadAndSave"))) // Deprecated in UE 5.3
	{
		UE_LOGF(LogCook, Warning, "-FullLoadAndSave has been deprecated; remove the argument to remove this warning.\n"
			"For cook optimizations, try using multiprocess cook (-cookprocesscount=<N>, N>1).\n"
			"If you still need further optimizations, contact Epic on UDN.");
	}
	CookOptions |= bUseZenStore ? ECookByTheBookOptions::ZenStore : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("NoGameAlwaysCook")) ? ECookByTheBookOptions::NoGameAlwaysCookPackages : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("DisableUnsolicitedPackages")) ? (ECookByTheBookOptions::SkipHardReferences | ECookByTheBookOptions::SkipSoftReferences) : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("NoDefaultMaps")) ? ECookByTheBookOptions::NoDefaultMaps : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("SkipSoftReferences")) ? ECookByTheBookOptions::SkipSoftReferences : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("SkipHardReferences")) ? ECookByTheBookOptions::SkipHardReferences : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("CookAgainstFixedBase")) ? ECookByTheBookOptions::CookAgainstFixedBase : ECookByTheBookOptions::None;
	CookOptions |= (Switches.Contains(TEXT("DlcLoadMainAssetRegistry")) || !bErrorOnEngineContentUse) ? ECookByTheBookOptions::DlcLoadMainAssetRegistry : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("DlcReevaluateUncookedAssets")) ? ECookByTheBookOptions::DlcReevaluateUncookedAssets : ECookByTheBookOptions::None;
	CookOptions |= Switches.Contains(TEXT("ForceDisableSaveGlobalShaders")) ? ECookByTheBookOptions::ForceDisableSaveGlobalShaders : ECookByTheBookOptions::None;
	bool bCookList = Switches.Contains(TEXT("CookList"));

	if (UE::Cook::bAllowContentValidation)
	{
		CookOptions |= Switches.Contains(TEXT("RunAssetValidation")) ? ECookByTheBookOptions::RunAssetValidation : ECookByTheBookOptions::None;
		CookOptions |= Switches.Contains(TEXT("RunMapValidation")) ? ECookByTheBookOptions::RunMapValidation : ECookByTheBookOptions::None;
		CookOptions |= Switches.Contains(TEXT("ValidationErrorsAreFatal")) ? ECookByTheBookOptions::ValidationErrorsAreFatal : ECookByTheBookOptions::None;
	}

	const ECookByTheBookOptions SkipRequestFlags = ECookByTheBookOptions::NoAlwaysCookMaps |
		ECookByTheBookOptions::NoDefaultMaps | ECookByTheBookOptions::NoGameAlwaysCookPackages |
		ECookByTheBookOptions::NoInputPackages | ECookByTheBookOptions::ForceDisableSaveGlobalShaders;
	if (bCookSinglePackage)
	{
		CookOptions |= SkipRequestFlags;
		CookOptions |= ECookByTheBookOptions::SkipSoftReferences;
		CookOptions |= bKeepSinglePackageRefs ? ECookByTheBookOptions::None : ECookByTheBookOptions::SkipHardReferences;
	}
	if (Switches.Contains(TEXT("CookSkipRequests")))
	{
		CookOptions |= SkipRequestFlags;
		CookOptions |= ECookByTheBookOptions::NoStartupPackages;
	}
	if (Switches.Contains(TEXT("CookSkipSoftRefs")))
	{
		CookOptions |= ECookByTheBookOptions::SkipSoftReferences;
	}
	if (Switches.Contains(TEXT("CookSkipHardRefs")))
	{
		CookOptions |= ECookByTheBookOptions::SkipHardReferences;
	}

	// Also append any cookdirs from the project ini files; these dirs are relative to the game content directory or start with a / root
	if (!(CookOptions & ECookByTheBookOptions::NoGameAlwaysCookPackages))
	{
		for (const FDirectoryPath& DirToCook : PackagingSettings->DirectoriesToAlwaysCook)
		{
			FString LocalPath;
			if (FPackageName::TryConvertGameRelativePackagePathToLocalPath(DirToCook.Path, LocalPath))
			{
				CmdLineDirEntries.Add(LocalPath);
			}
			else
			{
				UE_LOGF(LogCook, Warning, "'ProjectSettings -> PackagingSettings -> Directories to always cook' has invalid element '%ls'", *DirToCook.Path);
			}
		}
	}

	UCookOnTheFlyServer::FCookByTheBookStartupOptions StartupOptions;

	// Validate target platforms and add them to StartupOptions
	for (ITargetPlatform* TargetPlatform : Platforms)
	{
		if (TargetPlatform)
		{
			if (TargetPlatform->HasEditorOnlyData())
			{
				UE_LOGF(LogCook, Warning, "Target platform \"%ls\" is an editor platform and can not be a cook target", *TargetPlatform->PlatformName());
			}
			else
			{
				StartupOptions.TargetPlatforms.Add(TargetPlatform);
			}
		}
	}
	if (!StartupOptions.TargetPlatforms.Num())
	{
		UE_LOGF(LogCook, Error, "No target platforms specified or all target platforms are invalid");
		return false;
	}

	Swap(StartupOptions.CookMaps, CmdLineMapEntries);
	Swap(StartupOptions.CookDirectories, CmdLineDirEntries);
	Swap(StartupOptions.NeverCookDirectories, CmdLineNeverCookDirEntries);
	Swap(StartupOptions.CookCultures, CookCultures);
	Swap(StartupOptions.DLCName, DLCName);
	Swap(StartupOptions.BasedOnReleaseVersion, BasedOnReleaseVersion);
	Swap(StartupOptions.CreateReleaseVersion, CreateReleaseVersion);
	Swap(StartupOptions.IniMapSections, MapIniSections);
	StartupOptions.CookOptions = CookOptions;
	StartupOptions.bErrorOnEngineContentUse = bErrorOnEngineContentUse;
	StartupOptions.bGenerateDependenciesForMaps = Switches.Contains(TEXT("GenerateDependenciesForMaps"));
	StartupOptions.bGenerateStreamingInstallManifests = bGenerateStreamingInstallManifests;

	COOK_STAT(
		{
			for (const auto& Platform : Platforms)
			{
				DetailedCookStats::TargetPlatforms += Platform->PlatformName() + TEXT("+");
			}
			if (!DetailedCookStats::TargetPlatforms.IsEmpty())
			{
				DetailedCookStats::TargetPlatforms.RemoveFromEnd(TEXT("+"));
			}
		});

	// Cast to void as a workaround to support inability to foward-declare inner classes.
	// TODO: Change FCookByTheBookStartupOptions to a global class.
	void* StartupOptionsAsVoid = &StartupOptions;

	if (bCookList)
	{
		RunCookByTheBookList(CookOnTheFlyServer, StartupOptionsAsVoid, CookOptions);
	}
	else
	{
		RunCookByTheBookCook(CookOnTheFlyServer, StartupOptionsAsVoid, CookOptions);
	}
	return true;
}

void UCookCommandlet::RunCookByTheBookList(UCookOnTheFlyServer* CookOnTheFlyServer, void* StartupOptionsAsVoid,
	ECookByTheBookOptions CookOptions)
{
	UCookOnTheFlyServer::FCookByTheBookStartupOptions& StartupOptions =
		*reinterpret_cast<UCookOnTheFlyServer::FCookByTheBookStartupOptions*>(StartupOptionsAsVoid);

	ECookListOptions CookListOptions = ECookListOptions::None;
	if (Switches.Contains(TEXT("showrejected"))) CookListOptions |= ECookListOptions::ShowRejected;
	StartupOptions.bCookList = true;

	CookOnTheFlyServer->StartCookByTheBook(StartupOptions);
	CookOnTheFlyServer->RunCookList(CookListOptions);
}

void UCookCommandlet::RunCookByTheBookCook(UCookOnTheFlyServer* CookOnTheFlyServer, void* StartupOptionsAsVoid,
	ECookByTheBookOptions CookOptions)
{
	UCookOnTheFlyServer::FCookByTheBookStartupOptions& StartupOptions =
		*reinterpret_cast<UCookOnTheFlyServer::FCookByTheBookStartupOptions*>(StartupOptionsAsVoid);
	bool bTestCook = EnumHasAnyFlags(CookOnTheFlyServer->GetCookFlags(), ECookInitializationFlags::TestCook);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FDelegateHandle FlushUpdateHandle = FCoreDelegates::OnAsyncLoadingFlushUpdate.AddLambda([]()
		{
			FLowLevelMemTracker::Get().UpdateStatsPerFrame();
		});
	FDelegateHandle FlushHandle = FCoreDelegates::OnAsyncLoadingFlush.AddLambda([]()
		{
			FLowLevelMemTracker::Get().UpdateStatsPerFrame();
		});
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif
	bool bShouldVerifyEDLCookInfo = false;
	do
	{
		{
			COOK_STAT(FScopedDurationTimer StartCookByTheBookTimer(DetailedCookStats::StartCookByTheBookTimeSec));
			CookOnTheFlyServer->StartCookByTheBook(StartupOptions);
			bShouldVerifyEDLCookInfo = CookOnTheFlyServer->ShouldVerifyEDLCookInfo();
		}
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("CookByTheBook.MainLoop"), STAT_CookByTheBook_MainLoop, STATGROUP_LoadTime);
		while (CookOnTheFlyServer->IsInSession())
		{
			uint32 TickResults = CookOnTheFlyServer->TickCookByTheBook(MAX_flt,
				ShowProgress ? ECookTickFlags::None : ECookTickFlags::HideProgressDisplay);
			ConditionalCollectGarbage(TickResults, *CookOnTheFlyServer);
		}
	} while (bTestCook);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif

	if (bShouldVerifyEDLCookInfo)
	{
		bool bFullReferencesExpected = !(CookOptions & ECookByTheBookOptions::SkipHardReferences);
		auto WriteLogRecord = [](UE::FLogRecord&& Record)
			{
#if !NO_LOGGING
				Record.SetCategory(LogCook.GetCategoryName());
				UE::DispatchDynamicLogRecord(Record);
#endif
			};
		auto GetInstigatorChain = [CookOnTheFlyServer](FName PackageName)
			{
				return UE::Cook::InstigatorChainToString(
					CookOnTheFlyServer->GetInstigatorChain(PackageName), PackageName.ToString());
			};
		FEDLCookChecker::Verify(WriteLogRecord, GetInstigatorChain, bFullReferencesExpected);
	}

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FCoreDelegates::OnAsyncLoadingFlushUpdate.Remove(FlushUpdateHandle);
	FCoreDelegates::OnAsyncLoadingFlush.Remove(FlushHandle);
#endif
}

bool UCookCommandlet::CookAsCookWorker()
{
#if OUTPUT_COOKTIMING
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(CookAsCookWorker, CookChannel);
#endif // OUTPUT_COOKTIMING

	UCookOnTheFlyServer* CookOnTheFlyServer = NewObject<UCookOnTheFlyServer>();
	// make sure that the cookonthefly server doesn't get cleaned up while we are garbage collecting below
	UE::Cook::FScopeRootObject S(CookOnTheFlyServer);

	if (!CookOnTheFlyServer->TryInitializeCookWorker())
	{
		UE_LOGF(LogCook, Display, "CookWorker initialization failed, aborting CookCommandlet.");
		return false;
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("CookByTheBook.MainLoop"), STAT_CookByTheBook_MainLoop, STATGROUP_LoadTime);
		while (CookOnTheFlyServer->IsInSession())
		{
			uint32 TickResults = CookOnTheFlyServer->TickCookWorker();
			ConditionalCollectGarbage(TickResults, *CookOnTheFlyServer);
		}
	}
	CookOnTheFlyServer->ShutdownCookAsCookWorker();

	return true;
}

void UCookCommandlet::ConditionalCollectGarbage(uint32 TickResults, UCookOnTheFlyServer& COTFS)
{
	if ((TickResults & UCookOnTheFlyServer::COSR_RequiresGC) == 0)
	{
		return;
	}

	double GCStartTime = FPlatformTime::Seconds();
	bool bSoft = (TickResults & UCookOnTheFlyServer::COSR_RequiresGC_Soft) != 0;
	bool bPeriodic = (TickResults & UCookOnTheFlyServer::COSR_RequiresGC_Periodic) != 0;
	FString GCReason;
	FString GCType = bPartialGC && !bSoft ? TEXT(" partial gc") : TEXT("");

	if ((TickResults & UCookOnTheFlyServer::COSR_RequiresGC_PackageCount) != 0)
	{
		GCReason = TEXT("Exceeded packages per GC");
	}
	else if ((TickResults & UCookOnTheFlyServer::COSR_RequiresGC_OOM) != 0)
	{
		// this can cause thrashing if the cooker loads the same stuff into memory next tick
		GCReason = TEXT("Exceeded Max Memory");

		if ((TickResults & UCookOnTheFlyServer::COSR_RequiresGC_Soft) != 0)
		{
			int32 JobsToLogAt = GShaderCompilingManager->GetNumRemainingJobs();
			double NextFlushMsgSeconds = FPlatformTime::Seconds();
			UE_SCOPED_COOKTIMER_AND_DURATION(CookByTheBook_ShaderJobFlush, DetailedCookStats::ShaderFlushTimeSec);
			UE_LOGF(LogCookCommandlet, Display, "Detected max mem exceeded - forcing shader compilation flush");
			while (true)
			{
				int32 NumRemainingJobs = GShaderCompilingManager->GetNumRemainingJobs();
				if (NumRemainingJobs < 1000)
				{
					UE_LOGF(LogCookCommandlet, Display, "Finished flushing shader jobs at %d", NumRemainingJobs);
					break;
				}

				if (NumRemainingJobs < JobsToLogAt)
				{
					double Now = FPlatformTime::Seconds();
					if (NextFlushMsgSeconds <= Now)
					{
						UE_LOGF(LogCookCommandlet, Display, "Flushing shader jobs, remaining jobs %d", NumRemainingJobs);
						NextFlushMsgSeconds = Now + 10;
					}
				}

				GShaderCompilingManager->ProcessAsyncResults(0.f, false);

				FPlatformProcess::Sleep(0.05f);

				// GShaderCompilingManager->FinishAllCompilation();
			}
		}
	}
	else if (bPeriodic)
	{
		GCReason = TEXT("Periodic GC");
	}
	else
	{
		// cooker loaded some object which needs to be cleaned up before the cooker can proceed so force gc
		GCReason = TEXT("COSR_RequiresGC");
	}

	// Flush the asset registry before GC
	{
		UE_SCOPED_COOKTIMER(CookByTheBook_TickAssetRegistry);
		FAssetRegistryModule::TickAssetRegistry(-1.0f);
	}
	UE_SCOPED_COOKTIMER_AND_DURATION(CookCommandlet_GC, DetailedCookStats::TickLoopGCTimeSec);

	const FPlatformMemoryStats MemStatsBeforeGC = FPlatformMemory::GetStats();
	int32 NumObjectsBeforeGC = GUObjectArray.GetObjectArrayNumMinusAvailable();
	int32 NumPackagesBeforeGC = COTFS.NumLoadedPackages();
	FGenericMemoryStats AllocatorStatsBeforeGC;
	UE::Private::GMalloc->GetAllocatorStats(AllocatorStatsBeforeGC);
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif
	COTFS.SetGarbageCollectType(TickResults);

	UE_LOGF(LogCookCommandlet, Display, "GarbageCollection...%ls (%ls)", *GCType, *GCReason);
	TGuardValue<bool> SoftGCGuard(UPackage::bSupportCookerSoftGC, true);
	{
#if !NO_LOGGING
		ELogVerbosity::Type SavedVerbosity = LogGarbage.GetVerbosity();
		bool bSuppressGCLogs = bSoft && bPeriodic && SavedVerbosity > ELogVerbosity::Warning;
		if (bSuppressGCLogs)
		{
			LogGarbage.SetVerbosity(ELogVerbosity::Warning);
		}
#endif
		COTFS.OnCookerStartCollectGarbage(TickResults);
		CollectGarbage(RF_NoFlags);
		COTFS.OnCookerEndCollectGarbage(TickResults);
#if !NO_LOGGING
		if (bSuppressGCLogs)
		{
			LogGarbage.SetVerbosity(SavedVerbosity);
		}
#endif
		if (COTFS.NeedsDiagnosticSecondGC())
		{
			UE_LOGF(LogCookCommandlet, Display, "Second GarbageCollect requested by cooker...");
			COTFS.OnCookerStartCollectGarbage(TickResults);
			CollectGarbage(RF_NoFlags);
			COTFS.OnCookerEndCollectGarbage(TickResults);
		}
	}

	FPlatformMemoryStats MemStatsAfterGC = FPlatformMemory::GetStats();
	int32 NumObjectsAfterGC = GUObjectArray.GetObjectArrayNumMinusAvailable();
	int32 NumPackagesAfterGC = COTFS.NumLoadedPackages();
	FGenericMemoryStats AllocatorStatsAfterGC;
	UE::Private::GMalloc->GetAllocatorStats(AllocatorStatsAfterGC);
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif

	float GCDurationSeconds = FPlatformTime::Seconds() - GCStartTime;
	bool bWasDueToOOM = (TickResults & UCookOnTheFlyServer::COSR_RequiresGC_OOM) != 0;
	COTFS.EvaluateGarbageCollectionResults(bWasDueToOOM, bPartialGC, TickResults,
		NumObjectsBeforeGC, MemStatsBeforeGC, NumPackagesBeforeGC, AllocatorStatsBeforeGC,
		NumObjectsAfterGC, MemStatsAfterGC, NumPackagesAfterGC, AllocatorStatsAfterGC, GCDurationSeconds);

	COTFS.ClearGarbageCollectType();

	// Only check stalling in this is the director and if garbage collection was triggered by OOM.
	bool bRunStallDetector = !COTFS.IsCookWorkerMode() && bWasDueToOOM;
	if (bRunStallDetector)
	{
		bool bCookStalled = COTFS.IsStalled();
		if (bCookStalled)
		{
			UE_LOGF(LogCook, Fatal, "Cook stalled probably because of memory settings being too restrictive and triggering garbage collection.");
		}
	}
}
