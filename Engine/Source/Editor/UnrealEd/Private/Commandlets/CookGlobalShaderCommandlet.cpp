// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CookGlobalShadersCommandlet.h"
#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Class.h"
#include "ShaderCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "DerivedDataCacheInterface.h"
#include "DirectoryWatcherModule.h"
#include "IDirectoryWatcher.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogCookGlobalShaders, Log, All);

bool UCookGlobalShadersDeviceHelperStaged::CopyFilesToDevice(class ITargetDevice* Device, const TArray<TPair<FString, FString>>& FilesToCopy) const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	bool bSuccess = true;
	for (const TPair<FString, FString>& FileToCopy : FilesToCopy)
	{
		FString LocalFile = FileToCopy.Key;
		FString RemoteFile = FPaths::Combine(StagedBuildPath, FileToCopy.Value);
		bSuccess &= PlatformFile.CopyFile(*RemoteFile, *LocalFile);

		UE_LOGF(LogCookGlobalShaders, Warning, "%ls -> %ls", *LocalFile, *RemoteFile);
	}
	return bSuccess;
}

void UCookGlobalShadersCommandlet::CookGlobalShaders() const
{
	// Cook shaders
	UE_LOGF(LogCookGlobalShaders, Log, "Cooking Global Shaders...");
	FString OutputDir = FPaths::ProjectSavedDir() / TEXT("CookGlobalShaders") / PlatformName;
	TArray<uint8> OutGlobalShaderMap;
	FShaderRecompileData Arguments(PlatformName, SP_NumPlatforms, ODSCRecompileCommand::Global, nullptr, nullptr, &OutGlobalShaderMap);
	RecompileShadersForRemote(Arguments, OutputDir);

	// Build list of files to copy
	TArray<TPair<FString, FString>> FilesToCopy;
	for (FName ShaderFormat : ShaderFormats)
	{
		const FString GlobalShaderCacheName = FPaths::Combine(OutputDir, TEXT("Engine"), TEXT("GlobalShaderCache-") + FDataDrivenShaderPlatformInfo::GetName(ShaderFormatToLegacyShaderPlatform(ShaderFormat)).ToString() + TEXT(".bin"));
		const FString OverrideGlobalShaderCacheName = FPaths::Combine(TEXT("Engine"), TEXT("OverrideGlobalShaderCache-") + FDataDrivenShaderPlatformInfo::GetName(ShaderFormatToLegacyShaderPlatform(ShaderFormat)).ToString() + TEXT(".bin"));
		FilesToCopy.Emplace(GlobalShaderCacheName, OverrideGlobalShaderCacheName);
	}

	// Are we copying the built files somewhere?
	bool bCopySucceeded = false;
	if (DeviceHelper != nullptr)
	{
		// Execute Copy
		UE_LOGF(LogCookGlobalShaders, Display, "Copying Cooked Files...");
		bCopySucceeded = DeviceHelper->CopyFilesToDevice(TargetDevice.Get(), FilesToCopy);
	}
	// if no helper, but we want to deploy, use the TargetPlatform
	else if (bDeployToDevice && TargetDevice != nullptr)
	{
		bCopySucceeded = true;

		TMap<FString,FString> CustomPlatformData;
		CustomPlatformData.Add(TEXT("DeployFolder"), DeployFolder);

		for (auto It : FilesToCopy)
		{
			bool bThisCopySucceeded = TargetPlatform->CopyFileToTarget(TargetDevice->GetId().GetDeviceName(), It.Key, It.Value, CustomPlatformData);

			if (bThisCopySucceeded)
			{
				UE_LOGF(LogCookGlobalShaders, Display, "Deployement of %ls to devkit %ls succeeded", *It.Key, *TargetDevice->GetName());
			}
			else
			{
				UE_LOGF(LogCookGlobalShaders, Error, "Deployement of %ls to devkit %ls failed", *It.Key, *TargetDevice->GetName());
				bCopySucceeded = false;
			}
		}
	}

	// Execute Reload
	if (bExecuteReload && bCopySucceeded && TargetDevice.IsValid())
	{
		UE_LOGF(LogCookGlobalShaders, Display, "Sending ReloadGlobalShaders command to devkit %ls", *TargetDevice->GetName());
		TargetDevice->ReloadGlobalShadersMap(OutputDir / TEXT("Engine"));
	}
	
	// Wait for any DDC writes to complete
	GetDerivedDataCacheRef().WaitForQuiescence(true);
}

void UCookGlobalShadersCommandlet::HandleDirectoryChanged(const TArray<FFileChangeData>& InFileChangeDatas)
{
	bool bHasAnyShaderFileChanges = false;

	for (const FFileChangeData& It : InFileChangeDatas)
	{
		if (It.Filename.EndsWith(TEXT(".usf")) || It.Filename.EndsWith(TEXT(".ush")) || It.Filename.EndsWith(TEXT(".h")))
		{
			bHasAnyShaderFileChanges = true;
			UE_LOGF(LogCookGlobalShaders, Display, "Detected change on %ls", *It.Filename);
		}
	}

	if (bHasAnyShaderFileChanges)
	{
		CookGlobalShaders();

		UE_LOGF(LogCookGlobalShaders, Display, "Ready for new shader file changes");
	}
}

void UCookGlobalShadersCommandlet::CookGlobalShadersOnDirectoriesChanges()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	
	if (!DirectoryWatcher)
	{
		return;
	}
	
	// Handle if we are watching a directory for changes.
	TMap<FString, FDelegateHandle> DirectoryWatcherHandles;
	{
		UE_LOGF(LogCookGlobalShaders, Display, "Register directory watchers");

		const TMap<FString, FString>& ShaderSourceDirectoryMappings = AllShaderSourceDirectoryMappings();

		DirectoryWatcherHandles.Reserve(ShaderSourceDirectoryMappings.Num());

		for (const auto& It : ShaderSourceDirectoryMappings)
		{
			FString DirectoryToWatch = It.Value;
			if (FPaths::IsRelative(DirectoryToWatch))
			{
				DirectoryToWatch = FPaths::ConvertRelativePathToFull(DirectoryToWatch);
			}
			
			DirectoryWatcherHandles.Add(DirectoryToWatch, FDelegateHandle());
			FDelegateHandle& DirectoryWatcherHandle = DirectoryWatcherHandles[DirectoryToWatch];

			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
				DirectoryToWatch,
				IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UCookGlobalShadersCommandlet::HandleDirectoryChanged),
				DirectoryWatcherHandle);

			if (DirectoryWatcherHandle.IsValid())
			{
				UE_LOGF(LogCookGlobalShaders, Display, "Watching %ls -> %ls", *It.Key, *DirectoryToWatch);
			}
			else
			{
				UE_LOGF(LogCookGlobalShaders, Error, "Failed to set up directory watcher %ls -> %ls", *It.Key, *DirectoryToWatch);
			}
		}
	}

	GIsRunning = true;

	//@todo abstract properly or delete
#if PLATFORM_WINDOWS// Windows only
	// Used by the .com wrapper to notify that the Ctrl-C handler was triggered.
	// This shared event is checked each tick so that the log file can be cleanly flushed.
	FEvent* ComWrapperShutdownEvent = FPlatformProcess::GetSynchEventFromPool(true);
#endif

	UE_LOGF(LogCookGlobalShaders, Display, "Ready for new shader file changes");

	while (GIsRunning && !IsEngineExitRequested())
	{
		GEngine->UpdateTimeAndHandleMaxTickRate();
		GEngine->Tick(static_cast<float>(FApp::GetDeltaTime()), false);
		
		// tick the directory watcher
		DirectoryWatcherModule.Get()->Tick(static_cast<float>(FApp::GetDeltaTime()));
		
		// flush log
		GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);

#if PLATFORM_WINDOWS
		if (ComWrapperShutdownEvent->Wait(0))
		{
			RequestEngineExit(TEXT("CookGlobalShadersCommandlet ComWrapperShutdownEvent"));
		}
#endif
	}

	//@todo abstract properly or delete 
#if PLATFORM_WINDOWS
	FPlatformProcess::ReturnSynchEventToPool(ComWrapperShutdownEvent);
	ComWrapperShutdownEvent = nullptr;
#endif

	GIsRunning = false;
	
	// Unregisters all directories.
	{
		UE_LOGF(LogCookGlobalShaders, Display, "Unregister directory watchers");

		for (const auto& It : DirectoryWatcherHandles)
		{
			if (It.Value.IsValid() && DirectoryWatcher)
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(It.Key, It.Value);
			}
		}
	}
}

int32 UCookGlobalShadersCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogCookGlobalShaders, Display, "CookGlobalShaders");
		UE_LOGF(LogCookGlobalShaders, Display, "This commandlet will allow you to generate the global shaders file which can be used to override what is used in a cooked build by deploying the loose file.");
		UE_LOGF(LogCookGlobalShaders, Display, "Options:");
		UE_LOGF(LogCookGlobalShaders, Display, " Required: -platform=<platform>             (Which platform you want to cook for, i.e. windows)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -device=<name>                   (Set which device to use, when enabled the reload command will be sent to the device once the shaders are cooked)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -deploy=<optional deploy folder> (Must be used with -device and will deploy the shader file onto the device rather than in the staged builds folder)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -stage=<optional path>           (Moved the shader file into the staged builds folder, destination can be overriden)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -reload                          (Execute a shader reload on the device, only works if the device is valid or a default one was found)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -live                            (Keep commandlet open and automatically recompile shader map when shader file changes happen)");
		UE_LOGF(LogCookGlobalShaders, Display, " Optional: -shaderpdb=<path>                (Sets the shader pdb root)");
		return 0;
	}

	bDeployToDevice = Switches.Contains(TEXT("deploy")) || ParamVals.Contains(TEXT("deploy"));
	bCopyToStaged = Switches.Contains(TEXT("stage"));
	bExecuteReload = Switches.Contains(TEXT("reload"));

	if (ParamVals.Contains(TEXT("deploy")))
	{
		DeployFolder = ParamVals[TEXT("deploy")];
	}

	// Parse platform
	{
		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();

		if (!ParamVals.Contains(TEXT("platform")))
		{
			UE_LOGF(LogCookGlobalShaders, Warning, "You must include a target platform with -platform=xxx");
			for (ITargetPlatform* TP : TPM.GetTargetPlatforms())
			{
				UE_LOGF(LogCookGlobalShaders, Display, "   %ls", *TP->PlatformName());
			}
			return 1;
		}
		PlatformName = ParamVals.FindRef(TEXT("platform"));

		TargetPlatform = TPM.FindTargetPlatform(PlatformName);
		if (TargetPlatform == nullptr)
		{
			UE_LOGF(LogCookGlobalShaders, Warning, "Target platform '%ls' was not found", *PlatformName);
			for (ITargetPlatform* TP : TPM.GetTargetPlatforms())
			{
				UE_LOGF(LogCookGlobalShaders, Display, "   %ls", *TP->PlatformName());
			}
			return 1;
		}
	}

	// Get target device
	FString TargetDeviceName;
	if ( FParse::Value(*Params, TEXT("device="), TargetDeviceName, true) )
	{
		TArray<ITargetDevicePtr> TargetDevices;
		TargetPlatform->GetAllDevices(TargetDevices);

		for ( int i=0; i < TargetDevices.Num(); ++i )
		{
			if ( TargetDevices[i]->GetName().Equals(TargetDeviceName, ESearchCase::IgnoreCase) )
			{
				TargetDevice = TargetDevices[i];
				break;
			}
		}

		if ( !TargetDevice.IsValid() )
		{
			UE_LOGF(LogCookGlobalShaders, Warning, "Failed to find target device '%ls', reload / deploy will not be valid", *TargetDeviceName);

			for (int i = 0; i < TargetDevices.Num(); ++i)
			{
				UE_LOGF(LogCookGlobalShaders, Warning, "	%ls", *TargetDevices[i]->GetName());
			}
		}
	}
	else
	{
		TargetDevice = TargetPlatform->GetDefaultDevice();
	}

	if (!TargetDevice.IsValid() && (bDeployToDevice || bExecuteReload) )
	{
		UE_LOGF(LogCookGlobalShaders, Warning, "No device found to use for reload / deploy");
	}

	// Find DeviceHelper class to use
	if (TargetDevice.IsValid() && bDeployToDevice)
	{
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			if (ClassIt->IsChildOf(UCookGlobalShadersDeviceHelperBase::StaticClass()))
			{
				FString ClassName = ClassIt->GetName();
				ClassName.RemoveAt(0, UE_ARRAY_COUNT(TEXT("CookGlobalShadersDeviceHelperBase")), EAllowShrinking::No);
				if (ClassName.Equals(PlatformName))
				{
					DeviceHelper = NewObject<UCookGlobalShadersDeviceHelperBase>(GetTransientPackage(), *ClassIt);
					break;
				}
			}
		}

		if (DeviceHelper == nullptr)
		{
			UE_LOGF(LogCookGlobalShaders, Warning, "Failed to find Device Specific Implementation for '%ls' global shaders will not be deployed to the device!", *PlatformName);
		}
	}
	else if ( bCopyToStaged )
	{
		UCookGlobalShadersDeviceHelperStaged* StagedDeviceHelper = NewObject<UCookGlobalShadersDeviceHelperStaged>();
		if (!FParse::Value(*Params, TEXT("stage="), StagedDeviceHelper->StagedBuildPath, true))
		{
			StagedDeviceHelper->StagedBuildPath = FPaths::ProjectSavedDir() / TEXT("StagedBuilds") / PlatformName;
		}

		DeviceHelper = StagedDeviceHelper;
	}

	// Cook shaders
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);

	const bool bLive = Switches.Contains(TEXT("live"));
	if (bLive)
	{
		// Cook the global shaders once in case they have been modified before starting the commandlet.
		CookGlobalShaders();

		// Keep compiling shaders when directories changes.
		CookGlobalShadersOnDirectoriesChanges();
	}
	else
	{
		// Cook the global shaders once and return.
		CookGlobalShaders();
	}

	GShaderCompilingManager->PrintStats();

	UE_LOGF(LogCookGlobalShaders, Log, "Complete");

	DeviceHelper = nullptr;

	return 0;
}
