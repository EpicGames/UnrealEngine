// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/CookShadersCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "HAL/FileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "SceneTypes.h"
#include "ShaderCompiler.h"

#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookShadersCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCookShadersCommandlet, Log, All);

static const TCHAR* GlobalName = TEXT("Global");
static const TCHAR* NiagaraName = TEXT("Niagara");

// Examples
// UnrealEditor-Cmd.exe <proj> -run=CookShaders -targetPlatform=<platform> -infoFile=D:\ShaderSymbols\ShaderSymbols.info -ShaderSymbolsExport=D:\ShaderSymbols\Out -filter=Mannequin
// UnrealEditor-Cmd.exe <proj> -run=CookShaders -targetPlatform=<platform> -infoFile=D:\ShaderSymbols\ShaderSymbols.info -ShaderSymbolsExport=D:\ShaderSymbols\Out -filter=00FB89F127D2DC10 -noglobals
// UnrealEditor-Cmd.exe <proj> -run=CookShaders -targetPlatform=<platform> -ShaderSymbolsExport=D:\ShaderSymbols\Out -material=M_UI_Base_BordersAndButtons
//
// Use -dpcvars="r.Shaders.Symbols=1" to force on symbols writing from the commandline, you can also edit the appropriate [Platform]Engine.ini and uncomment or add "r.Shaders.Symbols=1", especially if you want symbols enabled longer term for that specific platform.
// To produce a new ShaderSymbols.info file, edit the cvar.Shaders.SymbolsInfo = 1 in the [Platform]Engine.ini.

namespace CookShadersCommandlet {

	// ShaderSymbols.info files will have a series of lines like the following, where the specifics of hash and extension are platform specific
	// hash0.extension Global/FTonemapCS/2233
	// hash1.extension M_Material_Name_ad9c64900150ee77/Default/FLocalVertexFactory/TBasePassPSFNoLightMapPolicy/0
	// hash2.extension NS_Niagara_System_Name/Emitted/ParticleGPUComputeScript/FNiagaraShader/0
	//
	// FInfoRecord will contain a deconstructed version of a single line from this file
	struct FInfoRecord
	{
		FString Hash;
		FString Type;
		FString Name;
		EMaterialQualityLevel::Type Quality;
		FString Emitter;
		FString Shader;
		FString VertexFactory;
		FString Pipeline;
		int32 Permutation;
	};

	// Commandlet can't get to similar list in MaterialShared.cpp as the accessors are just externs
	FName MaterialQualityLevelNames[] =
	{
		FName(TEXT("Low")),
		FName(TEXT("High")),
		FName(TEXT("Medium")),
		FName(TEXT("Epic")),
		FName(TEXT("Num"))
	};
	static_assert(UE_ARRAY_COUNT(MaterialQualityLevelNames) == EMaterialQualityLevel::Num + 1, "Missing entry from material quality level names.");

	bool LoadAndParse(const FString& Path, const FString& Filter, TArray<FInfoRecord>& OutInfo, bool bUseShortNames)
	{
		IFileManager& FileManager = IFileManager::Get();
		TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(FileManager.CreateFileReader(*Path));
		if (Reader.IsValid())
		{
			int64 Size = Reader->TotalSize();
			TArray<uint8> RawData;
			RawData.AddUninitialized(Size);
			Reader->Serialize(RawData.GetData(), Size);
			Reader->Close();

			TArray<FString> Lines;
			FString(StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(RawData.GetData())).Get()).ParseIntoArrayLines(Lines);

			for (const FString& Line : Lines)
			{
				int32 Space;
				Line.FindChar(TEXT(' '), Space);
				if (Space != INDEX_NONE)
				{
					FString HashString = Line.Left(Space);
					FString DataString = Line.Right(Line.Len() - Space - 1);

					// add to our list if it passes the filter
					if (Filter.IsEmpty() || HashString.Contains(Filter) || DataString.Contains(Filter))
					{
						FInfoRecord Record;
						Record.Hash = HashString;

						TArray<FString> Substrings;
						DataString.ParseIntoArray(Substrings, TEXT("/"));

						// need to have 3 or more parts
						if (Substrings.Num() >= 3)
						{
							// always ends in a shader/permutation
							Record.Permutation = FCString::Atoi(*Substrings[Substrings.Num() - 1]);
							Record.Shader = Substrings[Substrings.Num() - 2];

							// check for Niagara
							if (Record.Shader == TEXT("FNiagaraShader"))
							{
								Record.Type = NiagaraName;
								Record.Name = Substrings[0];
								Record.Emitter = Substrings[1];
							}
							else
							{
								// either material or global, we need to reconstruct the name
								TArray<FString> NameParts;
								Substrings[0].ParseIntoArray(NameParts, TEXT("_"));
								if (NameParts.Num() == 1)
								{
									// probably "Global"
									Record.Name = Substrings[0];
								}
								else
								{
									// probably "M_Name_MoreName_UIDNUM"
									FString UID = NameParts[NameParts.Num() - 1];
									Record.Name = Substrings[0].Left(Substrings[0].Len() - UID.Len() - 1);
								}

								if (Record.Name == GlobalName)
								{
									Record.Type = GlobalName;
									Record.Shader = Substrings[1];
									Record.Name = Record.Shader;
								}
								else
								{
									Record.Type = TEXT("Material");

									// default is Num
									Record.Quality = EMaterialQualityLevel::Num;
									FName QualityName(Substrings[1]);
									for (int32 q = 0; q < EMaterialQualityLevel::Num; ++q)
									{
										auto QualityLevel = static_cast<EMaterialQualityLevel::Type>(q);

										if (MaterialQualityLevelNames[q] == QualityName)
										{
											Record.Quality = QualityLevel;
											break;
										}
									}

									// if it has 5 or more parts, Num-3 is the vertex factory
									if (Substrings.Num() >= 5)
									{										
										Record.VertexFactory = Substrings[Substrings.Num() - 3];
 										if(bUseShortNames)
										{ 
											Record.VertexFactory.ReplaceInline(TEXT("Land"), TEXT("Landscape"));
											Record.VertexFactory.ReplaceInline(TEXT("Inst"), TEXT("Instanced"));
											Record.VertexFactory.ReplaceInline(TEXT("VF"), TEXT("VertexFactory"));
											Record.VertexFactory.ReplaceInline(TEXT("APEX"), TEXT("GPUSkinAPEXCloth"));
											Record.VertexFactory.ReplaceInline(TEXT("_1"), TEXT("true"));
											Record.VertexFactory.ReplaceInline(TEXT("_0"), TEXT("false"));
											if (Record.VertexFactory.Contains(TEXT("GPUSkin")))
											{
												Record.VertexFactory.InsertAt(0, TEXT("T"));
											}
											else
											{
												Record.VertexFactory.InsertAt(0, TEXT("F"));
											}
										}
									}

									// if it has 6 parts, Num-4 is the pipeline
									if (Substrings.Num() == 6)
									{
										Record.Pipeline = Substrings[Substrings.Num() - 4];
									}
								}
							}

							OutInfo.Emplace(Record);
						}
					}
				}
			}

			return true;
		}

		return false;
	}

	bool GetParentName(const FAssetData* InAssetData, FString& ParentName)
	{
		static const FName NAME_Parent = TEXT("Parent");
		FString ParentPathString = InAssetData->GetTagValueRef<FString>(NAME_Parent);

		int32 FirstCut = INDEX_NONE;
		ParentPathString.FindChar(L'\'', FirstCut);

		if (FirstCut != INDEX_NONE)
		{
			ParentName = ParentPathString.Mid(FirstCut + 1, ParentPathString.Len() - FirstCut - 2);
			return true;
		}

		return false;
	}
};

using namespace CookShadersCommandlet;

UCookShadersCommandlet::UCookShadersCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UCookShadersCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogCookShadersCommandlet, Log, "CookShadersCommandlet");
		UE_LOGF(LogCookShadersCommandlet, Log, "Cook shaders based upon the options, ideal for generating symbols for shaders you need");
		UE_LOGF(LogCookShadersCommandlet, Log, "Options:");
		UE_LOGF(LogCookShadersCommandlet, Log, " Required: -targetPlatform=<platform>     (Which target platform do you want results, e.g. WindowsClient, etc.");
		UE_LOGF(LogCookShadersCommandlet, Log, " Required: -ShaderSymbolsExport=<path>    (Set shader symbols output location.");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -infoFile=<path>               (Path to ShaderSymbols.info file you want to find shaders from.");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -filter=<string>               (Recommended! Filter to shaders with <string> in their hash or info data, requires -infoFile).");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -material=<string>             (Cook this material if you don't have a .info file, can be Global for global shaders).");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -noglobals                     (Don't do global shaders, even if they match the filter.)");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -nomaterialinstances           (Don't do material instances)");
		UE_LOGF(LogCookShadersCommandlet, Log, " Optional: -useshortnames				  (ShaderSymbols.info was produced with r.DumpShaderDebugShortNames=1. We need to convert back the vertex factory names");

		return 0;
	}

	// Setup
	FString Filter;
	FParse::Value(*Params, TEXT("filter="), Filter, true);
	FString MaterialString;
	FParse::Value(*Params, TEXT("material="), MaterialString, true);
	FString InfoFilePath;
	FParse::Value(*Params, TEXT("infoFile="), InfoFilePath, true);
	FString ExportPath;
	FParse::Value(*Params, TEXT("ShaderSymbolsExport="), ExportPath, true);
	const bool bNoGlobals = Switches.Contains(TEXT("noglobals"));
	const bool bNoMaterialInstances = Switches.Contains(TEXT("nomaterialinstances"));
	const bool bUseShortNames = Switches.Contains(TEXT("useshortnames"));

	TArray<FInfoRecord> Info;

	// Check to see if we want Globals specifically
	TSet<FString> GlobalsToFind;
	if (!MaterialString.IsEmpty())
	{
		if (MaterialString == GlobalName)
		{
			// we don't have a way to specify global shader name, or compile one specifically
			GlobalsToFind.Add(GlobalName);
			MaterialString = TEXT("");
		}
	}

	// Load info file if requested
	if (!InfoFilePath.IsEmpty())
	{
		if (!LoadAndParse(InfoFilePath, Filter, Info, bUseShortNames))
		{
			UE_LOGF(LogCookShadersCommandlet, Log, "Unabled to read / parse info file '%ls'", *InfoFilePath);
			return 0;
		}
	}

	// Pre-process the info we have, separating out individual requests
	TArray<FODSCRequestPayload> IndividualRequests;
	for (const auto& I : Info)
	{
		if (I.Type == GlobalName)
		{
			GlobalsToFind.Add(I.Name);
		}
		else if (I.Type == TEXT("Material"))
		{
			FODSCRequestPayload* Match = IndividualRequests.FindByPredicate(
				[&I](FODSCRequestPayload& Entry)
				{
					return (Entry.QualityLevel == I.Quality) && (Entry.VertexFactoryName == I.VertexFactory) && (Entry.MaterialName == I.Name);
				}
			);

			if (Match)
			{
				if (Match->ShaderTypeNames.Find(I.Shader) == INDEX_NONE)
				{
					Match->ShaderTypeNames.Add(I.Shader);
				}
			}
			else
			{
				TArray<FString> ShaderTypeNames;
				ShaderTypeNames.Add(I.Shader);

				IndividualRequests.Add(FODSCRequestPayload(
					EShaderPlatform::SP_NumPlatforms, ERHIFeatureLevel::Num,
					I.Quality, I.Name, I.VertexFactory, I.Pipeline, ShaderTypeNames, I.Permutation, I.Hash));
			}
		}
	}

	// Load asset lists
	UE_LOGF(LogCookShadersCommandlet, Display, "Loading Asset Registry...");
	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> MaterialList;
	TArray<FAssetData> MaterialInstanceList;
	if (!AssetRegistry.IsLoadingAssets())
	{
		if (!MaterialString.IsEmpty() || !IndividualRequests.IsEmpty())
		{
			AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), MaterialList, true);
			AssetRegistry.GetAssetsByClass(UMaterialInstance::StaticClass()->GetClassPathName(), MaterialInstanceList, true);
		}
	}

	// Locate full paths for the materials we have individual requests for & save materials to potentially find instances of
	TSet<FAssetData> MaterialsToFindInstancesOf;
	for (auto& Req : IndividualRequests)
	{
		FAssetData* Match = MaterialList.FindByPredicate(
			[&Req](FAssetData& Entry)
			{
				return Entry.AssetName == Req.MaterialName;
			}
		);

		if (Match)
		{
			Req.MaterialName = Match->GetObjectPathString();
			MaterialsToFindInstancesOf.Add(*Match);
		}
	}

	// Also locate and add materials matched from the command line switch
	TArray<FString> MaterialsRequested;
	if (!MaterialString.IsEmpty())
	{
		for (const FAssetData& It : MaterialList)
		{
			FString AssetString = It.AssetName.ToString();
			if (AssetString.Contains(MaterialString))
			{
				MaterialsRequested.Add(*It.GetObjectPathString());
				MaterialsToFindInstancesOf.Add(It);
			}
		}
	}

	// Iterate instances and find ones which depend upon the materials we are interested in
	if (!bNoMaterialInstances && MaterialsToFindInstancesOf.Num())
	{
		// for faster name lookups
		TMap<FString, int32> MaterialInstanceNameToIndex;
		int32 Index = 0;
		for (const FAssetData& Instance : MaterialInstanceList)
		{
			MaterialInstanceNameToIndex.Emplace(Instance.GetSoftObjectPath().ToString(), Index++);
		}

		TSet<FString> MaterialsToFindInstancesOfNames;
		for (const FAssetData& Instance : MaterialsToFindInstancesOf)
		{
			MaterialsToFindInstancesOfNames.Emplace(Instance.GetSoftObjectPath().ToString());
		}

		TArray<FODSCRequestPayload> InstancedRequests;
		for (const FAssetData& InstanceIt : MaterialInstanceList)
		{
			FString ParentName;
			const FAssetData* Cur = &InstanceIt;
			const FAssetData* Parent = Cur;
			while (Parent && GetParentName(Cur, ParentName))
			{
				if (MaterialsToFindInstancesOfNames.Find(ParentName))
				{
					FString InstanceName = *InstanceIt.GetSoftObjectPath().ToString();
					if (IndividualRequests.IsEmpty())
					{
						// We are matching a set of materials, and have no specific requests, simply add to the list of materials
						MaterialsRequested.Add(InstanceName);
					}
					else
					{
						// duplicate any relevent material requests using the instance name instead of material name
						for (auto& ReqIt : IndividualRequests)
						{
							if (ReqIt.MaterialName == ParentName)
							{
								FODSCRequestPayload Req(ReqIt);
								Req.MaterialName = InstanceName;
								InstancedRequests.Add(Req);
							}
						}
					}
					break;
				}

				// if our Parent is also an instance, iterate back up the hierarchy, otherwise stop iterating
				int32* IndexPtr = MaterialInstanceNameToIndex.Find(ParentName);
				Parent = IndexPtr ? &MaterialInstanceList[*IndexPtr] : nullptr;
				Cur = Parent;
			}
		}
		IndividualRequests.Append(InstancedRequests);
	}

	// Add all the unique materials found into the materials requested list
	// This is to make sure if individual requests fail to compile the shaders we want, we catch them
	// This helps catch niagara shaders, and unusual shader types which don't match their debug info
	if (!IndividualRequests.IsEmpty())
	{
		TSet<FString> UniqueRequestedMaterials;
		for (auto& I : IndividualRequests)
		{
			UniqueRequestedMaterials.Add(I.MaterialName);
		}
		for (auto& I : UniqueRequestedMaterials)
		{
			MaterialsRequested.Add(*I);
		}
	}

	// Did we find anything to do? 
	if (MaterialsRequested.IsEmpty() && GlobalsToFind.IsEmpty() && IndividualRequests.IsEmpty())
	{
		UE_LOGF(LogCookShadersCommandlet, Display, "Couldn't find anything to process!");
		return 0;
	}

	// Iterate over the active platforms
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
	for (int32 Index = 0; Index < Platforms.Num(); Index++)
	{
		TArray<FName> DesiredShaderFormats;
		Platforms[Index]->GetAllTargetedShaderFormats(DesiredShaderFormats);

		for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
		{
			const auto* Platform = Platforms[Index];
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
			FString ShaderPlatformName = LexToString(ShaderPlatform);
			FString PlatformName = Platform->PlatformName();
			ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel(ShaderPlatform);

			UE_LOGF(LogCookShadersCommandlet, Log, "Working on %ls %ls", *PlatformName, *ShaderPlatformName);

			// Setup
			TArray<uint8> OutGlobalShaderMap;
			TArray<uint8> OutMeshMaterialMaps;
			TArray<FString> OutModifiedFiles;
			FString OutputDir;
			FShaderRecompileData Arguments(PlatformName, ShaderPlatform, ODSCRecompileCommand::None, &OutModifiedFiles, &OutMeshMaterialMaps, &OutGlobalShaderMap);

			// Cook individual requests
			if (!IndividualRequests.IsEmpty())
			{
				UE_LOGF(LogCookShadersCommandlet, Display, "Cooking Individual Shaders...");

				// Adjust our requests for the current Platform and Feature Level and run them
				for (auto& I : IndividualRequests)
				{
					I.ShaderPlatform = ShaderPlatform;
					I.FeatureLevel = FeatureLevel;
				}
				Arguments.ShadersToRecompile = IndividualRequests;
				RecompileShadersForRemote(Arguments, OutputDir);
			}

			// Cook global shaders unless disabled
			if (!bNoGlobals && !GlobalsToFind.IsEmpty())
			{
				// GlobalsToFind has the list of global shaders we are interested in, although we can only compile all globals today
				UE_LOGF(LogCookShadersCommandlet, Display, "Cooking Global Shaders...");
				Arguments.CommandType = ODSCRecompileCommand::Global;
				RecompileShadersForRemote(Arguments, OutputDir);
			}

			// Cook materials
			if (!MaterialsRequested.IsEmpty())
			{
				UE_LOGF(LogCookShadersCommandlet, Display, "Cooking Materials...");
				Arguments.CommandType = ODSCRecompileCommand::Material;
				Arguments.MaterialsToLoad = MaterialsRequested;
				Arguments.ShadersToRecompile.Empty();
				RecompileShadersForRemote(Arguments, OutputDir);
			}

			const IShaderFormat* ShaderFormat = GetTargetPlatformManagerRef().FindShaderFormat(DesiredShaderFormats[FormatIndex]);
			if(ShaderFormat)
			{ 
				ShaderFormat->NotifyShaderCompilersShutdown(DesiredShaderFormats[FormatIndex]);
			}
		}
	}

	// Validate and note any missing symbol files we didn't generate, when we have enough info to do so
	if (!ExportPath.IsEmpty() && !InfoFilePath.IsEmpty())
	{
		for (const auto& I : Info)
		{
			FString Path = ExportPath + TEXT("\\") + I.Hash;
			if (!IFileManager::Get().FileExists(*Path))
			{
				UE_LOGF(LogCookShadersCommandlet, Warning, "Did not generate symbol file '%ls' for '%ls'", *I.Hash, *I.Name);
			}
		}
	}

	UE_LOGF(LogCookShadersCommandlet, Display, "Done CookShadersCommandlet");
	return 0;
}


