// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DDCCleanupCommandlet.h"

#include "CoreGlobals.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataCacheMaintainer.h"
#include "HAL/PlatformProcess.h"
#include "HAL/ThreadManager.h"
#include "Misc/OutputDeviceRedirector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DDCCleanupCommandlet)

int32 UDDCCleanupCommandlet::Main(const FString& Params)
{
	using namespace UE::DerivedData;
	ICacheStoreMaintainer& Maintainer = GetCache().GetMaintainer();
	Maintainer.BoostPriority();
	while (!Maintainer.IsIdle())
	{
		FThreadManager::Get().Tick();
		FPlatformProcess::SleepNoStats(0.05f);
		GLog->Flush();
	}
	return 0;
}
