// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DebugShaderCompileJobCommandlet.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DebugShaderCompileJobCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogDebugShaderCompileJobCommandlet, Log, All);

UDebugShaderCompileJobCommandlet::UDebugShaderCompileJobCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

// This function is intended for debugging, so disable optimizations here
UE_DISABLE_OPTIMIZATION_SHIP

int32 UDebugShaderCompileJobCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDebugShaderCompileJobCommandlet::Main);

	StaticExec(nullptr, TEXT("log LogShaders Verbose"));
	StaticExec(nullptr, TEXT("log LogShaderCompilers Verbose"));

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogDebugShaderCompileJobCommandlet, Display, "DebugShaderCompileJob FILE*");
		UE_LOGF(LogDebugShaderCompileJobCommandlet, Display, "This commandlet loads the specified shader compile jobs that were serialized and uploaded as a cook artifact and then breaks the debugger.");
		return 0;
	}

	// Validate input parameters
	if (Tokens.IsEmpty())
	{
		UE_LOGF(LogDebugShaderCompileJobCommandlet, Error, "Missing input filenames");
		return 1;
	}

	// Wait for a debugger to attach
	{
		constexpr double kNotificationWaitPeriod = 1.0;
		constexpr double kMaxWaitTime = 10.0;

		const double StartTime = FPlatformTime::Seconds();
		double LastWaitTime = 0.0;

		while (!FPlatformMisc::IsDebuggerPresent())
		{
			const double CurrentTime = FPlatformTime::Seconds();
			if (CurrentTime > LastWaitTime + kNotificationWaitPeriod)
			{
				if (CurrentTime > StartTime + kMaxWaitTime)
				{
					UE_LOGF(LogDebugShaderCompileJobCommandlet, Error, "No debugger attached after %.2fs => Exiting now", CurrentTime - StartTime);
					return 1;
				}
				else
				{
					LastWaitTime = CurrentTime;
					UE_LOGF(LogDebugShaderCompileJobCommandlet, Display, "Wait for debugger to attach ...");
				}
			}
			return 0;
		}
	}

	// Load compile job
	for (const FString& InputFilename : Tokens)
	{
		if (TUniquePtr<FArchive> InputFile = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*InputFilename)))
		{
			FShaderCompileJob Job;
			Job.SerializeArtifact(*InputFile);

			// Break debugger to investigate compile job
			UE_DEBUG_BREAK();
		}
		else
		{
			UE_LOGF(LogDebugShaderCompileJobCommandlet, Warning, "Failed to load input file: %ls", *InputFilename);
		}
	}

	UE_LOGF(LogDebugShaderCompileJobCommandlet, Display, "End debugging shader compile job");

	return 0;
}

// If this is part of a unity build file, turn optimizations back on
UE_ENABLE_OPTIMIZATION_SHIP
