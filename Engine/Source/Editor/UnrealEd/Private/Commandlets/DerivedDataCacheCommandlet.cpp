// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DerivedDataCacheCommandlet.cpp: Commandlet for DDC maintenence
=============================================================================*/
#include "Commandlets/DerivedDataCacheCommandlet.h"

#include "Algo/RemoveIf.h"
#include "Algo/StableSort.h"
#include "Algo/Transform.h"
#include "AssetCompilingManager.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "DerivedDataCacheInterface.h"
#include "DistanceFieldAtlas.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "Engine/AssetManager.h"
#include "Engine/Texture.h"
#include "GlobalShader.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "MeshCardRepresentation.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/RedirectCollector.h"
#include "PackageHelperFunctions.h"
#include "Settings/ProjectPackagingSettings.h"
#include "ShaderCompiler.h"
#include "TextureEncodingSettings.h"
#include "UObject/CoreRedirects.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DerivedDataCacheCommandlet)
DEFINE_LOG_CATEGORY_STATIC(LogDerivedDataCacheCommandlet, Log, All);

class UDerivedDataCacheCommandlet::FObjectReferencer : public FGCObject
{
public:
	FObjectReferencer(TMap<TObjectPtr<UObject>, FCachingData>& InReferencedObjects)
		: ReferencedObjects(InReferencedObjects)
	{
	}

private:
	void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AllowEliminatingReferences(false);
		Collector.AddReferencedObjects(ReferencedObjects);
		Collector.AllowEliminatingReferences(true);
	}

	FString GetReferencerName() const override
	{
		return TEXT("UDerivedDataCacheCommandlet");
	}

	FString ReferencerName;
	TMap<TObjectPtr<UObject>, FCachingData>& ReferencedObjects;
};

class UDerivedDataCacheCommandlet::FPackageListener : public FUObjectArray::FUObjectCreateListener, public FUObjectArray::FUObjectDeleteListener
{
public:
	FPackageListener()
	{
		GUObjectArray.AddUObjectDeleteListener(this);
		GUObjectArray.AddUObjectCreateListener(this);

		// We might be late to the party, check if some UPackage already have been created
		for (TObjectIterator<UPackage> PackageIter; PackageIter; ++PackageIter)
		{
			NewPackages.Add(*PackageIter);
		}
	}

	~FPackageListener()
	{
		GUObjectArray.RemoveUObjectDeleteListener(this);
		GUObjectArray.RemoveUObjectCreateListener(this);
	}

	TSet<UPackage*>& GetNewPackages()
	{
		return NewPackages;
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return NewPackages.GetAllocatedSize();
	}

private:
	void NotifyUObjectCreated(const class UObjectBase* Object, int32 Index) override
	{
		if (Object->GetClass() == UPackage::StaticClass())
		{
			NewPackages.Add(const_cast<UPackage*>(static_cast<const UPackage*>(Object)));
		}
	}

	void NotifyUObjectDeleted(const class UObjectBase* Object, int32 Index) override
	{
		if (Object->GetClass() == UPackage::StaticClass())
		{
			NewPackages.Remove(const_cast<UPackage*>(static_cast<const UPackage*>(Object)));
		}
	}

	void OnUObjectArrayShutdown() override
	{
		GUObjectArray.RemoveUObjectDeleteListener(this);
		GUObjectArray.RemoveUObjectCreateListener(this);
	}

	TSet<UPackage*> NewPackages;
};

UDerivedDataCacheCommandlet::UDerivedDataCacheCommandlet(FVTableHelper& Helper)
	: Super(Helper)
{
}

UDerivedDataCacheCommandlet::~UDerivedDataCacheCommandlet()
{
}

UDerivedDataCacheCommandlet::UDerivedDataCacheCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LogToConsole = false;
}

void UDerivedDataCacheCommandlet::MaybeMarkPackageAsAlreadyLoaded(UPackage* Package)
{
	if (ProcessedPackages.Contains(Package->GetFName()))
	{
		UE_LOGF(LogDerivedDataCacheCommandlet, Verbose, "Marking %ls already loaded.", *Package->GetName());
		Package->SetPackageFlags(PKG_ReloadingForCooker);
	}
}

namespace UE::Private::DerivedDataCacheCommandlet
{

static void TickCookObjects(bool bCookComplete = false)
{
	constexpr double TickPeriod = 0.1;
	static double LastTickTime = FPlatformTime::Seconds();
	const double CurrentTime = FPlatformTime::Seconds();
	if (bCookComplete || LastTickTime + TickPeriod <= CurrentTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TickCookObjects);
		FTickableCookObject::TickObjects(static_cast<float>(CurrentTime - LastTickTime), bCookComplete);
		LastTickTime = CurrentTime;
	}
}

static void WaitForCompilationToFinish(bool& bInOutHadActivity)
{
	auto LogStatus =
		[](IAssetCompilingManager* CompilingManager)
	{
		int32 AssetCount = CompilingManager->GetNumRemainingAssets();
		if (AssetCount > 0)
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Waiting for %d %ls to finish.", AssetCount, *FText::Format(CompilingManager->GetAssetNameFormat(), FText::AsNumber(AssetCount)).ToString());
		}
		else
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Done waiting for %ls to finish.", *FText::Format(CompilingManager->GetAssetNameFormat(), FText::AsNumber(100)).ToString());
		}
	};

	while (FAssetCompilingManager::Get().GetNumRemainingAssets() > 0)
	{
		TickCookObjects();

		for (IAssetCompilingManager* CompilingManager : FAssetCompilingManager::Get().GetRegisteredManagers())
		{
			int32 CachedAssetCount = CompilingManager->GetNumRemainingAssets();
			if (CachedAssetCount)
			{
				bInOutHadActivity = true;
				LogStatus(CompilingManager);
				int32 NumCompletedAssetsSinceLastLog = 0;
				while (CompilingManager->GetNumRemainingAssets() > 0)
				{
					const int32 CurrentAssetCount = CompilingManager->GetNumRemainingAssets();
					NumCompletedAssetsSinceLastLog += (CachedAssetCount - CurrentAssetCount);
					CachedAssetCount = CurrentAssetCount;

					if (NumCompletedAssetsSinceLastLog >= 1000)
					{
						LogStatus(CompilingManager);
						NumCompletedAssetsSinceLastLog = 0;
					}

					// Process any asynchronous Asset compile results that are ready, limit execution time
					FAssetCompilingManager::Get().ProcessAsyncTasks(true);
				}

				LogStatus(CompilingManager);
			}
		}
	}
}

} // UE::Private::DerivedDataCacheCommandlet

void UDerivedDataCacheCommandlet::CacheLoadedPackages(UPackage* CurrentPackage, uint8 PackageFilter, TSet<FName>& OutNewProcessedPackages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDerivedDataCacheCommandlet::CacheLoadedPackages);

	const double BeginCacheTimeStart = FPlatformTime::Seconds();

	// We will only remove what we process from the list to avoid unprocessed package being forever forgotten.
	TSet<UPackage*>& NewPackages = PackageListener->GetNewPackages();

	TArray<UObject*> ObjectsWithOuter;
	for (auto NewPackageIt = NewPackages.CreateIterator(); NewPackageIt; ++NewPackageIt)
	{
		UPackage* NewPackage = *NewPackageIt;
		const FName NewPackageName = NewPackage->GetFName();
		if (!ProcessedPackages.Contains(NewPackageName))
		{
			if ((PackageFilter & NORMALIZE_ExcludeEnginePackages) != 0 && NewPackage->GetName().StartsWith(TEXT("/Engine")))
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Skipping %ls as Engine package", *NewPackageName.ToString());

				// Add it so we don't convert the FName to a string everytime we encounter this package
				ProcessedPackages.Add(NewPackageName);
				NewPackageIt.RemoveCurrent();
			}
			else if (NewPackage == CurrentPackage || !PackagesToProcess.Contains(NewPackageName))
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Processing %ls", *NewPackageName.ToString());

				ProcessedPackages.Add(NewPackageName);
				OutNewProcessedPackages.Add(NewPackageName);
				NewPackageIt.RemoveCurrent();

				ObjectsWithOuter.Reset();
				GetObjectsWithOuter(NewPackage, ObjectsWithOuter, EGetObjectsFlags::IncludeNestedObjects, RF_ClassDefaultObject /* ExclusionFlags */);
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(NewPackage, PackageAccessTrackingOps::NAME_CookerBuildObject);
				for (UObject* Object : ObjectsWithOuter)
				{
					FCachingData& CachingData = CachingObjects.FindOrAdd(Object);

					if (!CachingData.ObjectValidityReference.IsValid())
					{
						CachingData = FCachingData();
						CachingData.ObjectValidityReference = Object;
					}

					// For texture ddc fills, we want to stagger the platforms so that the base texture is only encoded
					// once with shared linear texture encoding. If we kick everything off at the same time then all worker
					// tasks ask for the linear encoding at the same time. If we queue the other platforms after the first one
					// is done, then the subsequent platforms can retrieve that encoded texture and avoid a lot of work.
					//
					// This only actually helps if some of the platforms you're filling have tiling - otherwise
					// you're delaying the other platforms for no reason. When I did some measuring this had no visible effect
					// on ddc fill times - there's enough work to fill up the space, so rather than add a complicated system
					// for determining if the platform will utilize it, we just always do it.
					if (bSharedLinearTextureEncodingEnabled && Platforms.Num() > 1 && Object->IsA<UTexture>())
					{
						CachingData.bFirstPlatformIsSolo = true;
					}

					for (auto Platform : Platforms)
					{
						Object->BeginCacheForCookedPlatformData(Platform);
						if (CachingData.bFirstPlatformIsSolo)
						{
							// We will start the other platforms later.
							break;
						}
					}

					CachingData.PlatformIsComplete.Init(false, Platforms.Num());
				}
			}
		}
		else
		{
			NewPackageIt.RemoveCurrent();
		}
	}

	BeginCacheTime += FPlatformTime::Seconds() - BeginCacheTimeStart;

	ProcessCachingObjects();
}

bool UDerivedDataCacheCommandlet::ProcessCachingObjects()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDerivedDataCacheCommandlet::ProcessCachingObjects);

	bool bHadActivity = false;
	if (CachingObjects.Num() > 0)
	{
		FAssetCompilingManager::Get().ProcessAsyncTasks(true);

		double CurrentTime = FPlatformTime::Seconds();
		for (auto It = CachingObjects.CreateIterator(); It; ++It)
		{
			// Call IsCachedCookedPlatformDataLoaded once a second per object since it can be quite expensive
			FCachingData& CachingData = It->Value;
			if (CurrentTime - CachingData.LastTimeTested > 1.0)
			{
				if (!It->Value.ObjectValidityReference.IsValid())
				{
					It.RemoveCurrent();
					continue;
				}

				UObject* Object = It->Key;
				bool bIsFinished = true;
				const IInterface_AsyncCompilation* Interface_AsyncCompilation = Cast<IInterface_AsyncCompilation>(Object);
				if (Interface_AsyncCompilation && Interface_AsyncCompilation->IsCompiling())
				{
					bIsFinished = false;
				}

				{
					UE_TRACK_REFERENCING_PACKAGE_SCOPED(Object, PackageAccessTrackingOps::NAME_CookerBuildObject);
					for (int32 PlatformIndex = 0; PlatformIndex < Platforms.Num(); ++PlatformIndex)
					{
						// IsCachedCookedPlatformDataLoaded can be quite slow for some objects
						// Do not call it if bIsFinished is already false
						// Also, by the contract of IsCachedCookedPlatformDataLoaded, we are not allowed to call it
						// again once it returns true
						if (bIsFinished && !CachingData.PlatformIsComplete[PlatformIndex])
						{
							const ITargetPlatform* Platform = Platforms[PlatformIndex];
							if (Object->IsCachedCookedPlatformDataLoaded(Platform))
							{
								CachingData.PlatformIsComplete[PlatformIndex] = true;
								bHadActivity = true;

								if (CachingData.bFirstPlatformIsSolo)
								{
									bIsFinished = false;
									CachingData.bFirstPlatformIsSolo = false;

									// Start the remaining platforms now. We only launched PlatformIndex == 0.
									check(PlatformIndex == 0);
									check(Platforms.Num() > 1); // we shouldn't be here for only 1 platform
									for (int32 AddingPlatformIndex = 1; AddingPlatformIndex < Platforms.Num(); AddingPlatformIndex++)
									{
										Object->BeginCacheForCookedPlatformData(Platforms[AddingPlatformIndex]);
									}
								}
							}
							else
							{
								bIsFinished = false;
							}
						}
					}
				}

				if (bIsFinished)
				{
					bHadActivity = true;
					Object->WillNeverCacheCookedPlatformDataAgain();
					Object->ClearAllCachedCookedPlatformData();
					It.RemoveCurrent();
				}
				else
				{
					CachingData.LastTimeTested = CurrentTime;
				}
			}
		}
	}

	return bHadActivity;
}

void UDerivedDataCacheCommandlet::FinishCachingObjects()
{
	// Timing variables
	double DDCCommandletMaxWaitSeconds = 60. * 10.;
	GConfig->GetDouble(TEXT("CookSettings"), TEXT("DDCCommandletMaxWaitSeconds"), DDCCommandletMaxWaitSeconds, GEditorIni);

	const double FinishCacheTimeStart = FPlatformTime::Seconds();
	double LastActivityTime = FinishCacheTimeStart;

	while (CachingObjects.Num() > 0)
	{
		UE::Private::DerivedDataCacheCommandlet::TickCookObjects();

		bool bHadActivity = ProcessCachingObjects();

		double CurrentTime = FPlatformTime::Seconds();
		if (!bHadActivity)
		{
			UE::Private::DerivedDataCacheCommandlet::WaitForCompilationToFinish(bHadActivity);
		}
		if (!bHadActivity)
		{
			if (CurrentTime - LastActivityTime >= DDCCommandletMaxWaitSeconds)
			{
				UObject* Object = CachingObjects.CreateIterator()->Value.ObjectValidityReference.Get();
				UE_LOGF(LogDerivedDataCacheCommandlet, Warning, "Timed out for %.2lfs waiting for %d objects to finish caching. First object: %ls.",
					DDCCommandletMaxWaitSeconds, CachingObjects.Num(), Object ? *Object->GetFullName() : TEXT("Unknown deleted object"));
				break;
			}
			else
			{
				const float WaitingForCacheSleepTime = 0.050f;
				FPlatformProcess::Sleep(WaitingForCacheSleepTime);
			}
		}
		else
		{
			LastActivityTime = CurrentTime;
		}
	}

	FinishCacheTime += FPlatformTime::Seconds() - FinishCacheTimeStart;
}

void UDerivedDataCacheCommandlet::CacheWorldPackages(UWorld* World, uint8 PackageFilter, TSet<FName>& OutNewProcessedPackages)
{
	if (!UWorld::IsPartitionedWorld(World))
	{
		return;
	}

	// Setup the world
	UWorld::InitializationValues IVS;
	IVS.RequiresHitProxies(false);
	IVS.ShouldSimulatePhysics(false);
	IVS.EnableTraceCollision(false);
	IVS.CreateNavigation(false);
	IVS.CreateAISystem(false);
	IVS.AllowAudioPlayback(false);
	IVS.CreatePhysicsScene(true);
	FScopedEditorWorld EditorWorld(World, IVS);

	// If the world is partitioned
	// Ensure the world has a valid world partition.
	UWorldPartition* WorldPartition = World->GetWorldPartition();
	check(WorldPartition);

	FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [this, PackageFilter, &OutNewProcessedPackages](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		if (AActor* Actor = ActorDescInstance->GetActor())
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Loaded actor %ls", *Actor->GetName());
			CacheLoadedPackages(Actor->GetPackage(), PackageFilter, OutNewProcessedPackages);
		}
		return true;
	});
}

int32 UDerivedDataCacheCommandlet::Main( const FString& Params )
{
	// Avoid putting those directly in the constructor because we don't
	// want the CDO to have a second copy of these being active.
	PackageListener  = MakeUnique<FPackageListener>();
	ObjectReferencer = MakeUnique<FObjectReferencer>(CachingObjects);

	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	bool bFillCache = Switches.Contains("FILL");   // do the equivalent of a "loadpackage -all" to fill the DDC
	bool bStartupOnly = Switches.Contains("STARTUPONLY");   // regardless of any other flags, do not iterate packages
	const bool bDryRun = Switches.Contains("DRYRUN");   // build a list of stuff to process but don't start loading any packages

	// Subsets for parallel processing
	uint32 SubsetMod = 0;
	uint32 SubsetTarget = MAX_uint32;
	FParse::Value(*Params, TEXT("SubsetMod="), SubsetMod);
	FParse::Value(*Params, TEXT("SubsetTarget="), SubsetTarget);
	bool bDoSubset = SubsetMod > 0 && SubsetTarget < SubsetMod;

	if (FResolvedTextureEncodingSettings::Get().Project.bSharedLinearTextureEncoding)
	{
		bSharedLinearTextureEncodingEnabled = true;
	}

	double FindProcessedPackagesTime = 0.0;
	double GCTime = 0.0;
	FinishCacheTime = 0.;
	BeginCacheTime = 0.;
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	Platforms.Reset();
	Platforms.Append(TPM->GetActiveTargetPlatforms());

	if (!bStartupOnly && bFillCache)
	{
		FCoreUObjectDelegates::PackageCreatedForLoad.AddUObject(this, &UDerivedDataCacheCommandlet::MaybeMarkPackageAsAlreadyLoaded);

		Tokens.Empty(2);

		FString MapList;
		if(FParse::Value(*Params, TEXT("Map="), MapList))
		{
			for(int StartIdx = 0; StartIdx < MapList.Len();)
			{
				int EndIdx = StartIdx;
				while(EndIdx < MapList.Len() && MapList[EndIdx] != '+')
				{
					EndIdx++;
				}
				Tokens.Add(MapList.Mid(StartIdx, EndIdx - StartIdx) + FPackageName::GetMapPackageExtension());
				StartIdx = EndIdx + 1;
			}
		}

		// support MapIniSection parameter
		{
			TArray<FString> MapIniSections;
			FString SectionStr;
			if (FParse::Value(*Params, TEXT("MAPINISECTION="), SectionStr))
			{
				if (SectionStr.Contains(TEXT("+")))
				{
					TArray<FString> Sections;
					SectionStr.ParseIntoArray(Sections, TEXT("+"), true);
					for (int32 Index = 0; Index < Sections.Num(); Index++)
					{
						MapIniSections.Add(Sections[Index]);
					}
				}
				else
				{
					MapIniSections.Add(SectionStr);
				}

				TArray<FString> MapsFromIniSection;
				for (const FString& MapIniSection : MapIniSections)
				{
					GEditor->LoadMapListFromIni(*MapIniSection, MapsFromIniSection);
				}

				Tokens += MapsFromIniSection;
			}
		}

		TArray<FString> CommandLinePackageNames;

		// Allow adding collections to the list of packages to process
		if (FString CollectionArg; FParse::Value(*Params, TEXT("COLLECTION="), CollectionArg))
		{
			ICollectionManager& CollectionManager = FModuleManager::LoadModuleChecked<FCollectionManagerModule>("CollectionManager").Get();

			TArray<FString> Collections;
			CollectionArg.ParseIntoArray(Collections, TEXT("+"));
			for (const FString& Collection : Collections)
			{
				TSharedPtr<ICollectionContainer> CollectionContainer;
				FName CollectionName;
				ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
				if (!CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType) ||
					!CollectionContainer->CollectionExists(CollectionName, ShareType))
				{
					UE_LOGF(LogDerivedDataCacheCommandlet, Error, "Found no collections for command line argument %ls", *Collection);
					continue;
				}

				TArray<FSoftObjectPath> FoundAssets;
				CollectionContainer->GetAssetsInCollection(CollectionName, ShareType, FoundAssets, ECollectionRecursionFlags::SelfAndChildren);
				Tokens.Reserve(Tokens.Num() + FoundAssets.Num());
				for (const FSoftObjectPath& AssetPath : FoundAssets)
				{
					CommandLinePackageNames.Add(AssetPath.GetLongPackageName());
				}
			}
		}

		// check to see if we should use the primary asset as the starting set
		if (Switches.Contains("USEPRIMARYASSETS"))
		{
			// SearchAllAssets is required by ModifyCook
			IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
			AssetRegistry.SearchAllAssets(true /* bSynchronousSearch */);

			// Call ModifyCook with one TargetPlatform at a time and take the union of PrimaryAssets. If we have
			// multiple target platforms and they disagree about development objects, ModifyCook produces an error
			// because the cook output is invalid. We're not producing cooked output, so we don't care about that, so
			// call them one at a time to avoid the error.
			TSet<FName> PrimaryAssetNames;
			for (const ITargetPlatform* TargetPlatform : Platforms)
			{
				TArray<FName> PlatformPrimaryAssets;
				TArray<FName> Unused;
				UAssetManager::Get().ModifyCook({ TargetPlatform }, PlatformPrimaryAssets, Unused);
				PrimaryAssetNames.Append(PlatformPrimaryAssets);
			}
			for (FName PrimaryAsset : PrimaryAssetNames)
			{
				CommandLinePackageNames.Add(PrimaryAsset.ToString());
			}
		}

		// Add defaults if we haven't specifically found anything on the command line 
		if (Tokens.IsEmpty() && CommandLinePackageNames.IsEmpty())
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Adding default search tokens for all assets and maps");

			Tokens.Add(FString("*") + FPackageName::GetAssetPackageExtension());
			Tokens.Add(FString("*") + FPackageName::GetMapPackageExtension());
		}

		uint8 PackageFilter = NORMALIZE_DefaultFlags;
		if ( Switches.Contains(TEXT("MAPSONLY")) )
		{
			PackageFilter |= NORMALIZE_ExcludeContentPackages;
		}

		if ( Switches.Contains(TEXT("PROJECTONLY")) )
		{
			PackageFilter |= NORMALIZE_ExcludeEnginePackages;
		}

		if ( !Switches.Contains(TEXT("DEV")) )
		{
			PackageFilter |= NORMALIZE_ExcludeDeveloperPackages;
		}

		if ( !Switches.Contains(TEXT("NOREDIST")) )
		{
			PackageFilter |= NORMALIZE_ExcludeNoRedistPackages;
		}

		// assume the first token is the map wildcard/pathname
		TSet<FString> FilesInPath;
		TArray<FString> Unused;
		TArray<FString> TokenFiles;
		for ( int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++ )
		{
			TokenFiles.Reset();
			if (!NormalizePackageNames(Unused, TokenFiles, Tokens[TokenIndex], PackageFilter))
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "No packages found for parameter %i: '%ls'", TokenIndex, *Tokens[TokenIndex]);
				continue;
			}

			FilesInPath.Append(TokenFiles);
		}


		TArray<TPair<FString, FName>> PackagePaths;
		PackagePaths.Reserve(FilesInPath.Num());
		for (FString& Filename : FilesInPath)
		{
			FString PackageName;
			FString FailureReason;
			if (!FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName, &FailureReason))
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Error, "Unable to resolve filename %ls to package name because: %ls", *Filename, *FailureReason);
				continue;
			}
			PackagePaths.Emplace(MoveTemp(Filename), FName(*PackageName));
		}

		if (!CommandLinePackageNames.IsEmpty())
		{
			if (!NormalizePackageNames(CommandLinePackageNames, Unused, TEXT(""), PackageFilter))
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Failed to normalize command line package names");
			}
			else
			{
				for( const FString& PackageName : CommandLinePackageNames )
				{
					FString Filename;
					if (FPackageName::DoesPackageExist(PackageName, &Filename))
					{
						PackagePaths.Emplace(MoveTemp(Filename), FName(*PackageName));
					}
					else
					{
						UE_LOGF(LogDerivedDataCacheCommandlet, Warning, "Unable to resolve filename from package name %ls", *PackageName);
						continue;
					}
				}
			}
		}

		// Respect settings that instruct us not to enumerate some paths
		TArray<FString> LocalDirsToNotSearch;
		const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
		for (const FDirectoryPath& DirToNotSearch : PackagingSettings->TestDirectoriesToNotSearch)
		{
			FString LocalPath;
			if (FPackageName::TryConvertGameRelativePackagePathToLocalPath(DirToNotSearch.Path, LocalPath))
			{
				LocalDirsToNotSearch.Add(LocalPath);
			}
			else
			{
				UE_LOGF(LogCook, Warning, "'ProjectSettings -> Project -> Packaging -> Test directories to not search' has invalid element '%ls'", *DirToNotSearch.Path);
			}
		}

		TArray<FString> LocalFilenamesToSkip;
		if (FPackageName::FindPackagesInDirectories(LocalFilenamesToSkip, LocalDirsToNotSearch))
		{
			TSet<FName> PackageNamesToSkip;
			Algo::Transform(LocalFilenamesToSkip, PackageNamesToSkip, [](const FString& Filename)
				{
					FString PackageName;
					if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
					{
						return FName(*PackageName);
					}
					return FName(NAME_None);
				});

			int32 NewNum = Algo::StableRemoveIf(PackagePaths, [&PackageNamesToSkip](const TPair<FString,FName>& PackagePath) { return PackageNamesToSkip.Contains(PackagePath.Get<1>()); });
			PackagePaths.SetNum(NewNum);
		}

		if (PackagePaths.Num() == 0)
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "No packages found to load from command line arguments.");
		}
		else
		{
			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "%d packages to load from command line arguments", PackagePaths.Num());
			for( int32 Index=0; Index < PackagePaths.Num(); ++Index)
			{
				const TPair<FString, FName>& Pair = PackagePaths[Index];
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, " %d) %ls", Index + 1, *Pair.Get<1>().ToString());
			}
		}

		for (int32 Index = 0; Index < Platforms.Num(); Index++)
		{
			TArray<FName> DesiredShaderFormats;
			Platforms[Index]->GetAllTargetedShaderFormats(DesiredShaderFormats);

			for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
			{
				const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);
				// Kick off global shader compiles for each target platform. Note that shader platform alone is not sufficient to distinguish between WindowsEditor and WindowsClient, which after UE 4.25 have different DDC
				CompileGlobalShaderMap(ShaderPlatform, Platforms[Index], false);
				// Block to wait for global shaders for this platform to complete before submitting new jobs.
				// Only a single platform/target's global shader jobs can be in progress at the same time (otherwise these conflict with each other)
				// so block after each call to CompileGlobalShaderMap.
				GShaderCompilingManager->ProcessAsyncResults(false, true);
				check(GGlobalShaderMap[ShaderPlatform]->IsComplete(Platforms[Index]));
			}
		}

		const int32 GCInterval = 100;
		int32 NumProcessedSinceLastGC = 0;
		bool bLastPackageWasMap = false;

		// Mark command-line packages as already discovered so we don't double-add from soft refs and can avoid loading packages on other shards
		PackagesToProcess.Empty(PackagePaths.Num());
		for (int32 PackageIndex = PackagePaths.Num() - 1; PackageIndex >= 0; PackageIndex--)
		{
			PackagesToProcess.Add(PackagePaths[PackageIndex].Get<1>());
		}

		// Add all soft object references from no asset in particular to the packages to be processed, before filtering in the case of distributed work
		{
			int32 StartingPackageCount = PackagePaths.Num();
			TSet<FName> SoftReferencedPackages;
			GRedirectCollector.ProcessSoftObjectPathPackageList(NAME_None, false, SoftReferencedPackages);
			for (FName SoftRefName : SoftReferencedPackages)
			{
				if (PackagesToProcess.Contains(SoftRefName))
				{
					continue;
				}

				FString SoftRefFilename;
				if (FPackageName::DoesPackageExist(SoftRefName.ToString(), &SoftRefFilename))
				{
					PackagePaths.Push(TPair<FString, FName>(SoftRefFilename, SoftRefName));
					PackagesToProcess.Add(SoftRefName);
				}
			}

			if (StartingPackageCount == PackagePaths.Num())
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "No packages found to load from startup soft references.");
			}
			else
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "%d packages to load from startup soft references", PackagePaths.Num() - StartingPackageCount);
				for (int32 Index = StartingPackageCount; Index < PackagePaths.Num(); ++Index)
				{
					const TPair<FString, FName>& Pair = PackagePaths[Index];
					UE_LOGF(LogDerivedDataCacheCommandlet, Display, " %d) %ls", Index + 1 - StartingPackageCount, *Pair.Get<1>().ToString());
				}
			}
		}

		// Sort maps to the end of the list of packages to process to maximize the chance of sharded instances populating the DDC from individual packages.
		Algo::StableSortBy(PackagePaths, 
			[](TPair<FString, FName>& Pair){ return Pair.Get<0>().EndsWith(FPackageName::GetMapPackageExtension()); },
			TLess<bool>()
		);

		// If work is distributed, skip packages that are meant to be process by other machines
		// Do this before the main loop so that we don't filter soft refs that we enqueue 
		if (bDoSubset)
		{
			PackagePaths.RemoveAll([SubsetMod, SubsetTarget](const TPair<FString, FName>& P) {
				FName PackageFName = P.Value;

				FString PackageName = PackageFName.ToString();
				if (FCrc::StrCrc_DEPRECATED(*PackageName.ToUpper()) % SubsetMod != SubsetTarget)
				{
					return true;
				}
				return false;
			});

			if (PackagePaths.Num() == 0)
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "No packages to process after subset split!");
			}
			else
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Display, "%d packages to load after subset split", PackagePaths.Num());
				for (int32 Index = 0; Index < PackagePaths.Num(); ++Index)
				{
					const TPair<FString, FName>& Pair = PackagePaths[Index];
					UE_LOGF(LogDerivedDataCacheCommandlet, Display, " %d) %ls", Index + 1, *Pair.Get<1>().ToString());
				}
			}
		}

		if (bDryRun)
		{
			PackagePaths.Empty();
		}

		// Process each package
		int32 PackageOrder = 0;
		while( PackagePaths.Num())
		{
			TTuple<FString, FName> PackagePath = PackagePaths.Pop();
			const FString& Filename = PackagePath.Get<0>();
			FName PackageFName = PackagePath.Get<1>();
			if (ProcessedPackages.Contains(PackageFName))
			{
				// Soft refs may be queued, then processed as a hard ref from something else.
				continue;
			}

			UE_LOGF(LogDerivedDataCacheCommandlet, Display, "Loading (%d) %ls", ++PackageOrder, *Filename);

			UPackage* Package = LoadPackage(NULL, *Filename, LOAD_None);
			if (Package == NULL)
			{
				UE_LOGF(LogDerivedDataCacheCommandlet, Error, "Error loading %ls!", *Filename);
				bLastPackageWasMap = false;
			}
			else
			{
				bLastPackageWasMap = Package->ContainsMap();
				NumProcessedSinceLastGC++;
			}

			// Find any new packages and cache all the objects in each package
			TSet<FName> NewProcessedPackages;
			CacheLoadedPackages(Package, PackageFilter, NewProcessedPackages);

			// Ensure we load maps to process all their referenced packages in case they are using world partition.
			if (bLastPackageWasMap)
			{
				if (UWorld* World = UWorld::FindWorldInPackage(Package))
				{
					CacheWorldPackages(World, PackageFilter, NewProcessedPackages);
				}
			}

			// Queue up soft references of each package we just processed
			NewProcessedPackages.Add(NAME_None); // Always check for more references from non-asset systems each step
			for( FName NewProcessedPackage : NewProcessedPackages)
			{
				TSet<FName> SoftReferencedPackages;
				GRedirectCollector.ProcessSoftObjectPathPackageList(NewProcessedPackage, false, SoftReferencedPackages);
				for (FName SoftRefName : SoftReferencedPackages)
				{
					// Packages may already be enqueued on this or another machine 
					if (!PackagesToProcess.Contains(SoftRefName) && !ProcessedPackages.Contains(SoftRefName))
					{
						PackagesToProcess.Add(SoftRefName);
						FString SoftRefFilename;

						FCoreRedirectObjectName CoreRedirectName;
						CoreRedirectName.PackageName = SoftRefName;
						// Packages that are specifically identified by PackageRedirects as removed need to be skipped.
						if (FCoreRedirects::IsKnownMissing(ECoreRedirectFlags::Type_Package, CoreRedirectName))
						{
							continue;
						}

						if (FPackageName::DoesPackageExist(SoftRefName.ToString(), &SoftRefFilename))
						{
							UE_LOGF(LogDerivedDataCacheCommandlet, Log, "Package '%ls' queueing soft reference '%ls' for later processing", *NewProcessedPackage.ToString(), *SoftRefName.ToString());
							PackagePaths.Push(TPair<FString, FName>(SoftRefFilename, SoftRefName));
						}
						else
						{
							UE_LOGF(LogDerivedDataCacheCommandlet, Warning, "Package '%ls' failed to find soft reference '%ls'", *NewProcessedPackage.ToString(), *SoftRefName.ToString());
						}
					}
					else
					{
						UE_LOGF(LogDerivedDataCacheCommandlet, Verbose, "Package '%ls' skipping soft reference '%ls': %ls, %ls ", 
							*NewProcessedPackage.ToString(),
							*SoftRefName.ToString(),									 
							PackagesToProcess.Contains(SoftRefName) ? TEXT("ALREADY QUEUED") : TEXT("NOT QUEUED"),
							ProcessedPackages.Contains(SoftRefName) ? TEXT("ALREADY PROCESSED") : TEXT("NOT PROCESSED")
						);
					}
				}
			}

			UE::Private::DerivedDataCacheCommandlet::TickCookObjects();

			// Perform a GC if conditions are met
			if (NumProcessedSinceLastGC >= GCInterval || PackagePaths.IsEmpty() || bLastPackageWasMap)
			{
				const double StartGCTime = FPlatformTime::Seconds();
				if (NumProcessedSinceLastGC >= GCInterval || PackagePaths.IsEmpty())
				{
					UE_LOGF(LogDerivedDataCacheCommandlet, Display, "GC (Full)...");
					CollectGarbage(RF_NoFlags);
					NumProcessedSinceLastGC = 0;
				}
				else
				{
					UE_LOGF(LogDerivedDataCacheCommandlet, Display, "GC...");
					CollectGarbage(RF_Standalone);
				}
				GCTime += FPlatformTime::Seconds() - StartGCTime;

				bLastPackageWasMap = false;
			}
		}
	}

	FinishCachingObjects();

	UE::Private::DerivedDataCacheCommandlet::TickCookObjects(/*bCookComplete*/ true);

	GetDerivedDataCacheRef().WaitForQuiescence(true);

	UE_LOGF(LogDerivedDataCacheCommandlet, Display, "BeginCacheTime=%.2lfs, FinishCacheTime=%.2lfs, GCTime=%.2lfs.", BeginCacheTime, FinishCacheTime, GCTime);

	return 0;
}
