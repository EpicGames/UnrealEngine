// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DetectOrphanedLocalizedAssetsCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Misc/FileHelper.h"
#include "Internationalization/PackageLocalizationUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DetectOrphanedLocalizedAssetsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogDetectOrphanedLocalizedAssetsCommandlet, Log, All);

/**
 *	UDetectOrphanedLocalizedAssetsCommandlet
 */
UDetectOrphanedLocalizedAssetsCommandlet::UDetectOrphanedLocalizedAssetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FString UDetectOrphanedLocalizedAssetsCommandlet::UsageText
	(
	TEXT("DetectOrphanedLocalizedAssetsCommandlet usage...\r\n")
	TEXT("    <GameName> DetectOrphanedLocalizedAssetsCommandlet -OutputOrphans=<path to output text file containing all orphaned assets>\r\n")
	);

int32 UDetectOrphanedLocalizedAssetsCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);
	if (Switches.Contains(TEXT("help")) || Switches.Contains(TEXT("Help")))
	{
		UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "%ls", *UsageText);
		return 0;
	}
	IAssetRegistry::Get()->SearchAllAssets(true);
	// We do not filter the asset registry against the Audit_InCook collection because orphaned localized assets will not be cooked and therefore not make it into the Audit_InCook collection 


	TArray<FAssetData> AllAssets;
	const double GetAllAssetsWithFirstPassFilterStartTime = FPlatformTime::Seconds();
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Detecting orphaned localized assets across all assets on disk.");
	IAssetRegistry::GetChecked().GetAllAssets(AllAssets);
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Getting all assets from asset registry took %.2f seconds.", FPlatformTime::Seconds() - GetAllAssetsWithFirstPassFilterStartTime);

	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Processing %d assets.", AllAssets.Num());
	TSet<FSoftObjectPath> LocalizedAssets;
	LocalizedAssets.Reserve(AllAssets.Num());
	const double AllAssetsIterationStartTime = FPlatformTime::Seconds();
	for (const FAssetData& Asset : AllAssets)
	{
		if (FPackageName::IsLocalizedPackage(Asset.GetSoftObjectPath().GetLongPackageName()) && !Asset.IsRedirector())
		{
			LocalizedAssets.Add(Asset.GetSoftObjectPath());
		}
	}
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Iterating through all assets took %.2f seconds.", FPlatformTime::Seconds() - AllAssetsIterationStartTime);
	float LocalizedAssetsPercentage = static_cast<float>(LocalizedAssets.Num()) / static_cast<float>(AllAssets.Num()) * 100.0f;
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Found %d localized assets out of %d assets. %.2f%% of assets are localized.", LocalizedAssets.Num(), AllAssets.Num(), LocalizedAssetsPercentage);

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	FString SourceObjectPath;
	FAssetData OutAssetData;
	TArray<FSoftObjectPath> OrphanedLocalizedAssets;
	TArray<FName> Referencers;
	const double DetectOrphansStartTime = FPlatformTime::Seconds();
	for (const FSoftObjectPath& LocalizedAsset: LocalizedAssets)
	{
		SourceObjectPath.Reset();
		// Despite the namespace, the function can indeed also convert localized assets to source 
		if (FPackageLocalizationUtil::ConvertLocalizedToSource(LocalizedAsset.GetAssetPathString(), SourceObjectPath))
		{
			UE::AssetRegistry::EExists Exists = AssetRegistry.TryGetAssetByObjectPath(FSoftObjectPath(SourceObjectPath), OutAssetData);
			if (Exists == UE::AssetRegistry::EExists::Exists && !OutAssetData.IsRedirector())
			{
				// The source version of this asset exists and it's not a redirector. This localized asset is definitely not orphaned. Moving along 
				continue;
			}
			// If the source asset doesn't exist, we still need to check and make sure that none of the referencers are a localized asset 
			Referencers.Reset();
			AssetRegistry.GetReferencers(LocalizedAsset.GetLongPackageFName(), Referencers);
			if (Referencers.Num() == 0)
			{
				OrphanedLocalizedAssets.Add(LocalizedAsset);
				continue;
			}
			bool bOrphaned = true;
			// We go through all of the referencers and see if any of those packages are localized packages.
			for (const FName& Reference : Referencers)
			{
				if (FPackageName::IsLocalizedPackage(Reference.ToString()))
				{
					// This is a localized asset with no source asset but there is another localized asset referencing it. This is not considered orphaned 
					bOrphaned = false;
					break;
				}
			}
			if (bOrphaned)
			{
				OrphanedLocalizedAssets.Add(LocalizedAsset);
			}
		}
	}
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Detecting orphaned localized assets took %.2f seconds.", FPlatformTime::Seconds() - DetectOrphansStartTime);
	float OrphanedPercentage = static_cast<float>(OrphanedLocalizedAssets.Num()) / static_cast<float>(LocalizedAssets.Num()) * 100.0f;
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "%d out of %d localized assets are orphaned. %.2f%% of all localized assets are orphaned.", OrphanedLocalizedAssets.Num(), LocalizedAssets.Num(), OrphanedPercentage);

	if (OrphanedLocalizedAssets.Num() == 0)
	{
		UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "No orphaned localized assets found.");
		return 0;
	}

	// We sort the soft object paths first to cluster all related assets together 
	OrphanedLocalizedAssets.Sort([](const FSoftObjectPath& A, const FSoftObjectPath& B)
	{
		int32 Diff = FCString::Strcmp(*A.GetAssetName(), *B.GetAssetName());
		// localized variants will share identical asset names, we want to sort by package path to cluster all the locales together in that case 
		if (Diff == 0)
		{
			return A.GetLongPackageName() < B.GetLongPackageName();
		}
		return (Diff < 0);
	});
	TArray<FString> OrphanedLocalizedAssetsStrings;
	OrphanedLocalizedAssetsStrings.Reserve(OrphanedLocalizedAssets.Num());
	for (const FSoftObjectPath& OrphanedAsset : OrphanedLocalizedAssets)
	{
		OrphanedLocalizedAssetsStrings.Add(OrphanedAsset.ToString());
	}

	if (ParamVals.Contains("OutputOrphans"))
	{
		UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "An output file was provided. Dumping all found orphaned assets to the file: %ls", *ParamVals["OutputOrphans"]);
		FFileHelper::SaveStringArrayToFile(OrphanedLocalizedAssetsStrings, *ParamVals["OutputOrphans"]);
	}
	UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Display, "Dumping %d found orphaned assets to the console.", OrphanedLocalizedAssetsStrings.Num());
	for (const FString& OrphanedLocalizedAssetString : OrphanedLocalizedAssetsStrings)
	{
		UE_LOGF(LogDetectOrphanedLocalizedAssetsCommandlet, Warning, "'%ls' is an orphaned localized asset. Either delete the asset or re-parent the localized asset to its source asset.", *OrphanedLocalizedAssetString);
	}

	return 0;
}
