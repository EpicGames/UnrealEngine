// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DiffAssetRegistriesCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"

#include "UObject/Class.h"
#include "PlatformInfo.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Serialization/ArrayReader.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Regex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DiffAssetRegistriesCommandlet)

DEFINE_LOG_CATEGORY(LogDiffAssets);

void UDiffAssetRegistriesCommandlet::PopulateChangelistMap(const FString &Branch, const FString &CL, bool bEnginePackages)
{
	FString FilePattern = FString::Printf(TEXT("%s%s@*.p4cache"), *FPaths::DiffDir(), *Branch);
	FString FileName = FString::Printf(TEXT("%s%s@%s.p4cache"), *FPaths::DiffDir(), *Branch, *CL);

	TArray<FString> CacheFiles;

	IFileManager::Get().FindFiles( CacheFiles, *FilePattern, true, false );

	// search through the list of cache files for the newest one we can use
	int32 CLNum;
	LexTryParseString<int32>(CLNum, *CL);

	int32 BestCLNum = 0;
	FString BestCache;

	for (TArray<FString>::TConstIterator It(CacheFiles); It; ++It)
	{
		FString TestCL;
		FPaths::GetBaseFilename(*It).Split(TEXT("@"), nullptr, &TestCL);
		int32 TestCLNum;
		LexTryParseString<int32>(TestCLNum, *TestCL);

		if (TestCLNum <= CLNum && TestCLNum > BestCLNum)
		{
			BestCLNum = TestCLNum;
			BestCache = *It;
		}
	}

	FArchive* CacheFile = nullptr;

	// read in the baseline from the newest one
	if (BestCLNum > 0)
	{
		CacheFile = IFileManager::Get().CreateFileReader(*(FPaths::DiffDir() / BestCache));
		*CacheFile << AssetPathToChangelist;
		delete CacheFile;
	}

	FString CLRange = CL;

	// grab the newer file lists, or all of them if we had no best one, and merge them in
	if (BestCLNum < CLNum)
	{
		if (BestCLNum > 0)
			CLRange = FString::Printf(TEXT("%d,%d"), BestCLNum, CLNum);

		// skip the game packages if we're doing engine packages only
		if (!bEnginePackages)
		{
			FillChangelists(Branch, CLRange, P4GameBasePath, P4GameAssetPath);
		}
		FillChangelists(Branch, CLRange, P4EngineBasePath, P4EngineAssetPath);
	}

	// save out the new table
	if (BestCLNum != CLNum)
	{
		CacheFile = IFileManager::Get().CreateFileWriter(*FileName);
		*CacheFile << AssetPathToChangelist;

		delete CacheFile;
	}
}

int32 UDiffAssetRegistriesCommandlet::Main(const FString& FullCommandLine)
{
	UE_LOGF(LogDiffAssets, Display, "--------------------------------------------------------------------------------------------");
	UE_LOGF(LogDiffAssets, Display, "Running DiffAssetRegistries Commandlet");

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;

	ParseCommandLine(*FullCommandLine, Tokens, Switches, Params);

	DiffChunkID = -1;
	bIsVerbose = Switches.Contains(TEXT("VERBOSE"));
	{
		bSaveCSV = Switches.Contains(TEXT("CSV"));
		FString CSVName;
		bSaveCSV |= FParse::Value(*FullCommandLine, TEXT("CSVName="), CSVName);
		FString CSVPath;
		bSaveCSV |= FParse::Value(*FullCommandLine, TEXT("CSVPath="), CSVPath);

		if (bSaveCSV)
		{
			if (CSVFilename.IsEmpty() && !CSVName.IsEmpty())
			{
				CSVFilename = FString::Printf(TEXT("%s%s"), *FPaths::DiffDir(), *CSVName);
			}
			if (CSVFilename.IsEmpty())
			{
				CSVFilename = CSVPath;
			}
			if (CSVFilename.IsEmpty())
			{
				CSVFilename = FPaths::Combine( *FPaths::DiffDir(), TEXT("AssetChanges.csv"));
			}
			if (FPaths::GetExtension(CSVFilename).IsEmpty())
			{
				CSVFilename += TEXT(".csv");
			}
		}
	}

	// options to ignore small changes/sizes
	FParse::Value(*FullCommandLine, TEXT("MinChanges="), MinChangeCount);
	FParse::Value(*FullCommandLine, TEXT("MinChangeSize="), MinChangeSizeMB);
	FParse::Value(*FullCommandLine, TEXT("ChunkID="), DiffChunkID);
	FParse::Value(*FullCommandLine, TEXT("WarnPercentage="), WarnPercentage);
	FParse::Value(*FullCommandLine, TEXT("WarnSizeMin="), WarnSizeMinMB);
	FParse::Value(*FullCommandLine, TEXT("WarnTotalChangedSize="), WarnTotalChangedSizeMB);

	FString OldPath;
	FString NewPath;

	const bool bUseSourceGuid = Switches.Contains(TEXT("SOURCEGUID"));
	const bool bConsistency = Switches.Contains(TEXT("CONSISTENCY"));
	const bool bEnginePackages = Switches.Contains(TEXT("ENGINEPACKAGES"));
	bGroupByChunk = Switches.Contains(TEXT("GROUPBYCHUNK"));

	FString SortOrder;
	FParse::Value(*FullCommandLine, TEXT("Sort="), SortOrder);

	if (SortOrder == TEXT("name"))
	{
		ReportedFileOrder = SortOrder::ByName;
	}
	else if (SortOrder == TEXT("size"))
	{
		ReportedFileOrder = SortOrder::BySize;
	}
	else if (SortOrder == TEXT("class"))
	{
		ReportedFileOrder = SortOrder::ByClass;
	}
	else if (SortOrder == TEXT("change"))
	{
		ReportedFileOrder = SortOrder::ByChange;
	}



	FString Branch;
	FString CL;
	FString Spec;

	FParse::Value(*FullCommandLine, TEXT("platform="), TargetPlatform);

	if (TargetPlatform.IsEmpty())
	{
		UE_LOGF(LogDiffAssets, Error, "No platform specified on the commandline use \"-platform=<platform>\".");
	}

	TArray<FString> LocalSearchPaths = AssetRegistrySearchPath;
	LocalSearchPaths.AddUnique(TEXT("[buildversion]"));

	auto FindAssetRegistryPath = [&](const FString& PathVal, FString& OutPath) {
			for (const FString& SearchPath : LocalSearchPaths)
			{
				FString FinalSearchPath = SearchPath;
				FinalSearchPath.ReplaceInline(TEXT("[buildversion]"), *PathVal);
				FinalSearchPath.ReplaceInline(TEXT("[platform]"), *TargetPlatform);
				if (IFileManager::Get().FileExists(*FinalSearchPath))
				{
					OutPath = FinalSearchPath;
					return true;
				}
			}
			return false;
		};

	const FString* OldPathVal = Params.Find(FString(TEXT("OldPath")));
	if (OldPathVal)
	{
		FindAssetRegistryPath(*OldPathVal, OldPath);
	}
	else
	{
		UE_LOGF(LogDiffAssets, Error, "No old path specified \"-oldpath=<>\", use full path to asset registry or build version.");
		return -1;
	}

	const FString* NewPathVal = Params.Find(FString(TEXT("NewPath")));
	if (NewPathVal)
	{
		FindAssetRegistryPath(*NewPathVal, NewPath);

		if (!RegexBranchCL.IsEmpty())
		{
			const FRegexPattern CLPattern(RegexBranchCL);
			FRegexMatcher CLMatcher(CLPattern, NewPath);
			if (CLMatcher.FindNext())
			{
				Branch = CLMatcher.GetCaptureGroup(1);
				CL = CLMatcher.GetCaptureGroup(2);
			}
		}
	}
	else
	{
		UE_LOGF(LogDiffAssets, Error, "No new path specified \"-newpath=<>\", use full path to asset registry or build version.");
		return -1;
	}

	bMatchChangelists = false;
	if (FParse::Value(*FullCommandLine, TEXT("Branch="), Spec))
	{
		FString NewBranch, NewCL;
		
		bMatchChangelists = true;
		
		Spec.Split(TEXT("@"), &NewBranch, &NewCL);
		
		bMatchChangelists = true;
		
		PopulateChangelistMap(NewBranch, NewCL, bEnginePackages);
	}
	else if (Switches.Contains(TEXT("CHANGELISTS")))
	{
		bMatchChangelists = true;
		PopulateChangelistMap(Branch, CL, bEnginePackages);
	}

	if (OldPath.IsEmpty())
	{
		UE_LOGF(LogDiffAssets, Error, "Unable to locate AssetRegistry.bin for supplied oldpath (%ls), use full path to asset registry or build version.", **OldPathVal);
		return -1;
	}
	if (NewPath.IsEmpty())
	{
		UE_LOGF(LogDiffAssets, Error, "Unable to locate AssetRegistry.bin for supplied newpath (%ls), use full path to asset registry or build version.", **NewPathVal);
		return -1;
	}

	FPaths::NormalizeFilename(NewPath);
	FPaths::NormalizeFilename(OldPath);

	// try to discern platform
	/*FString AssetRegistrySubPath = FString::Printf(TEXT("/Metadata/%s"), GetDevelopmentAssetRegistryFilename());
	if (NewPath.Contains(AssetRegistrySubPath))
	{
		FString NewPlatformDir = NewPath.Left(NewPath.Find(AssetRegistrySubPath));
		FString PlatformPath = FPaths::GetCleanFilename(NewPlatformDir);

		for (const FPlatformInfo& PlatformInfo : PlatformInfo::GetPlatformInfoArray())
		{
			if (PlatformPath == PlatformInfo.TargetPlatformName.ToString())
			{
				TargetPlatform = PlatformPath;
				break;
			}
		}
	}*/

	if (bConsistency)
	{
		ConsistencyCheck(OldPath, NewPath);
	}
	else
	{
		DiffAssetRegistries(OldPath, NewPath, bUseSourceGuid, bEnginePackages);
	}
	UE_LOGF(LogDiffAssets, Display, "Successfully finished running DiffAssetRegistries Commandlet");
	UE_LOGF(LogDiffAssets, Display, "--------------------------------------------------------------------------------------------");
	return 0;
}

void UDiffAssetRegistriesCommandlet::FillChangelists(FString Branch, FString CL, FString BasePath, FString AssetPath)
{
	TArray<FString> Results;
	int32 ReturnCode = 0;
	if (LaunchP4(TEXT("files ") + P4Repository + Branch + BasePath + TEXT("....uasset@") + CL, Results, ReturnCode))
	{
		if (ReturnCode == 0)
		{
			for (const FString& Result : Results)
			{
				FString DepotPathName;
				FString ExtraInfoAfterPound;
				if (Result.Split(TEXT("#"), &DepotPathName, &ExtraInfoAfterPound))
				{
					FString PostContentPath;
					if (DepotPathName.Split(BasePath, nullptr, &PostContentPath))
					{
						if (!PostContentPath.IsEmpty() && !PostContentPath.StartsWith(TEXT("Cinematics")) &&
							!PostContentPath.StartsWith(FPaths::DevelopersFolderName()) && !PostContentPath.StartsWith(TEXT("Maps/Test_Maps")))
						{
							const FString PostContentPathWithoutExtension = FPaths::GetBaseFilename(PostContentPath, false);
							const FString FullPackageName = AssetPath + PostContentPathWithoutExtension;
							
							TArray<FString> Chunks;
							
							ExtraInfoAfterPound.ParseIntoArray(Chunks, TEXT(" "), true);
							
							int32 Changelist;
							
							LexTryParseString<int32>(Changelist, *Chunks[4]);
							if (Changelist)
							{
								int32& Entry = AssetPathToChangelist.FindOrAdd(*FullPackageName);

								if (Changelist > Entry)
									Entry = Changelist;
							}
						}
					}
				}
			}
		}
	}
	if (LaunchP4(TEXT("files ") + P4Repository + Branch + BasePath + TEXT("....umap@") + CL, Results, ReturnCode))
	{
		if (ReturnCode == 0)
		{
			for (const FString& Result : Results)
			{
				FString DepotPathName;
				FString ExtraInfoAfterPound;
				if (Result.Split(TEXT("#"), &DepotPathName, &ExtraInfoAfterPound))
				{
					FString PostContentPath;
					if (DepotPathName.Split(BasePath, nullptr, &PostContentPath))
					{
						if (!PostContentPath.IsEmpty() && !PostContentPath.StartsWith(TEXT("Cinematics")) &&
							!PostContentPath.StartsWith(FPaths::DevelopersFolderName()) && !PostContentPath.StartsWith(TEXT("Maps/Test_Maps")))
						{
							const FString PostContentPathWithoutExtension = FPaths::GetBaseFilename(PostContentPath, false);
							const FString FullPackageName = AssetPath + PostContentPathWithoutExtension;
							
							TArray<FString> Chunks;
							
							ExtraInfoAfterPound.ParseIntoArray(Chunks, TEXT(" "), true);
							
							int32 Changelist;
							
							LexTryParseString<int32>(Changelist, *Chunks[4]);
							if (Changelist)
							{
								int32& Entry = AssetPathToChangelist.FindOrAdd(*FullPackageName);

								if (Changelist > Entry)
									Entry = Changelist;
							}
						}
					}
				}
			}
		}
	}
}

void UDiffAssetRegistriesCommandlet::ConsistencyCheck(const FString& OldPath, const FString& NewPath)
{
	{
		FArrayReader SerializedAssetData;

		if (!IFileManager::Get().FileExists(*OldPath))
		{
			UE_LOGF(LogDiffAssets, Error, "File '%ls' does not exist.", *OldPath);
			return;
		}
		if (!FFileHelper::LoadFileToArray(SerializedAssetData, *OldPath))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to load file '%ls'.", *OldPath);
			return;
		}
		if (!OldState.Load(SerializedAssetData))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to parse file '%ls' as asset registry.", *OldPath);
			return;
		}
	}
	{
		FArrayReader SerializedAssetData;

		if (!IFileManager::Get().FileExists(*NewPath))
		{
			UE_LOGF(LogDiffAssets, Error, "File '%ls' does not exist.", *NewPath);
			return;
		}
		if (!FFileHelper::LoadFileToArray(SerializedAssetData, *NewPath))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to load file '%ls'.", *NewPath);
			return;
		}

		if (!NewState.Load(SerializedAssetData))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to parse file '%ls' as asset registry.", *NewPath);
			return;
		}
	}
	UE_LOGF(LogDiffAssets, Display, "Comparing asset registries '%ls' and '%ls'.", *OldPath, *NewPath);
	UE_LOGF(LogDiffAssets, Display, "Source vs Cooked Consistency Diff");
	if (bIsVerbose)
	{
		UE_LOGF(LogDiffAssets, Display, "Cooked files that differ, where source guids do not:");
	}
	// We're looking for packages that the Cooked check says are modified, but that the Guid check says are not
	// We're ignoring new packages for this, as those are obviously going to change
	TSet<FName> HashModified, CookModified;
	TSet<FName> New;

	for (const TPair<FName, const FAssetPackageData*>& Pair : NewState.GetAssetPackageDataMap())
	{
		FName Name = Pair.Key;
		const FAssetPackageData* Data = Pair.Value;
		const FAssetPackageData* PrevData = OldState.GetAssetPackageData(Name);

		if (!PrevData)
		{
			New.Add(Name);
		}
		else
		{
			if (Data->GetPackageSavedHash() != PrevData->GetPackageSavedHash())
			{
				HashModified.Add(Name);
			}
			if (Data->CookedHash != PrevData->CookedHash)
			{
				CookModified.Add(Name);
			}
		}
	}

	// recurse through the referencer lists to fill out HashModified
	TArray<FName> Recurse = HashModified.Array();
	
	for (int32 RecurseIndex = 0; RecurseIndex < Recurse.Num(); RecurseIndex++)
	{
		FName Package = Recurse[RecurseIndex];
		TArray<FAssetIdentifier> Referencers;
		NewState.GetReferencers(Package, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
		
		for (const FAssetIdentifier& Referencer : Referencers)
		{
			FName ReferencerPackage = Referencer.PackageName;
			if (!New.Contains(ReferencerPackage) && !HashModified.Contains(ReferencerPackage))
			{
				HashModified.Add(ReferencerPackage);
				Recurse.Add(ReferencerPackage);
			}
		}
	}

	int64 Changes = 0;
	int64 ChangeBytes = 0;

	// find all entries of CookModified that do not exist in HashModified
	const TMap<FName, const FAssetPackageData*>& PackageMap = NewState.GetAssetPackageDataMap();
	for (FName const &Package : CookModified)
	{
		const FAssetPackageData* Data = PackageMap[Package];

		if (!HashModified.Contains(Package) && Data->DiskSize >= 0)
		{
			++Changes;
			ChangeBytes += Data->DiskSize;
			if (bIsVerbose)
			{
				UE_LOGF(LogDiffAssets, Display, "%ls : %lld bytes", *Package.ToString(), Data->DiskSize);
			}
		}
	}

	double ChangeValue = 0.0;
	int32 ChangeExp = 0;

	auto Rescale = [](int64 Bytes, double& Value, int32& Exp)
	{
		Value = Bytes;
		Value = fabs(Value);

		while (Value > 1024.0)
		{
			Value /= 1024.0;
			Exp++;
		}
	};

	Rescale(ChangeBytes, ChangeValue, ChangeExp);

	UE_LOGF(LogDiffAssets, Display, "Summary:");
	UE_LOGF(LogDiffAssets, Display, "%lld nondeterministic cooks, %8.3f %cB", Changes, ChangeValue, TCHAR(" KMGTP"[ChangeExp]));
}

bool	UDiffAssetRegistriesCommandlet::IsInRelevantChunk(FAssetRegistryState& InRegistryState, FName InAssetPath)
{
	if (DiffChunkID == -1)
	{
		return true;

	}
	TArray<const FAssetData*> Assets = InRegistryState.CopyAssetsByPackageName(InAssetPath);
	if (!Assets.IsEmpty())
	{
		const FAssetData::FChunkArrayView ChunkIDs = Assets[0]->GetChunkIDs();
		if (!ChunkIDs.IsEmpty())
		{
			return ChunkIDs.Contains(DiffChunkID);
		}
	}

	return true;
}

FName UDiffAssetRegistriesCommandlet::GetClassName(FAssetRegistryState& InRegistryState, FName InAssetPath)
{
	if (AssetPathToClassName.Contains(InAssetPath) == false)
	{
		TArray<const FAssetData*> Assets = InRegistryState.CopyAssetsByPackageName(InAssetPath);

		FName NewName;
		if (Assets.Num() > 0)
		{
			NewName = Assets[0]->AssetClassPath.GetAssetName();
		}
		else
		{
			if (InAssetPath.ToString().StartsWith(TEXT("/Script/")))
			{
				NewName = NAME_Class;
			}
		}

		if (NewName == NAME_None)
		{
			UE_LOGF(LogDiffAssets, Log, "Unable to find class type of asset %ls", *InAssetPath.ToString());
		}
		AssetPathToClassName.Add(InAssetPath) = NewName;
	}

	return AssetPathToClassName[InAssetPath];
}

TArray<int32> UDiffAssetRegistriesCommandlet::GetAssetChunks(FAssetRegistryState& InRegistryState, FName InAssetPath)
{
	if (ChunkIdByAssetPath.Contains(InAssetPath) == false)
	{
		TArray<const FAssetData* > Assets = InRegistryState.CopyAssetsByPackageName(InAssetPath);
		const FAssetData::FChunkArrayView ChunkIDs = Assets.IsEmpty() ? FAssetData::FChunkArrayView() : Assets[0]->GetChunkIDs();
		if (!ChunkIDs.IsEmpty())
		{
			if (ChunkIDs.Num() > 1)
			{
				UE_LOGF(LogDiffAssets, Log, "Multiple ChunkIds for asset %ls", *InAssetPath.ToString());
			}

			for (int32 id : ChunkIDs)
			{
				ChangesByChunk.FindOrAdd(id).IncludedAssets.Add(InAssetPath);
				ChunkIdByAssetPath.Add(InAssetPath, id);
			}
		}
		else
		{
			UE_LOGF(LogDiffAssets, Log, "Unable to find chunk ids of asset %ls", *InAssetPath.ToString());
			ChunkIdByAssetPath.Add(InAssetPath, -1);
		}
	}

	TArray<int32> ChunkIds;
	ChunkIdByAssetPath.MultiFind(InAssetPath, ChunkIds);
	return ChunkIds;
}

void UDiffAssetRegistriesCommandlet::RecordAdd(FName InAssetPath, const FAssetPackageData& InNewData)
{
	FChangeInfo AssetChange;

	++AssetChange.Adds;
	if (InNewData.DiskSize > 0)
	{
		AssetChange.AddedBytes += InNewData.DiskSize;
	}

	FName ClassName = GetClassName(NewState, InAssetPath);
	TArray<int32> ChunkIds = GetAssetChunks(NewState, InAssetPath);

	int changelist = AssetPathToChangelist.FindOrAdd(*InAssetPath.ToString());

	ChangeInfoByAsset.FindOrAdd(InAssetPath) = AssetChange;
	ChangeSummaryByClass.FindOrAdd(ClassName) += AssetChange;
	ChangeSummaryByChangelist.FindOrAdd(changelist) += AssetChange;
	for (int32 ChunkId : ChunkIds)
	{
		ChangesByChunk.FindOrAdd(ChunkId).ChangesByClass.FindOrAdd(ClassName) += AssetChange;
	}
	ChangeSummary += AssetChange;
}

void UDiffAssetRegistriesCommandlet::RecordEdit(FName InAssetPath, const FAssetPackageData& InNewData, const FAssetPackageData& InOldData)
{
	FChangeInfo AssetChange;

	if (InNewData.DiskSize > 0)
	{
		++AssetChange.Changes;
		AssetChange.ChangedBytes += InNewData.DiskSize;
	}

	FName ClassName = GetClassName(NewState, InAssetPath);
	TArray<int32> ChunkIds = GetAssetChunks(NewState, InAssetPath);

	int changelist = AssetPathToChangelist.FindOrAdd(*InAssetPath.ToString());

	ChangeInfoByAsset.FindOrAdd(InAssetPath) = AssetChange;
	ChangeSummaryByClass.FindOrAdd(ClassName) += AssetChange;
	ChangeSummaryByChangelist.FindOrAdd(changelist) += AssetChange;
	for (int32 ChunkId : ChunkIds)
	{
		ChangesByChunk.FindOrAdd(ChunkId).ChangesByClass.FindOrAdd(ClassName) += AssetChange;
	}
	ChangeSummary += AssetChange;
}

void UDiffAssetRegistriesCommandlet::RecordDelete(FName InAssetPath, const FAssetPackageData& InData)
{
	FChangeInfo AssetChange;

	++AssetChange.Deletes;

	if (InData.DiskSize >= 0)
	{
		AssetChange.DeletedBytes += InData.DiskSize;
	}

	FName ClassName = GetClassName(OldState, InAssetPath);
	TArray<int32> ChunkIds = GetAssetChunks(NewState, InAssetPath);

	ChangeInfoByAsset.FindOrAdd(InAssetPath) = AssetChange;
	ChangeSummaryByClass.FindOrAdd(ClassName) += AssetChange;
	for (int32 ChunkId : ChunkIds)
	{
		ChangesByChunk.FindOrAdd(ChunkId).ChangesByClass.FindOrAdd(ClassName) += AssetChange;
	}
	ChangeSummary += AssetChange;
}

void UDiffAssetRegistriesCommandlet::RecordNoChange(FName InAssetPath, const FAssetPackageData& InData)
{
	FChangeInfo AssetChange;

	AssetChange.Unchanged++;

	if (InData.DiskSize >= 0)
	{
		AssetChange.UnchangedBytes += InData.DiskSize;
	}

	FName ClassName = GetClassName(NewState, InAssetPath);
	TArray<int32> ChunkIds = GetAssetChunks(NewState, InAssetPath);

	ChangeInfoByAsset.FindOrAdd(InAssetPath) = AssetChange;
	ChangeSummaryByClass.FindOrAdd(ClassName) += AssetChange;
	for (int32 ChunkId : ChunkIds)
	{
		ChangesByChunk.FindOrAdd(ChunkId).ChangesByClass.FindOrAdd(ClassName) += AssetChange;
	}
	ChangeSummary += AssetChange;
}

void UDiffAssetRegistriesCommandlet::SummarizeDeterminism()
{
	TArray<FName> AssetPaths;
	ChangeInfoByAsset.GetKeys(AssetPaths);

	for (const FName& AssetPath : AssetPaths)
	{
		const FChangeInfo& ChangeInfo = ChangeInfoByAsset[AssetPath];

		char classification;

		// classify the asset change by the flags
		int32 flags = AssetPathFlags.FindOrAdd(AssetPath);
		{
			bool hash = (flags & EAssetFlags::HashChange) != 0;
			bool guid = (flags & EAssetFlags::GuidChange) != 0;
			bool dephash = (flags & EAssetFlags::DepHashChange) != 0;
			bool depguid = (flags & EAssetFlags::DepGuidChange) != 0;

			if (!hash)
				classification = 'x'; // shouldn't see this in here, no binary change
			else 
			{
				if (guid)
					classification = 'e'; // explicit edit
				else if (dephash & depguid)
					classification = 'd'; // dependency edit
				else if (dephash & !depguid)
					classification = 'n'; // nondeterministic dependency
				else
					classification = 'c'; // nondeterministic
			}
		}

		TArray<int32> ChunkIds = GetAssetChunks(NewState, AssetPath);
		FName ClassName = GetClassName(NewState, AssetPath);
		if (classification == 'c')
		{
			NondeterministicSummary += ChangeInfo;
			
			for (int32 ChunkId : ChunkIds)
			{
				ChangesByChunk.FindOrAdd(ChunkId).Determinism.FindOrAdd(ClassName).AddDirect(ChangeInfo);
			}
			DeterminismByClass.FindOrAdd(ClassName).AddDirect(ChangeInfo);
		}
		else if (classification == 'n')
		{
			IndirectNondeterministicSummary += ChangeInfo;

			for (int32 ChunkId : ChunkIds)
			{
				ChangesByChunk.FindOrAdd(ChunkId).Determinism.FindOrAdd(ClassName).AddIndirect(ChangeInfo);
			}
			DeterminismByClass.FindOrAdd(ClassName).AddIndirect(ChangeInfo);
		}
	}
}

void UDiffAssetRegistriesCommandlet::LogChangedFiles(FArchive *CSVFile, FString const &OldPath, FString const &NewPath)
{
	if (!bIsVerbose && !bSaveCSV)
	{
		return;
	}

	TArray<FName> AssetPaths;
	ChangeInfoByAsset.GetKeys(AssetPaths);

	// sort by size
	if (ReportedFileOrder == SortOrder::BySize)
	{
		// Sort by size of change
		AssetPaths.Sort([this](const FName& Lhs, const FName& Rhs) {
			return ChangeInfoByAsset[Lhs].GetTotalChangeSize() > ChangeInfoByAsset[Rhs].GetTotalChangeSize();
		});
	}
	// Sort by class type then size size
	else if (ReportedFileOrder == SortOrder::ByClass)
	{
		AssetPaths.Sort([this](const FName& Lhs, const FName& Rhs) {
			FString LhsName = GetClassName(NewState, Lhs).ToString();
			FString RhsName = GetClassName(NewState, Rhs).ToString();

			if (LhsName != RhsName)
			{
				return LhsName < RhsName;
			}

			return ChangeInfoByAsset[Lhs].GetTotalChangeSize() > ChangeInfoByAsset[Rhs].GetTotalChangeSize();
		});
	}
	// sort by change type then size
	else if (ReportedFileOrder == SortOrder::ByChange)
	{
		AssetPaths.Sort([this](const FName& Lhs, const FName& Rhs) {

			int32 LHSChanges = ChangeInfoByAsset[Lhs].GetChangeFlags();
			int32 RHSChanges = ChangeInfoByAsset[Rhs].GetChangeFlags();

			if (LHSChanges != RHSChanges)
			{
				return LHSChanges > RHSChanges;
			}

			// sort by size
			return ChangeInfoByAsset[Lhs].GetTotalChangeSize() > ChangeInfoByAsset[Rhs].GetTotalChangeSize();
		});
	}
	// sort by name
	else if (ReportedFileOrder == SortOrder::ByName)
	{
		AssetPaths.Sort([this](const FName& Lhs, const FName& Rhs) {
			return Lhs.ToString() < Rhs.ToString();
		});
	}

	if (CSVFile)
	{
		CSVFile->Logf(TEXT("Type Key"));
		CSVFile->Logf(TEXT("a, file added"));
		CSVFile->Logf(TEXT("r, file removed"));
		CSVFile->Logf(TEXT("e, explicit edit (this file specifically has been modified)"));
		CSVFile->Logf(TEXT("d, dependency edit (this file is different likely because a dependency has also been changed)"));
		CSVFile->Logf(TEXT("n, indirect non deterministic (a dependency file changed but wasn't changed directly (Indicates the dependency was either non determinisitc or another indirect non deterministic file))"));
		CSVFile->Logf(TEXT("c, non deterministic (the hashes for all dependencies are the same but this file is not)"));
		CSVFile->Logf(TEXT("x, no binary change (shouldn't ever happen)"));
		CSVFile->Logf(TEXT(""));

		CSVFile->Logf(TEXT("Modification,Name,Class,NewSize,OldSize,Changelist,Chunk"));

		UE_LOGF(LogDiffAssets, Display, "Saving CSV results to %ls", *CSVFilename);
	}

	for (const FName& AssetPath : AssetPaths)
	{
		const FChangeInfo& ChangeInfo = ChangeInfoByAsset[AssetPath];

		int Changelist = bMatchChangelists ? AssetPathToChangelist.FindOrAdd(*AssetPath.ToString()) : 0;
		
		FName ClassName;

		if (ChangeInfo.Deletes)
		{
			ClassName = GetClassName(OldState, AssetPath);
		}
		else
		{
			ClassName = GetClassName(NewState, AssetPath);
		}
		auto GetChunkIDString = [this, AssetPath=AssetPath](FAssetRegistryState& State) {
				TArray<int32> ChunkIds = GetAssetChunks(State, AssetPath);
				FString ChunkIdString = FString::JoinBy(ChunkIds, TEXT(" & "), [](int32 Value) { return FString::Printf(TEXT("%d"), Value); });
				return ChunkIdString;
			};
		

		if (ChangeInfo.Adds)
		{
			if (CSVFile)
			{
				CSVFile->Logf(TEXT("a,%s,%s,%lld,0,%d,%s"), *AssetPath.ToString(), *ClassName.ToString(), ChangeInfo.AddedBytes, Changelist, *GetChunkIDString(NewState));
			}

			if (bIsVerbose)
			{
				UE_LOGF(LogDiffAssets, Display, "a %ls : (Class=%ls,NewSize=%lld bytes)", *AssetPath.ToString(), *ClassName.ToString(), ChangeInfo.AddedBytes);
			}
		}
		else if (ChangeInfo.Changes)
		{
			const FAssetPackageData* PrevData = OldState.GetAssetPackageData(AssetPath);

			char classification;

			// classify the asset change by the flags
			int32 flags = AssetPathFlags.FindOrAdd(AssetPath);
			{
				bool hash = (flags & EAssetFlags::HashChange) != 0;
				bool guid = (flags & EAssetFlags::GuidChange) != 0;
				bool dephash = (flags & EAssetFlags::DepHashChange) != 0;
				bool depguid = (flags & EAssetFlags::DepGuidChange) != 0;

				if (!hash)
					classification = 'x'; // shouldn't see this in here, no binary change
				else 
				{
					if (guid)
						classification = 'e'; // explicit edit
					else if (dephash & depguid)
						classification = 'd'; // dependency edit
					else if (dephash & !depguid)
						classification = 'n'; // nondeterministic dependency
					else
						classification = 'c'; // nondeterministic
				}
			}

			if (CSVFile)
			{
				CSVFile->Logf(TEXT("%c,%s,%s,%lld,%lld,%d,%s"), TCHAR(classification), *AssetPath.ToString(), *ClassName.ToString(), ChangeInfo.ChangedBytes, PrevData->DiskSize, Changelist, *GetChunkIDString(NewState));
			}

			if (bIsVerbose)
			{
				UE_LOGF(LogDiffAssets, Display, "%c %ls : (Class=%ls,NewSize=%lld bytes,OldSize=%lld bytes)", TCHAR(classification), *AssetPath.ToString(), *ClassName.ToString(), ChangeInfo.ChangedBytes, PrevData->DiskSize);
			}
		}
		else if (ChangeInfo.Deletes)
		{
			const FAssetPackageData* PrevData = OldState.GetAssetPackageData(AssetPath);

			if (CSVFile)
			{
				CSVFile->Logf(TEXT("r,%s,%s,0,%lld,0,0"), *AssetPath.ToString(), *ClassName.ToString(), PrevData->DiskSize);
			}

			if (bIsVerbose)
			{
				UE_LOGF(LogDiffAssets, Display, "r %ls : (Class=%ls,OldSize=%lld bytes)", *AssetPath.ToString(), *ClassName.ToString(), PrevData->DiskSize);
			}
		}
	}
}

void UDiffAssetRegistriesCommandlet::LogClassSummary(FArchive *CSVFile, const FString& HeaderPrefix, const TMap<FName, FChangeInfo>& InChangeInfoByAsset, bool bDoWarnings, TMap<FName, FDeterminismInfo> DeterminismInfo)
{
	const float InvToMB = 1.0 / (1024 * 1024);
	FString HeaderSpacer = (HeaderPrefix.Len() ? TEXT(" ") : TEXT(""));

	// show class totals first
	TArray<FName> ClassNames;
	InChangeInfoByAsset.GetKeys(ClassNames);

	// Sort keys by the desired order
	if (ReportedFileOrder == SortOrder::ByName || ReportedFileOrder == SortOrder::ByClass)
	{
		ClassNames.Sort([](const FName& Lhs, const FName& Rhs) {
			return Lhs.ToString() < Rhs.ToString();
		});

	}
	else // Default to size for everything else for class list
	{
		// sort by size of changes (number can also be a big impact on patch size but depends on datalayout and patch algo..)
		ClassNames.Sort([this, InChangeInfoByAsset](const FName& Lhs, const FName& Rhs) {
			const FChangeInfo& LHSChanges = InChangeInfoByAsset[Lhs];
			const FChangeInfo& RHSChanges = InChangeInfoByAsset[Rhs];
			return LHSChanges.GetTotalChangeSize() > RHSChanges.GetTotalChangeSize();
		});
	}

	int64 TotalChangesSize = 0;
	int64 TotalSize = 0;

	//Overall Class Summary
	bool bChangesPastThreshold = false;
	for (FName ClassName : ClassNames)
	{
		const FChangeInfo& Changes = InChangeInfoByAsset[ClassName];

		TotalSize += Changes.GetTotalSize();
		TotalChangesSize += Changes.GetTotalChangeSize();

		if (Changes.GetTotalChangeSize() == 0)
		{
			continue;
		}

		if (Changes.GetTotalChangeCount() < MinChangeCount || Changes.GetTotalChangeSize() < (MinChangeSizeMB * 1024 * 1024))
		{
			continue;
		}
		else if (!bChangesPastThreshold)
		{
			//We'll need to display the header, since this is the first row that has sufficient changes
			if (CSVFile)
			{
				FString ExtraNondeterminismHeader = TEXT("");
				if (DeterminismInfo.Num())
				{
					ExtraNondeterminismHeader = TEXT(",DirectNondeterministicCount,DirectNondeterministicSize,IndirectNondeterministicCount,IndirectNondeterministicSize");
				}

				CSVFile->Logf(TEXT(""));
				CSVFile->Logf(TEXT("%s%sClass Summary"), *HeaderPrefix, *HeaderSpacer);
				CSVFile->Logf(TEXT("Name,Percentage,TotalCount,TotalSize,Adds,AddedSize,Changes,ChangesSize,Deletes,DeletedSize,Unchanged,UnchangedSize%s"), *ExtraNondeterminismHeader);
			}

			bChangesPastThreshold = true;
		}

		if (CSVFile)
		{
			FString ExtraNondeterminismData = TEXT("");
			if (DeterminismInfo.Num())
			{
				int64 DirectCount = 0;
				int64 IndirectCount = 0;

				int64 DirectSize = 0;
				int64 IndirectSize = 0;
				if (DeterminismInfo.Contains(ClassName))
				{
					int64 TotalCount = Changes.GetTotalChangeCount();
					DirectCount = DeterminismInfo[ClassName].DirectCount;
					IndirectCount = DeterminismInfo[ClassName].IndirectCount;

					DirectSize = DeterminismInfo[ClassName].DirectSize;
					IndirectSize = DeterminismInfo[ClassName].IndirectSize;
				}


				ExtraNondeterminismData = FString::Printf(TEXT(",%lld,%lld,%lld,%lld"),
					DirectCount, DirectSize, IndirectCount, IndirectSize);
			}


			CSVFile->Logf(TEXT("%s,%0.02f,%lld,%lld,%lld,%lld,%lld,%lld,%lld,%lld,%lld,%lld%s"),
				*ClassName.ToString(),
				Changes.GetChangePercentage() * 100.0,
				Changes.GetTotalChangeCount(),
				Changes.GetTotalChangeSize(),
				Changes.Adds, Changes.AddedBytes,
				Changes.Changes, Changes.ChangedBytes,
				Changes.Deletes, Changes.DeletedBytes,
				Changes.Unchanged, Changes.UnchangedBytes,
				*ExtraNondeterminismData);
		}

		// log summary & change
		UE_LOGF(LogDiffAssets, Display, "%ls%lsClass Summary: ", *HeaderPrefix, *HeaderSpacer);

		UE_LOGF(LogDiffAssets, Display, "%ls: %.02f%% changes (%.02f MB Total)",
			*ClassName.ToString(), Changes.GetChangePercentage() * 100.0, Changes.GetTotalChangeSize() * InvToMB);

		if (Changes.Adds)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%lld packages added,    %8.3f MB", Changes.Adds, Changes.AddedBytes * InvToMB);
		}

		if (Changes.Changes)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%lld packages modified, %8.3f MB", Changes.Changes, Changes.ChangedBytes * InvToMB);
		}

		if (Changes.Deletes)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%lld packages removed,  %8.3f MB", Changes.Deletes, Changes.DeletedBytes * InvToMB);
		}

		UE_LOGF(LogDiffAssets, Display, "\t%lld packages unchanged,  %8.3f MB", Changes.Unchanged, Changes.UnchangedBytes * InvToMB);

		// Warn on a certain % of changes if that's enabled
		if (bDoWarnings 
			&& Changes.Changes >= 10
			&& (WarnPercentage > 0 || WarnSizeMinMB > 0)
			&& Changes.ChangedBytes * InvToMB >= WarnSizeMinMB
			&& Changes.GetChangePercentage() * 100.0 > WarnPercentage)
		{
			UE_LOGF(LogDiffAssets, Warning, "\t%ls Assets for %ls are %.02f%% changed. (%.02f MB of data)",
				*TargetPlatform, *ClassName.ToString(), Changes.GetChangePercentage() * 100.0, Changes.ChangedBytes * InvToMB);
		}
	}

	// log summary & change
	UE_LOGF(LogDiffAssets, Display, "%ls%ls FullSummary: ", *HeaderPrefix, *HeaderSpacer);
	UE_LOGF(LogDiffAssets, Display, "  : %.02f%% changes (%.02f MB Changed out of %.02f MB Total)", TotalSize ? (float)TotalChangesSize / (float)TotalSize * 100.0 : 0.0f, TotalChangesSize * InvToMB, TotalSize * InvToMB);

	//If we didn't find any changes of sufficient size, note as much instead of the header
	if (!bChangesPastThreshold)
	{
		FString SummaryName = (HeaderPrefix.Len() ? FString::Printf(TEXT("%s: "), *HeaderPrefix) : TEXT(""));

		if (CSVFile)
		{
			CSVFile->Logf(TEXT(""));
			CSVFile->Logf(TEXT("%sNo classes had changes past change threshold"), *SummaryName);
		}

		UE_LOGF(LogDiffAssets, Display, "%lsNo classes had changes past thresholds", *SummaryName);
	}
}

void UDiffAssetRegistriesCommandlet::DiffAssetRegistries(const FString& OldPath, const FString& NewPath, bool bUseSourceGuid, bool bEnginePackagesOnly)
{
	{
		FArrayReader SerializedAssetData;

		if (!IFileManager::Get().FileExists(*OldPath))
		{
			UE_LOGF(LogDiffAssets, Error, "File '%ls' does not exist.", *OldPath);
			return;
		}
		if (!FFileHelper::LoadFileToArray(SerializedAssetData, *OldPath))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to load file '%ls'.", *OldPath);
			return;
		}
		if (!OldState.Load(SerializedAssetData))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to parse file '%ls' as asset registry.", *OldPath);
			return;
		}
	}
	{
		FArrayReader SerializedAssetData;

		if (!IFileManager::Get().FileExists(*NewPath))
		{
			UE_LOGF(LogDiffAssets, Error, "File '%ls' does not exist.", *NewPath);
			return;
		}
		if (!FFileHelper::LoadFileToArray(SerializedAssetData, *NewPath))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to load file '%ls'.", *NewPath);
			return;
		}
		if (!NewState.Load(SerializedAssetData))
		{
			UE_LOGF(LogDiffAssets, Error, "Failed to parse file '%ls' as asset registry.", *NewPath);
			return;
		}
	}

	
	int64 newtotal = 0;
	int64 oldtotal = 0;
	int64 newuncooked = 0;
	int64 olduncooked = 0;
	int64 newassets = 0;
	int64 oldassets = 0;

	UE_LOGF(LogDiffAssets, Display, "Comparing asset registries '%ls' and '%ls'.", *OldPath, *NewPath);
	if (bUseSourceGuid)
	{
		UE_LOGF(LogDiffAssets, Display, "Source Package Diff");
	}
	else
	{
		UE_LOGF(LogDiffAssets, Display, "Cooked Package Diff");
	}
	if (bIsVerbose)
	{
		UE_LOGF(LogDiffAssets, Display, "Package changes:");
	}

	TSet<FName> Modified;
	TSet<FName> New;

	if (bUseSourceGuid)
	{
		for (const TPair<FName, const FAssetPackageData*>& Pair : NewState.GetAssetPackageDataMap())
		{
			FName Name = Pair.Key;

			FString NameString = Name.ToString();

			if (bEnginePackagesOnly && !NameString.StartsWith(TEXT("/Engine/")))
			{
				continue;
			}

			if (!IsInRelevantChunk(NewState, Name))
			{
				continue;
			}

			const FAssetPackageData* Data = Pair.Value;
			const FAssetPackageData* PrevData = OldState.GetAssetPackageData(Name);
			
			if (Data->DiskSize < 0)
			{
				newuncooked++;
			}
			
			newassets += NewState.NumAssetsByPackageName(Name);
			
			if (!PrevData)
			{
				New.Add(Name);
				RecordAdd(Name, *Data);
			}
			else if (Data->GetPackageSavedHash() != PrevData->GetPackageSavedHash())
			{
				Modified.Add(Name);
			}
			else
			{
				RecordNoChange(Name, *Data);
			}
			++newtotal;
		}

		TArray<FName> Recurse = Modified.Array();

		for (int32 RecurseIndex = 0; RecurseIndex < Recurse.Num(); RecurseIndex++)
		{
			FName Package = Recurse[RecurseIndex];
			TArray<FAssetIdentifier> Referencers;
			NewState.GetReferencers(Package, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

			for (const FAssetIdentifier& Referencer : Referencers)
			{
				FName ReferencerPackage = Referencer.PackageName;
				if (!New.Contains(ReferencerPackage) && !Modified.Contains(ReferencerPackage))
				{
					Modified.Add(ReferencerPackage);
					Recurse.Add(ReferencerPackage);
				}
			}
		}

		const TMap<FName, const FAssetPackageData*>& PackageMap = NewState.GetAssetPackageDataMap();
		for (FName const &Package : Modified)
		{
			const FAssetPackageData* Data = PackageMap[Package];
			const FAssetPackageData* PrevData = OldState.GetAssetPackageData(Package);

			RecordEdit(Package, *Data,*PrevData);
		}
	}
	else
	{
		for (const TPair<FName, const FAssetPackageData*>& Pair : NewState.GetAssetPackageDataMap())
		{
			FName Name = Pair.Key;

			if (bEnginePackagesOnly && !Name.ToString().StartsWith(TEXT("/Engine/")))
			{
				continue;
			}

			if (!IsInRelevantChunk(NewState, Name))
			{
				continue;
			}

			const FAssetPackageData* Data = Pair.Value;
			const FAssetPackageData* PrevData = OldState.GetAssetPackageData(Name);
			
			if (Data->DiskSize < 0)
			{
				newuncooked++;
			}
			
			newassets += NewState.NumAssetsByPackageName(Name);
			
			if (!PrevData)
			{
				RecordAdd(Name, *Data);
				AssetPathFlags.FindOrAdd(Name) |= EAssetFlags::Add;
			}
			else if (Data->CookedHash != PrevData->CookedHash)
			{
				RecordEdit(Name, *Data, *PrevData);
				AssetPathFlags.FindOrAdd(Name) |= EAssetFlags::HashChange;
				if (Data->GetPackageSavedHash() != PrevData->GetPackageSavedHash())
				{
					AssetPathFlags.FindOrAdd(Name) |= EAssetFlags::GuidChange;
				}
			}
			else
			{
				RecordNoChange(Name, *Data);
			}
			newtotal++;
		}
	}

	for (const TPair<FName, const FAssetPackageData*>& Pair : OldState.GetAssetPackageDataMap())
	{
		FName Name = Pair.Key;

		FString NameString = Name.ToString();

		if (bEnginePackagesOnly && !NameString.StartsWith(TEXT("/Engine/")))
		{
			continue;
		}

		if (!IsInRelevantChunk(OldState, Name))
		{
			continue;
		}

		const FAssetPackageData* PrevData = Pair.Value;
		const FAssetPackageData* Data = NewState.GetAssetPackageData(Name);

		if (PrevData->DiskSize < 0)
		{
			olduncooked++;
		}

		oldassets += OldState.NumAssetsByPackageName(Name);

		if (!Data)
		{
			RecordDelete(Name, *PrevData);
			AssetPathFlags.FindOrAdd(Name) |= EAssetFlags::Remove;
		}
		oldtotal++;
	}

	// Propagate hash/guid changes down through referencers
	{
		TArray<FName> Recurse;
		AssetPathFlags.GetKeys(Recurse);

		for (int32 RecurseIndex = 0; RecurseIndex < Recurse.Num(); RecurseIndex++)
		{
			FName Package = Recurse[RecurseIndex];
			
			TArray<FAssetIdentifier> Referencers;
			
			// grab the hash/guid change flags, shift up to the dependency ones
			int32 PackageFlags = AssetPathFlags.FindOrAdd(Package);
			int32 NewFlags = (PackageFlags & 0x0C) << 2;

			// If we have a dependency chain like C -> B -> A, and A changes, this does not
			// necessarily cause B's binary representation to change, but it can still impact C.
			// We must propagate the dependency change flags too, otherwise C will be marked as
			// non-deterministic when this happens.
			NewFlags |= PackageFlags & (EAssetFlags::DepGuidChange | EAssetFlags::DepHashChange);
			
			// don't bother touching anything if this asset or its dependencies didn't change
			if (NewFlags)
			{
				NewState.GetReferencers(Package, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

				for (const FAssetIdentifier& Referencer : Referencers)
				{
					FName RefPackage = Referencer.PackageName;
					
					// merge new dependency flags in, add to the list if something changed
					int32& RefFlags = AssetPathFlags.FindOrAdd(RefPackage);
					int32 OldFlags = RefFlags;
					RefFlags |= NewFlags;
					if (RefFlags != OldFlags)
					{
						Recurse.Add(RefPackage);
					}
				}
			}
		}
	}

	FArchive *CSVFile = nullptr;

	if (bSaveCSV)
	{
		CSVFile = IFileManager::Get().CreateFileWriter(*CSVFilename);
		CSVFile->Logf(TEXT("old,%s"), *OldPath);
		CSVFile->Logf(TEXT("new,%s"), *NewPath);
		CSVFile->Logf(TEXT(""));
	}

	SummarizeDeterminism();
	LogChangedFiles(CSVFile, OldPath, NewPath);

	// start summary
	UE_LOGF(LogDiffAssets, Display, "Summary:");
	UE_LOGF(LogDiffAssets, Display, "Old AssetRegistry: %ls", *OldPath);
	UE_LOGF(LogDiffAssets, Display, "%lld packages total, %lld uncooked, %lld cooked assets", oldtotal, olduncooked, oldassets);
	UE_LOGF(LogDiffAssets, Display, "New AssetRegistry: %ls", *NewPath);

	const float InvToMB = 1.0 / (1024 * 1024);

	//Overall Class Summary
	//Actually do the warnings in this run, since it's the overall one
	LogClassSummary(CSVFile, TEXT("Overall"), ChangeSummaryByClass, true, DeterminismByClass);

	//Chunk-by-chunk class summaries
	if (bGroupByChunk)
	{
		TArray<int32> ChunkIDs;
		ChangesByChunk.GetKeys(ChunkIDs);
		ChunkIDs.Sort();
		for (int32 ChunkID : ChunkIDs)
		{
			FString ChunkHeader = (ChunkID == -1) ? TEXT("Untagged") : FString::Printf(TEXT("Chunk %d"), ChunkID);
			if (ChangesByChunk[ChunkID].ChangesByClass.Num())
			{
				LogClassSummary(CSVFile, *ChunkHeader, ChangesByChunk[ChunkID].ChangesByClass, false, ChangesByChunk[ChunkID].Determinism);
			}
		}
	}


#if 0
	TArray<int32> Changelists;
	ChangeSummaryByChangelist.GetKeys(Changelists);
	
	Changelists.Sort([this](const int32& Lhs, const int32& Rhs) {
			const FChangeInfo& LHSChanges = ChangeSummaryByChangelist[Lhs];
			const FChangeInfo& RHSChanges = ChangeSummaryByChangelist[Rhs];
			return LHSChanges.GetTotalChangeSize() > RHSChanges.GetTotalChangeSize();
		});
	
	for (int32 Changelist : Changelists)
	{
		const FChangeInfo& Changes = ChangeSummaryByChangelist[Changelist];
		
		if (Changes.GetTotalChangeSize() == 0)
		{
			continue;
		}
		
		if (Changes.GetTotalChangeCount() < MinChangeCount || Changes.GetTotalChangeSize() < (MinChangeSizeMB*1024*1024))
		{
			continue;
		}
		
		if (Changelist)
		{
			UE_LOGF(LogDiffAssets, Display, "%d: %.02f%% changes (%.02f MB Total)",
				   Changelist, Changes.GetChangePercentage() * 100.0, Changes.GetTotalChangeSize() * InvToMB);
		}
		else
		{
			UE_LOGF(LogDiffAssets, Display, "Unattributable (nondeterministic?): %.02f%% changes (%.02f MB Total)",
				   Changes.GetChangePercentage() * 100.0, Changes.GetTotalChangeSize() * InvToMB);
		}
		
		if (Changes.Adds)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%d packages added,    %8.3f MB", Changes.Adds, Changes.AddedBytes * InvToMB);
		}
		
		if (Changes.Changes)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%d packages modified, %8.3f MB", Changes.Changes, Changes.ChangedBytes * InvToMB);
		}
		
		if (Changes.Deletes)
		{
			UE_LOGF(LogDiffAssets, Display, "\t%d packages removed,  %8.3f MB", Changes.Deletes, Changes.DeletedBytes * InvToMB);
		}
	}
#endif
	// these are parsed by scripts, so please don't modify :)
	UE_LOGF(LogDiffAssets, Display, "%lld total packages, %lld uncooked, %lld cooked assets", newtotal, newuncooked, newassets);
	UE_LOGF(LogDiffAssets, Display, "%lld total unchanged,         %8.3f MB", ChangeSummary.Unchanged, ChangeSummary.UnchangedBytes * InvToMB);
	UE_LOGF(LogDiffAssets, Display, "%lld total packages added,    %8.3f MB", ChangeSummary.Adds, ChangeSummary.AddedBytes * InvToMB);
	UE_LOGF(LogDiffAssets, Display, "%lld total packages modified, %8.3f MB", ChangeSummary.Changes, ChangeSummary.ChangedBytes * InvToMB);
	UE_LOGF(LogDiffAssets, Display, "%lld total packages removed,  %8.3f MB", ChangeSummary.Deletes, ChangeSummary.DeletedBytes * InvToMB);

	UE_LOGF(LogDiffAssets, Display, "Nondeterministic summary:");
	UE_LOGF(LogDiffAssets, Display, "direct   %lld total packages modified, %8.3f MB", NondeterministicSummary.Changes, NondeterministicSummary.ChangedBytes * InvToMB);
	UE_LOGF(LogDiffAssets, Display, "indirect %lld total packages modified, %8.3f MB", IndirectNondeterministicSummary.Changes, IndirectNondeterministicSummary.ChangedBytes * InvToMB);

	//Warn when meeting or exceeding a certain total changed size, if that's enabled
	if (WarnTotalChangedSizeMB > 0
		&& ChangeSummary.ChangedBytes * InvToMB >= WarnTotalChangedSizeMB)
	{
		UE_LOGF(LogDiffAssets, Warning, "Total Changed Bytes exceeded %d MB! (%8.3f MB)", WarnTotalChangedSizeMB, ChangeSummary.ChangedBytes * InvToMB);
	}

	if (CSVFile)
	{
		CSVFile->Logf(TEXT(""));
		CSVFile->Logf(TEXT("Summary"));
		CSVFile->Logf(TEXT("total,%d"), newtotal);
		CSVFile->Logf(TEXT("unchanged,%lld,%lld"), ChangeSummary.Unchanged, ChangeSummary.UnchangedBytes);
		CSVFile->Logf(TEXT("added,%lld,%lld"), ChangeSummary.Adds, ChangeSummary.AddedBytes);
		CSVFile->Logf(TEXT("modified,%lld,%lld"), ChangeSummary.Changes, ChangeSummary.ChangedBytes);
		CSVFile->Logf(TEXT("removed,%lld,%lld"), ChangeSummary.Deletes, ChangeSummary.DeletedBytes);
		CSVFile->Logf(TEXT("direct non-deterministic,%lld,%lld"), NondeterministicSummary.Changes, NondeterministicSummary.ChangedBytes);
		CSVFile->Logf(TEXT("indirect non-deterministic,%lld,%lld"), IndirectNondeterministicSummary.Changes, IndirectNondeterministicSummary.ChangedBytes);
		delete CSVFile;
	}
}

bool UDiffAssetRegistriesCommandlet::LaunchP4(const FString& Args, TArray<FString>& Output, int32& OutReturnCode) const
{
	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	bool bInvoked = false;
	OutReturnCode = -1;
	FString StringOutput;
	FProcHandle ProcHandle = FPlatformProcess::CreateProc(TEXT("p4.exe"), *Args, false, true, true, nullptr, 0, nullptr, PipeWrite);
	if (ProcHandle.IsValid())
	{
		while (FPlatformProcess::IsProcRunning(ProcHandle))
		{
			FString ThisRead = FPlatformProcess::ReadPipe(PipeRead);
			StringOutput += ThisRead;
//			Re-enable waits if constant pipe-querying is somehow damaging, but keep the wait SMALL
//			if (ThisRead.Len() <= 0)
//			{
//				FPlatformProcess::Sleep(0.001f);
//			}
		}

		StringOutput += FPlatformProcess::ReadPipe(PipeRead);
		FPlatformProcess::GetProcReturnCode(ProcHandle, &OutReturnCode);
		bInvoked = true;
	}
	else
	{
		UE_LOGF(LogDiffAssets, Error, "Failed to launch p4.");
	}

	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);

	StringOutput.ParseIntoArrayLines(Output);

	return bInvoked;
}
