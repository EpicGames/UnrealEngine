// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Commandlet to allow diff in P4V, and expose that functionality to the editor
 */

#include "Commandlets/DiffAssetsCommandlet.h"

#include "Containers/Map.h"
#include "Exporters/Exporter.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectIterator.h"
#include "UnrealExporter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DiffAssetsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogDiffAssetsCommandlet, Log, All);

UDiffAssetsCommandlet::UDiffAssetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


bool UDiffAssetsCommandlet::ExportFilesToTextAndDiff(const FString& InParams)
{
	FString Params = InParams.Replace(TEXT("\\\""), TEXT("\"")); // this deals with an anomaly of P4V
	UE_LOGF(LogDiffAssetsCommandlet, Log, "Params: %ls", *Params);
	TArray<FString> Tokens;
	TArray<FString> Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	const FString EqualStr(TEXT("="));
	const FString DiffCmdKey(TEXT("DiffCmd"));
	TArray<FString> PositionalTokens;
	FString DiffCmd;
	for (FString& Token : Tokens)
	{
		FString Key, Value;
		if (Token.Split(EqualStr, &Key, &Value))
		{
			if (DiffCmd.IsEmpty() && Key.Equals(DiffCmdKey, ESearchCase::IgnoreCase))
			{
				DiffCmd = Value;
			}
		}
		else
		{
			PositionalTokens.Add(MoveTemp(Token));
		}
	}

	const FString AssetPackageExtension = FPackageName::GetAssetPackageExtension();
	if (DiffCmd.IsEmpty() || PositionalTokens.Num() < 2)
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Usage: UDiffAssets File1%ls File2%ls DiffCmd=\"C:/Program Files/Araxis/Araxis Merge/AraxisP4Diff.exe {1} {2}\"", *AssetPackageExtension, *AssetPackageExtension);
		return false;
	}
	if (!DiffCmd.Contains(TEXT("{1}")) || !DiffCmd.Contains(TEXT("{2}")) )
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Usage: UDiffAssets File1%ls File2%ls DiffCmd=\"C:/Program Files/Araxis/Araxis Merge/AraxisP4Diff.exe {1} {2}\"", *AssetPackageExtension, *AssetPackageExtension);
		return false;
	}

	return ExportFilesToTextAndDiff(PositionalTokens[0], PositionalTokens[1], DiffCmd);
}

bool UDiffAssetsCommandlet::CopyFileToTempLocation(FString& InOutFilename)
{
	FString InFilename = InOutFilename;
	// Get base filename (no extension) so we can easily fix it up without
	// having to worry about skipping dots and slashes.
	FString OutBaseFilename = FPaths::GetBaseFilename(InFilename);	

	// Make sure the filename doesn't contain any invalid name characters.
	// Replace them with '_' if found.
	const TCHAR* InvalidCharacters = INVALID_LONGPACKAGE_CHARACTERS;
	for (; *InvalidCharacters; ++InvalidCharacters)
	{
		const TCHAR InvalidStr[] = { *InvalidCharacters, '\0' };
		OutBaseFilename.ReplaceInline(InvalidStr, TEXT("_"));
	}	
	// Add path and the extension (with a dot)
	FString OutFilename = FString::Printf(TEXT("%s%s%s"), *FPaths::DiffDir(), *OutBaseFilename, *FPaths::GetExtension(InFilename, true));

	if (IFileManager::Get().Copy(*OutFilename, *InFilename, true, true))
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Failed to copy %ls to %ls.", *InFilename, *OutFilename);
		return false;
	}
	InOutFilename = *OutFilename;
	return true;
}

bool UDiffAssetsCommandlet::LoadFile(const FString& Filename, TArray<UObject *>& LoadedObjects)
{
	UPackage* Package = LoadPackage( NULL, *Filename, LOAD_ForDiff);
	if (!Package)
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Could not load %ls", *Filename);
		return false;
	}
	for (TObjectIterator<UObject> It; It; ++It)
	{
		if (It->GetOuter() == Package)
		{
			LoadedObjects.Add(*It);
		}
	}
	if (!LoadedObjects.Num())
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Loaded %ls, but it didn't contain any objects.", *Filename);
		return false;
	}
	LoadedObjects.Sort();

	return true;
}

bool UDiffAssetsCommandlet::ExportFile(const FString& Filename, const TArray<UObject *>& LoadedObjects)
{
	FString Extension = TEXT("t3d");
	FStringOutputDevice Buffer;
	const FExportObjectInnerContext Context;
	for (int32 Index = 0; Index < LoadedObjects.Num(); Index++)
	{
		UExporter* Exporter = UExporter::FindExporter( LoadedObjects[Index], *Extension );
		if (!Exporter)
		{
			UE_LOGF(LogDiffAssetsCommandlet, Warning, "Could not find exporter.");
			return false;
		}
		UExporter::ExportToOutputDevice( &Context, LoadedObjects[Index], Exporter, Buffer, *Extension, 0, PPF_ExportsNotFullyQualified, false );
		TMap<FString,FString> NativePropertyValues;
		if ( LoadedObjects[Index]->GetNativePropertyValues(NativePropertyValues) && NativePropertyValues.Num())
		{
			int32 LargestKey = 0;
			for ( TMap<FString,FString>::TIterator It(NativePropertyValues); It; ++It )
			{
				LargestKey = FMath::Max(LargestKey, It.Key().Len());
			}
			for ( TMap<FString,FString>::TIterator It(NativePropertyValues); It; ++It )
			{
				Buffer.Logf(TEXT("  %s=%s"), *It.Key().RightPad(LargestKey), *It.Value());
			}
		}
	}
	if (!Buffer.Len())
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "No text was exported!");
		return false;
	}
	if( !FFileHelper::SaveStringToFile( Buffer, *Filename ) )
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Could not write %ls", *Filename);
		return false;
	}
	return true;
}

bool UDiffAssetsCommandlet::ExportFilesToTextAndDiff(const FString& InFilename1, const FString& InFilename2, const FString& DiffCommand)
{
	FString Filename1 = InFilename1;
	FString Filename2 = InFilename2;
	if (!CopyFileToTempLocation(Filename1))
	{
		return false;
	}
	if (!CopyFileToTempLocation(Filename2))
	{
		return false;
	}

	FString TextFilename1 = Filename1 + TEXT(".t3d");
	FString TextFilename2 = Filename2 + TEXT(".t3d");
	{
		TArray<UObject *> ObjectsToExport;
		if (!LoadFile(Filename1, ObjectsToExport))
		{
			return false;
		}
		if (!ExportFile(TextFilename1, ObjectsToExport))
		{
			return false;
		}
	}
	{
		TArray<UObject *> ObjectsToExport;
		if (!LoadFile(Filename2, ObjectsToExport))
		{
			return false;
		}
		if (!ExportFile(TextFilename2, ObjectsToExport))
		{
			return false;
		}
	}

	FString ReplacedDiffCmd = DiffCommand.Replace(TEXT("{1}"), *TextFilename1, ESearchCase::CaseSensitive).Replace(TEXT("{2}"), *TextFilename2, ESearchCase::CaseSensitive);

	int32 ArgsAt = DiffCommand.Find(TEXT("{1}"), ESearchCase::CaseSensitive) - 1;
	FString Args;
	if (ArgsAt > 0)
	{
		Args = *ReplacedDiffCmd + ArgsAt + 1;
		ReplacedDiffCmd.LeftInline(ArgsAt, EAllowShrinking::No);
	}

	if (!FPlatformProcess::CreateProc(*ReplacedDiffCmd, *Args, true, false, false, NULL, 0, NULL, NULL).IsValid())
	{
		UE_LOGF(LogDiffAssetsCommandlet, Warning, "Could not launch %ls.", *ReplacedDiffCmd);
		return false;
	}
	return true;
}



