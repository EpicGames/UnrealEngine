// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DiffCookCommandlet.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Async/ParallelFor.h"
#include "Containers/RingBuffer.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/UnrealMemory.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/CString.h"
#include "Misc/FeedbackContext.h"
#include "Misc/PackageSegment.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DiffCookCommandlet)

namespace UE::PathViews
{
/** The same as UE::String::FindFirst, but it only accepts strings bracketed by directory separators. */
int32 FindFirstFolderIndex(FStringView View, FStringView Search, ESearchCase::Type SearchCase);
/** The same as UE::String::FindLast, but it only accepts strings bracketed by directory separators. */
int32 FindLastFolderIndex(FStringView View, FStringView Search, ESearchCase::Type SearchCase);
}

namespace UE::DiffCook
{

constexpr FStringView STR_Content(TEXTVIEW("Content"));
constexpr FStringView STR_Engine(TEXTVIEW("Engine"));
constexpr FStringView STR_Game(TEXTVIEW("Game"));
constexpr FStringView STR_SlashEngine(TEXTVIEW("/Engine"));
constexpr FStringView STR_SlashGame(TEXTVIEW("/Game"));
constexpr FStringView STR_Plugins(TEXTVIEW("Plugins"));
constexpr FStringView STR_Metadata(TEXTVIEW("Metadata"));
constexpr int32 WarningCountForNumExtensionsReadFromDisk = 1000;
constexpr int64 BinaryDiffCompareBufferSize = 1000000;

}

int32 UDiffCookCommandlet::Main(const FString& CmdLineParams)
{
	ON_SCOPE_EXIT
	{
		Shutdown();
	};
	if (!TryParseCommandLine(CmdLineParams))
	{
		return 1;
	}

	InitializePlugins();
	if (!TryLoadDepots())
	{
		return 1;
	}

	if (CompDepot.bValid)
	{
		FDiffResult Diff = DiffDepotAsBinary();
		if (bShowPackages)
		{
			PrintPackageDiffs(Diff);
		}
		if (bShowSummary)
		{
			PrintSummary(Diff);
		}
	}
	else
	{
		// Summarizing just the BaseDepot
	}

	return 0;
}

void UDiffCookCommandlet::Shutdown()
{
}

bool UDiffCookCommandlet::TryParseCommandLine(const FString& CmdLineParams)
{
	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	TArray<FString> Tokens;
	TArray<FString> Switches;
	ParseCommandLine(*CmdLineParams, Tokens, Switches);

	bool bDisplayHelp = false;
	bool bResult = true;
	bool bSingleDepot = false;

	for (const FString& Switch : Switches)
	{
		FString Key;
		FString Value;
		if (!Switch.Split(TEXT("="), &Key, &Value, ESearchCase::CaseSensitive))
		{
			Key = Switch;
		}
		if (Key == TEXT("h") || Key == TEXT("help"))
		{
			bDisplayHelp = true;
			bResult = false;
		}
		else if (Key == TEXT("base"))
		{
			Args.BasePath = Value;
		}
		else if (Key == TEXT("comp"))
		{
			Args.CompPath = Value;
		}
		else if (Key == TEXT("targetplatform="))
		{
			if (TargetPlatforms.IsEmpty())
			{
				TargetPlatforms = TPM.GetActiveTargetPlatforms();
			}
		}
		else if (Key == TEXT("package"))
		{
			UE::String::ParseTokensMultiple(Value, { '+', ',' },
				[this](FStringView PackageStr)
				{
					Args.RequestedPackages.Add(FString(PackageStr));
				});
		}
		else if (Key == TEXT("show") || Key == TEXT("hide"))
		{
			bool bShow = Key == TEXT("show");
			UE::String::ParseTokensMultiple(Value, { '+', ',' },
				[this, bShow](FStringView ShowStr)
				{
					if (ShowStr.Equals(TEXT("package"), ESearchCase::IgnoreCase)
						|| ShowStr.Equals(TEXT("packages"), ESearchCase::IgnoreCase))
					{
						bShowPackages = bShow;
					}
					if (ShowStr.Equals(TEXT("addedpackages"), ESearchCase::IgnoreCase))
					{
						bShowAddedPackages = bShow;
					}
					if (ShowStr.Equals(TEXT("removedpackages"), ESearchCase::IgnoreCase))
					{
						bShowRemovedPackages = bShow;
					}
					if (ShowStr.Equals(TEXT("modifiedpackages"), ESearchCase::IgnoreCase))
					{
						bShowModifiedPackages = bShow;
					}
					else if (ShowStr.Equals(TEXT("header"), ESearchCase::IgnoreCase)
						|| ShowStr.Equals(TEXT("headers"), ESearchCase::IgnoreCase))
					{
						bShowHeaders = bShow;
					}
					else if (ShowStr.Equals(TEXT("serialize"), ESearchCase::IgnoreCase))
					{
						bShowSerialize = bShow;
					}
					else
					{
						UE_LOGF(LogCook, Warning, "Unrecognized showflag '-%ls=%.*ls'."
							" Valid Options are {'packages', 'removedpackages', 'addedpackages', 'modifiedpackages', 'headers', 'serialize'}.",
							bShow ? TEXT("show") : TEXT("hide"), ShowStr.Len(), ShowStr.GetData());
					}
				}, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);
		}
		else if (Key == TEXT("singledepot"))
		{
			bSingleDepot = true;
		}
		else if (Key == TEXT("addedverbosity"))
		{
			AddedVerbosity = ParseDiffVerbosity(Value);
		}
		else if (Key == TEXT("removedverbosity"))
		{
			RemovedVerbosity = ParseDiffVerbosity(Value);
		}
		else if (Key == TEXT("modifiedverbosity"))
		{
			ModifiedVerbosity = ParseDiffVerbosity(Value);
		}
	}

	if (Args.BasePath.IsEmpty() || (!bSingleDepot && Args.CompPath.IsEmpty()))
	{
		bDisplayHelp = true;
		bResult = false;
		UE_LOGF(LogCook, Error, "Expected two paths specified with -base=<BasePath> -comp=<CompPath>, or one path -base=<BasePath> with -singledepot.");
	}
	if (bSingleDepot)
	{
		Args.CompPath.Empty();
	}

	if (bDisplayHelp)
	{
		UE_LOGF(LogCook, Display, "Usage: -run=DiffCook -base=<BasePath> -comp=<CompPath> {<Optional Arguments>}"
			"\n\tEach -path should point to the Saved/Cooked/Platform directory created by the cooker."
			"\n\tOptional Arguments:"
			"\n\t\t-help: Show this message and exit."
			"\n\t\t-show=<ShowFlags>: Enable calculation and display of the given ShowFlags."
			"\n\t\t-hide=<ShowFlags>: Disable calculation and display of the given ShowFlags."
			"\n\t\tShowFlags: {packages|headers|serialize}, can be a list delimited with +, e.g. {packages+headers}."
			"\n\t\t\tpackages: Write out information per package that is different."
			"\n\t\t\theaders: If writing out information per package, include differences in the package header, if any."
			"\n\t\t\tserialize: If writing out information per package, load and save the package in the current version of the engine"
			"\n\t\t\t\tto attempt to find the callstack of differences in the exports, if any."
			"\n\t\t\tDefault ShowFlags: packages+headers."
			"\n\t\t-package=<FileNameOrLongPackageNamesDelimitedBy+>: Show per-package diffs for these and only these packages."
			"\n\t\t-targetplatform=<CookPlatformName>: Interpret the result as for the given platform."
			"\n\t\t\tIf not specified, platform will be inferred from path, if that doesn't work, platform-specific data will be skipped."
			"\n\t\t-singledepot: Ignore -comp and display information about -base without any diff."
			"\n\t\t-addedverbosity=error|warning|display: Specify the verbosity at which the count of added files will be reported."
			"\n\t\t-removedverbosity=error|warning|display: Specify the verbosity at which the count of removed files will be reported."
			"\n\t\t-modifiedverbosity=error|warning|display: Specify the verbosity at which the count of modified files will be reported.");
	}
	return bResult;
}

void UDiffCookCommandlet::InitializePlugins()
{
	using namespace UE::DiffCook;

	TArray<FString> LongPackageNameRoots;
	FPackageName::QueryRootContentPaths(LongPackageNameRoots,
		false /* bIncludeReadOnly Roots*/, false /* bWithoutLeadingSlashes */, false /* bWithoutTrailingSlashes */);
	for (const FString& LongPackageNameRoot : LongPackageNameRoots)
	{
		FStringView RootNameWithoutSlashes = FStringView(LongPackageNameRoot).SubStr(1, LongPackageNameRoot.Len() - 2);
		if (RootNameWithoutSlashes == STR_Engine || RootNameWithoutSlashes == STR_Game)
		{
			continue;
		}

		TStringBuilder<256> LocalPathToContentDir(InPlace,
			FPackageName::GetContentPathForPackageRoot(LongPackageNameRoot));
		NormalizeLocalDir(LocalPathToContentDir);
		if (LocalPathToContentDir.Len() == 0)
		{
			continue;
		}

		FStringView LocalPath;
		FStringView ContentLeafDir;
		FStringView ContentExtension;
		FPathViews::Split(LocalPathToContentDir, LocalPath, ContentLeafDir, ContentExtension);
		if (ContentLeafDir != STR_Content || !ContentExtension.IsEmpty())
		{
			continue;
		}

		FStringView LeafFolderName = FPathViews::GetBaseFilename(LocalPath);
		if (LeafFolderName == RootNameWithoutSlashes)
		{
			continue;
		}
		int32 PluginsFolderIndex = UE::PathViews::FindLastFolderIndex(LocalPath, UE::DiffCook::STR_Plugins, ESearchCase::IgnoreCase);
		if (PluginsFolderIndex == INDEX_NONE || PluginsFolderIndex == 0)
		{
			continue;
		}
		FStringView ParentOfPluginsFolder = FStringView(LocalPath).Left(PluginsFolderIndex - 1);

		FStringView PathStartingWithPlugins;
		if (!FPathViews::TryMakeChildPathRelativeTo(LocalPath, ParentOfPluginsFolder, PathStartingWithPlugins))
		{
			continue;
		}
		FStringView RootWithNoEndSlash = FStringView(LongPackageNameRoot).LeftChop(1);

		TArray<FMountWithLeafFolderNameNotEqualLongPackageNameRoot>& Mounts =
			MountsWithLeafFolderNameNotEqualLongPackageNameRoot.FindOrAdd(FString(LeafFolderName));
		FMountWithLeafFolderNameNotEqualLongPackageNameRoot& Mount = Mounts.Emplace_GetRef();
		Mount.PathStartingWithPlugins = FString(PathStartingWithPlugins);
		Mount.LongPackageNameRoot = FString(RootWithNoEndSlash);
	}
}

bool UDiffCookCommandlet::TryLoadDepots()
{
	if (!TryLoadDepotSummaries())
	{
		return false;
	}

	UE_LOGF(LogCook, Display, "Comparing depots:\n\tBase: %ls\n\tComp: %ls",
		*BaseDepot.CookPath, *CompDepot.CookPath);

	LoadDepotPackageLists();
	if (!TryConstructFilterLists())
	{
		return false;
	}

	return true;
}

bool UDiffCookCommandlet::TryLoadDepotSummaries()
{
	using namespace UE::DiffCook;

	bool bResult = true;

	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	const ITargetPlatform* OverrideTargetPlatform = nullptr;
	if (TargetPlatforms.Num() > 0)
	{
		if (TargetPlatforms.Num() > 1)
		{
			UE_LOGF(LogCook, Warning, "Too many targetplatforms provided; using only the first one: %ls.",
				*TargetPlatforms[0]->IniPlatformName());
			OverrideTargetPlatform = TargetPlatforms[0];
		}
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	for (int32 DepotIndex = 0; DepotIndex < 2; ++DepotIndex)
	{
		const TCHAR* DepotName = DepotIndex == 0 ? TEXT("base") : TEXT("comp");
		FCookedDepot& Depot = DepotIndex == 0 ? BaseDepot : CompDepot;
		if (DepotIndex == 1 && Args.CompPath.IsEmpty())
		{
			continue;
		}

		Depot.CookPath = GetNormalizedLocalDir(DepotIndex == 0 ? Args.BasePath : Args.CompPath);
		if (!PlatformFile.DirectoryExists(*Depot.CookPath))
		{
			UE_LOGF(LogCook, Error, "Directory does not exist for -%ls at '%ls'.", DepotName, *Depot.CookPath);
			bResult = false;
			continue;
		}
		Depot.bValid = true;

		// Find the project directory
		TArray<FString> ProjectNames;
		PlatformFile.IterateDirectory(*Depot.CookPath,
			[&Depot, &PlatformFile, &ProjectNames](const TCHAR* FullPath, bool bDirectory)
			{
				FString FileName = FPaths::GetCleanFilename(FullPath);
				// Ignore files and the Engine directory
				if (!bDirectory || FileName == STR_Engine)
				{
					return true;
				}

				FString CookedSettingsFilename = FPaths::Combine(
					Depot.CookPath, FileName, TEXT("Metadata"), TEXT("CookedSettings.txt"));
				if (PlatformFile.FileExists(*CookedSettingsFilename))
				{
					ProjectNames.Emplace(FileName);
				}
				return true;
			});
		if (ProjectNames.Num() == 0)
		{
			UE_LOGF(LogCook, Error,
				"Could not find project name in -%ls at '%ls'. Looked for a path of the form %ls, but none was found.",
				DepotName, *Depot.CookPath,
				*FPaths::Combine(Depot.CookPath, TEXT("<ProjectName>"), TEXT("MetaData"), TEXT("CookedSettings.txt")));
			bResult = false;
			continue;
		}
		if (ProjectNames.Num() > 1)
		{
			UE_LOGF(LogCook, Error,
				"Could not determine project name in -%ls at '%ls'; multiple candidates were found. " " Looked for paths of the form %ls, and found more than one: { %ls }.",
				DepotName, *Depot.CookPath,
				*FPaths::Combine(
					Depot.CookPath, TEXT("<ProjectName>"), TEXT("MetaData"), TEXT("CookedSettings.txt")),
				*FString::Join(ProjectNames, TEXT(", ")));
			bResult = false;
			continue;
		}
		Depot.ProjectName = ProjectNames[0];

		if (OverrideTargetPlatform)
		{
			Depot.TargetPlatform = OverrideTargetPlatform;
		}
		else
		{
			Depot.TargetPlatform = TPM.FindTargetPlatform(FPathViews::GetCleanFilename(Depot.CookPath));
		}

		// Load AssetRegistries from CookRoot/<ProjectName>/AssetRegistry.bin
		FString ProjectDir = FPaths::Combine(Depot.CookPath, Depot.ProjectName);
		FString BaseAssetRegistryFileName = FPaths::Combine(ProjectDir, TEXT("AssetRegistry.bin"));
		FAssetRegistryLoadOptions Options;
		int32 MaxWorkers = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
		Options.ParallelWorkers = FMath::Clamp(MaxWorkers, 0, 16);
		Depot.bARStateValid = FAssetRegistryState::LoadFromDisk(*BaseAssetRegistryFileName, Options, Depot.ARState);
		if (!Depot.bARStateValid)
		{
			UE_LOGF(LogCook, Warning,
				"Loading %ls AssetRegistry from path '%ls' failed. Information requiring the AssetRegistry will not be available for the depot.",
				DepotName, *BaseAssetRegistryFileName);
		}
		else
		{
			// Load chunk-specific AssetRegistries from CookRoot/<ProjectName>/AssetRegistry*.bin
			FAssetRegistrySerializationOptions DevelopmentSerializationOptions(
				UE::AssetRegistry::ESerializationTarget::ForDevelopment);
			PlatformFile.IterateDirectory(*ProjectDir,
				[&Depot, &Options, &ProjectDir, &DevelopmentSerializationOptions, DepotName](const TCHAR* FullPath, bool bDirectory)
				{
					FString FileName = FPaths::GetCleanFilename(FullPath);
					// Ignore directories and the base AssetRegistry
					if (bDirectory || FileName == TEXT("AssetRegistry.bin"))
					{
						return true;
					}
					if (FileName.MatchesWildcard(TEXT("AssetRegistry*.bin")))
					{
						FString ChunkFileName = FPaths::Combine(ProjectDir, FileName);
						FAssetRegistryState ChunkState;
						if (!FAssetRegistryState::LoadFromDisk(*ChunkFileName, Options, ChunkState))
						{
							UE_LOGF(LogCook, Warning,
								"Loading %ls chunk-specific AssetRegistry from path '%ls' failed." " Information requiring the AssetRegistry will not be available for files in that chunk.",
								DepotName, *ChunkFileName);
						}
						Depot.ARState.InitializeFromExisting(ChunkState, DevelopmentSerializationOptions,
							FAssetRegistryState::EInitializationMode::Append);
					}
					return true;
				});
		}

		FString DevARFileName = FPaths::Combine(ProjectDir, TEXT("Metadata"), TEXT("DevelopmentAssetRegistry.bin"));
		Depot.bDevARStateValid = FAssetRegistryState::LoadFromDisk(*DevARFileName, Options, Depot.DevARState);
		if (!Depot.bDevARStateValid && Depot.bARStateValid)
		{
			UE_LOGF(LogCook, Warning,
				"Loading %ls DevelopmentAssetRegistry from path %ls failed. Information requiring the DevelopmentAssetRegistry will not be available for the depot.",
				DepotName, *DevARFileName);
		}
	}

	if (!bResult)
	{
		return false;
	}

	if (CompDepot.bValid && BaseDepot.ProjectName != CompDepot.ProjectName)
	{
		UE_LOGF(LogCook, Error,
			"Projectnames do not match for the requested depots. -base at '%ls' has projectname=%ls, -comp at '%ls' has projectname=%ls." " Projectnames are found by looking for paths of the form %ls under each cookroot.",
			*BaseDepot.CookPath, *BaseDepot.ProjectName,
			*CompDepot.CookPath, *CompDepot.ProjectName,
			*FPaths::Combine(
				TEXT("<CookRoot>"), TEXT("<ProjectName>"), TEXT("MetaData"), TEXT("CookedSettings.txt")));
		bResult = false;
	}

	if (CompDepot.bValid && !OverrideTargetPlatform)
	{
		if (!BaseDepot.TargetPlatform)
		{
			BaseDepot.TargetPlatform = CompDepot.TargetPlatform;
		}
		else if (!CompDepot.TargetPlatform)
		{
			CompDepot.TargetPlatform = BaseDepot.TargetPlatform;
		}
	}
	if (CompDepot.bValid && BaseDepot.TargetPlatform != CompDepot.TargetPlatform)
	{
		check(BaseDepot.TargetPlatform != nullptr && CompDepot.TargetPlatform != nullptr);
		UE_LOGF(LogCook, Error,
			"TargetPlatforms do not match for the requested depots. -base at '%ls' has targetplatform=%ls, -comp at '%ls' has targetplatform=%ls." " TargetPlatforms are found from the leafname of the depotpath, or from the -targetplatform argument.",
			*BaseDepot.CookPath, *BaseDepot.TargetPlatform->IniPlatformName(),
			*CompDepot.CookPath, *CompDepot.TargetPlatform->IniPlatformName());
		bResult = false;
	}
	if (!BaseDepot.TargetPlatform)
	{
		UE_LOGF(LogCook, Warning,
			"TargetPlatform could not be detected. TargetPlatforms are found from the leafname of the depotpath, or from the -targetplatform argument." " Information that relies on targetplatform (such as -show=serialize) will not be available.");
	}

	return bResult;
}


void UDiffCookCommandlet::LoadDepotPackageLists()
{
	for (FCookedDepot* DepotPtr : { &BaseDepot, &CompDepot })
	{
		FCookedDepot& Depot = *DepotPtr;
		if (!Depot.bValid)
		{
			continue;
		}
		LoadDepotContentRoots(Depot);

		UE_LOGF(LogCook, Display, "Scanning filelist from %ls...", *Depot.CookPath)
		TArray<FString> CookPaths;
		FPlatformFileManager::Get().GetPlatformFile().FindFilesRecursively(
			CookPaths, *Depot.CookPath, nullptr /* FileExtension */);

		Depot.PackageDatas.Reserve(CookPaths.Num());
		Depot.PackageDatasByPackageName.Reserve(CookPaths.Num());
		for (FString& CookPath : CookPaths)
		{
			CookPath = GetNormalizedLocalPath(CookPath);
			FString PackageName;
			if (!TryConvertCookPathToLongPackageName(Depot, CookPath, PackageName))
			{
				continue;
			}

			EPackageExtension Extension = FPackagePath::ParseExtension(CookPath);
			bool bInvalidExtension = false;
			bool bRecordAsPackageCookPath = false;
			switch (Extension)
			{
			case EPackageExtension::Asset:
			case EPackageExtension::Map:
				bRecordAsPackageCookPath = true;
				break;
			case EPackageExtension::Unspecified:
			case EPackageExtension::Custom:
			case EPackageExtension::EmptyString:
				bInvalidExtension = true;
				break;
			default:
				bInvalidExtension = Extension >= EPackageExtension::Count;
				break;
			}
			if (bInvalidExtension)
			{
				continue;
			}

			FName PackageFName = FName(PackageName);
			FPackageData*& PackageData = Depot.PackageDatasByPackageName.FindOrAdd(PackageFName);
			if (!PackageData)
			{
				PackageData = Depot.PackageDatas.Add_GetRef(MakeUnique<FPackageData>()).Get();
				PackageData->PackageName = PackageFName;
			}

			if (bRecordAsPackageCookPath)
			{
				PackageData->CookPath = MoveTemp(CookPath);
				PackageData->HeaderExtension = Extension;
			}
			static_assert((uint32)EPackageExtension::Count <= sizeof(PackageData->HasExtensionBitfield) * 8,
				"We are assuming we can treat each EPackageExtension value as a bit index into a uint32 bitfield.");
			PackageData->SetHasExtension(Extension, true);
		}
		Depot.PackageDatas.Shrink();
		Depot.PackageDatasByPackageName.Empty(Depot.PackageDatas.Num());

		Depot.PackageDatas.RemoveAll([](const TUniquePtr<FPackageData>& PackageData)
			{
				return PackageData->HeaderExtension == EPackageExtension::Count;
			});
		Algo::Sort(Depot.PackageDatas, [](TUniquePtr<FPackageData>& A, TUniquePtr<FPackageData>& B)
			{
				return A->PackageName.LexicalLess(B->PackageName);
			});
		Depot.PackageDatasByPackageName.Reserve(Depot.PackageDatas.Num());
		for (TUniquePtr<FPackageData>& PackageData : Depot.PackageDatas)
		{
			Depot.PackageDatasByPackageName.Add(PackageData->PackageName, PackageData.Get());
		}
	}
}

bool UDiffCookCommandlet::TryConstructFilterLists()
{
	if (!Args.RequestedPackages.IsEmpty())
	{
		bool bAnyPackageFailed = false;
		FilterByPackageName.Reserve(Args.RequestedPackages.Num());
		for (FString& RequestedPackage : Args.RequestedPackages)
		{
			RequestedPackage = GetNormalizedFlexPath(RequestedPackage);

			FString PackageNameStr;
			if (TryConvertCookPathToLongPackageName(BaseDepot, RequestedPackage, PackageNameStr))
			{
				FilterByPackageName.Add(FName(PackageNameStr));
			}
			else if (CompDepot.bValid && TryConvertCookPathToLongPackageName(CompDepot, RequestedPackage, PackageNameStr))
			{
				FilterByPackageName.Add(FName(PackageNameStr));
			}
			else
			{
				UE_LOGF(LogCook, Error, "Could not find PackageName for -package '%ls', it will not be diffed.",
					*RequestedPackage);
				bAnyPackageFailed = true;
			}
		}
		if (bAnyPackageFailed && FilterByPackageName.IsEmpty())
		{
			return false;
		}
	}
	return true;
}

void UDiffCookCommandlet::LoadDepotContentRoots(FCookedDepot& Depot)
{
	using namespace UE::DiffCook;

	FString EngineLocalPath;
	FString GameLocalPath;
	TArray<FString> ContentRoots = FindContentRootsFromDepotTree(Depot, EngineLocalPath, GameLocalPath);
	Depot.LongPackageNameRoots.Add(FString(STR_SlashEngine), EngineLocalPath);
	Depot.LocalPathRoots.Add(EngineLocalPath, FString(STR_SlashEngine));
	Depot.LongPackageNameRoots.Add(FString(STR_SlashGame), GameLocalPath);
	Depot.LocalPathRoots.Add(GameLocalPath, FString(STR_SlashGame));

	Algo::Sort(ContentRoots, [](const FString& A, const FString& B)
		{
			return A.Compare(B, ESearchCase::IgnoreCase) < 0;
		});
	for (FString& ContentRoot : ContentRoots)
	{
		if (ContentRoot == EngineLocalPath || ContentRoot == GameLocalPath)
		{
			continue;
		}
		FString LongPackageNameRoot = FPaths::GetCleanFilename(ContentRoot);

		// See the variable comment for MountsWithLeafFolderNameNotEqualLongPackageNameRoot.
		// Check whether the leafname matches any of those mount points, and if so, look it up from our list.
		const TArray<FMountWithLeafFolderNameNotEqualLongPackageNameRoot>* MountsForLeaf =
			MountsWithLeafFolderNameNotEqualLongPackageNameRoot.Find(LongPackageNameRoot);
		if (MountsForLeaf)
		{
			LongPackageNameRoot.Empty();
			for (const FMountWithLeafFolderNameNotEqualLongPackageNameRoot& Mount : *MountsForLeaf)
			{
				if (ContentRoot.EndsWith(Mount.PathStartingWithPlugins))
				{
					LongPackageNameRoot = Mount.LongPackageNameRoot;
					break;
				}
			}
			if (LongPackageNameRoot.IsEmpty())
			{
				continue;
			}
		}
		else
		{
			LongPackageNameRoot = FString::Printf(TEXT("/%s"), *LongPackageNameRoot);
		}

		FString& LocalPathRoot = Depot.LongPackageNameRoots.FindOrAdd(LongPackageNameRoot);
		if (!LocalPathRoot.IsEmpty())
		{
			UE_LOGF(LogCook, Error, "ContentRoot %ls exists in multiple locations: %ls and %ls. Discarding %ls.",
				*LongPackageNameRoot, *LocalPathRoot, *ContentRoot, *LocalPathRoot);
			continue;
		}
		LocalPathRoot = ContentRoot;
		Depot.LocalPathRoots.Add(LocalPathRoot, LongPackageNameRoot);
	}
}

namespace UE::DiffCook::DiffDepotUtils
{

struct FDepotData
{
	FDepotData(UDiffCookCommandlet::FCookedDepot& InDepot)
		: Depot(InDepot)
		, PackageDatasByPackageName(InDepot.PackageDatasByPackageName)
		, PackageDatas(InDepot.PackageDatas)
		, NameMap(Depot.PackageDatasByPackageName)
		, Num(PackageDatas.Num())
	{

	}
	UDiffCookCommandlet::FCookedDepot& Depot;
	TMap<FName, UDiffCookCommandlet::FPackageData*>& PackageDatasByPackageName;
	TArray<TUniquePtr<UDiffCookCommandlet::FPackageData>>& PackageDatas;
	TMap<FName, UDiffCookCommandlet::FPackageData*>& NameMap;
	int32 Num = 0;
	int32 PackageIndex = 0;
};

enum class EWhichSegment
{
	Header,
	Exports,
	Other,
	Done,
};

template <typename Enum>
void EnumIncrement(Enum& Value)
{
	using UnderlyingType = __underlying_type(Enum);
	Value = (Enum)(((UnderlyingType)Value) + 1);
}

} // namespace UE::DiffCook::DiffDepotUtils

UDiffCookCommandlet::FDiffResult UDiffCookCommandlet::DiffDepotAsBinary()
{
	using namespace UE::DiffCook::DiffDepotUtils;

	FDiffResult Diff;
	FDepotData Base = FDepotData(BaseDepot);
	FDepotData Comp = FDepotData(CompDepot);
	TArray<TPair<FPackageData*, FPackageData*>> PackagesToCompare;
	PackagesToCompare.Reserve(Base.Num);

	for (TUniquePtr<FPackageData>& CompPackage : Comp.PackageDatas)
	{
		if (!FilterByPackageName.IsEmpty() && !FilterByPackageName.Contains(CompPackage->PackageName))
		{
			continue;
		}

		FPackageData* const* BasePackagePtr = Base.NameMap.Find(CompPackage->PackageName);
		if (!BasePackagePtr)
		{
			Diff.PackageDiffs.Emplace(CompPackage->PackageName, EPackageDiffResult::Added);
		}
		else
		{
			PackagesToCompare.Emplace(*BasePackagePtr, CompPackage.Get());
		}
	}
	for (TUniquePtr<FPackageData>& BasePackage : Base.PackageDatas)
	{
		if (!FilterByPackageName.IsEmpty() && !FilterByPackageName.Contains(BasePackage->PackageName))
		{
			continue;
		}

		if (!Comp.NameMap.Contains(BasePackage->PackageName))
		{
			Diff.PackageDiffs.Emplace(BasePackage->PackageName, EPackageDiffResult::Removed);
		}
	}

	int32 ThreadCount = FMath::Max(FPlatformMisc::NumberOfCoresIncludingHyperthreads(),1);

	TArray<TArray<FPackageDiffResult>> ThreadLocalPackageDiffs;
	ThreadLocalPackageDiffs.SetNum(ThreadCount);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	ParallelFor(ThreadCount,
		[&PackagesToCompare, &ThreadLocalPackageDiffs, ThreadCount, &PlatformFile, this](int32 ThreadIndex)
		{
			constexpr int64 BufferSize = UE::DiffCook::BinaryDiffCompareBufferSize;
			TUniquePtr<uint8[]> BaseBuffer(new uint8[BufferSize]);
			TUniquePtr<uint8[]> CompBuffer(new uint8[BufferSize]);
			FString BaseCookPathBuffer;
			FString CompCookPathBuffer;
			TArray<FPackageDiffResult>& LocalDiffs = ThreadLocalPackageDiffs[ThreadIndex];

			for (int32 Index = ThreadIndex; Index < PackagesToCompare.Num(); Index += ThreadCount)
			{
				const TPair<FPackageData*, FPackageData*>& ComparePair = PackagesToCompare[Index];
				FName PackageName = ComparePair.Key->PackageName;

				FPackageData& BaseData = *ComparePair.Key;
				FPackageData& CompData = *ComparePair.Value;
				EWhichSegment NextSegment = EWhichSegment::Header;
				EPackageExtension NextOtherExtension = (EPackageExtension)0;
				int64 OffsetOfCurrentSegment = 0;

				for (; NextSegment != EWhichSegment::Done;)
				{
					const FString* BaseCookPath = nullptr;
					const FString* CompCookPath = nullptr;
					bool bSameExtension = true;
					bool bCanUseCombinedOffset = false;
					int64 SegmentSize = 0;
					EPackageExtension DiffExtension = EPackageExtension::Count;
					switch (NextSegment)
					{
					case EWhichSegment::Header:
						BaseCookPath = &BaseData.CookPath;
						CompCookPath = &CompData.CookPath;
						bSameExtension = BaseData.HeaderExtension == CompData.HeaderExtension;
						DiffExtension = BaseData.HeaderExtension;
						bCanUseCombinedOffset = true;
						EnumIncrement(NextSegment);
						break;
					case EWhichSegment::Exports:
						BaseCookPath = GetFileNameForExtension(BaseData, BaseCookPathBuffer, EPackageExtension::Exports);
						CompCookPath = GetFileNameForExtension(CompData, CompCookPathBuffer, EPackageExtension::Exports);
						DiffExtension = EPackageExtension::Exports;
						bCanUseCombinedOffset = true;
						EnumIncrement(NextSegment);
						break;
					case EWhichSegment::Other:
						BaseCookPath = GetFileNameForExtension(BaseData, BaseCookPathBuffer, NextOtherExtension);
						CompCookPath = GetFileNameForExtension(CompData, CompCookPathBuffer, NextOtherExtension);
						DiffExtension = NextOtherExtension;
						if (NextOtherExtension < EPackageExtension::Count)
						{
							EnumIncrement(NextOtherExtension);
						}
						else
						{
							EnumIncrement(NextSegment);
						}
						break;
					default:
						checkNoEntry();
						NextSegment = EWhichSegment::Done;
						break;
					}
					if (!BaseCookPath && !CompCookPath)
					{
						continue;
					}

					bool bDifferent = false;
					bool bError = false;
					int64 DiffOffset = -1;
					if (!BaseCookPath || !CompCookPath || !bSameExtension)
					{
						bDifferent = true;
						DiffOffset = 0;
					}
					else
					{
						FFileStatData BaseStat = PlatformFile.GetStatData(**BaseCookPath);
						FFileStatData CompStat = PlatformFile.GetStatData(**CompCookPath);
						if (BaseStat.FileSize == -1 || CompStat.FileSize == -1)
						{
							if (BaseStat.FileSize != CompStat.FileSize)
							{
								bDifferent = true;
								DiffOffset = 0;
							}
						}
						else
						{
							SegmentSize = BaseStat.FileSize;

							TUniquePtr<IFileHandle> BaseFile(PlatformFile.OpenRead(**BaseCookPath));
							TUniquePtr<IFileHandle> CompFile(PlatformFile.OpenRead(**CompCookPath));
							if (!BaseFile || !CompFile)
							{
								bDifferent = true;
								bError = true;
							}
							else
							{
								// Read from both files until we find a difference and report that offset.
								// If the files are different in size and no difference is found until we reach
								// the end of the smaller file, then report the end of the smaller file as the
								// difference offset.
								int64 MinSize = FMath::Min(BaseStat.FileSize, CompStat.FileSize);
								int64 MaxSize = FMath::Max(BaseStat.FileSize, CompStat.FileSize);
								int64 Offset = 0; 
								while (Offset < MaxSize)
								{
									if (Offset >= MinSize)
									{
										bDifferent = true;
										DiffOffset = Offset;
										break;
									}
									int64 ReadSize = FMath::Min((int64)BufferSize, MinSize - Offset);
									if (!BaseFile->Read(BaseBuffer.Get(), ReadSize) ||
										!CompFile->Read(CompBuffer.Get(), ReadSize))
									{
										bDifferent = true;
										bError = true;
										break;
									}
									uint8* BasePtr = BaseBuffer.Get();
									uint8* CompPtr = CompBuffer.Get();
									for (int64 BufferOffset = 0; BufferOffset < ReadSize; ++BufferOffset)
									{
										if (BasePtr[BufferOffset] != CompPtr[BufferOffset])
										{
											bDifferent = true;
											DiffOffset = Offset + BufferOffset;
											break;
										}
									}
									if (bDifferent)
									{
										break;
									}
									Offset += ReadSize;
								}
							}
						}
					}

					if (bDifferent)
					{
						if (bError)
						{
							LocalDiffs.Emplace(PackageName, EPackageDiffResult::Error);
						}
						else
						{
							int64 Combined = bCanUseCombinedOffset ? (DiffOffset + OffsetOfCurrentSegment) : -1;
							LocalDiffs.Emplace(PackageName, EPackageDiffResult::Modified, DiffExtension, DiffOffset, Combined);
						}
						break; // Done with the package
					}
					OffsetOfCurrentSegment += SegmentSize;
				}
			}
		});

	for (TArray<FPackageDiffResult>& LocalDiffs : ThreadLocalPackageDiffs)
	{
		Diff.PackageDiffs.Append(MoveTemp(LocalDiffs));
	}

	Algo::Sort(Diff.PackageDiffs, [](const FPackageDiffResult& A, const FPackageDiffResult& B)
		{
			return A.PackageName.LexicalLess(B.PackageName);
		});
	return Diff;
}

void UDiffCookCommandlet::PrintSummary(FDiffResult& Diff)
{
	UE_LOGF(LogCook, Display, "CookDiff Result: %ls",
		Diff.PackageDiffs.IsEmpty() ? TEXT("IDENTICAL") : TEXT("DIFFERENT"));
	GWarn->CategorizedLogf(LogCook.GetCategoryName(), (NumAdded == 0 ? ELogVerbosity::Display : AddedVerbosity),
		TEXT("%d files added."), NumAdded);
	GWarn->CategorizedLogf(LogCook.GetCategoryName(), (NumRemoved == 0 ? ELogVerbosity::Display : RemovedVerbosity),
		TEXT("%d files removed."), NumRemoved);
	GWarn->CategorizedLogf(LogCook.GetCategoryName(), (NumModified == 0 ? ELogVerbosity::Display : ModifiedVerbosity),
		TEXT("%d files modified."), NumModified);
}

void UDiffCookCommandlet::PrintPackageDiffs(FDiffResult& Diff)
{
	if (Diff.PackageDiffs.IsEmpty())
	{
		return;
	}

	for (const FPackageDiffResult& PackageDiff : Diff.PackageDiffs)
	{
		FName PackageName = PackageDiff.PackageName;
		TStringBuilder<256> PackageNameStr(InPlace, PackageName);
		FPackageData** Base = BaseDepot.PackageDatasByPackageName.Find(PackageName);
		FPackageData** Comp = CompDepot.PackageDatasByPackageName.Find(PackageName);
		FString BasePathBuffer;
		FString CompPathBuffer;
		FString* BasePath = Base ? &(*Base)->CookPath : nullptr;
		FString* CompPath = Comp ? &(*Comp)->CookPath : nullptr;;
		if (!BasePath)
		{
			EPackageExtension Extension = CompPath ? FPackagePath::ParseExtension(*CompPath) : EPackageExtension::EmptyString;
			TryConvertLongPackageNameToCookPath(BaseDepot, PackageNameStr, Extension, BasePathBuffer);
			BasePath = &BasePathBuffer;
		}
		if (!CompPath)
		{
			EPackageExtension Extension = BasePath ? FPackagePath::ParseExtension(*BasePath) : EPackageExtension::EmptyString;
			TryConvertLongPackageNameToCookPath(CompDepot, PackageNameStr, Extension, CompPathBuffer);
			CompPath = &CompPathBuffer;
		}

		bool bModified = false;
		switch (PackageDiff.Result)
		{
		case EPackageDiffResult::Removed:
			++NumRemoved;
			if (bShowRemovedPackages)
			{
				UE_LOGF(LogCook, Display, "Removed:  %ls\n\tBase: %ls\n\tComp: %ls (Missing)",
					*PackageNameStr, **BasePath, **CompPath);
			}
			break;
		case EPackageDiffResult::Added:
			++NumAdded;
			if (bShowAddedPackages)
			{
				UE_LOGF(LogCook, Display, "Added:    %ls\n\tBase: %ls (Missing)\n\tComp: %ls",
					*PackageNameStr, **BasePath, **CompPath);
			}
			break;
		case EPackageDiffResult::Modified:
			++NumModified;
			bModified = true;
			break;
		case EPackageDiffResult::Error: [[fallthrough]];
		default:
			UE_LOGF(LogCook, Error, "Could Not Diff: %ls", *PackageNameStr);
			break;
		}
		if (!bModified)
		{
			continue;
		}

		if (bShowModifiedPackages)
		{
			FString WhichFile = FPathViews::SetExtension(FPathViews::GetCleanFilename(*BasePath),
				LexToString(PackageDiff.Extension));
			UE_LOGF(LogCook, Display, "Modified: %ls\n\tBase: %ls\n\tComp: %ls"
				"\n\t      Different at Byte %lld in %ls%ls.",
				*PackageNameStr, **BasePath, **CompPath,
				PackageDiff.Offset, *WhichFile,
				(PackageDiff.CombinedOffset >= 0
					? *FString::Printf(TEXT(", Combined/DiffBreak Offset %lld"), PackageDiff.CombinedOffset)
					: TEXT("")));

			if (bShowHeaders)
			{
				UE_CALL_ONCE([]() { UE_LOGF(LogCook, Error, "-show=Headers is not yet implemented"); });
			}
			if (bShowSerialize)
			{
				UE_CALL_ONCE([]() { UE_LOGF(LogCook, Error, "-show=Serialize is not yet implemented"); });
			}
		}
	}
}

bool UDiffCookCommandlet::TryConvertCookPathToLongPackageName(FCookedDepot& Depot, FStringView CookPath,
	FString& OutLongPackageName)
{
	using namespace UE::DiffCook;

	if (CookPath.IsEmpty())
	{
		return false;
	}
	// Is it already a package name?
	if (FPathViews::IsSeparator(CookPath[0]))
	{
		FStringView FirstComponent;
		FStringView Remainder;
		FPathViews::SplitFirstComponent(CookPath.RightChop(1), FirstComponent, Remainder);
		if (!FirstComponent.IsEmpty())
		{
			FString LongPackageNameRoot = FString::Printf(TEXT("/%.*s"), FirstComponent.Len(), FirstComponent.GetData());
			if (Depot.LongPackageNameRoots.Contains(LongPackageNameRoot))
			{
				OutLongPackageName = CookPath;
				return true;
			}
		}
	}

	// Not a known LongPackageName, look for format of form <LocalPathRoot>/Content/<RelativePath>
	int32 FoundIndex = UE::PathViews::FindFirstFolderIndex(CookPath, STR_Content, ESearchCase::IgnoreCase);
	if (FoundIndex == INDEX_NONE || FoundIndex == 0)
	{
		return false;
	}

	FStringView ParentDir = CookPath.Left(FoundIndex - 1);
	const FString* LongPackageNameRoot = Depot.LocalPathRoots.FindByHash(GetTypeHash(ParentDir), ParentDir);
	if (!LongPackageNameRoot)
	{
		return false;
	}
	FStringView RelativePath = CookPath.RightChop(FoundIndex + STR_Content.Len() + 1);
	RelativePath = FPathViews::GetBaseFilenameWithPath(RelativePath);

	OutLongPackageName = FPaths::Combine(*LongPackageNameRoot, RelativePath);
	return true;
}

bool UDiffCookCommandlet::TryConvertLongPackageNameToCookPath(FCookedDepot& Depot, FStringView LongPackageName,
	EPackageExtension Extension, FString& OutCookPath)
{
	if (LongPackageName.Len() == 0 || LongPackageName[0] != '/')
	{
		return false;
	}
	int32 MountIndex;
	if (!LongPackageName.RightChop(1).FindChar('/', MountIndex))
	{
		return false;
	}
	MountIndex = MountIndex + 1; // Add back in the prefix length that we chopped off
	FStringView MountPoint = LongPackageName.Left(MountIndex);
	FStringView RelPath = LongPackageName.RightChop(MountIndex + 1);

	FString* LocalContentRoot = Depot.LongPackageNameRoots.FindByHash(GetTypeHash(MountPoint), MountPoint);
	if (!LocalContentRoot)
	{
		return false;
	}
	TStringBuilder<1024> CookPath(InPlace, *LocalContentRoot);
	FPathViews::Append(CookPath, UE::DiffCook::STR_Content);
	FPathViews::Append(CookPath, RelPath);
	CookPath += LexToString(Extension);
	OutCookPath = CookPath.ToView();
	return true;
}

TArray<FString> UDiffCookCommandlet::FindContentRootsFromDepotTree(FCookedDepot& Depot, FString& OutEngineLocalPath, FString& OutGameLocalPath)
{
	using namespace UE::DiffCook;

	int32 ThreadCount = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	ThreadCount = FMath::Max(ThreadCount, 1);

	TRingBuffer<FString> ScanQueue;
	ScanQueue.Add(Depot.CookPath);
	TArray<FString> ContentRoots;
	TArray<FString> LocalSubDirs;
	OutEngineLocalPath = GetNormalizedLocalDir(FPaths::Combine(Depot.CookPath, STR_Engine));
	OutGameLocalPath = GetNormalizedLocalDir(FPaths::Combine(Depot.CookPath, Depot.ProjectName));

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	while (!ScanQueue.IsEmpty())
	{
		LocalSubDirs.Reset();
		FString ScanDir = ScanQueue.PopFrontValue();
		bool bIsContentRoot = false;
		PlatformFile.IterateDirectory(*ScanDir,
			[&LocalSubDirs, &bIsContentRoot](const TCHAR* FullPath, bool bDirectory)
			{
				if (bDirectory)
				{
					FStringView LeafName = FPathViews::GetCleanFilename(FullPath);
					if (LeafName == STR_Content)
					{
						bIsContentRoot = true;
					}
					LocalSubDirs.Emplace(GetNormalizedLocalDir(FullPath));
				}
				return true;
			});
		
		bool bIsContentRootWithSubPlugins = ScanDir == OutEngineLocalPath || ScanDir == OutGameLocalPath;
		if (!bIsContentRoot || bIsContentRootWithSubPlugins)
		{
			if (bIsContentRootWithSubPlugins)
			{
				LocalSubDirs.RemoveAll([](const FString& LocalSubDir)
					{
						FStringView LeafName = FPathViews::GetBaseFilename(LocalSubDir);
						return LeafName == STR_Content || LeafName == STR_Metadata;
					});
			}
			for (FString& SubDir : LocalSubDirs)
			{
				ScanQueue.Add(MoveTemp(SubDir));
			}
			LocalSubDirs.Reset();
		}
		if (bIsContentRoot)
		{
			ContentRoots.Add(MoveTemp(ScanDir));
		}
	}
	return ContentRoots;
}

FString* UDiffCookCommandlet::GetFileNameForExtension(UDiffCookCommandlet::FPackageData& PackageData,
	FString& Buffer, EPackageExtension Extension)
{
	if (PackageData.HasExtension(Extension))
	{
		Buffer = FPathViews::ChangeExtension(PackageData.CookPath, LexToString(EPackageExtension::Exports));
		return &Buffer;
	}
	return nullptr;
};

FString UDiffCookCommandlet::GetNormalizedLocalDir(FStringView Dir)
{
	TStringBuilder<256> Normalized(InPlace, Dir);
	NormalizeLocalDir(Normalized);
	return FString(Normalized);
}

void UDiffCookCommandlet::NormalizeLocalDir(FStringBuilderBase& Dir)
{
	FPathViews::NormalizeDirectoryName(Dir);
}

FString UDiffCookCommandlet::GetNormalizedLocalPath(FStringView Path)
{
	TStringBuilder<256> Normalized(InPlace, Path);
	NormalizeLocalPath(Normalized);
	return FString(Normalized);
}

void UDiffCookCommandlet::NormalizeLocalPath(FStringBuilderBase& Path)
{
	FPathViews::NormalizeFilename(Path);
}

FString UDiffCookCommandlet::GetNormalizedFlexPath(FStringView Path)
{
	TStringBuilder<256> Normalized(InPlace, Path);
	NormalizeFlexPath(Normalized);
	return FString(Normalized);
}

void UDiffCookCommandlet::NormalizeFlexPath(FStringBuilderBase& Path)
{
	FPathViews::NormalizeFilename(Path);
}

ELogVerbosity::Type UDiffCookCommandlet::ParseDiffVerbosity(const FString& Text)
{
	ELogVerbosity::Type Result = ParseLogVerbosityFromString(Text);
	Result = (ELogVerbosity::Type)FMath::Min((uint8)Result, (uint8)ELogVerbosity::Display);
	return Result;
}

bool UDiffCookCommandlet::FPackageData::HasExtension(EPackageExtension Segment) const
{
	check((uint32)Segment < sizeof(HasExtensionBitfield) * 8);
	uint32 BitFlag = 1 << (uint32)Segment;
	return (HasExtensionBitfield & BitFlag) != 0;
}

void UDiffCookCommandlet::FPackageData::SetHasExtension(EPackageExtension Segment, bool bValue)
{
	check((uint32)Segment < sizeof(HasExtensionBitfield) * 8);
	uint32 BitFlag = 1 << (uint32)Segment;
	HasExtensionBitfield = (HasExtensionBitfield & ~BitFlag) | (bValue ? BitFlag : 0);
}

UDiffCookCommandlet::FPackageDiffResult::FPackageDiffResult(FName InPackageName, EPackageDiffResult InResult,
	EPackageExtension InExtension, int64 InOffset, int64 InCombinedOffset)
	: PackageName(InPackageName)
	, Result(InResult)
	, Extension(InExtension)
	, Offset(InOffset)
	, CombinedOffset(InCombinedOffset)
{
}

namespace UE::PathViews
{

int32 FindFirstFolderIndex(FStringView View, FStringView Search, ESearchCase::Type SearchCase)
{
	if (Search.IsEmpty())
	{
		return INDEX_NONE;
	}

	int32 StartPosition = 0;
	for (;;)
	{
		int32 FoundIndex = View.Find(Search, StartPosition, SearchCase);
		if (FoundIndex == INDEX_NONE)
		{
			break;
		}
		if ((FoundIndex != 0 && !FPathViews::IsSeparator(View[FoundIndex - 1]))
			|| (FoundIndex + Search.Len() < View.Len() && !FPathViews::IsSeparator(View[FoundIndex + Search.Len()])))
		{
			StartPosition = FoundIndex + 1;
			continue;
		}
		return FoundIndex;
	}
	return INDEX_NONE;
}

int32 FindLastFolderIndex(FStringView View, FStringView Search, ESearchCase::Type SearchCase)
{
	if (Search.IsEmpty())
	{
		return INDEX_NONE;
	}

	int32 EndPosition = View.Len();
	for (;;)
	{
		int32 FoundIndex = UE::String::FindLast(View.Left(EndPosition), Search, SearchCase);
		if (FoundIndex == INDEX_NONE)
		{
			break;
		}
		if ((FoundIndex != 0 && !FPathViews::IsSeparator(View[FoundIndex - 1]))
			|| (FoundIndex + Search.Len() < View.Len() && !FPathViews::IsSeparator(View[FoundIndex + Search.Len()])))
		{
			EndPosition = FoundIndex + Search.Len() - 1;
			continue;
		}
		return FoundIndex;
	}
	return INDEX_NONE;
}

} // namespace UE::PathViews
