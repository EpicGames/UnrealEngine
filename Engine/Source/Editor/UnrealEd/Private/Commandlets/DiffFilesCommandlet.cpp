// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DiffFilesCommandlet.cpp: Commandlet used for comparing two packages.

=============================================================================*/

#include "Commandlets/DiffFilesCommandlet.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDevice.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Trace/Detail/Channel.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DiffFilesCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogDiffFilesCommandlet, Log, All);

UDiffFilesCommandlet::UDiffFilesCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

bool UDiffFilesCommandlet::Initialize(const TCHAR* Parms)
{
	bool bResult = false;

	// parse the command line into tokens and switches
	TArray<FString> Tokens, Switches;
	ParseCommandLine(Parms, Tokens, Switches);


	uint32 PackageCounter = 0;

	// find the package files that should be diffed - doesn't need to be a valid package path (i.e. can be a package located in a tmp directory or something)
	for (int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++)
	{
		TArray<FString> FilesInPath;

		bool bMergePackage = false, bAncestorPackage = false, bFirstPackage = false, bSecondPackage = false;
		FString	PackageWildcard = Tokens[TokenIndex];
		/*if (PackageWildcard.Contains(TEXT("=")))
		{
			FString ParsedFilename;
			if (FParse::Value(*PackageWildcard, TEXT("MERGE="), ParsedFilename))
			{
			bMergePackage = true;
			}
			// look for a common ancestor setting
			else if (FParse::Value(*PackageWildcard, TEXT("ANCESTOR="), ParsedFilename))
			{
			bAncestorPackage = true;
			}
			PackageWildcard = ParsedFilename;
		}*/

		if (PackageWildcard.Len() == 0)
		{
			SET_WARN_COLOR(COLOR_RED);
			UE_LOGF(LogDiffFilesCommandlet, Error, "No package specified for parameter %i: %ls.  Use 'help DiffFilesCommandlet' to view correct usage syntax for this commandlet.",
				TokenIndex, *Tokens[TokenIndex]);

			CLEAR_WARN_COLOR();
			bResult = false;
			break;
		}

		IFileManager::Get().FindFiles((TArray<FString>&)FilesInPath, *PackageWildcard, true, false);
		if (FilesInPath.Num() == 0)
		{
			// if no files were found in the script directory, search all valid package paths
			TArray<FString> Paths;
			if (GConfig->GetArray(TEXT("Core.System"), TEXT("Paths"), Paths, GEngineIni) > 0)
			{
				for (int32 i = 0; i < Paths.Num(); i++)
				{
					TArray<FString> Temp;
					IFileManager::Get().FindFiles(Temp, *(Paths[i] / PackageWildcard), 1, 0);
					FString Path = FPaths::GetPath(Paths[i] / PackageWildcard);
					for (const auto& T : Temp)
					{
						FilesInPath.Add(Path / T);
					}
				}
			}
		}
		else
		{
			// re-add the path information so that GetPackageLinker finds the correct version of the file.
			FString WildcardPath = PackageWildcard;
			for (int32 FileIndex = 0; FileIndex < FilesInPath.Num(); FileIndex++)
			{
				FilesInPath[FileIndex] = FPaths::GetPath(WildcardPath) / FilesInPath[FileIndex];
			}
		}

		for (const auto& FileInPath : FilesInPath)
		{
			FPackageInfo PackageInfo;
			PackageInfo.FullPath = FileInPath;

			PackageInfo.FriendlyName = FString::Printf(TEXT("%s(%d)"), *FPaths::GetBaseFilename(PackageInfo.FullPath), PackageCounter++);
			PackageInfos.Add(PackageInfo);
			bResult = true;
		}
	}

	if (PackageInfos.Num() < 2)
	{
		SET_WARN_COLOR(COLOR_RED);
		UE_LOGF(LogDiffFilesCommandlet, Error, "You must specify two packages to use this commandlet.  Use 'help DiffFilesCommandlet' to view correct usage syntax for this commandlet.");

		CLEAR_WARN_COLOR();
		bResult = false;
	}

	return bResult;
}

int32 UDiffFilesCommandlet::Main(const FString& Params)
{
	if (!Initialize(*Params))
	{
		// Initialize fails if the command-line parameters were invalid.
		return 1;
	}


	LoadAndDiff();

	return 0;
}



void UDiffFilesCommandlet::LoadAndDiff()
{
	// load the first package and use our super diff archiver

	check(PackageInfos.Num() >= 2);

	UPackage *Package = CreatePackage( TEXT("Package_(0)"));


	Package = LoadPackage(Package, *FString::Printf(TEXT("%s;%s"), *PackageInfos[0].FullPath, *PackageInfos[1].FullPath), LOAD_ForDiff | LOAD_ForFileDiff);

	if (Package->GetLinker()->IsTextFormat())
	{
		UE_LOGF(LogDiffFilesCommandlet, Warning, "FileDiffs are not currently supported for text based assets");
	}
	else
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		IAssetRegistry::FLoadPackageRegistryData AssetData;
		AssetRegistry.LoadPackageRegistryData(*Package->GetLinker()->GetLoader_Unsafe(), AssetData);
	}
}
