// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiffPackagesCommandlet.cpp: Commandlet used for comparing two packages.

=============================================================================*/

#include "Commandlets/EditorCommandlets.h"
#include "UObject/LinkerLoad.h"
#include "UObject/PropertyPortFlags.h"

// the maximum number of packages that can be compared
#define MAX_PACKAGECOUNT 3

// whether to serialize object recursively when looking for changes (for debugging)
#define USE_DEEP_RECURSION 0

PRAGMA_DISABLE_DEPRECATION_WARNINGS;

/**
 * The different types of comparison differences that can exist between packages.
 */
enum EObjectDiff
{
	/** no difference */
	OD_None,

	/** the object exist in the first package only */
	OD_AOnly,

	/** the object exists in the second package only */
	OD_BOnly,

	/** (three-way merges) the value has been changed from the ancestor package, but the new value is identical in the two packages being compared */
	OD_ABSame,

	/** @todo */
	OD_ABConflict,

	/** @todo */
	OD_Invalid,
};

/**
 * Contains the results for a comparison between two values of a single property.
 */
struct FPropertyComparison
{
	/** Constructor */
	FPropertyComparison()
	: Prop(NULL), DiffType(OD_None)
	{}

	/** the property that was compared */
	FProperty* Prop;

	/**
	 * The comparison result type for this property comparison.
	 */
	EObjectDiff DiffType;

	/** The name of the property that was compared; only used when comparing native property data (which will have no corresponding FProperty) */
	FString PropText;

	/**
	 * Contains the result of the comparison.
	 */
	FString DiffText;
};

/**
 * Contains information about a comparison of the property values for two object graphs.  One FObjectComparison is created for each
 * top-level object in a package (i.e. each object that has the package's LinkerRoot as its Outer), which contains comparison data for
 * the top-level object as well as its subobjects.
 */
struct FObjectComparison
{
	/** Constructor */
	FObjectComparison()
	: OverallDiffType(OD_None)
	{
		FMemory::Memzero(ObjectSets, sizeof(ObjectSets));
	}

	/**
	 * The path name for the top-level object in this FObjectComparison, minus the package portion of the path name.
	 */
	FString RootObjectPath;

	/**
	 * The graph of objects represented by this comparison from each package.  The graphs contain the top-level object along with
	 * all of its subobjects.
	 */
	FObjectGraph* ObjectSets[MAX_PACKAGECOUNT];

	/**
	 * The list of comparison results for all property values which not identical in all packages.
	 */
	TArray<FPropertyComparison> PropDiffs;

	/**
	 * The cumulative comparison result type for the entire object graph comparison.
	 */
	EObjectDiff OverallDiffType;
};


/**
 * Constructor
 *
 * Populates the Objects array with RootObject and its subobjects.
 *
 * @param	RootObject			the top-level object for this object graph
 * @param	PackageIndex		the index [into the Packages array] for the package that this object graph belongs to
 * @param	ObjectsToIgnore		optional list of objects to not include in this object graph, even if they are contained within RootObject
 */
FObjectGraph::FObjectGraph( UObject* RootObject, int32 PackageIndex, TArray<FObjectComparison>* pObjectsToIgnore/*=NULL*/ )
{
	new(Objects) FObjectReference(RootObject);

	// start with just looking in the root object, but collect references on everything
	// that is put in to objects, etc
	for (int32 ObjIndex = 0; ObjIndex < Objects.Num(); ObjIndex++)
	{
		FObjectReference& ObjSet = Objects[ObjIndex];

		// find all objects inside this object that are referenced by properties in the object
		TArray<UObject*> Subobjects;

		// if we want to ignore certain objects, pre-fill the Subobjects array with the list
		if ( pObjectsToIgnore != NULL )
		{
			TArray<FObjectComparison>& ObjectsToIgnore = *pObjectsToIgnore;
			for ( int32 IgnoreIndex = 0; IgnoreIndex < ObjectsToIgnore.Num(); IgnoreIndex++ )
			{
				FObjectGraph* IgnoreGraph = ObjectsToIgnore[IgnoreIndex].ObjectSets[PackageIndex];
				if ( IgnoreGraph != NULL )
				{
					Subobjects.AddUnique(IgnoreGraph->GetRootObject());
					/*
					adding the root object *should* be sufficient (as theoretically, the only object that should have a reference to its subobjects
					is that object....but if that doesn't work correctly, we'll add all of the objects in the graph
					for ( int32 GraphIndex = 1; GraphIndex < IgnoreGraph->Objects.Num(); GraphIndex++ )
					{
						Subobjects.AddUniqueItem(IgnoreGraph->Objects(GraphIndex).Object);
					}
					*/
				}
			}
		}

		const int32 StartIndex = Subobjects.Num();
		FReferenceFinder ObjectReferenceCollector(Subobjects, ObjSet.Object, true, false, USE_DEEP_RECURSION);
		ObjectReferenceCollector.FindReferences(ObjSet.Object);

		// add all the newly serialized objects to the object set
		for (int32 Index = StartIndex; Index < Subobjects.Num(); Index++)
		{
			new(Objects) FObjectReference(Subobjects[Index]);
		}
	}
}

UObject* FObjectGraph::GetRootObject() const
{
	return Objects[0].Object;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS;
