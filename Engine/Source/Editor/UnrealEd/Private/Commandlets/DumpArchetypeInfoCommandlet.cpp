// Copyright Epic Games, Inc. All Rights Reserved.

#include "DumpArchetypeInfoCommandlet.h"

#include "HAL/FileManager.h"
#include "JsonObjectGraph/Stringify.h"
#include "Misc/Paths.h"
#include "UObject/UObjectIterator.h"

namespace UE::Private
{
static void SnapshotClasses(const FString& Label)
{
	const auto GetIntermediateAssetName = [](const UObject* Object, const TCHAR* Prefix)
	{
		return
			FPaths::ProjectSavedDir() +
			FString(TEXT("Temp/")) + Prefix +
			Object->GetPathName() +
			FString(TEXT("_snap.json"));
	};

	FString ScratchFilenameWritten;
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		const UClass* Class = *ClassIt;
		// skip editor only blueprint classes, they may have unstable
		// data because they are never cooked:
		if (!Class->GetDefaultObject(false) || 
			Class->GetAuthoritativeClass() != Class ||
			Class->HasAnyClassFlags(CLASS_Transient) || 
			!Class->HasMetaData(TEXT("BlueprintType")))
		{
			continue;
		}

		FJsonStringifyOptions Options;
		Options.Flags |= EJsonStringifyFlags::FilterEditorOnlyData;
		FUtf8String Result = UE::JsonObjectGraph::Stringify({ Class, Class->GetDefaultObject(false) }, Options);
		FString Filename = GetIntermediateAssetName(Class, *Label);
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
		if(FileArchive) // some verse types generate filesnames that are too long, i do not care about them atm
		{
			FileArchive->Serialize((void*)Result.GetCharArray().GetData(), Result.GetCharArray().Num() - 1);
		}
	}
}
static void PrintUsage()
{
	UE_LOG(LogTemp, Error, TEXT("<project> -run=\"DumpArchetypeInfo\" <label> - Dumps snapshots of all loaded archetype objets to the folder specified by label, e.g. \
		<project_dir>/Saved/Temp/<label>/<archetype_path>"));
}
}

int32 UDumpArchetypeInfoCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;

	ParseCommandLine(*Params, Tokens, Switches);
	if(Tokens.Num() != 1)
	{
		UE::Private::PrintUsage();
		return 1;
	}

	UE::Private::SnapshotClasses(Tokens[0]);
	return 0;
}
