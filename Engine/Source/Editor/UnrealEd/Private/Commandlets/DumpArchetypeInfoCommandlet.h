// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Commandlets/Commandlet.h"
#include "DumpArchetypeInfoCommandlet.generated.h"

UCLASS()
class UDumpArchetypeInfoCommandlet : public UCommandlet
{
	GENERATED_BODY()

private:
	virtual int32 Main(const FString& Params) override;
};
