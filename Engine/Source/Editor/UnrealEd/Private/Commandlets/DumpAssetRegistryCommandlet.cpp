// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpAssetRegistryCommandlet.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ArrayReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpAssetRegistryCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogAssetRegistryDump, Log, All);

UDumpAssetRegistryCommandlet::UDumpAssetRegistryCommandlet(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

int32 UDumpAssetRegistryCommandlet::Main(const FString& FullCommandLine)
{
	UE_LOGF(LogAssetRegistryDump, Display, "--------------------------------------------------------------------------------------------");
	UE_LOGF(LogAssetRegistryDump, Display, "Running DumpAssetRegistry Commandlet");
	ON_SCOPE_EXIT
	{
		UE_LOGF(LogAssetRegistryDump, Display, "Completed DumpAssetRegistry Commandlet");
		UE_LOGF(LogAssetRegistryDump, Display, "--------------------------------------------------------------------------------------------");
	};

	if (!TryParseArgs())
	{
		return 1;
	}
	if (!TryDumpAssetRegistry())
	{
		return 1;
	}

	return 0;
}

bool UDumpAssetRegistryCommandlet::TryParseArgs()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	TArray<FString> Tokens;
	TArray<FString> Switches;
	ParseCommandLine(CommandLine, Tokens, Switches);

	TArray<FString> AllFormattingArgs = {
		TEXT("All"), TEXT("ObjectPath"), TEXT("PackageName"), TEXT("Path"), TEXT("Class"), TEXT("Tag"),
		TEXT("Dependencies"), TEXT("DependencyDetails"), TEXT("LegacyDependencies"), TEXT("PackageData")
	};
	FormattingArgs.Reset();
	LinesPerPage = 10000;
	Path.Reset();
	OutDir.Reset();
	bLowerCase = false;
	for (const FString& Switch : Switches)
	{
		if (FParse::Value(*Switch, TEXT("LinesPerPage="), LinesPerPage))
		{
		}
		else if (FParse::Value(*Switch, TEXT("Path="), Path))
		{
		}
		else if (FParse::Value(*Switch, TEXT("OutDir="), OutDir))
		{
		}
		else if (Switch == TEXT("LowerCase"))
		{
			bLowerCase = true;
		}
		else if (AllFormattingArgs.Contains(Switch))
		{
			FormattingArgs.Add(Switch);
		}
	}

	if (Path.IsEmpty())
	{
		TStringBuilder<256> AllFormattingArgsText;
		for (const FString& Arg : AllFormattingArgs)
		{
			AllFormattingArgsText << TEXT("-") << Arg << TEXT(", ");
		}
		AllFormattingArgsText.RemoveSuffix(2); // Remove trailing ", "
		
		UE_LOGF(LogAssetRegistryDump, Error, "Missing path argument.");
		UE_LOGF(LogAssetRegistryDump, Display, "Usage: -Path=<Path> [-OutDir=<Path>] [-LinesPerPage=<int>] [-FormatSwitch]...");
		UE_LOGF(LogAssetRegistryDump, Display, "FormatSwitches: %ls", *AllFormattingArgsText);
		return false;
	}
	if (FormattingArgs.IsEmpty())
	{
		FormattingArgs.Add(TEXT("All"));
	}
	if (OutDir.IsEmpty())
	{
		OutDir = FPaths::Combine(FPaths::ProjectDir(), TEXT("Saved"), TEXT("Reports"), TEXT("AssetRegistryDump"));
	}
	return true;
}

bool UDumpAssetRegistryCommandlet::TryDumpAssetRegistry()
{
#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	IFileManager& FileManager = IFileManager::Get();
	if (!FileManager.FileExists(*Path))
	{
		UE_LOGF(LogAssetRegistryDump, Error, "File '%ls' does not exist.", *Path);
		return false;
	}
	FArrayReader SerializedAssetData;
	if (!FFileHelper::LoadFileToArray(SerializedAssetData, *Path))
	{
		UE_LOGF(LogAssetRegistryDump, Error, "Failed to load file '%ls'.", *Path);
		return false;
	}
	FAssetRegistryState State;
	if (!State.Load(SerializedAssetData))
	{
		UE_LOGF(LogAssetRegistryDump, Error, "Failed to parse file '%ls' as asset registry.", *Path);
		return false;
	}
	if (!FileManager.DirectoryExists(*OutDir))
	{
		if (!FileManager.MakeDirectory(*OutDir, true /* Tree */))
		{
			UE_LOGF(LogAssetRegistryDump, Error, "Failed to create OutDir '%ls'.", *OutDir);
			return false;
		}
	}

	TArray<FString> Pages;
	State.Dump(FormattingArgs, Pages, LinesPerPage);
	int PageIndex = 0;
	TStringBuilder<256> FileName;
	for (FString& PageText : Pages)
	{
		FileName.Reset();
		FileName.Appendf(TEXT("%s_%05d.txt"), *(OutDir / TEXT("Page")), PageIndex++);
		if (bLowerCase)
		{
			PageText.ToLowerInline();
		}
		FFileHelper::SaveStringToFile(PageText, *FileName);
	}

	UE_LOGF(LogAssetRegistryDump, Display, "Wrote %d files to %ls.", Pages.Num(), *OutDir);
	return true;
#else
	UE_LOGF(LogAssetRegistryDump, Error, "Asset registry state dumping disabled in this build.");
	return false;
#endif //ASSET_REGISTRY_STATE_DUMPING_ENABLED
}
