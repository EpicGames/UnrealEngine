// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpBlueprintsInfoCommandlet.h"

#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpBlueprintsInfoCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintInfoDump, Log, All);

/*******************************************************************************
 * UDumpBlueprintsInfoCommandlet
 ******************************************************************************/

//------------------------------------------------------------------------------
UDumpBlueprintsInfoCommandlet::UDumpBlueprintsInfoCommandlet(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
int32 UDumpBlueprintsInfoCommandlet::Main(FString const& Params)
{
	UE_LOGF(LogBlueprintInfoDump, Error, "DumpBlueprintsInfo has been removed - consider using the GenerateBlueprintAPI commandlet instead\n");
	return 0;
}

