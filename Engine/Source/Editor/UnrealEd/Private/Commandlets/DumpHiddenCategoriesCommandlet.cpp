// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpHiddenCategoriesCommandlet.h"

#include "Containers/Array.h"
#include "Containers/StringConv.h"
#include "EditorCategoryUtils.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/Paths.h"
//#include "DumpHiddenCategoriesCommandlet.h"
#include "ObjectEditorUtils.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Serialization/Archive.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpHiddenCategoriesCommandlet)

/*******************************************************************************
 * Static Helpers
*******************************************************************************/

//------------------------------------------------------------------------------
static FString GetIndentString(uint32 IndentCount)
{
	FString IndentString;
	while (IndentCount > 0)
	{
		IndentString += "\t";
		--IndentCount;
	}
	return IndentString;
}

//------------------------------------------------------------------------------
static int32 GetHideCategories(uint32 Indent, UClass* Class, FString& JsonOut)
{
	FString IndentString = GetIndentString(Indent);
	JsonOut += IndentString + TEXT("\"HiddenCategories\" : [");

	TArray<FString> HideCategories;
	FEditorCategoryUtils::GetClassHideCategories(Class, HideCategories);

	for (FString const& Category : HideCategories)
	{
		JsonOut += TEXT("\n\t") + IndentString + TEXT("\"") + Category + TEXT("\",");
	}
	if (HideCategories.Num() > 0)
	{
		JsonOut.RemoveAt(JsonOut.Len() - 1); // remove the last comma
	}
	JsonOut += TEXT("\n") + IndentString + TEXT("]");

	return HideCategories.Num();
}

//------------------------------------------------------------------------------
static int32 GetShowCategories(uint32 Indent, UClass* Class, FString& JsonOut)
{
	FString IndentString = GetIndentString(Indent);
	JsonOut += IndentString + TEXT("\"ShownCategories\" : [");

	TArray<FString> ShowCategories;
	FEditorCategoryUtils::GetClassShowCategories(Class, ShowCategories);

	for (FString const& Category : ShowCategories)
	{
		JsonOut += TEXT("\n\t") + IndentString + TEXT("\"") + Category + TEXT("\",");
	}
	if (ShowCategories.Num() > 0)
	{
		JsonOut.RemoveAt(JsonOut.Len() - 1); // remove the last comma
	}
	JsonOut += TEXT("\n") + IndentString + TEXT("]");

	return ShowCategories.Num();
}

//------------------------------------------------------------------------------
static int32 GetHiddenFunctions(uint32 Indent, UClass* Class, bool bShowFunLibs, FString& JsonOut)
{
	int32 HiddenFuncCount = 0;

	FString IndentString = GetIndentString(Indent);
	JsonOut += IndentString + TEXT("\"HiddenFunctions\" : [");

	FString& OutputString = JsonOut;
	UClass*  CallingClass = Class;
	auto FindHiddenFuncs = [&IndentString, &CallingClass, &HiddenFuncCount, &OutputString] (UClass* FunctionClass) 
	{
		for (TFieldIterator<UFunction> FunctionIt(FunctionClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
		{
			UFunction* Function = *FunctionIt;
			if (FObjectEditorUtils::IsFunctionHiddenFromClass(Function, CallingClass))
			{
				++HiddenFuncCount;
				OutputString += TEXT("\n\t") + IndentString + TEXT("\"") + Function->GetPathName() + TEXT("\",");
			}
		}

		/*for (TFieldIterator<FObjectProperty> PropIt(FuncClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
		{
			UClass* PropClass = PropIt->PropertyClass;
			for (TFieldIterator<UFunction> FunctionIt(PropClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				if (FObjectEditorUtils::IsFunctionHiddenFromClass(*FunctionIt, ThisClass))
				{
					++HiddenFuncCount;
					OutputString += TEXT("\n\t\t\t\"") + FunctionIt->GetPathName() + TEXT("\",");
				}
			}
		}*/
	};

	FindHiddenFuncs(CallingClass); // find all this class's functions

	if (bShowFunLibs)
	{
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt) // find all functions in each function library
		{
			UClass* TestClass = *ClassIt;
			// if this is a skeleton class, don't bother
			if (FKismetEditorUtilities::IsClassABlueprintSkeleton(TestClass))
			{
				continue;
			}

			if (TestClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()))
			{
				FindHiddenFuncs(TestClass);
			}
		}
	}
	if (HiddenFuncCount > 0)
	{
		OutputString.RemoveAt(OutputString.Len() - 1); // remove the last comma
	}
	OutputString += TEXT("\n") + IndentString + TEXT("]");

	return HiddenFuncCount;
}

//------------------------------------------------------------------------------
static int32 GetHiddenProperties(uint32 Indent, UClass* Class, FString& JsonOut)
{
	int32 HiddenPropCount = 0;

	FString IndentString = GetIndentString(Indent);
	JsonOut += IndentString + TEXT("\"HiddenProperties\" : [");

	for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (FObjectEditorUtils::IsVariableCategoryHiddenFromClass(Property, Class))
		{
			++HiddenPropCount;
			JsonOut += TEXT("\n\t") + IndentString + TEXT("\"") + Property->GetPathName() + TEXT("\",");
		}
	}
	if (HiddenPropCount > 0)
	{
		JsonOut.RemoveAt(JsonOut.Len() - 1); // remove the last comma
	}
	JsonOut += TEXT("\n") + IndentString + TEXT("]");

	return HiddenPropCount;
}

/*******************************************************************************
 * UDumpHiddenCategoriesCommandlet
*******************************************************************************/

//------------------------------------------------------------------------------
UDumpHiddenCategoriesCommandlet::UDumpHiddenCategoriesCommandlet(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
int32 UDumpHiddenCategoriesCommandlet::Main(FString const& Params)
{
	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	bool const bShowHiddenLibraryFuncs = Switches.Contains(TEXT("IncludeFuncLibs"));
	
	FString CommandletSaveDir = FPaths::ProjectSavedDir() + TEXT("Commandlets/");
	IFileManager::Get().MakeDirectory(*CommandletSaveDir);

	FString Filename = FString::Printf(TEXT("HiddenCategoryDump_%s.json"), FPlatformTime::StrTimestamp());
	Filename = Filename.Replace(TEXT(" "), TEXT("_"));
	Filename = Filename.Replace(TEXT("/"), TEXT("-"));
	Filename = Filename.Replace(TEXT(":"), TEXT("."));
	Filename = FPaths::GetCleanFilename(Filename);

	FString FilePath = CommandletSaveDir / *Filename;
	FArchive* FileOut = IFileManager::Get().CreateFileWriter(*FilePath);
	if (FileOut != nullptr)
	{
		FileOut->Serialize(TCHAR_TO_ANSI(TEXT("{")), 1);

		double Duration = 0.0;
		FDurationTimer DurationTimer(Duration);

		bool bFirstEntry = true;
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* ThisClass = *ClassIt;

			FString ClassEntry;
			if (!bFirstEntry)
			{
				ClassEntry += TEXT(",");
			}
			ClassEntry += TEXT("\n\t\"") + ThisClass->GetName() + TEXT("\": {");


			FString HideCategoriesJson;
			if (GetHideCategories(2u, ThisClass, HideCategoriesJson) > 0)
			{
				ClassEntry += TEXT("\n") + HideCategoriesJson;
			}
			else 
			{
				// no need to make an entry for a class that has no "HideCategories" meta data (it can see all)
				continue;
			}

			FString ShowCategoriesJson;
			if (GetShowCategories(2u, ThisClass, ShowCategoriesJson) > 0)
			{
				ClassEntry += TEXT(",\n") + ShowCategoriesJson;
			}

			FString HiddenFunctionJson;
			if (GetHiddenFunctions(2u, ThisClass, bShowHiddenLibraryFuncs, HiddenFunctionJson) > 0)
			{
				ClassEntry += TEXT(",\n") + HiddenFunctionJson;
			}
			
			FString HiddenPropertyJson;
			if (GetHiddenProperties(2u, ThisClass, HiddenPropertyJson) > 0)
			{
				ClassEntry += TEXT(",\n") + HiddenPropertyJson;
			}

			ClassEntry += TEXT("\n\t}");
			FileOut->Serialize(TCHAR_TO_ANSI(*ClassEntry), ClassEntry.Len());
			bFirstEntry = false;
		}
		DurationTimer.Stop();

		FString ClosingStatement = FString::Printf(TEXT(", \"Duration\" : %f\n}"), Duration);
		FileOut->Serialize(TCHAR_TO_ANSI(*ClosingStatement), ClosingStatement.Len());

		FileOut->Close();
		delete FileOut;
		FileOut = nullptr;
	}
	return 0;
}
