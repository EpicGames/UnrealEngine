// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpLightFunctionMaterialInfo.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "MaterialDomain.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpLightFunctionMaterialInfo)

DEFINE_LOG_CATEGORY_STATIC(LogDumpLightFunctionMaterialInfo, Log, All);

UDumpLightFunctionMaterialInfoCommandlet::UDumpLightFunctionMaterialInfoCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UDumpLightFunctionMaterialInfoCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "DumpLightFunctionMaterialInfo");
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "This commandlet will dump to information about light function materials.");
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "A typical way to invoke it is: <YourProject> -run=DumpLightFunctionMaterialInfo -targetplatform=Windows -unattended -sm6 -allowcommandletrendering -nomaterialshaderddc.");
		return 0;
	}



	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "Searching for materials within the project...");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> MaterialAssets;
	AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), MaterialAssets, true);



	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "Found %d materials", MaterialAssets.Num());

	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
	const EShaderPlatform ShaderPlatform = EShaderPlatform::SP_PCD3D_SM6;

	TArray<UMaterialInterface*> LightFunctionMaterialsCompatible;
	TArray<UMaterialInterface*> LightFunctionMaterialsNotCompatible;
	TSet<UMaterialInterface*> MaterialsToCompile;
	TSet<UMaterialInterface*> MaterialsToAnalyse;

	// only run for a single platform as this is enough to know if a light function material will be compatible with that LFAtlas.
	if (Platforms.Num() > 0)
	{
		ITargetPlatform* Platform = Platforms[0];
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Compiling shaders for %ls...", *Platform->PlatformName());
		for (const FAssetData& AssetData : MaterialAssets)
		{
			if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
			{
				UMaterial* Material = MaterialInterface->GetMaterial();
				if (Material && Material->MaterialDomain == MD_LightFunction)
				{
					UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "BeginCache for %ls", *MaterialInterface->GetFullName());
					MaterialInterface->BeginCacheForCookedPlatformData(Platform);
					// need to call this once for all objects before any calls to ProcessAsyncResults as otherwise we'll potentially upload
					// incremental/incomplete shadermaps to DDC (as this function actually triggers compilation, some compiles for a particular
					// material may finish before we've even started others - if we call ProcessAsyncResults in that case the associated shader
					// maps will think they are "finished" due to having no outstanding dependencies).
					if (!MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						MaterialsToCompile.Add(MaterialInterface);
					}
				}
			}
		}
		MaterialsToAnalyse = MaterialsToCompile;
		LightFunctionMaterialsCompatible.Reserve(MaterialsToAnalyse.Num());
		LightFunctionMaterialsNotCompatible.Reserve(MaterialsToAnalyse.Num());



		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "Found %d light function materials to compile.", MaterialsToCompile.Num());

		static constexpr bool bLimitExecutationTime = false;
		int32 PreviousOutstandingJobs = 0;
		constexpr int32 MaxOutstandingJobs = 20000; // Having a max is a way to try to reduce memory usage.. otherwise outstanding jobs can reach 100k+ and use up 300gb committed memory
		// Submit all the jobs.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SubmitJobs);

			UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Submit Jobs");

			while (MaterialsToCompile.Num())
			{
				for (auto It = MaterialsToCompile.CreateIterator(); It; ++It)
				{
					UMaterialInterface* MaterialInterface = *It;
					if (MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						It.RemoveCurrent();
						UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Finished cache for %ls.", *MaterialInterface->GetFullName());
						UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Materials remaining: %d", MaterialsToCompile.Num());
					}

					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

					while (true)
					{
						const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
						if (CurrentOutstandingJobs != PreviousOutstandingJobs)
						{
							UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
							PreviousOutstandingJobs = CurrentOutstandingJobs;
						}

						// Flush rendering commands to release any RHI resources (shaders and shader maps).
						// Delete any FPendingCleanupObjects (shader maps).
						FlushRenderingCommands();

						if (CurrentOutstandingJobs < MaxOutstandingJobs)
						{
							break;
						}
						FPlatformProcess::Sleep(1);
					}
				}
			}
		}



		// Process the shader maps and save to the DDC.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessShaderCompileResults);

			UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "ProcessAsyncResults");

			while (GShaderCompilingManager->IsCompiling())
			{
				GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

				while (true)
				{
					const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
					if (CurrentOutstandingJobs != PreviousOutstandingJobs)
					{
						UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
						PreviousOutstandingJobs = CurrentOutstandingJobs;
					}

					// Flush rendering commands to release any RHI resources (shaders and shader maps).
					// Delete any FPendingCleanupObjects (shader maps).
					FlushRenderingCommands();

					if (CurrentOutstandingJobs < MaxOutstandingJobs)
					{
						break;
					}
					FPlatformProcess::Sleep(1);
				}
			}
		}



		// Look up compilation result for our light function materials
		for(UMaterialInterface* MaterialInterface : MaterialsToAnalyse)
		{
			UMaterial* Material = MaterialInterface->GetMaterial();
			if (Material && Material->MaterialDomain == MD_LightFunction)
			{
				TArray<FMaterialResource*> ResourcesToCache;
				FMaterialResource* CurrentResource = FindOrCreateMaterialResource(ResourcesToCache, Material, nullptr, GetFeatureLevelShaderPlatform_Checked(ERHIFeatureLevel::SM6), EMaterialQualityLevel::High);
				check(CurrentResource);

				FMaterialRelevance MaterialRelevance = CurrentResource->GetMaterialInterface()->GetRelevance(GetFeatureLevelShaderPlatform_Checked(ERHIFeatureLevel::SM6));
				bool bIsLightFunctionAtlasCompatible = MaterialRelevance.bIsLightFunctionAtlasCompatible;


				if (bIsLightFunctionAtlasCompatible)
				{
					LightFunctionMaterialsCompatible.Add(Material);
				}
				else
				{
					LightFunctionMaterialsNotCompatible.Add(Material);
				}

				FMaterial::DeferredDeleteArray(ResourcesToCache);
			}
		}



		// Perform cleanup and clear cached data for cooking.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ClearCachedCookedPlatformData);

			UE_LOGF(LogDumpLightFunctionMaterialInfo, Display, "Clear Cached Cooked Platform Data");

			for (const FAssetData& AssetData : MaterialAssets)
			{
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
				{
					MaterialInterface->ClearAllCachedCookedPlatformData();
				}
			}
		}
	} // Platforms
	

	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "**********************************");
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "* Material compatible with atlas *");
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "**********************************");
	for (UMaterialInterface* MaterialInterface : LightFunctionMaterialsCompatible)
	{
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "  - %ls", *MaterialInterface->GetPathName());
	}
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "**************************************");
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "* Material not compatible with atlas *");
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "**************************************");
	for (UMaterialInterface* MaterialInterface : LightFunctionMaterialsNotCompatible)
	{
		UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "  - %ls", *MaterialInterface->GetPathName());
	}

	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "Material compatible with atlas:.....%d.", LightFunctionMaterialsCompatible.Num());
	UE_LOGF(LogDumpLightFunctionMaterialInfo, Log, "Material not compatible with atlas:.%d.", LightFunctionMaterialsNotCompatible.Num());
	return 0;
}

