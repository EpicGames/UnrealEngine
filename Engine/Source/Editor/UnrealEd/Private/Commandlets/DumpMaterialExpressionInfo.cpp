// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpMaterialExpressionInfo.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionCustom.h"
#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/TextProperty.h"
#include "Internationalization/Regex.h"
#include "String/ParseTokens.h"
#include "ProfilingDebugging/DiagnosticTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpMaterialExpressionInfo)

DEFINE_LOG_CATEGORY_STATIC(LogDumpMaterialExpressionInfo, Log, All);

UDumpMaterialExpressionInfoCommandlet::UDumpMaterialExpressionInfoCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

namespace DumpMaterialExpressionInfo
{
	class FDumper
	{
		TStringBuilder<2048> StringBuilder;

		TArray<FField*> ColumnFields;
		TMap<FField*, int> ColumnFieldsToIndex;
		TArray<FString> ColumnValues;

	public:
		FDumper()
		: StringBuilder()
		{ }

		bool ShouldExcludePropertyType(FProperty* Property)
		{
			if (FStructProperty const* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct->GetFName() == TEXT("ExpressionOutput")
					|| StructProperty->Struct->GetFName() == TEXT("ExpressionInput"))
				{
					return true;
				}
			}
			return false;
		}

		void FindColumnProperties(const FRegexPattern* RequestedExpressionPattern, const TSet<FString>* ColumnNames)
		{
			const bool bMatchAllExpressions = RequestedExpressionPattern == nullptr;
			const bool bMatchAllColumns = ColumnNames == nullptr;

			for (TObjectIterator<UClass> It; It; ++It)
			{
				if (It->IsChildOf(UMaterialExpression::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract))
				{
					bool bInclude = (bMatchAllExpressions || FRegexMatcher(*RequestedExpressionPattern, It->GetName()).FindNext());
					if (!bInclude)
					{
						continue;
					}

					for (FProperty* Property = It->PropertyLink; Property; Property = Property->PropertyLinkNext)
					{
						bInclude = (bMatchAllColumns || ColumnNames->Contains(Property->GetName()));
						bInclude = bInclude && !ShouldExcludePropertyType(Property);
						if (!bInclude)
						{
							continue;
						}

						int Index = ColumnFieldsToIndex.FindOrAdd(Property, (int)ColumnFields.Num());
						if (Index == (int)ColumnFields.Num())
						{
							ColumnFields.Add(Property);
						}
					}
				}
			}
		}

		void WriteHeader(FDiagnosticTableWriterCSV& CsvWriter)
		{
			CsvWriter.AddColumn(TEXT("Asset"));
			CsvWriter.AddColumn(TEXT("ExpressionType"));
			for (FField* Field : ColumnFields)
			{
				CsvWriter.AddColumn(TEXT("%s"), *Field->GetName());
			}
			CsvWriter.CycleRow();
		}

		template<typename TType>
		bool NumericToString(const void* Container, const FField* Property, FString& Result)
		{
			if (TProperty_Numeric<TType> const* NumericProperty = CastField<TProperty_Numeric<TType>>(Property))
			{
				const TType* Val = NumericProperty->template ContainerPtrToValuePtr<TType>(Container);
				Result = LexToString(*Val);
				return true;
			}
			return false;
		}

		FString ToString(const void* Container, const FField* Field)
		{
			FString Result;

			if (FByteProperty const* ByteProperty = CastField<FByteProperty>(Field))
			{
				int8 Value = *ByteProperty->ContainerPtrToValuePtr<int8>(Container);
				if (ByteProperty->Enum) //TEnumAsByte
				{
					Result = ByteProperty->Enum->GetValueOrBitfieldAsString(Value);
				}
				else
				{
					Result = LexToString(Value);
				}
			}
			else if (FEnumProperty const* EnumProperty = CastField<FEnumProperty>(Field))
			{
				UEnum* EnumDef = EnumProperty->GetEnum();
				Result = EnumDef->GetValueOrBitfieldAsString(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(Container));
			}
			else if (NumericToString<uint8>(Container, Field, Result)
				|| NumericToString<int8>(Container, Field, Result)
				|| NumericToString<uint16>(Container, Field, Result)
				|| NumericToString<int16>(Container, Field, Result)
				|| NumericToString<uint32>(Container, Field, Result)
				|| NumericToString<int32>(Container, Field, Result)
				|| NumericToString<uint64>(Container, Field, Result)
				|| NumericToString<int64>(Container, Field, Result)
				|| NumericToString<float>(Container, Field, Result)
				|| NumericToString<double>(Container, Field, Result)
			)
			{ }
			else if (FBoolProperty const* BoolProperty = CastField<FBoolProperty>(Field))
			{
				Result = LexToString(*BoolProperty->ContainerPtrToValuePtr<bool>(Container));
			}
			else if (FStrProperty const* StringProperty = CastField<FStrProperty>(Field))
			{
				Result = *StringProperty->ContainerPtrToValuePtr<FString>(Container);
			}
			else if (FTextProperty const* TextProperty = CastField<FTextProperty>(Field))
			{
				Result = TextProperty->ContainerPtrToValuePtr<FText>(Container)->ToString();
			}
			else if (FNameProperty const* NameProperty = CastField<FNameProperty>(Field))
			{
				Result = NameProperty->ContainerPtrToValuePtr<FName>(Container)->ToString();
			}
			else if (FArrayProperty const* ArrayProperty = CastField<FArrayProperty>(Field))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(Container));
				for (int32 i = 0; i < ArrayHelper.Num(); ++i)
				{
					void* ArrayData = ArrayHelper.GetRawPtr(i);
					Result += ToString(ArrayHelper.GetRawPtr(i), ArrayProperty->Inner);
					if (i != ArrayHelper.Num()-1)
					{
						Result += TEXT(", ");
					}
				}
			}
			else if (FMapProperty const* MapProperty = CastField<FMapProperty>(Field))
			{
				Result = TEXT("[not implemented]");
			}
			else if (FObjectProperty const* ObjectProperty = CastField<FObjectProperty>(Field))
			{
				UObject* Object = ObjectProperty->GetObjectPropertyValue_InContainer(Container, 0);
				Result = Object ? Object->GetName() : TEXT("null");
			}
			else if (FStructProperty const* Property = CastField<FStructProperty>(Field))
			{
				if (Property->Struct == TBaseStructure<FVector>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FVector>(Container)->ToString();
				}
				else if (Property->Struct == TVariantStructure<FVector4f>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FVector4f>(Container)->ToString();
				}
				else if (Property->Struct == TBaseStructure<FVector4d>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FVector4d>(Container)->ToString();
				}
				else if (Property->Struct == TBaseStructure<FVector2D>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FVector2D>(Container)->ToString();
				}
				else if (Property->Struct == TBaseStructure<FLinearColor>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FLinearColor>(Container)->ToString();
				}
				else if (Property->Struct == TBaseStructure<FLinearColor>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FLinearColor>(Container)->ToString();
				}
				else if (Property->Struct == TBaseStructure<FGuid>::Get())
				{
					Result = Property->ContainerPtrToValuePtr<FGuid>(Container)->ToString();
				}
				else if(Property->Struct->GetFName() == TEXT("ExpressionOutput"))
				{
					Result = Property->ContainerPtrToValuePtr<FExpressionOutput>(Container)->OutputName.ToString();
				}
				else if (Property->Struct->GetFName() == TEXT("ExpressionInput"))
				{
					Result = Property->ContainerPtrToValuePtr<FExpressionInput>(Container)->InputName.ToString();
				}
				else
				{
					Result = TEXT("{");
					for (FField* SubField = Property->Struct->ChildProperties; SubField; SubField = SubField->Next)
					{
						FString Name = SubField->GetName();
						FString Value = ToString(Property->ContainerPtrToValuePtr<void>(Container), SubField);
						Result += FString::Printf(TEXT("'%s': '%s'%s"), *Name, *Value, SubField->Next != nullptr ? TEXT(", ") : TEXT(""));
					}
					Result += TEXT("}");
				}
			}
			else
			{
				Result = TEXT("?");
			}

			return Result;
		}

		void DumpAsset(
			const FAssetData& AssetData,
			const FMaterialExpressionCollection& ExpressionCollection, 
			FDiagnosticTableWriterCSV& CsvWriter,
			const FRegexPattern* RequestedExpressionPattern = nullptr)
		{
			const bool bMatchAllExpressions = RequestedExpressionPattern == nullptr;

			for (const TObjectPtr<UMaterialExpression>& Expression : ExpressionCollection.Expressions)
			{
				bool bInclude = (bMatchAllExpressions || FRegexMatcher(*RequestedExpressionPattern, Expression->GetClass()->GetName()).FindNext());
				if (!bInclude)
				{
					continue;
				}

				ColumnValues.Reset(0);
				ColumnValues.Init(FString(), ColumnFields.Num());
				for (FPropertyValueIterator PropertyIt(FProperty::StaticClass(), Expression->GetClass(), Expression, EPropertyValueIteratorFlags::NoRecursion); PropertyIt; ++PropertyIt)
				{
					const FProperty* Property = PropertyIt.Key();

					int* IndexPtr = ColumnFieldsToIndex.Find(Property);
					bInclude = IndexPtr != nullptr;
					if (!bInclude)
					{
						continue;
					}

					ColumnValues[*IndexPtr] = ToString(Expression, Property);
				}

				CsvWriter.AddColumn(TEXT("%s"), *AssetData.GetObjectPathString());
				CsvWriter.AddColumn(TEXT("%s"), *Expression->GetClass()->GetName());
				for (const FString& ColumnValue : ColumnValues)
				{
					CsvWriter.AddColumn(TEXT("%s"), *ColumnValue);
				}
				CsvWriter.CycleRow();
			}
		}
	};
}

static FARFilter SetupAssetFilter(const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals)
{
	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UMaterialFunction::StaticClass()->GetClassPathName());

	// Collections
	const FString* CollectionsParam = ParamVals.Find(TEXT("collections"));
	if (CollectionsParam && !CollectionsParam->IsEmpty())
	{
		TArray<FString> Collections;
		CollectionsParam->ParseIntoArray(Collections, TEXT(","));

		for(const FString& Collection : Collections)
		{
			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
			TSharedPtr<ICollectionContainer> CollectionContainer;
			FName CollectionName;
			ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
			if (CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType))
			{
				CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
			}
		}
	}

	// Tags
	for (const TPair<FString, FString>& Parameter : ParamVals)
	{
		FString TagIdentifier = TEXT("tag:");
		if (Parameter.Key.StartsWith(TagIdentifier))
		{
			FString TagKey = Parameter.Key.Mid(TagIdentifier.Len());
			Filter.TagsAndValues.Add(FName(*TagKey), Parameter.Value);
		}
	}

	return Filter;
}

int32 UDumpMaterialExpressionInfoCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	auto PrintHelp = []()
	{
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "DumpMaterialExpressionInfo");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "Find all instances of material expressions and dump their data.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "For example, to dump a listing of all custom hlsl nodes along with their inputs and snippets:");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "<YourProject> -dx12 -run=DumpMaterialExpressionInfo -unattended -expression=MaterialExpressionCustom -columns=Inputs,Code -csv=C:/output.csv");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, "Options:");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -help                 Print this message.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -csv=filename         Write the output to a CSV file at this path.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -collections=name     Optional. Comma-separated list of asset collections that should be searched.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -tag:TagName=TagValue Optional. Only dump assets with a matching tag. Can have multiple of these.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -material=name        Optional. Only dump materials or material functions matching this name or regular expression.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -expression=name      Optional. Only dump expressions matching this name or regular expression.");
		UE_LOGF(LogDumpMaterialExpressionInfo, Display, " -columns=a,b          Optional. Comma-separated list of the properties that should be included in the output. Dumps all by default.");
	};

	// Display help
	if (Switches.Contains("help"))
	{
		PrintHelp();
		return 0;
	}

	// Parse params
	FString* CsvPath = ParamVals.Find(TEXT("csv"));
	if (!CsvPath)
	{
		UE_LOGF(LogDumpMaterialExpressionInfo, Error, "No output CSV file path was specified. \n");
		PrintHelp();
		return 1;
	}

	FARFilter Filter = SetupAssetFilter(Tokens, Switches, ParamVals);

	FString* RequestedMaterialPatternString = ParamVals.Find(TEXT("material"));
	bool bMatchAllMaterials = !RequestedMaterialPatternString;
	FRegexPattern RequestedMaterialPattern(RequestedMaterialPatternString ? *RequestedMaterialPatternString : FString());

	FString* RequestedExpressionPatternString = ParamVals.Find(TEXT("expression"));
	bool bMatchAllExpressions = !RequestedExpressionPatternString;
	FRegexPattern RequestedExpressionPattern(RequestedExpressionPatternString ? *RequestedExpressionPatternString : FString());

	FString* ColumnsString = ParamVals.Find(TEXT("columns"));
	bool bMatchAllColumns = ColumnsString == nullptr;
	TSet<FString> Columns {};
	if (!bMatchAllColumns)
	{
		UE::String::ParseTokens(*ColumnsString, TEXT(","), [&Columns](const auto& SubString)
								{
									Columns.Add(FString(SubString));
								}, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);
	}

	// Search assets
	UE_LOGF(LogDumpMaterialExpressionInfo, Display, "Searching for materials...");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssets(Filter, Assets);
	UE_LOGF(LogDumpMaterialExpressionInfo, Display, "Found %d materials and material functions", Assets.Num());

	// Open CSV
	FArchive* CsvFileWriter = IFileManager::Get().CreateFileWriter(**CsvPath);
	if (!CsvFileWriter)
	{
		UE_LOGF(LogDumpMaterialExpressionInfo, Error, "Failed to open output file %ls", **CsvPath);
		return 1;
	}

	FDiagnosticTableWriterCSV CsvWriter(CsvFileWriter);
	DumpMaterialExpressionInfo::FDumper Dumper;
		
	Dumper.FindColumnProperties(
		bMatchAllExpressions ? nullptr : &RequestedExpressionPattern,
		bMatchAllColumns ? nullptr : &Columns
	);

	Dumper.WriteHeader(CsvWriter);
	CsvFileWriter->Flush();
	
	// Dump data
	for (TArray<FAssetData>::SizeType Index = 0; Index < Assets.Num(); Index++)
	{
		const FAssetData& AssetData = Assets[Index];

		bool bInclude = (bMatchAllMaterials || FRegexMatcher(RequestedMaterialPattern, AssetData.GetFullName()).FindNext());
		if (!bInclude)
		{
			continue;
		}
		
		const FMaterialExpressionCollection* Expressions = nullptr;
		if (UMaterial* Material = Cast<UMaterial>(AssetData.GetAsset()))
		{
			Expressions = &Material->GetExpressionCollection();
		}
		else if (UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(AssetData.GetAsset()))
		{
			Expressions = &MaterialFunction->GetExpressionCollection();
		}
		check(Expressions);

		Dumper.DumpAsset(
			AssetData, 
			*Expressions, 
			CsvWriter, 
			bMatchAllExpressions ? nullptr : &RequestedExpressionPattern
		);

		// Print progress
		uint64 NumAssetsDone = Index+1;
		if (NumAssetsDone % 100 == 0 || NumAssetsDone == Assets.Num())
		{
			UE_LOGF(LogDumpMaterialExpressionInfo, Display, "%llu/%lld done (%f%%)", (uint64)NumAssetsDone, (uint64)Assets.Num(), (float)NumAssetsDone * 100.0f / Assets.Num());
		}
	}

	CsvFileWriter->Flush();

	return 0;
}
