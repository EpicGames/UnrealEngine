// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpMaterialExpressionsCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionComment.h"
#include "Materials/MaterialExpressionComposite.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionMaterialLayerOutput.h"
#include "Materials/MaterialExpressionNamedReroute.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstance.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpMaterialExpressionsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogDumpMaterialExpressionsCommandlet, Log, All);

FString GetFormattedText(const FString& InText)
{
	FString OutText = InText;
	OutText = InText.Replace(TEXT("\n"), TEXT(" "));
	if (OutText.IsEmpty())
	{
		OutText = TEXT("N/A");
	}
	return OutText;
};

FString GenerateSpacePadding(int32 MaxLen, int32 TextLen)
{
	FString Padding;
	for (int i = 0; i < MaxLen - TextLen; ++i)
	{
		Padding += TEXT(" ");
	}
	return Padding;
};

void WriteLine(FArchive* FileWriter, const TArray<FString>& FieldNames, const TArray<uint32>& MaxFieldLengths)
{
	check(FieldNames.Num() >= MaxFieldLengths.Num());

	FString OutputLine;
	for (int32 i = 0; i < FieldNames.Num(); ++i)
	{
		// The last field doesn't need space padding, it changes to a new line.
		FString Padding = (i + 1 < FieldNames.Num()) ? GenerateSpacePadding(MaxFieldLengths[i], FieldNames[i].Len()) : TEXT("\n");
		OutputLine += (FieldNames[i] + Padding);
	}
	FileWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OutputLine).Get()), OutputLine.Len());
};

void WriteOutMaterialExpressions(FDiagnosticTableWriterCSV& CsvFile)
{
	struct FMaterialExpressionInfo
	{
		FString Name;
		FString Keywords;
		FString CreationName;
		FString CreationDescription;
		FString Caption;
		FString Description;
		FString Tooltip;
		FString	Type;
		FString ClassFlags;
		FString ShowInCreateMenu;
		int32 Uses = 0;
	};
	TMap<UMaterialExpression*, FMaterialExpressionInfo> MaterialExpressionInfos;

	// Collect all default material expression objects
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		// Skip the base UMaterialExpression class
		if (!Class->HasAnyClassFlags(CLASS_Abstract))
		{
			if (UMaterialExpression* DefaultExpression = Cast<UMaterialExpression>(Class->GetDefaultObject()))
			{
				const bool bClassDeprecated = Class->HasAnyClassFlags(CLASS_Deprecated);
				const bool bControlFlow = Class->HasMetaData("MaterialControlFlow");
				const bool bNewHLSLGenerator = Class->HasMetaData("NewMaterialTranslator");

				// If the expression is listed in the material node creation dropdown menu
				// See class exclusions in:
				//    MaterialExpressionClasses::InitMaterialExpressionClasses()
				//    FMaterialEditorUtilities::AddMaterialExpressionCategory() 
				//    and IsAllowedIn in UMaterialExpression::IsAllowedIn and overridden methods
				bool bShowInCreateMenu = !bClassDeprecated
						&& DefaultExpression->IsAllowedIn(UMaterial::StaticClass()->GetDefaultObject())
						&& Class != UMaterialExpressionMaterialLayerOutput::StaticClass()
						&& Class != UMaterialExpressionNamedRerouteUsage::StaticClass()
						&& Class != UMaterialExpressionComposite::StaticClass();

				const bool bCollapseCategories = Class->HasAnyClassFlags(CLASS_CollapseCategories);
				const bool bHideCategories = Class->HasMetaData(TEXT("HideCategories"));
				FString ClassFlags;
				if (Class->HasAnyClassFlags(CLASS_MinimalAPI)) { ClassFlags = TEXT("MinimalAPI"); }
				if (!ClassFlags.IsEmpty() && bCollapseCategories) { ClassFlags += TEXT("|"); }
				if (bCollapseCategories) { ClassFlags += TEXT("CollapseCategories"); }
				if (!ClassFlags.IsEmpty() && bHideCategories) { ClassFlags += TEXT("|"); }
				if (bHideCategories) { ClassFlags += Class->GetMetaData(TEXT("HideCategories")); }

				FString ExpressionType;
				if (bControlFlow) { ExpressionType = TEXT("ControlFlow"); }
				if (!ExpressionType.IsEmpty() && bNewHLSLGenerator) { ExpressionType += TEXT("|"); }
				if (bNewHLSLGenerator) { ExpressionType += TEXT("HLSLGenerator"); }
				if (!ExpressionType.IsEmpty() && bClassDeprecated) { ExpressionType += TEXT("|"); }
				if (bClassDeprecated) { ExpressionType += TEXT("CLASS_Deprecated"); }

				TArray<FString> MultilineCaption;
				DefaultExpression->GetCaption(MultilineCaption);
				FString Caption;
				for (const FString& Line : MultilineCaption)
				{
					Caption += Line;
				}

				TArray<FString> MultilineToolTip;
				DefaultExpression->GetExpressionToolTip(MultilineToolTip);
				FString Tooltip;
				for (const FString& Line : MultilineToolTip)
				{
					Tooltip += Line;
				}

				FString DisplayName = Class->GetMetaData(TEXT("DisplayName"));
				FString CreationName = DefaultExpression->GetCreationName().ToString();

				FMaterialExpressionInfo& ExpressionInfo = MaterialExpressionInfos.Add(DefaultExpression);
				ExpressionInfo.Name = Class->GetName().Mid(FCString::Strlen(TEXT("MaterialExpression")));
				ExpressionInfo.Keywords = DefaultExpression->GetKeywords().ToString();
				ExpressionInfo.CreationName = (CreationName.IsEmpty() ? (DisplayName.IsEmpty() ? ExpressionInfo.Name : DisplayName) : CreationName);
				ExpressionInfo.CreationDescription = DefaultExpression->GetCreationDescription().ToString();
				ExpressionInfo.Caption = Caption;
				ExpressionInfo.Description = DefaultExpression->GetDescription();
				ExpressionInfo.Tooltip = Tooltip;
				ExpressionInfo.Type = ExpressionType;
				ExpressionInfo.ClassFlags = ClassFlags;
				ExpressionInfo.ShowInCreateMenu = bShowInCreateMenu ? TEXT("Yes") : TEXT("No");
			}
		}
	}

	// Collect all materials for current project and count how many times each material expressions is referenced
	TArray<FAssetData> MaterialList;
	FARFilter MaterialAssetFilter;
	{
		MaterialAssetFilter.bRecursiveClasses = true;
		MaterialAssetFilter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());
		MaterialAssetFilter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());
	}
	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.SearchAllAssets(true);
	AssetRegistry.GetAssets(MaterialAssetFilter, MaterialList);

	for (const FAssetData& AssetData : MaterialList)
	{
		UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset());
		if (!MaterialInterface)
		{
			continue;
		}

		UMaterial* Material = MaterialInterface->GetMaterial();
		if (!Material)
		{
			continue;
		}

		for (const TObjectPtr<UMaterialExpression>& MaterialExpression : Material->GetExpressions())
		{
			const UMaterialExpression* DefaultExpression = Cast<UMaterialExpression>(MaterialExpression->GetClass()->GetDefaultObject());
			if (!DefaultExpression)
			{
				continue;
			}

			FMaterialExpressionInfo* ExpressionInfo = MaterialExpressionInfos.Find(DefaultExpression);
			if (!ExpressionInfo)
			{
				continue;
			}

			// Increment use count for this material expression type
			ExpressionInfo->Uses++;
		}
	}

	// Write the material expression list to a text file
	CsvFile.AddColumn(TEXT("NAME"));
	CsvFile.AddColumn(TEXT("TYPE"));
	CsvFile.AddColumn(TEXT("USES"));
	CsvFile.AddColumn(TEXT("CLASS_FLAGS"));
	CsvFile.AddColumn(TEXT("SHOW_IN_CREATE_MENU"));
	CsvFile.AddColumn(TEXT("KEYWORDS"));
	CsvFile.AddColumn(TEXT("CREATION_NAME"));
	CsvFile.AddColumn(TEXT("CREATION_DESCRIPTION"));
	CsvFile.AddColumn(TEXT("CAPTION"));
	CsvFile.AddColumn(TEXT("DESCRIPTION"));
	CsvFile.AddColumn(TEXT("TOOLTIP"));
	CsvFile.CycleRow();

	for (auto Iter = MaterialExpressionInfos.CreateConstIterator(); Iter; ++Iter)
	{
		const FMaterialExpressionInfo& ExpressionInfo = Iter->Value;
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Name));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Type));
		CsvFile.AddColumn(*FString::FromInt(ExpressionInfo.Uses));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.ClassFlags));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.ShowInCreateMenu));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Keywords));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.CreationName));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.CreationDescription));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Caption));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Description));
		CsvFile.AddColumn(*GetFormattedText(ExpressionInfo.Tooltip));
		CsvFile.CycleRow();
	}
}

void WriteOutMaterialFunctions(FDiagnosticTableWriterCSV& CsvFile)
{
	struct FMaterialFunctionInfo
	{
		FString Name;
		FString Description;
		FString Path;
	};
	TArray<FMaterialFunctionInfo> MaterialFunctionInfos;

	// See UMaterialGraphSchema::GetMaterialFunctionActions for reference
	TArray<FAssetData> AssetDataList;
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByClass(UMaterialFunction::StaticClass()->GetClassPathName(), AssetDataList);
	for (const FAssetData& AssetData : AssetDataList)
	{
		// If this is a function that is selected to be exposed to the library
		if (AssetData.GetTagValueRef<bool>("bExposeToLibrary"))
		{
			const FString FunctionPathName = AssetData.GetObjectPathString();
			const FString Description = AssetData.GetTagValueRef<FText>("Description").ToString();

			FString FunctionName = FunctionPathName;
			int32 PeriodIndex = FunctionPathName.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (PeriodIndex != INDEX_NONE)
			{
				FunctionName = FunctionPathName.Right(FunctionPathName.Len() - PeriodIndex - 1);
			}

			FMaterialFunctionInfo FunctionInfo;
			FunctionInfo.Name = FunctionName;
			FunctionInfo.Description = Description;
			FunctionInfo.Path = FunctionPathName;
			MaterialFunctionInfos.Add(FunctionInfo);
		}
	}

	// Write the material expression list to a text file
	CsvFile.AddColumn(TEXT("NAME"));
	CsvFile.AddColumn(TEXT("DESCRIPTION"));
	CsvFile.AddColumn(TEXT("PATH"));
	CsvFile.CycleRow();

	for (FMaterialFunctionInfo& FunctionInfo : MaterialFunctionInfos)
	{
		CsvFile.AddColumn(*GetFormattedText(FunctionInfo.Name));
		CsvFile.AddColumn(*GetFormattedText(FunctionInfo.Description));
		CsvFile.AddColumn(*GetFormattedText(FunctionInfo.Path));
		CsvFile.CycleRow();
	}
}

UDumpMaterialExpressionsCommandlet::UDumpMaterialExpressionsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UDumpMaterialExpressionsCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	if (Switches.Contains("help"))
	{
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "DumpMaterialExpressions");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "This commandlet will dump to a plain text file an info table of all material expressions in the engine and the plugins enabled on the project.");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "The output fields include:");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Name - The class name of the material expression");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Type - ControlFlow | HLSLGenerator | CLASS_Deprecated");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "ShowInCreateMenu - If the expression appears in the create node dropdown menu");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "CreationName - The name displayed in the create node dropdown menu to add an expression");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "CreationDescription - The tooltip displayed on the CreationName in the create node dropdown menu");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Caption - The caption displayed on the material expression node");
		UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Tooltip - The tooltip displayed on the material expression node");
		return 0;
	}

	const FString OutputFilePath = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("MaterialEditor"), TEXT("MaterialExpressions.csv"));
	TUniquePtr<FArchive> CSVTableFile = TUniquePtr<FArchive>{ IFileManager::Get().CreateFileWriter(*OutputFilePath) };
	FDiagnosticTableWriterCSV CsvFile(CSVTableFile.Get());

	WriteOutMaterialExpressions(CsvFile);


	const FString MatFuncOutputFilePath = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("MaterialEditor"), TEXT("MaterialFunctions.csv"));
	TUniquePtr<FArchive> MatFuncCSVTableFile = TUniquePtr<FArchive>{ IFileManager::Get().CreateFileWriter(*MatFuncOutputFilePath) };
	FDiagnosticTableWriterCSV MatFuncCsvFile(MatFuncCSVTableFile.Get());

	WriteOutMaterialFunctions(MatFuncCsvFile);

	UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Results are written to %ls", *OutputFilePath);
	UE_LOGF(LogDumpMaterialExpressionsCommandlet, Log, "Results are written to %ls", *MatFuncOutputFilePath);

	return 0;
}
