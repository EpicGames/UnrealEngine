// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpMaterialInfo.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "MaterialStatsCommon.h"
#include "MaterialShared.h"
#include "MaterialDomain.h"
#include "ShaderCompiler.h"
#include "Algo/Accumulate.h"
#include "MaterialShaderType.h"
#include "RendererUtils.h"
#include "Internationalization/Regex.h"
#include "String/ParseTokens.h"
#include "ProfilingDebugging/DiagnosticTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpMaterialInfo)

DEFINE_LOG_CATEGORY_STATIC(LogDumpMaterialInfo, Log, All);

namespace MaterialInfo
{
	class FOutput;

	// Material data from which properties are read to be dumped to csv
	struct FPropertyDumpInput
	{
		UMaterial* Material;
		FMaterialResource* MaterialResource;
		FMaterialRelevance MaterialRelevance;
		TArray<const FShaderType*>* ShaderTypes;
	};

	// Set of material properties that can be dumped
	struct FPropertySet
	{
		using FDumpFunction = TFunction<void(const FPropertyDumpInput& Input, FOutput& Output)>;

		// Names of all the properties in this set, used as column headers and for filtering
		TArray<FString> PropertyNames;
		// Read each property value in this set from Input, and write it to Output, in order specified by PropertyNames
		FDumpFunction DumpFunction;

		FPropertySet() { }
		FPropertySet(TArray<FString> InNames, FDumpFunction InDumpFunction)
			: PropertyNames(InNames)
			, DumpFunction(InDumpFunction)
		{ }
		FPropertySet(FString InName, FDumpFunction InDumpFunction)
			: PropertyNames( { InName } )
			, DumpFunction(InDumpFunction)
		{ }

		template<typename TValue, TValue(FMaterialResource::* MemberAccessor)() const>
		static FPropertySet FromAccessor(const FString& InName);

		template<typename TValue, TValue(FMaterialResource::* MemberAccessor)() const>
		static FPropertySet FromEnumAccessor(const FString& InName);
	};

	struct FPropertyValue
	{
		// Column name
		const FString& Name;
		// Value as string view, valid until FOutput::Reset is called.
		FStringView Value;
	};

	struct FSlice
	{
		int Start;
		int End;
	};

	// Class to efficiently write out property values to a stringbuffer and verify the output.
	class FOutput
	{
		int NumExpectedValues = 0;
		// Storage for output strings
		TStringBuilder<2048> Buffer;
		TArray<FSlice> BufferSlices;
		// Populated after all values have been written.
		TArray<FPropertyValue> Values;

	public:
		void DumpPropertySet(const FPropertySet& PropertySet, const FPropertyDumpInput& Input)
		{
			NumExpectedValues = PropertySet.PropertyNames.Num();
			PropertySet.DumpFunction(Input, *this);
			check(NumExpectedValues == 0);
			for (int Index = 0; Index < BufferSlices.Num(); Index++)
			{
				const FSlice& Slice = BufferSlices[Index];
				Values.Push( { PropertySet.PropertyNames[Index], Buffer.ToView().SubStr(Slice.Start, Slice.End - Slice.Start) });
			}
			BufferSlices.Reset();
		}

		const TArray<FPropertyValue>& GetValues()
		{
			return Values;
		}

		void Reset()
		{
			NumExpectedValues = 0;
			Buffer.Reset();
			BufferSlices.Reset();
			Values.Reset();
		}

		template <typename FmtType, typename... Types>
		void WriteFormat(const FmtType& Fmt, Types... Args)
		{
			check(NumExpectedValues-- > 0);
			int Start = Buffer.Len();
			Buffer.Appendf(Fmt, Forward<Types>(Args)...);
			int End = Buffer.Len();
			BufferSlices.Push( { Start, End } );
		}

		void Write(int32 Value)
		{
			WriteFormat(TEXT("%d"), Value);
		}

		void Write(uint32 Value)
		{
			WriteFormat(TEXT("%u"), Value);
		}

		void Write(bool Value)
		{
			WriteFormat(TEXT("%d"), Value);
		}

		void Write(float Value)
		{
			WriteFormat(TEXT("%f"), Value);
		}

		void Write(const FString& Value)
		{
			WriteFormat(TEXT("%s"), *Value);
		}
	};

	template<typename TValue, TValue(FMaterialResource::*MemberAccessor)() const>
	FPropertySet FPropertySet::FromAccessor(const FString& InName)
	{ 
		return FPropertySet(
			InName, 
			[](const FPropertyDumpInput& Input, FOutput& Output)
			{ 
				Output.Write((Input.MaterialResource->*MemberAccessor)()); 
			});
	}

	template<typename TValue, TValue(FMaterialResource::*MemberAccessor)() const>
	FPropertySet FPropertySet::FromEnumAccessor(const FString& InName)
	{ 
		return FPropertySet(
			InName, 
			[](const FPropertyDumpInput& Input, FOutput& Output)
			{ 
				Output.Write(UEnum::GetValueOrBitfieldAsString((Input.MaterialResource->*MemberAccessor)()));
			});
	}
}


UDumpMaterialInfoCommandlet::UDumpMaterialInfoCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TArray<MaterialInfo::FPropertySet> GetMaterialInfoProperties()
{
#if WITH_EDITOR
	using namespace MaterialInfo;
	TArray<FPropertySet> PropertySets = 
	{
		FPropertySet::FromAccessor<int32, &FMaterialResource::GetSamplerUsage>(TEXT("GetSamplerUsage")),
		FPropertySet
		{
			{ TEXT("NumUsedUVScalars"), TEXT("NumUsedCustomInterpolatorScalars") }, 
			[](const FPropertyDumpInput& Input, FOutput& Output)
			{
				uint32 NumUsedUVScalars = 0;
				uint32 NumUsedCustomInterpolatorScalars = 0;
				Input.MaterialResource->GetUserInterpolatorUsage(NumUsedUVScalars, NumUsedCustomInterpolatorScalars);
				Output.Write(NumUsedUVScalars);
				Output.Write(NumUsedCustomInterpolatorScalars);
			} 
		},
		FPropertySet::FromAccessor<uint32, &FMaterialResource::GetEstimatedNumVirtualTextureLookups>(TEXT("GetEstimatedNumVirtualTextureLookups")),
		FPropertySet
		{
			{ TEXT("LWCUsagesVS"), TEXT("LWCUsagesPS"), TEXT("LWCUsagesCS") }, 
			[](const FPropertyDumpInput& Input, FOutput& Output)
			{
				FMaterialResource::FLWCUsagesArray LWCUsagesVS {};
				FMaterialResource::FLWCUsagesArray LWCUsagesPS {};
				FMaterialResource::FLWCUsagesArray LWCUsagesCS {};
				Input.MaterialResource->GetEstimatedLWCFuncUsages(LWCUsagesVS, LWCUsagesPS, LWCUsagesCS);
				Output.Write(Algo::Accumulate(LWCUsagesVS, 0));
				Output.Write(Algo::Accumulate(LWCUsagesPS, 0));
				Output.Write(Algo::Accumulate(LWCUsagesCS, 0));
			} 
		},
		FPropertySet::FromAccessor<uint32, &FMaterialResource::GetNumVirtualTextureStacks>(TEXT("GetNumVirtualTextureStacks")),
		//FPropertySet::FromAccessor<FString, &FMaterialResource::GetMaterialUsageDescription>(TEXT("MaterialUsageDescription")),
		//FPropertySet::FromAccessor<void, &FMaterialResource::GetShaderMapId>(TEXT("ShaderMapId")),
		//FPropertySet::FromAccessor<void, &FMaterialResource::GetStaticParameterSet>(TEXT("StaticParameterSet")),
		FPropertySet::FromEnumAccessor<EMaterialDomain, &FMaterialResource::GetMaterialDomain>(TEXT("GetMaterialDomain")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTranslucencyWritingFrontLayerTransparency>(TEXT("IsTranslucencyWritingFrontLayerTransparency")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTranslucencyWritingFrontLayerTransparency>(TEXT("IsTranslucencyWritingFrontLayerTransparency")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTangentSpaceNormal>(TEXT("IsTangentSpaceNormal")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldGenerateSphericalParticleNormals>(TEXT("ShouldGenerateSphericalParticleNormals")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldDisableDepthTest>(TEXT("ShouldDisableDepthTest")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldWriteOnlyAlpha>(TEXT("ShouldWriteOnlyAlpha")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldEnableResponsiveAA>(TEXT("ShouldEnableResponsiveAA")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldDoSSR>(TEXT("ShouldDoSSR")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldDoContactShadows>(TEXT("ShouldDoContactShadows")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasPixelAnimation>(TEXT("HasPixelAnimation")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsLightFunction>(TEXT("IsLightFunction")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithEditorCompositing>(TEXT("IsUsedWithEditorCompositing")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsDeferredDecal>(TEXT("IsDeferredDecal")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsVolumetricPrimitive>(TEXT("IsVolumetricPrimitive")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsWireframe>(TEXT("IsWireframe")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsVariableRateShadingAllowed>(TEXT("IsVariableRateShadingAllowed")),
		FPropertySet::FromEnumAccessor<EMaterialShadingRate, &FMaterialResource::GetShadingRate>(TEXT("GetShadingRate")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUIMaterial>(TEXT("IsUIMaterial")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsPostProcessMaterial>(TEXT("IsPostProcessMaterial")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsSpecialEngineMaterial>(TEXT("IsSpecialEngineMaterial")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithSkeletalMesh>(TEXT("IsUsedWithSkeletalMesh")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithLandscape>(TEXT("IsUsedWithLandscape")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithParticleSystem>(TEXT("IsUsedWithParticleSystem")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithParticleSprites>(TEXT("IsUsedWithParticleSprites")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithBeamTrails>(TEXT("IsUsedWithBeamTrails")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithMeshParticles>(TEXT("IsUsedWithMeshParticles")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithNiagaraSprites>(TEXT("IsUsedWithNiagaraSprites")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithNiagaraRibbons>(TEXT("IsUsedWithNiagaraRibbons")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithNiagaraMeshParticles>(TEXT("IsUsedWithNiagaraMeshParticles")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithStaticLighting>(TEXT("IsUsedWithStaticLighting")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithMorphTargets>(TEXT("IsUsedWithMorphTargets")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithSplineMeshes>(TEXT("IsUsedWithSplineMeshes")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithInstancedStaticMeshes>(TEXT("IsUsedWithInstancedStaticMeshes")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithGeometryCollections>(TEXT("IsUsedWithGeometryCollections")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithAPEXCloth>(TEXT("IsUsedWithAPEXCloth")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithGeometryCache>(TEXT("IsUsedWithGeometryCache")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithWater>(TEXT("IsUsedWithWater")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithHairStrands>(TEXT("IsUsedWithHairStrands")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithLidarPointCloud>(TEXT("IsUsedWithLidarPointCloud")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithVirtualHeightfieldMesh>(TEXT("IsUsedWithVirtualHeightfieldMesh")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithNeuralNetworks>(TEXT("IsUsedWithNeuralNetworks")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithNanite>(TEXT("IsUsedWithNanite")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithVoxels>(TEXT("IsUsedWithVoxels")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithCurves>(TEXT("IsUsedWithCurves")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithVolumetricCloud>(TEXT("IsUsedWithVolumetricCloud")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsedWithHeterogeneousVolumes>(TEXT("IsUsedWithHeterogeneousVolumes")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsFullyRough>(TEXT("IsFullyRough")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::GetForceCompatibleWithLightFunctionAtlas>(TEXT("GetForceCompatibleWithLightFunctionAtlas")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::UseNormalCurvatureToRoughness>(TEXT("UseNormalCurvatureToRoughness")),
		FPropertySet::FromEnumAccessor<EMaterialFloatPrecisionMode, &FMaterialResource::GetMaterialFloatPrecisionMode>(TEXT("GetMaterialFloatPrecisionMode")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsingAlphaToCoverage>(TEXT("IsUsingAlphaToCoverage")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsingPreintegratedGFForSimpleIBL>(TEXT("IsUsingPreintegratedGFForSimpleIBL")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsingHQForwardReflections>(TEXT("IsUsingHQForwardReflections")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::GetForwardBlendsSkyLightCubemaps>(TEXT("GetForwardBlendsSkyLightCubemaps")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsUsingPlanarForwardReflections>(TEXT("IsUsingPlanarForwardReflections")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsNonmetal>(TEXT("IsNonmetal")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::UseLmDirectionality>(TEXT("UseLmDirectionality")),
		FPropertySet::FromEnumAccessor<EBlendMode, &FMaterialResource::GetBlendMode>(TEXT("GetBlendMode")),
		FPropertySet::FromEnumAccessor<ERefractionMode, &FMaterialResource::GetRefractionMode>(TEXT("GetRefractionMode")),
		FPropertySet::FromAccessor<uint32, &FMaterialResource::GetMaterialDecalResponse>(TEXT("GetMaterialDecalResponse")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasBaseColorConnected>(TEXT("HasBaseColorConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasNormalConnected>(TEXT("HasNormalConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasRoughnessConnected>(TEXT("HasRoughnessConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasSpecularConnected>(TEXT("HasSpecularConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasMetallicConnected>(TEXT("HasMetallicConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasEmissiveColorConnected>(TEXT("HasEmissiveColorConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasAnisotropyConnected>(TEXT("HasAnisotropyConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasAmbientOcclusionConnected>(TEXT("HasAmbientOcclusionConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasDisplacementConnected>(TEXT("HasDisplacementConnected")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsSubstrateMaterial>(TEXT("IsSubstrateMaterial")),
		//FPropertySet::FromAccessor<bool, &FMaterialResource::HasMaterialPropertyConnected>(TEXT("HasMaterialPropertyConnected")),
		//FPropertySet::FromAccessor<FMaterialShadingModelField, &FMaterialResource::GetShadingModels>(TEXT("GetShadingModels")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsShadingModelFromMaterialExpression>(TEXT("IsShadingModelFromMaterialExpression")),
		FPropertySet::FromEnumAccessor<ETranslucencyLightingMode, &FMaterialResource::GetTranslucencyLightingMode>(TEXT("GetTranslucencyLightingMode")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetOpacityMaskClipValue>(TEXT("GetOpacityMaskClipValue")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::GetCastDynamicShadowAsMasked>(TEXT("GetCastDynamicShadowAsMasked")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsDistorted>(TEXT("IsDistorted")),
		FPropertySet::FromEnumAccessor<ERefractionCoverageMode, &FMaterialResource::GetRefractionCoverageMode>(TEXT("GetRefractionCoverageMode")),
		FPropertySet::FromEnumAccessor<EPixelDepthOffsetMode, &FMaterialResource::GetPixelDepthOffsetMode>(TEXT("GetPixelDepthOffsetMode")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucencyDirectionalLightingIntensity>(TEXT("GetTranslucencyDirectionalLightingIntensity")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentShadowDensityScale>(TEXT("GetTranslucentShadowDensityScale")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentSelfShadowDensityScale>(TEXT("GetTranslucentSelfShadowDensityScale")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentSelfShadowSecondDensityScale>(TEXT("GetTranslucentSelfShadowSecondDensityScale")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentSelfShadowSecondOpacity>(TEXT("GetTranslucentSelfShadowSecondOpacity")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentBackscatteringExponent>(TEXT("GetTranslucentBackscatteringExponent")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTranslucencyAfterDOFEnabled>(TEXT("IsTranslucencyAfterDOFEnabled")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTranslucencyAfterMotionBlurEnabled>(TEXT("IsTranslucencyAfterMotionBlurEnabled")),
		//FPropertySet::FromAccessor<bool, &FMaterialResource::IsDualBlendingEnabled>(TEXT("IsDualBlendingEnabled")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsMobileSeparateTranslucencyEnabled>(TEXT("IsMobileSeparateTranslucencyEnabled")),
		//FPropertySet::FromAccessor<FDisplacementScaling, &FMaterialResource::GetDisplacementScaling>(TEXT("GetDisplacementScaling")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsDisplacementFadeEnabled>(TEXT("IsDisplacementFadeEnabled")),
		//FPropertySet::FromAccessor<FDisplacementFadeRange, &FMaterialResource::GetDisplacementFadeRange>(TEXT("GetDisplacementFadeRange")),
		//FPropertySet::FromAccessor<FLinearColor, &FMaterialResource::GetTranslucentMultipleScatteringExtinction>(TEXT("GetTranslucentMultipleScatteringExtinction")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetTranslucentShadowStartOffset>(TEXT("GetTranslucentShadowStartOffset")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsMasked>(TEXT("IsMasked")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsDitherMasked>(TEXT("IsDitherMasked")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::AllowNegativeEmissiveColor>(TEXT("AllowNegativeEmissiveColor")),
		//FPropertySet::FromAccessor<FString, &FMaterialResource::GetFriendlyName>(TEXT("GetFriendlyName")),
		FPropertySet::FromAccessor<FString, &FMaterialResource::GetAssetName>(TEXT("GetAssetName")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::RequiresSynchronousCompilation>(TEXT("RequiresSynchronousCompilation")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsDefaultMaterial>(TEXT("IsDefaultMaterial")),
		FPropertySet::FromAccessor<int32, &FMaterialResource::GetNumCustomizedUVs>(TEXT("GetNumCustomizedUVs")),
		FPropertySet::FromAccessor<int32, &FMaterialResource::GetBlendableLocation>(TEXT("GetBlendableLocation")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::GetBlendableOutputAlpha>(TEXT("GetBlendableOutputAlpha")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::GetDisablePreExposureScale>(TEXT("GetDisablePreExposureScale")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsStencilTestEnabled>(TEXT("IsStencilTestEnabled")),
		FPropertySet::FromAccessor<uint32, &FMaterialResource::GetStencilRefValue>(TEXT("GetStencilRefValue")),
		FPropertySet::FromAccessor<uint32, &FMaterialResource::GetStencilCompare>(TEXT("GetStencilCompare")),
		FPropertySet::FromAccessor<float, &FMaterialResource::GetRefractionDepthBiasValue>(TEXT("GetRefractionDepthBiasValue")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldApplyFogging>(TEXT("ShouldApplyFogging")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldApplyCloudFogging>(TEXT("ShouldApplyCloudFogging")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ShouldAlwaysEvaluateWorldPositionOffset>(TEXT("ShouldAlwaysEvaluateWorldPositionOffset")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsSky>(TEXT("IsSky")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::ComputeFogPerPixel>(TEXT("ComputeFogPerPixel")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasPerInstanceCustomData>(TEXT("HasPerInstanceCustomData")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasPerInstanceRandom>(TEXT("HasPerInstanceRandom")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasVertexInterpolator>(TEXT("HasVertexInterpolator")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasRuntimeVirtualTextureOutput>(TEXT("HasRuntimeVirtualTextureOutput")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::CastsRayTracedShadows>(TEXT("CastsRayTracedShadows")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::IsTessellationEnabled>(TEXT("IsTessellationEnabled")),
		FPropertySet::FromAccessor<bool, &FMaterialResource::HasRenderTracePhysicalMaterialOutputs>(TEXT("HasRenderTracePhysicalMaterialOutputs")),
		FPropertySet::FromAccessor<uint16, &FMaterialResource::GetPreshaderGap>(TEXT("GetPreshaderGap")),
		FPropertySet::FromAccessor<int32, &FMaterialResource::GetNeuralProfileId>(TEXT("GetNeuralProfileId")),

		FPropertySet
		{
			{
				TEXT("ShadingModelMask"),
				TEXT("SubstrateUintPerPixel"),
				TEXT("SubstrateClosureCountMask"),
				TEXT("SubstrateTileTypeMask"),
				TEXT("bOpaque"),
				TEXT("bMasked"),
				TEXT("bDistortion"),
				TEXT("bHairStrands"),
				TEXT("bTwoSided"),
				TEXT("bSeparateTranslucency"),
				TEXT("bTranslucencyModulate"),
				TEXT("bPostMotionBlurTranslucency"),
				TEXT("bNormalTranslucency"),
				TEXT("bUsesSceneColorCopy"),
				TEXT("bOutputsTranslucentVelocity"),
				TEXT("bUsesGlobalDistanceField"),
				TEXT("bUsesWorldPositionOffset"),
				TEXT("bUsesDisplacement"),
				TEXT("bUsesPixelDepthOffset"),
				TEXT("bDecal"),
				TEXT("bTranslucentSurfaceLighting"),
				TEXT("bUsesSceneDepth"),
				TEXT("bUsesSkyMaterial"),
				TEXT("bUsesSingleLayerWaterMaterial"),
				TEXT("bHasVolumeMaterialDomain"),
				TEXT("CustomDepthStencilUsageMask"),
				TEXT("bUsesDistanceCullFade"),
				TEXT("bDisableDepthTest"),
				TEXT("bUsesAnisotropy"),
				TEXT("bIsLightFunctionAtlasCompatible"),
			},
			[](const FPropertyDumpInput& Input, FOutput& Output)
			{
				Output.Write(Input.MaterialRelevance.ShadingModelMask);
				Output.Write(Input.MaterialRelevance.SubstrateUintPerPixel);
				Output.Write(Input.MaterialRelevance.SubstrateClosureCountMask);
				Output.Write(Input.MaterialRelevance.SubstrateTileTypeMask);
				Output.Write(Input.MaterialRelevance.bOpaque);
				Output.Write(Input.MaterialRelevance.bMasked);
				Output.Write(Input.MaterialRelevance.bDistortion);
				Output.Write(Input.MaterialRelevance.bHairStrands);
				Output.Write(Input.MaterialRelevance.bTwoSided);
				Output.Write(Input.MaterialRelevance.bSeparateTranslucency);
				Output.Write(Input.MaterialRelevance.bTranslucencyModulate);
				Output.Write(Input.MaterialRelevance.bPostMotionBlurTranslucency);
				Output.Write(Input.MaterialRelevance.bNormalTranslucency);
				Output.Write(Input.MaterialRelevance.bUsesSceneColorCopy);
				Output.Write(Input.MaterialRelevance.bOutputsTranslucentVelocity);
				Output.Write(Input.MaterialRelevance.bUsesGlobalDistanceField);
				Output.Write(Input.MaterialRelevance.bUsesWorldPositionOffset);
				Output.Write(Input.MaterialRelevance.bUsesDisplacement);
				Output.Write(Input.MaterialRelevance.bUsesPixelDepthOffset);
				Output.Write(Input.MaterialRelevance.bDecal);
				Output.Write(Input.MaterialRelevance.bTranslucentSurfaceLighting);
				Output.Write(Input.MaterialRelevance.bUsesSceneDepth);
				Output.Write(Input.MaterialRelevance.bUsesSkyMaterial);
				Output.Write(Input.MaterialRelevance.bUsesSingleLayerWaterMaterial);
				Output.Write(Input.MaterialRelevance.bHasVolumeMaterialDomain);
				Output.Write(Input.MaterialRelevance.CustomDepthStencilUsageMask);
				Output.Write(Input.MaterialRelevance.bUsesDistanceCullFade);
				Output.Write(Input.MaterialRelevance.bDisableDepthTest);
				Output.Write(Input.MaterialRelevance.bUsesAnisotropy);
				Output.Write(Input.MaterialRelevance.bIsLightFunctionAtlasCompatible);
			} 
		},
	};
	return PropertySets;
#else
	return {};
#endif
}

static void DumpMaterials(
	FDiagnosticTableWriterCSV& CsvWriter,
	TArrayView<FAssetData> MaterialInterfaceAssets,
	TArray<MaterialInfo::FPropertySet>& MaterialInfoProperties,
	TSet<FString>& Columns,
	ITargetPlatform* Platform,
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type MaterialQualityLevel,
	bool bMatchAllMaterials,
	const FRegexPattern& RequestedMaterialPattern
);

int32 UDumpMaterialInfoCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogDumpMaterialInfo, Log, "DumpMaterialInfo");
		UE_LOGF(LogDumpMaterialInfo, Log, "This commandlet will dump information about materials.");
		UE_LOGF(LogDumpMaterialInfo, Log, "A typical way to invoke it is: <YourProject> -run=DumpMaterialInfo -targetplatform=Windows -unattended -sm6 -allowcommandletrendering -nomaterialshaderddc -csv=C:/output.csv");
		UE_LOGF(LogDumpMaterialInfo, Log, "");
		UE_LOGF(LogDumpMaterialInfo, Log, "Options:");
		UE_LOGF(LogDumpMaterialInfo, Log, " -help           Print this message");
		UE_LOGF(LogDumpMaterialInfo, Log, " -help=columns   Print the list of available columns");
		UE_LOGF(LogDumpMaterialInfo, Log, " -csv=filename   Writes the output to a CSV file");
		UE_LOGF(LogDumpMaterialInfo, Log, " -material=name  Only dump materials matching this material name or regular expression");
		UE_LOGF(LogDumpMaterialInfo, Log, " -columns=a,b    Comma-seperated list of the columns that should be included in the output");
		return 0;
	}

	FString* Help = ParamVals.Find(TEXT("help"));
	if (Help)
	{
		if ((*Help) == TEXT("columns"))
		{
			UE_LOGF(LogDumpMaterialInfo, Display, "Available columns:");
			for (const MaterialInfo::FPropertySet& PropertySet : GetMaterialInfoProperties())
			{
				for (const FString& PropertyName : PropertySet.PropertyNames)
				{
					UE_LOGF(LogDumpMaterialInfo, Display, "   %ls", *PropertyName);
				}
			}
		}
		else
		{
			UE_LOGF(LogDumpMaterialInfo, Error, "Unknown help option %ls", **Help);
			return 1;
		}
		return 0;
	}

	// Parse params
	FString* CsvPath = ParamVals.Find(TEXT("csv"));
	if (!CsvPath)
	{
		UE_LOGF(LogDumpMaterialInfo, Error, "No output CSV file path was specified");
		return 1;
	}
	FString* RequestedMaterialPatternString = ParamVals.Find(TEXT("material"));
	bool bMatchAllMaterials = !RequestedMaterialPatternString;
	FRegexPattern RequestedMaterialPattern(RequestedMaterialPatternString ? *RequestedMaterialPatternString : FString());
	FString* ColumnsString = ParamVals.Find(TEXT("columns"));
	TSet<FString> Columns {};
	if (ColumnsString)
	{
		UE::String::ParseTokens(*ColumnsString, TEXT(","), [&Columns](const auto& SubString)
		{
			Columns.Add(FString(SubString));
		}, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);
	}

	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::SM6;
	EMaterialQualityLevel::Type MaterialQualityLevel = EMaterialQualityLevel::High;
	EShaderPlatform ShaderPlatform = SP_PCD3D_SM6;

	// Get available material properties and filter them
	TArray<MaterialInfo::FPropertySet> MaterialInfoProperties = GetMaterialInfoProperties();
	if (!Columns.IsEmpty())
	{
		// Retain property sets that have at least 1 requested property
		MaterialInfoProperties = MaterialInfoProperties.FilterByPredicate([&Columns](const MaterialInfo::FPropertySet& PropertySet)
		{
			return PropertySet.PropertyNames.ContainsByPredicate([&Columns](const FString& PropertyName)
			{
				return Columns.Contains(PropertyName);
			});
		});
	}

	UE_LOGF(LogDumpMaterialInfo, Log, "Searching for materials within the project...");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> MaterialInterfaceAssets;
	{
		TArray<FAssetData> MaterialAssets;
		AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), MaterialAssets, true);
		UE_LOGF(LogDumpMaterialInfo, Log, "Found %d materials", MaterialAssets.Num());
		MaterialInterfaceAssets = MaterialAssets;
	}
	{
		TArray<FAssetData> MaterialInstanceAssets;
		AssetRegistry.GetAssetsByClass(UMaterialInstance::StaticClass()->GetClassPathName(), MaterialInstanceAssets, true);
		UE_LOGF(LogDumpMaterialInfo, Log, "Found %d material instances", MaterialInstanceAssets.Num());
		MaterialInterfaceAssets.Append(MaterialInstanceAssets);
	}

	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

	FArchive* CsvFileWriter = IFileManager::Get().CreateFileWriter(**CsvPath);
	if (!CsvFileWriter)
	{
		UE_LOGF(LogDumpMaterialInfo, Error, "Failed to open output file %ls", **CsvPath);
		return 1;
	}


	FDiagnosticTableWriterCSV CsvWriter(CsvFileWriter);

	// CSV header
	{
		for (const MaterialInfo::FPropertySet& Property : MaterialInfoProperties)
		{
			for (const FString& PropertyName : Property.PropertyNames)
			{
				if (Columns.IsEmpty() || Columns.Contains(PropertyName))
				{
					CsvWriter.AddColumn(TEXT("%s"), *PropertyName);
				}
			}
		}
		CsvWriter.CycleRow();
		CsvFileWriter->Flush();
	}

	for (ITargetPlatform* Platform : Platforms)
	{
		UE_LOGF(LogDumpMaterialInfo, Display, "Compiling shaders for %ls...", *Platform->PlatformName());

		const int MaxBatchSize = 1000;
		int NumBatches = FMath::DivideAndRoundUp(MaterialInterfaceAssets.Num(), MaxBatchSize);
		for (int BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			UE_LOGF(LogDumpMaterialInfo, Display, "Dumping batch %d of %d", BatchIndex, NumBatches);
			DumpMaterials(
				CsvWriter,
				TArrayView<FAssetData>(MaterialInterfaceAssets).Mid(BatchIndex * MaxBatchSize, MaxBatchSize),
				MaterialInfoProperties,
				Columns,
				Platform,
				ShaderPlatform,
				FeatureLevel,
				MaterialQualityLevel,
				bMatchAllMaterials,
				RequestedMaterialPattern);
		}

		CsvFileWriter->Flush();
	} // Platforms

	return 0;
}

static void DumpMaterials(
	FDiagnosticTableWriterCSV& CsvWriter,
	TArrayView<FAssetData> MaterialInterfaceAssets,
	TArray<MaterialInfo::FPropertySet>& MaterialInfoProperties,
	TSet<FString>& Columns,
	ITargetPlatform* Platform,
	EShaderPlatform ShaderPlatform,
	ERHIFeatureLevel::Type FeatureLevel,
	EMaterialQualityLevel::Type MaterialQualityLevel,
	bool bMatchAllMaterials,
	const FRegexPattern& RequestedMaterialPattern
)
{
	TSet<UMaterialInterface*> MaterialsToCompile;
	for (const FAssetData& AssetData : MaterialInterfaceAssets)
	{
		bool bInclude = (bMatchAllMaterials || FRegexMatcher(RequestedMaterialPattern, AssetData.GetFullName()).FindNext());
		if (!bInclude)
		{
			continue;
		}
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
		{
			UMaterial* Material = MaterialInterface->GetMaterial();
			if (Material)
			{
				UE_LOGF(LogDumpMaterialInfo, Display, "BeginCache for %ls", *MaterialInterface->GetFullName());
				MaterialInterface->BeginCacheForCookedPlatformData(Platform);
				// need to call this once for all objects before any calls to ProcessAsyncResults as otherwise we'll potentially upload
				// incremental/incomplete shadermaps to DDC (as this function actually triggers compilation, some compiles for a particular
				// material may finish before we've even started others - if we call ProcessAsyncResults in that case the associated shader
				// maps will think they are "finished" due to having no outstanding dependencies).
				if (!MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
				{
					MaterialsToCompile.Add(MaterialInterface);
				}
			}
		}
	}
	TSet<UMaterialInterface*> MaterialsToAnalyse = MaterialsToCompile;

	UE_LOGF(LogDumpMaterialInfo, Log, "Found %d materials to compile.", MaterialsToCompile.Num());

	static constexpr bool bLimitExecutationTime = false;
	int32 PreviousOutstandingJobs = 0;
	constexpr int32 MaxOutstandingJobs = 20000; // Having a max is a way to try to reduce memory usage.. otherwise outstanding jobs can reach 100k+ and use up 300gb committed memory
	// Submit all the jobs.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SubmitJobs);

		UE_LOGF(LogDumpMaterialInfo, Display, "Submit Jobs");

		while (MaterialsToCompile.Num())
		{
			for (auto It = MaterialsToCompile.CreateIterator(); It; ++It)
			{
				UMaterialInterface* MaterialInterface = *It;
				if (MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
				{
					It.RemoveCurrent();
					UE_LOGF(LogDumpMaterialInfo, Display, "Finished cache for %ls.", *MaterialInterface->GetFullName());
					UE_LOGF(LogDumpMaterialInfo, Display, "Materials remaining: %d", MaterialsToCompile.Num());
				}

				GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

				while (true)
				{
					const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
					if (CurrentOutstandingJobs != PreviousOutstandingJobs)
					{
						UE_LOGF(LogDumpMaterialInfo, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
						PreviousOutstandingJobs = CurrentOutstandingJobs;
					}

					// Flush rendering commands to release any RHI resources (shaders and shader maps).
					// Delete any FPendingCleanupObjects (shader maps).
					FlushRenderingCommands();

					if (CurrentOutstandingJobs < MaxOutstandingJobs)
					{
						break;
					}
					FPlatformProcess::Sleep(1);
				}
			}
		}
	}

	// Process the shader maps and save to the DDC.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessShaderCompileResults);

		UE_LOGF(LogDumpMaterialInfo, Log, "ProcessAsyncResults");

		while (GShaderCompilingManager->IsCompiling())
		{
			GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

			while (true)
			{
				const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
				if (CurrentOutstandingJobs != PreviousOutstandingJobs)
				{
					UE_LOGF(LogDumpMaterialInfo, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
					PreviousOutstandingJobs = CurrentOutstandingJobs;
				}

				// Flush rendering commands to release any RHI resources (shaders and shader maps).
				// Delete any FPendingCleanupObjects (shader maps).
				FlushRenderingCommands();

				if (CurrentOutstandingJobs < MaxOutstandingJobs)
				{
					break;
				}
				FPlatformProcess::Sleep(1);
			}
		}
	}

	// Look up compilation result for the materials
	TArray<const FVertexFactoryType*> VFTypes;
	TArray<const FShaderPipelineType*> PipelineTypes;
	TArray<const FShaderType*> ShaderTypes;
	for (UMaterialInterface* MaterialInterface : MaterialsToAnalyse)
	{
		VFTypes.Empty();
		PipelineTypes.Empty();
		ShaderTypes.Empty();

		UMaterial* Material = MaterialInterface->GetMaterial();
		if (Material)
		{
			TArray<FMaterialResource*> ResourcesToCache;

			FMaterialResource* CurrentResource = FindOrCreateMaterialResource(ResourcesToCache, Material, nullptr, ShaderPlatform, MaterialQualityLevel);
			check(CurrentResource);

			TMap<FName, TArray<FMaterialStatsUtils::FRepresentativeShaderInfo>> ShaderTypeNamesAndDescriptions;
			FMaterialStatsUtils::GetRepresentativeShaderTypesAndDescriptions(ShaderTypeNamesAndDescriptions, CurrentResource);

			for (auto& DescriptionPair : ShaderTypeNamesAndDescriptions)
			{
				const FVertexFactoryType* VFType = FindVertexFactoryType(DescriptionPair.Key);
				check(VFType);

				auto& DescriptionArray = DescriptionPair.Value;
				for (const FMaterialStatsUtils::FRepresentativeShaderInfo& ShaderInfo : DescriptionArray)
				{
					const FShaderType* ShaderType = FindShaderTypeByName(ShaderInfo.ShaderName);

					if (ShaderType && VFType)
					{
						VFTypes.Add(VFType);
						ShaderTypes.Add(ShaderType);
						PipelineTypes.Add(nullptr);
					}
				}
			}

			// Prepare the resource for compilation, but don't compile the completed shader map.
			const bool bSuccess = CurrentResource->CacheShaders(EMaterialShaderPrecompileMode::None);

			if (bSuccess)
			{
				// Compile just the types we want.
				CurrentResource->CacheGivenTypes(VFTypes, PipelineTypes, ShaderTypes);
			}

			if (!CurrentResource->IsGameThreadShaderMapComplete()) { UE_LOGF(LogDumpMaterialInfo, Warning, "Missing shader map data"); }

			FMaterialRelevance MaterialRelevance = CurrentResource->GetMaterialInterface()->GetRelevance(ShaderPlatform);

			// CSV line
			{
				MaterialInfo::FOutput Output;
				for (const MaterialInfo::FPropertySet& Property : MaterialInfoProperties)
				{
					MaterialInfo::FPropertyDumpInput Input;
					Input.Material = Material;
					Input.MaterialResource = CurrentResource;
					Input.MaterialRelevance = MaterialRelevance;
					Input.ShaderTypes = &ShaderTypes;
					Output.DumpPropertySet(Property, Input);
					for (const MaterialInfo::FPropertyValue& Value : Output.GetValues())
					{
						if (Columns.IsEmpty() || Columns.Contains(Value.Name))
						{
							CsvWriter.AddColumn(TEXT("%s"), *FString(Value.Value));
						}
					}
					Output.Reset();
				}
				CsvWriter.CycleRow();
			}

			FMaterial::DeferredDeleteArray(ResourcesToCache);
		}
	}

	// Perform cleanup and clear cached data for cooking.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ClearCachedCookedPlatformData);

		UE_LOGF(LogDumpMaterialInfo, Display, "Clear Cached Cooked Platform Data");

		for (const FAssetData& AssetData : MaterialInterfaceAssets)
		{
			if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
			{
				MaterialInterface->ClearAllCachedCookedPlatformData();
			}
		}
	}
}
