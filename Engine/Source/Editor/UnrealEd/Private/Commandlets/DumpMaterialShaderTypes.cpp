// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/DumpMaterialShaderTypes.h"
#include "AnalyticsET.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "GlobalShader.h"
#include "HAL/FileManager.h"
#include "IAnalyticsProviderET.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialDependencyWalker.h"
#include "Templates/Greater.h"
#include "MaterialShared.h"
#include "CollectionManagerTypes.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "Misc/EngineVersion.h"
#include "Experimental/Containers/RobinHoodHashTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DumpMaterialShaderTypes)

DEFINE_LOG_CATEGORY_STATIC(LogDumpMaterialShaderTypesCommandlet, Log, All);

class FShaderStatsGatheringContext
{
public:
	FShaderStatsGatheringContext() = delete;
	FShaderStatsGatheringContext(const FString& InFileName) : FileName(InFileName)
	{
		OutputFileName = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("MaterialStats"), FileName);
		DebugWriter = IFileManager::Get().CreateFileWriter(*OutputFileName);
	}
	~FShaderStatsGatheringContext()
	{
		DebugWriter->Close();
		delete DebugWriter;

		// Copy to the automation directory.
		const FString AutomationFilePath = FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"), FileName);
		IFileManager::Get().Copy(*AutomationFilePath, *OutputFileName);
	}

	void AddToGlobalShaderTypeHistogram(const TCHAR* GlobalShaderName)
	{
		FString Name(GlobalShaderName);
		if (int32* Existing = GlobalShaderTypeHistogram.Find(Name))
		{
			++(*Existing);
		}
		else
		{
			GlobalShaderTypeHistogram.FindOrAdd(Name, 1);
		}
	}

	void AddToHistogram(const TCHAR* VertexFactoryName, const TCHAR* ShaderPipelineName, const TCHAR* ShaderTypeName)
	{
		FString ShaderType(ShaderTypeName);
		if (int32* Existing = ShaderTypeHistogram.Find(ShaderType))
		{
			++(*Existing);
		}
		else
		{
			ShaderTypeHistogram.FindOrAdd(ShaderType, 1);
		}

#if 0	// the output of the full list is spammy and not usable. Needs to be replaced by a [Type x VF] matrix probably
		FString AbsoluteShaderName = (ShaderPipelineName != nullptr) ? FString::Printf(TEXT("%s.%s.%s"), VertexFactoryName, ShaderPipelineName, ShaderTypeName) : FString::Printf(TEXT("%s.%s"), VertexFactoryName, ShaderTypeName);
		if (int32* Existing = FullShaderTypeHistogram.Find(AbsoluteShaderName))
		{
			++(*Existing);
		}
		else
		{
			FullShaderTypeHistogram.FindOrAdd(AbsoluteShaderName, 1);
		}
#endif // 0

		if (VertexFactoryName)
		{
			FString VFTypeName(VertexFactoryName);
			if (int32* Existing = VertexFactoryTypeHistogram.Find(VFTypeName))
			{
				++(*Existing);
			}
			else
			{
				VertexFactoryTypeHistogram.FindOrAdd(VFTypeName, 1);
			}
		}
	}

	void PrintHistogram(int TotalShaders)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrintHistogram);

		if (ShaderTypeHistogram.Num() > 0)
		{
			{
				ShaderTypeHistogram.ValueSort(TGreater<int32>());
				const char ShaderTypeHeader[] = "\nSorted by count:\nShaderType, Count, Percent Total\n";
				DebugWriter->Serialize(const_cast<char*>(ShaderTypeHeader), sizeof(ShaderTypeHeader) - 1);
				for (TPair<FString, int32> ShaderUsage : ShaderTypeHistogram)
				{
					FString OuputLine = FString::Printf(TEXT("%s, %d, %.2f\n"), *ShaderUsage.Key, ShaderUsage.Value, (ShaderUsage.Value / (float)TotalShaders) * 100.0f);
					DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
				}
			}

			// sort one more time, alphabetically for easier comparison, and print again
			{
				ShaderTypeHistogram.KeySort(TLess<FString>());
				const char ShaderTypeHeader[] = "\nSorted by shader type:\nShaderType, Count, Percent Total\n";
				DebugWriter->Serialize(const_cast<char*>(ShaderTypeHeader), sizeof(ShaderTypeHeader) - 1);
				for (TPair<FString, int32> ShaderUsage : ShaderTypeHistogram)
				{
					FString OuputLine = FString::Printf(TEXT("%s, %d, %.2f\n"), *ShaderUsage.Key, ShaderUsage.Value, (ShaderUsage.Value / (float)TotalShaders) * 100.0f);
					DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
				}
			}
		}

		if (FullShaderTypeHistogram.Num() > 0)
		{
			FullShaderTypeHistogram.ValueSort(TGreater<int32>());
			const char FullShaderTypeHeader[] = "\nFullShaderType, Count, Percent Total\n";
			DebugWriter->Serialize(const_cast<char*>(FullShaderTypeHeader), sizeof(FullShaderTypeHeader) - 1);
			for (TPair<FString, int32> ShaderUsage : FullShaderTypeHistogram)
			{
				FString OuputLine = FString::Printf(TEXT("%s, %d, %.2f\n"), *ShaderUsage.Key, ShaderUsage.Value, (ShaderUsage.Value / (float)TotalShaders) * 100.0f);
				DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
			}
		}

		if (VertexFactoryTypeHistogram.Num() > 0)
		{
			{
				VertexFactoryTypeHistogram.ValueSort(TGreater<int32>());
				const char FullVFTypeHeader[] = "\nSorted by count:\nVFType, Count, Percent Total\n";
				DebugWriter->Serialize(const_cast<char*>(FullVFTypeHeader), sizeof(FullVFTypeHeader) - 1);
				for (TPair<FString, int32> VFTypeUsage : VertexFactoryTypeHistogram)
				{
					FString OuputLine = FString::Printf(TEXT("%s, %d, %.2f\n"), *VFTypeUsage.Key, VFTypeUsage.Value, (VFTypeUsage.Value / (float)TotalShaders) * 100.0f);
					DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
				}
			}

			// sort one more time, alphabetically for easier comparison, and print again
			{
				VertexFactoryTypeHistogram.KeySort(TLess<FString>());
				const char FullVFTypeHeader[] = "\nSorted by VF:\nVFType, Count, Percent Total\n";
				DebugWriter->Serialize(const_cast<char*>(FullVFTypeHeader), sizeof(FullVFTypeHeader) - 1);
				for (TPair<FString, int32> VFTypeUsage : VertexFactoryTypeHistogram)
				{
					FString OuputLine = FString::Printf(TEXT("%s, %d, %.2f\n"), *VFTypeUsage.Key, VFTypeUsage.Value, (VFTypeUsage.Value / (float)TotalShaders) * 100.0f);
					DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
				}
			}
		}
	}

	void PrintAlphabeticList()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrintAlphabeticList);

		if (ShaderTypeHistogram.Num() > 0)
		{
			ShaderTypeHistogram.KeySort(TLess<FString>());
			const char ShaderTypeAlphabeticHeader[] = "\nShaderType only\n";
			DebugWriter->Serialize(const_cast<char*>(ShaderTypeAlphabeticHeader), sizeof(ShaderTypeAlphabeticHeader) - 1);
			for (TPair<FString, int32> ShaderUsage : ShaderTypeHistogram)
			{
				// do not print numbers here as it complicates the diff
				FString OuputLine = FString::Printf(TEXT("%s\n"), *ShaderUsage.Key);
				DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
			}
		}

		if (FullShaderTypeHistogram.Num() > 0)
		{
			FullShaderTypeHistogram.KeySort(TLess<FString>());
			const char FullShaderTypeAlphabeticHeader[] = "\nFullShaderType only\n";
			DebugWriter->Serialize(const_cast<char*>(FullShaderTypeAlphabeticHeader), sizeof(FullShaderTypeAlphabeticHeader) - 1);
			for (TPair<FString, int32> ShaderUsage : FullShaderTypeHistogram)
			{
				// do not print numbers here as it complicates the diff
				FString OuputLine = FString::Printf(TEXT("%s\n"), *ShaderUsage.Key);
				DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
			}
		}

		if (VertexFactoryTypeHistogram.Num() > 0)
		{
			VertexFactoryTypeHistogram.KeySort(TLess<FString>());
			const char FullVFTypeAlphabeticHeader[] = "\nVertexFactoryType only\n";
			DebugWriter->Serialize(const_cast<char*>(FullVFTypeAlphabeticHeader), sizeof(FullVFTypeAlphabeticHeader) - 1);
			for (TPair<FString, int32> VFTypeUsage : VertexFactoryTypeHistogram)
			{
				// do not print numbers here as it complicates the diff
				FString OuputLine = FString::Printf(TEXT("%s\n"), *VFTypeUsage.Key);
				DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
			}
		}
	}

	void PrintUniqueMaterialInstances()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrintUniqueMaterialInstances);

		// Sort by number of mat instances.
		struct TArraySizeSort
		{
			bool operator()(const TArray<FString>& A, const TArray<FString>& B) const
			{
				return A.Num() > B.Num();
			}
		};
		UniqueMaterialInstances.ValueSort(TArraySizeSort());

		// Write out header.
		const char MatInstHeader[] = "\nUnique Material Instances\n";
		DebugWriter->Serialize(const_cast<char*>(MatInstHeader), sizeof(MatInstHeader) - 1);

		// Print each item.
		for (TPair<uint64, TArray<FString>> UniqueMaterialInstance : UniqueMaterialInstances)
		{
			FString DuplicateLine = FString::Printf(TEXT("Duplicates: %d\n"), UniqueMaterialInstance.Value.Num());
			DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*DuplicateLine).Get()), DuplicateLine.Len());

			for (const FString& InstanceName : UniqueMaterialInstance.Value)
			{
				FString OuputLine = FString::Printf(TEXT("\t%s\n"), *InstanceName);
				DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
			}
		}
	}

	void Log(const FString& OutString)
	{
		FString OuputLine = OutString + TEXT("\n");
		DebugWriter->Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(*OuputLine).Get()), OuputLine.Len());
	}

	void AddMaterialInstance(const FString& MaterialInstanceName, uint64 StaticParameterHash)
	{
		TArray<FString>& InstanceList = UniqueMaterialInstances.FindOrAdd(StaticParameterHash);
		InstanceList.Add(MaterialInstanceName);
	}

private:
	FArchive* DebugWriter = nullptr;

	/** Map of shader type names (no matter the vertex factory) to their counts. */
	TMap<FString, int32>	ShaderTypeHistogram;

	/** Map of full shader display names to their counts. */
	TMap<FString, int32>	FullShaderTypeHistogram;

	/** Map of vertex factory display names to their counts. */
	TMap<FString, int32>	VertexFactoryTypeHistogram;

	/** Map of global shader type display names to their counts. */
	TMap<FString, int32>	GlobalShaderTypeHistogram;

	/** Unique material instances. */
	TMap<uint64, TArray<FString>> UniqueMaterialInstances;

	/** Store a copy of the the filename. */
	FString FileName;

	/** Store the full path to the output file. */
	FString OutputFileName;
};

static const TCHAR* kCssBgStyleSolidRed = TEXT("background-color:red;");
static const TCHAR* kCssBgStyleSolidGray = TEXT("background-color:silver;");
static const TCHAR* kCssBgStyleDashedRed = TEXT("background: repeating-linear-gradient(45deg,red 0px,red 4px,transparent 4px,transparent 8px)");
static const TCHAR* kCssBgStyleDashedGray = TEXT("background: repeating-linear-gradient(45deg,silver 0px,silver 4px,transparent 4px,transparent 8px)");
static const TCHAR* kCssBgStyleDashedLiteGray = TEXT("background: repeating-linear-gradient(45deg,whitesmoke 0px,whitesmoke 4px,transparent 4px,transparent 8px)");

// Helper class to emit HTML page with pre-defined CSS styles to improve readability of the analysis output.
class FHtmlPageWriter
{
public:
	FHtmlPageWriter(FString&& BaseName) :
		BaseName(MoveTemp(BaseName))
	{
	}

	~FHtmlPageWriter()
	{
		CloseDocument();
	}

	void OpenNewDocument()
	{
		CloseDocument();
		Output = TUniquePtr<FShaderStatsGatheringContext>(new FShaderStatsGatheringContext(FString::Printf(TEXT("%s-Part_%d.html"), *BaseName, PartCounter)));
		WriteHtmlHeader();
		++PartCounter;
		LineCounter = 0;
	}

	void WriteLine(const FString& Line)
	{
		if (!Output)
		{
			OpenNewDocument();
		}
		Output->Log(Line);
		++LineCounter;
	}

	int32 NumLines() const
	{
		return LineCounter;
	}

private:
	void CloseDocument()
	{
		if (Output)
		{
			WriteHtmlFooter();
			Output.Reset();
		}
	}

	void WriteHtmlHeader()
	{
		// Write HTML header
		Output->Log(TEXT("<!DOCTYPE html>"));
		Output->Log(TEXT("<html>"));
		Output->Log(TEXT("<head>"));
		Output->Log(TEXT("\t<title>StaticSwitchOptimizer</title>"));
		Output->Log(TEXT("\t<style>"));
		Output->Log(TEXT("\t\ttable {border: 1px solid black; font-family: monospace;}"));
		Output->Log(TEXT("\t\tth {padding-right: 5px; padding-left: 5px; padding-top: 2px; padding-bottom: 2px;}"));
		Output->Log(TEXT("\t</style>"));
		Output->Log(TEXT("</head>"));

		// Write HTML legend table
		Output->Log(TEXT("<body>"));

		Output->Log(TEXT("<table>"));
		Output->Log(TEXT("\t<tr><th>"));
		Output->Log(TEXT("\t\t<h3>LEGEND</h3>"));
		Output->Log(TEXT("\t</th></tr>"));
		Output->Log(TEXT("\t<tr><td>"));
		Output->Log(TEXT("\t\t<table>"));
		Output->Log(FString::Printf(TEXT("\t\t\t<tr><th>Unique static switch</th><th style=\"%s\">Gray background</th></tr>"), kCssBgStyleSolidGray));
		Output->Log(FString::Printf(TEXT("\t\t\t<tr><th>Varying static switch ON</th><th style=\"%s\">Red background</th></tr>"), kCssBgStyleSolidRed));
		Output->Log(FString::Printf(TEXT("\t\t\t<tr><th>Trivial graph dependency (No texture input)</th><th style=\"%s\">Dashed background</th></tr>"), kCssBgStyleDashedGray));
		Output->Log(TEXT("\t\t</table>"));
		Output->Log(TEXT("\t</th></tr>"));
		Output->Log(TEXT("</table>"));
		Output->Log(TEXT("<br></br>"));
	}

	void WriteHtmlFooter()
	{
		Output->Log(TEXT("</body>"));
		Output->Log(TEXT("</html>"));
	}

private:
	FString BaseName;
	TUniquePtr<FShaderStatsGatheringContext> Output;
	int32 PartCounter = 0;
	int32 LineCounter = 0;
};


UDumpMaterialShaderTypesCommandlet::UDumpMaterialShaderTypesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

static int GetTotalShaders(const TArray<FDebugShaderTypeInfo>& OutShaderInfo)
{
	int TotalShadersForMaterial = 0;
	for (const FDebugShaderTypeInfo& ShaderInfo : OutShaderInfo)
	{
		TotalShadersForMaterial += ShaderInfo.ShaderTypes.Num();

		for (const FDebugShaderPipelineInfo& PipelineInfo : ShaderInfo.Pipelines)
		{
			TotalShadersForMaterial += PipelineInfo.ShaderTypes.Num();
		}
	}
	return TotalShadersForMaterial;
}

static void PrintDebugShaderInfo(FShaderStatsGatheringContext& Output, const TArray<FDebugShaderTypeInfo>& OutShaderInfo)
{
	for (const FDebugShaderTypeInfo& ShaderInfo : OutShaderInfo)
	{
		Output.Log(TEXT(""));

		// FMeshMaterialShader
		if (ShaderInfo.VFType)
		{
			int TotalShadersForVF = 0;
			TotalShadersForVF += ShaderInfo.ShaderTypes.Num();

			for (const FDebugShaderPipelineInfo& PipelineInfo : ShaderInfo.Pipelines)
			{
				TotalShadersForVF += PipelineInfo.ShaderTypes.Num();
			}

			Output.Log(FString::Printf(TEXT("\t%s - %d shaders"), ShaderInfo.VFType->GetName(), TotalShadersForVF));

			for (FShaderType* ShaderType : ShaderInfo.ShaderTypes)
			{
				Output.Log(FString::Printf(TEXT("\t\t%s"), ShaderType->GetName()));
				Output.AddToHistogram(ShaderInfo.VFType->GetName(), nullptr, ShaderType->GetName());
			}

			for (const FDebugShaderPipelineInfo& PipelineInfo : ShaderInfo.Pipelines)
			{
				Output.Log(FString::Printf(TEXT("\t\t%s"), PipelineInfo.Pipeline->GetName()));

				for (FShaderType* ShaderType : PipelineInfo.ShaderTypes)
				{
					Output.Log(FString::Printf(TEXT("\t\t\t%s"), ShaderType->GetName()));
					Output.AddToHistogram(ShaderInfo.VFType->GetName(), PipelineInfo.Pipeline->GetName(), ShaderType->GetName());
				}
			}
		}
		// FMaterialShader
		else
		{
			check(ShaderInfo.Pipelines.Num() == 0);

			TMap<FString, int32> ShaderTypeMap;
			for (FShaderType* ShaderType : ShaderInfo.ShaderTypes)
			{
				FString ShaderTypeName(ShaderType->GetName());
				if (int32* Existing = ShaderTypeMap.Find(ShaderTypeName))
				{
					++(*Existing);
				}
				else
				{
					ShaderTypeMap.FindOrAdd(ShaderTypeName, 1);
				}

				Output.AddToHistogram(nullptr, nullptr, ShaderType->GetName());
			}

			if (ShaderTypeMap.Num() > 0)
			{
				ShaderTypeMap.ValueSort(TGreater<int32>());
				for (TPair<FString, int32> ShaderTypeSum : ShaderTypeMap)
				{
					Output.Log(FString::Printf(TEXT("\t%s - %d shaders"), *ShaderTypeSum.Key, ShaderTypeSum.Value));
				}
			}
		}

		Output.Log(TEXT(""));
	}
}

static int ProcessMaterials(const ITargetPlatform* TargetPlatform, const EShaderPlatform ShaderPlatform, FShaderStatsGatheringContext& Output, const TArray<FAssetData>& MaterialList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessMaterials);

	int TotalShaders = 0;

	for (const FAssetData& AssetData : MaterialList)
	{
		if (UMaterial* Material = Cast<UMaterial>(AssetData.GetAsset()))
		{
			TArray<FDebugShaderTypeInfo> OutShaderInfo;
			Material->GetShaderTypes(ShaderPlatform, TargetPlatform, OutShaderInfo);

			const int TotalShadersForMaterial = GetTotalShaders(OutShaderInfo);
			TotalShaders += TotalShadersForMaterial;

			Output.Log(TEXT(""));
			Output.Log(FString::Printf(TEXT("Material: %s - %d shaders"), *AssetData.GetObjectPathString(), TotalShadersForMaterial));

			PrintDebugShaderInfo(Output, OutShaderInfo);
		}
	}

	Output.Log(TEXT(""));
	Output.Log(TEXT("MaterialSummary"));
	Output.Log(FString::Printf(TEXT("Total Materials: %d"), MaterialList.Num()));
	Output.Log(FString::Printf(TEXT("Total Material Shaders: %d"), TotalShaders));

	return TotalShaders;
}

static void ProcessSwitchOptimizer(const ITargetPlatform* TargetPlatform, const EShaderPlatform ShaderPlatform, const TArray<FAssetData>& MaterialList, const TArray<FAssetData>& MaterialInstanceList, const FString& TimeNow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessSwitchOptimizer);

	struct FOuterKeyFuncs
	{
		static inline bool Matches(const FMaterialShaderMapId& A, const FMaterialShaderMapId& B)
		{
			return A.Equals(B, false);
		}

		static inline uint32 GetKeyHash(const FMaterialShaderMapId& Key)
		{
			return (uint32)Key.GetMaterialHash(false);
		}
	};

	using StaticSwitchArrayType = TArray<FStaticSwitchParameter>;
	struct FInnerKeyFuncs
	{
		static inline bool Matches(const StaticSwitchArrayType& A, const StaticSwitchArrayType& B)
		{
			return A == B;
		}

		static inline uint32 GetKeyHash(const StaticSwitchArrayType& Keys)
		{
			FXxHash64Builder Hash;
			for(const FStaticSwitchParameter& Key : Keys)
			{
				Key.UpdateHash(Hash);
			}
			return (uint32)Hash.Finalize().Hash;
		}
	};

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);
	TArray<FDebugShaderTypeInfo> OutShaderInfo;

	using MaterialAndSizeSet = Experimental::TRobinHoodHashMap<UMaterialInterface*, int32>;
	using StaticSwitchGroupSet = Experimental::TRobinHoodHashMap<StaticSwitchArrayType, MaterialAndSizeSet, FInnerKeyFuncs>;
	using ShaderIdGroupSet = Experimental::TRobinHoodHashMap<FMaterialShaderMapId, StaticSwitchGroupSet, FOuterKeyFuncs>;
	ShaderIdGroupSet ShaderMapHashMap;

	using FMaterialExpressionArray = TArray<UMaterialExpression*>;
	using FShaderMapIdToMaterialExpressionMap = TMap<FMaterialShaderMapId, FMaterialExpressionArray>;
	FShaderMapIdToMaterialExpressionMap MaterialExpressionMap;

	auto AppendExpressionsToNameMap = [&MaterialExpressionMap](const FMaterialShaderMapId& ShaderMapId, UMaterial* Material) -> void
		{
			FMaterialExpressionArray& ExpressionArrayForShaderMap = MaterialExpressionMap.FindOrAdd(ShaderMapId);
			Material->GetAllExpressionsInMaterialAndFunctionsOfType<UMaterialExpression>(ExpressionArrayForShaderMap);
		};

	auto FindMaterialExpressionByName = [&MaterialExpressionMap](const FMaterialShaderMapId& ShaderMapId, const FName& Name) -> UMaterialExpression*
		{
			// Try to find expression for specified shadermap
			if (const FMaterialExpressionArray* ExpressionsForShaderMap = MaterialExpressionMap.Find(ShaderMapId))
			{
				for (UMaterialExpression* Expression : *ExpressionsForShaderMap)
				{
					if (Expression->GetParameterName() == Name)
					{
						return Expression;
					}
				}
			}
			return nullptr;
		};

	// Log the analysis progress ever 5 seconds as some projects can take a long time (minutes to hours) to analyze
	constexpr int64 ProgressUpdateIntervalInSeconds = 5;
	int64 LastProgressUpdateTimestamp = FDateTime::Now().ToUnixTimestamp();

	auto LogProgressInInterval = [&LastProgressUpdateTimestamp](const TCHAR* Info, int32 Progress, int32 Max) -> void
		{
			const int64 CurrentTimestamp = FDateTime::Now().ToUnixTimestamp();
			if (CurrentTimestamp - LastProgressUpdateTimestamp > ProgressUpdateIntervalInSeconds)
			{
				LastProgressUpdateTimestamp = CurrentTimestamp;
				UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Display, "%ls%d/%d (%0.2f%%)", Info, Progress, Max, 100.0*static_cast<double>(Progress)/static_cast<double>(Max));
			}
		};

	for (int32 MaterialListIndex = 0; MaterialListIndex < MaterialList.Num(); ++MaterialListIndex)
	{
		const FAssetData& AssetData = MaterialList[MaterialListIndex];
		if (UMaterial* Material = Cast<UMaterial>(AssetData.GetAsset()))
		{
			TArray<FMaterialResource*> ResourcesToCache;
			for (int32 QualityLevel = 0; QualityLevel < EMaterialQualityLevel::Num; QualityLevel++)
			{
				FMaterialResource* Resource = FindOrCreateMaterialResource(ResourcesToCache, Material, nullptr, GMaxRHIShaderPlatform, EMaterialQualityLevel::Type(QualityLevel));
				FMaterialShaderMapId ShaderMapId;
				Resource->BuildShaderMapId(ShaderMapId, TargetPlatform);
				StaticSwitchGroupSet& InnerHashMap = *ShaderMapHashMap.FindOrAdd(ShaderMapId, StaticSwitchGroupSet());
				MaterialAndSizeSet& InnerValue = *InnerHashMap.FindOrAdd(ShaderMapId.GetStaticSwitchParameters(), MaterialAndSizeSet());

				OutShaderInfo.Reset();
				Resource->GetShaderTypes(LayoutParams, OutShaderInfo, FConservativeShaderParameters(Resource));
				uint32 TotalNumShaders = GetTotalShaders(OutShaderInfo);
				if(TotalNumShaders)
				{
					InnerValue.FindOrAdd(Material, TotalNumShaders);
				}

				// Store mapping from shadermap ID to expression array
				AppendExpressionsToNameMap(ShaderMapId, Material);
			}
			FMaterial::DeferredDeleteArray(ResourcesToCache);

			LogProgressInInterval(TEXT("Building shader maps for materials in progress: "), MaterialListIndex + 1, MaterialList.Num());
		}
	}

	for (int32 MaterialInstanceIndex = 0; MaterialInstanceIndex < MaterialInstanceList.Num(); ++MaterialInstanceIndex)
	{
		const FAssetData& AssetData = MaterialInstanceList[MaterialInstanceIndex];
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(AssetData.GetAsset()))
		{
			TArray<FMaterialResource*> ResourcesToCache;
			for (int32 QualityLevel = 0; QualityLevel < EMaterialQualityLevel::Num; QualityLevel++)
			{
				UMaterial* BaseMaterial = MaterialInstance->GetMaterial();
				FMaterialResource* Resource = FindOrCreateMaterialResource(ResourcesToCache, BaseMaterial, MaterialInstance, GMaxRHIShaderPlatform, EMaterialQualityLevel::Type(QualityLevel));
				FMaterialShaderMapId ShaderMapId;
				Resource->BuildShaderMapId(ShaderMapId, TargetPlatform);
				StaticSwitchGroupSet& InnerHashMap = *ShaderMapHashMap.FindOrAdd(ShaderMapId, StaticSwitchGroupSet());
				MaterialAndSizeSet& InnerValue = *InnerHashMap.FindOrAdd(ShaderMapId.GetStaticSwitchParameters(), MaterialAndSizeSet());

				OutShaderInfo.Reset();
				Resource->GetShaderTypes(LayoutParams, OutShaderInfo, FConservativeShaderParameters(Resource));
				uint32 TotalNumShaders = GetTotalShaders(OutShaderInfo);
				if (TotalNumShaders)
				{
					InnerValue.FindOrAdd(MaterialInstance, TotalNumShaders);
				}

				// Store mapping from shadermap ID to expression array
				AppendExpressionsToNameMap(ShaderMapId, BaseMaterial);
			}
			FMaterial::DeferredDeleteArray(ResourcesToCache);

			LogProgressInInterval(TEXT("Building shader maps for material instances in progress: "), MaterialInstanceIndex + 1, MaterialInstanceList.Num());
		}
	}

	using ShaderIdAndSwitchGroup = TPair<const FMaterialShaderMapId, StaticSwitchGroupSet>;
	using StaticSwitchGroupType = TPair<const StaticSwitchArrayType, MaterialAndSizeSet>;

	struct FStaticSwitchMetaData
	{
		bool bIsVarying = false;
		bool bHasTrivialDependency = false; // Trivial dependency with only a limited number of material expressions and no texture dependency
		TArray<bool> PermutationVector; // Row-vector in the permutation matrix
	};

	using ShaderIdAndStaticSwitchGroupType = TPair<const FMaterialShaderMapId, StaticSwitchGroupType>;
	using ShaderIdGroupArray = TArray<ShaderIdAndStaticSwitchGroupType>;
	using SwitchPermutationVectorType = Experimental::TRobinHoodHashMap<const FName, FStaticSwitchMetaData>;

	struct FInnerFilteredType
	{
		ShaderIdGroupArray ShaderIdGroups;
		SwitchPermutationVectorType StaticSwitches;
	};
	
	TArray<FInnerFilteredType> FilteredHashMap;

	int32 ShaderMapIndex = 0;
	for(const ShaderIdAndSwitchGroup& OuterElement : ShaderMapHashMap)
	{
		LogProgressInInterval(TEXT("Analyzing static switches in progress: "), ++ShaderMapIndex, ShaderMapHashMap.Num());

		if(OuterElement.Value.Num() > 1)
		{
			ShaderIdGroupArray InnerMap;
			SwitchPermutationVectorType VaryingSwitches;
			StaticSwitchArrayType First;
			for(const StaticSwitchGroupType& Value : OuterElement.Value)
			{
				if(Value.Key.Num())
				{
					First = Value.Key;
					break;
				}
			}

			for (const StaticSwitchGroupType& Inner : OuterElement.Value)
			{
				for(int32 i = 0; i < Inner.Key.Num(); i++)
				{
					const FStaticSwitchParameter& Key = Inner.Key[i];
					const bool bIsVarying = (Key.Value != First[i].Value);
					FStaticSwitchMetaData* MetaData = VaryingSwitches.FindOrAdd(Key.ParameterInfo.Name, FStaticSwitchMetaData{});
					if (bIsVarying)
					{
						MetaData->bIsVarying = true;
					}
					if (UMaterialExpression* ExpressionForStaticSwitch = FindMaterialExpressionByName(OuterElement.Key, Key.ParameterInfo.Name))
					{
						constexpr int32 MaxDependencyWalkDepth = 16;
						FMaterialDependencySearchMetadata DependencyMetaData;
						const int32 WalkDepth = WalkMaterialDependencyGraph(ExpressionForStaticSwitch, MaxDependencyWalkDepth, MDSF_TextureDependencyOnly, DependencyMetaData);
						if (WalkDepth != INDEX_NONE && !DependencyMetaData.bHasTextureInput)
						{
							MetaData->bHasTrivialDependency = true;
						}
					}
					else
					{
						UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Error, "Failed to find static switch parameter \"%ls\"", *Key.ParameterInfo.Name.ToString());
					}
				}
				InnerMap.Emplace(OuterElement.Key, Inner);
			}
			InnerMap.Sort([](const ShaderIdAndStaticSwitchGroupType& A, const ShaderIdAndStaticSwitchGroupType& B)
			{
				if(A.Value.Key.Num() != B.Value.Key.Num())
				{
					return A.Value.Key.Num() < B.Value.Key.Num();
				}

				for(int32 i = 0; i < A.Value.Key.Num(); i++)
				{
					if(A.Value.Key[i].Value != B.Value.Key[i].Value)
					{
						return A.Value.Key[i].Value < B.Value.Key[i].Value;
					}
				}
				return false;
			});

			if(VaryingSwitches.Num())
			{
				FilteredHashMap.Add(FInnerFilteredType{ MoveTemp(InnerMap), MoveTemp(VaryingSwitches) });
			}
		}
	}

	FilteredHashMap.Sort([](const FInnerFilteredType& A, const FInnerFilteredType& B)
	{
		int32 NumA = 0;
		for (const ShaderIdAndStaticSwitchGroupType& InnerA : A.ShaderIdGroups)
		{
			NumA += (*InnerA.Value.Value.begin()).Value;
		}

		int32 NumB = 0;
		for (const ShaderIdAndStaticSwitchGroupType& InnerB : B.ShaderIdGroups)
		{
			NumB += (*InnerB.Value.Value.begin()).Value;
		}
		return NumA > NumB;
	});

	const FString BaseHtmlDocumentName = FString::Printf(TEXT("%s-StaticSwitches-%s-%s-%s"), FApp::GetProjectName(), *TargetPlatform->PlatformName(), *LexToString(ShaderPlatform), *TimeNow);
	FHtmlPageWriter PageWriter(FPaths::Combine(*TimeNow, *TargetPlatform->PlatformName(), *LexToString(ShaderPlatform), BaseHtmlDocumentName));

	for (FInnerFilteredType& InnerFilteredType : FilteredHashMap)
	{
		const ShaderIdGroupArray& InnerMap = InnerFilteredType.ShaderIdGroups;
		SwitchPermutationVectorType& StaticSwitches = InnerFilteredType.StaticSwitches;
		const UMaterialInterface* Parent = (*(*InnerMap.begin()).Value.Value.begin()).Key->GetMaterial();
		const FMaterialShaderMapId& ShaderId = (*InnerMap.begin()).Key;

		int32 NumShaders = 0;
		int32 NumStaticSwitchPermutations = 0;
		int32 NumStaticSwitchesTotal = 0;

		for (const ShaderIdAndStaticSwitchGroupType& Inner : InnerMap)
		{
			NumShaders += (*Inner.Value.Value.begin()).Value;
		}

		for (const ShaderIdAndStaticSwitchGroupType& Inner : InnerMap)
		{
			const StaticSwitchGroupType& Groups = Inner.Value;

			if (Groups.Key.Num())
			{
				NumStaticSwitchesTotal = FMath::Max(NumStaticSwitchesTotal, Groups.Key.Num());
				for (const FStaticSwitchParameter& Param : Groups.Key)
				{
					if (FStaticSwitchMetaData* MetaData = StaticSwitches.Find(Param.ParameterInfo.Name))
					{
						MetaData->PermutationVector.Add(Param.Value);
						NumStaticSwitchPermutations = FMath::Max(NumStaticSwitchPermutations, MetaData->PermutationVector.Num());
					}
				}
			}
		}

		// Open a new HTML document after the previous one reached the maximum size. Otherwise, it's hard to browse such large HTML documents.
		constexpr int32 MaxLinesPerHtmlPage = 10000;
		if (PageWriter.NumLines() > MaxLinesPerHtmlPage)
		{
			PageWriter.OpenNewDocument();
		}

		PageWriter.WriteLine(TEXT("<table>"));
		PageWriter.WriteLine(FString::Printf(TEXT("<tr><th><h3>Candidate %s (Shaders: %d)</h3></th></tr>"), *Parent->GetOuter()->GetFName().ToString(), NumShaders));
		PageWriter.WriteLine(TEXT("<tr><td><table>"));

		// Row for captions
		PageWriter.WriteLine(TEXT("\t<tr>"));
		PageWriter.WriteLine(FString::Printf(TEXT("\t\t<th style=\"background-color:gray;\">%d Static Switch(es)</th>"), NumStaticSwitchesTotal));
		PageWriter.WriteLine(FString::Printf(TEXT("\t\t<th style=\"background-color:gray;\" colspan=\"%d\">%d Permutation(s)</th>"), NumStaticSwitchPermutations, NumStaticSwitchPermutations));
		PageWriter.WriteLine(TEXT("\t</tr>"));

		// Row for each static switch parameter that is included in at least one permutation
		for (const TPair<const FName, FStaticSwitchMetaData>& Param : StaticSwitches)
		{
			PageWriter.WriteLine(TEXT("\t<tr>"));
			PageWriter.WriteLine(FString::Printf(TEXT("\t\t<th>%s</th>"), *Param.Key.ToString()));

			if (Param.Value.bIsVarying)
			{
				for (bool bSwitchEnabled : Param.Value.PermutationVector)
				{
					if (Param.Value.bHasTrivialDependency)
					{
						PageWriter.WriteLine(
							FString::Printf(
								TEXT("\t\t<th style=\"%s\">%s</th>"),
								bSwitchEnabled ? kCssBgStyleDashedRed : kCssBgStyleDashedLiteGray,
								bSwitchEnabled ? TEXT("1") : TEXT("0"))
						);
					}
					else
					{
						PageWriter.WriteLine(
							bSwitchEnabled
								? FString::Printf(TEXT("\t\t<th style=\"%s\">1</th>"), kCssBgStyleSolidRed)
								: TEXT("\t\t<th>0</th>")
						);
					}
				}
			}
			else
			{
				for (bool bSwitchEnabled : Param.Value.PermutationVector)
				{
					PageWriter.WriteLine(
						FString::Printf(
							TEXT("\t\t<th style=\"%s\">%s</th>"),
							Param.Value.bHasTrivialDependency ? kCssBgStyleDashedGray : kCssBgStyleSolidGray,
							bSwitchEnabled ? TEXT("1") : TEXT("0"))
					);
				}
			}
			PageWriter.WriteLine(TEXT("\t</tr>"));
		}

		PageWriter.WriteLine(TEXT("</table></td></tr>"));
		PageWriter.WriteLine(TEXT("</table>"));
		PageWriter.WriteLine(TEXT("<br></br>"));
	}
}

static int ProcessMaterialInstances(const ITargetPlatform* TargetPlatform, const EShaderPlatform ShaderPlatform, FShaderStatsGatheringContext& Output, const TArray<FAssetData>& MaterialInstanceList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessMaterialInstances);

	int TotalShaders = 0;

	int StaticPermutations = 0;
	for (const FAssetData& AssetData : MaterialInstanceList)
	{
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(AssetData.GetAsset()))
		{
			TArray<FDebugShaderTypeInfo> OutShaderInfo;
			MaterialInstance->GetShaderTypes(ShaderPlatform, TargetPlatform, OutShaderInfo);

			const int TotalShadersForMaterial = GetTotalShaders(OutShaderInfo);
			TotalShaders += TotalShadersForMaterial;

			// Find the root parent that is a material.
			UMaterialInterface* Top = MaterialInstance->Parent;
			while (Top)
			{
				if (UMaterial* Material = Cast<UMaterial>(Top))
				{
					break;
				}
				else if (UMaterialInstance* MatInst = Cast<UMaterialInstance>(Top))
				{
					Top = MatInst->Parent;
				}
				else
				{
					Top = nullptr;
				}
			}

			Output.Log(TEXT(""));
			Output.Log(FString::Printf(TEXT("Material Instance: %s - %d shaders"), *AssetData.AssetName.ToString(), TotalShadersForMaterial));
			Output.Log(FString::Printf(TEXT("Parent: %s"), Top ? *Top->GetFullName() : TEXT("NO PARENT")));
			Output.Log(FString::Printf(TEXT("Static Parameters: %d"), MaterialInstance->bHasStaticPermutationResource ? MaterialInstance->GetStaticParameters().StaticSwitchParameters.Num() : 0));

			uint64 OutHash;

			if (MaterialInstance->bHasStaticPermutationResource)
			{
				FXxHash64Builder Hasher;
				const FStaticParameterSet& ParameterSet = MaterialInstance->GetStaticParameters();
				for (int32 StaticSwitchIndex = 0; StaticSwitchIndex < ParameterSet.StaticSwitchParameters.Num(); ++StaticSwitchIndex)
				{
					const FStaticSwitchParameter& StaticSwitchParameter = ParameterSet.StaticSwitchParameters[StaticSwitchIndex];

					StaticSwitchParameter.UpdateHash(Hasher);

					Output.Log(FString::Printf(TEXT("\t%s : %s"), *StaticSwitchParameter.ParameterInfo.ToString(), StaticSwitchParameter.Value ? TEXT("True") : TEXT("False")));
				}

				OutHash = Hasher.Finalize().Hash;

				Output.AddMaterialInstance(AssetData.AssetName.ToString(), OutHash);
				Output.Log(FString::Printf(TEXT("Static Parameter Hash: %" UINT64_FMT), OutHash));
			}

			Output.Log(FString::Printf(TEXT("Base Property Overrides: %s"), MaterialInstance->HasOverridenBaseProperties() ? TEXT("True") : TEXT("False")));

			if (MaterialInstance->HasOverridenBaseProperties())
			{
				Output.Log(FString::Printf(TEXT("\t%s"), *MaterialInstance->GetBasePropertyOverrideString()));
			}

			PrintDebugShaderInfo(Output, OutShaderInfo);

			if (MaterialInstance->bHasStaticPermutationResource)
			{
				StaticPermutations++;
			}
		}
	}

	Output.Log(TEXT(""));
	Output.Log(TEXT("Material Instances Summary"));
	Output.Log(FString::Printf(TEXT("Total Material Instances: %d"), MaterialInstanceList.Num()));
	Output.Log(FString::Printf(TEXT("Material Instances w/ Static Permutations: %d"), StaticPermutations));
	Output.Log(FString::Printf(TEXT("Total Material Instances Shaders: %d"), TotalShaders));

	return TotalShaders;
}

static int ProcessGlobalShaders(const ITargetPlatform* TargetPlatform, const EShaderPlatform ShaderPlatform, FShaderStatsGatheringContext& Output)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessGlobalShaders);

	Output.Log(TEXT(""));
	Output.Log(TEXT("Global Shaders"));

	TArray<const FGlobalShaderType*> GlobalShaderTypes;
	for (TLinkedList<FShaderType*>::TIterator ShaderTypeIt(FShaderType::GetTypeList()); ShaderTypeIt; ShaderTypeIt.Next())
	{
		FGlobalShaderType* GlobalShaderType = ShaderTypeIt->GetGlobalShaderType();
		if (!GlobalShaderType)
		{
			continue;
		}

		GlobalShaderTypes.Add(GlobalShaderType);
	}
	Algo::SortBy(GlobalShaderTypes, [](const FGlobalShaderType* GS) { return GS->GetName(); }, FNameLexicalLess());

	int TotalShaders = 0;

	FPlatformTypeLayoutParameters LayoutParams;
	LayoutParams.InitializeForPlatform(TargetPlatform);
	EShaderPermutationFlags PermutationFlags = GetShaderPermutationFlags(LayoutParams);

	for (const FGlobalShaderType* GS : GlobalShaderTypes)
	{
		int PermutationCount = 0;
		for (int32 id = 0; id < GS->GetPermutationCount(); id++)
		{
			if (GS->ShouldCompilePermutation(ShaderPlatform, id, PermutationFlags))
			{
				PermutationCount++;
				TotalShaders++;
				Output.AddToGlobalShaderTypeHistogram(GS->GetName());
			}
		}

		if (PermutationCount)
		{
			Output.Log(FString::Printf(TEXT("%s - %d permutations"), GS->GetName(), PermutationCount));
		}
	}

	Output.Log(TEXT(""));
	Output.Log(TEXT("Global Shaders Summary"));
	Output.Log(FString::Printf(TEXT("Total Global Shaders: %d"), TotalShaders));

	return TotalShaders;
}

static void ProcessForTargetAndShaderPlatform(const ITargetPlatform* TargetPlatform, const EShaderPlatform ShaderPlatform, const FString& Params, const TArray<FAssetData>& MaterialList, const TArray<FAssetData>& MaterialInstanceList, TSharedPtr<IAnalyticsProviderET> Provider)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessForTargetAndShaderPlatform);

	const FString TimeNow = FDateTime::Now().ToString();
	FString FileName = FString::Printf(TEXT("%s-ShaderTypes-%s-%s-%s.txt"), FApp::GetProjectName(), *TargetPlatform->PlatformName(), *LexToString(ShaderPlatform), *TimeNow);

	FShaderStatsGatheringContext Output(FileName);

	int TotalShaders = 0;
	int TotalAssets = 0;

	// Cache for all the shader formats that the cooking target requires
	TotalShaders += ProcessMaterials(TargetPlatform, ShaderPlatform, Output, MaterialList);
	TotalAssets += MaterialList.Num();

	TotalShaders += ProcessMaterialInstances(TargetPlatform, ShaderPlatform, Output, MaterialInstanceList);
	TotalAssets += MaterialInstanceList.Num();

	const int TotalGlobalShaders = ProcessGlobalShaders(TargetPlatform, ShaderPlatform, Output);
	TotalShaders += TotalGlobalShaders;

	int TotalDefaultMaterialShaders = 0;
	{
		for (int32 Domain = 0; Domain < MD_MAX; ++Domain)
		{
			UMaterial* Material = UMaterial::GetDefaultMaterial((EMaterialDomain)Domain);
			if (Material)
			{
				TArray<FDebugShaderTypeInfo> OutShaderInfo;
				Material->GetShaderTypes(ShaderPlatform, TargetPlatform, OutShaderInfo);
				TotalDefaultMaterialShaders += GetTotalShaders(OutShaderInfo);
			}
		}
	}

	Output.Log(TEXT(""));
	Output.Log(TEXT("Summary"));
	Output.Log(FString::Printf(TEXT("Total Assets: %d"), TotalAssets));
	Output.Log(FString::Printf(TEXT("Total Shaders: %d"), TotalShaders));
	Output.Log(FString::Printf(TEXT("Total Default Material Shaders: %d"), TotalDefaultMaterialShaders));
	Output.Log(FString::Printf(TEXT("Histogram:")));
	Output.PrintHistogram(TotalShaders);
	Output.Log(FString::Printf(TEXT("\nAlphabetic list of types:")));
	Output.PrintAlphabeticList();

	if (Provider.IsValid())
	{
		Provider->RecordEvent(TEXT("DumpMaterialShaderTypes"), MakeAnalyticsEventAttributeArray(
			TEXT("ProjectName"), FApp::GetProjectName(),
			TEXT("BuildVersion"), FApp::GetBuildVersion(),
			TEXT("Platform"), TargetPlatform->PlatformName(),
			TEXT("ShaderPlatform"), LexToString(ShaderPlatform),
			TEXT("TotalShaders"), TotalShaders,
			TEXT("TotalMaterials"), MaterialList.Num(),
			TEXT("TotalMaterialInstances"), MaterialInstanceList.Num(),
			TEXT("TotalGlobalShaders"), TotalGlobalShaders,
			TEXT("TotalDefaultMaterialShaders"), TotalDefaultMaterialShaders
			));
	}
}

int32 UDumpMaterialShaderTypesCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, "DumpMaterialShaderTypes");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, "This commandlet will dump to a human readable plain text file of all the shaders that would be compiled for all materials in a project.");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, "Options:");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, " Required: -targetplatform=<platform(s)>     (Which target platform do you want results, e.g. WindowsClient, WindowsEditor. Multiple shader platforms are allowed).");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, " Optional: -collection=<name>                (You can also specify a collection of assets to narrow down the results e.g. if you maintain a collection that represents the actually used in-game assets).");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, " Optional: -analytics                        (Whether or not to send analytics data for tracking purposes).");
		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Log, " Optional: -staticswitches					 (Gain more detailed information of StaticSwitch use and cost).");
		return 0;
	}

	const bool bStaticSwitches = FParse::Param(FCommandLine::Get(), TEXT("staticswitches"));
	const bool bSendAnalytics = FParse::Param(FCommandLine::Get(), TEXT("analytics"));

	const double AssetRegistryStart = FPlatformTime::Seconds();

	TArray<FAssetData> MaterialList;
	TArray<FAssetData> MaterialInstanceList;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UDumpMaterialShaderTypesCommandlet.AssetRegistryScan);

		UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Display, "Searching the asset registry for all assets...");
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		AssetRegistry.SearchAllAssets(true);

		// Parse collection
		FString CollectionName;
		if (FParse::Value(*Params, TEXT("collection="), CollectionName, true))
		{
			if (!CollectionName.IsEmpty())
			{
				// Get the list of materials from a collection
				FARFilter Filter;
				Filter.PackagePaths.Add(FName(TEXT("/Game")));
				Filter.bRecursivePaths = true;
				Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());

				FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
CollectionManagerModule.Get().GetObjectsInCollection(
	FName(*CollectionName), ECollectionShareType::CST_All, Filter.SoftObjectPaths,
	ECollectionRecursionFlags::SelfAndChildren);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				AssetRegistry.GetAssets(Filter, MaterialList);

				Filter.ClassPaths.Empty();
				Filter.ClassPaths.Add(UMaterialInstance::StaticClass()->GetClassPathName());
				Filter.ClassPaths.Add(UMaterialInstanceConstant::StaticClass()->GetClassPathName());

				AssetRegistry.GetAssets(Filter, MaterialInstanceList);
			}
		}
		else
		{
			if (!AssetRegistry.IsLoadingAssets())
			{
				AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), MaterialList, true);
				AssetRegistry.GetAssetsByClass(UMaterialInstance::StaticClass()->GetClassPathName(), MaterialInstanceList, true);
			}
		}
	}

	const double AssetRegistryEnd = FPlatformTime::Seconds();
	UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Display, "Asset scan took: %.3f", AssetRegistryEnd - AssetRegistryStart);

	// Sort the material lists by name so the order is stable.
	Algo::SortBy(MaterialList, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath(); }, [](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.LexicalLess(B); });
	Algo::SortBy(MaterialInstanceList, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath(); }, [](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.LexicalLess(B); });

	// For all active platforms
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();

	TSharedPtr<IAnalyticsProviderET> Provider;
	if (bSendAnalytics)
	{
		FAnalyticsET::Config Config;
		Config.APIKeyET = FString(TEXT("StudioAnalytics.Dev"));
		Config.APIServerET = FString(TEXT("https://datarouter.ol.epicgames.com/"));
		Config.AppVersionET = FEngineVersion::Current().ToString();

		// There are other things to configure, but the default are usually fine.
		Provider = FAnalyticsET::Get().CreateAnalyticsProvider(Config);
		if (Provider.IsValid())
		{
			const FString UserID = FPlatformProcess::UserName(false);
			Provider->SetUserID(UserID);
			Provider->StartSession(MakeAnalyticsEventAttributeArray(
				TEXT("ProjectName"), FApp::GetProjectName(),
				TEXT("Version"), FApp::GetBuildVersion()
			));
		}
	}

	const FString TimeNow = FDateTime::Now().ToString();

	for (int32 Index = 0; Index < Platforms.Num(); Index++)
	{
		TArray<FName> DesiredShaderFormats;
		Platforms[Index]->GetAllTargetedShaderFormats(DesiredShaderFormats);

		for (int32 FormatIndex = 0; FormatIndex < DesiredShaderFormats.Num(); FormatIndex++)
		{
			const EShaderPlatform ShaderPlatform = ShaderFormatToLegacyShaderPlatform(DesiredShaderFormats[FormatIndex]);

			UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Display, "Dumping material shader types for '%ls' - '%ls'...", *Platforms[Index]->PlatformName(), *LexToString(ShaderPlatform));
			if(bStaticSwitches)
			{
				ProcessSwitchOptimizer(Platforms[Index], ShaderPlatform, MaterialList, MaterialInstanceList, TimeNow);
			}
			else
			{
				ProcessForTargetAndShaderPlatform(Platforms[Index], ShaderPlatform, Params, MaterialList, MaterialInstanceList, Provider);
			}
		}
	}

	UE_LOGF(LogDumpMaterialShaderTypesCommandlet, Display, "Dumping stats took: %.3f", FPlatformTime::Seconds() - AssetRegistryEnd);

	return 0;
}
