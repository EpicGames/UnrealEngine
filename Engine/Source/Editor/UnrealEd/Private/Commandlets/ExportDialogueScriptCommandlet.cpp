// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ExportDialogueScriptCommandlet.h"
#include "UObject/UnrealType.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "Sound/SoundWave.h"
#include "Sound/DialogueWave.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "UObject/PropertyPortFlags.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Sound/DialogueVoice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExportDialogueScriptCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogExportDialogueScriptCommandlet, Log, All);

namespace
{
	struct FCollapsedDialogueContextKey
	{
		FCollapsedDialogueContextKey(const UDialogueWave* InDialogueWave, const FDialogueContextMapping* InContext, FString InLocalizedSpokenText)
			: DialogueWave(InDialogueWave)
			, Context(InContext)
			, LocalizedSpokenText(MoveTemp(InLocalizedSpokenText))
		{
		}

		bool operator==(const FCollapsedDialogueContextKey& InOther) const
		{
			// We only care about the text that is spoken, and the voice that is speaking it
			return LocalizedSpokenText.Equals(InOther.LocalizedSpokenText, ESearchCase::CaseSensitive)
				&& Context->Context.Speaker == InOther.Context->Context.Speaker;
		}

		bool operator!=(const FCollapsedDialogueContextKey& InOther) const
		{
			return !(*this == InOther);
		}

		friend inline uint32 GetTypeHash(const FCollapsedDialogueContextKey& InKey)
		{
			// We only care about the text that is spoken, and the voice that is speaking it
			uint32 KeyHash = 0;
			KeyHash = HashCombine(KeyHash, FCrc::StrCrc32(*InKey.LocalizedSpokenText)); // Need case-sensitive hash
			KeyHash = HashCombine(KeyHash, GetTypeHash(InKey.Context->Context.Speaker));
			return KeyHash;
		}

		const UDialogueWave* DialogueWave;
		const FDialogueContextMapping* Context;
		FString LocalizedSpokenText;
	};
}

UExportDialogueScriptCommandlet::UExportDialogueScriptCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UExportDialogueScriptCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config path
	FString ConfigPath;
	{
		const FString* ConfigPathParamVal = ParamVals.Find(FString(TEXT("Config")));
		if (!ConfigPathParamVal)
		{
			UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No config specified.");
			return -1;
		}
		ConfigPath = *ConfigPathParamVal;
	}

	// Set config section
	FString SectionName;
	{
		const FString* SectionNameParamVal = ParamVals.Find(FString(TEXT("Section")));
		if (!SectionNameParamVal)
		{
			UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No config section specified.");
			return -1;
		}
		SectionName = *SectionNameParamVal;
	}

	// Source path to the root folder that manifest/archive files live in
	FString SourcePath;
	if (!GetPathFromConfig(*SectionName, TEXT("SourcePath"), SourcePath, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No source path specified.");
		return -1;
	}

	// Destination path to the root folder that dialogue script CSV files live in
	FString DestinationPath;
	if (!GetPathFromConfig(*SectionName, TEXT("DestinationPath"), DestinationPath, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No destination path specified.");
		return -1;
	}

	// Get culture directory setting, default to true if not specified (used to allow picking of export directory with windows file dialog from Translation Editor)
	bool bUseCultureDirectory = true;
	if (!GetBoolFromConfig(*SectionName, TEXT("bUseCultureDirectory"), bUseCultureDirectory, ConfigPath))
	{
		bUseCultureDirectory = true;
	}

	// Get the native culture
	FString NativeCulture;
	if (!GetStringFromConfig(*SectionName, TEXT("NativeCulture"), NativeCulture, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No native culture specified.");
		return -1;
	}

	// Get cultures to generate
	TArray<FString> CulturesToGenerate;
	if (GetStringArrayFromConfig(*SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, ConfigPath) == 0)
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No cultures specified for import.");
		return -1;
	}

	// Get the manifest name
	FString ManifestName;
	if (!GetStringFromConfig(*SectionName, TEXT("ManifestName"), ManifestName, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No manifest name specified.");
		return -1;
	}

	// Get the archive name
	FString ArchiveName;
	if (!GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No archive name specified.");
		return -1;
	}

	// Get the dialogue script name
	FString DialogueScriptName;
	if (!GetStringFromConfig(*SectionName, TEXT("DialogueScriptName"), DialogueScriptName, ConfigPath))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "No dialogue script name specified.");
		return -1;
	}

	// We may only have a single culture if using this setting
	if (!bUseCultureDirectory && CulturesToGenerate.Num() > 1)
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "bUseCultureDirectory may only be used with a single culture.");
		return -1;
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, NativeCulture, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGF(LogExportDialogueScriptCommandlet, Error, "%ls", *LoadError.ToString());
			return -1;
		}
	}

	const FString RootAssetPath = FApp::HasProjectName() ? TEXT("/Game") : TEXT("/Engine");

	// Prepare the asset registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	// We want all the non-localized project specific dialogue waves
	TArray<FAssetData> AssetDataArrayForDialogueWaves;
	if (!FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, *RootAssetPath, UDialogueWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/false, AssetDataArrayForDialogueWaves))
	{
		UE_LOGF(LogExportDialogueScriptCommandlet, Error, "Unable to get dialogue wave asset data from asset registry.");
		return -1;
	}

	for (const FString& CultureName : CulturesToGenerate)
	{
		const bool bIsNativeCulture = CultureName == NativeCulture;
		const FString CultureSourcePath = SourcePath / CultureName;
		const FString CultureDestinationPath = DestinationPath / (bUseCultureDirectory ? CultureName : TEXT(""));

		TArray<TSharedPtr<FDialogueScriptEntry>> ExportedDialogueLines;
		for (const FAssetData& AssetData : AssetDataArrayForDialogueWaves)
		{
			// Verify that the found asset is a dialogue wave
			if (AssetData.GetClass() != UDialogueWave::StaticClass())
			{
				UE_LOGF(LogExportDialogueScriptCommandlet, Error, "Asset registry found asset '%ls', but the asset with this name is not actually a dialogue wave.", *AssetData.AssetName.ToString());
				continue;
			}

			// Get the dialogue wave
			UDialogueWave* const DialogueWave = Cast<UDialogueWave>(AssetData.GetAsset());

			// Verify that the dialogue wave was loaded
			if (!DialogueWave)
			{
				UE_LOGF(LogExportDialogueScriptCommandlet, Error, "Asset registry found asset '%ls', but the dialogue wave could not be accessed.", *AssetData.AssetName.ToString());
				continue;
			}
			
			// This maps collapsed context keys to additional contexts that were collapsed into the primary context (the context within the key) - all contexts belong to the dialogue wave in the key
			// If multiple contexts have the same speaking voice and use the same dialogue (because it translates to the same text), then only one of those contexts needs to be exported
			// The resultant audio file will create a shared asset automatically when the dialogue is imported
			TMap<FCollapsedDialogueContextKey, TArray<const FDialogueContextMapping*>> CollapsedDialogueContexts;

			// Iterate over each context to build the list of unique entries
			for (FDialogueContextMapping& ContextMapping : DialogueWave->ContextMappings)
			{
				const FString ContextLocalizationKey = DialogueWave->GetContextLocalizationKey(ContextMapping);

				// Check that this entry exists in the manifest file, as we want to skip over dialogue that we aren't gathering
				TSharedPtr<FManifestEntry> ContextManifestEntry = LocTextHelper.FindSourceText(FDialogueConstants::DialogueNamespace, ContextLocalizationKey, &DialogueWave->SpokenText);
				if (!ContextManifestEntry.IsValid())
				{
					UE_LOGF(LogExportDialogueScriptCommandlet, Log, "No internationalization manifest entry was found for context '%ls' in culture '%ls'. This context will be skipped.", *ContextLocalizationKey, *CultureName);
					continue;
				}

				// Find the correct entry for our context
				const FManifestContext* ContextManifestEntryContext = ContextManifestEntry->FindContextByKey(ContextLocalizationKey);
				check(ContextManifestEntryContext); // This should never fail as we pass in the key to FindSourceText

				// Get the localized text to export
				FLocItem ExportedSource;
				FLocItem ExportedTranslation;
				LocTextHelper.GetExportText(CultureName, FDialogueConstants::DialogueNamespace, ContextManifestEntryContext->Key, ContextManifestEntryContext->KeyMetadataObj, ELocTextExportSourceMethod::NativeText, ContextManifestEntry->Source, ExportedSource, ExportedTranslation);

				if (ExportedTranslation.Text.IsEmpty())
				{
					UE_LOGF(LogExportDialogueScriptCommandlet, Log, "Empty translation found for context '%ls' in culture '%ls'. This context will be skipped.", *ContextLocalizationKey, *CultureName);
					continue;
				}

				const auto CollapsedDialogueContextKey = FCollapsedDialogueContextKey(DialogueWave, &ContextMapping, ExportedTranslation.Text);
				TArray<const FDialogueContextMapping*>* MergedContextsPtr = CollapsedDialogueContexts.Find(CollapsedDialogueContextKey);
				if (MergedContextsPtr)
				{
					MergedContextsPtr->Add(&ContextMapping);
				}
				else
				{
					CollapsedDialogueContexts.Add(CollapsedDialogueContextKey);
				}
			}

			// Get the localized voice actor direction
			FLocItem ExportedVoiceActorDirectionSource;
			FLocItem ExportedVoiceActorDirectionTranslation;
			LocTextHelper.GetExportText(CultureName, FDialogueConstants::DialogueNamespace, DialogueWave->LocalizationGUID.ToString() + FDialogueConstants::ActingDirectionKeySuffix, nullptr, ELocTextExportSourceMethod::NativeText, FLocItem(DialogueWave->VoiceActorDirection), ExportedVoiceActorDirectionSource, ExportedVoiceActorDirectionTranslation);

			// Get the localized version of the dialogue wave for the current culture
			UDialogueWave* LocalizedDialogueWave = nullptr;
			{
				const FString LocalizedDialogueWavePackagePath = FPackageName::GetLocalizedPackagePath(AssetData.PackageName.ToString(), CultureName);
				const FAssetData LocalizedDialogueWaveAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(FTopLevelAssetPath(*LocalizedDialogueWavePackagePath, AssetData.AssetName)));
				LocalizedDialogueWave = Cast<UDialogueWave>(LocalizedDialogueWaveAssetData.GetAsset());
				if (LocalizedDialogueWave == DialogueWave)
				{
					LocalizedDialogueWave = nullptr;
				}
			}

			// Iterate over the unique contexts and generate exported data for them
			for (const auto& CollapsedDialogueContextPair : CollapsedDialogueContexts)
			{
				TSharedRef<FDialogueScriptEntry> ExportedDialogueLine = MakeShareable(new FDialogueScriptEntry());
				PopulateDialogueScriptEntry(DialogueWave, LocalizedDialogueWave, *CollapsedDialogueContextPair.Key.Context, CollapsedDialogueContextPair.Value, CollapsedDialogueContextPair.Key.LocalizedSpokenText, ExportedVoiceActorDirectionTranslation.Text, *ExportedDialogueLine);
				ExportedDialogueLines.Add(ExportedDialogueLine);
			}
		}

		// Sort the exported lines to maintain a consistent order between exports
		// Sort order is speaking voice name, then localized dialogue
		ExportedDialogueLines.Sort([](const TSharedPtr<FDialogueScriptEntry>& InFirstEntry, const TSharedPtr<FDialogueScriptEntry>& InSecondEntry) -> bool
		{
			const int32 SpeakingVoiceResult = InFirstEntry->SpeakingVoice.Compare(InSecondEntry->SpeakingVoice, ESearchCase::CaseSensitive);
			if (SpeakingVoiceResult < 0)
			{
				return true;
			}

			if (SpeakingVoiceResult == 0 && InFirstEntry->SpokenDialogue.Compare(InSecondEntry->SpokenDialogue, ESearchCase::CaseSensitive) < 0)
			{
				return true;
			}

			return false;
		});

		{
			FString CSVFileData;
			CSVFileData += GenerateCSVHeader() + TEXT("\n");
			for (const auto& DialogueScriptEntry : ExportedDialogueLines)
			{
				CSVFileData += GenerateCSVRow(*DialogueScriptEntry) + TEXT("\n");
			}

			const FString CSVFileName = CultureDestinationPath / DialogueScriptName;
			const bool bCSVFileSaved = FLocalizedAssetSCCUtil::SaveFileWithSCC(SourceControlInfo, CSVFileName, [&](const FString& InSaveFileName) -> bool
			{
				return FFileHelper::SaveStringToFile(CSVFileData, *InSaveFileName, FFileHelper::EEncodingOptions::ForceUTF8);
			});

			if (!bCSVFileSaved)
			{
				UE_LOGF(LogExportDialogueScriptCommandlet, Error, "Failed to write CSV file for culture '%ls' to '%ls'.", *CultureName, *CSVFileName);
				continue;
			}
		}
	}

	return 0;
}

FString UExportDialogueScriptCommandlet::GenerateCSVHeader()
{
	FString CSVHeader;
	
	for (TFieldIterator<const FProperty> PropertyIt(FDialogueScriptEntry::StaticStruct(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); PropertyIt; ++PropertyIt)
	{
		if (!CSVHeader.IsEmpty())
		{
			CSVHeader += TEXT(",");
		}

		const FString PropertyName = PropertyIt->GetName();

		CSVHeader += TEXT("\"");
		CSVHeader += PropertyName.Replace(TEXT("\""), TEXT("\"\""));
		CSVHeader += TEXT("\"");
	}

	return CSVHeader;
}

FString UExportDialogueScriptCommandlet::GenerateCSVRow(const FDialogueScriptEntry& InDialogueScriptEntry)
{
	FString CSVRow;

	for (TFieldIterator<const FProperty> PropertyIt(FDialogueScriptEntry::StaticStruct(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); PropertyIt; ++PropertyIt)
	{
		if (!CSVRow.IsEmpty())
		{
			CSVRow += TEXT(",");
		}

		FString PropertyValue;
		PropertyIt->ExportTextItem_InContainer(PropertyValue, &InDialogueScriptEntry, nullptr, nullptr, PPF_None);

		CSVRow += TEXT("\"");
		CSVRow += PropertyValue.Replace(TEXT("\""), TEXT("\"\""));
		CSVRow += TEXT("\"");
	}

	return CSVRow;
}

void UExportDialogueScriptCommandlet::PopulateDialogueScriptEntry(const UDialogueWave* InDialogueWave, const UDialogueWave* InLocalizedDialogueWave, const FDialogueContextMapping& InPrimaryContext, const TArray<const FDialogueContextMapping*>& InAdditionalContexts, const FString& InLocalizedDialogue, const FString& InLocalizedVoiceActorDirection, FDialogueScriptEntry& OutDialogueScriptEntry)
{
	auto AppendTargetVoices = [&](const FDialogueContext& InContext)
	{
		if (InContext.Targets.Num() > 0)
		{
			FString TargetVoicesText;

			const bool bIsArray = InContext.Targets.Num() > 1;
			if (bIsArray)
			{
				TargetVoicesText += TEXT("[");
			}

			bool bIsFirst = true;
			for (const UDialogueVoice* TargetVoice : InContext.Targets)
			{
				if (!bIsFirst)
				{
					TargetVoicesText += TEXT(",");
				}
				bIsFirst = false;

				TargetVoicesText += TargetVoice->GetName();
			}

			if (bIsArray)
			{
				TargetVoicesText += TEXT("]");
			}

			OutDialogueScriptEntry.TargetVoices.Add(MoveTemp(TargetVoicesText));
		}
	};

	auto AppendTargetVoiceGUIDs = [&](const FDialogueContext& InContext)
	{
		if (InContext.Targets.Num() > 0)
		{
			FString TargetVoiceGUIDsText;

			const bool bIsArray = InContext.Targets.Num() > 1;
			if (bIsArray)
			{
				TargetVoiceGUIDsText += TEXT("[");
			}

			bool bIsFirst = true;
			for (const UDialogueVoice* TargetVoice : InContext.Targets)
			{
				if (!bIsFirst)
				{
					TargetVoiceGUIDsText += TEXT(",");
				}
				bIsFirst = false;

				TargetVoiceGUIDsText += TargetVoice->LocalizationGUID.ToString();
			}

			if (bIsArray)
			{
				TargetVoiceGUIDsText += TEXT("]");
			}

			OutDialogueScriptEntry.TargetVoiceGUIDs.Add(MoveTemp(TargetVoiceGUIDsText));
		}
	};

	auto HasLocalizedSoundWave = [&](const FDialogueContext& InContext) -> bool
	{
		if (InLocalizedDialogueWave)
		{
			for (const FDialogueContextMapping& LocalizedContextMapping : InLocalizedDialogueWave->ContextMappings)
			{
				if (LocalizedContextMapping.Context == InContext)
				{
					const UPackage* ObjPackage = LocalizedContextMapping.SoundWave ? LocalizedContextMapping.SoundWave->GetOutermost() : nullptr;
                	return ObjPackage && FPackageName::IsLocalizedPackage(ObjPackage->GetPathName());
				}
			}
		}
		return false;
	};

	OutDialogueScriptEntry.SpokenDialogue = InLocalizedDialogue;
	OutDialogueScriptEntry.VoiceActorDirection = InLocalizedVoiceActorDirection;
	OutDialogueScriptEntry.AudioFileName = InDialogueWave->GetContextRecordedAudioFilename(InPrimaryContext);
	OutDialogueScriptEntry.DialogueAsset = InDialogueWave->GetPathName();
	OutDialogueScriptEntry.IsRecorded = HasLocalizedSoundWave(InPrimaryContext.Context);

	OutDialogueScriptEntry.SpeakingVoice = InPrimaryContext.Context.Speaker->GetName();
	OutDialogueScriptEntry.SpeakingVoiceGUID = InPrimaryContext.Context.Speaker->LocalizationGUID.ToString();
	OutDialogueScriptEntry.DialogueAssetGUID = InDialogueWave->LocalizationGUID.ToString();

	OutDialogueScriptEntry.LocalizationKeys.Add(InDialogueWave->GetContextLocalizationKey(InPrimaryContext));
	AppendTargetVoices(InPrimaryContext.Context);
	AppendTargetVoiceGUIDs(InPrimaryContext.Context);

	for (const FDialogueContextMapping* AdditionalContext : InAdditionalContexts)
	{
		if (!OutDialogueScriptEntry.IsRecorded)
		{
			OutDialogueScriptEntry.IsRecorded = HasLocalizedSoundWave(AdditionalContext->Context);
		}

		OutDialogueScriptEntry.LocalizationKeys.Add(InDialogueWave->GetContextLocalizationKey(*AdditionalContext));
		AppendTargetVoices(AdditionalContext->Context);
		AppendTargetVoiceGUIDs(AdditionalContext->Context);
	}
}
