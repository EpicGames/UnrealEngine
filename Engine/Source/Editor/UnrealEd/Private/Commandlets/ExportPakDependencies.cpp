// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ExportPakDependencies.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMath.h"
#include "IPlatformFilePak.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/Archive.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExportPakDependencies)

struct FPackage
{
public:
	TSet<FPackage*> DirectlyReferencing;
	TSet<FPackage*> DirectlyReferencedBy;
	TSet<FPackage*> AllReferences;

	FName Name;
	int64 InclusiveSize;
	int64 ExclusiveSize;
	int Id;

private:
	bool bUpdateHelper;
	static TMap<FName,FPackage*> NameToPackageMap;

public:
	FPackage(const FName& InName,int InId)
		: Name(InName)
		, InclusiveSize(0)
		, ExclusiveSize(0)
		, Id(InId)
		, bUpdateHelper(false)
	{}

	static FPackage* FindOrCreatePackage( FName PackageName )
	{
		static int Id = 1;
		FPackage* Package = NameToPackageMap.FindRef(PackageName);
		if(!Package)
		{
			Package = new FPackage(PackageName,Id++);
			NameToPackageMap.Add(PackageName,Package);
		}
		return Package;
	}

	void ResetUpdateHelper()
	{
		bUpdateHelper = false;
	}

	void RecurseUpdateReferences()
	{
		if( !bUpdateHelper )
		{
			bUpdateHelper = true;
			for( auto& DirectReference : DirectlyReferencing )
			{
				AllReferences.Add(DirectReference);
				DirectReference->RecurseUpdateReferences();
				AllReferences.Append(DirectReference->AllReferences);
			}
		}
	}

	void UpdateInclusiveSize()
	{
		InclusiveSize = ExclusiveSize;
		for(auto& Reference : AllReferences)
		{
			InclusiveSize += Reference->ExclusiveSize;
		}
	}

	static void GetAllPackages( TArray<FPackage*>& OutPackages )
	{
		OutPackages.Reset(NameToPackageMap.Num());
		for( const auto& Entry : NameToPackageMap )
		{
			OutPackages.Add(Entry.Value);
		}
	}

	TArray< TSharedPtr<FJsonValue> > ToJsonHelper( const TSet<FPackage*>& Packages )
	{
		TArray< TSharedPtr<FJsonValue> > JsonPackageNames;
		for( const auto Package : Packages )
		{
			JsonPackageNames.Add(MakeShareable(new FJsonValueString(Package->Name.ToString())));
		}
		return JsonPackageNames;
	}

	TSharedPtr<FJsonObject> ToJsonObject()
	{
		TSharedPtr<FJsonObject> JsonPackageObject = MakeShareable(new FJsonObject);
		
		JsonPackageObject->SetStringField(TEXT("Name"),*Name.ToString());
		JsonPackageObject->SetNumberField(TEXT("InclusiveSize"),static_cast<double>(InclusiveSize));
		JsonPackageObject->SetNumberField(TEXT("ExclusiveSize"),static_cast<double>(ExclusiveSize));

		JsonPackageObject->SetArrayField(TEXT("DirectlyReferencing"),ToJsonHelper(DirectlyReferencing));
		JsonPackageObject->SetArrayField(TEXT("DirectlyReferencedBy"),ToJsonHelper(DirectlyReferencedBy));
		JsonPackageObject->SetArrayField(TEXT("AllReferences"),ToJsonHelper(AllReferences));

		return JsonPackageObject;
	}
};
TMap<FName,FPackage*> FPackage::NameToPackageMap;

bool ExportDependencies(const TCHAR * PakFilename, const TCHAR* GameName, const TCHAR* OutputFilenameBase, bool bSigned)
{
	// Example command line used for this tool
	// C:\Development\BB\WEX\Saved\StagedBuilds\WindowsClient\WorldExplorers\Content\Paks\WorldExplorers-WindowsNoEditor.pak WorldExplorers WEX -exportdependencies=c:\dvtemp\output -debug -NoAssetRegistryCache -ForceDependsGathering
	
	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), PakFilename, bSigned);
	FPakFile& PakFile = *PakFilePtr;

	if(PakFile.IsValid())
	{
		// Get size information from PAK file.
		{
			TArray<FPakFile::FPakEntryIterator> Records;
			FString PakGameContentFolder = FString(GameName) + TEXT("/Content");
			if (!PakFile.HasFilenames())
			{
				UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, cannot export.");
				return false;
			}
			for(FPakFile::FPakEntryIterator It(PakFile); It; ++It)
			{
				FString PackageName;
				It.TryGetFilename()->Split(TEXT("."),&PackageName,NULL);
				int64 Size = It.Info().Size;			

				if( PackageName.StartsWith(TEXT("Engine/Content")) )
				{
					PackageName = PackageName.Replace(TEXT("Engine/Content"),TEXT("/Engine"));
				}
				else if( PackageName.StartsWith(*PakGameContentFolder))
				{
					PackageName = PackageName.Replace(*PakGameContentFolder,TEXT("/Game"));
				}

				FPackage* Package = FPackage::FindOrCreatePackage(FName(*PackageName));
				Package->ExclusiveSize += Size;
			}
		}

		TMap<FName, FTopLevelAssetPath> PackageToClassMap;

		// Combine with dependency information from asset registry.
		{
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		
			AssetRegistryModule.Get().SearchAllAssets(true);
			TArray<FAssetData> AssetData;		
			AssetRegistryModule.Get().GetAllAssets(AssetData,true);	
	
			TSet<FName> PackageNames;
			for( int i=0; i<AssetData.Num(); i++ )
			{
				PackageNames.Add(AssetData[i].PackageName);
				PackageToClassMap.Add(AssetData[i].PackageName,AssetData[i].AssetClassPath);
			}

			for( const auto& PackageName : PackageNames )
			{
				TArray<FName> DependencyArray;
				AssetRegistryModule.Get().GetDependencies(PackageName,DependencyArray);

				FPackage* Package = FPackage::FindOrCreatePackage(PackageName);
				for( const auto& DependencyName : DependencyArray )
				{
					// exclude '/Script/' as it clutters up things significantly.
					if( !DependencyName.ToString().StartsWith(TEXT("/Script/")) )
					{
						FPackage* Dependency = FPackage::FindOrCreatePackage(DependencyName);
						Package->DirectlyReferencing.Add(Dependency);
						Dependency->DirectlyReferencedBy.Add(Package);
					}
				}
			}

			// 2 passes are required to deal with cycles.
			for(const auto& PackageName : PackageNames)
			{
				FPackage* Package = FPackage::FindOrCreatePackage(PackageName);
				Package->RecurseUpdateReferences();
			}
			for(const auto& PackageName : PackageNames)
			{
				FPackage* Package = FPackage::FindOrCreatePackage(PackageName);
				Package->ResetUpdateHelper();
			}
			for(const auto& PackageName : PackageNames)
			{
				FPackage* Package = FPackage::FindOrCreatePackage(PackageName);
				Package->RecurseUpdateReferences();
			}
		}

		// Update inclusive size, asset class, and export to CSV, JSON, and GDF
		{
			TSharedPtr<FJsonObject> JsonRootObject = MakeShareable(new FJsonObject);
			TArray< TSharedPtr<FJsonValue> > JsonPackages;

			TArray<FPackage*> AllPackages;
			FPackage::GetAllPackages(AllPackages);

			for(auto Package : AllPackages)
			{
				Package->UpdateInclusiveSize();
				JsonPackages.Add( MakeShareable(new FJsonValueObject(Package->ToJsonObject())) );
			}
			JsonRootObject->SetArrayField(TEXT("Packages"),JsonPackages);

			FString JsonOutputString;
			TSharedRef<TJsonWriter<TCHAR,TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR,TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonOutputString);
			FJsonSerializer::Serialize(JsonRootObject.ToSharedRef(),JsonWriter);

			FArchive* JsonFileWriter = IFileManager::Get().CreateFileWriter(*(FString(OutputFilenameBase)+TEXT(".json")));
			if(JsonFileWriter)
			{
				JsonFileWriter->Logf(TEXT("%s"),*JsonOutputString);
				JsonFileWriter->Close();
				delete JsonFileWriter;
			}

			FArchive* CSVFileWriter = IFileManager::Get().CreateFileWriter(*(FString(OutputFilenameBase)+TEXT(".csv")));
			if(CSVFileWriter)
			{
				CSVFileWriter->Logf(TEXT("class,name,inclusive,exclusive"));
				for(auto Package : AllPackages)
				{
					FTopLevelAssetPath ClassName = PackageToClassMap.FindRef(Package->Name);
					CSVFileWriter->Logf(TEXT("%s,%s,%i,%i"),*ClassName.ToString(),*Package->Name.ToString(),Package->InclusiveSize,Package->ExclusiveSize);
				}
				CSVFileWriter->Close();
				delete CSVFileWriter;
				CSVFileWriter = NULL;
			}

			FArchive* GDFFileWriter = IFileManager::Get().CreateFileWriter(*(FString(OutputFilenameBase)+TEXT(".gdf")));
			if(GDFFileWriter)
			{
				GDFFileWriter->Logf(TEXT("nodedef> name VARCHAR,label VARCHAR,inclusive DOUBLE,exclusive DOUBLE"));
				GDFFileWriter->Logf(TEXT("0,root,0,0"));
				for(auto Package : AllPackages)
				{
					GDFFileWriter->Logf(TEXT("%i,%s,%i,%i"),Package->Id,*Package->Name.ToString(),Package->InclusiveSize,Package->ExclusiveSize);
				}
				GDFFileWriter->Logf(TEXT("edgedef> node1 VARCHAR,node2 VARCHAR"));
				// fake root to ensure spanning tree
				for(auto Package : AllPackages)
				{
					GDFFileWriter->Logf(TEXT("0,%i"),Package->Id);
				}
				for(auto Package : AllPackages)
				{
					for( auto ReferencedPackage : Package->DirectlyReferencing )
					{
						GDFFileWriter->Logf(TEXT("%i,%i"),Package->Id,ReferencedPackage->Id);
					}
				}
				GDFFileWriter->Close();
				delete GDFFileWriter;
				GDFFileWriter = NULL;
			}
		}
		return true;
	}
	else
	{
		return false;
	}
}

UExportPakDependenciesCommandlet::UExportPakDependenciesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UExportPakDependenciesCommandlet::Main(const FString& Params)
{
	FString PakFilename;
	FString ExportDependencyFilename;
	if (!FParse::Value(FCommandLine::Get(), TEXT("PakFile="), PakFilename) || 
		!FParse::Value(FCommandLine::Get(), TEXT("Output="), ExportDependencyFilename))
	{
		UE_LOGF(LogPakFile,Error,"Incorrect arguments. Expected: -PakFile=<FileName> -Output=<FileName> [-Signed]");
		return false;
	}

	bool bSigned = FParse::Param(FCommandLine::Get(), TEXT("signed"));
	return ExportDependencies(*PakFilename, FApp::GetProjectName(), *ExportDependencyFilename, bSigned)? 0 : 1;
}
