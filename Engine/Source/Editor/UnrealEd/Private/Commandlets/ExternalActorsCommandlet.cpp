// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ExternalActorsCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Editor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectHash.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PackageHelperFunctions.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalActorsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogExternalActorsCommandlet, Log, All);

UExternalActorsCommandlet::UExternalActorsCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UWorld* UExternalActorsCommandlet::LoadWorld(const FString& LevelToLoad)
{
	SET_WARN_COLOR(COLOR_WHITE);
	UE_LOGF(LogExternalActorsCommandlet, Log, "Loading level %ls.", *LevelToLoad);
	CLEAR_WARN_COLOR();

	UPackage* MapPackage = LoadPackage(nullptr, *LevelToLoad, LOAD_None);
	if (!MapPackage)
	{
		UE_LOGF(LogExternalActorsCommandlet, Error, "Error loading %ls.", *LevelToLoad);
		return nullptr;
	}

	return UWorld::FindWorldInPackage(MapPackage);
}

int32 UExternalActorsCommandlet::Main(const FString& Params)
{
	FPackageSourceControlHelper PackageHelper;

	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);
	ApplyCommandLineSwitches(this, Switches);

	if (bDisable)
	{
		if (bRepair)
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Bad parameters: -repair cannot be used with -disable");
			return 1;
		}

		if (bEnable)
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Bad parameters: -enable cannot be used with -disable");
			return 1;
		}

		bListMaps |= !DumpCSVFile.IsEmpty();
	}
	else if (bForce || bReport || bListMaps || DumpCSVFile.Len())
	{
		UE_LOGF(LogExternalActorsCommandlet, Error, "Bad parameters: -force, -report, -listmaps and -dumpcsv can only be used along with -disable");
		return 1;
	}

	if (bDisable && bListMaps)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);

		UE_LOGF(LogExternalActorsCommandlet, Display, "Waiting for asset registry...");
		AssetRegistryModule.Get().SearchAllAssets(true);

		TArray<FAssetData> WorldAssets;
		AssetRegistryModule.Get().GetAssetsByClass(UWorld::StaticClass()->GetClassPathName(), WorldAssets, true);
		WorldAssets.Sort();

		struct FWorldEntry
		{
			FString Path;
			int32 NumExternalActors;
			FString DateModified;

			inline bool operator<(const FWorldEntry& Other) const
			{
				return NumExternalActors > Other.NumExternalActors;
			}
		};

		TArray<FWorldEntry> WorldEntries;
		for (const FAssetData& WorldAsset : WorldAssets)
		{
			if (ULevel::GetIsLevelUsingExternalActorsFromAsset(WorldAsset))
			{
				const FString ExternalActorsPath = ULevel::GetExternalActorsPath(WorldAsset.PackageName.ToString());

				TArray<FAssetData> WorldExternalActors;
				AssetRegistryModule.Get().GetAssetsByPath(*ExternalActorsPath, WorldExternalActors, true, true);

				if (!ULevel::GetIsLevelPartitionedFromAsset(WorldAsset) || ULevel::GetIsStreamingDisabledFromAsset(WorldAsset))
				{
					FString DateModified;
					WorldAsset.GetTagValue(TEXT("DateModified"), DateModified);

					WorldEntries.Add(
					{
						.Path = WorldAsset.PackageName.ToString(),
						.NumExternalActors = WorldExternalActors.Num(),
						.DateModified = DateModified
					});

					UE_LOGF(LogExternalActorsCommandlet, Display, "Level '%ls' is a potential candidate to have external actors disabled (%d).", *WorldAsset.GetSoftObjectPath().ToString(), WorldExternalActors.Num());
				}
				else
				{
					UE_LOGF(LogExternalActorsCommandlet, Verbose, "Level '%ls' is not a potential candidate to have external actors disabled (%d).", *WorldAsset.GetSoftObjectPath().ToString(), WorldExternalActors.Num());
				}
			}
		}

		if (DumpCSVFile.Len())
		{
			FArchive* LogFile = IFileManager::Get().CreateFileWriter(*DumpCSVFile);
			if (!LogFile)
			{
				UE_LOGF(LogExternalActorsCommandlet, Error, "Cannot create csv file '%ls'", *DumpCSVFile);
				return 1;
			}

			FString LineEntry = TEXT("Map,NumOFPA,DateModified") LINE_TERMINATOR;
			LogFile->Serialize(TCHAR_TO_ANSI(*LineEntry), LineEntry.Len());

			WorldEntries.Sort();

			for (const FWorldEntry& WorldEntry : WorldEntries)
			{
				LineEntry = FString::Printf(TEXT("%s,%d,%s" LINE_TERMINATOR), *WorldEntry.Path, WorldEntry. NumExternalActors, *WorldEntry.DateModified);
				LogFile->Serialize(TCHAR_TO_ANSI(*LineEntry), LineEntry.Len());
			}

			LogFile->Close();
			delete LogFile;
		}

		return 0;
	}

	TArray<FString> MapList;

	if (!MapListFile.IsEmpty())
	{
		if (!FFileHelper::LoadFileToStringArray(MapList, *MapListFile))
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Invalid map list filename");
			return 1;
		}
	}
	else
	{
		if (Tokens.Num() < 1)
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Missing map name");
			return 1;
		}

		MapList.Add(Tokens[0]);
	}

	int32 Result = 1;
	for (const FString& MapName : MapList)
	{
		FString FullMapName;
		if (!FPackageName::SearchForPackageOnDisk(MapName, &FullMapName))
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Unknown level '%ls'", *MapName);
			Result = 0;
			continue;
		}

		// Load world
		UWorld* MainWorld = LoadWorld(FullMapName);
		if (!MainWorld)
		{
			UE_LOGF(LogExternalActorsCommandlet, Error, "Unknown world '%ls'", *FullMapName);
			Result = 0;
			continue;
		}

		TSet<UPackage*> PackagesToSave;
		TArray<FString> PackagesToDelete;	

		if (bDisable)
		{
			if (!MainWorld->PersistentLevel->IsUsingExternalActors())
			{
				UE_LOGF(LogExternalActorsCommandlet, Error, "Cannot disable external actors for level '%ls' (already disabled)", *FullMapName);
				Result = 0;
				continue;
			}

			if (UWorldPartition* WorldPartition = MainWorld->GetWorldPartition())
			{
				int32 NumErrors = 0;

				WorldPartition->Initialize(MainWorld, FTransform::Identity);

				if (WorldPartition->IsStreamingEnabled())
				{
					if (bForce)
					{
						WorldPartition->SetEnableStreaming(false);
					}
					else
					{
						UE_LOGF(LogExternalActorsCommandlet, Error, "Cannot disable external actors for partitioned level '%ls' with streaming enabled", *FullMapName);
						NumErrors++;
					}
				}

				if (UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager())
				{
					if (DataLayerManager->GetDataLayerInstances().Num())
					{
						UE_LOGF(LogExternalActorsCommandlet, Error, "Cannot disable external actors for partitioned level '%ls' with data layers", *FullMapName);
						NumErrors++;
					}
				}

				if (!NumErrors)
				{
					UE_LOGF(LogExternalActorsCommandlet, Display, "External actors for partitioned level '%ls' can be disabled", *FullMapName);
				}
				else if (bForce)
				{
					UE_LOGF(LogExternalActorsCommandlet, Display, "External actors for partitioned level '%ls' will be forcibly disabled", *FullMapName);
				}
				else
				{
					Result = 0;
					continue;
				}

				if (!bReport)
				{
					TArray<FString> LocalPackagesToDelete;
					WorldPartition->ForEachActorDescContainerInstance([&LocalPackagesToDelete](UActorDescContainerInstance* ActorDescContainerInstance)
					{
						for (const FAssetData& InvalidActor : ActorDescContainerInstance->GetContainer()->GetInvalidActors())
						{
							LocalPackagesToDelete.Add(InvalidActor.ToSoftObjectPath().ToString());
						}
					});

					// This will uninitialize the world partition object 
					if (!UWorldPartition::RemoveWorldPartition(WorldPartition->GetWorld()->GetWorldSettings()))
					{
						WorldPartition->Uninitialize();
						UE_LOGF(LogExternalActorsCommandlet, Error, "Error disabling external actors for partitioned level '%ls'", *FullMapName);
						Result = 0;
						continue;
					}

					PackagesToDelete.Append(LocalPackagesToDelete);
				}
			}

			if (!bReport)
			{
				ForEachObjectWithOuter(MainWorld->PersistentLevel, [MainWorld, &PackagesToDelete](UObject* Object)
				{
					if (Object->IsPackageExternal())
					{
						PackagesToDelete.Add(Object->GetPackage()->GetLoadedPath().GetLocalFullPath());

						if (IsValid(Object))
						{
							if (AActor* Actor = Cast<AActor>(Object))
							{
								Actor->SetPackageExternal(false);
							}
							else
							{
								FExternalPackageHelper::SetPackagingMode(Object, MainWorld->PersistentLevel, false);
							}
						}
					}
					return true;
				});

				if (PackagesToDelete.Num())
				{
					PackagesToSave.Add(MainWorld->GetPackage());
				}

				MainWorld->PersistentLevel->SetUseActorFolders(false);
				MainWorld->PersistentLevel->SetUseExternalActors(false);
			}
		}
		else if (bEnable)
		{
			if (MainWorld->PersistentLevel->IsUsingExternalActors())
			{
				UE_LOGF(LogExternalActorsCommandlet, Error, "Cannot enable external actors for level '%ls' (already enabled)", *FullMapName);
				Result = 0;
				continue;
			}

			if (!bReport)
			{
				for (AActor* Actor : MainWorld->PersistentLevel->Actors)
				{
					if (IsValid(Actor) && Actor->SupportsExternalPackaging())
					{
						Actor->SetPackageExternal(true);
						PackagesToSave.Add(Actor->GetPackage());
					}
				}

				if (PackagesToSave.Num())
				{
					PackagesToSave.Add(MainWorld->GetPackage());
				}

				MainWorld->PersistentLevel->SetUseExternalActors(true);
			}
		}
		else
		{
			// Validate external actors
			FString ExternalActorsPath = ULevel::GetExternalActorsPath(FullMapName);
			FString ExternalActorsFilePath = FPackageName::LongPackageNameToFilename(ExternalActorsPath);

			// Look for duplicated actor GUIDs
			TMap<FGuid, AActor*> ActorGuids;

			if (IFileManager::Get().DirectoryExists(*ExternalActorsFilePath))
			{
				bool bResult = IFileManager::Get().IterateDirectoryRecursively(*ExternalActorsFilePath, [this, &PackagesToSave, &PackagesToDelete, &ActorGuids](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
				{
					if (!bIsDirectory)
					{
						FString Filename(FilenameOrDirectory);
						if (Filename.EndsWith(FPackageName::GetAssetPackageExtension()))
						{
							AActor* MainPackageActor = nullptr;
							AActor* PotentialMainPackageActor = nullptr;

							const FString PackageName = FPackageName::FilenameToLongPackageName(*Filename);
							if (UPackage* Package = LoadPackage(nullptr, *Filename, LOAD_None, nullptr, nullptr))
							{
								ForEachObjectWithPackage(Package, [&MainPackageActor, &PotentialMainPackageActor](UObject* Object)
								{
									if (AActor* Actor = Cast<AActor>(Object))
									{
										if (Actor->IsMainPackageActor())
										{
											MainPackageActor = Actor;
											PotentialMainPackageActor = nullptr;
										}
										else if (!MainPackageActor)
										{
											if (!Actor->IsChildActor())
											{
												PotentialMainPackageActor = Actor;
											}
										}
									}
									return true;
								});
							}

							if (!MainPackageActor)
							{
								UE_LOGF(LogExternalActorsCommandlet, Error, "Missing main actor for file '%ls'", *Filename);

								if (bRepair)
								{
									if (PotentialMainPackageActor)
									{
										PotentialMainPackageActor->SetPackageExternal(false);
										PotentialMainPackageActor->SetPackageExternal(true);
								
										UPackage* PackageToSave = PotentialMainPackageActor->GetPackage();
										PackagesToSave.Add(PackageToSave);

										MainPackageActor = PotentialMainPackageActor;
									}

									PackagesToDelete.Add(Filename);
								}
							}

							if (MainPackageActor)
							{
								if (AActor** ExistingActor = ActorGuids.Find(MainPackageActor->GetActorGuid()))
								{
									UE_LOGF(LogExternalActorsCommandlet, Error, "Duplicated actor guid '%ls' for file'%ls':", *MainPackageActor->GetActorGuid().ToString(), *Filename);
									UE_LOGF(LogExternalActorsCommandlet, Error, "\tActor 1: %ls (ignore)", *(*ExistingActor)->GetName())
									UE_LOGF(LogExternalActorsCommandlet, Error, "\tActor 2: %ls (keep)", *MainPackageActor->GetName());									

									if (bRepair)
									{
										FSetActorGuid SetActorGuid(MainPackageActor, FGuid::NewGuid());

										UPackage* PackageToSave = MainPackageActor->GetPackage();
										PackagesToSave.Add(PackageToSave);
									}
								}
								else
								{
									const FString ActorPath = MainPackageActor->GetPathName();
									const FString ActorPackageName = ULevel::GetActorPackageName(MainPackageActor->GetLevel()->GetPackage(), MainPackageActor->GetLevel()->GetActorPackagingScheme(), ActorPath);

									if (MainPackageActor->GetPackage()->GetName() != ActorPackageName)
									{
										UE_LOGF(LogExternalActorsCommandlet, Warning, "Mismatched actor filename for actor '%ls':", *MainPackageActor->GetName());
										UE_LOGF(LogExternalActorsCommandlet, Warning, "\t Current filename: %ls", *MainPackageActor->GetPackage()->GetName());
										UE_LOGF(LogExternalActorsCommandlet, Warning, "\tExpected filename: %ls", *ActorPackageName);										
									}

									ActorGuids.Add(MainPackageActor->GetActorGuid(), MainPackageActor);
								}
							}
						}
						else
						{
							UE_LOGF(LogExternalActorsCommandlet, Error, "Invalid actor file '%ls'", *Filename);

							if (bRepair)
							{
								PackagesToDelete.Add(Filename);
							}
						}
					}
					return true;
				});
			}
		}

		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;

		bool bForceInitializedWorld;
		const bool bInitializedPhysicsSceneForSave = GEditor->InitializePhysicsSceneForSaveIfNecessary(MainWorld, bForceInitializedWorld);

		for (UPackage* PackageToSave : PackagesToSave)
		{
			const FString PackageFileName = SourceControlHelpers::PackageFilename(PackageToSave);
			UE_LOGF(LogExternalActorsCommandlet, Display, "Saving package '%ls'", *PackageFileName);

			bool bCanSavePackage = true;
			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*PackageFileName))
			{
				bCanSavePackage = PackageHelper.Checkout(PackageToSave);
			}

			if (bCanSavePackage && UPackage::SavePackage(PackageToSave, nullptr, *PackageFileName, SaveArgs))
			{
				PackageHelper.AddToSourceControl(PackageToSave);
			}
		}

		if (bInitializedPhysicsSceneForSave)
		{
			GEditor->CleanupPhysicsSceneThatWasInitializedForSave(MainWorld, bForceInitializedWorld);
		}

		CollectGarbage(RF_NoFlags);

		PackagesToDelete.Sort();
		for (const FString& PackageToDelete : PackagesToDelete)
		{
			const FString PackageFileName = SourceControlHelpers::PackageFilename(PackageToDelete);
			UE_LOGF(LogExternalActorsCommandlet, Display, "Deleting package '%ls'", *PackageFileName);

			PackageHelper.Delete(*PackageToDelete);
		}
	}

	return Result;
}
