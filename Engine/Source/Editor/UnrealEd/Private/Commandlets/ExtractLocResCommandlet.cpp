// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ExtractLocResCommandlet.h"

#include "Commandlets/GatherTextCommandletBase.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/LocalizedTextSourceTypes.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextLocalizationResource.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExtractLocResCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogExtractLocRes, Log, All);

int32 UExtractLocResCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, Parameters);

	// Build up the complete list of LocRes files to process
	TArray<FString> LocResFileNames;
	if (const FString* LocResParamPtr = Parameters.Find(TEXT("LocRes")))
	{
		TArray<FString> PotentialLocResFileNames;
		LocResParamPtr->ParseIntoArray(PotentialLocResFileNames, TEXT(";"));

		const FString& ProjectBasePath = UGatherTextCommandletBase::GetProjectBasePath();

		for (FString& PotentialLocResFileName : PotentialLocResFileNames)
		{
			if (FPaths::IsRelative(PotentialLocResFileName))
			{
				PotentialLocResFileName = FPaths::Combine(*ProjectBasePath, *PotentialLocResFileName);
			}

			if (FPaths::FileExists(PotentialLocResFileName))
			{
				LocResFileNames.Add(PotentialLocResFileName);
			}
			else if (FPaths::DirectoryExists(PotentialLocResFileName))
			{
				IFileManager::Get().FindFilesRecursive(LocResFileNames, *PotentialLocResFileName, TEXT("*.locres"), /*bFiles*/true, /*bDirectories*/false, /*bClearFilesArray*/false);
			}
			else
			{
				UE_LOGF(LogExtractLocRes, Warning, "'%ls' did not map to a known file or directory.", *PotentialLocResFileName);
				continue;
			}
		}
	}

	if (LocResFileNames.Num() == 0)
	{
		UE_LOGF(LogExtractLocRes, Error, "-LocRes not specified or failed to resolve to a valid file.");
		return -1;
	}

	// Load each LocRes and dump it as CSV
	for (const FString& LocResFileName : LocResFileNames)
	{
		FTextLocalizationResource LocResFile;
		if (!LocResFile.LoadFromFile(LocResFileName, 0))
		{
			UE_LOGF(LogExtractLocRes, Error, "'%ls' failed to load.", *LocResFileName);
			continue;
		}

		FString LocResCSV;

		auto WriteCSVStringValue = [&LocResCSV](const FString& InValue)
		{
			LocResCSV += TEXT("\"");
			LocResCSV += InValue.ReplaceCharWithEscapedChar();
			LocResCSV += TEXT("\"");
		};

		// Write the header
		{
			LocResCSV += TEXT("Namespace");
			LocResCSV += TEXT(",");
			LocResCSV += TEXT("Key");
			LocResCSV += TEXT(",");
			LocResCSV += TEXT("SourceStringHash");
			LocResCSV += TEXT(",");
			LocResCSV += TEXT("LocalizedString");
			LocResCSV += TEXT("\n");
		}

		// Write each row
		for (const auto& LocResEntryPair : LocResFile.Entries)
		{
			WriteCSVStringValue(LocResEntryPair.Key.GetNamespace().ToString());
			LocResCSV += TEXT(",");
			WriteCSVStringValue(LocResEntryPair.Key.GetKey().ToString());
			LocResCSV += TEXT(",");
			LocResCSV += FString::Printf(TEXT("0x%08x"), LocResEntryPair.Value.SourceStringHash);
			LocResCSV += TEXT(",");
			WriteCSVStringValue(*LocResEntryPair.Value.LocalizedString);
			LocResCSV += TEXT("\n");
		}

		const FString LocResCSVFileName = FPaths::GetPath(LocResFileName) / FString::Printf(TEXT("%s_Extracted.csv"), *FPaths::GetBaseFilename(LocResFileName));
		if (!FFileHelper::SaveStringToFile(LocResCSV, *LocResCSVFileName, FFileHelper::EEncodingOptions::ForceUTF8))
		{
			UE_LOGF(LogExtractLocRes, Error, "'%ls' failed to write.", *LocResCSVFileName);
			continue;
		}

		UE_LOGF(LogExtractLocRes, Display, "Extracted '%ls' as '%ls'.", *LocResFileName, *LocResCSVFileName);
	}

	return 0;
}
