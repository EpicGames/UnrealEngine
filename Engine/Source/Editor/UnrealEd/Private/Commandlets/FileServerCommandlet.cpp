// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/FileServerCommandlet.h"
#include "Misc/MessageDialog.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "FileServerMessages.h"

#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "IMessageContext.h"
#include "INetworkFileServer.h"
#include "INetworkFileSystemModule.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "IPAddress.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileServerCommandlet)


DEFINE_LOG_CATEGORY_STATIC(LogFileServerCommandlet, Log, All);


/* UFileServerCommandlet structors
 *****************************************************************************/

UFileServerCommandlet::UFileServerCommandlet( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{

	IsClient = false;
	IsEditor = false;
	IsServer = false;

	LogToConsole = false;
}


/* UFileServerCommandlet interface
 *****************************************************************************/

int32 UFileServerCommandlet::Main( const FString& Params )
{
	GIsRunning = true;

	//@todo abstract properly or delete
#if PLATFORM_WINDOWS// Windows only
	// Used by the .com wrapper to notify that the Ctrl-C handler was triggered.
	// This shared event is checked each tick so that the log file can be cleanly flushed.
	FEvent* ComWrapperShutdownEvent = FPlatformProcess::GetSynchEventFromPool(true);
#endif

	// parse instance identifier
	FString InstanceIdString;

	if (FParse::Value(*Params, TEXT("InstanceId="), InstanceIdString))
	{
		if (!FGuid::Parse(InstanceIdString, InstanceId))
		{
			UE_LOGF(LogFileServerCommandlet, Warning, "Invalid InstanceId on command line: %ls", *InstanceIdString);
		}
	}

	// start the listening thread
	INetworkFileServer* NetworkFileServer = FModuleManager::LoadModuleChecked<INetworkFileSystemModule>("NetworkFileSystem")
		.CreateNetworkFileServer(true, InstanceId.IsValid() ? 0 : -1);

	TArray<TSharedPtr<FInternetAddr> > AddressList;

	if ((NetworkFileServer == NULL) || !NetworkFileServer->GetAddressList(AddressList))
	{
		UE_LOGF(LogFileServerCommandlet, Error, "Failed to create network file server");

		return -1;
	}

	// broadcast our presence
	if (InstanceId.IsValid())
	{
		TArray<FString> AddressStringList;

		for (int32 AddressIndex = 0; AddressIndex < AddressList.Num(); ++AddressIndex)
		{
			AddressStringList.Add(AddressList[AddressIndex]->ToString(true));
		}

		TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = FMessageEndpoint::Builder("UFileServerCommandlet").Build();

		if (MessageEndpoint.IsValid())
		{
			MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FFileServerReady>(AddressStringList, InstanceId), EMessageScope::Network);
		}		
	}

	// main loop
	FDateTime LastConnectionTime = FDateTime::UtcNow();

	while (GIsRunning && !IsEngineExitRequested())
	{
		GEngine->UpdateTimeAndHandleMaxTickRate();
		GEngine->Tick(static_cast<float>(FApp::GetDeltaTime()), false);

		// tick the directory watcher
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		DirectoryWatcherModule.Get()->Tick(static_cast<float>(FApp::GetDeltaTime()));

		// update task graph
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		// execute deferred commands
		GEngine->TickDeferredCommands();

		// handle server timeout
		if (InstanceId.IsValid())
		{
			if (NetworkFileServer->NumConnections() > 0)
			{
				LastConnectionTime = FDateTime::UtcNow();
			}

			if ((FDateTime::UtcNow() - LastConnectionTime) > FTimespan::FromMinutes(3.0))
			{
				uint32 Result = FMessageDialog::Open(EAppMsgType::YesNo, NSLOCTEXT("UnrealEd", "FileServerIdle", "The file server did not receive any connections in the past 3 minutes. Would you like to shut it down?"));

				if (Result == EAppReturnType::No)
				{
					LastConnectionTime = FDateTime::UtcNow();
				}
				else
				{
					break;
				}
			}
		}

		// flush log
		GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);

#if PLATFORM_WINDOWS
		if (ComWrapperShutdownEvent->Wait(0))
		{
			RequestEngineExit(TEXT("FileServerCommandlet ComWrapperShutdownEvent"));
		}
#endif
	}

	// shutdown the server
	NetworkFileServer->Shutdown();
	delete NetworkFileServer;

	//@todo abstract properly or delete 
#if PLATFORM_WINDOWS
	FPlatformProcess::ReturnSynchEventToPool(ComWrapperShutdownEvent);
	ComWrapperShutdownEvent = nullptr;
#endif

	GIsRunning = false;

	return 0;
}
