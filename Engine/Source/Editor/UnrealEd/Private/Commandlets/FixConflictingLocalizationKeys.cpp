// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/FixConflictingLocalizationKeys.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "LocTextHelper.h"
#include "LocalizationSourceControlUtil.h"
#include "LocalizedAssetUtil.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Char.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FixConflictingLocalizationKeys)

DEFINE_LOG_CATEGORY_STATIC(LogFixConflictingLocalizationKeys, Log, All);

enum class EMangledPropertyContainerType : uint8
{
	Fixed,
	Dynamic,
	DynamicKey,
};

bool UnmanglePropertyName(const FString& InName, FString& OutName, EMangledPropertyContainerType& OutType, int32& OutIndex)
{
	// Undo the name manging done by FPropertyLocalizationDataGatherer...
	if (InName.Len() < 5)
	{
		return false;
	}

	if (InName[InName.Len() - 1] == TEXT(']'))
	{
		// Fixed size array "{PropertyName}[{Index}]"
		int32 IndexStartIndex = INDEX_NONE;
		if (InName.FindLastChar(TEXT('['), IndexStartIndex))
		{
			OutName = InName.Left(IndexStartIndex);

			const FString IndexStr = InName.Mid(IndexStartIndex + 1, InName.Len() - IndexStartIndex - 1);
			LexFromString(OutIndex, *IndexStr);

			OutType = EMangledPropertyContainerType::Fixed;
			return true;
		}
	}
	else if (InName[InName.Len() - 1] == TEXT(')'))
	{
		// Dynamic array or set "{PropertyName}({Index})"
		// Map key "{PropertyName}({Index} - Key)"
		// Map value "{PropertyName}({Index} - Value)"
		int32 IndexStartIndex = INDEX_NONE;
		if (InName.FindLastChar(TEXT('('), IndexStartIndex))
		{
			OutName = InName.Left(IndexStartIndex);

			int32 IndexLen = 0;
			for (int32 i = IndexStartIndex + 1; i < InName.Len() - 1; ++i, ++IndexLen)
			{
				if (!FChar::IsDigit(InName[i]))
				{
					break;
				}
			}

			const FString IndexStr = InName.Mid(IndexStartIndex + 1, IndexLen);
			LexFromString(OutIndex, *IndexStr);

			OutType = InName[InName.Len() - 2] == TEXT('y') ? EMangledPropertyContainerType::DynamicKey : EMangledPropertyContainerType::Dynamic;
			return true;
		}
	}

	return false;
}

bool ReKeyTextProperty(UObject* InOuter, const TArray<FString>& InConflictingSourceParts, const int32 InPartIndex);
bool ReKeyTextProperty(UStruct* InOuterType, void* InAddrToUpdate, const TArray<FString>& InConflictingSourceParts, const int32 InPartIndex);

bool ReKeyTextProperty(UObject* InOuter, const TArray<FString>& InConflictingSourceParts, const int32 InPartIndex)
{
	if (!InConflictingSourceParts.IsValidIndex(InPartIndex))
	{
		return false;
	}

	// The path contains both objects and properties...
	const FString& PathPart = InConflictingSourceParts[InPartIndex];

	// We test objects first...
	UObject *ObjToUpdate = StaticFindObject(UObject::StaticClass(), InOuter, *PathPart);
	if (ObjToUpdate)
	{
		return ReKeyTextProperty(ObjToUpdate, InConflictingSourceParts, InPartIndex + 1);
	}

	// Then start looking for properties...
	return ReKeyTextProperty(InOuter->GetClass(), InOuter, InConflictingSourceParts, InPartIndex);
}

bool ReKeyTextProperty(UStruct* InOuterType, void* InAddrToUpdate, const TArray<FString>& InConflictingSourceParts, const int32 InPartIndex)
{
	if (!InConflictingSourceParts.IsValidIndex(InPartIndex))
	{
		return false;
	}

	const FString& PathPart = InConflictingSourceParts[InPartIndex];

	FTextProperty* TextPropToUpdate = nullptr;
	void* AddrToUpdate = InAddrToUpdate;

	// First check using the name we were given (which may be mangled)
	if (FProperty* UnmangedPropToUpdate = InOuterType->FindPropertyByName(*PathPart))
	{
		// Is this a complex property? If so, we need to recurse into it
		if (FStructProperty* StructProp = CastField<FStructProperty>(UnmangedPropToUpdate))
		{
			return ReKeyTextProperty(StructProp->Struct, StructProp->ContainerPtrToValuePtr<void>(AddrToUpdate), InConflictingSourceParts, InPartIndex + 1);
		}

		// Is this a text property?
		TextPropToUpdate = CastField<FTextProperty>(UnmangedPropToUpdate);
	}
	else
	{
		// If we didn't find the property, it may have a mangled name... try and unmangle it
		FString PropertyName;
		EMangledPropertyContainerType PropertyContainerType;
		int32 ContainerIndex;
		if (UnmanglePropertyName(PathPart, PropertyName, PropertyContainerType, ContainerIndex))
		{
			if (FProperty* MangledPropToUpdate = InOuterType->FindPropertyByName(*PropertyName))
			{
				switch (PropertyContainerType)
				{
				case EMangledPropertyContainerType::Fixed:
					if (ContainerIndex < MangledPropToUpdate->ArrayDim)
					{
						AddrToUpdate = MangledPropToUpdate->ContainerPtrToValuePtr<void>(AddrToUpdate, ContainerIndex);

						// Is this a complex property? If so, we need to recurse into it
						if (FStructProperty* StructProp = CastField<FStructProperty>(MangledPropToUpdate))
						{
							return ReKeyTextProperty(StructProp->Struct, AddrToUpdate, InConflictingSourceParts, InPartIndex + 1);
						}

						// Is this a text property?
						TextPropToUpdate = CastField<FTextProperty>(MangledPropToUpdate);
					}
					break;

				case EMangledPropertyContainerType::Dynamic:
					if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(MangledPropToUpdate))
					{
						AddrToUpdate = MangledPropToUpdate->ContainerPtrToValuePtr<void>(AddrToUpdate);

						FScriptArrayHelper ScriptArrayHelper(ArrayProp, AddrToUpdate);
						if (ContainerIndex < ScriptArrayHelper.Num())
						{
							AddrToUpdate = ScriptArrayHelper.GetRawPtr(ContainerIndex);

							// Is this a complex property? If so, we need to recurse into it
							if (FStructProperty* StructProp = CastField<FStructProperty>(ArrayProp->Inner))
							{
								return ReKeyTextProperty(StructProp->Struct, AddrToUpdate, InConflictingSourceParts, InPartIndex + 2); // +2 because dynamic container properties double up their name in the path
							}

							// Is this a text property?
							TextPropToUpdate = CastField<FTextProperty>(ArrayProp->Inner);
						}
					}
					else if (FMapProperty* MapProp = CastField<FMapProperty>(MangledPropToUpdate))
					{
						AddrToUpdate = MangledPropToUpdate->ContainerPtrToValuePtr<void>(AddrToUpdate);

						FScriptMapHelper ScriptMapHelper(MapProp, AddrToUpdate);

						// ContainerIndex is the element index, but we need the sparse index which is computed by the iterator.
						const int32 InternalIndex = ScriptMapHelper.FindInternalIndex(ContainerIndex);
						if (InternalIndex != INDEX_NONE)
						{
							AddrToUpdate = ScriptMapHelper.GetPairPtr(InternalIndex) + MapProp->MapLayout.ValueOffset;

							// Is this a complex property? If so, we need to recurse into it
							if (FStructProperty* StructProp = CastField<FStructProperty>(MapProp->ValueProp))
							{
								return ReKeyTextProperty(StructProp->Struct, AddrToUpdate, InConflictingSourceParts, InPartIndex + 2); // +2 because dynamic container properties double up their name in the path
							}

							// Is this a text property?
							TextPropToUpdate = CastField<FTextProperty>(MapProp->ValueProp);
						}
					}
					else if (FSetProperty* SetProp = CastField<FSetProperty>(MangledPropToUpdate))
					{
						AddrToUpdate = MangledPropToUpdate->ContainerPtrToValuePtr<void>(AddrToUpdate);

						FScriptSetHelper ScriptSetHelper(SetProp, AddrToUpdate);

						// ContainerIndex is the element index, but we need the sparse index which is computed by the iterator.
						const int32 InternalIndex = ScriptSetHelper.FindInternalIndex(ContainerIndex);
						if (InternalIndex != INDEX_NONE)
						{
							AddrToUpdate = ScriptSetHelper.GetElementPtr(InternalIndex);

							// Is this a complex property? If so, we need to recurse into it
							if (FStructProperty* StructProp = CastField<FStructProperty>(SetProp->ElementProp))
							{
								return ReKeyTextProperty(StructProp->Struct, AddrToUpdate, InConflictingSourceParts, InPartIndex + 2); // +2 because dynamic container properties double up their name in the path
							}

							// Is this a text property?
							TextPropToUpdate = CastField<FTextProperty>(SetProp->ElementProp);
						}
					}
					break;

				case EMangledPropertyContainerType::DynamicKey:
					if (FMapProperty* MapProp = CastField<FMapProperty>(MangledPropToUpdate))
					{
						AddrToUpdate = MangledPropToUpdate->ContainerPtrToValuePtr<void>(AddrToUpdate);

						FScriptMapHelper ScriptMapHelper(MapProp, AddrToUpdate);

						// ContainerIndex is the element index, but we need the sparse index to get the element.
						const int32 InternalIndex = ScriptMapHelper.FindInternalIndex(ContainerIndex);
						if (InternalIndex != INDEX_NONE)
						{
							AddrToUpdate = ScriptMapHelper.GetPairPtr(InternalIndex);

							// Is this a complex property? If so, we need to recurse into it
							if (FStructProperty* StructProp = CastField<FStructProperty>(MapProp->KeyProp))
							{
								return ReKeyTextProperty(StructProp->Struct, AddrToUpdate, InConflictingSourceParts, InPartIndex + 2); // +2 because dynamic container properties double up their name in the path
							}

							// Is this a text property?
							TextPropToUpdate = CastField<FTextProperty>(MapProp->KeyProp);
						}
					}
					break;

				default:
					break;
				}
			}
		}
	}

	if (TextPropToUpdate)
	{
		FText& TextValue = *TextPropToUpdate->GetPropertyValuePtr_InContainer(AddrToUpdate);

		const FString TextNamespace = FTextInspector::GetNamespace(TextValue).Get(FString());
		const FString TextKey = FGuid::NewGuid().ToString();
		TextValue = FText::ChangeKey(TextNamespace, TextKey, TextValue);

		return true;
	}

	return false;
}

int32 UFixConflictingLocalizationKeysCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, Parameters);

	TSharedPtr<FLocalizationSCC> SourceControlInfo;
	const bool bEnableSourceControl = Switches.Contains(TEXT("EnableSCC"));
	if (bEnableSourceControl)
	{
		SourceControlInfo = MakeShared<FLocalizationSCC>();

		FText SCCErrorStr;
		if (!SourceControlInfo->IsReady(SCCErrorStr))
		{
			UE_LOGF(LogFixConflictingLocalizationKeys, Error, "Revision Control error: %ls", *SCCErrorStr.ToString());
			return -1;
		}
	}

	const FString LocTargetName = TEXT("Game");
	const FString LocTargetPath = FPaths::ProjectContentDir() / TEXT("Localization") / LocTargetName;

	FLocTextHelper LocTextHelper(LocTargetPath, FString::Printf(TEXT("%s.manifest"), *LocTargetName), FString::Printf(TEXT("%s.archive"), *LocTargetName), TEXT("en"), TArray<FString>(), MakeShared<FLocFileSCCNotifies>(SourceControlInfo));

	// We need the manifest to work with
	{
		FText LoadManifestError;
		if (!LocTextHelper.LoadManifest(ELocTextHelperLoadFlags::Load, &LoadManifestError))
		{
			UE_LOGF(LogFixConflictingLocalizationKeys, Error, "Failed to load manifest: %ls", *LoadManifestError.ToString());
			return -1;
		}
	}

	// Build up a list of conflicting texts from the manifest (mimicking the 4.15 collapsing behavior)
	TArray<FString> ConflictingSources;
	{
		TMap<FLocKey, FLocItem> NsKeyToSourceString;

		LocTextHelper.EnumerateSourceTexts([&NsKeyToSourceString, &ConflictingSources](TSharedRef<FManifestEntry> InManifestEntry) -> bool
		{
			for (const FManifestContext& Context : InManifestEntry->Contexts)
			{
				const FLocKey NsKey = FString::Printf(TEXT("%s:%s"), *TextNamespaceUtil::StripPackageNamespace(InManifestEntry->Namespace.GetString()), *Context.Key.GetString());

				const FLocItem* ExistingSourceItem = NsKeyToSourceString.Find(NsKey);
				if (ExistingSourceItem)
				{
					if (!InManifestEntry->Source.IsExactMatch(*ExistingSourceItem))
					{
						ConflictingSources.Add(Context.SourceLocation);
					}
				}
				else
				{
					NsKeyToSourceString.Add(NsKey, InManifestEntry->Source);
				}
			}

			return true; // continue enumeration
		}, true);
	}

	UE_LOGF(LogFixConflictingLocalizationKeys, Display, "Found %d conflicting text sources...", ConflictingSources.Num());

	// Batch the conflicts by package
	TMap<FString, TArray<FString>> PackageNameToConflictingSources;
	for (const FString& ConflictingSource : ConflictingSources)
	{
		// Split the path into its component parts
		TArray<FString> ConflictingSourceParts;
		ConflictingSource.ParseIntoArray(ConflictingSourceParts, TEXT("."));

		// We always get at least 3 parts; the package, the root object, and the property name
		if (ConflictingSourceParts.Num() < 3)
		{
			UE_LOGF(LogFixConflictingLocalizationKeys, Warning, "Skipping '%ls' as it doesn't look like a valid package path", *ConflictingSource);
			continue;
		}

		// Did we get a valid package name?
		if (!FPackageName::IsValidLongPackageName(ConflictingSourceParts[0]))
		{
			UE_LOGF(LogFixConflictingLocalizationKeys, Warning, "Skipping '%ls' as it isn't a valid package name", *ConflictingSourceParts[0]);
			continue;
		}

		// Find or add this conflict to a batch
		TArray<FString>& PackageTextConflicts = PackageNameToConflictingSources.FindOrAdd(ConflictingSourceParts[0]);
		PackageTextConflicts.Add(ConflictingSource);
	}

	UE_LOGF(LogFixConflictingLocalizationKeys, Display, "Found %d packages to update...", PackageNameToConflictingSources.Num());

	// Re-key any conflicts
	for (const auto& PackageNameToConflictingSourcesPair : PackageNameToConflictingSources)
	{
		const FString& PackageName = PackageNameToConflictingSourcesPair.Key;
		UE_LOGF(LogFixConflictingLocalizationKeys, Display, "Loading package: %ls", *PackageName);

		// Load the package
		UPackage* Package = LoadPackage(nullptr, *PackageName, LOAD_NoWarn | LOAD_Quiet);
		if (!Package)
		{
			UE_LOGF(LogFixConflictingLocalizationKeys, Error, "Failed to load package from: %ls", *PackageName);
			continue;
		}

		for (const FString& ConflictingSource : PackageNameToConflictingSourcesPair.Value)
		{
			// Split the path into its component parts
			TArray<FString> ConflictingSourceParts;
			ConflictingSource.ParseIntoArray(ConflictingSourceParts, TEXT("."));

			if (ReKeyTextProperty(Package, ConflictingSourceParts, 1))
			{
				UE_LOGF(LogFixConflictingLocalizationKeys, Display, "    Automatically updated the text for: %ls", *ConflictingSource);
			}
			else
			{
				UE_LOGF(LogFixConflictingLocalizationKeys, Error, "    Failed to automatically update the text for: %ls", *ConflictingSource);
			}
		}

		// Re-save the package
		FLocalizedAssetSCCUtil::SavePackageWithSCC(SourceControlInfo, Package);
	}

	return 0;
}
