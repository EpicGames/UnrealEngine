// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/FixSoundWaveTimecodeInfoCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "Sound/SoundWave.h"

DEFINE_LOG_CATEGORY_STATIC(LogFixSoundWaveTimecodeInfoCommandlet, Log, All);

UFixSoundWaveTimecodeInfoCommandlet::UFixSoundWaveTimecodeInfoCommandlet()
{
	LogToConsole = true;
	IsClient = false;
	IsEditor = true;
	IsServer = false;
}

int32 UFixSoundWaveTimecodeInfoCommandlet::Main(const FString& Params)
{
	UE_LOGF(LogFixSoundWaveTimecodeInfoCommandlet, Display, "Fixing soundwave timecodeInfo. Note: Params are ignored.");

	// Return 0 on success
	int32 ReturnCode = 0;

#if WITH_EDITORONLY_DATA
	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Update Registry Module
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> SoundWaveAssetDatas;
	AssetRegistry.GetAssetsByClass(USoundWave::StaticClass()->GetClassPathName(), SoundWaveAssetDatas);

	for (const FAssetData& SoundWaveAssetData : SoundWaveAssetDatas)
	{
		if (TObjectPtr<USoundWave> SoundWave = Cast<USoundWave>(SoundWaveAssetData.GetAsset()))
		{
			if (SoundWave->FixTimeCodeInfo())
			{
				check(SoundWave->GetTimecodeInfo().IsSet());
				UE_LOGF(LogFixSoundWaveTimecodeInfoCommandlet, Display, "Fixed: %ls's TimecodeInfo. TimecodeInfo::Description: %ls", 
					*SoundWave->GetName(), *SoundWave->GetTimecodeInfo()->Description);
				
				if (UPackage* SoundWavePackage = SoundWave->GetPackage())
				{
					TArray<UPackage*> PackagesToSave;
					PackagesToSave.Add(SoundWavePackage);

					const FString PackageFileName = FPackageName::LongPackageNameToFilename(SoundWavePackage->GetName(), FPackageName::GetAssetPackageExtension());
					const bool bSuccess = UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, false);

					if (bSuccess)
					{
						UE_LOGF(LogFixSoundWaveTimecodeInfoCommandlet, Display, "Fixed SoundWave: %ls saved.", *SoundWave->GetName());
					}
					else
					{
						UE_LOGF(LogFixSoundWaveTimecodeInfoCommandlet, Warning, "Fixed SoundWave: %ls failed to save.", *SoundWave->GetName());

						ReturnCode = 1;
					}
				}
			}
			else
			{
				UE_LOGF(LogFixSoundWaveTimecodeInfoCommandlet, Verbose, "%ls's TimecodeInfo is valid and no fix was needed.", *SoundWave->GetName());
			}
		}
	}
#endif //WITH_EDITORONLY_DATA
	return ReturnCode;
}