// Copyright Epic Games, Inc. All Rights Reserved.

/*==================================================================================================
	FixupNeedsLoadForEditorGameCommandlet.cpp: Fixes outdated NeedsLoadForEditorGame flags on exports
====================================================================================================*/

#include "Commandlets/FixupNeedsLoadForEditorGameCommandlet.h"

#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Object.h"
#include "UObject/ObjectResource.h"
#include "UObject/TopLevelAssetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FixupNeedsLoadForEditorGameCommandlet)

int32 UFixupNeedsLoadForEditorGameCommandlet::InitializeResaveParameters(const TArray<FString>& Tokens, TArray<FString>& MapPathNames)
{
	int32 Result = Super::InitializeResaveParameters(Tokens, MapPathNames);
	// We need ResaveClasses to be specified, otherwise we won't know what to update
	if (Result == 0 && !ResaveClasses.Num())
	{
		UE_LOGF(LogContentCommandlet, Error, "FixupNeedsLoadForEditorGame commandlet requires at least one resave class name. Use -RESAVECLASS=/Path/To.ClassA,/Path/To.ClassB,/Path/To.ClassC to specify resave classes.");
		Result = 1;
	}
	else
	{
		for (FString& ClassName : ResaveClasses)
		{			
			if (!ResaveClassNeedsLoadForEditorGameValues.Contains(ClassName))
			{
				UClass* ResaveClass = UClass::TryFindTypeSlow<UClass>(ClassName);
				if (ResaveClass)
				{
					UObject* DefaultObject = ResaveClass->GetDefaultObject();
					ResaveClassNeedsLoadForEditorGameValues.Add(ClassName, DefaultObject->NeedsLoadForEditorGame());
				}
			}
			else if (Verbosity != UResavePackagesCommandlet::ONLY_ERRORS)
			{
				UE_LOGF(LogContentCommandlet, Warning, "Resave Class \"%ls\" could not be found. Make sure the class name is valid and that it's a native class.", *ClassName);
			}
		}
		if (ResaveClassNeedsLoadForEditorGameValues.Num() == 0)
		{
			UE_LOGF(LogContentCommandlet, Error, "Got %d classes to resave but none of the exist.", ResaveClasses.Num());
			Result = 1;
		}
	}
	return Result;
}

void UFixupNeedsLoadForEditorGameCommandlet::PerformPreloadOperations(FLinkerLoad* PackageLinker, bool& bSavePackage)
{
	Super::PerformPreloadOperations(PackageLinker, bSavePackage);
	if (bSavePackage)
	{
		// The package contains a class we want to check but we don't want to save it unless
		// any of the exports has an outdated (compared to a CDO) NeedsLoadForEditorGame flag.
		bSavePackage = false;
		for (int32 ExportIndex = 0; ExportIndex < PackageLinker->ExportMap.Num(); ExportIndex++)
		{
			FTopLevelAssetPath ExportClassPathName(PackageLinker->GetExportClassPackage(ExportIndex), PackageLinker->GetExportClassName(ExportIndex));
			bool* bNeedsLoadForEditorGameValuePtr = ResaveClassNeedsLoadForEditorGameValues.Find(ExportClassPathName.ToString());
			if (bNeedsLoadForEditorGameValuePtr)
			{
				FObjectExport& Export = PackageLinker->ExportMap[ExportIndex];
				// The condition below may seem confusing because bNotAlwaysLoadedForEditorGame = !bNeedsLoadForEditorGame
				// At the moment we only update assets that had bNotAlwaysLoadedForEditorGame set to true but NeedsLoadForEditorGame returns true 
				// and either bNotForClient or bNotForServer is true. 
				if (Export.bNotAlwaysLoadedForEditorGame &&
					  Export.bNotAlwaysLoadedForEditorGame == *bNeedsLoadForEditorGameValuePtr &&
						(Export.bNotForClient || Export.bNotForServer))
				{
					bSavePackage = true;
					break;
				}				
			}
		}
	}	
}
