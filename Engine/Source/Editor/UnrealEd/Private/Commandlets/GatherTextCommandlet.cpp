// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextCommandlet.h"

#include "Async/ParallelFor.h"
#include "Commandlets/GatherTextFromAssetsCommandlet.h"
#include "Commandlets/GatherTextFromSourceCommandlet.h"
#include "GeneralProjectSettings.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "SourceControlHelpers.h"
#include "UObject/Class.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GatherTextCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextCommandlet, Log, All);
namespace GatherTextCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;

	static bool bForceLegacyScheduler = false;
	static FAutoConsoleVariableRef CVarForceLegacyScheduler(TEXT("Localization.ForceGatherTextLegacyScheduler"), bForceLegacyScheduler, TEXT("True to force the GatherText commandlet to use the legacy scheduler for its tasks, or false to use the new scheduler"));

	static bool bAllowParallelTasks = true;
	static FAutoConsoleVariableRef CVarAllowParallelTasks(TEXT("Localization.AllowParallelGatherTextTasks"), bAllowParallelTasks, TEXT("True to allow the GatherText commandlet to run tasks in parallel, or false to force everything to run in sequence"));
}

/**
 *	UGatherTextCommandlet
 */
UGatherTextCommandlet::UGatherTextCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FString UGatherTextCommandlet::UsageText
	(
	TEXT("GatherTextCommandlet usage...\r\n")
	TEXT("    <GameName> GatherTextCommandlet -Config=<path to config ini file> [-Preview -EnableSCC -DisableSCCSubmit -GatherType=<All | Source | Asset | Metadata>]\r\n")
	TEXT("    \r\n")
	TEXT("    <path to config ini file> Full path to the .ini config file that defines what gather steps the commandlet will run.\r\n")
	TEXT("    Preview\t Runs the commandlet and its child commandlets in preview. Some commandlets will not be executed in preview mode. Use this to dump all generated warnings without writing any files. Using this switch implies -DisableSCCSubmit\r\n")
	TEXT("    EnableSCC\t Enables revision control and allows the commandlet to check out files for editing.\r\n")
	TEXT("    DisableSCCSubmit\t Prevents the commandlet from submitting checked out files in revision control that have been edited.\r\n")
	TEXT("    GatherType\t Only performs a gather on the specified type of file (currently only works in preview mode). Source only runs commandlets that gather source files. Asset only runs commandlets that gather asset files. All runs commandlets that gather both source and asset files. Leaving this param out implies a gather type of All.")
	TEXT("Metadata only runs commandlets that gather metadata files. All runs commandlets that gather both source and asset files. Leaving this param out implies a gather type of All.\r\n")
	);


int32 UGatherTextCommandlet::Main( const FString& Params )
{
	return Execute(Params, nullptr);
}

int32 UGatherTextCommandlet::Execute(const FString& Params, const TSharedPtr<const FGatherTextCommandletEmbeddedContext>& InEmbeddedContext)
{
	EmbeddedContext = InEmbeddedContext;

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);
	if (Switches.Contains(TEXT("help")) || Switches.Contains(TEXT("Help")))
	{
		UE_LOGF(LogGatherTextCommandlet, Display, "%ls", *UsageText);
		return 0;
	}

	// Build up the complete list of config files to process
	TArray<FString> GatherTextConfigPaths;
	if (const FString* ConfigParamPtr = ParamVals.Find(UGatherTextCommandletBase::ConfigParam))
	{
		ConfigParamPtr->ParseIntoArray(GatherTextConfigPaths, TEXT(";"));
	}

	if (const FString* ConfigListFileParamPtr = ParamVals.Find(TEXT("ConfigList")))
	{
		if (FPaths::FileExists(*ConfigListFileParamPtr))
		{
			TArray<FString> ConfigFiles;
			FFileHelper::LoadFileToStringArray(ConfigFiles, **ConfigListFileParamPtr);
			if (ConfigFiles.Num() > 0)
			{
				GatherTextConfigPaths.Append(MoveTemp(ConfigFiles));
			}
			else
			{
				UE_LOGFMT(LogGatherTextCommandlet, Warning, "There are no config file paths in specified config ,list '{configList}'. Please check to see the is correctly populated.",
					("configList", *ConfigListFileParamPtr),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
			}
		}
		else
		{
			UE_LOGFMT(LogGatherTextCommandlet, Warning, "Specified config list file '{configList}' does not exist. No additional config files from -ConfigList can be added.",
				("configList", *ConfigListFileParamPtr),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	// @TODOLocalization: Handle the case where -config and -ConfigList both specify the same files.
	// Currently that would just mean that the config files will be launched with the relevant commandlets multiple times. The results should be correct, but it's wasted work.
	
	// Turn all relative paths into absolute paths 
	const FString& ProjectBasePath = UGatherTextCommandletBase::GetProjectBasePath();
	for (FString& GatherTextConfigPath : GatherTextConfigPaths)
	{
		if (FPaths::IsRelative(GatherTextConfigPath))
		{
			GatherTextConfigPath = FPaths::Combine(*ProjectBasePath, *GatherTextConfigPath);
		}
		GatherTextConfigPath = FConfigCacheIni::NormalizeConfigIniPath(GatherTextConfigPath);
	}

	if (GatherTextConfigPaths.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextCommandlet, Error, "-config or -ConfigList not specified. If -ConfigList was specified, please check that the file path is valid.\n{usageText}",
			("usageText", UsageText),
			("id", GatherTextCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	bLegacyScheduler = GatherTextCommandlet::bForceLegacyScheduler || Switches.Contains(TEXT("LegacyScheduler"));
	UE_LOGF(LogGatherTextCommandlet, Display, "LegacyScheduler: %ls", bLegacyScheduler ? TEXT("Yes") : TEXT("No"));

	bRunningInPreview = Switches.Contains(UGatherTextCommandletBase::PreviewSwitch);
	if (bRunningInPreview)
	{
		UE_LOGF(LogGatherTextCommandlet, Display, "Running commandlet in preview mode. Some child commandlets and steps will be skipped.");
		// -GatherType is only valid in preview mode right now 
		if (const FString* GatherTypeParamPtr = ParamVals.Find(UGatherTextCommandletBase::GatherTypeParam))
		{
			UE_LOGF(LogGatherTextCommandlet, Display, "Running commandlet with gather type %ls. Only %ls files will be gathered.", **GatherTypeParamPtr, **GatherTypeParamPtr);
		}
		else
		{
			// if the -GatherType param is not specified, we default to gathering both source, asset and metadata
			UE_LOGF(LogGatherTextCommandlet, Display, "-GatherType param not specified. Commandlet will gather source, assset and metadata.");
		}
	}

	// If we are only doing a preview run, we will not be writing to any files that will need to be submitted to source control 
	const bool bEnableSourceControl = Switches.Contains(UGatherTextCommandletBase::EnableSourceControlSwitch) && !bRunningInPreview;
	const bool bDisableSubmit = Switches.Contains(UGatherTextCommandletBase::DisableSubmitSwitch) || bRunningInPreview;

	TSharedPtr<FLocalizationSCC> CommandletSourceControlInfo;
	if (bEnableSourceControl)
	{
		CommandletSourceControlInfo = MakeShareable(new FLocalizationSCC());

		FText SCCErrorStr;
		if (!CommandletSourceControlInfo->IsReady(SCCErrorStr))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "Revision Control error: {error}",
				("error", SCCErrorStr.ToString()),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	double AllCommandletExecutionStartTime = FPlatformTime::Seconds();

	ON_SCOPE_EXIT
	{
		for (const FString& GatherTextConfigPath : GatherTextConfigPaths)
		{
			if (GConfig->FindConfigFile(GatherTextConfigPath))
			{
				GConfig->UnloadFile(GatherTextConfigPath);
			}
		}
	};

	// Schedule the work
	{
		if (!ScheduleGatherConfigs(GatherTextConfigPaths, CommandletSourceControlInfo, Tokens, Switches, ParamVals))
		{
			return -1;
		}

		if (Switches.Contains(TEXT("ScheduleOnly")))
		{
			// Only wanted to see how the tasks would be scheduled, but not actually run the work
			return 0;
		}
	}

	// Execute the work
	{
		ON_SCOPE_EXIT
		{
			FGatherTextFromAssetsWorkerDirector::Get().StopWorkers();

			UGatherTextFromSourceCommandlet::LogStats();
		};

		if (!ExecuteGatherTextPhases(CommandletSourceControlInfo))
		{
			if (CommandletSourceControlInfo.IsValid() && !bDisableSubmit)
			{
				FText SCCErrorStr;
				if (!CommandletSourceControlInfo->CleanUp(SCCErrorStr))
				{
					UE_LOGFMT(LogGatherTextCommandlet, Error, "Revision Control error: {error}",
						("error", SCCErrorStr.ToString()),
						("id", GatherTextCommandlet::LocalizationLogIdentifier)
					);
				}
			}
			return -1;
		}
	}

	CleanupStalePlatformData();

	if (CommandletSourceControlInfo.IsValid() && !bDisableSubmit)
	{
		FText SCCErrorStr;
		if (CommandletSourceControlInfo->CheckinFiles(GetChangelistDescription(GatherTextConfigPaths), SCCErrorStr))
		{
			UE_LOGF(LogGatherTextCommandlet, Log, "Submitted Localization files.");
		}
		else
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "Revision control error: {error}",
				("error", SCCErrorStr.ToString()),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			if (!CommandletSourceControlInfo->CleanUp(SCCErrorStr))
			{
				UE_LOGFMT(LogGatherTextCommandlet, Error, "Revision control error post-cleanup: {error}",
					("error", SCCErrorStr.ToString()),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
			}
			return -1;
		}
	}
	UE_LOGF(LogGatherTextCommandlet, Display, "Completed all steps in %.2f seconds", FPlatformTime::Seconds() - AllCommandletExecutionStartTime);

	// Note: Other things use the below log as a tracker for GatherText completing successfully - DO NOT remove or edit this line without also updating those places
	UE_LOGF(LogGatherTextCommandlet, Display, "GatherText completed with exit code 0");
	return 0;
}

bool UGatherTextCommandlet::ScheduleGatherConfigs(const TArray<FString>& GatherTextConfigPaths, const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo, const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals)
{
	FScopedSlowTask SlowTask(GatherTextConfigPaths.Num(), (EmbeddedContext && EmbeddedContext->SlowTaskMessageOverride) ? *EmbeddedContext->SlowTaskMessageOverride : NSLOCTEXT("GatherTextCommandlet", "GatherTextTask.SchedulingTasks", "Scheduling Localization Tasks for Targets..."));

	// Schedule the set of localization tasks we will run, as that needs to happen on the game-thread
	for (const FString& GatherTextConfigPath : GatherTextConfigPaths)
	{
		SlowTask.EnterProgressFrame();

		if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "GatherText aborted!",
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		if (!GConfig->Find(GatherTextConfigPath))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "Loading Config File '{configFile}' failed.",
				("configFile", GatherTextConfigPath),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		if (!ScheduleGatherConfig(GatherTextConfigPath, CommandletSourceControlInfo, Tokens, Switches, ParamVals))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "Scheduling tasks from '{configFile}' failed.",
				("configFile", GatherTextConfigPath),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}
	}

	// If any phases only have a single parallel task, then just add that to the end of the sequential tasks as the parallelization will just add overhead
	for (int32 PhaseIndex = (int32)EGatherTextCommandletPhase::FirstPhase; PhaseIndex < (int32)EGatherTextCommandletPhase::NumPhases; ++PhaseIndex)
	{
		if (FGatherTextCommandletPhase& Phase = GatherTextPhases[PhaseIndex];
			Phase.ParallelTasks.Num() == 1)
		{
			UE_LOGF(LogGatherTextCommandlet, Display, "Phase '%ls' has a single parallel task. Adding it to the end of its sequential tasks.", LexToString((EGatherTextCommandletPhase)PhaseIndex));
			Phase.SequentialTasks.Add(MoveTemp(Phase.ParallelTasks[0]));
			Phase.ParallelTasks.Reset();
		}
	}

	// Log the full set of work that we scheduled for each phase
	for (int32 PhaseIndex = (int32)EGatherTextCommandletPhase::FirstPhase; PhaseIndex < (int32)EGatherTextCommandletPhase::NumPhases; ++PhaseIndex)
	{
		auto BuildTaskStatsReport =
			[](const TArray<FGatherTextCommandletTask>& TasksArray)
			{
				TMap<FName, int32> NumTasksByClass;
				for (const FGatherTextCommandletTask& Task : TasksArray)
				{
					++NumTasksByClass.FindOrAdd(Task.Commandlet->GetClass()->GetFName());
				}

				FString TaskStatsReport;
				{
					bool bAddComma = false;
					for (const TTuple<FName, int32>& NumTasksForClass : NumTasksByClass)
					{
						if (bAddComma)
						{
							TaskStatsReport += TEXT(", ");
						}
						TaskStatsReport.AppendInt(NumTasksForClass.Value);
						TaskStatsReport += TEXT(' ');
						NumTasksForClass.Key.AppendString(TaskStatsReport);
						bAddComma = true;
					}
				}
				return TaskStatsReport;
			};

		const FGatherTextCommandletPhase& Phase = GatherTextPhases[PhaseIndex];
		UE_LOGF(LogGatherTextCommandlet, Display, "Phase '%ls':\n\t%d sequential task(s): %ls\n\t%d parallel tasks(s): %ls",
			LexToString((EGatherTextCommandletPhase)PhaseIndex),
			Phase.SequentialTasks.Num(), *BuildTaskStatsReport(Phase.SequentialTasks),
			Phase.ParallelTasks.Num(), *BuildTaskStatsReport(Phase.ParallelTasks)
			);
	}

	return true;
}

bool UGatherTextCommandlet::ScheduleGatherConfig(const FString& GatherTextConfigPath, const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo, const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals)
{
	UE_LOGF(LogGatherTextCommandlet, Display, "Scheduling GatherText tasks for '%ls'", *GatherTextConfigPath);

	// Read in the platform split mode to use
	ELocTextPlatformSplitMode PlatformSplitMode = ELocTextPlatformSplitMode::None;
	{
		FString PlatformSplitModeString;
		if (GetStringFromConfig(TEXT("CommonSettings"), TEXT("PlatformSplitMode"), PlatformSplitModeString, GatherTextConfigPath))
		{
			UEnum* PlatformSplitModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocTextPlatformSplitMode"));
			const int64 PlatformSplitModeInt = PlatformSplitModeEnum->GetValueByName(*PlatformSplitModeString);
			if (PlatformSplitModeInt != INDEX_NONE)
			{
				PlatformSplitMode = (ELocTextPlatformSplitMode)PlatformSplitModeInt;
			}
		}
	}

	FString LocalizationTargetName;
	{
		FString ManifestName;
		GetStringFromConfig(TEXT("CommonSettings"), TEXT("ManifestName"), ManifestName, GatherTextConfigPath);
		LocalizationTargetName = FPaths::GetBaseFilename(ManifestName);
	}

	FString CopyrightNotice;
	if (!GetStringFromConfig(TEXT("CommonSettings"), TEXT("CopyrightNotice"), CopyrightNotice, GatherTextConfigPath))
	{
		CopyrightNotice = GetDefault<UGeneralProjectSettings>()->CopyrightNotice;
	}

	// Basic helper that can be used only to gather a new manifest for writing
	TSharedRef<FLocTextHelper> CommandletGatherManifestHelper = MakeShared<FLocTextHelper>(LocalizationTargetName, MakeShared<FLocFileSCCNotifies>(CommandletSourceControlInfo), PlatformSplitMode);
	CommandletGatherManifestHelper->SetCopyrightNotice(CopyrightNotice);
	CommandletGatherManifestHelper->LoadManifest(ELocTextHelperLoadFlags::Create);

	const FString GatherTextStepPrefix = TEXT("GatherTextStep");

	// Read the list of steps from the config file (they all have the format GatherTextStep{N})
	TArray<FString> StepNames;
	GConfig->GetSectionNames(GatherTextConfigPath, StepNames);
	StepNames.RemoveAllSwap([&GatherTextStepPrefix](const FString& InStepName)
	{
		return !InStepName.StartsWith(GatherTextStepPrefix);
	});

	// Make sure the steps are sorted in ascending order (by numerical suffix)
	StepNames.Sort([&GatherTextStepPrefix](const FString& InStepNameOne, const FString& InStepNameTwo)
	{
		const FString NumericalSuffixOneStr = InStepNameOne.RightChop(GatherTextStepPrefix.Len());
		const int32 NumericalSuffixOne = FCString::Atoi(*NumericalSuffixOneStr);

		const FString NumericalSuffixTwoStr = InStepNameTwo.RightChop(GatherTextStepPrefix.Len());
		const int32 NumericalSuffixTwo = FCString::Atoi(*NumericalSuffixTwoStr);

		return NumericalSuffixOne < NumericalSuffixTwo;
	});
	// Generate the switches and params to be passed on to child commandlets
	FString GeneratedParamsAndSwitches;
	// Add all the command params with the exception of config
	for (auto ParamIter = ParamVals.CreateConstIterator(); ParamIter; ++ParamIter)
	{
		const FString& Key = ParamIter.Key();
		const FString& Val = ParamIter.Value();
		if (Key != UGatherTextCommandletBase::ConfigParam)
		{
			GeneratedParamsAndSwitches += FString::Printf(TEXT(" -%s=%s"), *Key, *Val);
		}
	}

	// Add all the command switches
	for (auto SwitchIter = Switches.CreateConstIterator(); SwitchIter; ++SwitchIter)
	{
		const FString& Switch = *SwitchIter;
		GeneratedParamsAndSwitches += FString::Printf(TEXT(" -%s"), *Switch);
	}

	// Schedule each step defined in the config file.
	EGatherTextCommandletPhase PreviousTaskPhase = EGatherTextCommandletPhase::FirstPhase;
	bool bPreviousTaskWasParallel = false;
	for (const FString& StepName : StepNames)
	{
		FString CommandletClassName = GConfig->GetStr( *StepName, TEXT("CommandletClass"), GatherTextConfigPath ) + TEXT("Commandlet");

		UClass* CommandletClass = FindFirstObject<UClass>(*CommandletClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("UGatherTextCommandlet::ProcessGatherConfig"));
		if (!CommandletClass)
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "The commandlet name {commandletName} in section {section} is invalid.",
				("commandletName", CommandletClassName),
				("section", StepName),
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			continue;
		}

		TStrongObjectPtr<UGatherTextCommandletBase> Commandlet(NewObject<UGatherTextCommandletBase>(GetTransientPackage(), CommandletClass));
		check(Commandlet);
		Commandlet->SetEmbeddedContext(EmbeddedContext);
		Commandlet->Initialize( CommandletGatherManifestHelper, CommandletSourceControlInfo );
		// As of now, all params and switches (with the exception of config) is passed on to child commandlets
		// Instead of parsing in child commandlets, we'll just pass the params and switches along to determine if we need to run the child commandlet 
		// If we are running in preview mode, then most commandlets should be skipped as ShouldRunInPreview() defaults to false in the base class.
		if (bRunningInPreview && !Commandlet->ShouldRunInPreview(Switches, ParamVals))
		{
			UE_LOGF(LogGatherTextCommandlet, Display, "Should not run %ls: %ls in preview. Skipping.", *StepName, *CommandletClassName);
			continue;
		}

		FString GeneratedCmdLine = FString::Printf(TEXT("-Config=\"%s\" -Section=%s"), *GatherTextConfigPath, *StepName);
		GeneratedCmdLine += GeneratedParamsAndSwitches;

		bool bParallelTask = false;
		EGatherTextCommandletPhase TaskPhase = PreviousTaskPhase;
		if (!bLegacyScheduler)
		{
			bParallelTask = Commandlet->CanParallelize();

			// Which phase should this commandlet run it?
			if (!Commandlet->ConfigurePhase(GeneratedCmdLine))
			{
				UE_LOGFMT(LogGatherTextCommandlet, Error, "The commandlet name {commandletName} in section {section} failed to configure a valid phase.",
					("commandletName", CommandletClassName),
					("section", StepName),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
				return false;
			}

			TaskPhase = Commandlet->GetPhase();
			if (TaskPhase == EGatherTextCommandletPhase::NumPhases)
			{
				TaskPhase = EGatherTextCommandletPhase::Undefined;
				UE_LOGFMT(LogGatherTextCommandlet, Error, "The commandlet name {commandletName} in section {section} returned an invalid phase. It will be set to Undefined.",
					("commandletName", CommandletClassName),
					("section", StepName),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
			}
			if (TaskPhase == EGatherTextCommandletPhase::Undefined)
			{
				TaskPhase = PreviousTaskPhase;
				if (bPreviousTaskWasParallel && !bParallelTask)
				{
					// If the previous task was parallel and this is sequential, then we actually want to place this in the next phase to 
					// ensure it runs after the previous task (as sequential tasks run before parallel tasks within the same phase)
					if (int32 PhaseIndex = (int32)TaskPhase;
						PhaseIndex < (int32)EGatherTextCommandletPhase::NumPhases - 1)
					{
						TaskPhase = (EGatherTextCommandletPhase)(PhaseIndex + 1);
					}
				}
				UE_LOGFMT(LogGatherTextCommandlet, Warning, "The commandlet name {commandletName} in section {section} is using an Undefined phase and has been placed in the {phase} phase. You should upgrade your commandlet to use a specific phase to ensure it gets sequenced correctly!",
					("commandletName", CommandletClassName),
					("section", StepName),
					("phase", LexToString(TaskPhase)),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
			}

			bPreviousTaskWasParallel = bParallelTask;
			PreviousTaskPhase = TaskPhase;
		}
		check(TaskPhase < EGatherTextCommandletPhase::NumPhases);

		FGatherTextCommandletPhase& Phase = GatherTextPhases[(int32)TaskPhase];
		TArray<FGatherTextCommandletTask>& TasksArray = bParallelTask ? Phase.ParallelTasks : Phase.SequentialTasks;

		FGatherTextCommandletTask& Task = TasksArray.AddDefaulted_GetRef();
		Task.CommandletParams = MoveTemp(GeneratedCmdLine);
		Task.Commandlet = Commandlet.Get();
	}

	// Schedule the clean-up of any stale per-platform data
	{
		FString DestinationPath;
		if (GetPathFromConfig(TEXT("CommonSettings"), TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath))
		{
			FString PlatformLocalizationPath = DestinationPath / FPaths::GetPlatformLocalizationFolderName();
			FSplitPlatformConfig& SplitPlatformConfig = SplitPlatformConfigs.FindOrAdd(MoveTemp(PlatformLocalizationPath));

			if (CommandletGatherManifestHelper->ShouldSplitPlatformData())
			{
				SplitPlatformConfig.bShouldSplitPlatformData = true;
				SplitPlatformConfig.PlatformsToSplit.Append(CommandletGatherManifestHelper->GetPlatformsToSplit());
			}
		}
		else
		{
			UE_LOGFMT(LogGatherTextCommandlet, Warning, "No destination path specified in the 'CommonSettings' section. Cannot check for stale per-platform data!",
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	return true;
}

bool UGatherTextCommandlet::ExecuteGatherTextPhases(const TSharedPtr<FLocalizationSCC>& CommandletSourceControlInfo)
{
	FScopedSlowTask SlowTask((int32)EGatherTextCommandletPhase::NumPhases, (EmbeddedContext && EmbeddedContext->SlowTaskMessageOverride) ? *EmbeddedContext->SlowTaskMessageOverride : NSLOCTEXT("GatherTextCommandlet", "GatherTextTask.RunningTasks", "Running Localization Tasks for Targets..."));

	auto RunTaskCommandlet =
		[](const FGatherTextCommandletTask& Task)
		{
			const double CommandletExecutionStartTime = FPlatformTime::Seconds();

			const FString CommandletClassName = Task.Commandlet->GetClass()->GetName();
			UE_LOGF(LogGatherTextCommandlet, Display, "Executing %ls (params: %ls)", *CommandletClassName, *Task.CommandletParams);

			if (Task.Commandlet->Main(Task.CommandletParams) != 0)
			{
				UE_LOGFMT(LogGatherTextCommandlet, Error, "{commandletName} reported an error (params: {params}).",
					("commandletName", CommandletClassName),
					("params", Task.CommandletParams),
					("id", GatherTextCommandlet::LocalizationLogIdentifier)
				);
				return false;
			}

			UE_LOGF(LogGatherTextCommandlet, Display, "Executing %ls took %.2f seconds (params: %ls)", *CommandletClassName, FPlatformTime::Seconds() - CommandletExecutionStartTime, *Task.CommandletParams);
			return true;
		};

	const bool bCanRunParallelTasks = GatherTextCommandlet::bAllowParallelTasks && (!EmbeddedContext || EmbeddedContext->IsThreadSafe());
	UE_LOGF(LogGatherTextCommandlet, Display, "CanRunParallelTasks: %ls", bCanRunParallelTasks ? TEXT("Yes") : TEXT("No"));

	for (int32 PhaseIndex = (int32)EGatherTextCommandletPhase::FirstPhase; PhaseIndex < (int32)EGatherTextCommandletPhase::NumPhases; ++PhaseIndex)
	{
		SlowTask.EnterProgressFrame();

		if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "GatherText aborted!",
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		const FGatherTextCommandletPhase& Phase = GatherTextPhases[PhaseIndex];
		if (Phase.SequentialTasks.IsEmpty() && Phase.ParallelTasks.IsEmpty())
		{
			continue;
		}

		const double PhaseExecutionStartTime = FPlatformTime::Seconds();
		UE_LOGF(LogGatherTextCommandlet, Display, "Running phase '%ls' (%d sequential tasks and %d parallel tasks).", LexToString((EGatherTextCommandletPhase)PhaseIndex), Phase.SequentialTasks.Num(), Phase.ParallelTasks.Num());

		// Run sequential tasks
		if (Phase.SequentialTasks.Num() > 0)
		{
			const double SequentialTasksExecutionStartTime = FPlatformTime::Seconds();
			UE_LOGF(LogGatherTextCommandlet, Display, "Running %d sequentials task(s) in phase '%ls'...", Phase.SequentialTasks.Num(), LexToString((EGatherTextCommandletPhase)PhaseIndex));

			int32 TaskProgress = 0;
			for (const FGatherTextCommandletTask& Task : Phase.SequentialTasks)
			{
				UE_LOGF(LogGatherTextCommandlet, Display, "Starting sequential task %d of %d in phase '%ls'...", ++TaskProgress, Phase.SequentialTasks.Num(), LexToString((EGatherTextCommandletPhase)PhaseIndex));

				if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
				{
					UE_LOGFMT(LogGatherTextCommandlet, Error, "GatherText aborted!",
						("id", GatherTextCommandlet::LocalizationLogIdentifier)
					);
					return false;
				}

				if (EmbeddedContext)
				{
					EmbeddedContext->RunTick();
				}

				if (!RunTaskCommandlet(Task))
				{
					return false;
				}
			}

			UE_LOGF(LogGatherTextCommandlet, Display, "%d sequentials task(s) in phase '%ls' took %.2f seconds.", Phase.SequentialTasks.Num(), LexToString((EGatherTextCommandletPhase)PhaseIndex), FPlatformTime::Seconds() - SequentialTasksExecutionStartTime);
		}
			
		if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "GatherText aborted!",
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		// Run parallel tasks
		if (Phase.ParallelTasks.Num() > 0)
		{
			const double ParallelTasksExecutionStartTime = FPlatformTime::Seconds();
			UE_LOGF(LogGatherTextCommandlet, Display, "Starting %d parallel task(s) in phase '%ls'...", Phase.ParallelTasks.Num(), LexToString((EGatherTextCommandletPhase)PhaseIndex));

			if (CommandletSourceControlInfo)
			{
				CommandletSourceControlInfo->BeginParallelTasks();
			}

			std::atomic<bool> bFailedParallelTasks = false;
			ParallelFor(TEXT("GatherTextCommandlet"), Phase.ParallelTasks.Num(), 1, [this, &Phase, &RunTaskCommandlet, &bFailedParallelTasks](int32 TaskIndex)
			{
				if (EmbeddedContext)
				{
					EmbeddedContext->RunTick();
				}

				const FGatherTextCommandletTask& Task = Phase.ParallelTasks[TaskIndex];
				if (!RunTaskCommandlet(Task))
				{
					bFailedParallelTasks = true;
				}
			},
			bCanRunParallelTasks ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

			if (CommandletSourceControlInfo)
			{
				FText SCCErrorStr;
				if (!CommandletSourceControlInfo->EndParallelTasks(SCCErrorStr))
				{
					UE_LOGFMT(LogGatherTextCommandlet, Error, "Revision Control error: {error}",
						("error", SCCErrorStr.ToString()),
						("id", GatherTextCommandlet::LocalizationLogIdentifier)
					);
				}
			}

			if (bFailedParallelTasks)
			{
				return false;
			}

			UE_LOGF(LogGatherTextCommandlet, Display, "%d parallel task(s) in phase '%ls' took %.2f seconds.", Phase.ParallelTasks.Num(), LexToString((EGatherTextCommandletPhase)PhaseIndex), FPlatformTime::Seconds() - ParallelTasksExecutionStartTime);
		}

		if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
		{
			UE_LOGFMT(LogGatherTextCommandlet, Error, "GatherText aborted!",
				("id", GatherTextCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		UE_LOGF(LogGatherTextCommandlet, Display, "Phase '%ls' took %.2f seconds (%d sequential tasks and %d parallel tasks).", LexToString((EGatherTextCommandletPhase)PhaseIndex), FPlatformTime::Seconds() - PhaseExecutionStartTime, Phase.SequentialTasks.Num(), Phase.ParallelTasks.Num());

		if ((EGatherTextCommandletPhase)PhaseIndex == EGatherTextCommandletPhase::PostUpdateManifests)
		{
			// No longer need the asset gather workers after the manifests have been updated
			FGatherTextFromAssetsWorkerDirector::Get().StopWorkers();
		}
	}

	return true;
}

void UGatherTextCommandlet::CleanupStalePlatformData()
{
	IFileManager& FileManager = IFileManager::Get();

	auto RemoveDirectory = [&FileManager](const TCHAR* InDirectory)
	{
		FileManager.IterateDirectoryRecursively(InDirectory, [&FileManager](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
		{
			if (!bIsDirectory)
			{
				if (!USourceControlHelpers::IsAvailable() || !USourceControlHelpers::MarkFileForDelete(FilenameOrDirectory))
				{
					FileManager.Delete(FilenameOrDirectory, false, true);
				}
			}
			return true;
		});
		FileManager.DeleteDirectory(InDirectory, false, true);
	};

	for (const TTuple<FString, FSplitPlatformConfig>& SplitPlatformConfig : SplitPlatformConfigs)
	{
		if (!FileManager.DirectoryExists(*SplitPlatformConfig.Key))
		{
			continue;
		}

		if (SplitPlatformConfig.Value.bShouldSplitPlatformData)
		{
			// Remove any stale platform sub-folders
			FileManager.IterateDirectory(*SplitPlatformConfig.Key, [&SplitPlatformConfig, &RemoveDirectory](const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
			{
				if (bIsDirectory)
				{
					const FString SplitPlatformName = FPaths::GetCleanFilename(FilenameOrDirectory);
					if (!SplitPlatformConfig.Value.PlatformsToSplit.Contains(SplitPlatformName))
					{
						RemoveDirectory(FilenameOrDirectory);
					}
				}
				return true;
			});
		}
		else
		{
			// Remove the entire Platforms folder
			RemoveDirectory(*SplitPlatformConfig.Key);
		}
	}
}

FText UGatherTextCommandlet::GetChangelistDescription(const TArray<FString>& GatherTextConfigPaths) const
{
	FString ProjectName = FApp::GetProjectName();
	if (ProjectName.IsEmpty())
	{
		ProjectName = TEXT("Engine");
	}

	FString ChangeDescriptionString = FString::Printf(TEXT("[Localization Update] %s\n\n"), *ProjectName);

	ChangeDescriptionString += TEXT("Targets:\n");
	for (const FString& GatherTextConfigPath : GatherTextConfigPaths)
	{
		const FString TargetName = FPaths::GetBaseFilename(GatherTextConfigPath, true);
		ChangeDescriptionString += FString::Printf(TEXT("  %s\n"), *TargetName);
	}

	return FText::FromString(MoveTemp(ChangeDescriptionString));
}
