// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextCommandletBase.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "EngineGlobals.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/ARFilter.h"
#include "PackageHelperFunctions.h"
#include "ObjectTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GatherTextCommandletBase)

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextCommandletBase, Log, All);

FGatherTextDelegates::FGetAdditionalGatherPaths FGatherTextDelegates::GetAdditionalGatherPaths;
FGatherTextDelegates::FGetAdditionalGatherPathsForContext FGatherTextDelegates::GetAdditionalGatherPathsForContext;

const TCHAR* LexToString(EGatherTextCommandletPhase Phase)
{
#define GATHERTEXTPHASETOSTRING(PhaseName) case EGatherTextCommandletPhase::PhaseName: return TEXT(#PhaseName)
	switch (Phase)
	{
		GATHERTEXTPHASETOSTRING(PreInitialize);
		GATHERTEXTPHASETOSTRING(Initialize);
		GATHERTEXTPHASETOSTRING(PostInitialize);
		GATHERTEXTPHASETOSTRING(PreUpdateManifests);
		GATHERTEXTPHASETOSTRING(UpdateManifests);
		GATHERTEXTPHASETOSTRING(PostUpdateManifests);
		GATHERTEXTPHASETOSTRING(PreUpdateArchives);
		GATHERTEXTPHASETOSTRING(UpdateArchives);
		GATHERTEXTPHASETOSTRING(PostUpdateArchives);
		GATHERTEXTPHASETOSTRING(PreImport);
		GATHERTEXTPHASETOSTRING(Import);
		GATHERTEXTPHASETOSTRING(PostImport);
		GATHERTEXTPHASETOSTRING(PreExport);
		GATHERTEXTPHASETOSTRING(Export);
		GATHERTEXTPHASETOSTRING(PostExport);
		GATHERTEXTPHASETOSTRING(PreCompile);
		GATHERTEXTPHASETOSTRING(Compile);
		GATHERTEXTPHASETOSTRING(PostCompile);
		GATHERTEXTPHASETOSTRING(PreFinalize);
		GATHERTEXTPHASETOSTRING(Finalize);
		GATHERTEXTPHASETOSTRING(PostFinalize);
		GATHERTEXTPHASETOSTRING(Undefined);
		default:
			break;
	}
#undef GATHERTEXTPHASETOSTRING

	checkf(false, TEXT("Unknown EGatherTextCommandletPhase!"));
	return TEXT("Unknown");
}

//////////////////////////////////////////////////////////////////////////
//UGatherTextCommandletBase

const TCHAR* UGatherTextCommandletBase::ConfigParam = TEXT("Config");
const TCHAR* UGatherTextCommandletBase::EnableSourceControlSwitch = TEXT("EnableSCC");
const TCHAR* UGatherTextCommandletBase::DisableSubmitSwitch = TEXT("DisableSCCSubmit");
const TCHAR* UGatherTextCommandletBase::PreviewSwitch = TEXT("Preview");
const TCHAR* UGatherTextCommandletBase::GatherTypeParam = TEXT("GatherType");
const TCHAR* UGatherTextCommandletBase::SkipNestedMacroPrepassSwitch = TEXT("SkipNestedMacroPrepass");

UGatherTextCommandletBase::UGatherTextCommandletBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ShowErrorCount = false;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Wrapper to call the older GetAdditionalGatherPaths callback
		FGatherTextDelegates::GetAdditionalGatherPathsForContext.AddStatic([](const FString& InLocalizationTargetName, const FGatherTextContext& InContext, TArray<FString>& InOutIncludePathFilters, TArray<FString>& InOutExcludePathFilters)
		{
			FGatherTextDelegates::GetAdditionalGatherPaths.Broadcast(InLocalizationTargetName, InOutIncludePathFilters, InOutExcludePathFilters);
		});
	}
}

void UGatherTextCommandletBase::SetEmbeddedContext(const TSharedPtr<const FGatherTextCommandletEmbeddedContext>& InEmbeddedContext)
{
	EmbeddedContext = InEmbeddedContext;
}

void UGatherTextCommandletBase::Initialize( const TSharedRef< FLocTextHelper >& InGatherManifestHelper, const TSharedPtr< FLocalizationSCC >& InSourceControlInfo )
{
	GatherManifestHelper = InGatherManifestHelper;
	SourceControlInfo = InSourceControlInfo;

	// Cache the split platform info
	SplitPlatforms.Reset();
	if (InGatherManifestHelper->ShouldSplitPlatformData())
	{
		for (const FString& SplitPlatformName : InGatherManifestHelper->GetPlatformsToSplit())
		{
			SplitPlatforms.Add(*SplitPlatformName, FString::Printf(TEXT("/%s/"), *SplitPlatformName));
		}
		SplitPlatforms.KeySort(FNameLexicalLess());
	}
}

void UGatherTextCommandletBase::BeginDestroy()
{
	Super::BeginDestroy();

	GatherManifestHelper.Reset();
	SourceControlInfo.Reset();
}

void UGatherTextCommandletBase::CreateCustomEngine(const FString& Params)
{
	GEngine = GEditor = NULL;//Force a basic default engine. 
}

bool UGatherTextCommandletBase::IsSplitPlatformName(const FName InPlatformName) const
{
	return SplitPlatforms.Contains(InPlatformName);
}

bool UGatherTextCommandletBase::ShouldSplitPlatformForPath(const FString& InPath, FName* OutPlatformName) const
{
	const FName SplitPlatformName = GetSplitPlatformNameFromPath(InPath);
	if (OutPlatformName)
	{
		*OutPlatformName = SplitPlatformName;
	}
	return !SplitPlatformName.IsNone();
}

FName UGatherTextCommandletBase::GetSplitPlatformNameFromPath(const FString& InPath) const
{
	return GetSplitPlatformNameFromPath_Static(InPath, SplitPlatforms);
}

FName UGatherTextCommandletBase::GetSplitPlatformNameFromPath_Static(const FString& InPath, const TMap<FName, FString>& InSplitPlatforms)
{
	for (const auto& Pair : InSplitPlatforms)
	{
		if (InPath.Contains(Pair.Value))
		{
			return Pair.Key;
		}
	}
	return FName();
}

bool UGatherTextCommandletBase::GetBoolFromConfig( const TCHAR* Section, const TCHAR* Key, bool& OutValue, const FString& Filename )
{
	bool bSuccess = GConfig->GetBool( Section, Key, OutValue, Filename );
	
	if( !bSuccess )
	{
		bSuccess = GConfig->GetBool( TEXT("CommonSettings"), Key, OutValue, Filename );
	}
	return bSuccess;
}

bool UGatherTextCommandletBase::GetStringFromConfig( const TCHAR* Section, const TCHAR* Key, FString& OutValue, const FString& Filename )
{
	bool bSuccess = GConfig->GetString( Section, Key, OutValue, Filename );

	if( !bSuccess )
	{
		bSuccess = GConfig->GetString( TEXT("CommonSettings"), Key, OutValue, Filename );
	}
	return bSuccess;
}

void UGatherTextCommandletBase::ResolveLocalizationPath(FString& InOutPath)
{
	static const FString AbsoluteEnginePath = FPaths::ConvertRelativePathToFull(FPaths::EngineDir()) / FString();
	static const FString AbsoluteProjectPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()) / FString();

	InOutPath.ReplaceInline(TEXT("%LOCENGINEROOT%"), *AbsoluteEnginePath, ESearchCase::CaseSensitive);
	InOutPath.ReplaceInline(TEXT("%LOCPROJECTROOT%"), *AbsoluteProjectPath, ESearchCase::CaseSensitive);

	if (FPaths::IsRelative(InOutPath))
	{
		static const FString AbsoluteTargetPath = FPaths::ConvertRelativePathToFull(UGatherTextCommandletBase::GetProjectBasePath()) / FString();

		InOutPath.InsertAt(0, AbsoluteTargetPath);
	}

	FPaths::CollapseRelativeDirectories(InOutPath);
}

bool UGatherTextCommandletBase::GetPathFromConfig( const TCHAR* Section, const TCHAR* Key, FString& OutValue, const FString& Filename )
{
	const bool bSuccess = GetStringFromConfig( Section, Key, OutValue, Filename );
	if (bSuccess)
	{
		ResolveLocalizationPath(OutValue);
	}
	return bSuccess;
}

int32 UGatherTextCommandletBase::GetStringArrayFromConfig( const TCHAR* Section, const TCHAR* Key, TArray<FString>& OutArr, const FString& Filename )
{
	int32 count = GConfig->GetArray( Section, Key, OutArr, Filename );

	if( count == 0 )
	{
		count = GConfig->GetArray( TEXT("CommonSettings"), Key, OutArr, Filename );
	}
	return count;
}

int32 UGatherTextCommandletBase::GetPathArrayFromConfig( const TCHAR* Section, const TCHAR* Key, TArray<FString>& OutArr, const FString& Filename )
{
	int32 count = GetStringArrayFromConfig( Section, Key, OutArr, Filename );

	for (FString& Path : OutArr)
	{
		ResolveLocalizationPath(Path);
	}

	return count;
}

const FString& UGatherTextCommandletBase::GetProjectBasePath()
{
	static const FString ProjectBasePath = FApp::HasProjectName() ? FPaths::ProjectDir() : FPaths::EngineDir();
	return ProjectBasePath;
}

FFuzzyPathMatcher::FFuzzyPath::FFuzzyPath(FString InPathFilter, const EPathType InPathType)
	: PathFilter(MoveTemp(InPathFilter))
	, PathType(InPathType)
	, PathTestPolicy(FFuzzyPathMatcher::CalculatePolicyForPath(PathFilter))
{
}

FFuzzyPathMatcher::FFuzzyPathMatcher(const TArray<FString>& InIncludePathFilters, const TArray<FString>& InExcludePathFilters)
{
	FuzzyPaths.Reserve(InIncludePathFilters.Num() + InExcludePathFilters.Num());

	for (const FString& IncludePath : InIncludePathFilters)
	{
		FuzzyPaths.Add(FFuzzyPath(FPaths::ConvertRelativePathToFull(IncludePath), EPathType::Include));
	}

	for (const FString& ExcludePath : InExcludePathFilters)
	{
		FuzzyPaths.Add(FFuzzyPath(FPaths::ConvertRelativePathToFull(ExcludePath), EPathType::Exclude));
	}

	// Sort the paths so that deeper paths with fewer wildcards appear first in the list
	FuzzyPaths.Sort([](const FFuzzyPath& PathOne, const FFuzzyPath& PathTwo) -> bool
	{
		auto GetFuzzRating = [](const FFuzzyPath& InFuzzyPath) -> int32
		{
			int32 PathDepth = 0;
			int32 PathFuzz = 0;
			for (const TCHAR Char : InFuzzyPath.PathFilter)
			{
				if (Char == TEXT('/') || Char == TEXT('\\'))
				{
					++PathDepth;
				}
				else if (Char == TEXT('*') || Char == TEXT('?'))
				{
					++PathFuzz;
				}
			}

			return (100 - PathDepth) + (PathFuzz * 1000);
		};

		const int32 PathOneFuzzRating = GetFuzzRating(PathOne);
		const int32 PathTwoFuzzRating = GetFuzzRating(PathTwo);
		if (PathOneFuzzRating == PathTwoFuzzRating)
		{
			// In the case of a tie, allow an exclusion to take priority
			return (uint8)PathOne.PathType > (uint8)PathTwo.PathType;
		}
		return PathOneFuzzRating < PathTwoFuzzRating;
	});

	// Now we pre-process and alter the path filter for paths that will be compared with EPathTestPolicy::StartsWith
	// We only do that here because we need the paths that end with the * wildcard to be intact for the above sorting 
	for (FFuzzyPath& FuzzyPath : FuzzyPaths)
	{
		if (FuzzyPath.PathTestPolicy == EPathTestPolicy::StartsWith)
		{
			FuzzyPath.PathFilter.LeftChopInline(1);
		}
	}
}

FFuzzyPathMatcher::EPathMatch FFuzzyPathMatcher::TestPath(const FString& InPathToTest) const
{
	for (const FFuzzyPath& FuzzyPath : FuzzyPaths)
	{
		if (FuzzyPath.PathTestPolicy == EPathTestPolicy::StartsWith)
		{
			// The wildcard at the end should already be be removed as part of a preprocessing step.
			check(FuzzyPath.PathFilter[FuzzyPath.PathFilter.Len() - 1] != TEXT('*'));
			if (InPathToTest.StartsWith(FuzzyPath.PathFilter))
			{
				return (FuzzyPath.PathType == EPathType::Include) ? EPathMatch::Included : EPathMatch::Excluded;
			}
		}
		else if (FuzzyPath.PathTestPolicy == EPathTestPolicy::MatchesWildcard)
		{
			if (InPathToTest.MatchesWildcard(FuzzyPath.PathFilter))
			{
				return (FuzzyPath.PathType == EPathType::Include) ? EPathMatch::Included : EPathMatch::Excluded;
			}
		}
	}

	return EPathMatch::NoMatch;
}

FFuzzyPathMatcher::EPathTestPolicy FFuzzyPathMatcher::CalculatePolicyForPath(const FString& InPath)
{
	// We can perform a StartsWith policy if there's a single * at the end of the string
	// FindChar looks from the start of the string, so if we find the match at the last character then we know there's a single *
	int32 FirstAsteriskIndex = INDEX_NONE;
	InPath.FindChar(TEXT('*'), FirstAsteriskIndex);

	bool bHasSingleWildcardAtEnd = FirstAsteriskIndex == InPath.Len() - 1;
	return bHasSingleWildcardAtEnd
		? EPathTestPolicy::StartsWith
		: EPathTestPolicy::MatchesWildcard;
}
