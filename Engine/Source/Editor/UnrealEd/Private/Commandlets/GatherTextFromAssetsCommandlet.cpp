// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextFromAssetsCommandlet.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/ParallelFor.h"
#include "CollectionManagerModule.h"
#include "Commandlets/CommandletHelpers.h"
#include "Commandlets/GatherTextFromSourceCommandlet.h"
#include "DistanceFieldAtlas.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "Engine/UserDefinedEnum.h"
#include "Engine/World.h"
#include "Framework/Commands/Commands.h"
#include "GatherTextMetaDataHelper.h"
#include "HAL/FileManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Logging/StructuredLog.h"
#include "MeshCardBuild.h"
#include "MeshCardRepresentation.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/AsciiSet.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "PackageHelperFunctions.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "ShaderCompiler.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Sound/DialogueWave.h"
#include "Templates/UniquePtr.h"
#include "UnrealEdMisc.h"
#include "UObject/Class.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/UObjectIterator.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GatherTextFromAssetsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextFromAssetsCommandlet, Log, All);
namespace GatherTextFromAssetsCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

/** Special feedback context used to stop the commandlet to reporting failure due to a package load error */
class FLoadPackageLogOutputRedirector : public FFeedbackContext
{
public:
	struct FScopedCapture
	{
		explicit FScopedCapture(FStringView InPackageContext)
			: LogOutputRedirector(&FLoadPackageLogOutputRedirector::Get())
		{
			LogOutputRedirector->BeginCapturingLogData(InPackageContext);
		}

		~FScopedCapture()
		{
			LogOutputRedirector->EndCapturingLogData();
		}

		FLoadPackageLogOutputRedirector* LogOutputRedirector;
	};

	virtual ~FLoadPackageLogOutputRedirector() = default;

	static FLoadPackageLogOutputRedirector& Get()
	{
		static FLoadPackageLogOutputRedirector Instance;
		return Instance;
	}

	virtual bool CanBeUsedOnAnyThread() const override
	{
		return true;
	}

	virtual bool CanBeUsedOnMultipleThreads() const override
	{
		return true;
	}

	void Hook(FOutputDevice* InTargetOutputDevice = GWarn)
	{
		UE::TScopeLock _(Mutex);

		TargetOutputDevice = InTargetOutputDevice;

		// Override GWarn so that we can capture any log data
		checkf(!OriginalWarningContext, TEXT("Hook called recursively!"));
		OriginalWarningContext = GWarn;
		GWarn = this;
	}

	void Unhook()
	{
		UE::TScopeLock _(Mutex);

		checkf(!bIsCapturing, TEXT("Unhook called while in a BeginCapturingLogData scope!"));

		// Restore the original GWarn now that we've finished capturing log data
		checkf(OriginalWarningContext, TEXT("Unhook mismatch with Hook!"));
		GWarn = OriginalWarningContext;
		OriginalWarningContext = nullptr;

		TargetOutputDevice = nullptr;
	}

	void BeginCapturingLogData(FStringView InPackageContext)
	{
		UE::TScopeLock _(Mutex);

		checkf(OriginalWarningContext, TEXT("BeginCapturingLogData called before Hook!"));
		checkf(!bIsCapturing, TEXT("BeginCapturingLogData called recursively!"));
		PackageContext = InPackageContext;
		bIsCapturing = true;
	}

	void EndCapturingLogData()
	{
		UE::TScopeLock _(Mutex);

		checkf(bIsCapturing, TEXT("EndCapturingLogData mismatch with BeginCapturingLogData!"));

		// Report any messages, and also report a warning if we silenced some warnings or errors when loading
		if (TargetOutputDevice && (ErrorCount > 0 || WarningCount > 0))
		{
			static const FString LogIndentation = TEXT("    ");

			TargetOutputDevice->CategorizedLogf(LogGatherTextFromAssetsCommandlet.GetCategoryName(), ELogVerbosity::Display, TEXT("Package '%s' produced %d error(s) and %d warning(s) while loading (see below). Please verify that your text has gathered correctly."), *PackageContext, ErrorCount, WarningCount);
			for (const FString& FormattedOutput : FormattedErrorsAndWarningsList)
			{
				TargetOutputDevice->Log(NAME_None, ELogVerbosity::Display, LogIndentation + FormattedOutput);
			}
		}

		PackageContext.Reset();
		bIsCapturing = false;

		// Reset the counts and previous log output
		ErrorCount = 0;
		WarningCount = 0;
		FormattedErrorsAndWarningsList.Reset();
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		Serialize(V, Verbosity, Category, -1.0);
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override
	{
		UE::TScopeLock _(Mutex);

		// We only want to capture logs while capture is active, and only from the GT and ALT
		if (!bIsCapturing || !(IsInGameThread() || IsInAsyncLoadingThread()))
		{
			FFeedbackContext* TargetWarningContext = OriginalWarningContext ? OriginalWarningContext : GWarn;
			TargetWarningContext->Serialize(V, Verbosity, Category, Time);
			return;
		}

		if (Verbosity == ELogVerbosity::Error)
		{
			++ErrorCount;
			// Downgrade Error to Log while loading packages to avoid false positives from things searching for "Error:" tokens in the log file
			FormattedErrorsAndWarningsList.Add(FOutputDeviceHelper::FormatLogLine(ELogVerbosity::Log, Category, V));
		}
		else if (Verbosity == ELogVerbosity::Warning)
		{
			++WarningCount;
			// Downgrade Warning to Log while loading packages to avoid false positives from things searching for "Warning:" tokens in the log file
			FormattedErrorsAndWarningsList.Add(FOutputDeviceHelper::FormatLogLine(ELogVerbosity::Log, Category, V));
		}
		else if (Verbosity == ELogVerbosity::Display)
		{
			// Downgrade Display to Log while loading packages
			OriginalWarningContext->Serialize(V, ELogVerbosity::Log, Category, Time);
		}
		else
		{
			// Pass anything else on to GWarn so that it can handle them appropriately
			OriginalWarningContext->Serialize(V, Verbosity, Category, Time);
		}
	}

	virtual void SerializeRecord(const UE::FLogRecord& Record) override
	{
		UE::TScopeLock _(Mutex);

		// We only want to capture logs while capture is active, and only from the GT and ALT
		if (!bIsCapturing || !(IsInGameThread() || IsInAsyncLoadingThread()))
		{
			FFeedbackContext* TargetWarningContext = OriginalWarningContext ? OriginalWarningContext : GWarn;
			TargetWarningContext->SerializeRecord(Record);
			return;
		}

		const ELogVerbosity::Type Verbosity = Record.GetVerbosity();
		if (Verbosity == ELogVerbosity::Error)
		{
			++ErrorCount;
			// Downgrade Error to Log while loading packages to avoid false positives from things searching for "Error:" tokens in the log file
			UE::FLogRecord LocalRecord = Record;
			LocalRecord.SetVerbosity(ELogVerbosity::Log);
			TStringBuilder<512> Line;
			FormatRecordLine(Line, LocalRecord);
			FormattedErrorsAndWarningsList.Emplace(Line);
		}
		else if (Verbosity == ELogVerbosity::Warning)
		{
			++WarningCount;
			// Downgrade Warning to Log while loading packages to avoid false positives from things searching for "Warning:" tokens in the log file
			UE::FLogRecord LocalRecord = Record;
			LocalRecord.SetVerbosity(ELogVerbosity::Log);
			TStringBuilder<512> Line;
			FormatRecordLine(Line, LocalRecord);
			FormattedErrorsAndWarningsList.Emplace(Line);
		}
		else if (Verbosity == ELogVerbosity::Display)
		{
			// Downgrade Display to Log while loading packages
			UE::FLogRecord LocalRecord = Record;
			LocalRecord.SetVerbosity(ELogVerbosity::Log);
			OriginalWarningContext->SerializeRecord(LocalRecord);
		}
		else
		{
			// Pass anything else on to GWarn so that it can handle them appropriately
			OriginalWarningContext->SerializeRecord(Record);
		}
	}

private:
	FLoadPackageLogOutputRedirector() = default;

	int32 ErrorCount = 0;
	int32 WarningCount = 0;
	TArray<FString> FormattedErrorsAndWarningsList;

	FString PackageContext;
	FFeedbackContext* OriginalWarningContext = nullptr;

	FOutputDevice* TargetOutputDevice = nullptr;

	bool bIsCapturing = false;

	mutable UE::FMutex Mutex;
};

class FAssetGatherCacheMetrics
{
public:
	FAssetGatherCacheMetrics()
		: CachedAssetCount(0)
		, UncachedAssetCount(0)
	{
		FMemory::Memzero(UncachedAssetBreakdown);
	}

	void CountCachedAsset()
	{
		++CachedAssetCount;
	}

	void CountUncachedAsset(const UGatherTextFromAssetsCommandlet::EPackageLocCacheState InState)
	{
		check(InState != UGatherTextFromAssetsCommandlet::EPackageLocCacheState::Cached);
		++UncachedAssetCount;
		++UncachedAssetBreakdown[(int32)InState];
	}

	void LogMetrics() const
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "%ls", *ToString());
	}

	FString ToString() const
	{
		return FString::Printf(
			TEXT("Asset gather cache metrics: %d cached, %d uncached (%d too old, %d no cache or contained bytecode)"), 
			CachedAssetCount, 
			UncachedAssetCount, 
			UncachedAssetBreakdown[(int32)UGatherTextFromAssetsCommandlet::EPackageLocCacheState::Uncached_TooOld], 
			UncachedAssetBreakdown[(int32)UGatherTextFromAssetsCommandlet::EPackageLocCacheState::Uncached_NoCache]
			);
	}

private:
	int32 CachedAssetCount;
	int32 UncachedAssetCount;
	int32 UncachedAssetBreakdown[(int32)UGatherTextFromAssetsCommandlet::EPackageLocCacheState::Cached];
};

namespace UE::Private::GatherTextFromAssetsCommandlet
{
	static FAssetGatherCacheMetrics AssetGatherCacheMetrics;
	/**
	 * Commandlets don't tick, but loading assets can queue async building work to various systems.
	 * We tick these systems periodically during a gather to prevent us from running out of memory due to the queued pending tasks.
	 * Refer to the cooker to determine if this function needs to be expanded to cover more systems.
	 */
	void TickBackgroundTasks()
	{
		if (GShaderCompilingManager)
		{
			GShaderCompilingManager->ProcessAsyncResults(true, false);
		}
		if (GDistanceFieldAsyncQueue)
		{
			GDistanceFieldAsyncQueue->ProcessAsyncTasks();
		}
		if (GCardRepresentationAsyncQueue)
		{
			GCardRepresentationAsyncQueue->ProcessAsyncTasks();
		}
	}

	static bool bParallelizeIncludeExcludePathFiltering = true;
	static FAutoConsoleVariableRef CVarParallelizeIncludeExcludePathFiltering(TEXT("Localization.GatherTextFromAssetsCommandlet.ParallelizeIncludeExcludePathFiltering"), bParallelizeIncludeExcludePathFiltering, TEXT("True to parallelize the include exclude path filtering. False to force it to be single threaded for easier debugging."));

	static bool bParallelizeProcessAndRemoveCachedPackages = true;
	static FAutoConsoleVariableRef CVarParallelizeProcessAndRemoveCachedPackages(TEXT("Localization.GatherTextFromAssetsCommandlet.ParallelizeProcessAndRemoveCachedPackages"), bParallelizeProcessAndRemoveCachedPackages, TEXT("True to parallelize the 'process and remove cached packages' step. False to force it to be single threaded for easier debugging."));

	static int32 ProcessAndRemoveCachedPackagesMaxThreads = -1;
	static FAutoConsoleVariableRef CVarProcessAndRemoveCachedPackagesMaxThreads(TEXT("Localization.GatherTextFromAssetsCommandlet.ProcessAndRemoveCachedPackagesMaxThreads"), ProcessAndRemoveCachedPackagesMaxThreads, TEXT("Max number of threads to use if parallelizing the 'process and remove cached packages' step, or <= 0 to use as many threads as possible."));

	// Bump whenever changing the data in any of the FGatherTextFromAssetsWorker*Message types
	static constexpr int32 WorkerProtocolVersion = 2;

	// Values controlling how packages are distributed to workers
	static int32 MinPackagesToUseWorkers = 1000;
	static FAutoConsoleVariableRef CVarMinPackagesToUseWorkers(TEXT("Localization.GatherTextFromAssetsCommandlet.MinPackagesToUseWorkers"), MinPackagesToUseWorkers, TEXT("How many packages should we have to process (in total) before we start the workers?"));
	static int32 MinPackagesToKeepLocal = 50;
	static FAutoConsoleVariableRef CVarMinPackagesToKeepLocal(TEXT("Localization.GatherTextFromAssetsCommandlet.MinPackagesToKeepLocal"), MinPackagesToKeepLocal, TEXT("How many packages should we keep in the main process, rather than distribute to workers? (to avoid waiting on workers at the end of the gather)"));
	static int32 MaxPackagesToDistribute = 100;
	static FAutoConsoleVariableRef CVarMaxPackagesToDistribute(TEXT("Localization.GatherTextFromAssetsCommandlet.MaxPackagesToDistribute"), MaxPackagesToDistribute, TEXT("How many packages should we distribute to each worker in a single batch?"));
	static int32 WorkerIdleThreshold = 20;
	static FAutoConsoleVariableRef CVarWorkerIdleThreshold(TEXT("Localization.GatherTextFromAssetsCommandlet.WorkerIdleThreshold"), WorkerIdleThreshold, TEXT("How many packages can a worker have pending in its queue before we consider it 'idle' and try and assign it more packages? (to avoid workers becoming idle)"));

	UPackage* LoadPackageToGather(const FString& PackageName, const bool bIsMapPackage)
	{
		UPackage* Package = nullptr;
		{
			FLoadPackageLogOutputRedirector::FScopedCapture ScopedCapture(PackageName);
			if (bIsMapPackage)
			{
				Package = LoadWorldPackageForEditor(PackageName, EWorldType::Editor, LOAD_NoWarn | LOAD_Quiet);
			}
			else
			{
				Package = LoadPackage(nullptr, *PackageName, LOAD_NoWarn | LOAD_Quiet);
			}
		}
		return Package;
	}

	bool GatherTextFromPackage(UPackage* Package, const bool bIsMapPackage, const TSet<FGuid>& ExternalActors, const TSet<FName>& ExternalPackages, const bool bGatherFromPrimaryPackage, const bool bGatherFromExternalPackages, TArray<FGatherableTextData>& OutGatherableTextDataArray)
	{
		// Because packages may not have been resaved after this flagging was implemented, we may have added packages to load that weren't flagged - potential false positives.
		// The loading process should have reflagged said packages so that only true positives will have this flag.
		const bool bHasExternalObjects = ExternalActors.Num() > 0 || ExternalPackages.Num() > 0;
		if (Package->RequiresLocalizationGather() || bHasExternalObjects)
		{
			if (bGatherFromPrimaryPackage)
			{
				// Gathers from the given package
				EPropertyLocalizationGathererResultFlags GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
				FPropertyLocalizationDataGatherer(OutGatherableTextDataArray, Package, GatherableTextResultFlags);
			}

			if (bGatherFromExternalPackages && bHasExternalObjects)
			{
				// If this is a WP world then query the localization for any external actors actors that were determined to be stale
				TOptional<FScopedEditorWorld> ScopeEditorWorld;
				if (UWorld* World = bIsMapPackage ? UWorld::FindWorldInPackage(Package) : nullptr)
				{
					if (!World->IsInitialized())
					{
						UWorld::InitializationValues IVS;
						IVS.InitializeScenes(false);
						IVS.AllowAudioPlayback(false);
						IVS.RequiresHitProxies(false);
						IVS.CreatePhysicsScene(false);
						IVS.CreateNavigation(false);
						IVS.CreateAISystem(false);
						IVS.ShouldSimulatePhysics(false);
						IVS.EnableTraceCollision(false);
						IVS.SetTransactional(false);
						IVS.CreateFXSystem(false);
						IVS.CreateWorldPartition(true);

						FLoadPackageLogOutputRedirector::FScopedCapture ScopedCapture(Package->GetName());
						ScopeEditorWorld.Emplace(World, IVS); // Initializing FScopedEditorWorld can log warnings, so capture those like we do with loading errors
					}

					if (UWorldPartition* WorldPartition = World->GetWorldPartition();
						WorldPartition && ExternalActors.Num() > 0)
					{
						// ForEachActorWithLoading may GC while running, so keep the world partition (and indirectly the world and its package) alive
						TGCObjectScopeGuard<UWorldPartition> WorldPartitionGCGuard(WorldPartition);

						FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorParams;
						ForEachActorParams.ActorGuids = ExternalActors.Array();

						FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition,
							[&OutGatherableTextDataArray](const FWorldPartitionActorDescInstance* ActorDescInstance)
							{
								if (const AActor* Actor = ActorDescInstance->GetActor())
								{
									EPropertyLocalizationGathererResultFlags GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
									FPropertyLocalizationDataGatherer(OutGatherableTextDataArray, Actor->GetExternalPackage(), GatherableTextResultFlags);
								}
								return true;
							}, ForEachActorParams);
					}
				}

				// Other external packages are currently loaded with their outer package
				for (const FName ExternalPackageName : ExternalPackages)
				{
					if (UPackage* ExternalPackage = FindPackage(nullptr, FNameBuilder(ExternalPackageName).ToString()))
					{
						EPropertyLocalizationGathererResultFlags GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
						FPropertyLocalizationDataGatherer(OutGatherableTextDataArray, ExternalPackage, GatherableTextResultFlags);
					}
					else
					{
						UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "'{externalPackage}' expected to load with '{package}'.",
							("externalPackage", ExternalPackageName.ToString()),
							("package", Package->GetName()),
							("id", ::GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
						);
					}
				}
			}

			return true;
		}

		return false;
	}

	bool HasExceededMemoryLimit(const uint64 MinFreeMemoryBytes, const uint64 MaxUsedMemoryBytes, const bool bLog)
	{
		const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();

		const uint64 FreeMemoryBytes = MemStats.AvailablePhysical;
		if (MinFreeMemoryBytes > 0u && FreeMemoryBytes < MinFreeMemoryBytes)
		{
			UE_CLOGF(bLog, LogGatherTextFromAssetsCommandlet, Display, "Free system memory is currently %ls of %ls (%ls used by process), which is less than the requested limit of %ls; a flush will be performed.", 
				*FText::AsMemory(FreeMemoryBytes).ToString(), *FText::AsMemory(MemStats.TotalPhysical).ToString(), *FText::AsMemory(MemStats.UsedPhysical).ToString(), *FText::AsMemory(MinFreeMemoryBytes).ToString());
			return true;
		}

		const uint64 UsedMemoryBytes = MemStats.UsedPhysical;
		if (MaxUsedMemoryBytes > 0u && UsedMemoryBytes >= MaxUsedMemoryBytes)
		{
			UE_CLOGF(bLog, LogGatherTextFromAssetsCommandlet, Display, "Used process memory is currently %ls, which is greater than the requested limit of %ls; a flush will be performed.", *FText::AsMemory(UsedMemoryBytes).ToString(), *FText::AsMemory(MaxUsedMemoryBytes).ToString());
			return true;
		}

		return false;
	}

	template <typename TPackagePendingGather>
	void PurgeGarbage(TArray<TPackagePendingGather>* PackagesPendingGather)
	{
		FlushAsyncLoading();

		TSet<FName> LoadedPackageNames;
		TSet<FName> PackageNamesToKeepAlive;
		TArray<UObject*> ObjectsToKeepAlive;

		const bool bHasPackagesPendingGather = PackagesPendingGather && PackagesPendingGather->Num() > 0;
		if (bHasPackagesPendingGather)
		{
			// Build a complete list of packages that we still need to keep alive, either because we still 
			// have to process them, or because they're a dependency for something we still have to process
			for (const TPackagePendingGather& PackagePendingGather : *PackagesPendingGather)
			{
				PackageNamesToKeepAlive.Add(PackagePendingGather.PackageName);
				PackageNamesToKeepAlive.Append(PackagePendingGather.Dependencies);
			}

			for (TObjectIterator<UPackage> PackageIt; PackageIt; ++PackageIt)
			{
				UPackage* Package = *PackageIt;
				if (PackageNamesToKeepAlive.Contains(Package->GetFName()))
				{
					LoadedPackageNames.Add(Package->GetFName());

					// Keep any requested packages (and their RF_Standalone inners) alive during a call to PurgeGarbage
					ObjectsToKeepAlive.Add(Package);
					ForEachObjectWithPackage(Package, [&ObjectsToKeepAlive](UObject* InPackageInner)
					{
						if (InPackageInner->HasAnyFlags(RF_Standalone | RF_HasExternalPackage))
						{
							ObjectsToKeepAlive.Add(InPackageInner);
						}
						return true;
					}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
				}
			}
		}

		if (ObjectsToKeepAlive.Num() > 0)
		{
			TGCObjectsScopeGuard<UObject> ObjectsToKeepAliveScopeGuard(ObjectsToKeepAlive);
			CollectGarbage(IsRunningCommandlet() ? RF_NoFlags : GARBAGE_COLLECTION_KEEPFLAGS);
		}
		else
		{
			CollectGarbage(IsRunningCommandlet() ? RF_NoFlags : GARBAGE_COLLECTION_KEEPFLAGS);
		}

		// Fully process the shader compilation results when performing a full purge, as it's the only way to reclaim that memory
		if (!PackagesPendingGather && GShaderCompilingManager)
		{
			GShaderCompilingManager->ProcessAsyncResults(false, false);
		}

		if (bHasPackagesPendingGather)
		{
			// Sort the remaining packages to gather so that currently loaded packages are processed first, followed by those with the most dependencies
			// This aims to allow packages to be GC'd as soon as possible once nothing is no longer referencing them as a dependency
			// Note: This array is processed backwards, so "first" is actually the end of the array
			PackagesPendingGather->Sort([&LoadedPackageNames](const TPackagePendingGather& PackagePendingGatherOne, const TPackagePendingGather& PackagePendingGatherTwo)
			{
				const bool bIsPackageOneLoaded = LoadedPackageNames.Contains(PackagePendingGatherOne.PackageName);
				const bool bIsPackageTwoLoaded = LoadedPackageNames.Contains(PackagePendingGatherTwo.PackageName);
				return (bIsPackageOneLoaded == bIsPackageTwoLoaded)
					? PackagePendingGatherOne.Dependencies.Num() < PackagePendingGatherTwo.Dependencies.Num()
					: bIsPackageTwoLoaded;
			});
		}
	}

	template <typename TPackagePendingGather>
	void ConditionalPurgeGarbage(const uint64 MinFreeMemoryBytes, const uint64 MaxUsedMemoryBytes, TArray<TPackagePendingGather>* PackagesPendingGather)
	{
		if (HasExceededMemoryLimit(MinFreeMemoryBytes, MaxUsedMemoryBytes, /*bLog*/true))
		{
			// First try a minimal purge to only remove things that are no longer referenced or needed by other packages pending gather
			PurgeGarbage<TPackagePendingGather>(PackagesPendingGather);

			if (HasExceededMemoryLimit(MinFreeMemoryBytes, MaxUsedMemoryBytes, /*bLog*/false))
			{
				// If we're still over the memory limit after a minimal purge, then attempt a full purge
				PurgeGarbage<TPackagePendingGather>(nullptr);

				// If we're still over the memory limit after both purges, then log a message that we may be about to OOM
				UE_CLOGF(HasExceededMemoryLimit(MinFreeMemoryBytes, MaxUsedMemoryBytes, /*bLog*/false), LogGatherTextFromAssetsCommandlet, Display, "Flushing failed to reduce process memory to within the requested limits; this process may OOM!");
			}
		}
	}

	void TickMessageBusGT()
	{
		// Flush the task graph to grab any pending messages
		// We put a dummy fence task into the queue to avoid potentially waiting indefinitely if other threads keep adding game thread events
		if (!FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread))
		{
			DECLARE_CYCLE_STAT(TEXT("GatherTextFromAssets.TickMessageBusGT"), STAT_GatherTextFromAssets_TickMessageBusGT, STATGROUP_TaskGraphTasks);
			FGraphEventRef FenceHandle = FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(FSimpleDelegateGraphTask::FDelegate(), GET_STATID(STAT_GatherTextFromAssets_TickMessageBusGT), nullptr, ENamedThreads::GameThread);
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(FenceHandle, ENamedThreads::GameThread);
		}
	}
}

#define LOC_DEFINE_REGION

//////////////////////////////////////////////////////////////////////////
//UGatherTextFromAssetsCommandlet

const FString UGatherTextFromAssetsCommandlet::UsageText
(
	TEXT("GatherTextFromAssetsCommandlet usage...\r\n")
	TEXT("    <GameName> UGatherTextFromAssetsCommandlet -root=<parsed code root folder> -exclude=<paths to exclude>\r\n")
	TEXT("    \r\n")
	TEXT("    <paths to include> Paths to include. Delimited with ';'. Accepts wildcards. eg \"*Content/Developers/*;*/TestMaps/*\" OPTIONAL: If not present, everything will be included. \r\n")
	TEXT("    <paths to exclude> Paths to exclude. Delimited with ';'. Accepts wildcards. eg \"*Content/Developers/*;*/TestMaps/*\" OPTIONAL: If not present, nothing will be excluded.\r\n")
);

UGatherTextFromAssetsCommandlet::UGatherTextFromAssetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MinFreeMemoryBytes(0)
	, MaxUsedMemoryBytes(0)
	, NumPackagesDupLocId(0)
	, bSkipGatherCache(false)
	, bSearchAllAssets(true)
	, bApplyRedirectorsToCollections(true)
	, bShouldGatherFromEditorOnlyData(false)
	, bShouldExcludeDerivedClasses(false)
	, bFixPackageLocalizationIdConflict(false)
{
}

void UGatherTextFromAssetsCommandlet::ProcessGatherableTextDataArray(const TArray<FGatherableTextData>& GatherableTextDataArray)
{
	for (const FGatherableTextData& GatherableTextData : GatherableTextDataArray)
	{
		for (const FTextSourceSiteContext& TextSourceSiteContext : GatherableTextData.SourceSiteContexts)
		{
			if (!TextSourceSiteContext.IsEditorOnly || bShouldGatherFromEditorOnlyData)
			{
				if (TextSourceSiteContext.KeyName.IsEmpty())
				{
					UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Detected missing key on asset '{location}'.",
						("location", *TextSourceSiteContext.SiteDescription),
						("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
					);
					continue;
				}

				static const FLocMetadataObject DefaultMetadataObject;

				FManifestContext Context;
				Context.Key = TextSourceSiteContext.KeyName;
				Context.KeyMetadataObj = !(FLocMetadataObject::IsMetadataExactMatch(&TextSourceSiteContext.KeyMetaData, &DefaultMetadataObject)) ? MakeShareable(new FLocMetadataObject(TextSourceSiteContext.KeyMetaData)) : nullptr;
				Context.InfoMetadataObj = !(FLocMetadataObject::IsMetadataExactMatch(&TextSourceSiteContext.InfoMetaData, &DefaultMetadataObject)) ? MakeShareable(new FLocMetadataObject(TextSourceSiteContext.InfoMetaData)) : nullptr;
				Context.bIsOptional = TextSourceSiteContext.IsOptional;
				Context.DevNotes = TextSourceSiteContext.DevNotes;
				Context.SourceLocation = TextSourceSiteContext.SiteDescription;
				Context.PlatformName = GetSplitPlatformNameFromPath(TextSourceSiteContext.SiteDescription);

				FLocItem Source(GatherableTextData.SourceData.SourceString);

				GatherManifestHelper->AddSourceText(GatherableTextData.NamespaceName, Source, Context);
			}
		}
	}
}

void CalculateDependenciesImpl(IAssetRegistry& InAssetRegistry, const FName& InPackageName, TSet<FName>& OutDependencies, TMap<FName, TSet<FName>>& InOutPackageNameToDependencies)
{
	const TSet<FName>* CachedDependencies = InOutPackageNameToDependencies.Find(InPackageName);

	if (!CachedDependencies)
	{
		// Add a dummy entry now to avoid any infinite recursion for this package as we build the dependencies list
		InOutPackageNameToDependencies.Add(InPackageName);

		// Build the complete list of dependencies for this package
		TSet<FName> LocalDependencies;
		{
			TArray<FName> LocalDependenciesArray;
			InAssetRegistry.GetDependencies(InPackageName, LocalDependenciesArray);

			LocalDependencies.Append(LocalDependenciesArray);
			for (const FName& LocalDependency : LocalDependenciesArray)
			{
				CalculateDependenciesImpl(InAssetRegistry, LocalDependency, LocalDependencies, InOutPackageNameToDependencies);
			}
		}

		// Add the real data now
		CachedDependencies = &InOutPackageNameToDependencies.Add(InPackageName, MoveTemp(LocalDependencies));
	}

	check(CachedDependencies);
	OutDependencies.Append(*CachedDependencies);
}

void UGatherTextFromAssetsCommandlet::CalculateDependenciesForPackagesPendingGather()
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::CalculateDependenciesForPackagesPendingGather"), LogGatherTextFromAssetsCommandlet, Display);
	
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	TMap<FName, TSet<FName>> PackageNameToDependencies;

	for (FPackagePendingGather& PackagePendingGather : PackagesPendingGather)
	{
		CalculateDependenciesImpl(AssetRegistry, PackagePendingGather.PackageName, PackagePendingGather.Dependencies, PackageNameToDependencies);
	}
}

bool IsGatherableTextDataIdentical(const TArray<FGatherableTextData>& GatherableTextDataArrayOne, const TArray<FGatherableTextData>& GatherableTextDataArrayTwo)
{
	struct FSignificantGatherableTextData
	{
		FLocKey Identity;
		FString SourceString;
	};

	auto ExtractSignificantGatherableTextData = [](const TArray<FGatherableTextData>& InGatherableTextDataArray)
	{
		TArray<FSignificantGatherableTextData> SignificantGatherableTextDataArray;

		for (const FGatherableTextData& GatherableTextData : InGatherableTextDataArray)
		{
			for (const FTextSourceSiteContext& TextSourceSiteContext : GatherableTextData.SourceSiteContexts)
			{
				SignificantGatherableTextDataArray.Add({ FString::Printf(TEXT("%s:%s"), *GatherableTextData.NamespaceName, *TextSourceSiteContext.KeyName), GatherableTextData.SourceData.SourceString });
			}
		}

		SignificantGatherableTextDataArray.Sort([](const FSignificantGatherableTextData& SignificantGatherableTextDataOne, const FSignificantGatherableTextData& SignificantGatherableTextDataTwo)
		{
			return SignificantGatherableTextDataOne.Identity < SignificantGatherableTextDataTwo.Identity;
		});

		return SignificantGatherableTextDataArray;
	};

	TArray<FSignificantGatherableTextData> SignificantGatherableTextDataArrayOne = ExtractSignificantGatherableTextData(GatherableTextDataArrayOne);
	TArray<FSignificantGatherableTextData> SignificantGatherableTextDataArrayTwo = ExtractSignificantGatherableTextData(GatherableTextDataArrayTwo);

	if (SignificantGatherableTextDataArrayOne.Num() != SignificantGatherableTextDataArrayTwo.Num())
	{
		return false;
	}

	// These arrays are sorted by identity, so everything should match as we iterate through the array
	// If it doesn't, then these caches aren't identical
	for (int32 Idx = 0; Idx < SignificantGatherableTextDataArrayOne.Num(); ++Idx)
	{
		const FSignificantGatherableTextData& SignificantGatherableTextDataOne = SignificantGatherableTextDataArrayOne[Idx];
		const FSignificantGatherableTextData& SignificantGatherableTextDataTwo = SignificantGatherableTextDataArrayTwo[Idx];

		if (SignificantGatherableTextDataOne.Identity != SignificantGatherableTextDataTwo.Identity)
		{
			return false;
		}

		if (!SignificantGatherableTextDataOne.SourceString.Equals(SignificantGatherableTextDataTwo.SourceString, ESearchCase::CaseSensitive))
		{
			return false;
		}
	}

	return true;
}

bool UGatherTextFromAssetsCommandlet::ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
{
	const FString* GatherType = ParamVals.Find(UGatherTextCommandletBase::GatherTypeParam);
	// If the param is not specified, it is assumed that both source and assets are to be gathered 
	return !GatherType || *GatherType == TEXT("Asset") || *GatherType == TEXT("All");
}

/**
 * Builds the first pass filter which currently consists of the collection filter and the optional derived class filter.
 * See BuildCollectionFilter and BuildExcludeDerivedClassesFilter
 */
bool UGatherTextFromAssetsCommandlet::BuildFirstPassFilter(FARFilter& InOutFilter) const
{
	// Filter object paths to only those in any of the specified collections.
	if (!BuildCollectionFilter(InOutFilter, CollectionFilters))
	{
		return false;
	}

	// Filter object paths to those in IncludePathFilters, if possible
	if (!BuildPackagePathsFilter(InOutFilter))
	{
		return false;
	}

	// Filter out any objects of the specified classes and their children at this point.
	if (bShouldExcludeDerivedClasses)
	{
		if (!BuildExcludeDerivedClassesFilter(InOutFilter))
		{
			return false;
		}
	}

	InOutFilter.bIncludeOnlyOnDiskAssets = true;
	InOutFilter.WithoutPackageFlags = PKG_Cooked;

	return true;
}

/** Builds a filter based on the specified collections to be used for gathering.*/
bool UGatherTextFromAssetsCommandlet::BuildCollectionFilter(FARFilter& InOutFilter, const TArray<FString>& Collections) const
{
	if (bApplyRedirectorsToCollections && Collections.Num() > 0)
	{
		UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::DiscoverAllAssetsForCollectionQuery"), LogGatherTextFromAssetsCommandlet, Display);

		// The asset registry scan must finish before redirectors have been applied to collections, 
		// so wait on that now, unless we've been asked to skip it
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Waiting for asset discovery to finish before querying collections...");
		IAssetRegistry::GetChecked().SearchAllAssets(true);
	}

	bool bHasFailedToGetACollection = false;
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();
	ICollectionManager& CollectionManager = CollectionManagerModule.Get();
	for (const FString& Collection : Collections)
	{
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (!CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType) ||
			!CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, InOutFilter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "Failed get objects in specified collection: {collection}",
				("collection", *Collection),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			bHasFailedToGetACollection = true;
		}
	}


	return !bHasFailedToGetACollection;
}

/** Builds a filter to include assets based on the current IncludePathFilters, if those filters can be represented as an asset registry filter */
bool UGatherTextFromAssetsCommandlet::BuildPackagePathsFilter(FARFilter& InOutFilter) const
{
	TArray<FName> IncludePackagePaths;

	for (const FString& IncludePath : IncludePathFilters)
	{
		FString AbsoluteIncludePath = FPaths::ConvertRelativePathToFull(IncludePath);
		if (FFuzzyPathMatcher::CalculatePolicyForPath(AbsoluteIncludePath) != FFuzzyPathMatcher::EPathTestPolicy::StartsWith)
		{
			// Not valid to use as an asset registry filter, but not an error
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Skipping first pass PackagePaths filter due to complex IncludePathFilters: %ls", *IncludePath);
			return true;
		}

		FNameBuilder IncludePackagePath;
		AbsoluteIncludePath.LeftChopInline(1);
		if (!FPackageName::TryConvertFilenameToLongPackageName(AbsoluteIncludePath, IncludePackagePath))
		{
			// Check if we're just missing the Content folder (eg, "Plugins/Foo" rather than "Plugins/Foo/Content")
			AbsoluteIncludePath /= TEXT("Content");
			if (!FPackageName::TryConvertFilenameToLongPackageName(AbsoluteIncludePath, IncludePackagePath))
			{
				// Not valid to use as an asset registry filter, but not an error
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Skipping first pass PackagePaths filter due to non-content IncludePathFilters: %ls", *IncludePath);
				return true;
			}
		}

		IncludePackagePaths.Add(FName(IncludePackagePath));
	}

	InOutFilter.bRecursivePaths = true;
	InOutFilter.PackagePaths.Append(MoveTemp(IncludePackagePaths));
	return true;
}

/** Builds a filter to remove classes and derived classes of ExactClassFilter.*/
bool UGatherTextFromAssetsCommandlet::BuildExcludeDerivedClassesFilter(FARFilter& InOutFilter) const
{
	InOutFilter.bRecursiveClasses = true;
	InOutFilter.ClassPaths.Add(UObject::StaticClass()->GetClassPathName());
	for (const FString& ExcludeClassName : ExcludeClassNames)
	{
		FTopLevelAssetPath ExcludedClassPathName = UClass::TryConvertShortTypeNameToPathName<UClass>(ExcludeClassName, ELogVerbosity::Warning, TEXT("GatherTextFromAssetsCommandlet"));
		if (!ExcludedClassPathName.IsNull())
		{
			// Note: Can't necessarily validate these class names here, as the class may be a generated blueprint class that hasn't been loaded yet.
			InOutFilter.RecursiveClassPathsExclusionSet.Add(FTopLevelAssetPath(ExcludeClassName));
		}
		else
		{
			UE_CLOGF(!ExcludeClassName.IsEmpty(), LogGatherTextFromAssetsCommandlet, Error, "Unable to convert short class name \"%ls\" to path name. Please use path names fo ExcludeClassNames", *ExcludeClassName);
		}
	}
	
	return true;
}

/** Builds a filter to exclude exactly the specified classes. This will retrieve the exact assets from the asset registry to exclude. */
bool UGatherTextFromAssetsCommandlet::BuildExcludeExactClassesFilter(FARFilter& InOutFilter) const
{
	InOutFilter.bRecursiveClasses = false;
	for (const FString& ExcludeClassName : ExcludeClassNames)
	{
		FTopLevelAssetPath ExcludedClassPathName = UClass::TryConvertShortTypeNameToPathName<UClass>(ExcludeClassName, ELogVerbosity::Warning, TEXT("GatherTextFromAssetsCommandlet"));
		if (!ExcludedClassPathName.IsNull())
		{
			// Note: Can't necessarily validate these class names here, as the class may be a generated blueprint class that hasn't been loaded yet.
			InOutFilter.ClassPaths.Add(FTopLevelAssetPath(ExcludeClassName));
		}
		else
		{
			UE_CLOGF(!ExcludeClassName.IsEmpty(), LogGatherTextFromAssetsCommandlet, Error, "Unable to convert short class name \"%ls\" to path name. Please use path names fo ExcludeClassNames", *ExcludeClassName);
		}
	}
	return true;
}

/** Filters out assets that fail the IncludePath and ExcludePath wildcard filters. */
void UGatherTextFromAssetsCommandlet::FilterAssetsBasedOnIncludeExcludePaths(TArray<FAssetData>& InOutAssetDataArray) const
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::FilterAssetsBasedOnIncludeExcludePaths"), LogGatherTextFromAssetsCommandlet, Display);
	// We pre-process the package filters into 2 sets because comparing wild cards is expensive 
	// This is the array for cases like *.uasset, *.umap 
	// We only store the extension without the wildcard for an optimization later 
	TArray<FString> PackageFileFiltersStartingWithWildcard;
	// For everything else. We will assume that we will need a wildcard match in this case 
	TArray<FString> OtherPackageFileFilters;

	for (const FString& PackageFileNameFilter : PackageFileNameFilters)
	{
		FString Extension;
		FString CleanPackageFileName;
		PackageFileNameFilter.Split(TEXT("."), &CleanPackageFileName, &Extension);
		if ((CleanPackageFileName.Len() == 1) && (CleanPackageFileName[0] == TEXT('*')) && !Extension.Contains(TEXT("*")))
		{
			// We drop the * from say *.uasset and just keep the extension 
			PackageFileFiltersStartingWithWildcard.Add(PackageFileNameFilter.RightChop(1));
		}
		else
		{
			OtherPackageFileFilters.Add(PackageFileNameFilter);
		}
	}

	const FFuzzyPathMatcher FuzzyPathMatcher = FFuzzyPathMatcher(IncludePathFilters, ExcludePathFilters);
	TArray<bool> PackagesToFilter;
	PackagesToFilter.Init( false, InOutAssetDataArray.Num());
	ParallelFor(InOutAssetDataArray.Num(), [&](int32 Index)
		{
			const FAssetData& PartiallyFilteredAssetData = InOutAssetDataArray[Index];
			if (PartiallyFilteredAssetData.IsRedirector())
			{
				// Redirectors never have localization
				PackagesToFilter[Index] = true;
				return;
			}

			FString PackageFilePathWithoutExtension;
			if (!FPackageName::TryConvertLongPackageNameToFilename(PartiallyFilteredAssetData.PackageName.ToString(), PackageFilePathWithoutExtension))
			{
				// This means the asset data is for content that isn't mounted - this can happen when using a cooked asset registry
				PackagesToFilter[Index] = true;
				return;
			}

			FString PackageFilePathWithExtension;
			if (!FPackageName::FindPackageFileWithoutExtension(PackageFilePathWithoutExtension, PackageFilePathWithExtension))
			{
				// This means the package file doesn't exist on disk, which means we cannot gather it
				PackagesToFilter[Index] = true;
				return;
			}

			PackageFilePathWithExtension = FPaths::ConvertRelativePathToFull(PackageFilePathWithExtension);
			const FString PackageFileName = FPaths::GetCleanFilename(PackageFilePathWithExtension);

			// Filter out assets whose package file names DO NOT match any of the package file name filters.
			{
				bool bHasPassedAnyFileNameFilter = false;
				// This is an optimization to process package file filters in the form *.uasset or *.umap differently
				// FString::MatchesWildcard is an expensive call so we try and minimize the call to that and we go with FString::EndsWith instead for better performance
				for (const FString& PackageFileNameFilter : PackageFileFiltersStartingWithWildcard)
				{
					if (PackageFileName.EndsWith(PackageFileNameFilter))
					{
						bHasPassedAnyFileNameFilter = true;
						break;
					}
				}

				for (const FString& PackageFileNameFilter : OtherPackageFileFilters)
				{
					if (PackageFileName.MatchesWildcard(PackageFileNameFilter))
					{
						bHasPassedAnyFileNameFilter = true;
						break;
					}
				}
				if (!bHasPassedAnyFileNameFilter)
				{
					PackagesToFilter[Index] = true;
					return;
				}
			}

			// Filter out assets whose package file paths do not pass the "fuzzy path" filters.
			if (FuzzyPathMatcher.TestPath(PackageFilePathWithExtension) != FFuzzyPathMatcher::EPathMatch::Included)
			{
				PackagesToFilter[Index] = true;
				return;
			}
		}, UE::Private::GatherTextFromAssetsCommandlet::bParallelizeIncludeExcludePathFiltering ? EParallelForFlags::None: EParallelForFlags::ForceSingleThread);

	check(PackagesToFilter.Num() == InOutAssetDataArray.Num());
	for (int32 Index = InOutAssetDataArray.Num() - 1; Index >= 0; --Index)
	{
		if (PackagesToFilter[Index])
		{
			InOutAssetDataArray.RemoveAtSwap(Index, EAllowShrinking::No);
		}
	}
}

/** Remove any external packages that currently exist in InOutAssetDataArray. OutExternalActorsWorldPackageNames is populated with the package paths of worlds using external actors.*/
void UGatherTextFromAssetsCommandlet::RemoveExistingExternalPackages(TArray<FAssetData>& InOutAssetDataArray, const TSet<FName>* WorldPackageFilter, TSet<FName>& OutExternalActorsWorldPackageNames, TSet<FName>& OutGameFeatureDataPackageNames) const
{
	InOutAssetDataArray.RemoveAllSwap(
		[WorldPackageFilter, &OutExternalActorsWorldPackageNames, &OutGameFeatureDataPackageNames](const FAssetData& AssetData)
		{
			static const TSet<FTopLevelAssetPath> GameFeatureDataClassNames =
				[]()
				{
					TSet<FTopLevelAssetPath> GFDClassNames;
					if (const UClass* GFDClass = FindObject<UClass>(FTopLevelAssetPath("/Script/GameFeatures", "GameFeatureData"))) // cannot include UGameFeatureData as it is inside a plugin
					{
						TArray<UClass*> DerivedGFDClasses;
						GetDerivedClasses(GFDClass, DerivedGFDClasses);

						GFDClassNames.Add(GFDClass->GetClassPathName());
						Algo::Transform(DerivedGFDClasses, GFDClassNames, &UClass::GetClassPathName);
					}
					return GFDClassNames;
				}();

			static const FTopLevelAssetPath WorldClassName = UWorld::StaticClass()->GetClassPathName();

			if (AssetData.AssetClassPath == WorldClassName)
			{
				if (WorldPackageFilter && !WorldPackageFilter->Contains(AssetData.PackageName))
				{
					return true;
				}
				else if (ULevel::GetIsLevelUsingExternalActorsFromAsset(AssetData))
				{
					OutExternalActorsWorldPackageNames.Add(AssetData.PackageName);
				}
			}
			else if (!AssetData.GetOptionalOuterPathName().IsNone())
			{
				// Remove any external packages that are already in the list, as they will be re-added providing their outer package passed the gather criteria
				// It is possible for an external package to be directly specified for gather in the configs but have their outer package not pass the gather criteria.
				return true;
			}
			else if (GameFeatureDataClassNames.Contains(AssetData.AssetClassPath))
			{
				OutGameFeatureDataPackageNames.Add(AssetData.PackageName);
			}

			return false;
		}, EAllowShrinking::No);
}

/** Appends any external packages that also need to be gathered to the InOutAssetDataArray. */
bool UGatherTextFromAssetsCommandlet::DiscoverExternalPackages(TArray<FAssetData>& InOutAssetDataArray)
{
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Discovering external packages to gather...");
	const double DiscoveringExternalPackagesStartTime = FPlatformTime::Seconds();

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	// If we have WorldCollectionFilters specified, then use those to filter the external actors we gather
	TOptional<TSet<FName>> WorldPackageFilter;
	if (WorldCollectionFilters.Num() > 0)
	{
		FARFilter Filter;
		if (!BuildCollectionFilter(Filter, WorldCollectionFilters))
		{
			return false;
		}
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.WithoutPackageFlags = PKG_Cooked;

		TArray<FAssetData> FilteredWorldAssets;
		AssetRegistry.GetAssets(Filter, FilteredWorldAssets);

		TSet<FName> FilteredWorldPackages;
		FilteredWorldPackages.Reserve(FilteredWorldAssets.Num());
		Algo::Transform(FilteredWorldAssets, FilteredWorldPackages, &FAssetData::PackageName);
		WorldPackageFilter.Emplace(MoveTemp(FilteredWorldPackages));
	}

	TSet<FName> ExternalActorsWorldPackageNames;
	TSet<FName> GameFeatureDataPackageNames;
	RemoveExistingExternalPackages(InOutAssetDataArray, WorldPackageFilter.GetPtrOrNull(), ExternalActorsWorldPackageNames, GameFeatureDataPackageNames);

	// Discover external packages by their outer path
	{
		TArray<FAssetData> ExternalAssets;
		UE::FMutex ExternalAssetsMutex;
		ParallelFor(TEXT("UGatherTextFromAssetsCommandlet::DiscoverExternalPackages"), InOutAssetDataArray.Num(), 1,
			[&InOutAssetDataArray, &ExternalAssets, &ExternalAssetsMutex](int32 Index)
			{
				const FAssetData& AssetData = InOutAssetDataArray[Index];

				if (TArray<FAssetData> AssetsWithPackageAsOuter = UAssetRegistryHelpers::GetAssetsWithOuterForPaths({ AssetData.PackageName }, AssetData.PackageName, EAssetsWithOuterForPathsFlags::RecursivePaths | EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset);
					!AssetsWithPackageAsOuter.IsEmpty())
				{
					UE::TScopeLock _(ExternalAssetsMutex);
					ExternalAssets.Append(MoveTemp(AssetsWithPackageAsOuter));
				}
			});
		InOutAssetDataArray.Append(MoveTemp(ExternalAssets));
	}

	// Append the actors that are directly known by each world (by looking for their external actors under the expected path)
	if (ExternalActorsWorldPackageNames.Num() > 0)
	{
		// Note: This doesn't add AActor to ClassPaths as that doesn't work correctly doing a partial asset scan (see bSearchAllAssets)
		FARFilter Filter;
		for (const FName ExternalActorsWorldPackageName : ExternalActorsWorldPackageNames)
		{
			FNameBuilder PackageNameStr(ExternalActorsWorldPackageName);
			Filter.PackagePaths.Add(*ULevel::GetExternalActorsPath(*PackageNameStr));
		}
		Filter.bRecursivePaths = true;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.WithoutPackageFlags = PKG_Cooked;
		
		TArray<FAssetData> PotentialExternalActors;
		AssetRegistry.GetAssets(Filter, PotentialExternalActors);
		for (FAssetData& PotentialExternalActor : PotentialExternalActors)
		{
			if (FWorldPartitionActorDescUtils::IsValidActorDescriptorFromAssetData(PotentialExternalActor))
			{
				InOutAssetDataArray.Add(MoveTemp(PotentialExternalActor));
			}
		}
	}

	// Append additional actors for each world
	// These are actors added via ExternalDataLayers or ContentBundles, and will be referenced by the GameFeatureData assets that add them (via a GameFeatureAction)
	if (GameFeatureDataPackageNames.Num() > 0)
	{
		TArray<FAssetData> GameFeatureDataDependencies;
		{
			// Note: This doesn't add AActor to ClassPaths as that doesn't work correctly doing a partial asset scan (see bSearchAllAssets)
			FARFilter Filter;
			for (const FName GameFeatureDataPackageName : GameFeatureDataPackageNames)
			{
				AssetRegistry.GetDependencies(GameFeatureDataPackageName, Filter.PackageNames);
			}
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.WithoutPackageFlags = PKG_Cooked;

			if (Filter.PackageNames.Num() > 0)
			{
				AssetRegistry.GetAssets(Filter, GameFeatureDataDependencies);
			}
		}

		// External actors may be filtered in two ways;
		//  1. If WorldCollectionFilters were provided, then we only include actors related to those worlds
		//  2. If ExternalActorsWorldPackageNames was populated (meaning there are worlds in this gather), then we only include actors related to those worlds
		// If neither of the above is true then we include all actors related to the GFDs in this gather, as we assume this is a plugin hosting external actors for worlds owned by another localization target
		const bool bHasWorldFilter = WorldPackageFilter.IsSet() || ExternalActorsWorldPackageNames.Num() > 0;
		const TSet<FName>& GameFeatureDataActorsWorldPackageFilter = WorldPackageFilter.Get(ExternalActorsWorldPackageNames);
		for (FAssetData& GameFeatureDataDependency : GameFeatureDataDependencies)
		{
			const FName OptionalOuterPathName = GameFeatureDataDependency.GetOptionalOuterPathName();
			if (!OptionalOuterPathName.IsNone() && FWorldPartitionActorDescUtils::IsValidActorDescriptorFromAssetData(GameFeatureDataDependency))
			{
				const FName OptionalOuterPackageName = FName(FPackageName::ObjectPathToPackageName(FNameBuilder(OptionalOuterPathName).ToView()));
				if (!bHasWorldFilter || GameFeatureDataActorsWorldPackageFilter.Contains(OptionalOuterPackageName))
				{
					InOutAssetDataArray.Add(MoveTemp(GameFeatureDataDependency));
				}
				else
				{
					UE_LOGF(LogGatherTextFromAssetsCommandlet, VeryVerbose, "Skipping external actor package (%ls) as its associated world package (%ls) is not relevant to this gather.", *GameFeatureDataDependency.GetSoftObjectPath().ToString(), *OptionalOuterPackageName.ToString());
				}
			}
		}
	}

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Discovering external packages took %.2f seconds.", FPlatformTime::Seconds() - DiscoveringExternalPackagesStartTime);
	return true;
}

/** Applies the passed in filter to the asset registry. If the filter is empty, the entire asset registry will be returned in InOutAssetDataArray. Else assets that pass the filter will be in InOutAssetDataArray.*/
void UGatherTextFromAssetsCommandlet::ApplyFirstPassFilter(const FARFilter& InFilter, TArray<FAssetData>& InOutAssetDataArray) const
{
	// Apply filter if valid to do so, get all assets otherwise.
	if (InFilter.IsEmpty())
	{
		// @TODOLocalization: Logging that the first path filter is empty resulting in all assets being gathered can confuse users who generally rely on the second pass.
		// Figure out a good way to still convey the information in a log or clog.
		const double GetAllAssetsStartTime = FPlatformTime::Seconds();
		IAssetRegistry::GetChecked().GetAllAssets(InOutAssetDataArray);
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Getting all assets from asset registry took %.2f seconds.", FPlatformTime::Seconds() - GetAllAssetsStartTime);
	}
	else
	{
		const double GetAllAssetsWithFirstPassFilterStartTime = FPlatformTime::Seconds();
		if (!IAssetRegistry::GetChecked().GetAssets(InFilter, InOutAssetDataArray))
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "The first pass asset registry filter was invalid. Falling back to getting all assets from the asset registry...");
			IAssetRegistry::GetChecked().GetAllAssets(InOutAssetDataArray);
		}
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Getting all assets with first pass filter from asset registry took %.2f seconds.", FPlatformTime::Seconds() - GetAllAssetsWithFirstPassFilterStartTime);
	}
}

/**
 * Builds and applies the first pass filter to the entire AssetRegistry. OutAssetDataArray will hold all the assets that pass the first pass filter.
 * See BuildFirstPassFilter and ApplyFirstPassFilter
 */
bool UGatherTextFromAssetsCommandlet::PerformFirstPassFilter(TArray<FAssetData>& OutAssetDataArray) const
{
	FARFilter FirstPassFilter;

	if (!BuildFirstPassFilter(FirstPassFilter))
	{
		return false;
	}
	ApplyFirstPassFilter(FirstPassFilter, OutAssetDataArray);
	return true;
}

void UGatherTextFromAssetsCommandlet::ApplyExcludeExactClassesFilter(const FARFilter& InFilter, TArray<FAssetData>& InOutAssetDataArray) const
{
	// NOTE: The filter applied is actually the inverse, due to API limitations, so the resultant set must be removed from the current set.
	TArray<FAssetData> AssetsToExclude = InOutAssetDataArray;
	IAssetRegistry::GetChecked().RunAssetsThroughFilter(AssetsToExclude, InFilter);
	InOutAssetDataArray.RemoveAllSwap([AssetsToExcludeSet = TSet<FAssetData>(AssetsToExclude)](const FAssetData& AssetData)
		{
			return AssetsToExcludeSet.Contains(AssetData);
		}, EAllowShrinking::No);
}

/**
 * Filters out assets from the exact specified classes . Assets that pass the filter will be in InOutAssetDataArray.
 * See BuildExactClassesFilter and ApplyExactClassesFilter
 */
bool UGatherTextFromAssetsCommandlet::PerformExcludeExactClassesFilter(TArray<FAssetData>& InOutAssetDataArray) const
{
	const double ExcludeDerivedClassesStartTime = FPlatformTime::Seconds();
	// Filter out any objects of the specified classes.
	FARFilter ExcludeExactClassesFilter;
	if (!BuildExcludeExactClassesFilter(ExcludeExactClassesFilter))
	{
		return false;
	}

	// Reapply filter over the current set of assets.
	if (!ExcludeExactClassesFilter.IsEmpty())
	{
		ApplyExcludeExactClassesFilter(ExcludeExactClassesFilter, InOutAssetDataArray);
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Filtering out derived classes took %.2f seconds.", FPlatformTime::Seconds() - ExcludeDerivedClassesStartTime);
	}
	return true;
}

bool UGatherTextFromAssetsCommandlet::ParseCommandLineHelper(const FString& InCommandLine)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*InCommandLine, Tokens, Switches, ParamVals);

	FString GatherTextConfigPath;
	FString SectionName;
	if (!GetConfigurationScript(ParamVals, GatherTextConfigPath, SectionName))
	{
		return false;
	}

	if (!ConfigureFromScript(GatherTextConfigPath, SectionName))
	{
		return false;
	}

	{
		FGatherTextContext Context;
		Context.CommandletClass = GetClass()->GetClassPathName();
		Context.PreferredPathType = FGatherTextContext::EPreferredPathType::Content;

		FGatherTextDelegates::GetAdditionalGatherPathsForContext.Broadcast(GatherManifestHelper->GetTargetName(), Context, IncludePathFilters, ExcludePathFilters);
	}

	// Get destination path
	if (!GetPathFromConfig(*SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath))
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "No destination path specified.",
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Add any manifest dependencies if they were provided
	{
		bool bHasFailedToAddManifestDependency = false;
		for (const FString& ManifestDependency : ManifestDependenciesList)
		{
			FText OutError;
			if (!GatherManifestHelper->AddDependency(ManifestDependency, &OutError))
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "The GatherTextFromAssets commandlet couldn't load the specified manifest dependency: '{manifestDependency}'. {error}",
					("manifestDependency", *ManifestDependency),
					("error", *OutError.ToString()),
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
				bHasFailedToAddManifestDependency = true;
			}
		}
		if (bHasFailedToAddManifestDependency)
		{
			return false;
		}
	}

	// Preload necessary modules.
	{
		bool bHasFailedToPreloadAnyModules = false;
		for (const FString& ModuleName : ModulesToPreload)
		{
			EModuleLoadResult ModuleLoadResult;
			FModuleManager::Get().LoadModuleWithFailureReason(*ModuleName, ModuleLoadResult);

			if (ModuleLoadResult != EModuleLoadResult::Success)
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to preload dependent module {module}. Please check if the modules have been renamed or moved to another folder.",
					("module", *ModuleName),
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
				bHasFailedToPreloadAnyModules = true;
				continue;
			}
		}

		if (bHasFailedToPreloadAnyModules)
		{
			return false;
		}
	}

	return true;
}

UGatherTextFromAssetsCommandlet::FPackagePendingGather* UGatherTextFromAssetsCommandlet::AppendPackagePendingGather(const FName PackageNameToGather)
{
	FString PackageFilename;
	if (!FPackageName::FindPackageFileWithoutExtension(FPackageName::LongPackageNameToFilename(PackageNameToGather.ToString()), PackageFilename))
	{
		return nullptr;
	}
	PackageFilename = FPaths::ConvertRelativePathToFull(PackageFilename);

	bool bIsMapPackage = false;
	{
		FStringView PackageExt = FPathViews::GetExtension(PackageFilename, /*bIncludeDot*/true);
		bIsMapPackage = PackageExt == FPackageName::GetMapPackageExtension() || PackageExt == FPackageName::GetTextMapPackageExtension();
	}

	FPackagePendingGather& PackagePendingGather = PackagesPendingGather.AddDefaulted_GetRef();
	PackagePendingGather.PackageName = PackageNameToGather;
	PackagePendingGather.PackageFilename = MoveTemp(PackageFilename);
	PackagePendingGather.bIsMapPackage = bIsMapPackage;
	PackagePendingGather.PackageLocCacheState = EPackageLocCacheState::Cached;
	return &PackagePendingGather;
}

void UGatherTextFromAssetsCommandlet::ExtractPackageNamesToGather(const TArray<FAssetData>& InAssetDataArray, TSet<FName>& OutPackageNames, TMap<FName, FName>& OutExternalToOuterPackages) const
{
	// Collapse the assets down to a set of packages
	OutPackageNames.Empty(InAssetDataArray.Num());
	for (const FAssetData& AssetData : InAssetDataArray)
	{
		OutPackageNames.Add(AssetData.PackageName);

		if (!AssetData.GetOptionalOuterPathName().IsNone())
		{
			const FName OuterPackageName = FName(FPackageName::ObjectPathToPackageName(FNameBuilder(AssetData.GetOptionalOuterPathName()).ToView()));
			OutExternalToOuterPackages.Add(AssetData.PackageName, OuterPackageName);
		}
	}
}

void UGatherTextFromAssetsCommandlet::PopulatePackagesPendingGather(TSet<FName> PackageNamesToGather)
{
	const double PopulationStartTime = FPlatformTime::Seconds();
	// Build the basic information for the packages to gather (dependencies are filled in later once we've processed cached packages)
	PackagesPendingGather.Reserve(PackageNamesToGather.Num());
	for (const FName& PackageNameToGather : PackageNamesToGather)
	{
		AppendPackagePendingGather(PackageNameToGather);
	}
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Populating pending packages took %.2f seconds.", FPlatformTime::Seconds() - PopulationStartTime);
}

/** Process packages with loc data cached in its header and removes them from the pending packages.*/
void UGatherTextFromAssetsCommandlet::ProcessAndRemoveCachedPackages(const TMap<FName, FName>& ExternalToOuterPackages, TMap<FName, TSet<FGuid>>& OutExternalActorsWithStaleOrMissingCaches, TMap<FName, TSet<FName>>& OutExternalPackagesWithStaleOrMissingCaches)
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::ProcessAndRemoveCachedPackages"), LogGatherTextFromAssetsCommandlet, Display);

	const int32 InitialNumPackagesToGather = PackagesPendingGather.Num();

	// Load any cached localization data in parallel to maximize file throughput
	// We do not update any shared state during this pass, and only update the data within FPackagePendingGather
	{
		const EParallelForFlags LoadingLoopFlags = UE::Private::GatherTextFromAssetsCommandlet::bParallelizeProcessAndRemoveCachedPackages ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
		const int32 LoadingLoopNumElements = InitialNumPackagesToGather;
		const int32 LoadingLoopMinBatchSize = UE::Private::GatherTextFromAssetsCommandlet::ProcessAndRemoveCachedPackagesMaxThreads <= 0
			? 1 // Batch size of 1 uses as many threads as ParallelFor allows
			: (LoadingLoopNumElements / FMath::Max(UE::Private::GatherTextFromAssetsCommandlet::ProcessAndRemoveCachedPackagesMaxThreads - 1, 1)) + 1; // -1 from MaxThreads as ParallelFor will include the game thread internally
		const int32 LoadingLoopNumThreads = ParallelForImpl::GetNumberOfThreadTasks(LoadingLoopNumElements, LoadingLoopMinBatchSize, LoadingLoopFlags);

		const double LoadStartTime = FPlatformTime::Seconds();
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Loading the cached localization data for %d package(s) using %d thread(s). This may take a while...", InitialNumPackagesToGather, LoadingLoopNumThreads);

		ParallelFor(TEXT("UGatherTextFromAssetsCommandlet::ProcessAndRemoveCachedPackages"), LoadingLoopNumElements, LoadingLoopMinBatchSize, [this](int32 Index)
		{
			FPackagePendingGather& PackagePendingGather = PackagesPendingGather[Index];

			const FNameBuilder PackageNameStr(PackagePendingGather.PackageName);
			const bool bIsExternalActorPackage = PackageNameStr.ToView().Contains(FPackagePath::GetExternalActorsFolderName());

			TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*PackagePendingGather.PackageFilename));
			if (!FileReader)
			{
				return;
			}

			// Read package file summary from the file.
			FPackageFileSummary PackageFileSummary;
			*FileReader << PackageFileSummary;
			FileReader->SetCustomVersions(PackageFileSummary.GetCustomVersionContainer());

			// Determine whether this package must be gathered locally (not by workers).
			// Metadata text from Blueprint/UserDefinedStruct/UserDefinedEnum is written directly to the manifest
			// via GatherManifestHelper->AddSourceText(), which is not available in worker processes.
			if (MetaDataHelper && MetaDataHelper->IsConfigured())
			{
				TArray<FAssetData> AllAssetDataInPackage;
				IAssetRegistry::GetChecked().GetAssetsByPackageName(PackagePendingGather.PackageName, AllAssetDataInPackage, /*bIncludeOnlyOnDiskAssets*/true);
				for (const FAssetData& AssetData : AllAssetDataInPackage)
				{
					if (AssetData.IsInstanceOf(UBlueprint::StaticClass()) ||
						AssetData.AssetClassPath == UUserDefinedStruct::StaticClass()->GetClassPathName() ||
						AssetData.AssetClassPath == UUserDefinedEnum::StaticClass()->GetClassPathName())
					{
						PackagePendingGather.bRequiresMetaDataGather = true;
						break;
					}
				}
			}

			PackagePendingGather.PackageLocalizationId = PackageFileSummary.LocalizationId;
			PackagePendingGather.PackageLocCacheState = CalculatePackageLocCacheState(PackageFileSummary, PackagePendingGather.PackageName, bIsExternalActorPackage, PackagePendingGather.bRequiresMetaDataGather);

			// Read the cached localization data
			if (PackagePendingGather.PackageLocCacheState == EPackageLocCacheState::Cached && PackageFileSummary.GatherableTextDataOffset > 0)
			{
				FileReader->Seek(PackageFileSummary.GatherableTextDataOffset);

				PackagePendingGather.GatherableTextDataArray.SetNum(PackageFileSummary.GatherableTextDataCount);
				for (int32 GatherableTextDataIndex = 0; GatherableTextDataIndex < PackageFileSummary.GatherableTextDataCount; ++GatherableTextDataIndex)
				{
					*FileReader << PackagePendingGather.GatherableTextDataArray[GatherableTextDataIndex];
				}
			}
		}, LoadingLoopFlags);

		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Loading the cached localization data for %d package(s) took %.2f seconds.", InitialNumPackagesToGather, FPlatformTime::Seconds() - LoadStartTime);
	}

	// Now that everything has been loaded (or not), run through the cached data, ingest it, update the metrics, and update PackagesPendingGather
	{
		const double IngestStartTime = FPlatformTime::Seconds();
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Ingesting the cached localization data for %d package(s). This may take a while...", InitialNumPackagesToGather);

		int32 NumPackagesProcessed = 0;
		TMap<FString, FName> AssignedPackageLocalizationIds;
		PackagesPendingGather.RemoveAllSwap([this, &ExternalToOuterPackages, &OutExternalActorsWithStaleOrMissingCaches, &OutExternalPackagesWithStaleOrMissingCaches, &AssignedPackageLocalizationIds, &NumPackagesProcessed, InitialNumPackagesToGather](const FPackagePendingGather& PackagePendingGather) -> bool
		{
			FNameBuilder PackageNameStr(PackagePendingGather.PackageName);
			const int32 CurrentPackageNum = ++NumPackagesProcessed;
			const float PercentageComplete = static_cast<float>(CurrentPackageNum) / static_cast<float>(InitialNumPackagesToGather) * 100.0f;

			// Track the package localization ID of this package (if known) and detect duplicates
			bool bThisPackageHasLocIdConflictToFix = false;
			if (!PackagePendingGather.PackageLocalizationId.IsEmpty())
			{
				// if this package's localization ID is a duplicate
				if (const FName* ExistingLongPackageName = AssignedPackageLocalizationIds.Find(PackagePendingGather.PackageLocalizationId))
				{
					// Use a simple heuristic to see if we should actually emit a warning (otherwise, in big projects this warning 
					// might get emitted when no actual problems will happen on the vast majority of duplicate package id).
					// Heuristic: Only emit the warning if the package's cache cannot be used by the heuristic OR if the package have non-empty localization cache
					if (PackagePendingGather.PackageLocCacheState != EPackageLocCacheState::Cached ||
						!PackagePendingGather.GatherableTextDataArray.IsEmpty())
					{
						UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Package '{file}' and '{conflictFile}' have the same localization ID ({locKey}). Please reset both of these to avoid conflicts (one will probably deterministically keep the ID and the other one should actually reset). In the Content Browser, Right-Click the assets -> Asset Localization -> Reset Localization ID.",
							("file", *FPackageName::LongPackageNameToFilename(*PackageNameStr, TEXT(".uasset"))),
							("conflictFile", *FPackageName::LongPackageNameToFilename(ExistingLongPackageName->ToString(), TEXT(".uasset"))),
							("locKey", *PackagePendingGather.PackageLocalizationId),
							("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier));
					}

					if (bFixPackageLocalizationIdConflict) 
					{
						bThisPackageHasLocIdConflictToFix = true;
						// add this package to the list of packages with a duplicate localization ID
						PackagesWithDuplicateLocalizationIds.Add(PackagePendingGather.PackageName);
						NumPackagesDupLocId++;
					}
				}
				else
				{
					// This package is not a duplicate so it is added to Assigned Package Localization IDs dictionary
					AssignedPackageLocalizationIds.Add(PackagePendingGather.PackageLocalizationId, PackagePendingGather.PackageName);
				}
			}

			if (PackagePendingGather.PackageLocCacheState != EPackageLocCacheState::Cached)
			{
				if (const FName* OuterPackage = ExternalToOuterPackages.Find(PackagePendingGather.PackageName))
				{
					// External packages must be gathered via their outer package rather than via a raw LoadPackage call
					// Remove them from PackagesToGather as they will be merged back in by MergeInExternalPackagesWithStaleOrMissingCaches
					if (PackageNameStr.ToView().Contains(FPackagePath::GetExternalActorsFolderName()))
					{
						TArray<FAssetData> ActorsInPackage;
						IAssetRegistry::GetChecked().GetAssetsByPackageName(PackagePendingGather.PackageName, ActorsInPackage, /*bIncludeOnlyOnDiskAssets*/true);
						for (const FAssetData& ActorInPackage : ActorsInPackage)
						{
							if (TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorInPackage))
							{
								FName WorldPackageName = *FPackageName::ObjectPathToPackageName(ActorDesc->GetActorSoftPath().ToString());
								ensureAlwaysMsgf(WorldPackageName == *OuterPackage, TEXT("Actor '%s' was expected to be part of '%s'"), *ActorDesc->GetActorSoftPath().ToString(), *OuterPackage->ToString());
								OutExternalActorsWithStaleOrMissingCaches.FindOrAdd(WorldPackageName).Add(ActorDesc->GetGuid());
							}
						}
					}
					else
					{
						OutExternalPackagesWithStaleOrMissingCaches.FindOrAdd(*OuterPackage).Add(PackagePendingGather.PackageName);
					}
					return true;
				}

				UE::Private::GatherTextFromAssetsCommandlet::AssetGatherCacheMetrics.CountUncachedAsset(PackagePendingGather.PackageLocCacheState);
				return false;
			}

			// Process packages that don't require loading to process.
			if (PackagePendingGather.GatherableTextDataArray.Num() > 0)
			{
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "[%6.2f%%] Gathering package: '%ls'...", PercentageComplete, *PackageNameStr);

				UE::Private::GatherTextFromAssetsCommandlet::AssetGatherCacheMetrics.CountCachedAsset();

				ProcessGatherableTextDataArray(PackagePendingGather.GatherableTextDataArray);
			}

			// If we're reporting or fixing assets with a stale gather cache then we still need to load this 
			// package in order to do that, but the PackageLocCacheState prevents it being gathered again
			if (bReportStaleGatherCache || bFixStaleGatherCache || bThisPackageHasLocIdConflictToFix)
			{
				check(PackagePendingGather.PackageLocCacheState == EPackageLocCacheState::Cached);
				return false;
			}

			return true;
		}, EAllowShrinking::No);

		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Ingesting the cached localization data for %d package(s) took %.2f seconds.", InitialNumPackagesToGather, FPlatformTime::Seconds() - IngestStartTime);
	}
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Number of packages with duplicate loc ids: %lld", NumPackagesDupLocId);
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Number of packages to load: %d", PackagesPendingGather.Num());
}

void UGatherTextFromAssetsCommandlet::MergeInExternalPackagesWithStaleOrMissingCaches(TMap<FName, TSet<FGuid>>& ExternalActorsWithStaleOrMissingCaches, TMap<FName, TSet<FName>>& ExternalPackagesWithStaleOrMissingCaches)
{
	auto FindOrAddPackagePendingGather = 
		[this](const FName PackageName) -> FPackagePendingGather*
		{
			FPackagePendingGather* PackagePendingGather = PackagesPendingGather.FindByPredicate(
				[PackageName](const FPackagePendingGather& PotentialPackagePendingGather)
				{
					return PotentialPackagePendingGather.PackageName == PackageName;
				});
			if (!PackagePendingGather)
			{
				PackagePendingGather = AppendPackagePendingGather(PackageName);
			}
			return PackagePendingGather;
		};

	for (TTuple<FName, TSet<FGuid>>& StaleExternalActorsPair : ExternalActorsWithStaleOrMissingCaches)
	{
		if (FPackagePendingGather* WorldPackagePendingGather = FindOrAddPackagePendingGather(StaleExternalActorsPair.Key))
		{
			WorldPackagePendingGather->ExternalActors = MoveTemp(StaleExternalActorsPair.Value);
			WorldPackagePendingGather->PackageLocCacheState = EPackageLocCacheState::Uncached_TooOld;
		}
		else
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to queue world package '{package}' for {nbExternalActors} external actor(s).",
				("package", *StaleExternalActorsPair.Key.ToString()),
				("nbExternalActors", StaleExternalActorsPair.Value.Num()),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	for (TTuple<FName, TSet<FName>>& StaleExternalPackagesPair : ExternalPackagesWithStaleOrMissingCaches)
	{
		if (FPackagePendingGather* WorldPackagePendingGather = FindOrAddPackagePendingGather(StaleExternalPackagesPair.Key))
		{
			WorldPackagePendingGather->ExternalPackages = MoveTemp(StaleExternalPackagesPair.Value);
			WorldPackagePendingGather->PackageLocCacheState = EPackageLocCacheState::Uncached_TooOld;
		}
		else
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to queue outer package '{package}' for {nbExternalPackages} external package(s).",
				("package", *StaleExternalPackagesPair.Key.ToString()),
				("nbExternalPackages", StaleExternalPackagesPair.Value.Num()),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
	}
}

/** Load the remaining pending packages for gather.*/
bool UGatherTextFromAssetsCommandlet::LoadAndProcessUncachedPackages(TArray<FName>& OutPackagesWithStaleGatherCache)
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::LoadAndProcessUncachedPackages"), LogGatherTextFromAssetsCommandlet, Display);
	TArray<FGatherableTextData> GatherableTextDataArray;
	int32 LocalNumPackagesProcessed = 0;
	int32 LocalNumPackagesFailedLoading = 0;
	int32 LocalPackageCount = PackagesPendingGather.Num();
	
	FGatherTextFromAssetsWorkerDirector& WorkerDirector = FGatherTextFromAssetsWorkerDirector::Get();

	FScopedSlowTask SlowTask(LocalPackageCount, NSLOCTEXT("GatherTextCommandlet", "LoadAndProcessUncachedPackages", "Loading and Gathering Packages..."));

	// Auxiliary lambda function for resaving packages
	auto ResavePackage = [this](
		const FStringView PackageNameStr,
		UPackage* Package,
		const FString& PackageFileName
		) -> bool
	{
		FLoadPackageLogOutputRedirector::FScopedCapture ScopedCapture(PackageNameStr);
		return FLocalizedAssetSCCUtil::SavePackageWithSCC(SourceControlInfo, Package, PackageFileName);
	};

	for (;;)
	{
		TOptional<FPackagePendingGather> MaybePackagePendingGather;
		{
			UE::TScopeLock _(PackagesPendingGatherMutex);

			// If we have (soon to be) idle workers, distribute some work to them now
			AssignPackagesToWorkers(WorkerDirector.GetAvailableWorkerIds(UE::Private::GatherTextFromAssetsCommandlet::WorkerIdleThreshold));

			if (PackagesPendingGather.IsEmpty())
			{
				break;
			}

			MaybePackagePendingGather = PackagesPendingGather.Pop(EAllowShrinking::No);
		}

		check(MaybePackagePendingGather);
		const FPackagePendingGather& PackagePendingGather = MaybePackagePendingGather.GetValue();
		FNameBuilder PackageNameStr(PackagePendingGather.PackageName);

		const int32 TotalNumUncachedPackagesProcessed = ++NumUncachedPackagesProcessedLocally + NumUncachedPackagesProcessedRemotely;
		const float PercentageComplete = static_cast<float>(TotalNumUncachedPackagesProcessed) / static_cast<float>(TotalNumUncachedPackages) * 100.0f;
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "[%6.2f%%] Loading package: '%ls'...", PercentageComplete, *PackageNameStr);

		++LocalNumPackagesProcessed;
		SlowTask.EnterProgressFrame();

		if (SlowTask.ShouldCancel() || (EmbeddedContext && EmbeddedContext->ShouldAbort()))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "GatherText aborted!",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		UPackage* Package = UE::Private::GatherTextFromAssetsCommandlet::LoadPackageToGather(*PackageNameStr, PackagePendingGather.bIsMapPackage);
		if (!Package)
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to load package: '{package}'.",
				("package", *PackageNameStr),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			++LocalNumPackagesFailedLoading;
			continue;
		}

		// if fixing duplicate localization package IDs mode is enabled, 
		// and the package is in the list of packages with duplicate localization IDs,
		// reset the package's localization ID
		bool bPackageLocIdWasReset = false;
#if USE_STABLE_LOCALIZATION_KEYS
		if (bFixPackageLocalizationIdConflict)
		{
			if (PackagesWithDuplicateLocalizationIds.Contains(PackagePendingGather.PackageName))
			{
				// This package's localization ID and name have been found in the duplicates dictionary, and the flag for fixing duplicates is on
				// So resetting the localization ID for that package
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "About to reset localization id for package: '%ls'.", *PackageNameStr);
				TextNamespaceUtil::ClearPackageNamespace(Package);
				TextNamespaceUtil::EnsurePackageNamespace(Package);
				bPackageLocIdWasReset = true;
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Localization ID has been reset for package: '%ls'.", *PackageNameStr);
				
				// Now resaving to ensure the in-memory IDs are updated in the package before the data gatherer runs over it
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Resaving package: '%ls'...", *PackageNameStr);
				if (!ResavePackage(PackageNameStr, Package, PackagePendingGather.PackageFilename))
				{
					UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to resave package: '{package}'.",
						("package", *PackageNameStr),
						("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
					);
				}
			}
		}
#endif

		// Tick background tasks
		UE::Private::GatherTextFromAssetsCommandlet::TickBackgroundTasks();
		if (EmbeddedContext)
		{
			EmbeddedContext->RunTick();
		}
		WorkerDirector.TickWorkers();

		if (PackagePendingGather.bRequiresMetaDataGather)
		{
			check(MetaDataHelper && MetaDataHelper->IsConfigured());

			// Gather metadata from BlueprintGeneratedClass, UserDefinedStruct, and UserDefinedEnum top-level objects in the package.
			// These are script-defined types whose field metadata (UPROPERTY display names etc) is not captured by
			// FPropertyLocalizationDataGatherer. The gathered text is written directly to the manifest.
			ForEachObjectWithOuter(Package, 
				[this](UObject* Object)
				{
					if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(Object))
					{
						if (!FNameBuilder(BlueprintClass->GetFName()).ToView().StartsWith(TEXTVIEW("SKEL_")))
						{
							MetaDataHelper->GatherTextFromField(BlueprintClass);
						}
					}
					else if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Object))
					{
						MetaDataHelper->GatherTextFromField(UserDefinedStruct);
					}
					else if (UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(Object))
					{
						MetaDataHelper->GatherTextFromField(UserDefinedEnum);
					}
				}, EGetObjectsFlags::None);
		}

		if (UE::Private::GatherTextFromAssetsCommandlet::GatherTextFromPackage(Package, PackagePendingGather.bIsMapPackage, PackagePendingGather.ExternalActors, PackagePendingGather.ExternalPackages, /*bGatherFromPrimaryPackage*/true, /*bGatherFromExternalPackages*/false, GatherableTextDataArray))
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "[%6.2f%%] Gathering package: '%ls'...", PercentageComplete, *PackageNameStr);

			bool bSavePackage = false;

			// Optionally check to see whether the clean gather we did is in-sync with the gather cache and deal with it accordingly
			if ((bReportStaleGatherCache || bFixStaleGatherCache) && PackagePendingGather.PackageLocCacheState == EPackageLocCacheState::Cached)
			{
				// Gathers from the given package
				TArray<FGatherableTextData> CurrentGatherableTextDataArray;
				EPropertyLocalizationGathererResultFlags CurrentGatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
				FPropertyLocalizationDataGatherer(CurrentGatherableTextDataArray, Package, CurrentGatherableTextResultFlags);

				// Look for any structurally significant changes (missing, added, or changed texts) in the cache
				// Ignore insignificant things (like source changes caused by assets moving or being renamed)
				if (EnumHasAnyFlags(CurrentGatherableTextResultFlags, EPropertyLocalizationGathererResultFlags::HasTextWithInvalidPackageLocalizationID)
					|| !IsGatherableTextDataIdentical(CurrentGatherableTextDataArray, PackagePendingGather.GatherableTextDataArray))
				{
					OutPackagesWithStaleGatherCache.Add(PackagePendingGather.PackageName);

					if (bFixStaleGatherCache)
					{
						bSavePackage = true;
					}
				}
			}

			// Optionally save the package if it is missing a gather cache
			if (bFixMissingGatherCache && PackagePendingGather.PackageLocCacheState == EPackageLocCacheState::Uncached_TooOld)
			{
				bSavePackage = true;
			}

			// if the package localization ID was reset in duplicate localization ID fixing mode, it needs to be resaved
			if(bPackageLocIdWasReset)
			{ 
				bSavePackage = true;
			}

			// Re-save the package to attempt to fix it?
			if (bSavePackage)
			{
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Resaving package: '%ls'...", *PackageNameStr);
				if (!ResavePackage(PackageNameStr, Package, PackagePendingGather.PackageFilename))
				{
					UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to resave package: '{package}'.",
						("package", *PackageNameStr),
						("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
					);
				}
			}

			// Include any external packages that were determined to be stale
			if (PackagePendingGather.ExternalActors.Num() > 0 || PackagePendingGather.ExternalPackages.Num() > 0)
			{
				UE::Private::GatherTextFromAssetsCommandlet::GatherTextFromPackage(Package, PackagePendingGather.bIsMapPackage, PackagePendingGather.ExternalActors, PackagePendingGather.ExternalPackages, /*bGatherFromPrimaryPackage*/false, /*bGatherFromExternalPackages*/true, GatherableTextDataArray);
			}

			// This package may have already been cached in cases where we're reporting or fixing assets with a stale gather cache
			// This check prevents it being gathered a second time
			if (PackagePendingGather.PackageLocCacheState != EPackageLocCacheState::Cached && GatherableTextDataArray.Num() > 0)
			{
				UE::TScopeLock _(GatherManifestHelperMutex);
				ProcessGatherableTextDataArray(GatherableTextDataArray);
			}

			GatherableTextDataArray.Reset();
		}
		else if (bPackageLocIdWasReset)
		{
			// Resaving a second time in case of package localization ID reset
			// to fix the on-disk cache to match the new in-memory IDs
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Resaving package: '%ls'...", *PackageNameStr);
			if (!ResavePackage(PackageNameStr, Package, PackagePendingGather.PackageFilename))
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to resave package: '{package}'.",
					("package", *PackageNameStr),
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
			}
		}

		{
			UE::TScopeLock _(PackagesPendingGatherMutex);
			UE::Private::GatherTextFromAssetsCommandlet::ConditionalPurgeGarbage<FPackagePendingGather>(MinFreeMemoryBytes, MaxUsedMemoryBytes, &PackagesPendingGather);
		}
	}

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Loaded %d packages. %d failed.", LocalNumPackagesProcessed, LocalNumPackagesFailedLoading);
	return true;
}

void UGatherTextFromAssetsCommandlet::ReportStaleGatherCache(TArray<FName>& InPackagesWithStaleGatherCache) const
{
	InPackagesWithStaleGatherCache.Sort(FNameLexicalLess());

	FString StaleGatherCacheReport;
	for (const FName& PackageWithStaleGatherCache : InPackagesWithStaleGatherCache)
	{
		StaleGatherCacheReport += PackageWithStaleGatherCache.ToString();
		StaleGatherCacheReport += TEXT("\n");
	}

	const FString StaleGatherCacheReportFilename = DestinationPath / TEXT("StaleGatherCacheReport.txt");
	const bool bStaleGatherCacheReportSaved = FLocalizedAssetSCCUtil::SaveFileWithSCC(SourceControlInfo, StaleGatherCacheReportFilename, [&StaleGatherCacheReport](const FString& InSaveFileName) -> bool
		{
			return FFileHelper::SaveStringToFile(StaleGatherCacheReport, *InSaveFileName, FFileHelper::EEncodingOptions::ForceUTF8);
		});

	if (!bStaleGatherCacheReportSaved)
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "Failed to save report: '{report}'.",
			("report", *StaleGatherCacheReportFilename),
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
	}
}

UGatherTextFromAssetsCommandlet::EPackageLocCacheState UGatherTextFromAssetsCommandlet::CalculatePackageLocCacheState(const FPackageFileSummary& PackageFileSummary, const FName PackageName, bool bIsExternalActorPackage, bool bRequiresMetaDataGather) const
{
	// Have we been asked to skip the cache of text that exists in the header of newer packages?
	if (bSkipGatherCache && PackageFileSummary.GetFileVersionUE() >= VER_UE4_SERIALIZE_TEXT_IN_PACKAGES)
	{
		// Fallback on the old package flag check.
		if (PackageFileSummary.GetPackageFlags() & PKG_RequiresLocalizationGather)
		{
			return EPackageLocCacheState::Uncached_NoCache;
		}
	}

	const FCustomVersion* const EditorVersion = PackageFileSummary.GetCustomVersionContainer().GetVersion(FEditorObjectVersion::GUID);
	const FCustomVersion* const FNMainVersion = PackageFileSummary.GetCustomVersionContainer().GetVersion(FFortniteMainBranchObjectVersion::GUID);

	// Packages not resaved since localization gathering flagging was added to packages must be loaded.
	if (PackageFileSummary.GetFileVersionUE() < VER_UE4_PACKAGE_REQUIRES_LOCALIZATION_GATHER_FLAGGING)
	{
		return EPackageLocCacheState::Uncached_TooOld;
	}
	// Package not resaved since gatherable text data was added to package headers must be loaded, since their package header won't contain pregathered text data.
	else if (PackageFileSummary.GetFileVersionUE() < VER_UE4_SERIALIZE_TEXT_IN_PACKAGES || (!EditorVersion || EditorVersion->Version < FEditorObjectVersion::GatheredTextEditorOnlyPackageLocId))
	{
		// Fallback on the old package flag check.
		if (PackageFileSummary.GetPackageFlags() & PKG_RequiresLocalizationGather)
		{
			return EPackageLocCacheState::Uncached_TooOld;
		}
	}
	else if (PackageFileSummary.GetFileVersionUE() < VER_UE4_DIALOGUE_WAVE_NAMESPACE_AND_CONTEXT_CHANGES)
	{
		TArray<FAssetData> AllAssetDataInSamePackage;
		IAssetRegistry::GetChecked().GetAssetsByPackageName(PackageName, AllAssetDataInSamePackage, /*bIncludeOnlyOnDiskAssets*/true);
		for (const FAssetData& AssetData : AllAssetDataInSamePackage)
		{
			if (AssetData.AssetClassPath == UDialogueWave::StaticClass()->GetClassPathName())
			{
				return EPackageLocCacheState::Uncached_TooOld;
			}
		}
	}
	else if (bIsExternalActorPackage && (!FNMainVersion || FNMainVersion->Version < FFortniteMainBranchObjectVersion::FixedLocalizationGatherForExternalActorPackage))
	{
		// Fallback on the old package flag check.
		if (PackageFileSummary.GetPackageFlags() & PKG_RequiresLocalizationGather)
		{
			return EPackageLocCacheState::Uncached_TooOld;
		}
	}

	// If metadata gathering is configured, Blueprint/UserDefinedStruct/UserDefinedEnum packages must be fully
	// loaded because their field metadata is not captured in the gather cache.
	if (bRequiresMetaDataGather)
	{
		check(MetaDataHelper && MetaDataHelper->IsConfigured());
		return EPackageLocCacheState::Uncached_NoCache;
	}

	// If this package doesn't have any cached data, then we have to load it for gather
	if (PackageFileSummary.GetFileVersionUE() >= VER_UE4_SERIALIZE_TEXT_IN_PACKAGES && PackageFileSummary.GatherableTextDataOffset == 0 && (PackageFileSummary.GetPackageFlags() & PKG_RequiresLocalizationGather))
	{
		return EPackageLocCacheState::Uncached_NoCache;
	}
	return EPackageLocCacheState::Cached;
}

int32 UGatherTextFromAssetsCommandlet::Main(const FString& Params)
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::Main"), LogGatherTextFromAssetsCommandlet, Display);
	// Parse command line.
	if (!ParseCommandLineHelper(Params))
	{
		return -1;
	}

	if (bStartWorkersImmediately)
	{
		StartWorkers(/*MinPackagesToUseWorkers*/0);
	}

	// If the editor has loaded a persistent world then create an empty world prior to starting the asset gather
	// This avoids any issues when loading and initializing worlds during the gather, as WP needs to re-initialize the world
	// Note: We can skip this when running embedded within a normal editor (ie, not a commandlet) as editor worlds are already fully initialized
	if (IsRunningCommandlet() && GEditor)
	{
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
			EditorWorld && !FPackageName::IsTempPackage(FNameBuilder(EditorWorld->GetPackage()->GetFName()).ToView()))
		{
			GEditor->CreateNewMapForEditing(/*bPromptForSave*/false);
		}
	}

	FARFilter FirstPassFilter;
	if (!BuildFirstPassFilter(FirstPassFilter))
	{
		return -1;
	}
	
	FGatherTextFromAssetsWorkerDirector& WorkerDirector = FGatherTextFromAssetsWorkerDirector::Get();

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Discovering assets to gather...");
	const double DiscoveringAssetsStartTime = FPlatformTime::Seconds();
	{
		UE_SCOPED_TIMER(TEXT("UGatherTextFromAssetsCommandlet::DiscoverAllAssetsForGatherFilter"), LogGatherTextFromAssetsCommandlet, Display);

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

		if (bSearchAllAssets || FirstPassFilter.PackagePaths.IsEmpty())
		{
			AssetRegistry.SearchAllAssets(true);

			// Workers are allowed to read the asset registry cache after we've performed a full asset scan
			WorkerDirector.SetWorkersCanReadAssetRegistryCache(true);
		}
		else
		{
			TArray<FString> ScanPaths;
			ScanPaths.Reserve(FirstPassFilter.PackagePaths.Num());
			Algo::Transform(FirstPassFilter.PackagePaths, ScanPaths, [](FName PackagePath) { return PackagePath.ToString(); });

			// Note: We don't use FirstPassFilter.SoftObjectPaths as the set of files to scan, as ScanSynchronous can perform poorly when given large numbers of files to scan
			AssetRegistry.ScanSynchronous(ScanPaths, TArray<FString>(), UE::AssetRegistry::EScanFlags::IgnoreInvalidPathWarning);
		}
	}

	TArray<FAssetData> AssetDataArray;
	ApplyFirstPassFilter(FirstPassFilter, AssetDataArray);

	if (!bShouldExcludeDerivedClasses)
	{
		if (!PerformExcludeExactClassesFilter(AssetDataArray))
		{
			return -1;
		}
	}

	// Note: AssetDataArray now contains all assets in the specified collections that are not instances of the specified excluded classes.
	FilterAssetsBasedOnIncludeExcludePaths(AssetDataArray);

	if (AssetDataArray.Num() == 0)
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "No assets matched the specified criteria.");
		return 0;
	}

	// Discover the external packages for any assets that are pending gather
	if (!DiscoverExternalPackages(AssetDataArray))
	{
		return -1;
	}

	// Collect the basic information about the packages that we're going to gather from
	TSet<FName> PackageNamesToGather;
	TMap<FName, FName> ExternalToOuterPackages;
	ExtractPackageNamesToGather(AssetDataArray, PackageNamesToGather, ExternalToOuterPackages);
	AssetDataArray.Empty();
	PopulatePackagesPendingGather(MoveTemp(PackageNamesToGather));

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Discovering assets to gather took %.2f seconds.", FPlatformTime::Seconds() - DiscoveringAssetsStartTime);

	// These are external actor packages that are stale or are missing a gather cache from their package
	// Map of world package name -> external actor Ids in the world 
	TMap<FName, TSet<FGuid>> ExternalActorsWithStaleOrMissingCaches;
	TMap<FName, TSet<FName>> ExternalPackagesWithStaleOrMissingCaches;
	// Process all packages that do not need to be loaded. Remove processed packages from the list.
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Processing assets to gather...");
	ProcessAndRemoveCachedPackages(ExternalToOuterPackages, ExternalActorsWithStaleOrMissingCaches, ExternalPackagesWithStaleOrMissingCaches);

	UE::Private::GatherTextFromAssetsCommandlet::AssetGatherCacheMetrics.LogMetrics();

	// Merge any pending external package requests back into PackagesPendingGather
	MergeInExternalPackagesWithStaleOrMissingCaches(ExternalActorsWithStaleOrMissingCaches, ExternalPackagesWithStaleOrMissingCaches);
	ExternalActorsWithStaleOrMissingCaches.Reset();

	// All packages left in PackagesPendingGather should now have to be loaded 
	if (PackagesPendingGather.Num() == 0)
	{
		// Nothing more to do!
		return 0;
	}

	TotalNumUncachedPackages = PackagesPendingGather.Num();

	const double PackageLoadingStartTime = FPlatformTime::Seconds();
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Preparing to load %d packages...", TotalNumUncachedPackages);

	StartWorkers(bStartWorkersAlways ? 0 : UE::Private::GatherTextFromAssetsCommandlet::MinPackagesToUseWorkers);

	CalculateDependenciesForPackagesPendingGather();

	// Collect garbage before beginning to load packages
	// This also sorts the list of packages into the best processing order
	UE::Private::GatherTextFromAssetsCommandlet::PurgeGarbage<FPackagePendingGather>(&PackagesPendingGather);

	// We don't need to have compiled shaders to gather text
	bool bWasShaderCompilationEnabled = false;
	if (GShaderCompilingManager)
	{
		bWasShaderCompilationEnabled = !GShaderCompilingManager->IsShaderCompilationSkipped();
		GShaderCompilingManager->SkipShaderCompilation(true);
	}

	WorkerDirector.SetIngestPackageResultHandler(
		[this](const FGatherTextFromAssetsWorkerMessage_PackageResult& PackageResult)
		{
			IngestPackageResultFromWorker(PackageResult, /*bSendWorkIfIdle*/true);
			return true;
		});

	FLoadPackageLogOutputRedirector::Get().Hook();

	TArray<FName> PackagesWithStaleGatherCache;
	for (;;)
	{
		// Process the main task work
		if (!LoadAndProcessUncachedPackages(PackagesWithStaleGatherCache))
		{
			return -1;
		}

		WorkerDirector.TickWorkers();

		// Ingest any results from workers
		while (TOptional<FGatherTextFromAssetsWorkerMessage_PackageResult> PackageResult = WorkerDirector.IngestPackageResult())
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Ingesting worker result for '%ls'...", *PackageResult->PackageName.ToString());
			IngestPackageResultFromWorker(*PackageResult);
		}

		// Re-add any jobs sent to crashed workers to the main task work queue
		{
			if (const TArray<FName> PackagesFromCrashedWorkers = WorkerDirector.IngestPackagesFromCrashedWorkers();
				PackagesFromCrashedWorkers.Num() > 0)
			{
				UE::TScopeLock _1(PackagesPendingGatherMutex);
				UE::TScopeLock _2(PackagesDistributedToWorkersMutex);
				for (const FName PackageName : PackagesFromCrashedWorkers)
				{
					if (FPackagePendingGather* PackagePendingGather = PackagesDistributedToWorkers.Find(PackageName))
					{
						PackagesPendingGather.Add(MoveTemp(*PackagePendingGather));
						PackagesDistributedToWorkers.Remove(PackageName);
					}
				}
			}
		}

		int32 NumPendingWorkerPackages = 0;
		{
			UE::TScopeLock _(PackagesPendingGatherMutex);
			if (PackagesPendingGather.Num() == 0 && WorkerDirector.IsIdle(&NumPendingWorkerPackages))
			{
				break;
			}
		}

		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Waiting for workers to finish (%d pending package(s))...", NumPendingWorkerPackages);
		FPlatformProcess::SleepNoStats(1.0f);
	}

	FLoadPackageLogOutputRedirector::Get().Unhook();

	WorkerDirector.ClearIngestPackageResultHandler();

	// Clear list of packages with duplicate localization IDs
	PackagesWithDuplicateLocalizationIds.Empty();

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Loading %d packages took %.2f seconds (%d local, %d remote).", TotalNumUncachedPackages, FPlatformTime::Seconds() - PackageLoadingStartTime, NumUncachedPackagesProcessedLocally.load(), NumUncachedPackagesProcessedRemotely.load());

	// Collect garbage after loading all packages
	// This reclaims as much memory as possible for the rest of the gather pipeline
	UE::Private::GatherTextFromAssetsCommandlet::PurgeGarbage<FPackagePendingGather>(nullptr);
	
	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->SkipShaderCompilation(!bWasShaderCompilationEnabled);
	}
	
	if (bReportStaleGatherCache)
	{
		ReportStaleGatherCache(PackagesWithStaleGatherCache);
	}

	return 0;
}

bool UGatherTextFromAssetsCommandlet::GetConfigurationScript(const TMap<FString, FString>& InCommandLineParameters, FString& OutFilePath, FString& OutStepSectionName)
{
	//Set config file
	const FString* ParamVal = InCommandLineParameters.Find(FString(TEXT("Config")));
	if (ParamVal)
	{
		OutFilePath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "No config specified.",
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	//Set config section
	ParamVal = InCommandLineParameters.Find(FString(TEXT("Section")));
	if (ParamVal)
	{
		OutStepSectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "No config section specified.",
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	return true;
}

bool UGatherTextFromAssetsCommandlet::ConfigureFromScript(const FString& GatherTextConfigPath, const FString& SectionName)
{
	bool bHasFatalError = false;

	// Modules to Preload
	GetStringArrayFromConfig(*SectionName, TEXT("ModulesToPreload"), ModulesToPreload, GatherTextConfigPath);

	// IncludePathFilters
	GetPathArrayFromConfig(*SectionName, TEXT("IncludePathFilters"), IncludePathFilters, GatherTextConfigPath);

	// IncludePaths (DEPRECATED)
	{
		TArray<FString> IncludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("IncludePaths"), IncludePaths, GatherTextConfigPath);
		if (IncludePaths.Num())
		{
			IncludePathFilters.Append(IncludePaths);
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "IncludePaths detected in section {section}. IncludePaths is deprecated, please use IncludePathFilters.",
				("section", *SectionName),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if (IncludePathFilters.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "No include path filters in section {section}.",
			("section", *SectionName),
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
		bHasFatalError = true;
	}

	// Collections
	{
		auto GetAndValidateCollections = [this, &bHasFatalError, &SectionName, &GatherTextConfigPath](const TCHAR* KeyName, TArray<FString>& OutCollections)
		{
			GetStringArrayFromConfig(*SectionName, KeyName, OutCollections, GatherTextConfigPath);
			if (OutCollections.Num() > 0)
			{
				FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();
				ICollectionManager& CollectionManager = CollectionManagerModule.Get();
				for (const FString& Collection : OutCollections)
				{
					TSharedPtr<ICollectionContainer> CollectionContainer;
					FName CollectionName;
					ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
					const bool bDoesCollectionExist = CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType) &&
						CollectionContainer->CollectionExists(CollectionName, ShareType);
					if (!bDoesCollectionExist)
					{
						UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "Failed to find collection '{collection}', collection does not exist.",
							("collection", *Collection),
							("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
						);
						bHasFatalError = true;
					}
				}
			}
		};

		GetAndValidateCollections(TEXT("CollectionFilters"), CollectionFilters);
		GetAndValidateCollections(TEXT("WorldCollectionFilters"), WorldCollectionFilters);
	}

	// ExcludePathFilters
	GetPathArrayFromConfig(*SectionName, TEXT("ExcludePathFilters"), ExcludePathFilters, GatherTextConfigPath);

	// ExcludePaths (DEPRECATED)
	{
		TArray<FString> ExcludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("ExcludePaths"), ExcludePaths, GatherTextConfigPath);
		if (ExcludePaths.Num())
		{
			ExcludePathFilters.Append(ExcludePaths);
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "ExcludePaths detected in section {section}. ExcludePaths is deprecated, please use ExcludePathFilters.",
				("section", *SectionName),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	// PackageNameFilters
	GetStringArrayFromConfig(*SectionName, TEXT("PackageFileNameFilters"), PackageFileNameFilters, GatherTextConfigPath);

	// PackageExtensions (DEPRECATED)
	{
		TArray<FString> PackageExtensions;
		GetStringArrayFromConfig(*SectionName, TEXT("PackageExtensions"), PackageExtensions, GatherTextConfigPath);
		if (PackageExtensions.Num())
		{
			PackageFileNameFilters.Append(PackageExtensions);
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "PackageExtensions detected in section {section}. PackageExtensions is deprecated, please use PackageFileNameFilters.",
				("section", *SectionName),
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if (PackageFileNameFilters.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "No package file name filters in section {section}.",
			("section", *SectionName),
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
		bHasFatalError = true;
	}

	// Recursive asset class exclusion
	if (!GetBoolFromConfig(*SectionName, TEXT("ShouldExcludeDerivedClasses"), bShouldExcludeDerivedClasses, GatherTextConfigPath))
	{
		bShouldExcludeDerivedClasses = false;
	}

	// Asset class exclude
	GetStringArrayFromConfig(*SectionName, TEXT("ExcludeClasses"), ExcludeClassNames, GatherTextConfigPath);

	GetPathArrayFromConfig(*SectionName, TEXT("ManifestDependencies"), ManifestDependenciesList, GatherTextConfigPath);

	if (!GetBoolFromConfig(*SectionName, TEXT("SearchAllAssets"), bSearchAllAssets, GatherTextConfigPath))
	{
		bSearchAllAssets = true;
	}

	if (!GetBoolFromConfig(*SectionName, TEXT("ApplyRedirectorsToCollections"), bApplyRedirectorsToCollections, GatherTextConfigPath))
	{
		bApplyRedirectorsToCollections = true;
	}

	// Get whether we should gather editor-only data. Typically only useful for the localization of UE itself.
	if (!GetBoolFromConfig(*SectionName, TEXT("ShouldGatherFromEditorOnlyData"), bShouldGatherFromEditorOnlyData, GatherTextConfigPath))
	{
		bShouldGatherFromEditorOnlyData = false;
	}

	auto ReadBoolFlagWithFallback = [this, &SectionName, &GatherTextConfigPath](const TCHAR* FlagName, bool& OutValue)
	{
		OutValue = FParse::Param(FCommandLine::Get(), FlagName);
		if (!OutValue)
		{
			GetBoolFromConfig(*SectionName, FlagName, OutValue, GatherTextConfigPath);
		}
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "%ls: %ls", FlagName, OutValue ? TEXT("true") : TEXT("false"));
	};

	ReadBoolFlagWithFallback(TEXT("SkipGatherCache"), bSkipGatherCache);
	ReadBoolFlagWithFallback(TEXT("ReportStaleGatherCache"), bReportStaleGatherCache);
	ReadBoolFlagWithFallback(TEXT("FixStaleGatherCache"), bFixStaleGatherCache);
	ReadBoolFlagWithFallback(TEXT("FixMissingGatherCache"), bFixMissingGatherCache);
	ReadBoolFlagWithFallback(TEXT("FixPackageLocalizationIdConflict"), bFixPackageLocalizationIdConflict);

	{
		bStartWorkersImmediately = FParse::Param(FCommandLine::Get(), TEXT("StartGatherTextWorkersImmediately"));
		bStartWorkersAlways = FParse::Param(FCommandLine::Get(), TEXT("StartGatherTextWorkersAlways"));

		NumWorkers = 0;
		FParse::Value(FCommandLine::Get(), TEXT("NumGatherTextWorkers="), NumWorkers);
		NumWorkers = FMath::Max(0, NumWorkers);

		if (NumWorkers > 0 && (bReportStaleGatherCache || bFixStaleGatherCache || bFixMissingGatherCache || bFixPackageLocalizationIdConflict))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Workers cannot be used when running in report/fix stale/missing gather cache mode. The request to use workers will be ignored.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			NumWorkers = 0;
		}

		// TODO: We should support this case by sending the ModulesToPreload to each worker when we start them
		if (NumWorkers > 0 && ModulesToPreload.Num() > 0)
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Workers cannot be used when module preloading is needed. The request to use workers will be ignored.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			NumWorkers = 0;
		}

		if (NumWorkers > 0 && !FParse::Param(FCommandLine::Get(), TEXT("Messaging")))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Workers cannot be used without MessageBus (did you forget to pass '-Messaging'?). The request to use workers will be ignored.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			NumWorkers = 0;
		}
	}

	// Read some settings from the editor config
	{
		int32 MinFreeMemoryMB = 0;
		GConfig->GetInt(TEXT("GatherTextFromAssets"), TEXT("MinFreeMemory"), MinFreeMemoryMB, GEditorIni);
		MinFreeMemoryMB = FMath::Max(MinFreeMemoryMB, 0);
		MinFreeMemoryBytes = MinFreeMemoryMB * 1024LL * 1024LL;

		int32 MaxUsedMemoryMB = 0;
		if (GConfig->GetInt(TEXT("GatherTextFromAssets"), TEXT("MaxMemoryAllowance"), MaxUsedMemoryMB, GEditorIni))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "The MaxMemoryAllowance config option is deprecated, please use MaxUsedMemory.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
		}
		else
		{
			GConfig->GetInt(TEXT("GatherTextFromAssets"), TEXT("MaxUsedMemory"), MaxUsedMemoryMB, GEditorIni);
		}
		MaxUsedMemoryMB = FMath::Max(MaxUsedMemoryMB, 0);
		MaxUsedMemoryBytes = MaxUsedMemoryMB * 1024LL * 1024LL;
	}

	// MetaData gathering configuration (InputKeys/OutputNamespaces/OutputKeys and field type filters)
	// Only active when InputKeys is non-empty. Metadata is gathered from Blueprint/UserDefinedStruct/UserDefinedEnum assets.
	{
		TArray<FString> InputKeys;
		TArray<FString> OutputNamespaces;
		TArray<FString> OutputKeys;
		GetStringArrayFromConfig(*SectionName, TEXT("MetadataInputKeys"), InputKeys, GatherTextConfigPath);
		GetStringArrayFromConfig(*SectionName, TEXT("MetadataOutputNamespaces"), OutputNamespaces, GatherTextConfigPath);
		GetStringArrayFromConfig(*SectionName, TEXT("MetadataOutputKeys"), OutputKeys, GatherTextConfigPath);

		if (InputKeys.Num() > 0)
		{
			MetaDataHelper = MakePimpl<FGatherTextMetaDataHelper>(GatherManifestHelper);
			MetaDataHelper->SetGatherParameters(MoveTemp(InputKeys), MoveTemp(OutputNamespaces), MoveTemp(OutputKeys));

			MetaDataHelper->SetShouldGatherFromEditorOnlyData(bShouldGatherFromEditorOnlyData);

			TArray<FString> FieldTypesToInclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldTypesToInclude"), FieldTypesToInclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldTypesFromStrings(FieldTypesToInclude, /*bInclude*/true, TEXT("MetadataFieldTypesToInclude"));

			TArray<FString> FieldTypesToExclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldTypesToExclude"), FieldTypesToExclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldTypesFromStrings(FieldTypesToExclude, /*bInclude*/false, TEXT("MetadataFieldTypesToExclude"));

			TArray<FString> FieldOwnerTypesToInclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldOwnerTypesToInclude"), FieldOwnerTypesToInclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldOwnerTypesFromStrings(FieldOwnerTypesToInclude, /*bInclude*/true, TEXT("MetadataFieldOwnerTypesToInclude"));

			TArray<FString> FieldOwnerTypesToExclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldOwnerTypesToExclude"), FieldOwnerTypesToExclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldOwnerTypesFromStrings(FieldOwnerTypesToExclude, /*bInclude*/false, TEXT("MetadataFieldOwnerTypesToExclude"));

			TArray<FString> FieldOuterTypesToInclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldOuterTypesToInclude"), FieldOuterTypesToInclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldOuterTypesFromStrings(FieldOuterTypesToInclude, /*bInclude*/true, TEXT("MetadataFieldOuterTypesToInclude"));

			TArray<FString> FieldOuterTypesToExclude;
			GetStringArrayFromConfig(*SectionName, TEXT("MetadataFieldOuterTypesToExclude"), FieldOuterTypesToExclude, GatherTextConfigPath);
			MetaDataHelper->SetFieldOuterTypesFromStrings(FieldOuterTypesToExclude, /*bInclude*/false, TEXT("MetadataFieldOuterTypesToExclude"));

			if (NumWorkers > 0)
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Display, "Workers will not be used for Blueprint/UserDefinedStruct/UserDefinedEnum packages when metadata gathering (InputKeys) is configured, as metadata text must be gathered in the main process.",
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
			}
		}
	}

	return !bHasFatalError;
}

void UGatherTextFromAssetsCommandlet::StartWorkers(const int32 MinPackagesToUseWorkers)
{
	FGatherTextFromAssetsWorkerDirector& WorkerDirector = FGatherTextFromAssetsWorkerDirector::Get();

	int32 NumWorkersToUse = 0;
	if (PackagesPendingGather.Num() >= MinPackagesToUseWorkers)
	{
		NumWorkersToUse = NumWorkers;
	}
	WorkerDirector.StartWorkers(NumWorkersToUse, /*bStopAdditionalWorkers*/false, /*NumRestartAttemptsIfCrashed*/4);
}

void UGatherTextFromAssetsCommandlet::AssignPackagesToWorkers(TConstArrayView<FGuid> IdleWorkerIds)
{
	if (IdleWorkerIds.Num() > 0)
	{
		if (PackagesPendingGather.Num() > UE::Private::GatherTextFromAssetsCommandlet::MinPackagesToKeepLocal)
		{
			if (const int32 NumPackagesToDistributeToEachWorker = FMath::Clamp((PackagesPendingGather.Num() - UE::Private::GatherTextFromAssetsCommandlet::MinPackagesToKeepLocal) / IdleWorkerIds.Num(), 0, UE::Private::GatherTextFromAssetsCommandlet::MaxPackagesToDistribute);
				NumPackagesToDistributeToEachWorker > 0)
			{
				FGatherTextFromAssetsWorkerDirector& WorkerDirector = FGatherTextFromAssetsWorkerDirector::Get();

				for (const FGuid& WorkerId : IdleWorkerIds)
				{
					check(PackagesPendingGather.Num() >= NumPackagesToDistributeToEachWorker);

					// We assign worker packages from the tail of the queue (the front of the array)
					// as these packages have the fewest number of dependencies so are the simplest
					// to distribute and load, which helps to balance the memory usage between
					// the main process (which takes the heavy tasks) and the worker processes
					TArray<FPackagePendingGather> PackagesToReinsertAtFront; // Packages that could not be assigned this tick (retry next tick)
					TArray<FPackagePendingGather> PackagesToReinsertAtBack;  // Packages requiring local processing (moved to back to avoid blocking workers)
					for (int32 Index = 0; Index < NumPackagesToDistributeToEachWorker; ++Index)
					{
						FPackagePendingGather PackagePendingGather = MoveTemp(PackagesPendingGather[Index]);

						// Packages that require local metadata gathering cannot be distributed to workers,
						// because metadata text is written directly to the manifest (not via FGatherableTextData).
						// Move them to the back so they don't block worker distribution on future ticks.
						if (PackagePendingGather.bRequiresMetaDataGather)
						{
							PackagesToReinsertAtBack.Add(MoveTemp(PackagePendingGather));
							continue;
						}

						FGatherTextFromAssetsWorkerMessage_PackageRequest PackageRequest;
						PackageRequest.PackageName = PackagePendingGather.PackageName;
						PackageRequest.Dependencies = PackagePendingGather.Dependencies;
						PackageRequest.ExternalActors = PackagePendingGather.ExternalActors;
						PackageRequest.ExternalPackages = PackagePendingGather.ExternalPackages;
						PackageRequest.bIsMapPackage = PackagePendingGather.bIsMapPackage;

						if (WorkerDirector.AssignPackageToWorker(WorkerId, PackageRequest))
						{
							UE::TScopeLock _(PackagesDistributedToWorkersMutex);

							const FName PackageName = PackagePendingGather.PackageName;
							PackagesDistributedToWorkers.Add(PackageName, MoveTemp(PackagePendingGather));
						}
						else
						{
							PackagesToReinsertAtFront.Add(MoveTemp(PackagePendingGather));
						}
					}
					PackagesPendingGather.RemoveAt(0, NumPackagesToDistributeToEachWorker, EAllowShrinking::No);
					if (PackagesToReinsertAtFront.Num() > 0)
					{
						PackagesPendingGather.Insert(MoveTemp(PackagesToReinsertAtFront), 0);
					}
					if (PackagesToReinsertAtBack.Num() > 0)
					{
						PackagesPendingGather.Append(MoveTemp(PackagesToReinsertAtBack));
					}
				}
			}
		}
	}
}

void UGatherTextFromAssetsCommandlet::IngestPackageResultFromWorker(const FGatherTextFromAssetsWorkerMessage_PackageResult& PackageResult, const bool bSendWorkIfIdle)
{
	const int32 TotalNumUncachedPackagesProcessed = NumUncachedPackagesProcessedLocally + ++NumUncachedPackagesProcessedRemotely;
	const float PercentageComplete = static_cast<float>(TotalNumUncachedPackagesProcessed) / static_cast<float>(TotalNumUncachedPackages) * 100.0f;
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "[%6.2f%%] Ingesting result for package: '%ls' (from worker: %ls)...", PercentageComplete, *PackageResult.PackageName.ToString(), *PackageResult.WorkerId.ToString());

	if (!PackageResult.LoadLogCapture.IsEmpty())
	{
		GWarn->Log(NAME_None, ELogVerbosity::Display, PackageResult.LoadLogCapture);
	}

	if (PackageResult.bLoadError)
	{
		UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to load package: '{package}'.",
			("package", PackageResult.PackageName),
			("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
		);
	}
	else
	{
		FMemoryReader MemoryReader(PackageResult.GatherableTextData);

		TArray<FGatherableTextData> GatherableTextData;
		MemoryReader << GatherableTextData;

		if (GatherableTextData.Num() > 0)
		{
			UE::TScopeLock _(GatherManifestHelperMutex);
			ProcessGatherableTextDataArray(GatherableTextData);
		}
	}

	{
		UE::TScopeLock _(PackagesDistributedToWorkersMutex);
		PackagesDistributedToWorkers.Remove(PackageResult.PackageName);
	}

	if (bSendWorkIfIdle)
	{
		UE::TScopeLock _(PackagesPendingGatherMutex);

		// If we are (soon to be) idle, distribute some work now
		if (TArray<FGuid> IdleWorkerIds = FGatherTextFromAssetsWorkerDirector::Get().GetAvailableWorkerIds(UE::Private::GatherTextFromAssetsCommandlet::WorkerIdleThreshold);
			IdleWorkerIds.Contains(PackageResult.WorkerId))
		{
			AssignPackagesToWorkers(MakeArrayView(&PackageResult.WorkerId, 1));
		}
	}
}

int32 UGatherTextFromAssetsWorkerCommandlet::Main(const FString& Params)
{
	FProcHandle DirectorProc;

	// Parse config data
	{
		TArray<FString> Tokens;
		TArray<FString> Switches;
		TMap<FString, FString> ParamVals;
		UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

		{
			uint32 DirectorPid = 0;
			if (const FString* DirectorPidStr = ParamVals.Find(TEXT("GatherDirectorPid")))
			{
				LexFromString(DirectorPid, **DirectorPidStr);
			}
			if (DirectorPid == 0)
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "No -GatherDirectorPid argument was provided. This process will continue to run even if the director process stops.",
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				DirectorProc = FPlatformProcess::OpenProcess(DirectorPid);
			}
			if (!DirectorProc.IsValid())
			{
				UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Warning, "Failed to open a handle for the director process. This process will continue to run even if the director process stops.",
					("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
				);
			}
		}

		if (const FString* WorkerIdStr = ParamVals.Find(TEXT("GatherWorkerId")))
		{
			FGuid::Parse(*WorkerIdStr, WorkerId);
		}
		if (!WorkerId.IsValid())
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "An invalid -GatherWorkerId argument was provided. Aborting.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}

		{
			int32 MinFreeMemoryMB = 0;
			if (!GConfig->GetInt(TEXT("GatherTextFromAssetsWorker"), TEXT("MinFreeMemory"), MinFreeMemoryMB, GEditorIni))
			{
				GConfig->GetInt(TEXT("GatherTextFromAssets"), TEXT("MinFreeMemory"), MinFreeMemoryMB, GEditorIni);
			}
			MinFreeMemoryMB = FMath::Max(MinFreeMemoryMB, 0);
			MinFreeMemoryBytes = MinFreeMemoryMB * 1024LL * 1024LL;
		}

		{
			int32 MaxUsedMemoryMB = 0;
			if (!GConfig->GetInt(TEXT("GatherTextFromAssetsWorker"), TEXT("MaxUsedMemory"), MaxUsedMemoryMB, GEditorIni))
			{
				GConfig->GetInt(TEXT("GatherTextFromAssets"), TEXT("MaxUsedMemory"), MaxUsedMemoryMB, GEditorIni);
			}
			MaxUsedMemoryMB = FMath::Max(MaxUsedMemoryMB, 0);
			MaxUsedMemoryBytes = MaxUsedMemoryMB * 1024LL * 1024LL;
		}
	}

	// Set-up MessageBus
	{
		if (!FParse::Param(FCommandLine::Get(), TEXT("Messaging")))
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "MessageBus is not enabled for the worker process (did you forget to pass '-Messaging'?). Aborting.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}

		MessageEndpoint = FMessageEndpoint::Builder(*(TEXT("GatherTextFromAssetsWorker") + WorkerId.ToString()))
			.ReceivingOnAnyThread()
			.Handling<FGatherTextFromAssetsWorkerMessage_Ping>(this, &UGatherTextFromAssetsWorkerCommandlet::HandlePingMessage)
			.Handling<FGatherTextFromAssetsWorkerMessage_PackageRequest>(this, &UGatherTextFromAssetsWorkerCommandlet::HandlePackageRequestMessage);
		if (!MessageEndpoint)
		{
			UE_LOGFMT(LogGatherTextFromAssetsCommandlet, Error, "Failed to create MessageEndpoint for the worker process. Aborting.",
				("id", GatherTextFromAssetsCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
		MessageEndpoint->Subscribe<FGatherTextFromAssetsWorkerMessage_Ping>();
	}

	// If the editor has loaded a persistent world then create an empty world prior to starting the asset gather
	// This avoids any issues when loading and initializing worlds during the gather, as WP needs to re-initialize the world
	if (GEditor)
	{
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
			EditorWorld && !FPackageName::IsTempPackage(FNameBuilder(EditorWorld->GetPackage()->GetFName()).ToView()))
		{
			GEditor->CreateNewMapForEditing(/*bPromptForSave*/false);
		}
	}

	// Run a GC now that we're ready to start accepting work, to clean-up anything loaded during start-up that is no longer needed
	UE::Private::GatherTextFromAssetsCommandlet::PurgeGarbage<FPackagePendingGather>(nullptr);

	// We don't need to have compiled shaders to gather text
	bool bWasShaderCompilationEnabled = false;
	if (GShaderCompilingManager)
	{
		bWasShaderCompilationEnabled = !GShaderCompilingManager->IsShaderCompilationSkipped();
		GShaderCompilingManager->SkipShaderCompilation(true);
	}

	FStringOutputDevice LogOutputTargetDevice;
	FLoadPackageLogOutputRedirector::Get().Hook(&LogOutputTargetDevice);

	// Wait for work, and process each package requested
	for (;;)
	{
		TOptional<FPackagePendingGather> PackagePendingGather;
		{
			UE::TScopeLock _(PackagesPendingGatherMutex);
			if (PackagesPendingGather.Num() > 0)
			{
				PackagePendingGather = PackagesPendingGather.Pop(EAllowShrinking::No);
			}
		}

		if (PackagePendingGather)
		{
			{
				UE::TScopeLock _(IdleStartTimeUtcMutex);
				IdleStartTimeUtc.Reset();
			}

			const FString PackageNameStr = PackagePendingGather->PackageName.ToString();
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Gathering package: '%ls'.", *PackageNameStr);

			FGatherTextFromAssetsWorkerMessage_PackageResult* PackageResultMessage = FMessageEndpoint::MakeMessage<FGatherTextFromAssetsWorkerMessage_PackageResult>();
			PackageResultMessage->WorkerId = WorkerId;
			PackageResultMessage->PackageName = PackagePendingGather->PackageName;

			LogOutputTargetDevice.Reset();
			if (UPackage* Package = UE::Private::GatherTextFromAssetsCommandlet::LoadPackageToGather(PackageNameStr, PackagePendingGather->bIsMapPackage))
			{
				TArray<FGatherableTextData> GatherableTextDataArray;
				UE::Private::GatherTextFromAssetsCommandlet::GatherTextFromPackage(Package, PackagePendingGather->bIsMapPackage, PackagePendingGather->ExternalActors, PackagePendingGather->ExternalPackages, /*bGatherFromPrimaryPackage*/true, /*bGatherFromExternalPackages*/true, GatherableTextDataArray);

				FMemoryWriter MemoryWriter(PackageResultMessage->GatherableTextData);
				MemoryWriter << GatherableTextDataArray;
			}
			else
			{
				PackageResultMessage->bLoadError = true;
			}
			PackageResultMessage->LoadLogCapture = LogOutputTargetDevice;

			MessageEndpoint->Send(
				PackageResultMessage, // Should be deleted by MessageBus
				EMessageFlags::Reliable,
				nullptr, // No Attachment
				{ PackagePendingGather->RequestorAddress },
				FTimespan::Zero(), // No Delay
				FDateTime::MaxValue() // No Expiration
				);

			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Sent package result for '%ls'.", *PackageNameStr);

			{
				UE::TScopeLock _(PackagesPendingGatherMutex);
				UE::Private::GatherTextFromAssetsCommandlet::ConditionalPurgeGarbage<FPackagePendingGather>(MinFreeMemoryBytes, MaxUsedMemoryBytes, &PackagesPendingGather);
			}
		}
		else
		{
			{
				UE::TScopeLock _(IdleStartTimeUtcMutex);
				if (!IdleStartTimeUtc)
				{
					IdleStartTimeUtc = FDateTime::UtcNow();
				}
			}
			FPlatformProcess::SleepNoStats(0.1f);
		}

		UE::Private::GatherTextFromAssetsCommandlet::TickBackgroundTasks();
		UE::Private::GatherTextFromAssetsCommandlet::TickMessageBusGT();

		// If our director process has stopped, then we can also exit
		if (DirectorProc.IsValid() && !FPlatformProcess::IsProcRunning(DirectorProc))
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Director process has stopped. Exiting worker...");
			break;
		}
	}

	FLoadPackageLogOutputRedirector::Get().Unhook();

	if (DirectorProc.IsValid())
	{
		FPlatformProcess::CloseProc(DirectorProc);
	}

	if (GShaderCompilingManager)
	{
		GShaderCompilingManager->SkipShaderCompilation(!bWasShaderCompilationEnabled);
	}

	// Disable the Endpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint)
	{
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}

	return 0;
}

void UGatherTextFromAssetsWorkerCommandlet::HandlePingMessage(const FGatherTextFromAssetsWorkerMessage_Ping& Message, const TSharedRef<IMessageContext>& Context)
{
	if (Message.ProtocolVersion == UE::Private::GatherTextFromAssetsCommandlet::WorkerProtocolVersion)
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received ping and responded with a pong.");

		FGatherTextFromAssetsWorkerMessage_Pong* PongMessage = FMessageEndpoint::MakeMessage<FGatherTextFromAssetsWorkerMessage_Pong>();
		PongMessage->WorkerId = WorkerId;
		{
			UE::TScopeLock _(IdleStartTimeUtcMutex);
			PongMessage->IdleStartTimeUtc = IdleStartTimeUtc;
		}

		MessageEndpoint->Send(
			PongMessage, // Should be deleted by MessageBus
			EMessageFlags::None,
			nullptr, // No Attachment
			{ Context->GetSender() },
			FTimespan::Zero(), // No Delay
			FDateTime::MaxValue() // No Expiration
			);
	}
	else
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received ping with the wrong protocol version (received %d, expected %d).", Message.ProtocolVersion, UE::Private::GatherTextFromAssetsCommandlet::WorkerProtocolVersion);
	}
}

void UGatherTextFromAssetsWorkerCommandlet::HandlePackageRequestMessage(const FGatherTextFromAssetsWorkerMessage_PackageRequest& Message, const TSharedRef<IMessageContext>& Context)
{
	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received a package request for '%ls'.", *Message.PackageName.ToString());
	
	// Note: This isn't an optimal way to insert into the array, but the array is ordered so it can pop from the end, so inserting at element 0 keeps it FIFO
	{
		UE::TScopeLock _(PackagesPendingGatherMutex);
		PackagesPendingGather.Insert(FPackagePendingGather{ Message.PackageName, Message.Dependencies, Message.ExternalActors, Message.ExternalPackages, Message.bIsMapPackage, Context->GetSender() }, 0);
	}
}

FGatherTextFromAssetsWorkerDirector::~FGatherTextFromAssetsWorkerDirector()
{
	checkf(CurrentWorkers.IsEmpty(), TEXT("FGatherTextFromAssetsWorkerDirector still had workers active during destruction. Did you forget to call StopWorkers?"));
}

FGatherTextFromAssetsWorkerDirector& FGatherTextFromAssetsWorkerDirector::Get()
{
	static FGatherTextFromAssetsWorkerDirector Instance;
	return Instance;
}

bool FGatherTextFromAssetsWorkerDirector::StartWorkers(const int32 NumWorkers, const bool bStopAdditionalWorkers, const int32 NumRestartAttemptsIfCrashed)
{
	// If no workers were requested, then this is the same as calling StopWorkers
	if (NumWorkers <= 0 && bStopAdditionalWorkers)
	{
		StopWorkers();
		return false;
	}

	// Start any extra worker processes that are needed
	// Stop any worker processes that are no longer needed
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		if (NumWorkers > CurrentWorkers.Num())
		{
			const int32 NumWorkersToCreate = NumWorkers - CurrentWorkers.Num();
			for (int32 WorkerIndex = 0; WorkerIndex < NumWorkersToCreate; ++WorkerIndex)
			{
				const FGuid WorkerId = FGuid::NewGuid();
				FString WorkerCommandLine = GenerateWorkerCommandLine(WorkerId);

				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Starting worker process '%ls'...", *WorkerId.ToString());

				FProcHandle WorkerProc = StartWorker(WorkerCommandLine);
				if (WorkerProc.IsValid())
				{
					TSharedPtr<FWorkerInfo> WorkerInfo = MakeShared<FWorkerInfo>();
					WorkerInfo->WorkerProc = WorkerProc;
					WorkerInfo->WorkerCommandLine = MoveTemp(WorkerCommandLine);
					CurrentWorkers.Add(WorkerId, MoveTemp(WorkerInfo));
				}
				else
				{
					UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Worker process '%ls' failed to start. This worker request will be ignored.", *WorkerId.ToString());
				}
			}
		}
		else if (bStopAdditionalWorkers && NumWorkers < CurrentWorkers.Num())
		{
			int32 WorkerCount = 0;
			for (auto It = CurrentWorkers.CreateIterator(); It; ++It)
			{
				const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair = *It;
				if (++WorkerCount > NumWorkers)
				{
					UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Stopping worker process '%ls' as it is no longer needed.", *CurrentWorkerPair.Key.ToString());
					FPlatformProcess::TerminateProc(CurrentWorkerPair.Value->WorkerProc);
					FPlatformProcess::CloseProc(CurrentWorkerPair.Value->WorkerProc);
					It.RemoveCurrent();
					continue;
				}
			}
		}
	}

	bool bHasWorkers = false;

	// Any existing workers may have been idle for a while, so reset their discovery and timeout, so that we have to rediscover them rather than have them immediately timeout
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		for (TTuple<FGuid, TSharedPtr<FWorkerInfo>> CurrentWorkerPair : CurrentWorkers)
		{
			CurrentWorkerPair.Value->NumRestartAttemptsIfCrashed = NumRestartAttemptsIfCrashed;
			ResetWorkerDiscoveryAndTimeout(*CurrentWorkerPair.Value);
			bHasWorkers = true;
		}
	}

	if (bHasWorkers)
	{
		// Ensure we have a MessageBus endpoint to communicate via
		if (!MessageEndpoint)
		{
			MessageEndpoint = FMessageEndpoint::Builder("GatherTextFromAssetsWorkerDirector")
				.ReceivingOnAnyThread()
				.Handling<FGatherTextFromAssetsWorkerMessage_Pong>(this, &FGatherTextFromAssetsWorkerDirector::HandlePongMessage)
				.Handling<FGatherTextFromAssetsWorkerMessage_PackageResult>(this, &FGatherTextFromAssetsWorkerDirector::HandlePackageResultMessage);
		}

		// Force broadcast a ping to start discovery
		// Newly started workers likely won't respond to this, so they will need to wait for the next TickWorkers (unless using WaitForWorkersToStart)
		BroadcastPingMessage(/*bIgnoreDelay*/true);
		UE::Private::GatherTextFromAssetsCommandlet::TickMessageBusGT();
	}

	return bHasWorkers;
}

bool FGatherTextFromAssetsWorkerDirector::WaitForWorkersToStart(const TOptional<FTimespan> Timeout)
{
	// Wait for the worker processes to reach a state where they are ready to accept work
	const FDateTime WaitStartUtc = FDateTime::UtcNow();
	for (;;)
	{
		const FDateTime UtcNow = FDateTime::UtcNow();
		const FTimespan WaitDuration = UtcNow - WaitStartUtc;

		int32 NumPendingWorkers = 0;
		{
			int32 NumReadyWorkers = 0;

			UE::TScopeLock _(CurrentWorkersMutex);
			for (auto It = CurrentWorkers.CreateIterator(); It; ++It)
			{
				const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair = *It;
				
				// Has this process crashed (or quit) during start-up?
				if (!FPlatformProcess::IsProcRunning(CurrentWorkerPair.Value->WorkerProc))
				{
					FPlatformProcess::CloseProc(CurrentWorkerPair.Value->WorkerProc);
					UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Worker process '%ls' crashed during start-up. This worker request will be ignored.", *CurrentWorkerPair.Key.ToString());
					It.RemoveCurrent();
					continue;
				}

				if (CurrentWorkerPair.Value->EndpointAddress.IsValid())
				{
					++NumReadyWorkers;
				}
				else
				{
					++NumPendingWorkers;
				}
			}

			if (NumReadyWorkers == CurrentWorkers.Num())
			{
				return true;
			}

			if (Timeout && WaitDuration >= *Timeout)
			{
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Timed-out waiting for %d worker(s) to be ready (duration %ls).", NumPendingWorkers, *WaitDuration.ToString());
				return false;
			}
		}

		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Waiting for %d worker(s) to be ready (duration %ls)...", NumPendingWorkers, *WaitDuration.ToString());

		// Force broadcast a ping request and sleep
		BroadcastPingMessage(/*bIgnoreDelay*/true);
		UE::Private::GatherTextFromAssetsCommandlet::TickMessageBusGT();
		FPlatformProcess::SleepNoStats(1.0f);
	}
}

bool FGatherTextFromAssetsWorkerDirector::StopWorkers()
{
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		for (const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair : CurrentWorkers)
		{
			FPlatformProcess::TerminateProc(CurrentWorkerPair.Value->WorkerProc);
			FPlatformProcess::CloseProc(CurrentWorkerPair.Value->WorkerProc);
		}
		CurrentWorkers.Reset();
	}

	// Disable the Endpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint)
	{
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}

	while (PackageResults.Dequeue()) {}
	PackagesFromCrashedWorkers.Reset();

	return true;
}

bool FGatherTextFromAssetsWorkerDirector::HasWorkers() const
{
	UE::TScopeLock _(CurrentWorkersMutex);
	return CurrentWorkers.Num() > 0;
}

void FGatherTextFromAssetsWorkerDirector::TickWorkers()
{
	if (MessageEndpoint)
	{
		BroadcastPingMessage();
		UE::Private::GatherTextFromAssetsCommandlet::TickMessageBusGT();
	}

	{
		const FDateTime UtcNow = FDateTime::UtcNow();

		UE::TScopeLock _(CurrentWorkersMutex);
		for (auto It = CurrentWorkers.CreateIterator(); It; ++It)
		{
			const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair = *It;

			// Has this process crashed?
			if (!FPlatformProcess::IsProcRunning(CurrentWorkerPair.Value->WorkerProc))
			{
				FPlatformProcess::CloseProc(CurrentWorkerPair.Value->WorkerProc);
				if (!HandleWorkerCrashed(CurrentWorkerPair.Key, *CurrentWorkerPair.Value))
				{
					It.RemoveCurrent();
				}
				continue;
			}

			// Is this worker still starting?
			// If so, we can skip the timeout detection as we won't have that data yet
			if (!CurrentWorkerPair.Value->EndpointAddress.IsValid())
			{
				continue;
			}

			// Has this process likely hung? (not received any MessageBus messages for over 5 minutes)
			if (CurrentWorkerPair.Value->LastMessageReceivedUtc && *CurrentWorkerPair.Value->LastMessageReceivedUtc + FTimespan(0, 5, 0) <= UtcNow)
			{
				FPlatformProcess::TerminateProc(CurrentWorkerPair.Value->WorkerProc);
				FPlatformProcess::CloseProc(CurrentWorkerPair.Value->WorkerProc);
				if (!HandleWorkerCrashed(CurrentWorkerPair.Key, *CurrentWorkerPair.Value))
				{
					It.RemoveCurrent();
				}
				continue;
			}

			// Were we asked to re-send any pending package requests? (eg, after rediscovering a previously crashed worker)
			if (CurrentWorkerPair.Value->bResendPendingPackageRequests)
			{
				if (CurrentWorkerPair.Value->PendingPackageRequests.Num() > 0)
				{
					UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Worker process '%ls' has packages assigned to it, and has been rediscovered. Re-sending %d pending package request(s)...", *CurrentWorkerPair.Key.ToString(), CurrentWorkerPair.Value->PendingPackageRequests.Num());
					ResendWorkerPendingPackageRequests(*CurrentWorkerPair.Value);
				}
				CurrentWorkerPair.Value->bResendPendingPackageRequests = false;
			}

			// Has this worker has been idle for over 2 minutes, and last had work assigned to it over 2 minutes ago?
			// If so, it's likely that either the request or response was lost, so try sending any pending requests again...
			if (CurrentWorkerPair.Value->PendingPackageRequests.Num() > 0 &&
				CurrentWorkerPair.Value->LastPackageRequestUtc && CurrentWorkerPair.Value->IdleStartTimeUtc &&
				*CurrentWorkerPair.Value->LastPackageRequestUtc + FTimespan(0, 2, 0) <= UtcNow && *CurrentWorkerPair.Value->IdleStartTimeUtc + FTimespan(0, 2, 0) <= UtcNow)
			{
				UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Worker process '%ls' has packages assigned to it, but has been idle for a while. Re-sending %d pending package request(s)...", *CurrentWorkerPair.Key.ToString(), CurrentWorkerPair.Value->PendingPackageRequests.Num());
				ResendWorkerPendingPackageRequests(*CurrentWorkerPair.Value);
			}
		}
	}
}

TArray<FGuid> FGatherTextFromAssetsWorkerDirector::GetAvailableWorkerIds(const TOptional<int32> IdleThreshold) const
{
	TArray<FGuid> WorkerIds;
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		for (const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair : CurrentWorkers)
		{
			if (CurrentWorkerPair.Value->EndpointAddress.IsValid() && (!IdleThreshold.IsSet() || CurrentWorkerPair.Value->PendingPackageRequests.Num() <= IdleThreshold.GetValue()))
			{
				WorkerIds.Add(CurrentWorkerPair.Key);
			}
		}
	}
	return WorkerIds;
}

bool FGatherTextFromAssetsWorkerDirector::IsIdle(int32* OutNumPendingWorkerPackages) const
{
	int32 NumPendingWorkerPackages = 0;
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		for (const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair : CurrentWorkers)
		{
			NumPendingWorkerPackages += CurrentWorkerPair.Value->PendingPackageRequests.Num();
		}
	}

	if (OutNumPendingWorkerPackages)
	{
		*OutNumPendingWorkerPackages = NumPendingWorkerPackages;
	}

	return NumPendingWorkerPackages == 0
		&& PackageResults.IsEmpty()
		&& PackagesFromCrashedWorkers.IsEmpty();
}

bool FGatherTextFromAssetsWorkerDirector::AssignPackageToWorker(const FGuid& WorkerId, const FGatherTextFromAssetsWorkerMessage_PackageRequest& PackageRequest)
{
	FMessageAddress WorkerAddress;
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		if (TSharedPtr<FWorkerInfo> CurrentWorker = CurrentWorkers.FindRef(WorkerId))
		{
			WorkerAddress = CurrentWorker->EndpointAddress;
			CurrentWorker->LastPackageRequestUtc = FDateTime::UtcNow();
			CurrentWorker->PendingPackageRequests.Add(PackageRequest.PackageName, PackageRequest);
		}
		else
		{
			return false;
		}
	}

	if (WorkerAddress.IsValid())
	{
		FGatherTextFromAssetsWorkerMessage_PackageRequest* PackageRequestMessage = FMessageEndpoint::MakeMessage<FGatherTextFromAssetsWorkerMessage_PackageRequest>(PackageRequest);

		MessageEndpoint->Send(
			PackageRequestMessage, // Should be deleted by MessageBus
			EMessageFlags::Reliable,
			nullptr, // No Attachment
			{ WorkerAddress },
			FTimespan::Zero(), // No Delay
			FDateTime::MaxValue() // No Expiration
			);
	}

	UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Sent package request to worker '%ls' for '%ls'.", *WorkerId.ToString(), *PackageRequest.PackageName.ToString());

	return true;
}

void FGatherTextFromAssetsWorkerDirector::SetIngestPackageResultHandler(TFunction<bool(const FGatherTextFromAssetsWorkerMessage_PackageResult&)>&& Handler)
{
	PackageResultHandler = MoveTemp(Handler);
}

void FGatherTextFromAssetsWorkerDirector::ClearIngestPackageResultHandler()
{
	PackageResultHandler.Reset();
}

TOptional<FGatherTextFromAssetsWorkerMessage_PackageResult> FGatherTextFromAssetsWorkerDirector::IngestPackageResult()
{
	return PackageResults.Dequeue();
}

TArray<FName> FGatherTextFromAssetsWorkerDirector::IngestPackagesFromCrashedWorkers()
{
	TArray<FName> TmpPackagesFromCrashedWorkers = MoveTemp(PackagesFromCrashedWorkers);
	PackagesFromCrashedWorkers.Reset();

	// Merge in any packages from workers that haven't started yet, as the main process may want to redistribute those or handle them itself
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		for (const TTuple<FGuid, TSharedPtr<FWorkerInfo>>& CurrentWorkerPair : CurrentWorkers)
		{
			if (!CurrentWorkerPair.Value->EndpointAddress.IsValid())
			{
				for (const TTuple<FName, FGatherTextFromAssetsWorkerMessage_PackageRequest>& PendingPackageRequestPair : CurrentWorkerPair.Value->PendingPackageRequests)
				{
					TmpPackagesFromCrashedWorkers.Add(PendingPackageRequestPair.Key);
				}
				CurrentWorkerPair.Value->PendingPackageRequests.Reset();
			}
		}
	}

	return TmpPackagesFromCrashedWorkers;
}

FString FGatherTextFromAssetsWorkerDirector::GenerateWorkerCommandLine(const FGuid& WorkerId)
{
	// Build the set of tokens to keep from the current command line args
	TArray<FString> Tokens;
	{
		const TCHAR* ProjectName = FApp::GetProjectName();
		const TCHAR* CommandLine = FCommandLine::Get();

		FString Token;
		while (FParse::Token(CommandLine, Token, /*bUseEscape*/false))
		{
			if (Token.IsEmpty())
			{
				continue;
			}
			if (Token == ProjectName || Token.EndsWith(TEXT(".uproject")) || 
				Token.StartsWith(TEXT("-run=")) ||
				Token.StartsWith(TEXT("-config=")) ||
				Token.StartsWith(TEXT("-configlist=")) ||
				Token.StartsWith(TEXT("-abslog=")) ||
				Token == TEXT("-unattended") ||
				Token == TEXT("-messaging") ||
				Token.StartsWith(TEXT("-NumGatherTextWorkers=")) ||
				Token == TEXT("-StartGatherTextWorkersImmediately")
				)
			{
				continue;
			}
			Tokens.Add(MoveTemp(Token));
		}
	}
	
	Tokens.Add(TEXT("-unattended"));
	Tokens.Add(TEXT("-messaging"));
	Tokens.Add(TEXT("-multiprocess"));
	Tokens.Add(TEXT("-GatherWorkerId=") + WorkerId.ToString());
	Tokens.Add(FString::Printf(TEXT("-GatherDirectorPid=%u"), FPlatformProcess::GetCurrentProcessId()));

	if (bWorkersCanReadAssetRegistryCache)
	{
		Tokens.Add(TEXT("-NoAssetRegistryCacheWrite")); // Disable writing the AR cache as the main process has already updated it
	}
	else
	{
		Tokens.Add(TEXT("-NoAssetRegistryCache")); // Disable the AR cache to avoid IO conflicts with the main process; the worker doesn't directly use the AR
	}

	// We are joining the tokens back into a commandline string; wrap tokens with whitespace in quotes
	for (FString& Token : Tokens)
	{
		static constexpr FAsciiSet WhitespaceSet(" \r\n");
		if (FAsciiSet::HasAny(Token, WhitespaceSet))
		{
			if (int32 IndexOfQuote = INDEX_NONE;
				!Token.FindChar(TEXT('\"'), IndexOfQuote))
			{
				Token.InsertAt(0, TEXT('\"'));
				Token += TEXT('\"');
			}
		}
	}

	const FString ProjectFilePath = FApp::HasProjectName() ? FString::Printf(TEXT("\"%s\""), *FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath())) : FString();
	return CommandletHelpers::BuildCommandletProcessArguments(TEXT("GatherTextFromAssetsWorker"), *ProjectFilePath, *FString::Join(Tokens, TEXT(" ")));
}

FProcHandle FGatherTextFromAssetsWorkerDirector::StartWorker(const FString& WorkerCommandLine)
{
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = true;
	constexpr bool bLaunchReallyHidden = true;
	return FPlatformProcess::CreateProc(*FUnrealEdMisc::Get().GetExecutableForCommandlets(), *WorkerCommandLine, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, nullptr);
}

bool FGatherTextFromAssetsWorkerDirector::HandleWorkerCrashed(const FGuid& WorkerId, FWorkerInfo& WorkerInfo)
{
	if (WorkerInfo.NumRestartAttemptsIfCrashed > 0)
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Restarting worker process '%ls'...", *WorkerId.ToString());
		--WorkerInfo.NumRestartAttemptsIfCrashed;

		// Restart the process and wait for it to start again
		// Any pending work will be resent once the worker is available
		WorkerInfo.WorkerProc = StartWorker(WorkerInfo.WorkerCommandLine);
		if (WorkerInfo.WorkerProc.IsValid())
		{
			ResetWorkerDiscoveryAndTimeout(WorkerInfo);
			return true;
		}
		else
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Worker process '%ls' failed to start. Its work will be returned to the main pool.", *WorkerId.ToString());
		}
	}
	else
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Warning, "Worker process '%ls' crashed or hung. Its work will be returned to the main pool.", *WorkerId.ToString());
	}

	// Failed to restart, or restart is not allowed
	// Move the pending work to the crashed queue
	for (const TTuple<FName, FGatherTextFromAssetsWorkerMessage_PackageRequest>& PendingPackageRequestPair : WorkerInfo.PendingPackageRequests)
	{
		PackagesFromCrashedWorkers.Add(PendingPackageRequestPair.Key);
	}
	WorkerInfo.PendingPackageRequests.Reset();

	return false;
}

void FGatherTextFromAssetsWorkerDirector::ResetWorkerDiscoveryAndTimeout(FWorkerInfo& WorkerInfo)
{
	WorkerInfo.EndpointAddress = FMessageAddress();
	WorkerInfo.LastMessageReceivedUtc.Reset();
	WorkerInfo.LastPackageRequestUtc.Reset();
	WorkerInfo.IdleStartTimeUtc.Reset();
}

void FGatherTextFromAssetsWorkerDirector::ResendWorkerPendingPackageRequests(FWorkerInfo& WorkerInfo)
{
	WorkerInfo.LastPackageRequestUtc = FDateTime::UtcNow();
	WorkerInfo.IdleStartTimeUtc.Reset(); // Reset our cached idle state so that we don't re-trigger the re-send until the worker confirms it's re-entered an idle state

	for (const TTuple<FName, FGatherTextFromAssetsWorkerMessage_PackageRequest>& PendingPackageRequestPair : WorkerInfo.PendingPackageRequests)
	{
		FGatherTextFromAssetsWorkerMessage_PackageRequest* PackageRequestMessage = FMessageEndpoint::MakeMessage<FGatherTextFromAssetsWorkerMessage_PackageRequest>(PendingPackageRequestPair.Value);

		MessageEndpoint->Send(
			PackageRequestMessage, // Should be deleted by MessageBus
			EMessageFlags::Reliable,
			nullptr, // No Attachment
			{ WorkerInfo.EndpointAddress },
			FTimespan::Zero(), // No Delay
			FDateTime::MaxValue() // No Expiration
			);
	}
}

void FGatherTextFromAssetsWorkerDirector::BroadcastPingMessage(const bool bIgnoreDelay)
{
	const FDateTime UtcNow = FDateTime::UtcNow();
	if (bIgnoreDelay || (LastPingBroadcastUtc + FTimespan(0, 0, 30) <= UtcNow))
	{
		LastPingBroadcastUtc = UtcNow;

		FGatherTextFromAssetsWorkerMessage_Ping* PingMessage = FMessageEndpoint::MakeMessage<FGatherTextFromAssetsWorkerMessage_Ping>();
		PingMessage->ProtocolVersion = UE::Private::GatherTextFromAssetsCommandlet::WorkerProtocolVersion;

		MessageEndpoint->Publish(
			PingMessage,
			EMessageScope::Network,
			FTimespan::Zero(), // No Delay
			FDateTime::MaxValue() // No Expiration
			);
	}
}

void FGatherTextFromAssetsWorkerDirector::HandlePongMessage(const FGatherTextFromAssetsWorkerMessage_Pong& Message, const TSharedRef<IMessageContext>& Context)
{
	UE::TScopeLock _(CurrentWorkersMutex);
	if (TSharedPtr<FWorkerInfo> CurrentWorker = CurrentWorkers.FindRef(Message.WorkerId))
	{
		const FDateTime UtcNow = FDateTime::UtcNow();
		if (Message.IdleStartTimeUtc)
		{
			const FTimespan IdleDuration = UtcNow - *Message.IdleStartTimeUtc;
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received pong from worker '%ls' with endpoint '%ls' (Idle for %ls).", *Message.WorkerId.ToString(), *Context->GetSender().ToString(), *IdleDuration.ToString());
		}
		else
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received pong from worker '%ls' with endpoint '%ls' (Working).", *Message.WorkerId.ToString(), *Context->GetSender().ToString());
		}

		if (!CurrentWorker->EndpointAddress.IsValid() && CurrentWorker->PendingPackageRequests.Num() > 0)
		{
			// If we (re)discovered this worker, resend any pending requests as they may have been lost (eg, if restarting the process after a crash)
			CurrentWorker->bResendPendingPackageRequests = true;
		}
		CurrentWorker->EndpointAddress = Context->GetSender();
		CurrentWorker->LastMessageReceivedUtc = UtcNow;
		CurrentWorker->IdleStartTimeUtc = Message.IdleStartTimeUtc;
	}
}

void FGatherTextFromAssetsWorkerDirector::HandlePackageResultMessage(const FGatherTextFromAssetsWorkerMessage_PackageResult& Message, const TSharedRef<IMessageContext>& Context)
{
	bool bWasSolicitedResult = false;
	{
		UE::TScopeLock _(CurrentWorkersMutex);
		if (TSharedPtr<FWorkerInfo> CurrentWorker = CurrentWorkers.FindRef(Message.WorkerId))
		{
			CurrentWorker->LastMessageReceivedUtc = FDateTime::UtcNow();
			CurrentWorker->IdleStartTimeUtc.Reset(); // Don't consider it idle if we're still receiving results from it
			bWasSolicitedResult = CurrentWorker->PendingPackageRequests.Contains(Message.PackageName);
		}
	}

	if (bWasSolicitedResult)
	{
		if (!PackageResultHandler || !PackageResultHandler(Message))
		{
			UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received package result from worker '%ls' for '%ls' and queued it for ingestion.", *Message.WorkerId.ToString(), *Message.PackageName.ToString());
			PackageResults.Enqueue(Message);
		}

		// Note: We do the remove after the ingestion to avoid a race condition with IsIdle
		{
			UE::TScopeLock _(CurrentWorkersMutex);
			if (TSharedPtr<FWorkerInfo> CurrentWorker = CurrentWorkers.FindRef(Message.WorkerId))
			{
				CurrentWorker->PendingPackageRequests.Remove(Message.PackageName);
			}
		}
	}
	else
	{
		UE_LOGF(LogGatherTextFromAssetsCommandlet, Display, "Received unsolicited package result from worker '%ls' for '%ls' and dropped it.", *Message.WorkerId.ToString(), *Message.PackageName.ToString());
	}
}

#undef LOC_DEFINE_REGION

//////////////////////////////////////////////////////////////////////////
