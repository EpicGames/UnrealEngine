// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextFromMetadataCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "GatherTextMetaDataHelper.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "SourceCodeNavigation.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GatherTextFromMetadataCommandlet)

class UObject;

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextFromMetaDataCommandlet, Log, All);
namespace GatherTextFromMetaDataCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

//////////////////////////////////////////////////////////////////////////
//GatherTextFromMetaDataCommandlet

UGatherTextFromMetaDataCommandlet::UGatherTextFromMetaDataCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UGatherTextFromMetaDataCommandlet::ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
{
	const FString* GatherType = ParamVals.Find(UGatherTextCommandletBase::GatherTypeParam);
	// If the param is not specified, it is assumed that both source and assets are to be gathered 
	return !GatherType || *GatherType == TEXT("Metadata") || *GatherType == TEXT("All");
}

int32 UGatherTextFromMetaDataCommandlet::Main( const FString& Params )
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromMetaDataCommandlet::Main"), LogGatherTextFromMetaDataCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	//Set config file
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));
	FString GatherTextConfigPath;
	
	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Error, "No config specified.",
			("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	//Set config section
	ParamVal = ParamVals.Find(FString(TEXT("Section")));
	FString SectionName;

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Error, "No config section specified.",
			("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	//Modules to Preload
	TArray<FString> ModulesToPreload;
	GetStringArrayFromConfig(*SectionName, TEXT("ModulesToPreload"), ModulesToPreload, GatherTextConfigPath);

	for (const FString& ModuleName : ModulesToPreload)
	{
		FModuleManager::Get().LoadModule(*ModuleName);
	}

	// IncludePathFilters
	TArray<FString> IncludePathFilters;
	GetPathArrayFromConfig(*SectionName, TEXT("IncludePathFilters"), IncludePathFilters, GatherTextConfigPath);

	// IncludePaths (DEPRECATED)
	{
		TArray<FString> IncludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("IncludePaths"), IncludePaths, GatherTextConfigPath);
		if (IncludePaths.Num())
		{
			IncludePathFilters.Append(IncludePaths);
			UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Warning, "IncludePaths detected in section {section}. IncludePaths is deprecated, please use IncludePathFilters.",
				("section", *SectionName),
				("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if (IncludePathFilters.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Error, "No include path filters in section {section}.",
			("section", *SectionName),
			("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// ExcludePathFilters
	TArray<FString> ExcludePathFilters;
	GetPathArrayFromConfig(*SectionName, TEXT("ExcludePathFilters"), ExcludePathFilters, GatherTextConfigPath);

	// ExcludePaths (DEPRECATED)
	{
		TArray<FString> ExcludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("ExcludePaths"), ExcludePaths, GatherTextConfigPath);
		if (ExcludePaths.Num())
		{
			ExcludePathFilters.Append(ExcludePaths);
			UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Warning, "ExcludePaths detected in section {section}. ExcludePaths is deprecated, please use ExcludePathFilters.",
				("section", *SectionName),
				("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	{
		FGatherTextContext Context;
		Context.CommandletClass = GetClass()->GetClassPathName();
		Context.PreferredPathType = FGatherTextContext::EPreferredPathType::Root;

		FGatherTextDelegates::GetAdditionalGatherPathsForContext.Broadcast(GatherManifestHelper->GetTargetName(), Context, IncludePathFilters, ExcludePathFilters);
	}

	MetaDataHelper = MakePimpl<FGatherTextMetaDataHelper>(GatherManifestHelper);

	// Get whether we should gather editor-only data. Typically only useful for the localization of UE itself.
	{
		bool bShouldGatherFromEditorOnlyData = false;
		GetBoolFromConfig(*SectionName, TEXT("ShouldGatherFromEditorOnlyData"), bShouldGatherFromEditorOnlyData, GatherTextConfigPath);
		MetaDataHelper->SetShouldGatherFromEditorOnlyData(bShouldGatherFromEditorOnlyData);
	}

	// FieldTypesToInclude/FieldTypesToExclude
	{
		TArray<FString> FieldTypesToInclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldTypesToInclude"), FieldTypesToInclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldTypesFromStrings(FieldTypesToInclude, /*bInclude*/true, TEXT("FieldTypesToInclude"));

		TArray<FString> FieldTypesToExclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldTypesToExclude"), FieldTypesToExclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldTypesFromStrings(FieldTypesToExclude, /*bInclude*/false, TEXT("FieldTypesToExclude"));
	}

	// FieldOwnerTypesToInclude/FieldOwnerTypesToExclude
	{
		TArray<FString> FieldOwnerTypesToInclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldOwnerTypesToInclude"), FieldOwnerTypesToInclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldOwnerTypesFromStrings(FieldOwnerTypesToInclude, /*bInclude*/true, TEXT("FieldOwnerTypesToInclude"));

		TArray<FString> FieldOwnerTypesToExclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldOwnerTypesToExclude"), FieldOwnerTypesToExclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldOwnerTypesFromStrings(FieldOwnerTypesToExclude, /*bInclude*/false, TEXT("FieldOwnerTypesToExclude"));
	}

	// FieldOuterTypesToInclude/FieldOuterTypesToExclude
	{
		TArray<FString> FieldOuterTypesToInclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldOuterTypesToInclude"), FieldOuterTypesToInclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldOuterTypesFromStrings(FieldOuterTypesToInclude, /*bInclude*/true, TEXT("FieldOuterTypesToInclude"));

		TArray<FString> FieldOuterTypesToExclude;
		GetStringArrayFromConfig(*SectionName, TEXT("FieldOuterTypesToExclude"), FieldOuterTypesToExclude, GatherTextConfigPath);
		MetaDataHelper->SetFieldOuterTypesFromStrings(FieldOuterTypesToExclude, /*bInclude*/false, TEXT("FieldOuterTypesToExclude"));
	}

	{
		TArray<FString> InputKeys;
		TArray<FString> OutputNamespaces;
		TArray<FString> OutputKeys;
		GetStringArrayFromConfig(*SectionName, TEXT("InputKeys"), InputKeys, GatherTextConfigPath);
		GetStringArrayFromConfig(*SectionName, TEXT("OutputNamespaces"), OutputNamespaces, GatherTextConfigPath);
		GetStringArrayFromConfig(*SectionName, TEXT("OutputKeys"), OutputKeys, GatherTextConfigPath);
		MetaDataHelper->SetGatherParameters(MoveTemp(InputKeys), MoveTemp(OutputNamespaces), MoveTemp(OutputKeys));
	}

	// Execute gather.
	GatherTextFromUObjects(IncludePathFilters, ExcludePathFilters);

	// Add any manifest dependencies if they were provided
	TArray<FString> ManifestDependenciesList;
	GetPathArrayFromConfig(*SectionName, TEXT("ManifestDependencies"), ManifestDependenciesList, GatherTextConfigPath);

	for (const FString& ManifestDependency : ManifestDependenciesList)
	{
		FText OutError;
		if (!GatherManifestHelper->AddDependency(ManifestDependency, &OutError))
		{
			UE_LOGFMT(LogGatherTextFromMetaDataCommandlet, Error, "The GatherTextFromMetaData commandlet couldn't load the specified manifest dependency: '{manifestDependency}'. {error}",
				("manifestDependency", *ManifestDependency),
				("error", *OutError.ToString()),
				("id", GatherTextFromMetaDataCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	return 0;
}

void UGatherTextFromMetaDataCommandlet::GatherTextFromUObjects(const TArray<FString>& IncludePaths, const TArray<FString>& ExcludePaths)
{
	const FFuzzyPathMatcher FuzzyPathMatcher = FFuzzyPathMatcher(IncludePaths, ExcludePaths);

	for (TObjectIterator<UField> It; It; ++It)
	{
		UField* Field = *It;

		const UPackage* FieldPackage = Field->GetOutermost();
		const FString FieldPackageName = FieldPackage->GetName();
		if (!FPackageName::IsScriptPackage(FieldPackageName) || FPackageName::IsMemoryPackage(FieldPackageName) || FPackageName::IsTempPackage(FieldPackageName))
		{
			continue;
		}

		FString SourceFilePath;
		if (!FSourceCodeNavigation::FindClassHeaderPath(Field, SourceFilePath))
		{
			continue;
		}
		SourceFilePath = FPaths::ConvertRelativePathToFull(SourceFilePath);
		check(!SourceFilePath.IsEmpty());

		const FFuzzyPathMatcher::EPathMatch PathMatch = FuzzyPathMatcher.TestPath(SourceFilePath);
		if (PathMatch != FFuzzyPathMatcher::EPathMatch::Included)
		{
			continue;
		}

		const FName MetaDataPlatformName = GetSplitPlatformNameFromPath(SourceFilePath);
		MetaDataHelper->GatherTextFromField(Field, MetaDataPlatformName);
	}
}
