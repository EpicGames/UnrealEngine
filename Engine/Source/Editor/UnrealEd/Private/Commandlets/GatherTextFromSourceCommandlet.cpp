// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextFromSourceCommandlet.h"

#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Internationalization/TextChar.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Logging/StructuredLog.h"
#include "Misc/AsciiSet.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GatherTextFromSourceCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGatherTextFromSourceCommandlet, Log, All);
namespace GatherTextFromSourceCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

//////////////////////////////////////////////////////////////////////////
//GatherTextFromSourceCommandlet

#define LOC_DEFINE_REGION

UGatherTextFromSourceCommandlet::UGatherTextFromSourceCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FString UGatherTextFromSourceCommandlet::DefineString(TEXT("#define "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::UndefString(TEXT("#undef "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::IfString(TEXT("#if "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::IfDefString(TEXT("#ifdef "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::ElIfString(TEXT("#elif "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::ElseString(TEXT("#else"));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::EndIfString(TEXT("#endif"));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::DefinedString(TEXT("defined "));
const FString UGatherTextFromSourceCommandlet::FPreProcessorDescriptor::IniNamespaceString(TEXT("["));
const FString UGatherTextFromSourceCommandlet::FMacroDescriptor::TextMacroString(TEXT("TEXT"));
const FString UGatherTextFromSourceCommandlet::MacroString_LOCTEXT(TEXT("LOCTEXT"));
const FString UGatherTextFromSourceCommandlet::MacroString_NSLOCTEXT(TEXT("NSLOCTEXT"));
const FString UGatherTextFromSourceCommandlet::MacroString_NOTELOCTEXT(TEXT("NOTELOCTEXT"));
const FString UGatherTextFromSourceCommandlet::MacroString_NSNOTELOCTEXT(TEXT("NSNOTELOCTEXT"));
const FString UGatherTextFromSourceCommandlet::MacroString_UI_COMMAND(TEXT("UI_COMMAND"));
const FString UGatherTextFromSourceCommandlet::MacroString_UI_COMMAND_EXT(TEXT("UI_COMMAND_EXT"));

// Nested macro statistics, to track any supported features
// See https://gcc.gnu.org/onlinedocs/gcc-3.0.1/cpp_3.html
struct FParsedNestedMacroStats
{
	std::atomic<int32> DuplicateExact = 0;
	std::atomic<int32> DuplicateMacroName = 0;
	std::atomic<int32> DuplicateExcluded = 0;
	std::atomic<int32> Concatenation = 0;
	std::atomic<int32> Variadic = 0;
	std::atomic<int32> SizeInBytes = 0;

	std::atomic<int32> Nested_LOCTEXT = 0;
	std::atomic<int32> Nested_NSLOCTEXT = 0;
	std::atomic<int32> Nested_NOTELOCTEXT = 0;
	std::atomic<int32> Nested_NSNOTELOCTEXT = 0;
	std::atomic<int32> Nested_UI_COMMAND = 0;
	std::atomic<int32> Nested_UI_COMMAND_EXT = 0;

	std::atomic<int32> PrepassBegin = 0;
	std::atomic<int32> PrepassEnd = 0;
	std::atomic<int32> MainpassBegin = 0;
	std::atomic<int32> MainpassMid = 0;
	std::atomic<int32> MainpassEnd = 0;

	std::atomic<int32> SubmitNested = 0;
	std::atomic<int32> Submit = 0;
};
static FParsedNestedMacroStats NestedMacroStats;

struct FFileTypeStats
{
	std::atomic<int32> FileType_h = 0;
	std::atomic<int32> FileType_inl = 0;
	std::atomic<int32> FileType_c = 0;
	std::atomic<int32> FileType_cpp = 0;
	std::atomic<int32> FileType_ini = 0;
	std::atomic<int32> FileType_other = 0;
	std::atomic<int32> FileType_total = 0;
	std::atomic<double> Duration_sec = 0.0f;
};
static FFileTypeStats FileTypeStats[EGatherSourcePasses::Mainpass + 1];

bool UGatherTextFromSourceCommandlet::ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
{
	const FString* GatherType = ParamVals.Find(UGatherTextCommandletBase::GatherTypeParam);
	// If the param is not specified, it is assumed that both source and assets are to be gathered 
	return !GatherType || *GatherType == TEXT("Source") || *GatherType == TEXT("All");
}

int32 UGatherTextFromSourceCommandlet::Main( const FString& Params )
{
	UE_SCOPED_TIMER(TEXT("UGatherTextFromSourceCommandlet::Main"), LogGatherTextFromSourceCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);
	
	//Set config file
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));
	FString GatherTextConfigPath;
	
	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "No config specified.",
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	//Set config section
	ParamVal = ParamVals.Find(FString(TEXT("Section")));
	FString SectionName;

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "No config section specified.",
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// GatheredSourceBasePath
	FString GatheredSourceBasePath;
	GetPathFromConfig(*SectionName, TEXT("GatheredSourceBasePath"), GatheredSourceBasePath, GatherTextConfigPath);
	if (GatheredSourceBasePath.IsEmpty())
	{
		GatheredSourceBasePath = UGatherTextCommandletBase::GetProjectBasePath();
	}

	// SearchDirectoryPaths
	TArray<FString> SearchDirectoryPaths;
	GetPathArrayFromConfig(*SectionName, TEXT("SearchDirectoryPaths"), SearchDirectoryPaths, GatherTextConfigPath);

	// IncludePaths (DEPRECATED)
	{
		TArray<FString> IncludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("IncludePaths"), IncludePaths, GatherTextConfigPath);
		if (IncludePaths.Num())
		{
			SearchDirectoryPaths.Append(IncludePaths);
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "IncludePaths detected in section {section}. IncludePaths is deprecated, please use SearchDirectoryPaths.",
				("section", *SectionName),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if (SearchDirectoryPaths.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "No search directory paths in section {section}.",
			("section", *SectionName),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return 0;
	}

	// ExcludePathFilters
	TArray<FString> ExcludePathFilters;
	GetPathArrayFromConfig(*SectionName, TEXT("ExcludePathFilters"), ExcludePathFilters, GatherTextConfigPath);

	// ExcludePaths (DEPRECATED)
	{
		TArray<FString> ExcludePaths;
		GetPathArrayFromConfig(*SectionName, TEXT("ExcludePaths"), ExcludePaths, GatherTextConfigPath);
		if (ExcludePaths.Num())
		{
			ExcludePathFilters.Append(ExcludePaths);
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "ExcludePaths detected in section {section}. ExcludePaths is deprecated, please use ExcludePathFilters.",
				("section", *SectionName),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	// FileNameFilters
	TArray<FString> FileNameFilters;
	GetStringArrayFromConfig(*SectionName, TEXT("FileNameFilters"), FileNameFilters, GatherTextConfigPath);

	// SourceFileSearchFilters (DEPRECATED)
	{
		TArray<FString> SourceFileSearchFilters;
		GetStringArrayFromConfig(*SectionName, TEXT("SourceFileSearchFilters"), SourceFileSearchFilters, GatherTextConfigPath);
		if (SourceFileSearchFilters.Num())
		{
			FileNameFilters.Append(SourceFileSearchFilters);
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "SourceFileSearchFilters detected in section {section}. SourceFileSearchFilters is deprecated, please use FileNameFilters.",
				("section", *SectionName),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if (FileNameFilters.Num() == 0)
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "No source filters in section {section}",
			("section", *SectionName),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return 0;
	}

	// Ensure all filters are unique.
	TArray<FString> UniqueSourceFileSearchFilters;
	for (const FString& SourceFileSearchFilter : FileNameFilters)
	{
		UniqueSourceFileSearchFilters.AddUnique(SourceFileSearchFilter);
	}

	TArray<FString> IncludePathFilters;
	TArray<FString> FilesToProcess;
	GetFilesToProcess(SearchDirectoryPaths, UniqueSourceFileSearchFilters, IncludePathFilters, ExcludePathFilters, FilesToProcess, true);
	CountFileTypes(FilesToProcess, EGatherSourcePasses::Mainpass);
	
	// Return if no source files were found
	if( FilesToProcess.Num() == 0 )
	{
		FString SpecifiedDirectoriesString;
		for (const FString& IncludePath : IncludePathFilters)
		{
			SpecifiedDirectoriesString.Append(FString(SpecifiedDirectoriesString.IsEmpty() ? TEXT("") : TEXT("\n")) + FString::Printf(TEXT("+ %s"), *IncludePath));
		}
		for (const FString& ExcludePath : ExcludePathFilters)
		{
			SpecifiedDirectoriesString.Append(FString(SpecifiedDirectoriesString.IsEmpty() ? TEXT("") : TEXT("\n")) + FString::Printf(TEXT("- %s"), *ExcludePath));
		}

		FString SourceFileSearchFiltersString;
		for (const FString& Filter : UniqueSourceFileSearchFilters)
		{
			SourceFileSearchFiltersString += FString(SourceFileSearchFiltersString.IsEmpty() ? TEXT("") : TEXT(", ")) + Filter;
		}

		UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "The GatherTextFromSource commandlet couldn't find any source files matching (%ls) in the specified directories:\n%ls", *SourceFileSearchFiltersString, *SpecifiedDirectoriesString);
		
		return 0;
	}

	// Add any manifest dependencies if they were provided
	TArray<FString> ManifestDependenciesList;
	GetPathArrayFromConfig(*SectionName, TEXT("ManifestDependencies"), ManifestDependenciesList, GatherTextConfigPath);
	
	for (const FString& ManifestDependency : ManifestDependenciesList)
	{
		FText OutError;
		if (!GatherManifestHelper->AddDependency(ManifestDependency, &OutError))
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "The GatherTextFromSource commandlet couldn't load the specified manifest dependency: '{manifestDependency}'. {error}",
				("manifestDependency", *ManifestDependency),
				("error", *OutError.ToString()),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	// Get whether we should gather editor-only data. Typically only useful for the localization of UE itself.
	bool ShouldGatherFromEditorOnlyData = false;
	if (!GetBoolFromConfig(*SectionName, TEXT("ShouldGatherFromEditorOnlyData"), ShouldGatherFromEditorOnlyData, GatherTextConfigPath))
	{
		ShouldGatherFromEditorOnlyData = false;
	}

	// Prepass for nested macros
	bool SkipNestedMacroPrepass = Switches.Contains(UGatherTextCommandletBase::SkipNestedMacroPrepassSwitch);
	static TArray<FParsedNestedMacro> PrepassResults;
	static bool RanPrepassOnce = false;
	if (!SkipNestedMacroPrepass && !RanPrepassOnce)
	{
		double StartTime = FPlatformTime::Seconds();

		// We parse all files, since we do not have an include graph to know which are needed for main pass
		TArray<FString> SearchDirectoryPathsPrepass;
		SearchDirectoryPathsPrepass.Add(TEXT("%LOCENGINEROOT%Source"));
		SearchDirectoryPathsPrepass.Add(TEXT("%LOCENGINEROOT%Plugins"));
		if (FApp::HasProjectName())
		{
			SearchDirectoryPathsPrepass.Add(TEXT("%LOCPROJECTROOT%Source"));
			SearchDirectoryPathsPrepass.Add(TEXT("%LOCPROJECTROOT%Plugins"));
		}
		for (FString& Path : SearchDirectoryPathsPrepass)
		{
			ResolveLocalizationPath(Path);
		}
		TArray<FString> ExcludePathFiltersPrepass;
		ExcludePathFiltersPrepass.Add(TEXT("%LOCENGINEROOT%Source/ThirdParty/*"));
		for (FString& Path : ExcludePathFiltersPrepass)
		{
			ResolveLocalizationPath(Path);
		}

		TArray<FString> IncludePathFiltersPrepass;
		TArray<FString> FilesToProcessPrepass;
		TArray<FString> FileNameFiltersPrepass = { TEXT("*.cpp"), TEXT("*.h"), TEXT(".inl") };
		GetFilesToProcess(SearchDirectoryPathsPrepass, FileNameFiltersPrepass, IncludePathFiltersPrepass, ExcludePathFiltersPrepass, FilesToProcessPrepass, false);
		CountFileTypes(FilesToProcessPrepass, EGatherSourcePasses::Prepass);

		RunPass(EGatherSourcePasses::Prepass, ShouldGatherFromEditorOnlyData, FilesToProcessPrepass, GatheredSourceBasePath, PrepassResults);

		double Duration = FPlatformTime::Seconds() - StartTime;
		FileTypeStats[EGatherSourcePasses::Prepass].Duration_sec = Duration;
		UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Ran source prepass for nested macros in %.2f seconds", Duration);

		RanPrepassOnce = true;
	}

	// Mainpass
	{
		double StartTime = FPlatformTime::Seconds();

		RunPass(EGatherSourcePasses::Mainpass, ShouldGatherFromEditorOnlyData, FilesToProcess, GatheredSourceBasePath, PrepassResults);

		double Duration = FPlatformTime::Seconds() - StartTime;
		FileTypeStats[EGatherSourcePasses::Mainpass].Duration_sec = Duration;
		UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Ran source mainpass in %.2f seconds", Duration);
	}

	return 0;
}

void UGatherTextFromSourceCommandlet::GetFilesToProcess(const TArray<FString>& SearchDirectoryPaths, const TArray<FString>& FileNameFilters, TArray<FString>& IncludePathFilters, TArray<FString>& ExcludePathFilters, TArray<FString>& FilesToProcess, bool bAdditionalGatherPaths) const
{
	// Build the final set of include/exclude paths to scan.
	Algo::Transform(SearchDirectoryPaths, IncludePathFilters, [](const FString& SearchDirectoryPath)
	{
		const TCHAR LastChar = SearchDirectoryPath.Len() > 0 ? SearchDirectoryPath[SearchDirectoryPath.Len() - 1] : 0;
		return (LastChar == TEXT('*') || LastChar == TEXT('?'))
			? SearchDirectoryPath								// Already a wildcard
			: FPaths::Combine(SearchDirectoryPath, TEXT("*"));	// Add a catch-all wildcard
	});

	if (bAdditionalGatherPaths)
	{
		FGatherTextContext Context;
		Context.CommandletClass = GetClass()->GetClassPathName();
		Context.PreferredPathType = FGatherTextContext::EPreferredPathType::Root;

		FGatherTextDelegates::GetAdditionalGatherPathsForContext.Broadcast(GatherManifestHelper->GetTargetName(), Context, IncludePathFilters, ExcludePathFilters);
	}

	// Search in the root folder for each of the wildcard filters specified and build a list of files
	{
		class FFileMatch : public IPlatformFile::FDirectoryVisitor
		{
		public:
			const TArray<FString>& WildCards;

			UE::FMutex RootSourceFilesLock;
			TArray<FString> RootSourceFiles;

			FFileMatch(const TArray<FString>& InWildCards)
				: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
				, WildCards(InWildCards)
			{
			}

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
			{
				if (!bIsDirectory)
				{
					FString FullFilename = FilenameOrDirectory;
					FString LeafFilename = FPaths::GetCleanFilename(FullFilename);

					bool bMatchesWildCard = false;
					if (!LeafFilename.EndsWith(TEXTVIEW(".generated.h")) && !LeafFilename.EndsWith(TEXTVIEW(".gen.cpp"))) // Always skip UHT generated files
					{
						for (const FString& WildCard : WildCards)
						{
							if (LeafFilename.MatchesWildcard(WildCard))
							{
								bMatchesWildCard = true;
								break;
							}
						}
					}

					if (bMatchesWildCard)
					{
						UE::TScopeLock ScopeLock(RootSourceFilesLock);
						RootSourceFiles.Add(MoveTemp(FullFilename));
					}
				}

				return true;
			}
		};

		FFileMatch Visitor(FileNameFilters);
		TSet<FString, FLocKeySetFuncs> ProcessedSearchDirectoryPaths;
		for (const FString& IncludePathFilter : IncludePathFilters)
		{
			constexpr FAsciiSet Wildcards("*?");

			FString SearchDirectoryPath = IncludePathFilter;
			if (const TCHAR* FirstWildcard = FAsciiSet::FindFirstOrEnd(*SearchDirectoryPath, Wildcards); *FirstWildcard != 0)
			{
				// Trim the wildcard from this search path
				SearchDirectoryPath = SearchDirectoryPath.Left(UE_PTRDIFF_TO_INT32(FirstWildcard - *SearchDirectoryPath));
				SearchDirectoryPath = FPaths::GetPath(MoveTemp(SearchDirectoryPath));
			}
			if (FPaths::IsRelative(SearchDirectoryPath))
			{
				SearchDirectoryPath = FPaths::ConvertRelativePathToFull(MoveTemp(SearchDirectoryPath));
			}

			bool bAlreadyProcessed = false;
			ProcessedSearchDirectoryPaths.Add(SearchDirectoryPath, &bAlreadyProcessed);
			if (bAlreadyProcessed)
			{
				continue;
			}

			IFileManager::Get().IterateDirectoryRecursively(*SearchDirectoryPath, Visitor);
			FilesToProcess.Append(MoveTemp(Visitor.RootSourceFiles));
			Visitor.RootSourceFiles.Reset();
		}
	}

	const FFuzzyPathMatcher FuzzyPathMatcher = FFuzzyPathMatcher(IncludePathFilters, ExcludePathFilters);
	FilesToProcess.RemoveAll([&FuzzyPathMatcher](const FString& FoundFile)
	{
		// Filter out assets whose package file paths do not pass the "fuzzy path" filters.
		if (FuzzyPathMatcher.TestPath(FoundFile) != FFuzzyPathMatcher::EPathMatch::Included)
		{
			return true;
		}

		return false;
	});
	FilesToProcess.Sort([](const FString& LHS, const FString& RHS)
	{
		return (LHS < RHS);
	});
	// Remove duplicates
	FilesToProcess.SetNum(Algo::Unique(FilesToProcess));
}

void UGatherTextFromSourceCommandlet::GetParsables(TArray<FParsableDescriptor*>& Parsables, EGatherSourcePasses Pass, TArray<FParsedNestedMacro>& PrepassResults)
{
	// Get the loc macros and their syntax
	if (Pass == EGatherSourcePasses::Prepass)
	{
		Parsables.Add(new FNestedMacroPrepassDescriptor(PrepassResults));
	}
	else
	{
		Parsables.Add(new FDefineDescriptor());

		Parsables.Add(new FUndefDescriptor());

		Parsables.Add(new FIfDescriptor());

		Parsables.Add(new FIfDefDescriptor());

		Parsables.Add(new FElIfDescriptor());

		Parsables.Add(new FElseDescriptor());

		Parsables.Add(new FEndIfDescriptor());

		Parsables.Add(new FUICommandMacroDescriptor());

		Parsables.Add(new FUICommandExtMacroDescriptor());
		
		Parsables.Add(new FStringMacroDescriptor( FString(MacroString_NSNOTELOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Namespace, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_DevNotes, true)));
	
		Parsables.Add(new FStringMacroDescriptor( FString(MacroString_NOTELOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_DevNotes, true)));

		// New Localization System with Namespace as literal argument.
		Parsables.Add(new FStringMacroDescriptor(FString(MacroString_NSLOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Namespace, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true)));

		// New Localization System with Namespace as preprocessor define.
		Parsables.Add(new FStringMacroDescriptor(FString(MacroString_LOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true)));

		Parsables.Add(new FStringTableMacroDescriptor());

		Parsables.Add(new FStringTableFromFileMacroDescriptor(TEXT("LOCTABLE_FROMFILE_ENGINE"), FPaths::EngineContentDir()));

		Parsables.Add(new FStringTableFromFileMacroDescriptor(TEXT("LOCTABLE_FROMFILE_GAME"), FPaths::ProjectContentDir()));

		Parsables.Add(new FStringTableEntryMacroDescriptor());

		Parsables.Add(new FStringTableNotedEntryMacroDescriptor());

		Parsables.Add(new FStringTableEntryMetaDataMacroDescriptor());

		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_LOGFMT_LOC"), FStructuredLogMacroDescriptor::EFlags::None));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_LOGFMT_LOC_EX"), FStructuredLogMacroDescriptor::EFlags::None));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_LOGFMT_NSLOC"), FStructuredLogMacroDescriptor::EFlags::Namespace));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_LOGFMT_NSLOC_EX"), FStructuredLogMacroDescriptor::EFlags::Namespace));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_CLOGFMT_LOC"), FStructuredLogMacroDescriptor::EFlags::Condition));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_CLOGFMT_LOC_EX"), FStructuredLogMacroDescriptor::EFlags::Condition));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_CLOGFMT_NSLOC"), FStructuredLogMacroDescriptor::EFlags::Condition | FStructuredLogMacroDescriptor::EFlags::Namespace));
		Parsables.Add(new FStructuredLogMacroDescriptor(TEXT("UE_CLOGFMT_NSLOC_EX"), FStructuredLogMacroDescriptor::EFlags::Condition | FStructuredLogMacroDescriptor::EFlags::Namespace));

		// UE_DECLARE_ERROR(ApiMacro, ModuleName, ErrorCode, ErrorName, FormatString, ...)
		Parsables.Add(new FDeclareErrorDescriptor(TEXT("UE_DECLARE_ERROR"), TEXT("ErrorName"), 5, 1, 3, 4));
		// UE_DECLARE_ERROR_CONTEXT(ApiMacro, ModuleName, ContextName, FormatString, ...)
		Parsables.Add(new FDeclareErrorDescriptor(TEXT("UE_DECLARE_ERROR_CONTEXT"), TEXT("ContextName"), 4, 1, 2, 3));

		Parsables.Add(new FIniNamespaceDescriptor());

		PrunePrepassResults(PrepassResults);

		for (const FParsedNestedMacro& Result : PrepassResults)
		{
			if (!Result.bExclude)
			{
				Parsables.Add(new FNestedMacroDescriptor(Result.MacroName, Result.MacroNameNested, Result.Filename, Result.Content));
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::RunPass(EGatherSourcePasses Pass, bool ShouldGatherFromEditorOnlyData, const TArray<FString>& FilesToProcess, const FString& GatheredSourceBasePath, TArray<FParsedNestedMacro>& PrepassResults)
{
	// Make a batch copy of all the data needed for each core
	// This avoids accessing member functions for thread safety. It also avoids expensive locking and atomics.
	// The only atomics used are FParsedNestedMacroStats, for simplicity.

	const int32 CountCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	const int32 CountThreads = ParallelForImpl::GetNumberOfThreadTasks(CountCores, 1, EParallelForFlags::None);

	struct FBatchPerThread
	{
		TArray<FString> Files;
		TArray<FSourceFileParseContext> Contexts;				// Context per file
		TArray<FParsableDescriptor*> Parsables;
		TArray<FParsedNestedMacro> PrepassResults;				// May be large data, currently 90kb, copy per core only not per file
		TArray<FManifestEntryResult> MainpassResults;
		TMap<FName, FString> SplitPlatforms;
	};
	TArray<FBatchPerThread> Batches;
	Batches.Init(FBatchPerThread(), CountThreads);

	int32 FilesPerCore = FMath::CeilToInt((float)FilesToProcess.Num() / (float)CountThreads);
	int32 FileIndex = 0;
	for (const FString& SourceFile : FilesToProcess)
	{
		int32 Core = FileIndex / FilesPerCore;
		FBatchPerThread& Batch = Batches[Core];

		Batch.Files.Add(SourceFile);

		FSourceFileParseContext ParseCtxt(Batch.SplitPlatforms, Batch.MainpassResults);
		ParseCtxt.Pass = Pass;
		ParseCtxt.ShouldGatherFromEditorOnlyData = ShouldGatherFromEditorOnlyData;
		ParseCtxt.Filename = SourceFile;
		ParseCtxt.FileTypes = ParseCtxt.Filename.EndsWith(TEXT(".ini")) ? EGatherTextSourceFileTypes::Ini : EGatherTextSourceFileTypes::Cpp;
		FPaths::MakePathRelativeTo(ParseCtxt.Filename, *GatheredSourceBasePath);
		ParseCtxt.LineIdx = 0;
		ParseCtxt.LineNumber = 0;
		ParseCtxt.FilePlatformName = GetSplitPlatformNameFromPath(SourceFile);
		ParseCtxt.LineText.Reset();
		ParseCtxt.Namespace.Reset();
		ParseCtxt.RawStringLiteralClosingDelim.Reset();
		ParseCtxt.ExcludedRegion = false;
		ParseCtxt.EndParsingCurrentLine = false;
		ParseCtxt.WithinBlockComment = false;
		ParseCtxt.WithinLineComment = false;
		ParseCtxt.WithinStringLiteral = false;
		ParseCtxt.WithinNamespaceDefineLineNumber = INDEX_NONE;
		ParseCtxt.WithinStartingLine = nullptr;
		ParseCtxt.TextLines.Reset();
		ParseCtxt.FlushMacroStack();
		Batch.Contexts.Add(ParseCtxt);

		++FileIndex;
	}

	// Setup batches
	for (int32 i = 0; i < CountThreads; ++i)
	{
		FBatchPerThread& Batch = Batches[i];

		Batch.PrepassResults = PrepassResults;
		GetParsables(Batch.Parsables, Pass, Batch.PrepassResults);

		Batch.SplitPlatforms = SplitPlatforms;
	}

	ParallelFor(Batches.Num(), [&Batches, &Pass](int32 Index)
	{
		FBatchPerThread& Batch = Batches[Index];
		TArray<FParsableDescriptor*>& Parsables = Batch.Parsables;
		TArray<FParsedNestedMacro>& PrepassResults = Batch.PrepassResults;

		// Parse source files for macros
		for (int32 i = 0; i < Batch.Files.Num(); ++i)
		{
			const FString& SourceFile = Batch.Files[i];
			FSourceFileParseContext& ParseCtxt = Batch.Contexts[i];

			FString SourceFileText;
			if (!FFileHelper::LoadFileToString(SourceFileText, *SourceFile))
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "{failingFile}: GatherTextFromSource failed to open file",
					("failingFile", *ParseCtxt.Filename),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				if (!ParseSourceText(SourceFileText, Parsables, ParseCtxt, PrepassResults))
				{
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{failingFile}: GatherTextSource error(s) parsing source file",
						("failingFile", *ParseCtxt.Filename),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
				}
				else if (ParseCtxt.WithinNamespaceDefineLineNumber != INDEX_NONE)
				{
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Missing '#undef LOCTEXT_NAMESPACE' for '#define LOCTEXT_NAMESPACE'",
						("file", *ParseCtxt.Filename),
						("line", ParseCtxt.WithinNamespaceDefineLineNumber),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
				}
			}
		}
	});

	// Collect results from batches
	for (int32 i = 0; i < CountThreads; ++i)
	{
		const FBatchPerThread& Batch = Batches[i];

		for (int32 j = 0; j < Batch.Files.Num(); ++j)
		{
			const FSourceFileParseContext& ParseCtxt = Batch.Contexts[j];

			// Process any parsed string tables
			for (const auto& ParsedStringTablePair : ParseCtxt.ParsedStringTables)
			{
				if (ParsedStringTablePair.Value.SourceLocation.Line == INDEX_NONE)
				{
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "String table with ID '{stringTableID}' had {nbEntries} entries parsed for it, but the table was never registered. Skipping for gather.",
						("stringTableID", *ParsedStringTablePair.Key.ToString()),
						("nbEntries", ParsedStringTablePair.Value.TableEntries.Num()),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
				}
				else
				{
					for (const auto& ParsedStringTableEntryPair : ParsedStringTablePair.Value.TableEntries)
					{
						if ((!ParsedStringTableEntryPair.Value.bIsEditorOnly || ParseCtxt.ShouldGatherFromEditorOnlyData) && FText::ShouldGatherForLocalization(ParsedStringTableEntryPair.Value.SourceString))
						{
							FManifestContext SourceContext;
							SourceContext.Key = ParsedStringTableEntryPair.Key;
							SourceContext.SourceLocation = ParsedStringTableEntryPair.Value.SourceLocation.ToString();
							SourceContext.PlatformName = ParsedStringTableEntryPair.Value.PlatformName;
							SourceContext.DevNotes = ParsedStringTableEntryPair.Value.DevNotes;

							const FParsedStringTableEntryMetaDataMap* ParsedMetaDataMap = ParsedStringTablePair.Value.MetaDataEntries.Find(ParsedStringTableEntryPair.Key);
							if (ParsedMetaDataMap && ParsedMetaDataMap->Num() > 0)
							{
								SourceContext.InfoMetadataObj = MakeShareable(new FLocMetadataObject());
								for (const auto& ParsedMetaDataPair : *ParsedMetaDataMap)
								{
									if (!ParsedMetaDataPair.Value.bIsEditorOnly || ParseCtxt.ShouldGatherFromEditorOnlyData)
									{
										SourceContext.InfoMetadataObj->SetStringField(ParsedMetaDataPair.Key.ToString(), ParsedMetaDataPair.Value.MetaData);
									}
								}
							}

							GatherManifestHelper->AddSourceText(ParsedStringTablePair.Value.TableNamespace, FLocItem(ParsedStringTableEntryPair.Value.SourceString), SourceContext);
						}
					}
				}
			}
		}

		if (Pass == EGatherSourcePasses::Prepass)
		{
			// Collect prepass results from Batches
			PrepassResults.Append(Batch.PrepassResults);
		}
		else if (Pass == EGatherSourcePasses::Mainpass)
		{
			// Submit mainpass results to manifest helper
			for (const FManifestEntryResult& Result : Batch.MainpassResults)
			{
				if (FText::ShouldGatherForLocalization(Result.Source))
				{
					GatherManifestHelper->AddSourceText(Result.Namespace, FLocItem(Result.Source), Result.Context, &Result.Description);
				}
			}
		}

		// Clear parsables list safely
		for (int32 j = 0; j < Batch.Parsables.Num(); ++j)
		{
			delete Batch.Parsables[j];
		}
	}
}

FString UGatherTextFromSourceCommandlet::UnescapeLiteralCharacterEscapeSequences(const FString& InString)
{
	// We need to un-escape any octal, hex, or universal character sequences that exist in this string to mimic what happens when the string is processed by the compiler
	enum class EParseState : uint8
	{
		Idle,		// Not currently parsing a sequence
		InOct,		// Within an octal sequence (\012)
		InHex,		// Within an hexadecimal sequence (\xBEEF)
		InUTF16,	// Within a UTF-16 sequence (\u1234)
		InUTF32,	// Within a UTF-32 sequence (\U12345678)
	};

	FString RetString;
	RetString.Reserve(InString.Len());

	EParseState ParseState = EParseState::Idle;
	FString EscapedLiteralCharacter;
	for (const TCHAR* CharPtr = *InString; ; ++CharPtr)
	{
		const TCHAR CurChar = *CharPtr;

		if (ParseState == EParseState::Idle && CurChar == 0)
		{
			// End of string
			break;
		}

		switch (ParseState)
		{
		case EParseState::Idle:
			{
				const TCHAR NextChar = *(CharPtr + 1);
				if (CurChar == TEXT('\\') && NextChar)
				{
					if (FChar::IsOctDigit(NextChar))
					{
						ParseState = EParseState::InOct;
					}
					else if (NextChar == TEXT('x'))
					{
						// Skip the format marker
						++CharPtr;
						ParseState = EParseState::InHex;
					}
					else if (NextChar == TEXT('u'))
					{
						// Skip the format marker
						++CharPtr;
						ParseState = EParseState::InUTF16;
					}
					else if (NextChar == TEXT('U'))
					{
						// Skip the format marker
						++CharPtr;
						ParseState = EParseState::InUTF32;
					}
				}
				
				if (ParseState == EParseState::Idle)
				{
					RetString.AppendChar(CurChar);
				}
				else
				{
					EscapedLiteralCharacter.Reset();
				}
			}
			break;

		case EParseState::InOct:
			{
				if (FChar::IsOctDigit(CurChar))
				{
					EscapedLiteralCharacter.AppendChar(CurChar);

					// Octal sequences can only be up-to 3 digits long
					check(EscapedLiteralCharacter.Len() <= 3);
					if (EscapedLiteralCharacter.Len() == 3)
					{
						RetString.AppendChar((TCHAR)FCString::Strtoi(*EscapedLiteralCharacter, nullptr, 8));
						ParseState = EParseState::Idle;
						// Deliberately not appending the current character here, as it was already pushed into the escaped literal character string
					}
				}
				else
				{
					if (EscapedLiteralCharacter.Len() > 0)
					{
						RetString.AppendChar((TCHAR)FCString::Strtoi(*EscapedLiteralCharacter, nullptr, 8));
					}
					ParseState = EParseState::Idle;
					--CharPtr; // Walk backwards as we need to consider whether the current character is the start of a new escape sequence
				}
			}
			break;

		case EParseState::InHex:
			{
				if (FChar::IsHexDigit(CurChar))
				{
					EscapedLiteralCharacter.AppendChar(CurChar);
				}
				else
				{
					if (EscapedLiteralCharacter.Len() > 0)
					{
						RetString.AppendChar((TCHAR)FCString::Strtoi(*EscapedLiteralCharacter, nullptr, 16));
					}
					ParseState = EParseState::Idle;
					--CharPtr; // Walk backwards as we need to consider whether the current character is the start of a new escape sequence
				}
			}
			break;

		case EParseState::InUTF16:
			{
				if (FChar::IsHexDigit(CurChar))
				{
					EscapedLiteralCharacter.AppendChar(CurChar);

					// UTF-16 sequences can only be up-to 4 digits long
					check(EscapedLiteralCharacter.Len() <= 4);
					if (EscapedLiteralCharacter.Len() == 4)
					{
						const uint32 UnicodeCodepoint = (uint32)FCString::Strtoi(*EscapedLiteralCharacter, nullptr, 16);

						FString UnicodeString;
						if (FTextChar::ConvertCodepointToString(UnicodeCodepoint, UnicodeString))
						{
							RetString.Append(MoveTemp(UnicodeString));
						}

						ParseState = EParseState::Idle;
						// Deliberately not appending the current character here, as it was already pushed into the escaped literal character string
					}
				}
				else
				{
					if (EscapedLiteralCharacter.Len() > 0)
					{
						const uint32 UnicodeCodepoint = (uint32)FCString::Strtoi(*EscapedLiteralCharacter, nullptr, 16);

						FString UnicodeString;
						if (FTextChar::ConvertCodepointToString(UnicodeCodepoint, UnicodeString))
						{
							RetString.Append(MoveTemp(UnicodeString));
						}
					}

					ParseState = EParseState::Idle;
					--CharPtr; // Walk backwards as we need to consider whether the current character is the start of a new escape sequence
				}
			}
			break;

		case EParseState::InUTF32:
			{
				if (FChar::IsHexDigit(CurChar))
				{
					EscapedLiteralCharacter.AppendChar(CurChar);

					// UTF-32 sequences can only be up-to 8 digits long
					check(EscapedLiteralCharacter.Len() <= 8);
					if (EscapedLiteralCharacter.Len() == 8)
					{
						const uint32 UnicodeCodepoint = (uint32)FCString::Strtoui64(*EscapedLiteralCharacter, nullptr, 16);

						FString UnicodeString;
						if (FTextChar::ConvertCodepointToString(UnicodeCodepoint, UnicodeString))
						{
							RetString.Append(MoveTemp(UnicodeString));
						}

						ParseState = EParseState::Idle;
						// Deliberately not appending the current character here, as it was already pushed into the escaped literal character string
					}
				}
				else
				{
					if (EscapedLiteralCharacter.Len() > 0)
					{
						const uint32 UnicodeCodepoint = (uint32)FCString::Strtoui64(*EscapedLiteralCharacter, nullptr, 16);

						FString UnicodeString;
						if (FTextChar::ConvertCodepointToString(UnicodeCodepoint, UnicodeString))
						{
							RetString.Append(MoveTemp(UnicodeString));
						}
					}

					ParseState = EParseState::Idle;
					--CharPtr; // Walk backwards as we need to consider whether the current character is the start of a new escape sequence
				}
			}
			break;

		default:
			break;
		}
	}

	return RetString.ReplaceEscapedCharWithChar();
}

FString UGatherTextFromSourceCommandlet::RemoveStringFromTextMacro(const FString& TextMacro, const FString& IdentForLogging, bool& Error)
{
	FString Text;
	Error = true;

	// need to strip text literal out of TextMacro ( format should be TEXT("stringvalue") ) 
	if (!TextMacro.StartsWith(FMacroDescriptor::TextMacroString))
	{
		Error = false;
		Text = TextMacro.TrimQuotes();
	}
	else
	{
		int32 OpenQuoteIdx = TextMacro.Find(TEXT("\""), ESearchCase::CaseSensitive);
		if (0 > OpenQuoteIdx || TextMacro.Len() - 1 == OpenQuoteIdx)
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "Missing quotes in {text}",
				("text", *FLocTextHelper::SanitizeLogOutput(IdentForLogging)),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
		else
		{
			int32 CloseQuoteIdx = TextMacro.Find(TEXT("\""), ESearchCase::CaseSensitive, ESearchDir::FromStart, OpenQuoteIdx+1);
			if (0 > CloseQuoteIdx)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "Missing quotes in {text}",
					("text", *FLocTextHelper::SanitizeLogOutput(IdentForLogging)),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Text = TextMacro.Mid(OpenQuoteIdx + 1, CloseQuoteIdx - OpenQuoteIdx - 1);
				Error = false;
			}
		}
	}

	if (!Error)
	{
		Text = UnescapeLiteralCharacterEscapeSequences(Text);
	}

	return Text;
}

FString UGatherTextFromSourceCommandlet::StripCommentsFromToken(const FString& InToken, FSourceFileParseContext& Context)
{
	check(!Context.WithinLineComment);
	check(!Context.WithinStringLiteral);

	// Remove both block and inline comments from the given token
	FString StrippedToken;
	StrippedToken.Reserve(InToken.Len());

	TCHAR WithinQuote = 0;
	bool bIgnoreNextQuote = false;
	for (const TCHAR* Char = *InToken; *Char; ++Char)
	{
		if (WithinQuote != 0)
		{
			StrippedToken += *Char;

			if (!bIgnoreNextQuote)
			{
				if (*Char == TEXT('\\'))
				{
					bIgnoreNextQuote = true;
					continue;
				}

				if (*Char == WithinQuote)
				{
					// Found an unescaped closing quote - we are no longer within quotes
					WithinQuote = 0;
				}
			}

			bIgnoreNextQuote = false;
		}
		else
		{
			if (*Char == TEXT('/'))
			{
				const TCHAR* NextChar = Char + 1;

				if (*NextChar == TEXT('/'))
				{
					// Found an inline quote - this strips the remainder of the string so just break out of the loop
					break;
				}

				if (*NextChar == TEXT('*'))
				{
					// Found a block comment - skip all characters until we find the closing quote
					Context.WithinBlockComment = true;
					++Char; // Skip over the opening slash, and the for loop will skip over the *
					continue;
				}
			}

			if (Context.WithinBlockComment)
			{
				if (*Char == TEXT('*'))
				{
					const TCHAR* NextChar = Char + 1;

					if (*NextChar == TEXT('/'))
					{
						// Found the end of a block comment
						Context.WithinBlockComment = false;
						++Char; // Skip over the opening *, and the for loop will skip over the slash
						continue;
					}
				}

				// Skip over all characters while within a block comment
				continue;
			}

			StrippedToken += *Char;

			if (*Char == TEXT('"') || *Char == TEXT('\''))
			{
				// We found an opening quote - keep track of it until we find a matching closing quote
				WithinQuote = *Char;
			}
		}
	}

	return StrippedToken.TrimStartAndEnd();
}

bool UGatherTextFromSourceCommandlet::ParseSourceText(const FString& Text, const TArray<FParsableDescriptor*>& Parsables, FSourceFileParseContext& ParseCtxt, TArray<FParsedNestedMacro>& PrepassResults)
{
	// Cache array of parsables and tokens valid for this filetype
	TArray< FParsableDescriptor*> ParsablesForFile;
	TArray<FString> ParsableTokensForFile;
	for (FParsableDescriptor* Parsable : Parsables)
	{
		if (Parsable->IsApplicableFileType(ParseCtxt.FileTypes) && Parsable->IsApplicableFile(ParseCtxt.Filename))
		{
			ParsablesForFile.Add(Parsable);
			ParsableTokensForFile.Add(Parsable->GetToken());
		}
	}
	check(ParsablesForFile.Num() == ParsableTokensForFile.Num());

	// Anything to parse for this filetype?
	if (ParsablesForFile.Num() == 0)
	{
		return true;
	}

	// Create array of ints, one for each parsable we're looking for.
	TArray<int32> ParsableMatchCountersForFile;
	ParsableMatchCountersForFile.AddZeroed(ParsablesForFile.Num());

	// Use the file extension to work out what comments look like for this file
	// We default to C++-style comments
	const TCHAR* LineComment = TEXT("//");
	const TCHAR* BlockCommentStart = TEXT("/*");
	const TCHAR* BlockCommentEnd = TEXT("*/");
	if (EnumHasAnyFlags(ParseCtxt.FileTypes, EGatherTextSourceFileTypes::Ini))
	{
		LineComment = TEXT(";");
		BlockCommentStart = nullptr;
		BlockCommentEnd = nullptr;
	}
	const int32 LineCommentLen = LineComment ? FCString::Strlen(LineComment) : 0;
	const int32 BlockCommentStartLen = BlockCommentStart ? FCString::Strlen(BlockCommentStart) : 0;
	const int32 BlockCommentEndLen = BlockCommentEnd ? FCString::Strlen(BlockCommentEnd) : 0;
	checkf((BlockCommentStartLen == 0 && BlockCommentEndLen == 0) || (BlockCommentStartLen > 0 && BlockCommentEndLen > 0), TEXT("Block comments require both a start and an end marker!"));

	// Split the file into lines
	TArray<FString>& TextLines = ParseCtxt.TextLines;
	Text.ParseIntoArrayLines(TextLines, false);

	// Move through the text lines looking for the tokens that denote the items in the Parsables list 
	int32& LineIdx = ParseCtxt.LineIdx;
	for (LineIdx = 0; LineIdx < TextLines.Num(); LineIdx++)
	{
		// Remove spaces at the end of the line.
		TextLines[LineIdx].TrimEndInline();
		const FString& Line = TextLines[LineIdx];
		if (Line.IsEmpty())
		{
			continue;
		}
		ParseCtxt.LineNumber = LineIdx + 1;

		// Skip any lines handled in prepass
		int32 AdvanceByLines = 0;
		if ((ParseCtxt.Pass != EGatherSourcePasses::Prepass) &&
			HandledInPrepass(PrepassResults, ParseCtxt.Filename, ParseCtxt.LineNumber, AdvanceByLines))
		{
			LineIdx += AdvanceByLines;
			continue;
		}

		// Use these pending vars to defer parsing a token hit until longer tokens can't hit too
		int32 PendingParseIdx = INDEX_NONE;
		const TCHAR* ParsePoint = NULL;
		for (int32& Element : ParsableMatchCountersForFile)
		{
			Element = 0;
		}
		ParseCtxt.LineText = Line;
		ParseCtxt.WithinLineComment = false;
		ParseCtxt.EndParsingCurrentLine = false;

		const TCHAR* Cursor = *Line;
		while (*Cursor && !ParseCtxt.EndParsingCurrentLine)
		{
			// Check if we're starting comments or string literals

			if (!ParseCtxt.WithinLineComment && !ParseCtxt.WithinBlockComment && !ParseCtxt.WithinStringLiteral)
			{
				// Detect that the line starts with a Line comment.
				if (LineCommentLen > 0 && FCString::Strncmp(Cursor, LineComment, LineCommentLen) == 0)
				{
					ParseCtxt.WithinLineComment = true;
					ParseCtxt.WithinStartingLine = *Line;
					ParseCtxt.EndParsingCurrentLine = true;
					Cursor += LineCommentLen;
					continue;
				}
				else if (BlockCommentStartLen > 0 && FCString::Strncmp(Cursor, BlockCommentStart, BlockCommentStartLen) == 0)
				{
					ParseCtxt.WithinBlockComment = true;
					ParseCtxt.WithinStartingLine = *Line;
					Cursor += BlockCommentStartLen;
					continue;
				}
			}

			// We are not in a comment (line or block) and we are not parsing a string.
			if (!ParseCtxt.WithinLineComment && !ParseCtxt.WithinBlockComment && !ParseCtxt.WithinStringLiteral)
			{
				if (*Cursor == TEXT('\"'))
				{
					if (Cursor == *Line)
					{
						ParseCtxt.WithinStringLiteral = true;
						ParseCtxt.WithinStartingLine = *Line;
						++Cursor;
						continue;
					}
					else if (Cursor > *Line)
					{
						const TCHAR* const ReverseCursor = Cursor - 1;
						if (EnumHasAnyFlags(ParseCtxt.FileTypes, EGatherTextSourceFileTypes::Cpp) && *ReverseCursor == TEXT('R'))
						{
							// Potentially a C++11 raw string literal, so walk forwards and validate that this looks legit
							// While doing this we can parse out its optional user defined delimiter so we can find when the string closes
							//   eg) For 'R"Delim(string)Delim"', ')Delim' would be the closing delimiter.
							//   eg) For 'R"(string)"', ')' would be the closing delimiter.
							ParseCtxt.RawStringLiteralClosingDelim = TEXT(")");
							{
								bool bIsValid = true;

								const TCHAR* ForwardCursor = Cursor + 1;
								for (;;)
								{
									const TCHAR DelimChar = *ForwardCursor++;
									if (DelimChar == TEXT('('))
									{
										break;
									}
									// We also permit '_' to support the use of _JSON as a delimiter for the raw strings. Also, '|' and '!' are common.
									if (DelimChar == 0 || !(FChar::IsAlnum(DelimChar) || DelimChar == TCHAR('_') || DelimChar == TCHAR('|') || DelimChar == TCHAR('!')))
									{
										bIsValid = false;
										break;
									}
									ParseCtxt.RawStringLiteralClosingDelim += DelimChar;
								}

								if (bIsValid)
								{
									ParseCtxt.WithinStringLiteral = true;
									ParseCtxt.WithinStartingLine = *Line;
									Cursor = ForwardCursor;
									continue;
								}
								else
								{
									ParseCtxt.RawStringLiteralClosingDelim.Reset();
									// Fall through to the quoted string parsing below
								}
							}
						}
						
						if (*ReverseCursor != TEXT('\\') && *ReverseCursor != TEXT('\''))
						{
							ParseCtxt.WithinStringLiteral = true;
							ParseCtxt.WithinStartingLine = *Line;
							++Cursor;
							continue;
						}
						else 
						{
							bool IsEscaped = false;
							{
								//if the backslash or single quote is itself escaped then the quote is good
								const TCHAR* EscapeCursor = ReverseCursor;
								while (EscapeCursor > *Line && *(--EscapeCursor) == TEXT('\\'))
								{
									IsEscaped = !IsEscaped;
								}
							}

							if (IsEscaped)
							{
								ParseCtxt.WithinStringLiteral = true;
								ParseCtxt.WithinStartingLine = *Line;
								++Cursor;
								continue;
							}
							else
							{
								//   check for '"'
								const TCHAR* const ForwardCursor = Cursor + 1;
								if (*ReverseCursor == TEXT('\'') && *ForwardCursor != TEXT('\''))
								{
									ParseCtxt.WithinStringLiteral = true;
									ParseCtxt.WithinStartingLine = *Line;
									++Cursor;
									continue;
								}
							}
						}
					}
				}
			}
			else if (ParseCtxt.WithinStringLiteral)
			{
				if (*Cursor == TEXT('\"'))
				{
					if (Cursor == *Line && ParseCtxt.RawStringLiteralClosingDelim.IsEmpty())
					{
						ParseCtxt.WithinStringLiteral = false;
						++Cursor;
						continue;
					}
					else if (Cursor > *Line)
					{
						// Is this ending a C++11 raw string literal?
						if (!ParseCtxt.RawStringLiteralClosingDelim.IsEmpty())
						{
							const TCHAR* EndDelimCursor = Cursor - ParseCtxt.RawStringLiteralClosingDelim.Len();
							if (EndDelimCursor >= *Line && FCString::Strncmp(EndDelimCursor, *ParseCtxt.RawStringLiteralClosingDelim, ParseCtxt.RawStringLiteralClosingDelim.Len()) == 0)
							{
								ParseCtxt.RawStringLiteralClosingDelim.Reset();
								ParseCtxt.WithinStringLiteral = false;
							}
							++Cursor;
							continue;
						}

						const TCHAR* const ReverseCursor = Cursor - 1;
						if (*ReverseCursor != TEXT('\\') && *ReverseCursor != TEXT('\''))
						{
							ParseCtxt.WithinStringLiteral = false;
							++Cursor;
							continue;
						}
						else
						{
							bool IsEscaped = false;
							{
								//if the backslash or single quote is itself escaped then the quote is good
								const TCHAR* EscapeCursor = ReverseCursor;
								while (EscapeCursor > *Line && *(--EscapeCursor) == TEXT('\\'))
								{
									IsEscaped = !IsEscaped;
								}
							}

							if (IsEscaped)
							{
								ParseCtxt.WithinStringLiteral = false;
								++Cursor;
								continue;
							}
							else
							{
								//   check for '"'
								const TCHAR* const ForwardCursor = Cursor + 1;
								if (*ReverseCursor == TEXT('\'') && *ForwardCursor != TEXT('\''))
								{
									ParseCtxt.WithinStringLiteral = false;
									++Cursor;
									continue;
								}
							}
						}
					}
				}
			}

			// Check if we're ending comments
			if (ParseCtxt.WithinBlockComment && BlockCommentEndLen > 0 && FCString::Strncmp(Cursor, BlockCommentEnd, BlockCommentEndLen) == 0)
			{
				ParseCtxt.WithinBlockComment = false;
				Cursor += BlockCommentEndLen;
				continue;
			}

			// Go through all the Parsables to find matches
			for (int32 ParIdx = 0; ParIdx < ParsablesForFile.Num(); ++ParIdx)
			{
				FParsableDescriptor* Parsable = ParsablesForFile[ParIdx];
				const FString& Token = ParsableTokensForFile[ParIdx];

				if (Token.Len() == ParsableMatchCountersForFile[ParIdx])
				{
					// already seen this entire token and are looking for longer matches - skip it
					continue;
				}

				if (*Cursor == Token[ParsableMatchCountersForFile[ParIdx]])
				{
					// Char at cursor matches the next char in the parsable's identifying token
					if (Token.Len() == ++(ParsableMatchCountersForFile[ParIdx]))
					{
						// don't immediately parse - this parsable has seen its entire token but a longer one could be about to hit too
						const TCHAR* TokenStart = Cursor + 1 - Token.Len();
						if (0 > PendingParseIdx || ParsePoint >= TokenStart)
						{
							PendingParseIdx = ParIdx;
							ParsePoint = TokenStart;
						}
					}
				}
				else
				{
					// Char at cursor doesn't match the next char in the parsable's identifying token
					// Reset the counter to start of the token
					ParsableMatchCountersForFile[ParIdx] = 0;
				}
			}

			// Now check PendingParse and only run it if there are no better candidates
			if (PendingParseIdx != INDEX_NONE)
			{
				FParsableDescriptor* PendingParsable = ParsablesForFile[PendingParseIdx];

				bool MustDefer = false; // pending will be deferred if another parsable has a equal and greater number of matched chars
				if (!PendingParsable->OverridesLongerTokens())
				{
					for (int32 ParIdx = 0; ParIdx < ParsablesForFile.Num(); ++ParIdx)
					{
						if (PendingParseIdx != ParIdx && ParsableMatchCountersForFile[ParIdx] >= ParsableTokensForFile[PendingParseIdx].Len())
						{
							// a longer token is matching so defer
							MustDefer = true;
						}
					}
				}

				if (!MustDefer)
				{
					// Do the parse now
					// TODO: Would be nice if TryParse returned what it consumed, and operated on const TCHAR*
					PendingParsable->TryParse(FString(ParsePoint), ParseCtxt);
					for (int32& Element : ParsableMatchCountersForFile)
					{
						Element = 0;
					}
					PendingParseIdx = INDEX_NONE;
					ParsePoint = NULL;
				}
			}

			// Advance cursor
			++Cursor;
		}

		// Handle a string literal that went beyond a single line
		if (ParseCtxt.WithinStringLiteral)
		{
			if (EnumHasAnyFlags(ParseCtxt.FileTypes, EGatherTextSourceFileTypes::Ini))
			{
				// INI files don't support multi-line literals; always terminate them after ending a line
				ParseCtxt.WithinStringLiteral = false;
			}
			else if (Cursor > *Line && ParseCtxt.RawStringLiteralClosingDelim.IsEmpty())
			{
				// C++ only allows multi-line literals if they're escaped with a trailing slash or within a C++11 raw string literal
				ParseCtxt.WithinStringLiteral = *(Cursor - 1) == TEXT('\\');
			}

			UE_CLOGFMT(!ParseCtxt.WithinStringLiteral, LogGatherTextFromSourceCommandlet, Warning, "A string literal was not correctly terminated. File {file} at line {line}, starting line: {startingLine}",
				("file", *ParseCtxt.Filename),
				("line", ParseCtxt.LineNumber),
				("startingLine", ParseCtxt.WithinStartingLine),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	// Handle a string C++11 raw string literal that was never closed as this is likely a false positive that needs to be fixed in the parser
	if (ParseCtxt.WithinStringLiteral && !ParseCtxt.RawStringLiteralClosingDelim.IsEmpty())
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): A C++11 raw string literal was not correctly terminated.",
			("file", *ParseCtxt.Filename),
			("line", ParseCtxt.WithinStartingLine),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
	}

	return true;
}

void UGatherTextFromSourceCommandlet::CountFileTypes(const TArray<FString>& FilesToProcess, EGatherSourcePasses Pass)
{
	FileTypeStats[Pass].FileType_total = FilesToProcess.Num();

	for (const FString& SourceFile : FilesToProcess)
	{
		if (SourceFile.EndsWith(TEXT(".h"), ESearchCase::IgnoreCase))
		{
			++FileTypeStats[Pass].FileType_h;
		}
		else if (SourceFile.EndsWith(TEXT(".inl"), ESearchCase::IgnoreCase))
		{
			++FileTypeStats[Pass].FileType_inl;
		}
		else if (SourceFile.EndsWith(TEXT(".c"), ESearchCase::IgnoreCase))
		{
			++FileTypeStats[Pass].FileType_c;
		}
		else if (SourceFile.EndsWith(TEXT(".cpp"), ESearchCase::IgnoreCase))
		{
			++FileTypeStats[Pass].FileType_cpp;
		}
		else if (SourceFile.EndsWith(TEXT(".ini"), ESearchCase::IgnoreCase))
		{
			++FileTypeStats[Pass].FileType_ini;
		}
		else
		{
			++FileTypeStats[Pass].FileType_other;
		}
	}
}

void UGatherTextFromSourceCommandlet::PrunePrepassResults(TArray<FParsedNestedMacro>& Results)
{
	Results.Sort([](const FParsedNestedMacro& LHS, const FParsedNestedMacro& RHS)
	{
		return (LHS.MacroName < RHS.MacroName);
	});

	{
		// Find exact duplicates
		NestedMacroStats.DuplicateExact = 0;
		for (int32 i = 0; i < Results.Num() - 1; ++i)
		{
			int32 j = i + 1;

			if (Results[i] == Results[j])
			{
				++NestedMacroStats.DuplicateExact;
			}
		}
	}

	// Remove exact duplicates
	Results.SetNum(Algo::Unique(Results));

	{
		// Find duplicates with the same macro name and contained macro
		NestedMacroStats.DuplicateMacroName = 0;
		NestedMacroStats.DuplicateExcluded = 0;

		for (int32 i = 0; i < Results.Num() - 1; ++i)
		{
			int32 j = i + 1;

			if (Results[i].MacroName == Results[j].MacroName &&
				Results[i].MacroNameNested == Results[j].MacroNameNested)
			{
				++NestedMacroStats.DuplicateMacroName;

				// If the duplicate macros are in header files, mark them for exclusion.
				// We exclude header (.h) files only, because we don't have a full include graph to determine where they are used, their scope.
				// In comparison, macros in translation units (.cpp files) are limited in scope to the same file.
				// Without an include graph, the pragmatic solution is to give these macros unique names.
				// We mark them excluded as opposed to removing them, because the regular macro descriptors need to check if they are nested.

				if (Results[i].Filename.EndsWith(TEXT(".h"), ESearchCase::IgnoreCase) ||
					Results[j].Filename.EndsWith(TEXT(".h"), ESearchCase::IgnoreCase))
				{
					++NestedMacroStats.DuplicateExcluded;

					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "Excluding duplicate {macroName} macros in header files: {file}({line}) and {conflictFile}({conflictLine}).",
						("macroName", *Results[i].MacroName),
						("file", *Results[i].Filename),
						("line", Results[i].LineStart),
						("conflictFile", *Results[j].Filename),
						("conflictLine", Results[j].LineStart),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);

					Results[i].bExclude = true;
					Results[j].bExclude = true;
				}
			}
		}
	}

	// Collect size of results, to know whether it's reasonable to make a copy per thread (to avoid locking and atomics)
	NestedMacroStats.SizeInBytes = 0;
	for (const FParsedNestedMacro& Result : Results)
	{
		NestedMacroStats.SizeInBytes += FParsedNestedMacro::Size(Result);
	}
}

bool UGatherTextFromSourceCommandlet::HandledInPrepass(const TArray<FParsedNestedMacro>& Results, const FString& Filename, int32 LineNumber, int32& AdvanceByLines)
{
	AdvanceByLines = 0;

	// Check whether this Filename+Linenumber was handled in prepass
	for (const FParsedNestedMacro& Result : Results)
	{
		if (LineNumber == Result.LineStart &&
			Filename == Result.Filename)
		{
			AdvanceByLines = (Result.LineCount - 1);
			return true;
		}
	}
	return false;
}

void UGatherTextFromSourceCommandlet::LogStats()
{
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "---Gather Source Stats------------------------");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "---Prepass------------------------------------");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.h =                   %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_h.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.inl =                 %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_inl.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.c =                   %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_c.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.cpp =                 %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_cpp.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.ini =                 %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_ini.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Other =                 %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_other.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Total =                 %14d files", FileTypeStats[EGatherSourcePasses::Prepass].FileType_total.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Duration =              %14.2f seconds", FileTypeStats[EGatherSourcePasses::Prepass].Duration_sec.load());

	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "---Mainpass-----------------------------------");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.h =                   %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_h.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.inl =                 %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_inl.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.c =                   %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_c.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.cpp =                 %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_cpp.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "*.ini =                 %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_ini.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Other =                 %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_other.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Total =                 %14d files", FileTypeStats[EGatherSourcePasses::Mainpass].FileType_total.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Duration =              %14.2f seconds", FileTypeStats[EGatherSourcePasses::Mainpass].Duration_sec.load());

	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "---Nested Macro-------------------------------");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "DuplicateExact =        %14d", NestedMacroStats.DuplicateExact.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "DuplicateMacroName =    %14d", NestedMacroStats.DuplicateMacroName.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "DuplicateExcluded =     %14d", NestedMacroStats.DuplicateExcluded.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Concatenation =         %14d", NestedMacroStats.Concatenation.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Variadic =              %14d", NestedMacroStats.Variadic.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "SizeInBytes =           %14d bytes", NestedMacroStats.SizeInBytes.load());

	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_LOCTEXT =        %14d", NestedMacroStats.Nested_LOCTEXT.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_NSLOCTEXT =      %14d", NestedMacroStats.Nested_NSLOCTEXT.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_NOTELOCTEXT =    %14d", NestedMacroStats.Nested_NOTELOCTEXT.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_NSNOTELOCTEXT =  %14d", NestedMacroStats.Nested_NSNOTELOCTEXT.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_UI_COMMAND =     %14d", NestedMacroStats.Nested_UI_COMMAND.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Nested_UI_COMMAND_EXT = %14d", NestedMacroStats.Nested_UI_COMMAND_EXT.load());

	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "PrepassBegin =          %14d", NestedMacroStats.PrepassBegin.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "PrepassEnd =            %14d", NestedMacroStats.PrepassEnd.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "MainpassBegin =         %14d", NestedMacroStats.MainpassBegin.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "MainpassMid =           %14d", NestedMacroStats.MainpassMid.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "MainpassEnd =           %14d", NestedMacroStats.MainpassEnd.load());

	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "");
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "Submit =                %14d", NestedMacroStats.Submit.load());
	UE_LOGF(LogGatherTextFromSourceCommandlet, Display, "SubmitNested =          %14d", NestedMacroStats.SubmitNested.load());
}

int32 UGatherTextFromSourceCommandlet::FMacroArgumentGatherer::GetNumberOfArguments() const 
{
	return Args.Num();
}


bool UGatherTextFromSourceCommandlet::FMacroArgumentGatherer::EndArgument()
{
	if (CurrentArgument.IsEmpty() || IsInDoubleQuotes())
	{
		return false;
	}
	Args.Push(CurrentArgument);
	CurrentArgument.Empty();
	return true;
}

bool UGatherTextFromSourceCommandlet::FMacroArgumentGatherer::Gather(const TCHAR* Arg, int32 Count)
{
	if(Arg == nullptr || Count == 0)
	{
		return false;
	}
	FString NewArgument = FString::ConstructFromPtrSize(Arg, Count);
	NewArgument.TrimEndInline();

	int32 CurrentArgLen = CurrentArgument.Len();
	if (CurrentArgLen == 0)
	{
		CurrentArgument = NewArgument;
		return true;
	}
	
	// If the last argument is a string that ends with " and the new argument too we append it
	// This is to support this and remove duplicate double quotes.
	// MYMACRO("Very long string\n"
	//         "Next part of the very long string\n");
	// And
	// MYMACRO("Very long string\n" \
	//         "Next part of the very long string\n");
	if (NewArgument[0] == '\"' && CurrentArgument[CurrentArgLen - 1] == '\"')
	{
		Arg++;
		CurrentArgLen--;
		CurrentArgument.RemoveAt(CurrentArgLen);
		Count--;
	}

	CurrentArgument.Append(Arg, Count);
	CurrentArgument.TrimEndInline();
	return true;
}
	

void UGatherTextFromSourceCommandlet::FMacroArgumentGatherer::ExtractArguments(TArray<FString>& Arguments)
{
	Arguments = Args;
	Args.Empty();
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddManifestText(const FString& Token, const FString& InNamespace, const FString& SourceText, const FManifestContext& Context, bool IsNested)
{
	const bool bIsEditorOnly = EvaluateEditorOnlyDefineState() == EEditorOnlyDefineState::Defined;

	if (!bIsEditorOnly || ShouldGatherFromEditorOnlyData)
	{
		const FString EntryDescription = FString::Printf(TEXT("%s macro"), *Token);

		MainpassResults.Emplace(InNamespace, SourceText, Context, EntryDescription);

		if (IsNested)
		{
			++NestedMacroStats.SubmitNested;
		}
		else
		{
			++NestedMacroStats.Submit;
		}
	}
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::PushMacroBlock( const FString& InBlockCtx )
{
	MacroBlockStack.Push(InBlockCtx);
	CachedEditorOnlyDefineState.Reset();
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::PopMacroBlock()
{
	if (MacroBlockStack.Num() > 0)
	{
		MacroBlockStack.Pop(EAllowShrinking::No);
		CachedEditorOnlyDefineState.Reset();
	}
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::FlushMacroStack()
{
	MacroBlockStack.Reset();
	CachedEditorOnlyDefineState.Reset();
}

UGatherTextFromSourceCommandlet::EEditorOnlyDefineState UGatherTextFromSourceCommandlet::FSourceFileParseContext::EvaluateEditorOnlyDefineState() const
{
	if (CachedEditorOnlyDefineState.IsSet())
	{
		return CachedEditorOnlyDefineState.GetValue();
	}

	static const FString WithEditorString = TEXT("WITH_EDITOR");
	static const FString WithEditorOnlyDataString = TEXT("WITH_EDITORONLY_DATA");

	CachedEditorOnlyDefineState = EEditorOnlyDefineState::Undefined;
	for (const FString& BlockCtx : MacroBlockStack)
	{
		if (BlockCtx.Equals(WithEditorString, ESearchCase::CaseSensitive) || BlockCtx.Equals(WithEditorOnlyDataString, ESearchCase::CaseSensitive))
		{
			CachedEditorOnlyDefineState = EEditorOnlyDefineState::Defined;
			break;
		}
	}

	return CachedEditorOnlyDefineState.GetValue();
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::SetDefine(const FString& InDefineCtx)
{
	static const FString LocDefRegionString = TEXT("LOC_DEFINE_REGION");
	static const FString LocNamespaceString = TEXT("LOCTEXT_NAMESPACE");

	if (InDefineCtx.Equals(LocDefRegionString, ESearchCase::CaseSensitive))
	{
		// #define LOC_DEFINE_REGION
		if (ExcludedRegion)
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Found a '#define LOC_DEFINE_REGION' within another '#define LOC_DEFINE_REGION'",
				("file", *Filename),
				("line", LineNumber),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
		else
		{
			ExcludedRegion = true;
		}
		return;
	}
	else if (!ExcludedRegion)
	{
		if (InDefineCtx.StartsWith(LocNamespaceString, ESearchCase::CaseSensitive) && InDefineCtx.IsValidIndex(LocNamespaceString.Len()) && (FTextChar::IsWhitespace(InDefineCtx[LocNamespaceString.Len()]) || InDefineCtx[LocNamespaceString.Len()] == TEXT('"')))
		{
			// #define LOCTEXT_NAMESPACE <namespace>
			if (WithinNamespaceDefineLineNumber != INDEX_NONE)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Found a '#define LOCTEXT_NAMESPACE' within another '#define LOCTEXT_NAMESPACE'",
					("file", *Filename),
					("line", LineNumber),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				FString RemainingText = InDefineCtx.RightChop(LocNamespaceString.Len()).TrimStart();

				bool RemoveStringError;
				const FString DefineDesc = FString::Printf(TEXT("%s define at %s:%d"), *RemainingText, *Filename, LineNumber);
				FString NewNamespace = RemoveStringFromTextMacro(RemainingText, DefineDesc, RemoveStringError);

				if (!RemoveStringError)
				{
					Namespace = MoveTemp(NewNamespace);
					WithinNamespaceDefineLineNumber = LineNumber;
				}
			}
			return;
		}
	}
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::RemoveDefine(const FString& InDefineCtx)
{
	static const FString LocDefRegionString = TEXT("LOC_DEFINE_REGION");
	static const FString LocNamespaceString = TEXT("LOCTEXT_NAMESPACE");

	if (InDefineCtx.Equals(LocDefRegionString, ESearchCase::CaseSensitive))
	{
		// #undef LOC_DEFINE_REGION
		if (!ExcludedRegion)
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Found an '#undef LOC_DEFINE_REGION' without a corresponding '#define LOC_DEFINE_REGION'",
				("file", *Filename), 
				("line", LineNumber), 
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
		else
		{
			ExcludedRegion = false;
		}
		return;
	}
	else if (!ExcludedRegion)
	{
		if (InDefineCtx.Equals(LocNamespaceString, ESearchCase::CaseSensitive))
		{
			// #undef LOCTEXT_NAMESPACE
			if (WithinNamespaceDefineLineNumber == INDEX_NONE)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Found an '#undef LOCTEXT_NAMESPACE' without a corresponding '#define LOCTEXT_NAMESPACE'",
					("file", *Filename), 
					("line", LineNumber), 
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Namespace.Empty();
				WithinNamespaceDefineLineNumber = INDEX_NONE;
			}
			return;
		}
	}
}

bool UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableImpl(const FName InTableId, const FString& InTableNamespace)
{
	// String table entries may be parsed before the string table itself (due to code ordering), so only warn about duplication here if we've already got a source location for the string table (as adding entries doesn't set that)
	FParsedStringTable& ParsedStringTable = ParsedStringTables.FindOrAdd(InTableId);
	if (ParsedStringTable.SourceLocation.Line != INDEX_NONE)
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): String table with ID '{stringTableID}' was already parsed at '{conflictLocation}'. Ignoring additional definition.",
			("file", *Filename),
			("line", LineNumber),
			("stringTableID", *InTableId.ToString()),
			("conflictLocation", *ParsedStringTable.SourceLocation.ToString()),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	ParsedStringTable.TableNamespace = InTableNamespace;
	ParsedStringTable.SourceLocation = FSourceLocation(Filename, LineNumber);
	return true;
}

bool UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableEntryImpl(const FName InTableId, const FString& InKey, const FString& InSourceString, const FSourceLocation& InSourceLocation, const FName InPlatformName, const FString& InDevNotes)
{
	const bool bIsEditorOnly = EvaluateEditorOnlyDefineState() == EEditorOnlyDefineState::Defined;

	// String table entries may be parsed before the string table itself (due to code ordering), so we may need to add our string table below
	FParsedStringTable& ParsedStringTable = ParsedStringTables.FindOrAdd(InTableId);

	FParsedStringTableEntry* ExistingEntry = ParsedStringTable.TableEntries.Find(InKey);
	if (ExistingEntry)
	{
		if (ExistingEntry->SourceString.Equals(InSourceString, ESearchCase::CaseSensitive))
		{
			ExistingEntry->bIsEditorOnly &= bIsEditorOnly;
			return true;
		}
		else
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): String table entry with ID '{stringTableID}' and key '{locKey}' was already parsed at '{conflictLocation}'. Ignoring additional definition.",
				("file", *Filename),
				("line", LineNumber),
				("stringTableID", *InTableId.ToString()),
				("locKey", *InKey),
				("conflictLocation", *ExistingEntry->SourceLocation.ToString()),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}
	}
	else
	{
		FParsedStringTableEntry& ParsedStringTableEntry = ParsedStringTable.TableEntries.Add(InKey);
		ParsedStringTableEntry.SourceString = InSourceString;
		ParsedStringTableEntry.SourceLocation = InSourceLocation;
		ParsedStringTableEntry.PlatformName = InPlatformName;
		ParsedStringTableEntry.DevNotes = InDevNotes;
		ParsedStringTableEntry.bIsEditorOnly = bIsEditorOnly;
		return true;
	}
}

bool UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableEntryMetaDataImpl(const FName InTableId, const FString& InKey, const FName InMetaDataId, const FString& InMetaData, const FSourceLocation& InSourceLocation)
{
	const bool bIsEditorOnly = EvaluateEditorOnlyDefineState() == EEditorOnlyDefineState::Defined;

	// String table meta-data may be parsed before the string table itself (due to code ordering), so we may need to add our string table below
	FParsedStringTable& ParsedStringTable = ParsedStringTables.FindOrAdd(InTableId);
	FParsedStringTableEntryMetaDataMap& MetaDataMap = ParsedStringTable.MetaDataEntries.FindOrAdd(InKey);

	FParsedStringTableEntryMetaData* ExistingMetaData = MetaDataMap.Find(InMetaDataId);
	if (ExistingMetaData)
	{
		if (ExistingMetaData->MetaData.Equals(InMetaData, ESearchCase::CaseSensitive))
		{
			ExistingMetaData->bIsEditorOnly &= bIsEditorOnly;
			return true;
		}
		else
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): String table entry meta-data with ID '{stringTableID}' and key '{locKey}' was already parsed at '{conflictLocation}'. Ignoring additional definition.",
				("file", *Filename),
				("line", LineNumber),
				("stringTableID", *InTableId.ToString()),
				("locKey", *InKey),
				("conflictLocation", *ExistingMetaData->SourceLocation.ToString()),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}
	}
	else
	{
		FParsedStringTableEntryMetaData& ParsedMetaData = MetaDataMap.Add(InMetaDataId);
		ParsedMetaData.MetaData = InMetaData;
		ParsedMetaData.SourceLocation = InSourceLocation;
		ParsedMetaData.bIsEditorOnly = bIsEditorOnly;
		return true;
	}
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTable(const FName InTableId, const FString& InTableNamespace)
{
	AddStringTableImpl(InTableId, InTableNamespace);
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableFromFile(const FName InTableId, const FString& InTableNamespace, const FString& InTableFilename, const FString& InRootPath)
{
	if (AddStringTableImpl(InTableId, InTableNamespace))
	{
		const FString FullImportPath = InRootPath / InTableFilename;

		FStringTableRef TmpStringTable = FStringTable::NewStringTable();
		if (TmpStringTable->ImportStringsFromCSVFile(FullImportPath))
		{
			const FSourceLocation SourceLocation = FSourceLocation(InTableFilename, INDEX_NONE);
			const FName TablePlatformName = GetSplitPlatformNameFromPath_Static(InTableFilename, SplitPlatforms);

			TmpStringTable->EnumerateSourceStrings([&](const FString& InKey, const FString& InSourceString, const FString& InDevNote)
			{
				AddStringTableEntryImpl(InTableId, InKey, InSourceString, SourceLocation, TablePlatformName, InDevNote);

				TmpStringTable->EnumerateMetaData(InKey, [&](const FName InMetaDataId, const FString& InMetaData)
				{
					AddStringTableEntryMetaDataImpl(InTableId, InKey, InMetaDataId, InMetaData, SourceLocation);
					return true; // continue enumeration
				});

				return true; // continue enumeration
			});
		}
		else
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): String table with ID '{stringTableID}' failed to import strings from '{importPath}'.",
				("file", *Filename),
				("line", LineNumber),
				("stringTableID", *InTableId.ToString()),
				("importPath", *FullImportPath),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
		}
	}
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableEntry(const FName InTableId, const FString& InKey, const FString& InSourceString, const FString& InDevNotes)
{
	AddStringTableEntryImpl(InTableId, InKey, InSourceString, FSourceLocation(Filename, LineNumber), FilePlatformName, InDevNotes);
}

void UGatherTextFromSourceCommandlet::FSourceFileParseContext::AddStringTableEntryMetaData(const FName InTableId, const FString& InKey, const FName InMetaDataId, const FString& InMetaData)
{
	AddStringTableEntryMetaDataImpl(InTableId, InKey, InMetaDataId, InMetaData, FSourceLocation(Filename, LineNumber));
}

void UGatherTextFromSourceCommandlet::FDefineDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #define <defname>
	//  or
	// #define <defname> <value>

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		FString RemainingText = Text.RightChop(GetToken().Len()).TrimStart();
		RemainingText = StripCommentsFromToken(RemainingText, Context);

		Context.SetDefine(RemainingText);
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FUndefDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #undef <defname>

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		FString RemainingText = Text.RightChop(GetToken().Len()).TrimStart();
		RemainingText = StripCommentsFromToken(RemainingText, Context);

		Context.RemoveDefine(RemainingText);
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FIfDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #if <defname>

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		FString RemainingText = Text.RightChop(GetToken().Len()).TrimStart();
		RemainingText = StripCommentsFromToken(RemainingText, Context);

		// Handle "#if defined <defname>"
		if (RemainingText.StartsWith(DefinedString, ESearchCase::CaseSensitive))
		{
			RemainingText.RightChopInline(DefinedString.Len(), EAllowShrinking::No);
			RemainingText.TrimStartInline();
		}

		Context.PushMacroBlock(RemainingText);
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FIfDefDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #ifdef <defname>

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		FString RemainingText = Text.RightChop(GetToken().Len()).TrimStart();
		RemainingText = StripCommentsFromToken(RemainingText, Context);

		Context.PushMacroBlock(RemainingText);
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FElIfDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #elif <defname>

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		FString RemainingText = Text.RightChop(GetToken().Len()).TrimStart();
		RemainingText = StripCommentsFromToken(RemainingText, Context);

		// Handle "#elif defined <defname>"
		if (RemainingText.StartsWith(DefinedString, ESearchCase::CaseSensitive))
		{
			RemainingText.RightChopInline(DefinedString.Len(), EAllowShrinking::No);
			RemainingText.TrimStartInline();
		}

		Context.PopMacroBlock(); // Pop the current #if or #ifdef state
		Context.PushMacroBlock(RemainingText);
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FElseDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #else

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		Context.PopMacroBlock(); // Pop the current #if or #ifdef state
		Context.PushMacroBlock(FString());
		Context.EndParsingCurrentLine = true;
	}
}

void UGatherTextFromSourceCommandlet::FEndIfDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// #endif

	if (!Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		Context.PopMacroBlock(); // Pop the current #if or #ifdef state
		Context.EndParsingCurrentLine = true;
	}
}

bool UGatherTextFromSourceCommandlet::FMacroDescriptor::ParseArgumentString(const FString& Text, const int32 CursorOffset, int32& BracketStack, const FSourceFileParseContext& Context, FMacroArgumentGatherer& ArgsGatherer) const
{
	bool bEscapeNextChar = false;

	const TCHAR* ArgStart = *Text + CursorOffset;
	const TCHAR* Cursor = ArgStart;
	for (; 0 < BracketStack; ++Cursor)
	{
		// First: check if we are at end of line.
		if ('\0' == *Cursor)
		{
			if (Cursor - ArgStart > 0)
			{
				// Here, we found the end of the line, but we don't know if it's the end of the argument.
				if (!ArgsGatherer.Gather(ArgStart, UE_PTRDIFF_TO_INT32(Cursor - ArgStart)))
				{
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
						("file", *Context.Filename),
						("line", Context.LineNumber),
						("macroName", *GetToken()),
						("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
					return false;
				}
			}
			break;
		}
		// Skip this character
		else if (bEscapeNextChar)
		{
			bEscapeNextChar = false;
		}
		else if ((ArgsGatherer.IsInDoubleQuotes() || ArgsGatherer.IsInSingleQuotes()) && !bEscapeNextChar && '\\' == *Cursor)
		{
			bEscapeNextChar = true;

			// If we hit the escape character, we must verify if we are at end of line
			if ('\0' == *(Cursor + 1))
			{
				if (Cursor - ArgStart - 1 > 0)
				{
					if (!ArgsGatherer.Gather(ArgStart, UE_PTRDIFF_TO_INT32(Cursor - ArgStart) - 1))
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						return false;
					}
				}
				break;
			}
		}
		// We are closing "
		else if (ArgsGatherer.IsInDoubleQuotes())
		{
			if ('\"' == *Cursor)
			{
				ArgsGatherer.CloseDoubleQuotes();
			}
		}
		// We are closing a '
		else if (ArgsGatherer.IsInSingleQuotes())
		{
			if ('\'' == *Cursor)
			{
				ArgsGatherer.CloseSingleQuotes();
			}
		}
		// We are opening a "
		else if ('\"' == *Cursor)
		{
			ArgsGatherer.OpenDoubleQuotes();
		}
		// We are opening a '
		else if ('\'' == *Cursor)
		{
			ArgsGatherer.OpenSingleQuotes();
		}
		// We found an opening bracket '(', increment the stack
		else if ('(' == *Cursor)
		{
			++BracketStack;
		}
		// We found the closing bracket ')' decrement the stack
		else if (')' == *Cursor)
		{
			--BracketStack;

			if (0 > BracketStack)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Unexpected bracket ')' in {macroName} macro while parsing. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return false;
			}
		}		
		else if (',' == *Cursor)
		{
			if (1 == BracketStack)
			{
				if (Cursor - ArgStart > 0)
				{
					// We have a single bracket open and we found a ',' this is the end of the argument. If Bracket stack is > 1 it means that we are in a function call in one of the parameters.
					if (!ArgsGatherer.Gather(ArgStart, UE_PTRDIFF_TO_INT32(Cursor - ArgStart)))
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						return false;
					}
					if (!ArgsGatherer.EndArgument())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						return false;
					}

					ArgStart = Cursor + 1;
				}
				else
				{
					// The ',' character is the first thing in the line. We need to close out the previous argument 
					// E.g MYMACRO(Param1, "Param2"
					//	    , "Param3")
					if (!ArgsGatherer.EndArgument())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						return false;
					}
					ArgStart = Cursor + 1;
				}
			}
		}
		else if ('\\' == *Cursor)
		{
			ensure(!ArgsGatherer.IsInDoubleQuotes());
			// We hit a escape character outside a double quote, if the next character is end of line, we are done for this line and the next one must start with a double quote
			if ('\0' == *(Cursor + 1))
			{
				if (Cursor - ArgStart - 1 > 0)
				{
					if (!ArgsGatherer.Gather(ArgStart, UE_PTRDIFF_TO_INT32(Cursor - ArgStart) - 1))
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						return false;
					}
				}
				break;
			}
		}

		// We just closed the last ')' this is the end of all args for this macro 
		if (0 == BracketStack)
		{
			// If the arg is empty it means we found a closing bracket after a ',' or at the begining of a line			
			if (Cursor - ArgStart > 0)
			{
				if (!ArgsGatherer.Gather(ArgStart, UE_PTRDIFF_TO_INT32(Cursor - ArgStart)))
				{
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
						("file", *Context.Filename),
						("line", Context.LineNumber),
						("macroName", *GetToken()),
						("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
					return false;
				}
			}
			if (!ArgsGatherer.EndArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Parsing Arguments failed in {macroName} macro. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return false;
			}
			break;
		}
	}
	return true;
}

bool UGatherTextFromSourceCommandlet::FMacroDescriptor::ParseArgsFromMacro(const FString& Text, TArray<FString>& Args, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// NAME(param0, param1, param2, etc)

	// Step over the token name and any whitespace after it
	FString RemainingText = Text.RightChop(GetToken().Len());

	//RemainingText could be empty if the bracket is at the begining of the next line
	RemainingText.TrimStartInline();

	// Find the Opening bracket.
	const int32 OpenBracketIdx = RemainingText.Find(TEXT("("), ESearchCase::CaseSensitive);

	// If we find a bracket it must be the first character of the remaining text
	if (OpenBracketIdx > 0)
	{
		// We stepped over the whitespace when building RemainingText, so if the bracket isn't the first character in the text then it means we only partially matched a longer token and shouldn't parse it
		return false;
	}

	Args.Empty();

	FMacroArgumentGatherer ArgumentGatherer;
	int32 BracketStack = OpenBracketIdx == INDEX_NONE ? 0:1;

	// If we found a bracket, we can start parsing argument on this line
	if (BracketStack > 0)
	{
		// Parse the argument that are on the same line as the macro.
		if (!ParseArgumentString(RemainingText, OpenBracketIdx + 1, BracketStack, Context, ArgumentGatherer))
		{
			return false;
		}
	}

	// We didn't find the end bracket, we must continue.
	if (BracketStack > 0)
	{
		if (!ParseArgsFromNextLines(ArgumentGatherer, BracketStack, Context))
		{
			return false;
		}
	}

	ArgumentGatherer.ExtractArguments(Args);

	return 0 < Args.Num() ? true : false;
}

bool UGatherTextFromSourceCommandlet::FMacroDescriptor::ParseArgsFromNextLines(FMacroArgumentGatherer& ArgsGatherer, int32& BracketStack, FSourceFileParseContext& Context) const
{
	// Loop until we have all arguments and the closing bracket.
	for (int32 i = Context.LineNumber; ArgsGatherer.GetNumberOfArguments() < GetMinNumberOfArgument() || BracketStack > 0; i++)
	{
		if(i >= Context.TextLines.Num())
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): We reached end of file while parsing specified {macroName} macro for arguments. {context}",
				("file", *Context.Filename),
				("line", Context.LineNumber),
				("macroName", *GetToken()),
				("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}

		FString LineText = StripCommentsFromToken(Context.TextLines[i], Context);
		// Ignore empty lines.
		if (LineText.IsEmpty())
		{
			continue;
		}

		int32 ParsingOffset = 0;

		// We are not in an opening bracket yet, look at the beginning of the line.
		if (BracketStack == 0)
		{
			int32 OpenBracketIdx = LineText.Find(TEXT("("), ESearchCase::CaseSensitive);
			// We did not find the opening bracket on the first line and on the second non empty line, we give up
			if (OpenBracketIdx == INDEX_NONE)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Opening bracket '(' not found while parsing specified {macroName} macro for arguments. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return false;
			}
			BracketStack = 1;
			// Start parsing right after the bracket
			ParsingOffset = OpenBracketIdx + 1;
		}
		
		if (!ParseArgumentString(LineText, ParsingOffset, BracketStack, Context, ArgsGatherer))
		{
			return false;
		}	
	}
	return true;
}

bool UGatherTextFromSourceCommandlet::FMacroDescriptor::PrepareArgument(FString& Argument, bool IsAutoText, const FString& IdentForLogging, bool& OutHasQuotes)
{
	bool Error = false;
	if (!IsAutoText)
	{
		Argument = RemoveStringFromTextMacro(Argument, IdentForLogging, Error);
		OutHasQuotes = Error ? false : true;
	}
	else
	{
		Argument = Argument.TrimEnd().TrimQuotes(&OutHasQuotes);
		Argument = UnescapeLiteralCharacterEscapeSequences(Argument);
	}
	return Error ? false : true;
}

void UGatherTextFromSourceCommandlet::FUICommandMacroDescriptor::TryParseArgs(const FString& Text, FSourceFileParseContext& Context, const TArray<FString>& Arguments, const int32 ArgIndexOffset) const
{
	FString Identifier = Arguments[ArgIndexOffset];
	Identifier.TrimStartInline(); // Remove whitespaces at the start of the line.

	// Identifier may optionally be in quotes, as it's sometimes a string literal (in UE_COMMAND_EXT), and sometimes stringified by the macro (in UI_COMMAND)
	// Because this is optional, we don't care if this processing fails
	bool HasQuotes = false;
	PrepareArgument(Identifier, true, FString(), HasQuotes);

	FString SourceLocation = FSourceLocation(Context.Filename, Context.LineNumber).ToString();
	if (Identifier.IsEmpty())
	{
		//The command doesn't have an identifier so we can't gather it
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: {macroName} macro has an empty identifier and cannot be gathered.",
			("location", *SourceLocation),
			("macroName", *GetToken()),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return;
	}

	FString SourceText = Arguments[ArgIndexOffset + 1];
	SourceText.TrimStartInline();

	static const FString UICommandRootNamespace = TEXT("UICommands");
	FString Namespace = Context.WithinNamespaceDefineLineNumber != INDEX_NONE && !Context.Namespace.IsEmpty() ? FString::Printf(TEXT("%s.%s"), *UICommandRootNamespace, *Context.Namespace) : UICommandRootNamespace;

	// parse DefaultLangString argument - this arg will be in quotes without TEXT macro
	FString MacroDesc = FString::Printf(TEXT("%s(%d): \"FriendlyName\" argument in %s macro."), *Context.Filename, Context.LineNumber, *GetToken());
	if (PrepareArgument(SourceText, true, MacroDesc, HasQuotes))
	{
		if (HasQuotes && !Identifier.IsEmpty() && !SourceText.IsEmpty())
		{
			// First create the command entry
			FManifestContext CommandContext;
			CommandContext.Key = Identifier;
			CommandContext.SourceLocation = SourceLocation;
			CommandContext.PlatformName = Context.FilePlatformName;

			Context.AddManifestText(GetToken(), Namespace, SourceText, CommandContext, Context.bIsNested);

			// parse DefaultLangTooltipString argument - this arg will be in quotes without TEXT macro
			FString TooltipSourceText = Arguments[ArgIndexOffset + 2];
			TooltipSourceText.TrimStartInline();
			MacroDesc = FString::Printf(TEXT("%s(%d): \"InDescription\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
			if (PrepareArgument(TooltipSourceText, true, MacroDesc, HasQuotes))
			{
				if (HasQuotes && !TooltipSourceText.IsEmpty())
				{
					// Create the tooltip entry
					FManifestContext CommandTooltipContext;
					CommandTooltipContext.Key = Identifier + TEXT("_ToolTip");
					CommandTooltipContext.SourceLocation = SourceLocation;
					CommandTooltipContext.PlatformName = CommandContext.PlatformName;

					Context.AddManifestText(GetToken(), Namespace, TooltipSourceText, CommandTooltipContext, Context.bIsNested);
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FUICommandMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// UI_COMMAND(LocKey, DefaultLangString, DefaultLangTooltipString, ...)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			// Validate that we got the rightnumber of Arguments
			if (Arguments.Num() < GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected at least {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}
			
			// Parse all arguments found
			TryParseArgs(Text, Context, Arguments, 0);
		}
	}
}

void UGatherTextFromSourceCommandlet::FUICommandExtMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// UI_COMMAND_EXT(<IgnoredParam>, <IgnoredParam>, LocKey, DefaultLangString, DefaultLangTooltipString, ...)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			// Need at least 5 arguments
			if (Arguments.Num() < GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected at least {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}
			TryParseArgs(Text, Context, Arguments, 2);
		}
	}
}

void UGatherTextFromSourceCommandlet::FNestedMacroPrepassDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	++NestedMacroStats.PrepassBegin;

	if (Context.ExcludedRegion || Context.WithinBlockComment || Context.WithinLineComment || Context.WithinStringLiteral)
	{
		return;
	}

	++NestedMacroStats.PrepassEnd;

	TArray<FString> TextLines;
	for (int32 i = Context.LineIdx; i < Context.TextLines.Num(); ++i)
	{
		// We do not use StripCommentsFromToken here, as it modifies Context
		FString TextLine = Context.TextLines[i].TrimStartAndEnd();
		if (TextLine.IsEmpty())
		{
			continue;
		}

		if (!TextLine.EndsWith("\\"))
		{
			TextLines.Add(TextLine);	// Collect trailing line
			break;
		}
		TextLines.Add(TextLine);
	}

	FString MacroLines = FString::Join(TextLines, TEXT("\n"));

	// Remove #define from start
	MacroLines = MacroLines.RightChop(UGatherTextFromSourceCommandlet::DefineString.Len()).TrimStart();

	// Find the Opening bracket
	int32 Pos = MacroLines.Find(TEXT("("), ESearchCase::CaseSensitive);
	if (Pos < 0)
	{
		return;
	}

	FString MacroName = MacroLines.Mid(0, Pos);		// excludes bracket
	FString Content = MacroLines.RightChop(Pos);	// includes open and close brackets
	MacroName.TrimEndInline();
	Content.TrimStartInline();

	// Any combination of the regular macros can be nested within this macro
	int32 LineStart = Context.LineNumber;
	int32 LineCount = TextLines.Num();
	bool bAdvance = false;
	if (Content.Contains(MacroString_NOTELOCTEXT, ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_NOTELOCTEXT, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_NOTELOCTEXT;
		bAdvance = true;
	}
	if (Content.Contains(MacroString_NSNOTELOCTEXT, ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_NSNOTELOCTEXT, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_NSNOTELOCTEXT;
		bAdvance = true;
	}
	if (Content.Contains(MacroString_LOCTEXT, ESearchCase::CaseSensitive) &&
		!Content.Contains(TEXT("LOCTEXT_NAMESPACE"), ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_LOCTEXT, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_LOCTEXT;
		bAdvance = true;
	}
	if (Content.Contains(MacroString_NSLOCTEXT, ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_NSLOCTEXT, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_NSLOCTEXT;
		bAdvance = true;
	}
	if (Content.Contains(MacroString_UI_COMMAND, ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_UI_COMMAND, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_UI_COMMAND;
		bAdvance = true;
	}
	if (Content.Contains(MacroString_UI_COMMAND_EXT, ESearchCase::CaseSensitive))
	{
		PrepassResults.Emplace(MacroName, MacroString_UI_COMMAND_EXT, Context.Filename, Content, LineStart, LineCount);
		++NestedMacroStats.Nested_UI_COMMAND_EXT;
		bAdvance = true;
	}

	if (bAdvance)
	{
		Context.LineIdx += (LineCount - 1);
	}
}

static int32 FindMatching(const FString& Params, TCHAR Opener, TCHAR Closer, int32 Depth)
{
	int32 Pos = 0;
	for (const TCHAR* Char = *Params; *Char; ++Char)
	{
		if (*Char == Opener)
		{
			++Depth;
		}
		else if (*Char == Closer)
		{
			--Depth;
			if (Depth == 0)
			{
				return Pos;
			}
		}
		++Pos;
	}
	return -1;
}

void UGatherTextFromSourceCommandlet::FNestedMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	++NestedMacroStats.MainpassBegin;

	if (Context.ExcludedRegion || Context.WithinBlockComment || Context.WithinLineComment || Context.WithinStringLiteral)
	{
		return;
	}

	// Ignore matches of the prefix, such as METASOUND_PARAM_EXTERN when we're looking for METASOUND_PARAM
	int32 Pos = Text.Find(TEXT("("), ESearchCase::CaseSensitive);
	if (Pos < 0)
	{
		return;
	}
	FString MacroName = GetToken();
	FString MacroNameCurrent = Text.Mid(0, Pos);		// excludes bracket
	MacroNameCurrent.TrimEndInline();
	if (MacroNameCurrent != MacroName)
	{
		return;
	}

	// Ignore matches of the suffix, such as DECLARE_METASOUND_PARAM when we're looking for METASOUND_PARAM
	Pos = Context.LineText.Find(MacroName, ESearchCase::CaseSensitive);
	if (Pos > 0)
	{
		TCHAR Char = Context.LineText[Pos - 1];
		if (!FTextChar::IsWhitespace(Char) && Char != TCHAR('(') && Char != TCHAR('{'))
		{
			return;
		}
	}

	// Parse outer macro values									MACRONAME("first", "second", "third")
	TArray<FString> ArgArrayValues;
	FSourceFileParseContext LocalCtxt1 = Context;		// Local context copy, to avoid changes to the main context
	ParseArgsFromMacro(StripCommentsFromToken(Text, LocalCtxt1), ArgArrayValues, LocalCtxt1);
	for (FString& Arg : ArgArrayValues)
	{
		Arg.TrimStartAndEndInline();
	}

	// Parse outer macro param names from token and content		MACRONAME(param0, param1, param2)
	FString MacroContent = MacroName;
	MacroContent.Append(Content);

	TArray<FString> ArgArray;
	FSourceFileParseContext LocalCtxt2 = Context;		// Local context copy, to avoid changes to the main context
	ParseArgsFromMacro(StripCommentsFromToken(MacroContent, LocalCtxt2), ArgArray, LocalCtxt2);
	Pos = 0;
	int32 PosLast = ArgArray.Num() - 1;
	bool bVariadic = false;
	for (FString& Arg : ArgArray)
	{
		Arg.TrimStartAndEndInline();

		if (Arg.Contains(TEXT("##")))
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Concatenation in {macroName} macro with '##' not supported",
				("file", *Context.Filename),
				("line", Context.LineNumber),
				("macroName", *MacroName),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			++NestedMacroStats.Concatenation;
			return;
		}

		if (Arg.Contains(TEXT("...")))
		{
			++NestedMacroStats.Variadic;
		
			if (Pos != PosLast)
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Variadic in {macroName} macro with '...' must be last param.",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *MacroName),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}
			bVariadic = true;
		}
		++Pos;
	}

	if (bVariadic)
	{
		if ((ArgArray.Num() - 1) > ArgArrayValues.Num())
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "{file}({line}): Expected minimum of {minimumNbArguments} arguments for {macroName} variadic macro, but got {nbArguments}. {context}",
				("file", *Context.Filename),
				("line", Context.LineNumber),
				("expectedNbArguments", (ArgArray.Num() - 1)),
				("macroName", *GetToken()),
				("nbArguments", ArgArrayValues.Num()),
				("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			return;
		}
	}
	else if (ArgArray.Num() != ArgArrayValues.Num())
	{
		UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
			("file", *Context.Filename),
			("line", Context.LineNumber),
			("expectedNbArguments", ArgArray.Num()),
			("macroName", *GetToken()),
			("nbArguments", ArgArrayValues.Num()),
			("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
			("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
		);
		return;
	}

	// Create map of argument to replacement argument
	TMap<FString, FString> ArgToValueMap;
	for (int32 ArgIdx = 0; ArgIdx < ArgArray.Num(); ++ArgIdx)
	{
		FString Arg = ArgArray[ArgIdx];	
		if (Arg.Contains(TEXT("...")))
		{
			// For variadic, collect remaining args
			TArray<FString> VarArgs;
			for (int32 VarIdx = ArgIdx; VarIdx < ArgArrayValues.Num(); ++VarIdx)
			{
				VarArgs.Add(ArgArrayValues[VarIdx]);
			}
			FString VarArgsAll = FString::Join(VarArgs, TEXT(", "));
			ArgToValueMap.Emplace(TEXT("__VA_ARGS__"), VarArgsAll);
			break;
		}
		else
		{
			ArgToValueMap.Emplace(Arg, ArgArrayValues[ArgIdx]);
		}
	}
	// Sort the map, so longer params with the same prefix are replaced first (example: NAME vs NAME_TEXT)
	ArgToValueMap.KeySort([](const FString& LHS, const FString& RHS)
	{
		return LHS.Len() > RHS.Len();
	});

	// Parse inner macro from Contents
	FMacroDescriptor* InnerDescriptor = nullptr;
	if (MacroNameNested == MacroString_LOCTEXT)
	{
		// New Localization System with Namespace as preprocessor define.
		InnerDescriptor = new FStringMacroDescriptor(FString(MacroString_LOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true));
	}
	else if (MacroNameNested == MacroString_NSLOCTEXT)
	{
		// New Localization System with Namespace as literal argument.
		InnerDescriptor = new FStringMacroDescriptor(FString(MacroString_NSLOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Namespace, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true));
	}
	else if (MacroNameNested == MacroString_NOTELOCTEXT)
	{
		InnerDescriptor = new FStringMacroDescriptor(FString(MacroString_NOTELOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_DevNotes, true));
	}
	else if (MacroNameNested == MacroString_NSNOTELOCTEXT)
	{
		InnerDescriptor = new FStringMacroDescriptor(FString(MacroString_NSNOTELOCTEXT),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Namespace, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_Identifier, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_SourceText, true),
			FStringMacroDescriptor::FMacroArg(FStringMacroDescriptor::MAS_DevNotes, true));
	}
	else if (MacroNameNested == MacroString_UI_COMMAND)
	{
		InnerDescriptor = new FUICommandMacroDescriptor();
	}
	else if (MacroNameNested == MacroString_UI_COMMAND_EXT)
	{
		InnerDescriptor = new FUICommandExtMacroDescriptor();
	}
	ensure(InnerDescriptor != nullptr);

	// Replace params in any contained macros
	Pos = 0;
	while ((Pos = Content.Find(MacroNameNested, ESearchCase::CaseSensitive, ESearchDir::FromStart, Pos)) >= 0)
	{
		// Trim content down to just the current arguments
		FString MacroInner = Content.RightChop(Pos + MacroNameNested.Len() + 1);	// exclude macro name
		int32 PosClose = FindMatching(MacroInner, TCHAR('('), TCHAR(')'), 1);
		if (PosClose < 0)
		{
			UE_LOGFMT(LogGatherTextFromSourceCommandlet, Error, "{file}({line}): Missing matching closing bracket in {macroName} macro",
				("file", *Context.Filename),
				("line", Context.LineNumber),
				("macroName", *MacroName),
				("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
			);
			delete InnerDescriptor;
			return;
		}
		FString MacroInnerParams = MacroInner.Mid(0, PosClose);						// exclude bracket

		for (const auto& Pair : ArgToValueMap)
		{
			MacroInnerParams.ReplaceInline(*Pair.Key, *Pair.Value, ESearchCase::CaseSensitive);
		}

		FString ParamsNewAll;
		TryParseArgs(MacroInnerParams, ParamsNewAll);

		FString ParamsWrapped = MacroNameNested;
		ParamsWrapped += TEXT('(');
		ParamsWrapped += ParamsNewAll;
		ParamsWrapped += TEXT(')');

		FSourceFileParseContext LocalCtxt3 = Context;				// Local context copy, to avoid changes to the main context
		LocalCtxt3.bIsNested = true;
		if (InnerDescriptor)
		{
			InnerDescriptor->TryParse(ParamsWrapped, LocalCtxt3);
		}

		++NestedMacroStats.MainpassMid;
		++Pos;
	}
	delete InnerDescriptor;

	++NestedMacroStats.MainpassEnd;
}

bool UGatherTextFromSourceCommandlet::FNestedMacroDescriptor::IsApplicableFile(const FString& InFilename) const
{
	// If this nested macro was found in a translation unit (.cpp) then it can only be used in the same file
	if (Filename.EndsWith(TEXT(".cpp"), ESearchCase::IgnoreCase) &&
		Filename != InFilename)
	{
		return false;
	}

	return true;
}

void UGatherTextFromSourceCommandlet::FNestedMacroDescriptor::TryParseArgs(const FString& MacroInnerParams, FString& ParamsNewAll)
{
	// Split by comma delimiter, but not within quotes
	TArray<FString> Params;
	FString Collect;
	Collect.Reserve(MacroInnerParams.Len());
	bool bWithinQuote = false;
	const TCHAR* CharPrev = *MacroInnerParams;
	for (const TCHAR* Char = *MacroInnerParams; *Char; ++Char)
	{
		if (*Char == TEXT('\"') && *CharPrev != TEXT('\\'))
		{
			// Within non-escaped quotes
			bWithinQuote = !bWithinQuote;
		}
		else if (*Char == TEXT(',') && !bWithinQuote)
		{
			Params.Add(Collect);
			Collect.Reset();
			CharPrev = Char;
			continue;
		}
		Collect += *Char;
		CharPrev = Char;
	}
	Params.Add(Collect);

	TArray<FString> ParamsNew;
	for (const FString& Param : Params)
	{
		FString ParamTrim = Param.TrimStartAndEnd();

		if (ParamTrim.StartsWith(TEXT("\"")) || ParamTrim.StartsWith(TEXT("#")))
		{
			Collect.Reset();
			Collect.Reserve(ParamTrim.Len());
			bWithinQuote = false;
			bool bWithinStringification = false;
			CharPrev = *ParamTrim;
			for (const TCHAR* Char = *ParamTrim; *Char; ++Char)
			{
				if (*Char == TEXT('\"') && *CharPrev != TEXT('\\'))
				{
					// Within non-escaped quotes
					bWithinQuote = !bWithinQuote;
					CharPrev = Char;
					// Skip quotes, we'll requote
					continue;
				}
				if (*Char == TEXT('#') && !bWithinQuote)
				{
					bWithinStringification = true;
					CharPrev = Char;
					continue;
				}
				if (bWithinStringification)
				{
					// Stringification ends when finding a quote or space
					bool bIsQuote = (*Char == TEXT('\"'));
					bool bIsSpace = (*Char == TEXT(' '));
					if (bIsQuote)
					{
						bWithinQuote = true;
					}
					if (bIsQuote || bIsSpace)
					{
						bWithinStringification = false;
						CharPrev = Char;
						continue;
					}
				}
				if (bWithinStringification || bWithinQuote)
				{
					Collect += *Char;
				}
				CharPrev = Char;
			}
		}
		else
		{
			Collect = ParamTrim;
		}

		FString ParamRebuild;
		ParamRebuild += TEXT('\"');
		ParamRebuild += Collect;
		ParamRebuild += TEXT('\"');

		ParamsNew.Add(ParamRebuild);
	}

	ParamsNewAll = FString::Join(ParamsNew, TEXT(", "));
}

void UGatherTextFromSourceCommandlet::FStringMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// MACRONAME(param0, param1, param2)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> ArgArray;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), ArgArray, Context))
		{
			int32 NumArgs = ArgArray.Num();

			if (NumArgs != Arguments.Num())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", Arguments.Num()),
					("macroName", *GetToken()),
					("nbArguments", NumArgs),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				FString Identifier;
				FString SourceLocation = FSourceLocation(Context.Filename, Context.LineNumber).ToString();
				FString SourceText;
				FString DevNotes;

				TOptional<FString> Namespace;
				if (Context.WithinNamespaceDefineLineNumber != INDEX_NONE || !Context.Namespace.IsEmpty())
				{
					Namespace = Context.Namespace;
				}

				bool ArgParseError = false;
				for (int32 ArgIdx=0; ArgIdx<Arguments.Num(); ArgIdx++)
				{
					FMacroArg Arg = Arguments[ArgIdx];
					ArgArray[ArgIdx].TrimStartInline();
					FString ArgText = ArgArray[ArgIdx];

					bool HasQuotes;
					FString MacroDesc = FString::Printf(TEXT("%s(%d): argument %d of %d in %s macro"), *Context.Filename, Context.LineNumber, ArgIdx+1, Arguments.Num(), *GetToken());
					if (!PrepareArgument(ArgText, Arg.IsAutoText, MacroDesc, HasQuotes))
					{
						ArgParseError = true;
						break;
					}

					switch (Arg.Semantic)
					{
					case MAS_Namespace:
						{
							Namespace = ArgText;
						}
						break;
					case MAS_Identifier:
						{
							Identifier = ArgText;
						}
						break;
					case MAS_SourceText:
						{
							SourceText = ArgText;
						}
						break;
					case MAS_DevNotes:
						{
							DevNotes = ArgText;
						}
						break;
					}
				}

				if (Identifier.IsEmpty())
				{
					// The command doesn't have an identifier so we can't gather it
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: {macroName} macro has an empty identifier and cannot be gathered.",
						("location", *SourceLocation),
						("macroName", *GetToken()),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
					return;
				}

				if (SourceText.IsEmpty())
				{
					// The command doesn't have a source text so we can't gather it
					UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: {macroName} macro has an empty source text and cannot be gathered.",
						("location", *SourceLocation),
						("macroName", *GetToken()),
						("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
					);
					return;
				}

				if (!ArgParseError && !Identifier.IsEmpty() && !SourceText.IsEmpty())
				{
					if (!Namespace.IsSet())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: {macroName} macro doesn't define a namespace and no external namespace was set. An empty namspace will be used.",
							("location", *SourceLocation),
							("macroName", *GetToken()),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
						Namespace = FString();
					}

					FManifestContext MacroContext;
					MacroContext.Key = Identifier;
					MacroContext.SourceLocation = SourceLocation;
					MacroContext.PlatformName = Context.FilePlatformName;
					MacroContext.DevNotes = DevNotes;

					if (EnumHasAnyFlags(Context.FileTypes, EGatherTextSourceFileTypes::Ini))
					{
						// Gather the text without its package ID, as the INI will strip it on load at runtime
						TextNamespaceUtil::StripPackageNamespaceInline(Namespace.GetValue());
					}

					Context.AddManifestText(GetToken(), Namespace.GetValue(), SourceText, MacroContext, Context.bIsNested);
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FStringTableMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// LOCTABLE_NEW(Id, Namespace)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			if (Arguments.Num() != GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Arguments[0].TrimStartInline();
				FString TableId = Arguments[0];
				Arguments[1].TrimStartInline();
				FString TableNamespace = Arguments[1];

				const FString TableIdMacroDesc = FString::Printf(TEXT("%s(%d): \"Id\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString TableNamespaceMacroDesc = FString::Printf(TEXT("%s(%d): \"Namespace\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());

				bool HasQuotes;
				if (PrepareArgument(TableId, true, TableIdMacroDesc, HasQuotes) && PrepareArgument(TableNamespace, true, TableNamespaceMacroDesc, HasQuotes))
				{
					const FName TableIdName = *TableId;

					if (TableIdName.IsNone())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty identifier and cannot be gathered.",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
					}
					else
					{
						Context.AddStringTable(TableIdName, TableNamespace);
					}
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FStringTableFromFileMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// LOCTABLE_FROMFILE_X(Id, Namespace, FilePath)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			if (Arguments.Num() != GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Arguments[0].TrimStartInline();
				FString TableId = Arguments[0];
				Arguments[1].TrimStartInline();
				FString TableNamespace = Arguments[1];
				Arguments[2].TrimStartInline();
				FString TableFilename = Arguments[2];

				const FString TableIdMacroDesc = FString::Printf(TEXT("%s(%d): \"Id\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString TableNamespaceMacroDesc = FString::Printf(TEXT("%s(%d): \"Namespace\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString TableFilenameMacroDesc = FString::Printf(TEXT("%s(%d): \"FilePath\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());

				bool HasQuotes;
				if (PrepareArgument(TableId, true, TableIdMacroDesc, HasQuotes) && PrepareArgument(TableNamespace, true, TableNamespaceMacroDesc, HasQuotes) && PrepareArgument(TableFilename, true, TableFilenameMacroDesc, HasQuotes))
				{
					const FName TableIdName = *TableId;

					if (TableIdName.IsNone())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty identifier and cannot be gathered.",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
					}
					else
					{
						Context.AddStringTableFromFile(TableIdName, TableNamespace, TableFilename, RootPath);
					}
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FStringTableEntryMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// LOCTABLE_SETSTRING(Id, Key, SourceString)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			if (Arguments.Num() != GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Arguments[0].TrimStartInline();
				FString TableId = Arguments[0];
				Arguments[1].TrimStartInline();
				FString Key = Arguments[1];
				Arguments[2].TrimStartInline();
				FString SourceString = Arguments[2];

				const FString TableIdMacroDesc = FString::Printf(TEXT("%s(%d): \"Id\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString KeyMacroDesc = FString::Printf(TEXT("%s(%d): \"Key\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString SourceStringMacroDesc = FString::Printf(TEXT("%s(%d): \"SourceString\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());

				bool HasQuotes;
				if (PrepareArgument(TableId, true, TableIdMacroDesc, HasQuotes) && PrepareArgument(Key, true, KeyMacroDesc, HasQuotes) && PrepareArgument(SourceString, true, SourceStringMacroDesc, HasQuotes))
				{
					const FName TableIdName = *TableId;

					if (TableIdName.IsNone() || Key.IsEmpty())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty identifier and cannot be gathered.",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
					}
					else if (!SourceString.IsEmpty())
					{
						Context.AddStringTableEntry(TableIdName, Key, SourceString, TEXT(""));
					}
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FStringTableNotedEntryMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// LOCTABLE_SETNOTESTRING(Id, Key, SourceString, DevNotes)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			if (Arguments.Num() != GetMinNumberOfArgument())
			{
				UE_LOGF(LogGatherTextFromSourceCommandlet, Warning, "%ls(%d): Expected %d arguments for %ls macro, but got %d. %ls", *Context.Filename, Context.LineNumber, GetMinNumberOfArgument(), *GetToken(), Arguments.Num(), *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd()));
			}
			else
			{
				Arguments[0].TrimStartInline();
				FString TableId = Arguments[0];
				Arguments[1].TrimStartInline();
				FString Key = Arguments[1];
				Arguments[2].TrimStartInline();
				FString SourceString = Arguments[2];
				Arguments[3].TrimStartInline();
				FString DevNotes = Arguments[3];

				const FString TableIdMacroDesc = FString::Printf(TEXT("%s(%d): \"Id\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString KeyMacroDesc = FString::Printf(TEXT("%s(%d): \"Key\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString SourceStringMacroDesc = FString::Printf(TEXT("%s(%d): \"SourceString\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString DevNoteMacroDesc = FString::Printf(TEXT("%s(%d): \"DevNote\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());

				bool HasQuotes;
				if (PrepareArgument(TableId, true, TableIdMacroDesc, HasQuotes) && PrepareArgument(Key, true, KeyMacroDesc, HasQuotes) && PrepareArgument(SourceString, true, SourceStringMacroDesc, HasQuotes) && PrepareArgument(DevNotes, true, DevNoteMacroDesc, HasQuotes))
				{
					const FName TableIdName = *TableId;

					if (TableIdName.IsNone() || Key.IsEmpty())
					{
						UE_LOGF(LogGatherTextFromSourceCommandlet, Warning, "%ls(%d): %ls macro has an empty identifier and cannot be gathered.", *Context.Filename, Context.LineNumber, *GetToken());
					}
					else if (!SourceString.IsEmpty())
					{
						Context.AddStringTableEntry(TableIdName, Key, SourceString, DevNotes);
					}
				}
			}
		}
	}
}

void UGatherTextFromSourceCommandlet::FStringTableEntryMetaDataMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// LOCTABLE_SETMETA(Id, Key, SourceString)

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			if (Arguments.Num() != GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): Expected {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}
			else
			{
				Arguments[0].TrimStartInline();
				FString TableId = Arguments[0];
				Arguments[1].TrimStartInline();
				FString Key = Arguments[1];
				Arguments[2].TrimStartInline();
				FString MetaDataId = Arguments[2];
				Arguments[3].TrimStartInline();
				FString MetaData = Arguments[3];

				const FString TableIdMacroDesc = FString::Printf(TEXT("%s(%d): \"Id\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString KeyMacroDesc = FString::Printf(TEXT("%s(%d): \"Key\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString MetaDataIdMacroDesc = FString::Printf(TEXT("%s(%d): \"MetaDataId\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());
				const FString MetaDataMacroDesc = FString::Printf(TEXT("%s(%d): \"MetaData\" argument in %s macro"), *Context.Filename, Context.LineNumber, *GetToken());

				bool HasQuotes;
				if (PrepareArgument(TableId, true, TableIdMacroDesc, HasQuotes) && PrepareArgument(Key, true, KeyMacroDesc, HasQuotes) && PrepareArgument(MetaDataId, true, MetaDataIdMacroDesc, HasQuotes) && PrepareArgument(MetaData, true, MetaDataMacroDesc, HasQuotes))
				{
					const FName TableIdName = *TableId;
					const FName MetaDataIdName = *MetaDataId;

					if (TableIdName.IsNone() || Key.IsEmpty() || MetaDataIdName.IsNone())
					{
						UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty identifier and cannot be gathered.",
							("file", *Context.Filename),
							("line", Context.LineNumber),
							("macroName", *GetToken()),
							("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
						);
					}
					else if (!MetaData.IsEmpty())
					{
						Context.AddStringTableEntryMetaData(TableIdName, Key, MetaDataIdName, MetaData);
					}
				}
			}
		}
	}
}

int32 UGatherTextFromSourceCommandlet::FStructuredLogMacroDescriptor::CalculateMinimumArgumentCount(EFlags Flags)
{
	// [Condition], CategoryName, Verbosity, [Namespace], Key, Format
	return 4 + !!(Flags & EFlags::Condition) + !!(Flags & EFlags::Namespace);
}

UGatherTextFromSourceCommandlet::FStructuredLogMacroDescriptor::FStructuredLogMacroDescriptor(const TCHAR* InName, EFlags InFlags)
	: FMacroDescriptor(InName, CalculateMinimumArgumentCount(InFlags))
	, Flags(InFlags)
{
}

void UGatherTextFromSourceCommandlet::FStructuredLogMacroDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// UE_LOGFMT_LOC[_EX](CategoryName, Verbosity, Key, Format, ...)
	// UE_LOGFMT_NSLOC[_EX](CategoryName, Verbosity, Namespace, Key, Format, ...)
	// UE_CLOGFMT_LOC[_EX](Condition, CategoryName, Verbosity, Key, Format, ...)
	// UE_CLOGFMT_NSLOC[_EX](Condition, CategoryName, Verbosity, Namespace, Key, Format, ...)

	// The index of the next argument to consume, initially Namespace or Key.
	int32 NextArg = !(Flags & EFlags::Condition) ? 2 : 3;

	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			const FString SourceLocation = FSourceLocation(Context.Filename, Context.LineNumber).ToString();

			if (Arguments.Num() < GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: Expected at least {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("location", *SourceLocation),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}

			bool bParseOk = true;
			const auto ParseArg = [this, &SourceLocation, &bParseOk](FString&& Arg, const TCHAR* ArgName) -> FString
			{
				FString Out = MoveTemp(Arg).TrimStart();
				constexpr bool bIsAutoText = true;
				const FString Desc = FString::Printf(TEXT("%s: \"%s\" argument in %s macro"), *SourceLocation, ArgName, *GetToken());
				bool bHasQuotes = false;
				bParseOk &= PrepareArgument(Out, bIsAutoText, Desc, bHasQuotes);
				return Out;
			};

			FString Namespace;
			if (!!(Flags & EFlags::Namespace))
			{
				Namespace = ParseArg(MoveTemp(Arguments[NextArg++]), TEXT("Namespace"));
			}
			else
			{
				Namespace = Context.Namespace;
			}

			if (Namespace.IsEmpty())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: {macroName} macro doesn't define a namespace and no external namespace was set. An empty namespace will be used.",
					("location", *SourceLocation),
					("macroName", *GetToken()),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
			}

			FString Key = ParseArg(MoveTemp(Arguments[NextArg++]), TEXT("Key"));
			FString Format = ParseArg(MoveTemp(Arguments[NextArg++]), TEXT("Format"));

			if (bParseOk && !Key.IsEmpty())
			{
				FManifestContext MacroContext;
				MacroContext.Key = Key;
				MacroContext.SourceLocation = SourceLocation;
				MacroContext.PlatformName = Context.FilePlatformName;

				Context.AddManifestText(GetToken(), Namespace, Format, MacroContext, Context.bIsNested);
			}
		}
	}
}

UGatherTextFromSourceCommandlet::FDeclareErrorDescriptor::FDeclareErrorDescriptor(const TCHAR* InMacroName, const TCHAR* InKeyParameterName, int32 InMinArguments, int32 InNamespaceIndex, int32 InKeyIndex, int32 InTextIndex)
	: FMacroDescriptor(InMacroName, InMinArguments)
	, KeyParameterName(InKeyParameterName)
	, NamespaceIndex(InNamespaceIndex)
	, KeyIndex(InKeyIndex)
	, TextIndex(InTextIndex)
{
}

void UGatherTextFromSourceCommandlet::FDeclareErrorDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// UE_DECLARE_ERROR(ApiMacro, ModuleName, ErrorCode, ErrorName, FormatString...)
	// UE_DECLARE_ERROR_CONTEXT(ApiMacro, ModuleName, ContextName, FormatString...)

	// A localization entry formed with namespace ModuleName, key ErrorName, and unlocalized text FormatString
	if (!Context.ExcludedRegion && !Context.WithinBlockComment && !Context.WithinLineComment && !Context.WithinStringLiteral)
	{
		TArray<FString> Arguments;
		if (ParseArgsFromMacro(StripCommentsFromToken(Text, Context), Arguments, Context))
		{
			const FString SourceLocation = FSourceLocation(Context.Filename, Context.LineNumber).ToString();

			if (Arguments.Num() < GetMinNumberOfArgument())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{location}: Expected at least {expectedNbArguments} arguments for {macroName} macro, but got {nbArguments}. {context}",
					("location", *SourceLocation),
					("expectedNbArguments", GetMinNumberOfArgument()),
					("macroName", *GetToken()),
					("nbArguments", Arguments.Num()),
					("context", *FLocTextHelper::SanitizeLogOutput(Context.LineText.TrimStartAndEnd())),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}

			// Macros take plain string literals and not TEXT("...")
			constexpr bool bIsAutoText = true;
			bool bHasQuotes = false;
			// Per-argument leading whitespace from "," separators is preserved by ParseArgumentString;
			// PrepareArgument only TrimEnds, so without TrimStart the leading space prevents TrimQuotes
			// from recognizing the opening quote on string literals.
			FString Namespace = Arguments[NamespaceIndex];
			Namespace.TrimStartInline();
			if (!PrepareArgument(Namespace, bIsAutoText, FString::Printf(TEXT("%s: \"Module\" argument in %s macro"), *SourceLocation, *GetToken()), bHasQuotes))
			{
				return;
			}
			if (Namespace.IsEmpty())
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty {parameterName} and cannot be gathered.",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("parameterName", "Module"),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}
			FString Key = Arguments[KeyIndex];
			Key.TrimStartInline();
			if (!PrepareArgument(Key, bIsAutoText, FString::Printf(TEXT("%s: \"ErrorName\" argument in %s macro"), *SourceLocation, *GetToken()), bHasQuotes))
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty {parameterName} and cannot be gathered.",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("parameterName", KeyParameterName),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}
			FString Format = Arguments[TextIndex];
			Format.TrimStartInline();
			if (!PrepareArgument(Format, bIsAutoText, FString::Printf(TEXT("%s: \"FormatString\" argument in %s macro"), *SourceLocation, *GetToken()), bHasQuotes))
			{
				UE_LOGFMT(LogGatherTextFromSourceCommandlet, Warning, "{file}({line}): {macroName} macro has an empty {parameterName} and cannot be gathered.",
					("file", *Context.Filename),
					("line", Context.LineNumber),
					("macroName", *GetToken()),
					("parameterName", "FormatString"),
					("id", GatherTextFromSourceCommandlet::LocalizationLogIdentifier)
				);
				return;
			}

			FManifestContext MacroContext;
			MacroContext.Key = Key;
			MacroContext.SourceLocation = SourceLocation;
			MacroContext.PlatformName = Context.FilePlatformName;

			Context.AddManifestText(GetToken(), Namespace, Format, MacroContext, Context.bIsNested);
		}
	}
}

void UGatherTextFromSourceCommandlet::FIniNamespaceDescriptor::TryParse(const FString& Text, FSourceFileParseContext& Context) const
{
	// Attempt to parse something of the format
	// [<config section name>]
	if (!Context.ExcludedRegion)
	{
		if( Context.LineText[ 0 ] == '[' )
		{
			int32 ClosingBracket;
			if( Text.FindChar( ']', ClosingBracket ) && ClosingBracket > 1 )
			{
				Context.Namespace = Text.Mid( 1, ClosingBracket - 1 );
				Context.EndParsingCurrentLine = true;
			}
		}
	}
}

int32 UGatherTextFromSourceCommandlet::FParsedNestedMacro::Size(const UGatherTextFromSourceCommandlet::FParsedNestedMacro& Result)
{
	int32 SizeInBytes = 0;
	SizeInBytes += Result.MacroName.GetAllocatedSize();
	SizeInBytes += Result.MacroNameNested.GetAllocatedSize();
	SizeInBytes += Result.Filename.GetAllocatedSize();
	SizeInBytes += Result.Content.GetAllocatedSize();
	SizeInBytes += sizeof(Result.LineStart);
	SizeInBytes += sizeof(Result.LineCount);
	SizeInBytes += sizeof(Result.bExclude);
	return SizeInBytes;
}

#undef LOC_DEFINE_REGION
