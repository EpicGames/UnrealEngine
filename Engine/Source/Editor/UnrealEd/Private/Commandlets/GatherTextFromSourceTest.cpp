// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GatherTextFromSourceCommandlet.h"
#include "Containers/UnrealString.h"


void UGatherTextFromSourceCommandlet::FNestedMacroDescriptor::TestNestedMacroDescriptorParseArgs()
{
	FString MacroInnerParams;
	FString ParamsNewAll;

	MacroInnerParams = TEXT("\"aaa_\" \"bbb\", \"cc ddd eee\"");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa_bbb\", \"cc ddd eee\""));

	// Stringification using hash '#'
	MacroInnerParams = TEXT("\"aaa \" #bbb \" ccc\"");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa bbb ccc\""));

	// Stringification using hash '#'
	MacroInnerParams = TEXT("#aaa\"bbb\"");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaabbb\""));

	// String containing hash '#'
	MacroInnerParams = TEXT("#aaa \"#bbb\" \"ccc ddd\"");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa#bbbccc ddd\""));

	// Param containing a comma
	MacroInnerParams = TEXT("\"aaa\", \"bbb, with comma\"");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa\", \"bbb, with comma\""));

	// Param without quotes
	MacroInnerParams = TEXT("aaa, \"bbb\", ccc");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa\", \"bbb\", \"ccc\""));

	// Param with escaped quotes
	MacroInnerParams = TEXT("aaa, \"bbb \\\"ccc\\\" ddd\", eee");
	TryParseArgs(MacroInnerParams, ParamsNewAll);
	ensure(ParamsNewAll == TEXT("\"aaa\", \"bbb \\\"ccc\\\" ddd\", \"eee\""));
}

