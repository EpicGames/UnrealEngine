// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenerateAssetManifestCommandlet.cpp: Commandlet for generating a filtered
	list of assets from the asset registry (intended use is for replacing
	assets with cooked version)
=============================================================================*/

#include "Commandlets/GenerateAssetManifestCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "AssetRegistry/ARFilter.h"
#include "Engine/World.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateAssetManifestCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateAssetManifestCommandlet, Log, All);

UGenerateAssetManifestCommandlet::UGenerateAssetManifestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

int32 UGenerateAssetManifestCommandlet::Main(const FString& InParams)
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*InParams, Tokens, Switches);

	// Support standard and BuildGraph style delimeters
	static const TCHAR* ParamDelims[] =
	{
		TEXT(";"),
		TEXT("+"),
	};

	const FString ManifestFileSwitch = TEXT("ManifestFile=");
	const FString IncludedPathsSwitch = TEXT("IncludedPaths=");
	const FString IncludedClassesSwitch = TEXT("IncludedClasses=");
	const FString ExcludedPathsSwitch = TEXT("ExcludedPaths=");
	const FString ExcludedClassesSwitch = TEXT("ExcludedClasses=");
	const FString ClassBasePathsSwitch = TEXT("ClassBasePaths=");
	FString ManifestFile;
	TArray<FString> IncludedPaths;
	TArray<FString> IncludedClasses;
	TArray<FString> ExcludedPaths;
	TArray<FString> ExcludedClasses;
	TArray<FString> ClassBasePaths;

	// Parse parameters
	for (int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); ++SwitchIdx)
	{
		const FString& Switch = Switches[SwitchIdx];
		FString SwitchValue;
		if (FParse::Value(*Switch, *ManifestFileSwitch, SwitchValue))
		{
			ManifestFile = SwitchValue;
		}
		else if (FParse::Value(*Switch, *IncludedPathsSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(IncludedPaths, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *IncludedClassesSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(IncludedClasses, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ExcludedPathsSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ExcludedPaths, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ExcludedClassesSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ExcludedClasses, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ClassBasePathsSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ClassBasePaths, ParamDelims, 2);
		}
	}

	// Check that output file path is specified
	if (ManifestFile.IsEmpty())
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Error, "Please specify a valid location for -ManifestFile on the commandline");
		return 1;
	}

	// by default only look for classes within the game project
	if (ClassBasePaths.Num() == 0)
	{
		ClassBasePaths.Add(TEXT("/Game"));
	}

	TArray<FName> ClassPackagePaths;
	for (FString BasePath : ClassBasePaths)
	{
		ClassPackagePaths.Add(*BasePath);
	}

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Searching Asset Registry");
	AssetRegistryModule.Get().SearchAllAssets(true);

	TArray<FAssetData> FinalAssetList;

	// Get assets from paths and classes that we want to include
	if (IncludedPaths.Num() > 0)
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Getting Assets from specified paths");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.bRecursivePaths = true;
		for (const FString& IncludedPath : IncludedPaths)
		{
			Filter.PackagePaths.AddUnique(*IncludedPath);
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		for (FAssetData& Asset : AssetList)
		{
			FinalAssetList.AddUnique(Asset);
		}
	}
	if (IncludedClasses.Num() > 0)
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Getting Assets of specified classes");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths = ClassPackagePaths;
		Filter.bRecursivePaths = true;
		for (const FString& IncludedClass : IncludedClasses)
		{
			FTopLevelAssetPath IncludedClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(IncludedClass, ELogVerbosity::Error, TEXT("UGenerateAssetManifestCommandlet::Main"));
			if (IncludedClassPathName.IsNull())
			{
				UE_LOGF(LogGenerateAssetManifestCommandlet, Error, "Failed to convert short class name \"%ls\" to path name. Please use class path names for IncludedClasses.", *IncludedClass);
			}
			else
			{
				Filter.ClassPaths.AddUnique(IncludedClassPathName);
			}
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		for (FAssetData& Asset : AssetList)
		{
			FinalAssetList.AddUnique(Asset);
		}
	}

	// Run through paths and classes that should be excluded
	if (FinalAssetList.Num() > 0 && ExcludedPaths.Num() > 0)
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Excluding Assets from specified paths");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.bRecursivePaths = true;
		for (const FString& ExcludedPath : ExcludedPaths)
		{
			Filter.PackagePaths.AddUnique(*ExcludedPath);
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		FinalAssetList.RemoveAll([&AssetList](const FAssetData& Asset) {return AssetList.Contains(Asset); });
	}
	if (FinalAssetList.Num() > 0 && ExcludedClasses.Num() > 0)
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Excluding Assets of specified classes");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths = ClassPackagePaths;
		Filter.bRecursivePaths = true;
		for (const FString& ExcludedClass : ExcludedClasses)
		{
			FTopLevelAssetPath ExcludedClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ExcludedClass, ELogVerbosity::Error, TEXT("UGenerateAssetManifestCommandlet::Main"));
			if (ExcludedClassPathName.IsNull())
			{
				UE_LOGF(LogGenerateAssetManifestCommandlet, Error, "Failed to convert short class name \"%ls\" to path name. Please use class path names for ExcludedClasses.", *ExcludedClass);
			}
			else
			{
				Filter.ClassPaths.AddUnique(ExcludedClassPathName);
			}
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		FinalAssetList.RemoveAll([&AssetList](const FAssetData& Asset) {return AssetList.Contains(Asset); });
	}

	FString FinalFileList;
	if (FinalAssetList.Num() > 0)
	{
		UE_LOGF(LogGenerateAssetManifestCommandlet, Display, "Converting Package Names to File Paths");
		for (FAssetData& RemovedAsset : FinalAssetList)
		{
			FString ActualFile;
			if (FPackageName::DoesPackageExist(RemovedAsset.PackageName.ToString(), &ActualFile))
			{
				ActualFile = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ActualFile);
				FinalFileList += FString::Printf(TEXT("%s") LINE_TERMINATOR, *ActualFile);
			}
		}

		if (!FFileHelper::SaveStringToFile(FinalFileList, *ManifestFile))
		{
			UE_LOGF(LogGenerateAssetManifestCommandlet, Error, "Failed to save output file '%ls'", *ManifestFile);
			return 1;
		}
	}

	return 0;
}
