// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GenerateBlueprintAPICommandlet.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Engine/Level.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Animation/AnimInstance.h"
#include "Engine/Engine.h"
#include "Animation/AnimBlueprint.h"
#include "EngineGlobals.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "UnrealEdGlobals.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "K2Node.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "K2Node_CommutativeAssociativeBinaryOperator.h"
#include "K2Node_Switch.h"
#include "K2Node_SwitchEnum.h"
#include "AnimationGraph.h"
#include "AnimGraphNode_StateMachine.h"
#include "EdGraphSchema_K2.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimStateConduitNode.h"
#include "AnimationStateMachineGraph.h"
#include "AnimStateNode.h"
#include "K2Node_Event.h"
#include "K2Node_CustomEvent.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "BlueprintActionMenuBuilder.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintActionMenuUtils.h"
#include "EditorCategoryUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateBlueprintAPICommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintAPIGenerate, Log, All);

/*******************************************************************************
 * Static Helpers
 ******************************************************************************/

namespace GenerateBlueprintAPIUtils
{
	static const FString HelpString = TEXT("\n\
\n\
GenerateBlueprintAPI commandlet params: \n\
\n\
    -class=<Class>      Used to specify the blueprint's parent class, if left  \n\
                        unset then it will go through EVERY blueprint parent   \n\
                        class available.                                       \n\
\n\
    -category=<CategoryPath1, CategoryPath2, ...>                              \n\
                        Used to specify the blueprint's category path, if      \n\
                        left unset then it will go through EVERY category      \n\
                        available. A category path can contain a single        \n\
                        category, or multiple category names concanated by     \n\
                        \'|\'. Multiple category paths can be supplied by using\n\
                        \',\' as separators.                                   \n\
\n\
    -multifile          Used to keep file size down, will split each blueprint \n\
                        into its own file (meaning only one file will be       \n\
                        created when used with -class).                        \n\
\n\
    -palette            Defaults to on, unless other flags were specified.     \n\
                        Dumps all actions from the blueprint's palette menu    \n\
                        (constant across all graphs).                          \n\
\n\
    -palfilter=<Class>  Simulates picking a class from the blueprint palette's \n\
                        drop down. Setting -palfilter=all will dump the palette\n\
                        for every possible class.                              \n\
\n\
    -time               When enabled, will record timings during menu building \n\
                        (as it has been time sync in the past). This is not    \n\
                        ideal for diffs though (since times can easily vary).  \n\
\n\
    -experimental       Uses a new way of constructing Blueprint action menus \n\
                        (that will replace the current system).                \n\
\n\
    -name=<Filename>    Overrides the default filename. Leave off the extention\n\
                        (this will add .json to the end). When -multifile is   \n\
                        supplied, the class name will be postfixed to the name.\n\
\n\
    -help, -h, -?       Display this message and then exit.                    \n\
\n");

	/** Flags that govern the verbosity of the dump. */
	enum EDumpFlags
	{
		BPDUMP_FilteredPalette		= (1<<0), 

		BPDUMP_LogHelp				= (1<<1),
		BPDUMP_RecordTiming			= (1<<2),
		BPDUMP_UseLegacyMenuBuilder = (1<<3),
	};

	/**
	 * A collection of variables that represent the various command switches 
	 * that users can specify when running the commandlet. See the HelpString 
	 * variable for a listing of supported switches.
	 */
	struct CommandletOptions
	{
		CommandletOptions()
			: BlueprintClass(nullptr)
			, DumpFlags(0)
			, PaletteFilter(nullptr)
		{
		}

		/**
		* Parses the string command switches into flags, class pointers, and 
		* booleans that will govern what should be dumped. Logs errors if any 
		* switch was misused.
		*/
		CommandletOptions(TArray<FString> const& Switches);

		UClass*    BlueprintClass;
		uint32     DumpFlags;
		UClass*    PaletteFilter;
		FString    SaveDir;
		FString    Filename;
		TArray<FString> CategoryFilter;
	};

	/** Static instance of the command switches (so we don't have to pass one along the call stack) */
	static CommandletOptions CommandOptions;

	/** Tracks instantiated blueprints (so we don't have to create more the we have to). */
	static TMap<UClass*, UBlueprint*> ClassBlueprints;

	/**
	 * Certain blueprints (like level blueprints) require a level outer, and 
	 * for certain actions we need a level actor selected. This utility function
	 * provides an easy way to grab the world (which has a level that we can use 
	 * for these purposes).
	 */
	static UWorld* GetWorld();

	/**
	 * Spawns a transient blueprint of the specified type. Adds all possible 
	 * graph types (function, macro, etc.), and does some additional setup for
	 * unique blueprint types (like level and anim blueprints).
	 */
	static UBlueprint* MakeTempBlueprint(UClass* BlueprintClass);

	/**
	 * Certain nodes add specific graph types that we want to dump info on (like 
	 * state machine graphs for anim blueprints). The best way to add those 
	 * graphs is through the natural process of adding those nodes (which this
	 * method is intended for).
	 */
	template<class NodeType>
	static NodeType* AddNodeToGraph(UEdGraph* Graph);

	/**
	 * Builds a fully qualified file path for a new dump file. If using the 
	 * -multifile command switch, then this will create a sub-directory and name
	 * the file after the class. Generally, dump files are placed in the 
	 * project's ".../Saved/Commandlets/" directory.
	 */
	static FString BuildDumpFilePath(UClass* BlueprintClass);

	/**
	 * Utility function to convert a tab integer into a string of whitespace.
	 * Defaults to tab characters, but if bUseSpaces is enabled, then single 
	 * spaces are used.
	 */
	static FString BuildIndentString(uint32 IndentCount, bool bUseSpaces = false);

	/**
	 * Concatenates the action's category with its menu name (to help
	 * distinguish similarly named actions). Can then be used to sort and 
	 * uniquely identify actions.
	 */
	static FString GetActionKey(TSharedPtr<FEdGraphSchemaAction> const& Action);

	/**
	* Goes through all of the blueprint skeleton's object properties and pulls
	* out the ones that are associated with an UActorComponent (and are visbile
	* to the blueprint).
	*/
	static void GetComponentProperties(UBlueprint* Blueprint, TArray<FObjectProperty*>& PropertiesOut);
	
	/**
	 * Constructs a temporary blueprint (of the class type specified) and kicks 
	 * off a dump of all its nested information (palette, graph, contextual 
	 * actions, etc.). 
	 */
	static void DumpInfoForClass(uint32 Indent, UClass* BlueprintClass, FArchive* FileOutWriter);
	
	/**
	 * Writes out all the category details that have been accumulated during the palette construction
	 */
	static void DumpCategoryInfo(uint32 Indent, FArchive* FileOutWriter);

	/**
	 * Assumes that the specified ActionListBuilder is configured with the 
	 * proper blueprint. Starts by clearing any actions it contained and then
	 * runs through and re-adds all actions.
	 *
	 * @return The amount of time (in seconds) that the menu building took.
	 */
	static double GetPaletteMenuActions(FCategorizedGraphActionListBuilder& PaletteBuilder, UBlueprint* Blueprint, UClass* PaletteFilter);

	/**
	 * Dumps all palette actions listed for the specified blueprint. Determines
	 * if the user specified any filter class for the palette and adjusts 
	 * accordingly (can dump multiple palettes if -palfilter=all was specified).
	 */
	static void DumpPalette(uint32 Indent, UBlueprint* Blueprint, FArchive* FileOutWriter);

	/**
	 * Dumps a single instance of the blueprint's palette (using the ClassFilter).
	 * ClassFilter can be null and the full unfiltered palette will be dumped.
	 */
	static void DumpPalette(uint32 Indent, UBlueprint* Blueprint, UClass* ClassFilter, FArchive* FileOutWriter);

	/**
	 * Generic function utilized by both palette and context-menu dumps. Take a
	 * GraphActionListBuilder and writes out every action that it has captured.
	 */
	static void DumpActionList(uint32 Indent, FGraphActionListBuilderBase& ActionList, FArchive* FileOutWriter);

	/**
	* Get the category information from a given action.
	*/
	static FString GetActionCategory(TSharedPtr<FEdGraphSchemaAction> const& Action);

	/**
	 * Generic function that dumps information on a single action (like it's 
	 * name, category, an associated node if it has one, etc.).
	 */
	static void DumpActionMenuItem(uint32 Indent, TSharedPtr<FEdGraphSchemaAction> const& Action, FGraphActionListBuilderBase& ActionList, FArchive* FileOutWriter);
}

//------------------------------------------------------------------------------
FString MakeJsonString( const FString& String )
{
	FString OutString;
	for (const TCHAR* Char = *String; *Char != TCHAR('\0'); ++Char)
	{
		switch (*Char)
		{
		case TCHAR('\\'): OutString += TEXT("\\\\"); break;
		case TCHAR('\n'): OutString += TEXT("\\n"); break;
		case TCHAR('\t'): OutString += TEXT("\\t"); break;
		case TCHAR('\b'): OutString += TEXT("\\b"); break;
		case TCHAR('\f'): OutString += TEXT("\\f"); break;
		case TCHAR('\r'): OutString += TEXT("\\r"); break;
		case TCHAR('\"'): OutString += TEXT("\\\""); break;
		default: OutString += *Char;
		}
	}

	OutString = OutString.Replace(TEXT("\xD7"), TEXT("&times;"));
	OutString = OutString.Replace(TEXT("\xF7"), TEXT("&divide;"));
	OutString = OutString.Replace(TEXT("\x2022"), TEXT("&middot;"));

	OutString = OutString.Replace(TEXT("<"), TEXT("&lt;"));
	OutString = OutString.Replace(TEXT(">"), TEXT("&gt;"));

	return OutString;
}

//------------------------------------------------------------------------------
GenerateBlueprintAPIUtils::CommandletOptions::CommandletOptions(TArray<FString> const& Switches)
	: BlueprintClass(AActor::StaticClass())
	, DumpFlags(BPDUMP_UseLegacyMenuBuilder)
	, PaletteFilter(nullptr)
{
	uint32 NewDumpFlags = BPDUMP_UseLegacyMenuBuilder;
	for (FString const& Switch : Switches)
	{
		if (Switch.StartsWith("class="))
		{
			FString ClassSwitch, ClassName;
			Switch.Split(TEXT("="), &ClassSwitch, &ClassName);
			BlueprintClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing GenerateBlueprintAPIUtils class"));

			if (BlueprintClass == nullptr)
			{
				UE_LOGF(LogBlueprintAPIGenerate, Error, "Unrecognized blueprint class '%ls', defaulting to 'Actor'", *ClassName);
				BlueprintClass = AActor::StaticClass();
			}
		}
		else if (Switch.StartsWith("palfilter="))
		{
			FString ClassSwitch, ClassName;
			Switch.Split(TEXT("="), &ClassSwitch, &ClassName);
			PaletteFilter = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing GenerateBlueprintAPIUtils palfilter"));

			NewDumpFlags |= BPDUMP_FilteredPalette;
			if (PaletteFilter == nullptr)
			{
				if (ClassName.Compare("all", ESearchCase::IgnoreCase))
				{
					UE_LOGF(LogBlueprintAPIGenerate, Error, "Unrecognized palette filter '%ls', defaulting to unfiltered", *ClassName);
					NewDumpFlags &= ~(GenerateBlueprintAPIUtils::BPDUMP_FilteredPalette);
				}
			}
		}
		else if (Switch.StartsWith("category=", ESearchCase::IgnoreCase))
		{
			FString ClassSwitch, Filters;
			Switch.Split(TEXT("="), &ClassSwitch, &Filters, ESearchCase::IgnoreCase);

			// Remove surounding " or ' if exist.
			Filters = Filters.Replace(TEXT("\""), TEXT(""), ESearchCase::IgnoreCase);
			Filters = Filters.Replace(TEXT("\'"), TEXT(""), ESearchCase::IgnoreCase);
			Filters.TrimStartAndEndInline();

			// Parse multiple category paths.
			while (Filters.Len() > 0)
			{
				FString FilterName, OtherFilters;
				if (Filters.Split(TEXT(","), &FilterName, &OtherFilters, ESearchCase::IgnoreCase))
				{
					FilterName.TrimStartAndEndInline();
					CategoryFilter.Add(FilterName);
					Filters = OtherFilters;
				}
				else
				{
					Filters.TrimStartAndEndInline();
					CategoryFilter.Add(Filters);
					break;
				}
			}
		}
		else if (!Switch.Compare("h", ESearchCase::IgnoreCase) || 
		         !Switch.Compare("?", ESearchCase::IgnoreCase) || 
				 !Switch.Compare("help", ESearchCase::IgnoreCase))
		{
			NewDumpFlags |= BPDUMP_LogHelp;
		}
		else if (!Switch.Compare("time", ESearchCase::IgnoreCase))
		{
			NewDumpFlags |= BPDUMP_RecordTiming;
		}
		else if (!Switch.Compare("experimental", ESearchCase::IgnoreCase))
		{
			NewDumpFlags &= ~BPDUMP_UseLegacyMenuBuilder;
		}
		else if (Switch.StartsWith("name="))
		{
			FString NameSwitch;
			Switch.Split(TEXT("="), &NameSwitch, &Filename);
		}
		else if (Switch.StartsWith("path="))
		{
			FString PathSwitch;
			Switch.Split(TEXT("="), &PathSwitch, &SaveDir);
		}
	}

	if (NewDumpFlags != 0)
	{
		DumpFlags = NewDumpFlags;
	}
}

//------------------------------------------------------------------------------
static UWorld* GenerateBlueprintAPIUtils::GetWorld()
{
	UWorld* World = nullptr;
	for (FWorldContext const& WorldContext : GEngine->GetWorldContexts())
	{
		World = WorldContext.World();
		if (World != nullptr)
		{
			break;
		}
	}

	if (World == nullptr)
	{
		static UWorld* CommandletWorld = nullptr;
		if (CommandletWorld == nullptr)
		{
			if (GUnrealEd == nullptr)
			{
				UE_LOGF(LogBlueprintAPIGenerate, Error, "Cannot create a temp map to test within, without a valid editor world");
			}
			else
			{
				CommandletWorld = GEditor->NewMap();
			}
		}

		World = CommandletWorld;
	}

	return World;
}


//------------------------------------------------------------------------------
static UBlueprint* GenerateBlueprintAPIUtils::MakeTempBlueprint(UClass* ParentClass)
{
	UBlueprint* MadeBlueprint = nullptr;
	if (UBlueprint** FoundBlueprint = ClassBlueprints.Find(ParentClass))
	{
		MadeBlueprint = *FoundBlueprint;
	}
	else
	{
		UObject* BlueprintOuter = GetTransientPackage();

		bool bIsAnimBlueprint = ParentClass->IsChildOf(UAnimInstance::StaticClass());
		bool bIsLevelBlueprint = ParentClass->IsChildOf(ALevelScriptActor::StaticClass());

		UClass* BlueprintClass = UBlueprint::StaticClass();
		UClass* GeneratedClass = UBlueprintGeneratedClass::StaticClass();
		EBlueprintType BlueprintType = BPTYPE_Normal;

		if (bIsAnimBlueprint)
		{
			BlueprintClass = UAnimBlueprint::StaticClass();
			GeneratedClass = UAnimBlueprintGeneratedClass::StaticClass();
		}
		else if (bIsLevelBlueprint)
		{
			UWorld* World = GetWorld();
			if (World == nullptr)
			{
				UE_LOGF(LogBlueprintAPIGenerate, Error, "Cannot make a proper level blueprint without a valid editor level for its outer.");
			}
			else
			{
				BlueprintClass = ULevelScriptBlueprint::StaticClass();
				BlueprintType = BPTYPE_LevelScript;
				BlueprintOuter = World->GetCurrentLevel();
			}
		}
		// @TODO: UEditorUtilityBlueprint

		FString const ClassName = ParentClass->GetName();
		FString const DesiredName = FString::Printf(TEXT("COMMANDLET_TEMP_Blueprint_%s"), *ClassName);
		FName   const TempBpName = MakeUniqueObjectName(BlueprintOuter, BlueprintClass, FName(*DesiredName));

		check(FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass));
		MadeBlueprint = FKismetEditorUtilities::CreateBlueprint(ParentClass, BlueprintOuter, TempBpName, BlueprintType, BlueprintClass, GeneratedClass);
		
		// if this is an animation blueprint, then we want anim specific graphs to test as well (if it has an anim graph)...
		if (bIsAnimBlueprint && (MadeBlueprint->FunctionGraphs.Num() > 0))
		{
			UAnimationGraph* AnimGraph = CastChecked<UAnimationGraph>(MadeBlueprint->FunctionGraphs[0]);
			check(AnimGraph != nullptr);

			// should add a state-machine graph
			UAnimGraphNode_StateMachine* StateMachineNode = AddNodeToGraph<UAnimGraphNode_StateMachine>(AnimGraph);

			UAnimationStateMachineGraph* StateMachineGraph = StateMachineNode->EditorStateMachineGraph;
			// should add an conduit graph 
			UAnimStateConduitNode* ConduitNode = AddNodeToGraph<UAnimStateConduitNode>((UEdGraph*)StateMachineGraph);

			UAnimStateNode* StateNode = AddNodeToGraph<UAnimStateNode>((UEdGraph*)StateMachineGraph);
			// should create a transition graph
			StateNode->AutowireNewNode(ConduitNode->GetOutputPin());
		}
		else if (bIsLevelBlueprint)
		{
			ULevel* Level = CastChecked<ULevel>(BlueprintOuter);
			Level->LevelScriptBlueprint = Cast<ULevelScriptBlueprint>(MadeBlueprint);
		}

		// may have been altered in CreateBlueprint()
		BlueprintType = MadeBlueprint->BlueprintType;

		bool bCanAddFunctions = (BlueprintType != BPTYPE_MacroLibrary); // taken from FBlueprintEditor::NewDocument_IsVisibleForType()
		if (bCanAddFunctions)
		{
			// add a functions graph that isn't the construction script (or an animation graph)
			FName FuncGraphName = MakeUniqueObjectName(MadeBlueprint, UEdGraph::StaticClass(), FName(TEXT("NewFunction")));
			UEdGraph* FuncGraph = FBlueprintEditorUtils::CreateNewGraph(MadeBlueprint, FuncGraphName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
			FBlueprintEditorUtils::AddFunctionGraph<UClass>(MadeBlueprint, FuncGraph, /*bIsUserCreated =*/true, nullptr);
		}

		bool bCanAddMacros = ((BlueprintType == BPTYPE_MacroLibrary) || (BlueprintType == BPTYPE_Normal) || (BlueprintType == BPTYPE_LevelScript));
		if (bCanAddMacros)
		{
			FName MacroGraphName = MakeUniqueObjectName(MadeBlueprint, UEdGraph::StaticClass(), FName(TEXT("NewMacro")));
			UEdGraph* MacroGraph = FBlueprintEditorUtils::CreateNewGraph(MadeBlueprint, MacroGraphName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
			FBlueprintEditorUtils::AddMacroGraph(MadeBlueprint, MacroGraph, /*bIsUserCreated =*/true, nullptr);
		}

		// if you can add custom events to this blueprint, do so (so show that we 
		// can call an event on ourselves)
		if (MadeBlueprint->UbergraphPages.Num() > 0)
		{
			UK2Node_CustomEvent* CustomEventNode = AddNodeToGraph<UK2Node_CustomEvent>(MadeBlueprint->UbergraphPages[0]);
			CustomEventNode->CustomFunctionName = FBlueprintEditorUtils::FindUniqueCustomEventName(MadeBlueprint);
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(MadeBlueprint);
		MadeBlueprint->AddToRoot(); // to keep the BP from being garbage collected
		FKismetEditorUtilities::CompileBlueprint(MadeBlueprint);
		ClassBlueprints.Add(ParentClass, MadeBlueprint);
	}

	check(MadeBlueprint != nullptr);
	return MadeBlueprint;
}

//------------------------------------------------------------------------------
template<class NodeType>
static NodeType* GenerateBlueprintAPIUtils::AddNodeToGraph(UEdGraph* Graph)
{
	check(Graph != nullptr);
	NodeType* NewNode = NewObject<NodeType>(Graph);
	Graph->AddNode(NewNode, /*bFromUI =*/true, /*bSelectNewNode =*/false);

	NewNode->CreateNewGuid();
	NewNode->PostPlacedNewNode();
	NewNode->AllocateDefaultPins();
	return NewNode;
}

//------------------------------------------------------------------------------
static FString GenerateBlueprintAPIUtils::BuildDumpFilePath(UClass* BlueprintClass)
{
	FString CommandletSaveDir, Filename;

	if (CommandOptions.SaveDir.IsEmpty())
	{
		CommandletSaveDir = FPaths::ProjectSavedDir() + TEXT("Commandlets/");
		CommandletSaveDir = FPaths::ConvertRelativePathToFull(CommandletSaveDir);
	}
	else
	{
		CommandletSaveDir= CommandOptions.SaveDir;
	}
	IFileManager::Get().MakeDirectory(*CommandletSaveDir);
	
	if (CommandOptions.Filename.IsEmpty())
	{
		Filename = FString::Printf(TEXT("GenerateBlueprintAPI_%s"), FPlatformTime::StrTimestamp());
		Filename = Filename.Replace(TEXT(" "), TEXT("_"));
		Filename = Filename.Replace(TEXT("/"), TEXT("-"));
		Filename = Filename.Replace(TEXT(":"), TEXT("."));
	}
	else
	{
		Filename = CommandOptions.Filename;
	}

	Filename += ".json";

	return CommandletSaveDir / *Filename;
}

//------------------------------------------------------------------------------
static FString GenerateBlueprintAPIUtils::BuildIndentString(uint32 IndentCount, bool bUseSpaces)
{
	char RepatingChar = '\t';
	if (bUseSpaces)
	{
		RepatingChar = ' ';
	}

	FString IndentString;
	while (IndentCount > 0)
	{
		IndentString += RepatingChar;
		--IndentCount;
	}
	return IndentString;
}

//------------------------------------------------------------------------------
static FString GenerateBlueprintAPIUtils::GetActionKey(TSharedPtr<FEdGraphSchemaAction> const& Action)
{
	TArray<FString> MenuHierarchy = Action->GetCategoryChain();

	FString ActionKey;
	for (FString const& SubCategory : MenuHierarchy)
	{
		ActionKey += SubCategory + TEXT("|");
	}
	if (MenuHierarchy.Num() > 0)
	{
		ActionKey.RemoveAt(ActionKey.Len() - 1); // remove the trailing '|'
	}

	ActionKey += Action->GetMenuDescription().ToString();

	return ActionKey;
}

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::GetComponentProperties(UBlueprint* Blueprint, TArray<FObjectProperty*>& PropertiesOut)
{
	UClass* BpClass = Blueprint->GeneratedClass;
	if (BpClass->IsChildOf<AActor>())
	{
		for (TFieldIterator<FObjectProperty> PropertyIt(BpClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			// SMyBlueprint filters out component variables in SMyBlueprint::CollectAllActions() using CPF_BlueprintVisible/CPF_Parm flags
			if (PropertyIt->PropertyClass->IsChildOf(UActorComponent::StaticClass()) &&
				PropertyIt->HasAnyPropertyFlags(CPF_BlueprintVisible) && !PropertyIt->HasAnyPropertyFlags(CPF_Parm))
			{
				PropertiesOut.Add(*PropertyIt);
			}
		}
	}
}

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpInfoForClass(uint32 Indent, UClass* BlueprintClass, FArchive* FileOutWriter)
{
	FString const ClassName = BlueprintClass->GetName();
	UE_LOGF(LogBlueprintAPIGenerate, Display, "%lsDumping BP class: '%ls'...", *BuildIndentString(Indent, true), *ClassName);

	FString const ClassEntryIndent = BuildIndentString(Indent);
	FString BeginClassEntry = FString::Printf(TEXT("%s\"%s\" : {"), *ClassEntryIndent, *ClassName);

	FString IndentedNewline = "\n" + BuildIndentString(Indent + 1);

	BeginClassEntry += IndentedNewline + "\"ClassContext\" : \"" + ClassName + "\",\n";
	FileOutWriter->Serialize(TCHAR_TO_ANSI(*BeginClassEntry), BeginClassEntry.Len());

	UBlueprint* TempBlueprint = MakeTempBlueprint(BlueprintClass);

	DumpPalette(Indent + 1, TempBlueprint, FileOutWriter);

	FString EndClassEntry = "\n" + ClassEntryIndent + "}";
	FileOutWriter->Serialize(TCHAR_TO_ANSI(*EndClassEntry), EndClassEntry.Len());
}

static TMap<FString, FText> CategoryTooltipsMap;

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpCategoryInfo(uint32 Indent, FArchive* FileOutWriter)
{
	const FString CategoryEntryIndent = BuildIndentString(Indent);
	FString CategoriesEntry = FString::Printf(TEXT(",\n%s\"Categories\" : {"), *CategoryEntryIndent);

	const FString IndentedNewline = "\n" + BuildIndentString(Indent + 1);
	const FString CategoryDetailsIndent = BuildIndentString(Indent + 2);
	bool bNeedComma = false;

	for (auto Category : CategoryTooltipsMap)
	{
		if (!Category.Value.IsEmpty())
		{
			CategoriesEntry += FString::Printf(TEXT("%s%s\"%s\" : {"), (bNeedComma ? TEXT(",") : TEXT("")), *IndentedNewline, *Category.Key);
			CategoriesEntry += FString::Printf(TEXT("\n%s\"Tooltip\"\t: \"%s\""), *CategoryDetailsIndent, *MakeJsonString(Category.Value.ToString()));
			CategoriesEntry += FString::Printf(TEXT("\n%s}"), *IndentedNewline);

			bNeedComma = true;
		}
	}

	CategoriesEntry += FString::Printf(TEXT("\n%s}"), *CategoryEntryIndent);

	FileOutWriter->Serialize(TCHAR_TO_ANSI(*CategoriesEntry), CategoriesEntry.Len());
}

//------------------------------------------------------------------------------
static double GenerateBlueprintAPIUtils::GetPaletteMenuActions(FCategorizedGraphActionListBuilder& PaletteBuilder, UBlueprint* Blueprint, UClass* PaletteFilter)
{
	PaletteBuilder.Empty();
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

	double MenuBuildDuration = 0.0;
	
	FBlueprintActionContext FilterContext;
	FilterContext.Blueprints.Add(const_cast<UBlueprint*>(Blueprint));

	FBlueprintActionMenuBuilder MenuBuilder;
	{
		// prime the database so it's not recorded in our timing capture
		FBlueprintActionDatabase::Get();

		FScopedDurationTimer DurationTimer(MenuBuildDuration);
		FBlueprintActionMenuUtils::MakePaletteMenu(FilterContext, PaletteFilter, MenuBuilder);
		PaletteBuilder.Append(MenuBuilder);
	}

	return MenuBuildDuration;
}

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpPalette(uint32 Indent, UBlueprint* Blueprint, FArchive* FileOutWriter)
{
	UClass* PaletteFilter = CommandOptions.PaletteFilter;
	DumpPalette(Indent, Blueprint, PaletteFilter, FileOutWriter);
	bool bNeedsEndline = true;

	if ((CommandOptions.DumpFlags & BPDUMP_FilteredPalette) && (PaletteFilter == nullptr))
	{
		// anim blueprints don't have a palette, so it is ok to assume this
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* Class = *ClassIt;
			if (!K2Schema->ClassHasBlueprintAccessibleMembers(Class))
			{
				continue;
			}

			FileOutWriter->Serialize(TCHAR_TO_ANSI(TEXT(",\n")), 2);
			DumpPalette(Indent, Blueprint, Class, FileOutWriter);
		}
	}
}

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpPalette(uint32 Indent, UBlueprint* Blueprint, UClass* ClassFilter, FArchive* FileOutWriter)
{
	FString const PaletteEntryIndent = BuildIndentString(Indent);
	FString BeginPaletteEntry = FString::Printf(TEXT("%s\"Palette"), *PaletteEntryIndent);

	FString FilterClassName("<UNFILTERED>");
	if (ClassFilter != nullptr)
	{
		FilterClassName = ClassFilter->GetName();
		BeginPaletteEntry += "-" + FilterClassName;
	}
	BeginPaletteEntry += "\" : {\n";

	FString const NestedIndent = BuildIndentString(Indent + 1);
	UE_LOGF(LogBlueprintAPIGenerate, Display, "%lsDumping palette: %ls", *BuildIndentString(Indent, true), *FilterClassName);

	bool bIsAnimBlueprint = (Cast<UAnimBlueprint>(Blueprint) != nullptr);
	// animation blueprints don't have a palette
	if (bIsAnimBlueprint)
	{
		BeginPaletteEntry += NestedIndent + "\"IsAnimBlueprint\" : true";
		FileOutWriter->Serialize(TCHAR_TO_ANSI(*BeginPaletteEntry), BeginPaletteEntry.Len());
	}
	else
	{
		FCategorizedGraphActionListBuilder PaletteBuilder;
		PaletteBuilder.OwnerOfTemporaries = NewObject<UEdGraph>((UObject*)Blueprint);
		PaletteBuilder.OwnerOfTemporaries->Schema = UEdGraphSchema_K2::StaticClass();
		PaletteBuilder.OwnerOfTemporaries->SetFlags(RF_Transient);

		double MenuBuildDuration = GetPaletteMenuActions(PaletteBuilder, Blueprint, ClassFilter);

		BeginPaletteEntry += NestedIndent + "\"FilterClass\" : \"" + FilterClassName + "\",\n";
		if (CommandOptions.DumpFlags & BPDUMP_RecordTiming)
		{
			BeginPaletteEntry += FString::Printf(TEXT("%s\"MenuBuildTime_Seconds\" : %f,\n"), *NestedIndent, MenuBuildDuration);
		}

		FileOutWriter->Serialize(TCHAR_TO_ANSI(*BeginPaletteEntry), BeginPaletteEntry.Len());
		DumpActionList(Indent + 1, PaletteBuilder, FileOutWriter);
	}

	FString EndPaletteEntry = "\n" + PaletteEntryIndent + "}";
	FileOutWriter->Serialize(TCHAR_TO_ANSI(*EndPaletteEntry), EndPaletteEntry.Len());
}

//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpActionList(uint32 Indent, FGraphActionListBuilderBase& ActionList, FArchive* FileOutWriter)
{
	TArray< TSharedPtr<FEdGraphSchemaAction> > SortedActions;
	const TArray<FString>& CategoryFilter = GenerateBlueprintAPIUtils::CommandOptions.CategoryFilter;
	bool HasCategoryFilter = CategoryFilter.Num() > 0;

	for (int32 ActionIndex = 0; ActionIndex < ActionList.GetNumActions(); ++ActionIndex)
	{
		TSharedPtr<FEdGraphSchemaAction>& Action = ActionList.GetSchemaAction(ActionIndex);
		if (!Action.IsValid())
		{
			continue;
		}

		if (!HasCategoryFilter)
		{
			SortedActions.Add(Action);
		}
		else
		{
			const FString& ActionCategory = GenerateBlueprintAPIUtils::GetActionCategory(Action);
			for (const FString& filter : CategoryFilter)
			{
				if (ActionCategory.Contains(filter, ESearchCase::CaseSensitive, ESearchDir::FromStart))
				{
					SortedActions.Add(Action);
					break;
				}
			}
		}
	}

	FString const ActionListIndent = BuildIndentString(Indent);
	FString const NestedIndent     = BuildIndentString(Indent + 1);

	FString BeginActionListEntry = FString::Printf(TEXT("%s\"ActionSet\" : {\n%s\"ActionCount\" : %d"), *ActionListIndent, *NestedIndent, SortedActions.Num());
	BeginActionListEntry += ",\n" + NestedIndent + "\"Actions\" : {";

	FileOutWriter->Serialize(TCHAR_TO_ANSI(*BeginActionListEntry), BeginActionListEntry.Len());

	struct ActionSortFunctor
	{
		bool operator()(TSharedPtr<FEdGraphSchemaAction> const& LHSAction, TSharedPtr<FEdGraphSchemaAction> const& RHSAction) const
		{
			if (LHSAction->GetGrouping() != RHSAction->GetGrouping())
			{
				return LHSAction->GetGrouping() > RHSAction->GetGrouping();
			}
			
			FString LhKey = GetActionKey(LHSAction);
			FString RhKey = GetActionKey(RHSAction);
			return (LhKey.Compare(RhKey) < 0);
		}
	};
	// need to sort so we can easily compare from one generation to the next
	SortedActions.Sort(ActionSortFunctor());

	FString LineEnding("\n");
	for (const TSharedPtr<FEdGraphSchemaAction>& Action : SortedActions)
	{
		FileOutWriter->Serialize(TCHAR_TO_ANSI(*LineEnding), LineEnding.Len());
		DumpActionMenuItem(Indent + 2, Action, ActionList, FileOutWriter);
		LineEnding = TEXT(",\n");
	}

	FString EndActionListEntry = "\n" + NestedIndent + "}";

	EndActionListEntry += "\n" + ActionListIndent + "}";
	FileOutWriter->Serialize(TCHAR_TO_ANSI(*EndActionListEntry), EndActionListEntry.Len());
}
//------------------------------------------------------------------------------
static FString GenerateBlueprintAPIUtils::GetActionCategory(TSharedPtr<FEdGraphSchemaAction> const& Action)
{
	// Get action category info
	const TArray<FString>& MenuHierarchy = Action->GetCategoryChain();

	FString ActionCategory;

	bool bHasCategory = (MenuHierarchy.Num() > 0);
	if (bHasCategory)
	{
		for (FString const& SubCategory : MenuHierarchy)
		{
			ActionCategory += SubCategory + TEXT("|");
		}
	}
	return ActionCategory;
}
//------------------------------------------------------------------------------
static void GenerateBlueprintAPIUtils::DumpActionMenuItem(uint32 Indent, TSharedPtr<FEdGraphSchemaAction> const& Action, FGraphActionListBuilderBase& ActionList, FArchive* FileOutWriter)
{
	const FString& ActionCategory = GenerateBlueprintAPIUtils::GetActionCategory(Action);
	TArray<FString> Categories;
	ActionCategory.ParseIntoArray(Categories, TEXT("|"), true);

	for (const FString& Category : Categories)
	{
		FString CategoryDisplayName = FEditorCategoryUtils::GetCategoryDisplayString(Category);
		if (!CategoryTooltipsMap.Contains(CategoryDisplayName))
		{
			FText Tooltip;
			FString DocLink, DocExcerpt;

			FEditorCategoryUtils::GetCategoryTooltipInfo(CategoryDisplayName, Tooltip, DocLink, DocExcerpt);
			CategoryTooltipsMap.Add(CategoryDisplayName, Tooltip);
		}
	}

	const FString ActionName = Action->GetMenuDescription().ToString();

	const FString ActionEntryIndent = BuildIndentString(Indent);
	FString ActionEntry = ActionEntryIndent + "\"" + MakeJsonString(ActionCategory + ActionName) + "\"";

	++Indent;
	FString const IndentedNewline = "\n" + BuildIndentString(Indent);

	ActionEntry += " : {";

	const FString TooltipFieldLabel("\"Tooltip\"      : \"");
	const FString TooltipStr = Action->GetTooltipDescription().ToString().Replace(TEXT("\n"), *(IndentedNewline + BuildIndentString(TooltipFieldLabel.Len(), /*bUseSpaces =*/true)));

	ActionEntry += IndentedNewline + TooltipFieldLabel + MakeJsonString(TooltipStr) + "\"";
		
	// Get action node type info
	UK2Node const* NodeTemplate = FBlueprintActionMenuUtils::ExtractNodeTemplateFromAction(Action);
	if (NodeTemplate != nullptr)
	{
		UK2Node* Node = DuplicateObject<UK2Node>(NodeTemplate,ActionList.OwnerOfTemporaries);
		ActionList.OwnerOfTemporaries->AddNode(Node);
		Node->AllocateDefaultPins();

		if (Node->ShouldDrawCompact())
		{
			ActionEntry += "," + IndentedNewline + "\"CompactTitle\" : \"" + MakeJsonString(Node->GetCompactNodeTitle().ToString()) + "\"";
		}

		if (Node->IsNodePure())
		{
			ActionEntry += "," + IndentedNewline + "\"NodeType\"     : \"pure\"";
		}
		else if (Node->IsA<UK2Node_Event>())
		{
			ActionEntry += "," + IndentedNewline + "\"NodeType\"     : \"event\"";
		}
		else if (Node->IsA<UK2Node_Switch>())
		{
			ActionEntry += "," + IndentedNewline + "\"NodeType\"     : \"switch\"";
		}
		else
		{
			ActionEntry += "," + IndentedNewline + "\"NodeType\"     : \"function\"";
		}

		if (Node->IsA<UK2Node_CommutativeAssociativeBinaryOperator>()
			|| (Node->IsA<UK2Node_Switch>() && !Node->IsA<UK2Node_SwitchEnum>()))
		{
			ActionEntry += "," + IndentedNewline + "\"ShowAddPin\"   : \"true\"";
		}

		if (Node->Pins.Num() > 0)
		{
			ActionEntry += "," + IndentedNewline + "\"Pins\"         : {";

			const FString PinEntryIndentedNewline = "\n" + BuildIndentString(Indent+1);
			const FString PinDetailsIndentedNewline = "\n" + BuildIndentString(Indent+2);

			static UEnum* PinDirectionEnum = StaticEnum<EEdGraphPinDirection>();

			bool bFirst = true;
			for (UEdGraphPin* Pin : Node->Pins)
			{
				if (!Pin->bHidden)
				{
					// @hack: Some pin data will not be available until requested for display, specifically tooltip strings for call function nodes:
					FString Scratch;
					Node->GetPinHoverText(*Pin, Scratch);

					if (!bFirst)
					{
						ActionEntry += ",";
					}
					bFirst = false;

					ActionEntry += PinEntryIndentedNewline + "\"" + MakeJsonString(Pin->GetName()) + "\" : {";

					const FString DisplayName = Pin->GetDisplayName().ToString();

					if (!DisplayName.IsEmpty())
					{
						ActionEntry += PinDetailsIndentedNewline + "\"Name\"                 : \"" + MakeJsonString(DisplayName) + "\",";
					}
					ActionEntry += PinDetailsIndentedNewline + "\"Direction\"            : \"" + (Pin->Direction == EGPD_Input ? "input" : "output") + "\",";

					ActionEntry += PinDetailsIndentedNewline + "\"TypeText\"             : \"" + UEdGraphSchema_K2::TypeToText(Pin->PinType).ToString() + "\"";

					if (!Pin->PinToolTip.IsEmpty())
					{
						const FString PinTooltipFieldLabel("\"Tooltip\"              : \"");
						const FString PinTooltipStr = Pin->PinToolTip.Replace(TEXT("\n"), *(PinDetailsIndentedNewline + BuildIndentString(PinTooltipFieldLabel.Len(), /*bUseSpaces =*/true)));

						ActionEntry += "," + PinDetailsIndentedNewline + PinTooltipFieldLabel + MakeJsonString(PinTooltipStr) + "\"";
					}

					ActionEntry += "," + PinDetailsIndentedNewline + "\"PinCategory\"          : \"" + MakeJsonString(Pin->PinType.PinCategory.ToString()) + "\"";

					if (!Pin->PinType.PinSubCategory.IsNone())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"PinSubCategory\"       : \"" + MakeJsonString(Pin->PinType.PinSubCategory.ToString()) + "\"";
					}

					if (Pin->PinType.PinSubCategoryObject.IsValid())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"PinSubCategoryObject\" : \"" + MakeJsonString(Pin->PinType.PinSubCategoryObject->GetName()) + "\"";
					}

					if (!CastChecked<UEdGraphSchema_K2>(Node->GetSchema())->ShouldShowAssetPickerForPin(Pin))
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"ShowAssetPicker\"      : \"false\"";
					}

					if (!Pin->DefaultValue.IsEmpty())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"DefaultValue\"         : \"" + MakeJsonString(Pin->DefaultValue) + "\"";
					}

					if (Pin->PinType.IsArray())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"IsArray\"              : \"true\"";
					}
					else if (Pin->PinType.IsSet())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"IsSet\"              : \"true\"";
					}
					else if (Pin->PinType.IsMap())
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"IsMap\"              : \"true\"";
						// TODO: Send the Map value type as well
					}

					if (Pin->PinType.bIsConst)
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"IsConst\"              : \"true\"";
					}

					if (Pin->PinType.bIsReference)
					{
						ActionEntry += "," + PinDetailsIndentedNewline + "\"IsReference\"          : \"true\"";
					}

					ActionEntry += PinEntryIndentedNewline + "}";
				}
			}

			ActionEntry += IndentedNewline + "}";
		}

		ActionList.OwnerOfTemporaries->RemoveNode(Node);
	}
	// Finish action entry
	ActionEntry += "\n" + ActionEntryIndent + "}";

	// Write entry to file
	FileOutWriter->Serialize(TCHAR_TO_ANSI(*ActionEntry), ActionEntry.Len());
}

/*******************************************************************************
 * UGenerateBlueprintAPICommandlet
 ******************************************************************************/

//------------------------------------------------------------------------------
UGenerateBlueprintAPICommandlet::UGenerateBlueprintAPICommandlet(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
int32 UGenerateBlueprintAPICommandlet::Main(FString const& Params)
{
	UEdGraphSchema_K2::bGeneratingDocumentation = true;

	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	GenerateBlueprintAPIUtils::CommandOptions = GenerateBlueprintAPIUtils::CommandletOptions(Switches);
	GenerateBlueprintAPIUtils::CommandletOptions const& CommandOptions = GenerateBlueprintAPIUtils::CommandOptions;

	FModuleManager::Get().LoadModuleChecked(TEXT("Blutility"));

	FString ActiveFilePath;
	FArchive* FileOut = nullptr;

	// this lambda is responsible for adding closing characters to the file, and 
	// closing out the writer (and diffing the resultant file if the user deigns us to do so)
	auto CloseFileStream = [](FArchive** FileOutPtr)
	{
		FArchive*& FileOutAr = (*FileOutPtr);
		if (FileOutAr != nullptr)
		{
			FileOutAr->Serialize(TCHAR_TO_ANSI(TEXT("\n}")), 2);
			FileOutAr->Close();

			delete FileOutAr;
			FileOutAr = nullptr;
		}
	};

	// this lambda is responsible for opening a file for write, and adding 
	// opening json characters to the file (contextually tracks the active filepath as well)
	auto OpenFileStream = [&ActiveFilePath, &CloseFileStream, &FileOut](UClass* Class)->FArchive*
	{
		CloseFileStream(&FileOut);

		ActiveFilePath = GenerateBlueprintAPIUtils::BuildDumpFilePath(Class);
		FileOut = IFileManager::Get().CreateFileWriter(*ActiveFilePath);
		check(FileOut != nullptr);
		FileOut->Serialize(TCHAR_TO_ANSI(TEXT("{\n")), 2);

		return FileOut;
	};
	
	bool bNeedsJsonComma = false;
	// this lambda will dump blueprint info for the specified class, if the user
	// set -multifile, then this will also close the existing file and open a new one for this class
	auto WriteClassInfo = [&bNeedsJsonComma, &OpenFileStream, &CloseFileStream, &FileOut](UClass* Class)
	{
		if (FileOut == nullptr)
		{
			FileOut = OpenFileStream(Class);
		}
		// if we're adding all the class entries into one file, then we need to 
		// separate them by a comma (or invalid json)
		else if (bNeedsJsonComma)
		{
			FileOut->Serialize(TCHAR_TO_ANSI(TEXT(",\n")), 2);
		}

		GenerateBlueprintAPIUtils::DumpInfoForClass(1, Class, FileOut);
		bNeedsJsonComma = true;
	};

	// this lambda is used as a precursory check to ensure that the specified 
	// class is a blueprintable type... broken into it's own lambda to save on reuse
	auto IsInvalidBlueprintClass = [](UClass* Class)->bool
	{
		return !IsValid(Class) ||
			Class->HasAnyClassFlags(CLASS_NewerVersionExists) ||
			FKismetEditorUtilities::IsClassABlueprintSkeleton(Class)  ||
			!FKismetEditorUtilities::CanCreateBlueprintOfClass(Class) ||
			(Class->GetOuterUPackage() == GetTransientPackage());
	};

	UClass* const BlueprintClass = CommandOptions.BlueprintClass;
	if (CommandOptions.DumpFlags & GenerateBlueprintAPIUtils::BPDUMP_LogHelp)
	{
		UE_LOGF(LogBlueprintAPIGenerate, Display, "%ls", *GenerateBlueprintAPIUtils::HelpString);
	}
	else if (BlueprintClass != nullptr)
	{
		UE_LOGF(LogBlueprintAPIGenerate, Display, "Dumping Blueprint info...");
		// make sure the class that the user specified is a blueprintable type
		if (IsInvalidBlueprintClass(BlueprintClass))
		{
			UE_LOGF(LogBlueprintAPIGenerate, Error, "Cannot create a blueprint from class: '%ls'", *BlueprintClass->GetName());
			if (FileOut != nullptr)
			{
				FString const InvalidClassEntry = GenerateBlueprintAPIUtils::BuildIndentString(1) + "\"INVALID_BLUEPRINT_CLASS\" : \"" + BlueprintClass->GetName() + "\"";
				FileOut->Serialize(TCHAR_TO_ANSI(*InvalidClassEntry), InvalidClassEntry.Len());
			}
		}
		else
		{
			WriteClassInfo(BlueprintClass);
		}
	}
	// if the user didn't specify a class, then we take that to mean dump ALL the classes!
	else
	{
		UE_LOGF(LogBlueprintAPIGenerate, Display, "Dumping Blueprint info...");
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* Class = *ClassIt;
			if (IsInvalidBlueprintClass(Class))
			{
				continue;
			}
			WriteClassInfo(Class);
		}
	}

	if(FileOut)
	{
		GenerateBlueprintAPIUtils::DumpCategoryInfo(1, FileOut);

		CloseFileStream(&FileOut);
	}

	return 0;
}




