// Copyright Epic Games, Inc. All Rights Reserved.


#include "Commandlets/GenerateDistillFileSetsCommandlet.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "UObject/UObjectHash.h"
#include "Misc/PackageName.h"
#include "Settings/ProjectPackagingSettings.h"
#include "FileHelpers.h"
#include "Misc/RedirectCollector.h"
#include "Editor.h"
#include "Engine/AssetManager.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateDistillFileSetsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateDistillFileSetsCommandlet, Log, All);

UGenerateDistillFileSetsCommandlet::UGenerateDistillFileSetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

int32 UGenerateDistillFileSetsCommandlet::Main( const FString& InParams )
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*InParams, Tokens, Switches);

	TArray<FString> MapList;
	for ( int32 MapIdx = 0; MapIdx < Tokens.Num(); ++MapIdx )
	{
		const FString& Map = Tokens[MapIdx];
		if ( FPackageName::IsShortPackageName(Map) )
		{
			FString LongPackageName;
			if ( FPackageName::SearchForPackageOnDisk(Map, &LongPackageName) )
			{
				MapList.Add(LongPackageName);
			}
			else
			{
				UE_LOGF(LogGenerateDistillFileSetsCommandlet, Error, "Unable to find package for map %ls.", *Map);
				return 1;
			}
		}
		else
		{
			MapList.Add(Map);
		}
	}

	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();

	if ( MapList.Num() <= 0 )
	{
		// No map tokens were supplied on the command line, so assume all maps
		TArray<FString> AllPackageFilenames;
		FEditorFileUtils::FindAllPackageFiles(AllPackageFilenames);
		for (int32 PackageIndex = 0; PackageIndex < AllPackageFilenames.Num(); PackageIndex++)
		{
			const FString& Filename = AllPackageFilenames[PackageIndex];
			if (FPaths::GetExtension(Filename, true) == FPackageName::GetMapPackageExtension() )
			{
				FString LongPackageName;
				if ( FPackageName::TryConvertFilenameToLongPackageName(Filename, LongPackageName) )
				{
					// Warn about maps in "NoShip" or "TestMaps" folders.  Those should have been filtered out during the Distill process!
					if( !Filename.Contains( "/NoShip/") && !Filename.Contains( "/TestMaps/"))
					{
						// @todo plugins add support for plugins?
						if ( LongPackageName.StartsWith(TEXT("/Game")) )
						{
							UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Discovered map package %ls...", *LongPackageName );
							MapList.Add(LongPackageName);
						}
					}
					else
					{
						UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Skipping map package %ls in TestMaps or NoShip folder", *Filename);
					}
				}
				else
				{
					UE_LOGF(LogGenerateDistillFileSetsCommandlet, Warning, "Failed to determine package name for map file %ls.", *Filename);
				}
			}
		}
	}
	else
	{
		// Add the default map section
		TArray<FString> AlwaysCookMapList;
		GEditor->LoadMapListFromIni(TEXT("AlwaysCookMaps"), AlwaysCookMapList);
		MapList.Append(AlwaysCookMapList);

		// Add Maps to cook from project packaging settings if any exist
		for (const FFilePath& MapToCook : PackagingSettings->MapsToCook)
		{
			MapList.AddUnique(MapToCook.FilePath);
		}
	}

	// Add any assets from the asset manager
	UAssetManager& Manager = UAssetManager::Get();
	TArray<FPrimaryAssetTypeInfo> TypeInfos;
	Manager.GetPrimaryAssetTypeInfoList(TypeInfos);

	for (const FPrimaryAssetTypeInfo& TypeInfo : TypeInfos)
	{
		TArray<FAssetData> AssetDataList;

		Manager.GetPrimaryAssetDataList(TypeInfo.PrimaryAssetType, AssetDataList);

		for (const FAssetData& AssetData : AssetDataList)
		{
			FString PackageName = AssetData.PackageName.ToString();
			// Warn about maps in "NoShip" or "TestMaps" folders.
			if (PackageName.Contains("/NoShip/") || PackageName.Contains("/TestMaps/"))
			{
				UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Skipping map package %ls in TestMaps or NoShip folder", *PackageName);
				continue;
			}
			MapList.AddUnique(AssetData.PackageName.ToString());
		}
	}
	
	const FString TemplateFileSwitch = TEXT("Template=");
	const FString OutputFileSwitch = TEXT("Output=");
	const FString TemplateFolderSwitch = TEXT("TemplateFolder=");
	const FString OutputFolderSwitch = TEXT("OutputFolder=");
	FString TemplateFilename;
	FString OutputFilename;
	FString TemplateFolder;
	FString OutputFolder;


	for (int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); ++SwitchIdx)
	{
		const FString& Switch = Switches[SwitchIdx];
		if ( Switch.StartsWith(TemplateFileSwitch) )
		{
			Switch.Split(TEXT("="), NULL, &TemplateFilename);
			TemplateFilename = TemplateFilename.TrimQuotes();
		}
		else if ( Switch.StartsWith(OutputFileSwitch) )
		{
			Switch.Split(TEXT("="), NULL, &OutputFilename);
			OutputFilename = OutputFilename.TrimQuotes();
		}
		else if ( Switch.StartsWith(TemplateFolderSwitch) )
		{
			Switch.Split(TEXT("="), NULL, &TemplateFolder);
			TemplateFolder = TemplateFolder.TrimQuotes();
			FPaths::NormalizeFilename(TemplateFolder);
			if ( !TemplateFolder.EndsWith(TEXT("/")) )
			{
				TemplateFolder += TEXT("/");
			}
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Using template folder: %ls", *TemplateFolder);
		}
		else if ( Switch.StartsWith(OutputFolderSwitch) )
		{
			Switch.Split(TEXT("="), NULL, &OutputFolder);
			OutputFolder = OutputFolder.TrimQuotes();
			FPaths::NormalizeFilename(OutputFolder);
			if ( !OutputFolder.EndsWith(TEXT("/")) )
			{
				OutputFolder += TEXT("/");
			}
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Using output folder: %ls", *OutputFolder);
		}
	}

	if ( OutputFilename.IsEmpty() )
	{
		UE_LOGF(LogGenerateDistillFileSetsCommandlet, Error, "You must supply -Output=OutputFilename. These files are relative to the Game/Build directory.");
		return 1;
	}
	if (OutputFolder.IsEmpty())
	{
		OutputFolder = FPaths::ProjectDir() + TEXT("Build/");
	}
	OutputFilename = OutputFolder + OutputFilename;

	bool bSimpleTxtOutput = false;
	// Load the template file
	FString TemplateFileContents;

	if (TemplateFilename.IsEmpty())
	{
		UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "No template specified, assuming a simple txt output.");
		bSimpleTxtOutput = true;
	}
	// If no folder was specified, filenames are relative to the build dir.
	else 
	{
		if (TemplateFolder.IsEmpty())
		{
			TemplateFolder = FPaths::ProjectDir() + TEXT("Build/");
		}
		TemplateFilename = TemplateFolder + TemplateFilename;

		if (!FFileHelper::LoadFileToString(TemplateFileContents, *TemplateFilename))
		{
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Error, "Failed to load template file '%ls'", *TemplateFilename);
			return 1;
		}
	}




	// Form a full unique package list
	TSet<FString> AllPackageNames;

	// Slate
	{
		TArray<FString> UIContentPaths;
		if (GConfig->GetArray(TEXT("UI"), TEXT("ContentDirectories"), UIContentPaths, GEditorIni) > 0)
		{
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Warning, "The [UI]ContentDirectories is deprecated. You may use DirectoriesToAlwaysCook in your project settings instead.");
		}
	}

	// Load all maps
	{
		for ( auto MapIt = MapList.CreateConstIterator(); MapIt; ++MapIt )
		{
			const FString& MapPackage = *MapIt;
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Loading %ls...", *MapPackage );
			UPackage* Package = LoadPackage( NULL, *MapPackage, LOAD_None );
			if( Package != NULL )
			{
				GRedirectCollector.ResolveAllSoftObjectPaths();

				AllPackageNames.Add(Package->GetName());

				UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Finding content referenced by %ls...", *MapPackage );

				auto GatherLoadedPackages = [&]()
				{
					TArray<UObject *> AllPackages;
					GetObjectsOfClass(UPackage::StaticClass(), AllPackages);
					for (int32 Index = 0; Index < AllPackages.Num(); Index++)
					{
						FString OtherName = AllPackages[Index]->GetOutermost()->GetName();
						if (!AllPackageNames.Contains(OtherName))
						{
							AllPackageNames.Add(OtherName);
							UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "Package: %ls", *OtherName);
						}
					}
				};

				// Load all external actor packages to gather their dependencies
				if (UWorld* World = UWorld::FindWorldInPackage(Package))
				{
					World->AddToRoot();

					uint32 ActorPackageIndex = 0;
					TArray<FString> ExternalActorPackages = World->PersistentLevel->GetOnDiskExternalActorPackages();

					for (const FString& ExternalActorPackage : ExternalActorPackages)
					{
						if (!AllPackageNames.Contains(ExternalActorPackage))
						{
							AllPackageNames.Add(ExternalActorPackage);
							UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "Package: %ls", *ExternalActorPackage);

							FString LongActorPackageName;
							FPackageName::TryConvertFilenameToLongPackageName(ExternalActorPackage, LongActorPackageName);
							UPackage* ActorPackage = LoadPackage(nullptr, *LongActorPackageName, LOAD_None);

							if (!(++ActorPackageIndex % 50))
							{
								GatherLoadedPackages();
								UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Collecting garbage..." );
								CollectGarbage(RF_NoFlags);
							}
						}
					}

					World->RemoveFromRoot();
				}

				GatherLoadedPackages();
				UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Collecting garbage..." );
				CollectGarbage(RF_NoFlags);
			}
		}
	}

	// Add assets from additional directories to always cook
	for (const auto& DirToCook : PackagingSettings->DirectoriesToAlwaysCook)
	{
		FString DirectoryPath;
		if (!FPackageName::TryConvertGameRelativePackagePathToLocalPath(DirToCook.Path, DirectoryPath))
		{
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Warning, "'ProjectSettings -> Project -> Packaging -> Directories to always cook' has invalid element '%ls'", *DirToCook.Path);
			continue;
		}

		UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "Examining directory to always cook: %ls...", *DirToCook.Path );

		TArray<FString> Files;
		IFileManager::Get().FindFilesRecursive(Files, *DirectoryPath, *(FString(TEXT("*")) + FPackageName::GetAssetPackageExtension()), true, false);
		for (int32 Index = 0; Index < Files.Num(); Index++)
		{
			FString StdFile = Files[Index];
			FPaths::MakeStandardFilename(StdFile);
			StdFile = FPackageName::FilenameToLongPackageName(StdFile);
			AllPackageNames.Add(StdFile);
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "Package: %ls", *StdFile );
		}
	}

	// Sort the results to make it easier to diff files. No necessary but useful sometimes.
	TArray<FString> SortedPackageNames = AllPackageNames.Array();
	SortedPackageNames.Sort();

	// For the list of FileSets to include in the distill
	FString AllFileSets;
	const FString FileSetPathRoot = TEXT("Content");
	for (auto PackageIt = SortedPackageNames.CreateConstIterator(); PackageIt; ++PackageIt)
	{
		const FString& PackageName = *PackageIt;
		// @todo plugins add support for plugins?
		if ( PackageName.StartsWith(TEXT("/Game")) )
		{
			const FString PathWithoutRoot( PackageName.Mid( 5 ) );
			const FString FileSetPath = FileSetPathRoot + PathWithoutRoot;
			if (bSimpleTxtOutput)
			{
				FString ActualFile;
				if (FPackageName::DoesPackageExist(PackageName, &ActualFile))
				{
					ActualFile = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ActualFile);
					AllFileSets += FString::Printf(TEXT("%s") LINE_TERMINATOR, *ActualFile);
					UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "File: %ls", *ActualFile);
				}
			}
			else
			{
				AllFileSets += FString::Printf(TEXT("<FileSet Path=\"%s.*\" bIsRecursive=\"false\"/>") LINE_TERMINATOR, *FileSetPath);
			}
		}
	}

	// Add additional files marked for distillation
	TArray<FString> AdditionalFilesToDistill;
	GConfig->GetArray(TEXT("DistillSettings"), TEXT("FilesToAlwaysDistill"), AdditionalFilesToDistill, GEngineIni);

	const FString AbsoluteGameContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
	TArray<FString> AdditionalFilesToDistillFullPath;
	for (const FString& File : AdditionalFilesToDistill)
	{
		//Only support path relative to content directory
		if (!FPaths::IsRelative(File))
		{
			continue;
		}

		FString FileAbsolutePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(AbsoluteGameContentDir, File));

		FString Path;
		FString Filename;
		FString Extension;
		FPaths::Split(FileAbsolutePath, Path, Filename, Extension);
		if (!FPaths::DirectoryExists(Path)
			|| Filename.IsEmpty())
		{
			continue;
		}

		//Verify if we're looking at an arbitrary amount of desired files
		if (Filename == TEXT("*"))
		{
			TArray<FString> SubDirectoryFiles;
			IFileManager::Get().FindFilesRecursive(SubDirectoryFiles, *Path, *(Filename + (!Extension.IsEmpty() ? TEXT(".") + Extension : TEXT(""))), true, false);
			AdditionalFilesToDistillFullPath.Append(SubDirectoryFiles);
		}
		else
		{
			AdditionalFilesToDistillFullPath.Add(FileAbsolutePath);
		}
	}

	for (FString& AdditionalFile : AdditionalFilesToDistillFullPath)
	{
		FPaths::MakeStandardFilename(AdditionalFile);
		if (FPaths::FileExists(AdditionalFile))
		{
			AdditionalFile = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*AdditionalFile);
			AllFileSets += FString::Printf(TEXT("%s") LINE_TERMINATOR, *AdditionalFile);
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Log, "Additional file: %ls", *AdditionalFile);
		}
	}

	// Write the output file
	FString OutputFileContents;
	if (bSimpleTxtOutput)
	{
		OutputFileContents = AllFileSets;
	}
	else
	{
		OutputFileContents = TemplateFileContents.Replace(TEXT("%INSTALLEDCONTENTFILESETS%"), *AllFileSets, ESearchCase::CaseSensitive);
		if (FApp::HasProjectName())
		{
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Display, "Replacing %%GAMENAME%% with (%ls)...", FApp::GetProjectName());
			OutputFileContents = OutputFileContents.Replace(TEXT("%GAMENAME%"), FApp::GetProjectName(), ESearchCase::CaseSensitive);
		}
		else
		{
			UE_LOGF(LogGenerateDistillFileSetsCommandlet, Warning, "Failed to replace %%GAMENAME%% since we are running without a game name.");
		}
	}

	if ( !FFileHelper::SaveStringToFile(OutputFileContents, *OutputFilename) )
	{
		UE_LOGF(LogGenerateDistillFileSetsCommandlet, Error, "Failed to save output file '%ls'", *OutputFilename);
		return 1;
	}

	return 0;
}
