// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GenerateGatherArchiveCommandlet.h"

#include "Async/ParallelFor.h"
#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Internationalization/Text.h"
#include "LocTextHelper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateGatherArchiveCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateArchiveCommandlet, Log, All);
namespace GenerateGatherArchiveCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

void ConditionTranslationMetadata(TSharedRef<FLocMetadataValue> MetadataValue)
{
	switch (MetadataValue->GetType())
	{
	case ELocMetadataType::String:
	{
		TSharedPtr<FLocMetadataValue> MetadataValuePtr = MetadataValue;
		TSharedPtr<FLocMetadataValueString> MetadataString = StaticCastSharedPtr<FLocMetadataValueString>(MetadataValuePtr);
		MetadataString->SetString(TEXT(""));
	}
	break;

	case ELocMetadataType::Array:
	{
		TArray< TSharedPtr< FLocMetadataValue > > MetadataArray = MetadataValue->AsArray();
		for (auto ArrayIter = MetadataArray.CreateIterator(); ArrayIter; ++ArrayIter)
		{
			TSharedPtr<FLocMetadataValue>& Item = *ArrayIter;
			if (Item.IsValid())
			{
				ConditionTranslationMetadata(Item.ToSharedRef());
			}
		}
	}
	break;

	case ELocMetadataType::Object:
	{
		TSharedPtr< FLocMetadataObject > MetadataObject = MetadataValue->AsObject();
		for (auto ValueIter = MetadataObject->Values.CreateConstIterator(); ValueIter; ++ValueIter)
		{
			const FString Name = (*ValueIter).Key;
			TSharedPtr< FLocMetadataValue > Value = (*ValueIter).Value;
			if (Value.IsValid())
			{
				if (Value->GetType() == ELocMetadataType::String)
				{
					MetadataObject->SetStringField(Name, TEXT(""));
				}
				else
				{
					ConditionTranslationMetadata(Value.ToSharedRef());
				}
			}
		}
	}

	default:
		break;
	}
}

void ConditionTranslation(FLocItem& LocItem)
{
	// We clear out the translation text because this should only be entered by translators
	LocItem.Text = TEXT("");

	// The translation might have metadata so we want to clear all the values of any string metadata
	if (LocItem.MetadataObj.IsValid())
	{
		ConditionTranslationMetadata(MakeShareable(new FLocMetadataValueObject(LocItem.MetadataObj)));
	}
}

void ConditionSourceMetadata(TSharedRef<FLocMetadataValue> MetadataValue)
{
	switch (MetadataValue->GetType())
	{
	case ELocMetadataType::Object:
	{
		TSharedPtr< FLocMetadataObject > MetadataObject = MetadataValue->AsObject();

		TArray< FString > NamesToBeReplaced;

		// We go through all the source metadata entries and look for any that have names prefixed with a '*'.  If we
		//  find any, we will replace them with a String type that contains an empty value
		for (auto Iter = MetadataObject->Values.CreateConstIterator(); Iter; ++Iter)
		{
			const FString& Name = (*Iter).Key;
			TSharedPtr< FLocMetadataValue > Value = (*Iter).Value;

			if (Name.StartsWith(FLocMetadataObject::COMPARISON_MODIFIER_PREFIX))
			{
				NamesToBeReplaced.Add(Name);
			}
			else
			{
				ConditionSourceMetadata(Value.ToSharedRef());
			}
		}

		for (auto Iter = NamesToBeReplaced.CreateConstIterator(); Iter; ++Iter)
		{
			MetadataObject->RemoveField(*Iter);
			MetadataObject->SetStringField(*Iter, TEXT(""));
		}
	}

	default:
		break;
	}
}

void ConditionSource(FLocItem& LocItem)
{
	if (LocItem.MetadataObj.IsValid())
	{
		ConditionSourceMetadata(MakeShareable(new FLocMetadataValueObject(LocItem.MetadataObj)));
	}
}

/**
 *	UGenerateGatherArchiveCommandlet
 */
UGenerateGatherArchiveCommandlet::UGenerateGatherArchiveCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGenerateGatherArchiveCommandlet::Main( const FString& Params )
{
	UE_SCOPED_TIMER(TEXT("UGenerateGatherArchiveCommandlet::Main"), LogGenerateArchiveCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	//Set config file
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));
	FString GatherTextConfigPath;

	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No config specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	//Set config section
	ParamVal = ParamVals.Find(FString(TEXT("Section")));
	FString SectionName;

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No config section specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get manifest name.
	FString ManifestName;
	if( !GetStringFromConfig( *SectionName, TEXT("ManifestName"), ManifestName, GatherTextConfigPath ) )
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No manifest name specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get archive name.
	FString ArchiveName;
	if (!(GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, GatherTextConfigPath)))
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No archive name specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get native culture.
	FString NativeCulture;
	if (!GetStringFromConfig(*SectionName, TEXT("NativeCulture"), NativeCulture, GatherTextConfigPath))
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No native culture specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get cultures to generate.
	TArray<FString> CulturesToGenerate;
	GetStringArrayFromConfig(*SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, GatherTextConfigPath);
	
	if( CulturesToGenerate.Num() == 0 )
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No cultures specified for generation.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get destination path.
	FString DestinationPath;
	if( !GetPathFromConfig( *SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath ) )
	{
		UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "No destination path specified.",
			("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(DestinationPath, ManifestName, ArchiveName, NativeCulture, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::LoadOrCreate, &LoadError))
		{
			UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "Load error: {error}",
				("error", *LoadError.ToString()),
				("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	auto UpdateArchiveForCulture = 
		[&NativeCulture, &LocTextHelper](const FString& CultureName)
		{
			// Add any missing manifest entries to the archive for this culture
			const bool bIsNativeCulture = CultureName == NativeCulture;
			LocTextHelper.EnumerateSourceTexts(
				[&LocTextHelper, &CultureName, &NativeCulture, &bIsNativeCulture](TSharedRef<FManifestEntry> InManifestEntry) -> bool
				{
					for (const FManifestContext& Context : InManifestEntry->Contexts)
					{
						if (!Context.bIsOptional)
						{
							TSharedPtr<FArchiveEntry> ArchiveEntry = LocTextHelper.FindTranslation(CultureName, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj);
							if (ArchiveEntry.IsValid())
							{
								// We only update existing entries for the native culture, as this lets us preserve stale translations in foreign archives so we can decide later whether we actually want to use them
								if (bIsNativeCulture && !ArchiveEntry->Source.IsExactMatch(InManifestEntry->Source))
								{
									LocTextHelper.UpdateTranslation(CultureName, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, InManifestEntry->Source, InManifestEntry->Source);
								}
							}
							else
							{
								// Get the source we should use for the new entry
								FLocItem ArchiveSource = InManifestEntry->Source;
								if (!bIsNativeCulture)
								{
									TSharedPtr<FArchiveEntry> NativeArchiveEntry = LocTextHelper.FindTranslation(NativeCulture, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj);
									if (NativeArchiveEntry.IsValid() && !NativeArchiveEntry->Source.IsExactMatch(NativeArchiveEntry->Translation))
									{
										ArchiveSource = NativeArchiveEntry->Source;
									}
								}

								// We want to process the source before adding it to the archive
								ConditionSource(ArchiveSource);

								FLocItem ArchiveTranslation = ArchiveSource;
								if (!bIsNativeCulture)
								{
									// We want to process the translation before adding it to the archive
									// We skip this for native entries so that the source text is also added as the translation
									ConditionTranslation(ArchiveTranslation);
								}

								LocTextHelper.AddTranslation(CultureName, InManifestEntry->Namespace, Context.Key, Context.KeyMetadataObj, ArchiveSource, ArchiveTranslation, Context.bIsOptional);
							}
						}
					}

					return true; // continue enumeration
				}, true);

			// Trim any dead entries out of the archive
			LocTextHelper.TrimArchive(CultureName);

			// Save the new archive
			{
				FText SaveError;
				if (!LocTextHelper.SaveArchive(CultureName, &SaveError))
				{
					UE_LOGFMT(LogGenerateArchiveCommandlet, Error, "Save error: {error}",
						("error", *SaveError.ToString()),
						("id", GenerateGatherArchiveCommandlet::LocalizationLogIdentifier)
					);
					return false;
				}
			}

			return true;
		};

	// We need to make sure the native culture is processed first, as other cultures query that when adding their sources
	if (CulturesToGenerate.RemoveSingle(NativeCulture) > 0)
	{
		if (!UpdateArchiveForCulture(NativeCulture))
		{
			return -1;
		}
	}

	// We can update the remaining cultures in parallel
	if (CulturesToGenerate.Num() > 0)
	{
		TSharedPtr<ILocFileNotifies> LocFileNotifies = LocTextHelper.GetLocFileNotifies();

		if (LocFileNotifies)
		{
			LocFileNotifies->BeginParallelTasks();
		}

		std::atomic<bool> bSuccess = true;
		ParallelFor(TEXT("GenerateGatherArchiveCommandlet"), CulturesToGenerate.Num(), 1, [&CulturesToGenerate, &UpdateArchiveForCulture, &bSuccess](int32 Index)
		{
			const FString& CultureName = CulturesToGenerate[Index];
			if (!UpdateArchiveForCulture(CultureName))
			{
				bSuccess = false;
			}
		});

		if (LocFileNotifies)
		{
			LocFileNotifies->EndParallelTasks();
		}

		if (!bSuccess)
		{
			return -1;
		}
	}
	
	return 0;
}
