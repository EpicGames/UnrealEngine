// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GenerateGatherManifestCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "LocTextHelper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateGatherManifestCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateManifestCommandlet, Log, All);
namespace GenerateGatherManifestCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

namespace GenerateManifestHelper
{
	FString GetManifestFileExtension()
	{
		static const FString ManifestFileExtension= TEXT(".manifest");
		return ManifestFileExtension;
	}

	/** Returns true if the passed in file has a .manifest extension. Else returns false. */
	bool IsManifestFileExtensionValid(const FString& InManifestFilename)
	{
		return InManifestFilename.EndsWith(GenerateManifestHelper::GetManifestFileExtension());
	}
} // namespace GenerateManifestHelper

namespace GeneratePreviewManifestHelper
{
	FString GetPreviewManifestSuffix()
	{
		// Note: This suffix is also hardcoded in Localisation.automation.cs
		// If you decide to change the suffix for preview manifest files here, also update the automation file. 
		static const FString PreviewSuffix = TEXT("_Preview");
		return PreviewSuffix;
	}

	/**
	* Given a manifest filename, provide the preview version of the manifest filename.
	* @param InOriginalManifestFilename The original manifest filename (e.g MyManifest.manifest)
	* @return The preview version of the manifest filename (e.g MyManifest_Preview.manifest with current implementation)
	*/
	FString GetPreviewManifestFilename(const FString& InOriginalManifestFilename)
	{
		// This check should already be performed outside of this function. The passed in file is assumed to have the correct extension. 
		check(GenerateManifestHelper::IsManifestFileExtensionValid(InOriginalManifestFilename));
		FString PreviewFilename= InOriginalManifestFilename.LeftChop(GenerateManifestHelper::GetManifestFileExtension().Len());
		PreviewFilename.Append(GeneratePreviewManifestHelper::GetPreviewManifestSuffix());
		PreviewFilename.Append(GenerateManifestHelper::GetManifestFileExtension());
		return PreviewFilename;
	}
} // namespace GeneratePreviewManifestHelper

/**
 *	UGenerateGatherManifestCommandlet
 */
UGenerateGatherManifestCommandlet::UGenerateGatherManifestCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGenerateGatherManifestCommandlet::Main( const FString& Params )
{
	UE_SCOPED_TIMER(TEXT("UGenerateGatherManifestCommandlet::Main"), LogGenerateManifestCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine( *Params, Tokens, Switches, ParamVals );
	// we will have different behavior when this commandlet is running in preview
	// We want to generate temporary preview manifest files.
	// The temp manifest files will be loaded as manifest dependencies to avoid localization duplicate key warnings from surfacing due to stale manifest dependencies. 
	// The Localisation.automation.cs file is set up to delete these temp manifest files with a run of the Localize UAT command with the preview switch.
	// Note that these temp files will persist if running this commandlet via the command line. 
	const bool bRunningInPreview = Switches.Contains(TEXT("Preview"));
	if (bRunningInPreview)
	{
		UE_LOGF(LogGenerateManifestCommandlet, Log, "Commandlet is running in preview mode. Preview versions of manifests will be saved and loaded.");
	}

	// Set config file.
	const FString* ParamVal = ParamVals.Find( FString( TEXT("Config") ) );
	FString GatherTextConfigPath;

	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "No config specified.",
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Set config section.
	ParamVal = ParamVals.Find( FString( TEXT("Section") ) );
	FString SectionName;

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "No config section specified.",
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get destination path.
	FString DestinationPath;
	if( !GetPathFromConfig( *SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath ) )
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "No destination path specified.",
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get manifest name.
	FString ManifestName;
	if( !GetStringFromConfig( *SectionName, TEXT("ManifestName"), ManifestName, GatherTextConfigPath ) )
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "No manifest name specified.",
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	if (!GenerateManifestHelper::IsManifestFileExtensionValid(ManifestName))
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "Found manifest file {file} is malformed. All manifest files should have a {extension} extension.",
			("file", *ManifestName),
			("extension", *GenerateManifestHelper::GetManifestFileExtension()),
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	if (bRunningInPreview)
	{
		// we change the manifest filename to reflect the preview filename 
		ManifestName = GeneratePreviewManifestHelper::GetPreviewManifestFilename(ManifestName);
	}

	//Grab any manifest dependencies
	TArray<FString> ManifestDependenciesList;
	GetPathArrayFromConfig(*SectionName, TEXT("ManifestDependencies"), ManifestDependenciesList, GatherTextConfigPath);

	// Check that all the dependent manifest files have valid file extensions
	for (const FString& ManifestDependency : ManifestDependenciesList)
	{
		if (!GenerateManifestHelper::IsManifestFileExtensionValid(ManifestDependency))
		{
			UE_LOGFMT(LogGenerateManifestCommandlet, Error, "Found manifest dependency {file} is malformed. All manifest files should have a {extension} extension.",
				("file", *ManifestDependency),
				("extension", *GenerateManifestHelper::GetManifestFileExtension()),
				("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}
	if (bRunningInPreview)
	{
		// Overwrite all the manifest dependency filenames with their preview counterparts 
		for (FString& ManifestDependency : ManifestDependenciesList)
		{
			FString PreviewManifestDependency= GeneratePreviewManifestHelper::GetPreviewManifestFilename(ManifestDependency);
			if (!FPaths::FileExists(PreviewManifestDependency))
			{
				UE_LOGFMT(LogGenerateManifestCommandlet, Warning, "Preview manifest dependency {dependencyManifest} does not exist. Make sure to generate all preview manifest dependencies of this localization target before trying again.",
					("dependencyManifest", *PreviewManifestDependency),
					("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
				);
				continue;
			}
			// make sure the preview manifest file is newer than the regular manifest file
			// we could be dealing with a scenario where the preview manifest file is from a previous run and isn't up to date or failed to be deleted
			else if (IFileManager::Get().GetTimeStamp(*PreviewManifestDependency) < IFileManager::Get().GetTimeStamp(*ManifestDependency))
			{
				UE_LOGFMT(LogGenerateManifestCommandlet, Warning, "Preview manifest dependency {previewDependencyManifest} is older than the original manifest {originalManifest}. Preview manifest is out of date and should be regenerated as a dependency.",
					("previewDependencyManifest", *PreviewManifestDependency),
					("originalManifest", *ManifestDependency),
					("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
				);
				continue;
			}
			ManifestDependency = MoveTemp(PreviewManifestDependency);
		}
	}

	for (const FString& ManifestDependency : ManifestDependenciesList)
	{
		FText OutError;
		if (!GatherManifestHelper->AddDependency(ManifestDependency, &OutError))
		{
			UE_LOGFMT(LogGenerateManifestCommandlet, Error, "Failed to add manifest dependency {dependencyManifest}. Error: {error}",
				("dependencyManifest", *ManifestDependency),
				("error", *OutError.ToString()),
				("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	// Trim the manifest to remove any entries that came from a dependency
	GatherManifestHelper->TrimManifest();
	
	const FString ManifestPath = FPaths::ConvertRelativePathToFull(DestinationPath) / ManifestName;
	FText ManifestSaveError;
	if (!GatherManifestHelper->SaveManifest(ManifestPath, &ManifestSaveError))
	{
		UE_LOGFMT(LogGenerateManifestCommandlet, Error, "Save error: {error}",
			("error", *ManifestSaveError.ToString()),
			("id", GenerateGatherManifestCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	return 0;
}

bool UGenerateGatherManifestCommandlet::ShouldRunInPreview(const TArray<FString>& Switches, const TMap<FString, FString>& ParamVals) const
{
	// we need the commandlet to run to generate the preview manifests to avoid false positives from duplicate detection during preview runs 
	return true;
}

