// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GenerateTextLocalizationReportCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "CoreGlobals.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LocTextHelper.h"
#include "LocalizationSourceControlUtil.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateTextLocalizationReportCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateTextLocalizationReportCommandlet, Log, All);
namespace GenerateTextLocalizationReportCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

UGenerateTextLocalizationReportCommandlet::UGenerateTextLocalizationReportCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGenerateTextLocalizationReportCommandlet::Main(const FString& Params)
{
	UE_SCOPED_TIMER(TEXT("UGenerateTextLocalizationReportCommandlet::Main"), LogGenerateTextLocalizationReportCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config file.
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));

	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No config specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
	}

	// Set config section.
	ParamVal = ParamVals.Find(FString(TEXT("Section")));

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No config section specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Common settings
	FString SourcePath;
	FString DestinationPath;

	// Settings for generating/appending to word count report file
	bool bWordCountReport = false;

	// Settings for generating loc conflict report file
	bool bConflictReport = false;
	
	// Get source path.
	if( !( GetPathFromConfig( *SectionName, TEXT("SourcePath"), SourcePath, GatherTextConfigPath ) ) )
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No source path specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get destination path.
	if( !( GetPathFromConfig( *SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath ) ) )
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No destination path specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get the timestamp from the commandline, if not provided we will use the current time.
	const FString* TimeStampParamVal = ParamVals.Find(FString(TEXT("TimeStamp")));
	if ( TimeStampParamVal && !TimeStampParamVal->IsEmpty() )
	{
		CmdlineTimeStamp = *TimeStampParamVal;
	}

	GetBoolFromConfig( *SectionName, TEXT("bWordCountReport"), bWordCountReport, GatherTextConfigPath );
	GetBoolFromConfig( *SectionName, TEXT("bConflictReport"), bConflictReport, GatherTextConfigPath );

	if( bWordCountReport )
	{
		if( !ProcessWordCountReport( SourcePath, DestinationPath ) )
		{
			UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "Failed to generate word count report.",
				("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	if( bConflictReport )
	{
		if( !ProcessConflictReport( DestinationPath) )
		{
			UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "Failed to generate localization conflict report.",
				("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
			);
		}
	}

	return 0;
}

bool UGenerateTextLocalizationReportCommandlet::ProcessWordCountReport(const FString& SourcePath, const FString& DestinationPath)
{
	FDateTime Timestamp = FDateTime::Now();
	if (!CmdlineTimeStamp.IsEmpty())
	{
		FDateTime::Parse(*CmdlineTimeStamp, Timestamp);
	}

	// Get manifest name.
	FString ManifestName;
	if( !( GetStringFromConfig( *SectionName, TEXT("ManifestName"), ManifestName, GatherTextConfigPath ) ) )
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No manifest name specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get archive name.
	FString ArchiveName;
	if( !( GetStringFromConfig( *SectionName, TEXT("ArchiveName"), ArchiveName, GatherTextConfigPath ) ) )
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No archive name specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get report name.
	FString WordCountReportName;
	if( !( GetStringFromConfig( *SectionName, TEXT("WordCountReportName"), WordCountReportName, GatherTextConfigPath ) ) )
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No word count report name specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get cultures to generate.
	TArray<FString> CulturesToGenerate;
	GetStringArrayFromConfig( *SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, GatherTextConfigPath );

	for (const FString& CultureToGenerate : CulturesToGenerate)
	{
		if (!FInternationalization::Get().GetCulture(CultureToGenerate))
		{
			UE_LOGF(LogGenerateTextLocalizationReportCommandlet, Verbose, "Specified culture is not a valid runtime culture, but may be a valid base language: %ls", *CultureToGenerate);
		}
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, FString(), CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "{error}",
				("error", LoadError.ToString()),
				("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
			);
			return false;
		}
	}

	const FString ReportFilePath = (DestinationPath / WordCountReportName);

	FText ReportSaveError;
	if (!LocTextHelper.SaveWordCountReport(Timestamp, ReportFilePath, &ReportSaveError))
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "{error}",
			("error", ReportSaveError.ToString()),
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	return true;
}

bool UGenerateTextLocalizationReportCommandlet::ProcessConflictReport(const FString& DestinationPath)
{
	// Get report name.
	FString ConflictReportName;
	if (!GetStringFromConfig(*SectionName, TEXT("ConflictReportName"), ConflictReportName, GatherTextConfigPath))
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Error, "No conflict report name specified.",
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}
	
	EConflictReportFormat ConflictReportFormat = EConflictReportFormat::None;
	static const FString TxtExtension = TEXT(".txt");
	static const FString CSVExtension = TEXT(".csv");
	FString FileExtension = FPaths::GetExtension(ConflictReportName, true);
	if (FileExtension == TxtExtension)
	{
		ConflictReportFormat = EConflictReportFormat::Txt;
	}
	else if (FileExtension == CSVExtension)
	{
		ConflictReportFormat = EConflictReportFormat::CSV;
	}
	// This is an unsupported extension or an empty extension
	else
	{
		// We default to csv 
		ConflictReportFormat = EConflictReportFormat::CSV;
		// We found a file extension somewhere in the specified name. 
		if (!FileExtension.IsEmpty())
		{
			UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "The conflict report filename {file} has an unsupported extension. Only .txt and .csv is supported at this time.",
				("file", ConflictReportName),
				("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
			);
		}
		else
		{
			UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "The conflict report filename {file} has no extension. Defaulting the report to be generated as a .csv file.",
				("file", ConflictReportName),
				("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
			);
		}
		ConflictReportName = FPaths::SetExtension(ConflictReportName, CSVExtension);
	}
	const FString ReportFilePath = (DestinationPath / ConflictReportName);

	FText ReportSaveError;
	if (!GatherManifestHelper->SaveConflictReport(ReportFilePath, ConflictReportFormat , &ReportSaveError))
	{
		UE_LOGFMT(LogGenerateTextLocalizationReportCommandlet, Warning, "{error}",
			("error", ReportSaveError.ToString()),
			("id", GenerateTextLocalizationReportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	return true;
}
