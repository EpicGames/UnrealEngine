// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/GenerateTextLocalizationResourceCommandlet.h"

#include "Async/ParallelFor.h"
#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Internationalization/CulturePointer.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextKey.h"
#include "Internationalization/TextLocalizationResource.h"
#include "LocalizationFileUtil.h"
#include "LocTextHelper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "TextLocalizationResourceGenerator.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateTextLocalizationResourceCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogGenerateTextLocalizationResourceCommandlet, Log, All);

UGenerateTextLocalizationResourceCommandlet::UGenerateTextLocalizationResourceCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UGenerateTextLocalizationResourceCommandlet::Main(const FString& Params)
{
	UE_SCOPED_TIMER(TEXT("UGenerateTextLocalizationResourceCommandlet::Main"), LogGenerateTextLocalizationResourceCommandlet, Display);
	// Parse command line - we're interested in the param vals
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config file.
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));
	FString GatherTextConfigPath;

	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No config specified.");
		return -1;
	}

	// Set config section.
	ParamVal = ParamVals.Find(FString(TEXT("Section")));
	FString SectionName;

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No config section specified.");
		return -1;
	}

	// Get source path.
	FString SourcePath;
	if( !( GetPathFromConfig( *SectionName, TEXT("SourcePath"), SourcePath, GatherTextConfigPath ) ) )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No source path specified.");
		return -1;
	}

	// Get manifest name.
	FString ManifestName;
	if( !( GetStringFromConfig( *SectionName, TEXT("ManifestName"), ManifestName, GatherTextConfigPath ) ) )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No manifest name specified.");
		return -1;
	}

	// Get archive name.
	FString ArchiveName;
	if (!GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, GatherTextConfigPath))
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No archive name specified.");
		return -1;
	}

	// Get cultures to generate.
	FString NativeCultureName;
	if( !( GetStringFromConfig( *SectionName, TEXT("NativeCulture"), NativeCultureName, GatherTextConfigPath ) ) )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No native culture specified.");
		return -1;
	}

	// Get cultures to generate.
	TArray<FString> CulturesToGenerate;
	GetStringArrayFromConfig( *SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, GatherTextConfigPath );

	if( CulturesToGenerate.Num() == 0 )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No cultures specified for generation.");
		return -1;
	}

	for (const FString& CultureToGenerate : CulturesToGenerate)
	{
		if (!FInternationalization::Get().GetCulture(CultureToGenerate))
		{
			UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Verbose, "Specified culture is not a valid runtime culture, but may be a valid base language: %ls", *CultureToGenerate);
		}
	}

	// Get destination path.
	FString DestinationPath;
	if( !( GetPathFromConfig( *SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath ) ) )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No destination path specified.");
		return -1;
	}

	// Get resource name.
	FString ResourceName;
	if( !( GetStringFromConfig( *SectionName, TEXT("ResourceName"), ResourceName, GatherTextConfigPath ) ) )
	{
		UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "No resource name specified.");
		return -1;
	}

	EGenerateLocResFlags GenerateFlags = EGenerateLocResFlags::None;

	// Get whether to skip the source check.
	{
		bool bSkipSourceCheck = false;
		GetBoolFromConfig(*SectionName, TEXT("bSkipSourceCheck"), bSkipSourceCheck, GatherTextConfigPath);
		GenerateFlags |= (bSkipSourceCheck ? EGenerateLocResFlags::AllowStaleTranslations : EGenerateLocResFlags::None);
	}

	// Get whether to validate format patterns.
	{
		bool bValidateFormatPatterns = false;
		GetBoolFromConfig(*SectionName, TEXT("bValidateFormatPatterns"), bValidateFormatPatterns, GatherTextConfigPath);
		GenerateFlags |= (bValidateFormatPatterns ? EGenerateLocResFlags::ValidateFormatPatterns : EGenerateLocResFlags::None);
	}

	// Get whether to validate whitespace.
	{
		bool bValidateSafeWhitespace = false;
		GetBoolFromConfig(*SectionName, TEXT("bValidateSafeWhitespace"), bValidateSafeWhitespace, GatherTextConfigPath);
		GenerateFlags |= (bValidateSafeWhitespace ? EGenerateLocResFlags::ValidateSafeWhitespace : EGenerateLocResFlags::None);
	}

	// Get whether to validate Rich Text tags.
	{
		bool bValidateRichTextTags = false;
		GetBoolFromConfig(*SectionName, TEXT("bValidateRichTextTags"), bValidateRichTextTags, GatherTextConfigPath);
		GenerateFlags |= (bValidateRichTextTags ? EGenerateLocResFlags::ValidateRichTextTags : EGenerateLocResFlags::None);
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, NativeCultureName, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Warning, "%ls", *LoadError.ToString());
			return 0;
		}
	}

	// Generate the LocMeta file for all cultures
	{
		const FString TextLocalizationMetaDataResourcePath = DestinationPath / FPaths::GetBaseFilename(ResourceName) + TEXT(".locmeta");

		const bool bLocMetaFileSaved = UE::Localization::FileUtil::WriteFileIfModified(TextLocalizationMetaDataResourcePath,
			[&LocTextHelper, &ResourceName](const FString& Filename)
			{
				FTextLocalizationMetaDataResource LocMeta;
				return FTextLocalizationResourceGenerator::GenerateLocMeta(LocTextHelper, ResourceName, LocMeta) && LocMeta.SaveToFile(Filename);
			}, &UE::Localization::FileUtil::BinaryHash, SourceControlInfo);

		if (bLocMetaFileSaved)
		{
			UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Log, "Generated file %ls", *TextLocalizationMetaDataResourcePath);
		}
		else
		{
			UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "Could not write file %ls", *TextLocalizationMetaDataResourcePath);
			return -1;
		}
	}

	TSharedPtr<ILocFileNotifies> LocFileNotifies = LocTextHelper.GetLocFileNotifies();

	if (LocFileNotifies)
	{
		LocFileNotifies->BeginParallelTasks();
	}

	// Generate the LocRes file for each culture
	std::atomic<bool> bSuccess = true;
	ParallelFor(TEXT("GenerateTextLocalizationResourceCommandlet"), CulturesToGenerate.Num(), 1, [&](int32 Index)
	{
		const FString& CultureName = CulturesToGenerate[Index];

		auto GenerateSingleLocRes = [this, &DestinationPath, &CultureName, &ResourceName](const FTextLocalizationResource& InLocRes, const FName InPlatformName) -> bool
		{
			FString TextLocalizationResourcePath;
			if (InPlatformName.IsNone())
			{
				TextLocalizationResourcePath = DestinationPath / CultureName / ResourceName;
			}
			else
			{
				TextLocalizationResourcePath = DestinationPath / FPaths::GetPlatformLocalizationFolderName() / InPlatformName.ToString() / CultureName / ResourceName;
			}

			const bool bLocResFileSaved = UE::Localization::FileUtil::WriteFileIfModified(TextLocalizationResourcePath,
				[&InLocRes](const FString& Filename)
				{
					return InLocRes.SaveToFile(Filename);
				}, &UE::Localization::FileUtil::BinaryHash, SourceControlInfo);

			if (bLocResFileSaved)
			{
				UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Log, "Generated file %ls", *TextLocalizationResourcePath);
			}
			else
			{
				UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "Could not write file %ls", *TextLocalizationResourcePath);
				return false;
			}

			return true;
		};

		FTextLocalizationResource PlatformAgnosticLocRes;
		TMap<FName, TSharedRef<FTextLocalizationResource>> PerPlatformLocRes;
		const FTextKey LocResId = DestinationPath / CultureName / ResourceName;
		if (!FTextLocalizationResourceGenerator::GenerateLocRes(LocTextHelper, CultureName, GenerateFlags, LocResId, PlatformAgnosticLocRes, PerPlatformLocRes))
		{
			UE_LOGF(LogGenerateTextLocalizationResourceCommandlet, Error, "Failed to generate LocRes %ls", *LocResId.ToString());
			bSuccess = false;
			return;
		}
	
		if (!GenerateSingleLocRes(PlatformAgnosticLocRes, FName()))
		{
			bSuccess = false;
		}
		for (const auto& PerPlatformLocResPair : PerPlatformLocRes)
		{
			if (!GenerateSingleLocRes(*PerPlatformLocResPair.Value, PerPlatformLocResPair.Key))
			{
				bSuccess = false;
			}
		}
	});

	if (LocFileNotifies)
	{
		LocFileNotifies->EndParallelTasks();
	}

	if (!bSuccess)
	{
		return -1;
	}

	return 0;
}
