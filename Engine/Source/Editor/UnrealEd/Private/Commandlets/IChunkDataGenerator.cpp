// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/IChunkDataGenerator.h"

TArray<TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>> IChunkDataGenerator::GeneratorsDataGeneratorFactories;

void IChunkDataGenerator::AddChunkDataGeneratorFactory(const TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>& InChunkGeneratorFactory)
{
	GeneratorsDataGeneratorFactories.Add(InChunkGeneratorFactory);
}

const TArray<TFunction<TSharedRef<IChunkDataGenerator>(const UE::Cook::ICookInfo&)>>& IChunkDataGenerator::GetChunkDataGeneratorFactories()
{
	return GeneratorsDataGeneratorFactories;
}