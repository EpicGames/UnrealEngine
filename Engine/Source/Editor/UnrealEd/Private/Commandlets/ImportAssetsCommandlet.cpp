// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ImportAssetsCommandlet.h"
#include "AutomatedAssetImportData.h"
#include "Modules/ModuleManager.h"
#include "Factories/Factory.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "JsonObjectConverter.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Factories/ImportSettings.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Misc/FeedbackContext.h"
#include "HAL/PlatformFileManager.h"
#include "GameFramework/WorldSettings.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportAssetsCommandlet)

void UImportAssetsCommandlet::PrintUsage()
{
	UE_LOGF(LogAutomatedImport, Display, "LogAutomatedImport Usage: LogAutomatedImport {arglist}");
	UE_LOGF(LogAutomatedImport, Display, "Arglist:");

	UE_LOGF(LogAutomatedImport, Display, "-help or -?");
	UE_LOGF(LogAutomatedImport, Display, "\tDisplays this help");

	UE_LOGF(LogAutomatedImport, Display, "-source=\"path\"");
	UE_LOGF(LogAutomatedImport, Display, "\tThe source file to import.  This must be specified when importing a single asset\n[IGNORED when using -importparams]");

	UE_LOGF(LogAutomatedImport, Display, "-dest=\"path\"");
	UE_LOGF(LogAutomatedImport, Display, "\tThe destination path in the project's content directory to import to.\nThis must be specified when importing a single asset\n[IGNORED when using -importparams]");

	UE_LOGF(LogAutomatedImport, Display, "-factory={factory class name}");
	UE_LOGF(LogAutomatedImport, Display, "\tForces the asset to be opened with a specific UFactory class type.  If not specified import type will be auto detected.\n[IGNORED when using -importparams]");

	UE_LOGF(LogAutomatedImport, Display, "-importsettings=\"path to import settings json file\"");
	UE_LOGF(LogAutomatedImport, Display, "\tPath to a json file that has asset import parameters when importing multiple files. If this argument is used all other import arguments are ignored as they are specified in the json file");

	UE_LOGF(LogAutomatedImport, Display, "-replaceexisting");
	UE_LOGF(LogAutomatedImport, Display, "\tWhether or not to replace existing assets when importing");

	UE_LOGF(LogAutomatedImport, Display, "-nosourcecontrol");
	UE_LOGF(LogAutomatedImport, Display, "\tDisables revision control.  Prevents checking out, adding files, and submitting files");

	UE_LOGF(LogAutomatedImport, Display, "-submitdesc");
	UE_LOGF(LogAutomatedImport, Display, "\tSubmit description/comment to use checking in to revision control.  If this is empty no files will be submitted");

	UE_LOGF(LogAutomatedImport, Display, "-skipreadonly");
	UE_LOGF(LogAutomatedImport, Display, "\tIf an asset cannot be saved because it is read only, the commandlet will not clear the read only flag and will not save the file");
}

UImportAssetsCommandlet::UImportAssetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UImportAssetsCommandlet::ParseParams(const FString& InParams)
{
	TArray<FString> Tokens;
	TArray<FString> Params;
	TMap<FString, FString> ParamVals;

	ParseCommandLine(*InParams, Tokens, Params, ParamVals);

	if( Params.Contains(TEXT("?")) || Params.Contains(TEXT("help") ) )
	{
		bShowHelp = true;
	}

	bAllowSourceControl = !Params.Contains(TEXT("nosourcecontrol"));

	GlobalImportData = NewObject<UAutomatedAssetImportData>(this);

	GlobalImportData->bSkipReadOnly = Params.Contains(TEXT("skipreadonly"));

	FString SourcePathParam = ParamVals.FindRef(TEXT("source"));
	if(!SourcePathParam.IsEmpty())
	{
		GlobalImportData->Filenames.Add(SourcePathParam);
	}
	
	GlobalImportData->DestinationPath = ParamVals.FindRef(TEXT("dest"));

	GlobalImportData->FactoryName = ParamVals.FindRef(TEXT("factoryname"));

	GlobalImportData->bReplaceExisting = Params.Contains(TEXT("replaceexisting"));

	GlobalImportData->LevelToLoad = ParamVals.FindRef(TEXT("level"));

	if (!GlobalImportData->LevelToLoad.IsEmpty())
	{
		FText FailReason;
		if (!FPackageName::IsValidLongPackageName(GlobalImportData->LevelToLoad, false, &FailReason))
		{
			UE_LOGF(LogAutomatedImport, Error, "Invalid level specified: %ls", *FailReason.ToString());
		}
	}

	ImportSettingsPath = ParamVals.FindRef(TEXT("importsettings"));

	GlobalImportData->Initialize(nullptr);

	if(ImportSettingsPath.IsEmpty() && (GlobalImportData->Filenames.Num() == 0 || GlobalImportData->DestinationPath.IsEmpty()))
	{
		UE_LOGF(LogAutomatedImport, Error, "Invalid Arguments.  Missing, Source (-source), Destination (-dest), or Import settings file (-importsettings)");
	}
	const bool bEnoughParams = (ParamVals.Num() > 1) || !ImportSettingsPath.IsEmpty();

	return bEnoughParams;
}

bool UImportAssetsCommandlet::ParseImportSettings(const FString& InImportSettingsFile)
{
	bool bInvalidParse = false;
	bool bSuccess = false;
	FString JsonString;
	if(FFileHelper::LoadFileToString(JsonString, *InImportSettingsFile))
	{
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
		TSharedPtr<FJsonObject> RootObject;
		if(FJsonSerializer::Deserialize(JsonReader, RootObject) && RootObject.IsValid())
		{
			const TArray< TSharedPtr<FJsonValue> > ImportGroupsJsonArray = RootObject->GetArrayField(TEXT("ImportGroups"));
			for(const TSharedPtr<FJsonValue>& ImportGroupsJson : ImportGroupsJsonArray)
			{
				const TSharedPtr<FJsonObject> ImportGroupsJsonObject = ImportGroupsJson->AsObject();
				if(ImportGroupsJsonObject.IsValid())
				{
					// All import data is based off of the global data defaults
					UAutomatedAssetImportData* Data = DuplicateObject<UAutomatedAssetImportData>(GlobalImportData, this);
					
					// Parse data from the json object
					if(FJsonObjectConverter::JsonObjectToUStruct(ImportGroupsJsonObject.ToSharedRef(), UAutomatedAssetImportData::StaticClass(), Data, 0, 0 ))
					{
						Data->Initialize(ImportGroupsJsonObject);
						if(Data->IsValid())
						{
							ImportDataList.Add(Data);
						}
					}
					else
					{
						bInvalidParse = true;
					}
					
				}
				else
				{
					bInvalidParse = true;
				}
			}
		}
		else
		{
			UE_LOGF(LogAutomatedImport, Error, "Json settings file was found but was invalid: %ls", *JsonReader->GetErrorMessage());
		}
	}
	else
	{
		UE_LOGF(LogAutomatedImport, Error, "Import settings file %ls could not be found", *InImportSettingsFile);
	}

	return bSuccess;
}

static bool SavePackage(UPackage* Package, const FString& PackageFilename)
{
	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	SaveArgs.Error = GWarn;
	return GEditor->SavePackage(Package, nullptr, *PackageFilename, SaveArgs);
}

bool UImportAssetsCommandlet::ImportAndSave(const TArray<UAutomatedAssetImportData*>& AssetImportList)
{
	bool bImportAndSaveSucceeded = true;

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	for(const UAutomatedAssetImportData* ImportData : AssetImportList)
	{
		UE_LOGF(LogAutomatedImport, Log, "Importing group %ls", *ImportData->GetDisplayName() );

		UFactory* Factory = ImportData->Factory;
		const TSharedPtr<FJsonObject>* ImportSettingsJsonObject = nullptr;
		if(ImportData->ImportGroupJsonData.IsValid())
		{
			ImportData->ImportGroupJsonData->TryGetObjectField(TEXT("ImportSettings"), ImportSettingsJsonObject);
		}

		if(Factory != nullptr && ImportSettingsJsonObject)
		{
			IImportSettingsParser* ImportSettings = Factory->GetImportSettingsParser();
			if(ImportSettings)
			{
				ImportSettings->ParseFromJson(ImportSettingsJsonObject->ToSharedRef());
			}
		}
		else if(Factory == nullptr && ImportSettingsJsonObject)
		{
			UE_LOGF(LogAutomatedImport, Warning, "A vaild factory name must be specfied in order to specify settings");
		}

		// Load a level if specified
		bImportAndSaveSucceeded = LoadLevel(ImportData->LevelToLoad);

		// Clear dirty packages that were created as a result of loading the level.  We do not want to save these
		ClearDirtyPackages();

		TArray<UObject*> ImportedAssets = AssetToolsModule.Get().ImportAssetsAutomated(ImportData);
		if(ImportedAssets.Num() > 0 && bImportAndSaveSucceeded)
		{
			TArray<UPackage*> DirtyPackages;

			TArray<FSourceControlStateRef> PackageStates;

			FEditorFileUtils::GetDirtyContentPackages(DirtyPackages);
			FEditorFileUtils::GetDirtyWorldPackages(DirtyPackages);

			bool bUseSourceControl = bHasSourceControl && SourceControlProvider.IsAvailable();
			if(bUseSourceControl)
			{
				SourceControlProvider.GetState(DirtyPackages, PackageStates, EStateCacheUsage::ForceUpdate);
			}

			for(int32 PackageIndex = 0; PackageIndex < DirtyPackages.Num(); ++PackageIndex)
			{
				UPackage* PackageToSave = DirtyPackages[PackageIndex];

				FString PackageFilename = SourceControlHelpers::PackageFilename(PackageToSave);

				bool bShouldAttemptToSave = false;
				bool bShouldAttemptToAdd = false;
				if(bUseSourceControl)
				{
					FSourceControlStateRef PackageSCState = PackageStates[PackageIndex];

					bool bPackageCanBeCheckedOut = false;
					if(PackageSCState->IsCheckedOutOther())
					{
						// Cannot checkout, file is already checked out
						UE_LOGF(LogAutomatedImport, Error, "%ls is already checked out by someone else, can not submit!", *PackageFilename);
						bImportAndSaveSucceeded = false;
					}
					else if(!PackageSCState->IsCurrent())
					{
						// Cannot checkout, file is not at head revision
						UE_LOGF(LogAutomatedImport, Error, "%ls is not at the head revision and cannot be checked out", *PackageFilename);
						bImportAndSaveSucceeded = false;
					}
					else if(PackageSCState->CanCheckout())
					{
						const bool bWasCheckedOut = SourceControlHelpers::CheckOutOrAddFile(PackageFilename);
						bShouldAttemptToSave = bWasCheckedOut;
						if(!bWasCheckedOut)
						{
							UE_LOGF(LogAutomatedImport, Error, "%ls could not be checked out", *PackageFilename);
							bImportAndSaveSucceeded = false;
						}
					}
					else
					{
						// package was not checked out by another user and is at the current head revision and could not be checked out
						// this means it should be added after save because it doesn't exist
						bShouldAttemptToSave = true;
						bShouldAttemptToAdd = true;
					}
				}
				else
				{
					bool bIsReadOnly = IFileManager::Get().IsReadOnly(*PackageFilename);
					if(bIsReadOnly && ImportData->bSkipReadOnly)
					{
						bShouldAttemptToSave = false;
						if(bIsReadOnly)
						{
							UE_LOGF(LogAutomatedImport, Error, "%ls is read only and -skipreadonly was specified.  Will not save", *PackageFilename);
							bImportAndSaveSucceeded = false;
						}
					}
					else if(bIsReadOnly)
					{
						bShouldAttemptToSave = FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFilename, false);
						if(!bShouldAttemptToSave)
						{
							UE_LOGF(LogAutomatedImport, Error, "%ls is read only and could not be made writable.  Will not save", *PackageFilename);
							bImportAndSaveSucceeded = false;
						}
					}
					else
					{
						bShouldAttemptToSave = true;
					}
				}

				if(bShouldAttemptToSave)
				{
					SavePackage(PackageToSave, PackageFilename);

					if(bShouldAttemptToAdd)
					{
						const bool bWasAdded = SourceControlHelpers::MarkFileForAdd(PackageFilename);
						if(!bWasAdded)
						{
							UE_LOGF(LogAutomatedImport, Error, "%ls could not be added to revision control", *PackageFilename);
							bImportAndSaveSucceeded = false;
						}
					}
				}
			}
		}
		else
		{
			bImportAndSaveSucceeded = false;
			UE_LOGF(LogAutomatedImport, Error, "Failed to import all assets in group %ls", *ImportData->GetDisplayName());
		}
	}

	return bImportAndSaveSucceeded;
}

bool UImportAssetsCommandlet::LoadLevel(const FString& LevelToLoad)
{
	bool bResult = false;

	if (!LevelToLoad.IsEmpty())
	{
		UE_LOGF(LogAutomatedImport, Log, "Loading Map %ls", *LevelToLoad);

		FString Filename;
		if (FPackageName::TryConvertLongPackageNameToFilename(LevelToLoad, Filename))
		{
			UPackage* Package = LoadPackage(NULL, *Filename, 0);

			UWorld* World = UWorld::FindWorldInPackage(Package);
			if (World)
			{
				// Clean up any previous world.  The world should have already been saved
				UWorld* ExistingWorld = GEditor->GetEditorWorldContext().World();

				GEngine->DestroyWorldContext(ExistingWorld);
				ExistingWorld->DestroyWorld(true, World);

				GWorld = World;

				World->WorldType = EWorldType::Editor;

				FWorldContext& WorldContext = GEngine->CreateNewWorldContext(World->WorldType);
				WorldContext.SetCurrentWorld(World);

				// add the world to the root set so that the garbage collection to delete replaced actors doesn't garbage collect the whole world
				World->AddToRoot();

				// initialize the levels in the world
				World->InitWorld(UWorld::InitializationValues().AllowAudioPlayback(false));
				World->GetWorldSettings()->PostEditChange();
				World->UpdateWorldComponents(true, false);

				bResult = true;
			}
		}
	}
	else
	{
		// a map was not specified, ignore
		bResult = true;
	}

	if (!bResult)
	{
		UE_LOGF(LogAutomatedImport, Error, "Could not find or load level %ls", *LevelToLoad);
	}

	return bResult;

}

void UImportAssetsCommandlet::ClearDirtyPackages()
{
	TArray<UPackage*> DirtyPackages;
	FEditorFileUtils::GetDirtyContentPackages(DirtyPackages);
	FEditorFileUtils::GetDirtyWorldPackages(DirtyPackages);

	for(UPackage* Package : DirtyPackages)
	{
		Package->SetDirtyFlag(false);
	}
}

int32 UImportAssetsCommandlet::Main(const FString& InParams)
{
	bool bEnoughParams = ParseParams(InParams);

	int32 Result = 0;

	if(!bEnoughParams || bShowHelp)
	{
		PrintUsage();
	}
	else
	{
		// Hack:  A huge amount of packages are marked dirty on startup.  This is normally prevented in editor but commandlets have special powers.  
		// We only want to save assets that were created or modified at import time so clear all existing ones now
		ClearDirtyPackages();

		if(bAllowSourceControl)
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			SourceControlProvider.Init();

			bHasSourceControl = SourceControlProvider.IsEnabled();
			if(!bHasSourceControl)
			{
				UE_LOGF(LogAutomatedImport, Error, "Could not connect to revision control!")
			}
		}
		else
		{
			bHasSourceControl = false;
		}


		if(!ImportSettingsPath.IsEmpty())
		{
			// Use settings file for importing assets
			ParseImportSettings(ImportSettingsPath);
		}
		else if(GlobalImportData->IsValid())
		{
			// Use single import path
			ImportDataList.Add(GlobalImportData);
		}

		if(!ImportAndSave(ImportDataList))
		{
			UE_LOGF(LogAutomatedImport, Error, "Could not import all groups");
		}
		else
		{
			Result = 0;
		}
		
	}
	return Result;
}
