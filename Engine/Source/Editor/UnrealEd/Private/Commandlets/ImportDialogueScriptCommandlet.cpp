// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ImportDialogueScriptCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "Commandlets/ExportDialogueScriptCommandlet.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/Text.h"
#include "LocTextHelper.h"
#include "LocalizationSourceControlUtil.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Serialization/Csv/CsvParser.h"
#include "Sound/DialogueWave.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportDialogueScriptCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogImportDialogueScriptCommandlet, Log, All);

UImportDialogueScriptCommandlet::UImportDialogueScriptCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UImportDialogueScriptCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config path
	FString ConfigPath;
	{
		const FString* ConfigPathParamVal = ParamVals.Find(FString(TEXT("Config")));
		if (!ConfigPathParamVal)
		{
			UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No config specified.");
			return -1;
		}
		ConfigPath = *ConfigPathParamVal;
	}

	// Set config section
	FString SectionName;
	{
		const FString* SectionNameParamVal = ParamVals.Find(FString(TEXT("Section")));
		if (!SectionNameParamVal)
		{
			UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No config section specified.");
			return -1;
		}
		SectionName = *SectionNameParamVal;
	}

	// Source path to the root folder that dialogue script CSV files live in
	FString SourcePath;
	if (!GetPathFromConfig(*SectionName, TEXT("SourcePath"), SourcePath, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No source path specified.");
		return -1;
	}

	// Destination path to the root folder that manifest/archive files live in
	FString DestinationPath;
	if (!GetPathFromConfig(*SectionName, TEXT("DestinationPath"), DestinationPath, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No destination path specified.");
		return -1;
	}

	// Get culture directory setting, default to true if not specified (used to allow picking of export directory with windows file dialog from Translation Editor)
	bool bUseCultureDirectory = true;
	if (!GetBoolFromConfig(*SectionName, TEXT("bUseCultureDirectory"), bUseCultureDirectory, ConfigPath))
	{
		bUseCultureDirectory = true;
	}

	// Get the native culture
	FString NativeCulture;
	if (!GetStringFromConfig(*SectionName, TEXT("NativeCulture"), NativeCulture, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No native culture specified.");
		return -1;
	}

	// Get cultures to generate
	TArray<FString> CulturesToGenerate;
	if (GetStringArrayFromConfig(*SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, ConfigPath) == 0)
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No cultures specified for import.");
		return -1;
	}

	// Get the manifest name
	FString ManifestName;
	if (!GetStringFromConfig(*SectionName, TEXT("ManifestName"), ManifestName, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No manifest name specified.");
		return -1;
	}

	// Get the archive name
	FString ArchiveName;
	if (!GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No archive name specified.");
		return -1;
	}

	// Get the dialogue script name
	FString DialogueScriptName;
	if (!GetStringFromConfig(*SectionName, TEXT("DialogueScriptName"), DialogueScriptName, ConfigPath))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "No dialogue script name specified.");
		return -1;
	}

	// We may only have a single culture if using this setting
	if (!bUseCultureDirectory && CulturesToGenerate.Num() > 1)
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "bUseCultureDirectory may only be used with a single culture.");
		return -1;
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(DestinationPath, ManifestName, ArchiveName, NativeCulture, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGF(LogImportDialogueScriptCommandlet, Error, "%ls", *LoadError.ToString());
			return -1;
		}
	}

	// Import the native culture first as this may trigger additional translations in foreign archives
	{
		const FString CultureSourcePath = SourcePath / (bUseCultureDirectory ? NativeCulture : TEXT(""));
		const FString CultureDestinationPath = DestinationPath / NativeCulture;
		if (!ImportDialogueScriptForCulture(LocTextHelper, CultureSourcePath / DialogueScriptName, NativeCulture, true))
		{
			return -1;
		}
	}

	// Import any remaining cultures
	for (const FString& CultureName : CulturesToGenerate)
	{
		// Skip the native culture as we already processed it above
		if (CultureName == NativeCulture)
		{
			continue;
		}

		const FString CultureSourcePath = SourcePath / (bUseCultureDirectory ? CultureName : TEXT(""));
		const FString CultureDestinationPath = DestinationPath / CultureName;
		if (!ImportDialogueScriptForCulture(LocTextHelper, CultureSourcePath / DialogueScriptName, CultureName, false))
		{
			return -1;
		}
	}

	return 0;
}

bool UImportDialogueScriptCommandlet::ImportDialogueScriptForCulture(FLocTextHelper& InLocTextHelper, const FString& InDialogueScriptFileName, const FString& InCultureName, const bool bIsNativeCulture)
{
	// Load dialogue script file contents to string
	FString DialogScriptFileContents;
	if (!FFileHelper::LoadFileToString(DialogScriptFileContents, *InDialogueScriptFileName))
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Failed to load contents of dialog script file '%ls' for culture '%ls'.", *InDialogueScriptFileName, *InCultureName);
		return false;
	}

	// Parse dialogue script file contents
	const FCsvParser DialogScriptFileParser(DialogScriptFileContents);
	const FCsvParser::FRows& Rows = DialogScriptFileParser.GetRows();

	// Validate dialogue script row count
	if (Rows.Num() <= 0)
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Dialogue script file has insufficient rows for culture '%ls'. Expected at least 1 row, got %d.", *InCultureName, Rows.Num());
		return false;
	}

	const FProperty* SpokenDialogueProperty = FDialogueScriptEntry::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDialogueScriptEntry, SpokenDialogue));
	const FProperty* LocalizationKeysProperty = FDialogueScriptEntry::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDialogueScriptEntry, LocalizationKeys));

	// We need the SpokenDialogue and LocalizationKeys properties in order to perform the import, so find their respective columns in the CSV data
	int32 SpokenDialogueColumnIndex = INDEX_NONE;
	int32 LocalizationKeysColumnIndex = INDEX_NONE;
	{
		const FString SpokenDialogueColumnName = SpokenDialogueProperty->GetName();
		const FString LocalizationKeysColumnName = LocalizationKeysProperty->GetName();

		const auto& HeaderRowData = Rows[0];
		for (int32 ColumnIndex = 0; ColumnIndex < HeaderRowData.Num(); ++ColumnIndex)
		{
			const TCHAR* const CellData = HeaderRowData[ColumnIndex];
			if (FCString::Stricmp(CellData, *SpokenDialogueColumnName) == 0)
			{
				SpokenDialogueColumnIndex = ColumnIndex;
			}
			else if (FCString::Stricmp(CellData, *LocalizationKeysColumnName) == 0)
			{
				LocalizationKeysColumnIndex = ColumnIndex;
			}

			if (SpokenDialogueColumnIndex != INDEX_NONE && LocalizationKeysColumnIndex != INDEX_NONE)
			{
				break;
			}
		}
	}

	if (SpokenDialogueColumnIndex == INDEX_NONE)
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Dialogue script file is missing the required column '%ls' for culture '%ls'.", *SpokenDialogueProperty->GetName(), *InCultureName);
		return false;
	}

	if (LocalizationKeysColumnIndex == INDEX_NONE)
	{
		UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Dialogue script file is missing the required column '%ls' for culture '%ls'.", *LocalizationKeysProperty->GetName(), *InCultureName);
		return false;
	}

	bool bHasUpdatedArchive = false;

	// Parse each row of the CSV data
	for (int32 RowIndex = 1; RowIndex < Rows.Num(); ++RowIndex)
	{
		const auto& RowData = Rows[RowIndex];

		FDialogueScriptEntry ParsedScriptEntry;

		// Parse the SpokenDialogue data
		{
			const TCHAR* const CellData = RowData[SpokenDialogueColumnIndex];
			if (SpokenDialogueProperty->ImportText_InContainer(CellData, &ParsedScriptEntry, nullptr, PPF_None) == nullptr)
			{
				UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Failed to parse the required column '%ls' for row '%d' for culture '%ls'.", *SpokenDialogueProperty->GetName(), RowIndex, *InCultureName);
				continue;
			}
		}

		// Parse the LocalizationKeys data
		{
			const TCHAR* const CellData = RowData[LocalizationKeysColumnIndex];
			if (LocalizationKeysProperty->ImportText_InContainer(CellData, &ParsedScriptEntry, nullptr, PPF_None) == nullptr)
			{
				UE_LOGF(LogImportDialogueScriptCommandlet, Error, "Failed to parse the required column '%ls' for row '%d' for culture '%ls'.", *LocalizationKeysProperty->GetName(), RowIndex, *InCultureName);
				continue;
			}
		}

		for (const FString& ContextLocalizationKey : ParsedScriptEntry.LocalizationKeys)
		{
			// Find the manifest entry so that we can find the corresponding archive entry
			TSharedPtr<FManifestEntry> ContextManifestEntry = InLocTextHelper.FindSourceText(FDialogueConstants::DialogueNamespace, ContextLocalizationKey);
			if (!ContextManifestEntry.IsValid())
			{
				UE_LOGF(LogImportDialogueScriptCommandlet, Log, "No internationalization manifest entry was found for context '%ls' in culture '%ls'. This context will be skipped.", *ContextLocalizationKey, *InCultureName);
				continue;
			}

			// Find the correct entry for our context
			const FManifestContext* ContextManifestEntryContext = ContextManifestEntry->FindContextByKey(ContextLocalizationKey);
			check(ContextManifestEntryContext); // This should never fail as we pass in the key to FindSourceText

			// Get the text we would have exported
			FLocItem ExportedSource;
			FLocItem ExportedTranslation;
			InLocTextHelper.GetExportText(InCultureName, FDialogueConstants::DialogueNamespace, ContextManifestEntryContext->Key, ContextManifestEntryContext->KeyMetadataObj, ELocTextExportSourceMethod::NativeText, ContextManifestEntry->Source, ExportedSource, ExportedTranslation);

			// Attempt to import the new text (if required)
			if (!ExportedTranslation.Text.Equals(ParsedScriptEntry.SpokenDialogue, ESearchCase::CaseSensitive))
			{
				if (InLocTextHelper.ImportTranslation(InCultureName, FDialogueConstants::DialogueNamespace, ContextManifestEntryContext->Key, ContextManifestEntryContext->KeyMetadataObj, ExportedSource, FLocItem(ParsedScriptEntry.SpokenDialogue), ContextManifestEntryContext->bIsOptional))
				{
					bHasUpdatedArchive = true;
				}
			}
		}
	}

	// Write out the updated archive file
	if (bHasUpdatedArchive)
	{
		FText SaveError;
		if (!InLocTextHelper.SaveArchive(InCultureName, &SaveError))
		{
			UE_LOGF(LogImportDialogueScriptCommandlet, Error, "%ls", *SaveError.ToString());
			return false;
		}
	}

	return true;
}
