// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ImportLocalizedDialogueCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AudioCompressionSettingsUtils.h"
#include "AudioEditorModule.h"
#include "EditorFramework/AssetImportData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Culture.h"
#include "Misc/App.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundWave.h"
#include "UObject/MetaData.h"
#include "Utils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportLocalizedDialogueCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogImportLocalizedDialogueCommandlet, Log, All);

namespace
{
	const FString GeneratedByCommandletMetaDataKey = TEXT("GeneratedByCommandlet");
	const FString GeneratedByCommandletMetaDataValue = TEXT("ImportLocalizedDialogueCommandlet");
}

UImportLocalizedDialogueCommandlet::UImportLocalizedDialogueCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UImportLocalizedDialogueCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config path
	FString ConfigPath;
	{
		const FString* ConfigPathParamVal = ParamVals.Find(FString(TEXT("Config")));
		if (!ConfigPathParamVal)
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No config specified.");
			return -1;
		}
		ConfigPath = *ConfigPathParamVal;
	}

	// Set config section
	FString SectionName;
	{
		const FString* SectionNameParamVal = ParamVals.Find(FString(TEXT("Section")));
		if (!SectionNameParamVal)
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No config section specified.");
			return -1;
		}
		SectionName = *SectionNameParamVal;
	}

	// Source path to the root folder that manifest/archive files live in
	FString SourcePath;
	if (!GetPathFromConfig(*SectionName, TEXT("SourcePath"), SourcePath, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No source path specified.");
		return -1;
	}

	// Get the native culture
	FString NativeCulture;
	if (!GetStringFromConfig(*SectionName, TEXT("NativeCulture"), NativeCulture, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No native culture specified.");
		return -1;
	}

	// Get cultures to generate
	TArray<FString> CulturesToGenerate;
	if (GetStringArrayFromConfig(*SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, ConfigPath) == 0)
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No cultures specified for import.");
		return -1;
	}

	// Get the manifest name
	FString ManifestName;
	if (!GetStringFromConfig(*SectionName, TEXT("ManifestName"), ManifestName, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No manifest name specified.");
		return -1;
	}

	// Get the archive name
	FString ArchiveName;
	if (!GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No archive name specified.");
		return -1;
	}

	// Should we import the native audio as source audio?
	bool bImportNativeAsSource = false;
	if (!GetBoolFromConfig(*SectionName, TEXT("bImportNativeAsSource"), bImportNativeAsSource, ConfigPath))
	{
		bImportNativeAsSource = false;
	}

	// Source path to the raw audio files that we're going to import
	FString RawAudioPath;
	if (!GetPathFromConfig(*SectionName, TEXT("RawAudioPath"), RawAudioPath, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No raw audio path specified.");
		return -1;
	}
	if (!FPaths::DirectoryExists(RawAudioPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Invalid raw audio path specified: %ls.", *RawAudioPath);
		return -1;
	}

	// Folder in which to place automatically imported sound wave assets
	FString ImportedDialogueFolder;
	if (!GetStringFromConfig(*SectionName, TEXT("ImportedDialogueFolder"), ImportedDialogueFolder, ConfigPath))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "No imported dialogue folder specified.");
		return -1;
	}
	if (ImportedDialogueFolder.IsEmpty())
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Imported dialogue folder cannot be empty.");
		return -1;
	}

	// Load the manifest and all archives
	FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, NativeCulture, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
	LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
	{
		FText LoadError;
		if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "%ls", *LoadError.ToString());
			return -1;
		}
	}
	
	const FString RootAssetPath = FApp::HasProjectName() ? TEXT("/Game") : TEXT("/Engine");
	const FString RootContentDir = FApp::HasProjectName() ? FPaths::ProjectContentDir() : FPaths::EngineContentDir();

	// Prepare the asset registry
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	// We want all the non-localized project specific dialogue waves
	TArray<FAssetData> AssetDataArrayForDialogueWaves;
	if (!FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, *RootAssetPath, UDialogueWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/false, AssetDataArrayForDialogueWaves))
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Unable to get dialogue wave asset data from asset registry.");
		return -1;
	}

	// Paths to scan for existing localized dialogue and sound waves
	TArray<FName> LocalizedDialogueWavePathsToSearch;
	TArray<FName> LocalizedSoundWavePathsToSearch;

	// Build up the culture specific import info
	TMap<FString, FCultureImportInfo> CultureImportInfoMap;
	for (const FString& CultureName : CulturesToGenerate)
	{
		FString PreferredCultureIdentifier = CultureName;
		if (FCulture::GetPreferredOnDiskIdentifierType() == ECultureIdentifierType::Verse)
		{
			PreferredCultureIdentifier = FCulture::CultureNameToVerseIdentifier(CultureName);
		}

		FCultureImportInfo& CultureImportInfo = CultureImportInfoMap.Add(CultureName);
		CultureImportInfo.Name = CultureName;
		CultureImportInfo.AudioPath = RawAudioPath / CultureName;
		CultureImportInfo.ArchiveFileName = SourcePath / CultureName / ArchiveName;
		CultureImportInfo.LocalizedRootContentPath = RootContentDir / TEXT("L10N") / PreferredCultureIdentifier;
		CultureImportInfo.LocalizedRootPackagePath = RootAssetPath / TEXT("L10N") / PreferredCultureIdentifier;
		CultureImportInfo.LocalizedImportedDialoguePackagePath = CultureImportInfo.LocalizedRootPackagePath / ImportedDialogueFolder;
		CultureImportInfo.bIsNativeCulture = CultureName == NativeCulture;

		// Need to check for both UE and Verse style identifiers, as both may be in use
		LocalizedDialogueWavePathsToSearch.Add(*(RootAssetPath / TEXT("L10N") / CultureName));
		LocalizedSoundWavePathsToSearch.Add(*(RootAssetPath / TEXT("L10N") / CultureName / ImportedDialogueFolder));
		if (const FString VerseIdentifier = FCulture::CultureNameToVerseIdentifier(CultureName);
			VerseIdentifier != CultureName)
		{
			LocalizedDialogueWavePathsToSearch.Add(*(RootAssetPath / TEXT("L10N") / VerseIdentifier));
			LocalizedSoundWavePathsToSearch.Add(*(RootAssetPath / TEXT("L10N") / VerseIdentifier / ImportedDialogueFolder));
		}
	}

	// Find all of the existing localized dialogue and sound waves - we'll keep track of which ones we process so we can delete any that are no longer needed
	TArray<FAssetData> LocalizedAssetsToPotentiallyDelete;
	{
		// We always add the source imported dialogue folder to ensure we clean it up correctly if we change the "import native as source" option
		// This is also why we always add the native culture, even though only one will be in use at any one time
		LocalizedSoundWavePathsToSearch.Add(*(RootAssetPath / ImportedDialogueFolder));

		FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, LocalizedDialogueWavePathsToSearch, UDialogueWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/true, LocalizedAssetsToPotentiallyDelete);
		FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, LocalizedSoundWavePathsToSearch, USoundWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/true, LocalizedAssetsToPotentiallyDelete);
	}

	// We're going to walk every context from every dialogue wave asset looking to see whether there's new audio to import for each culture we generate for
	// We filter these dialogue waves against the current manifest so that we only attempt to update assets that we gather text from
	for (const FAssetData& AssetData : AssetDataArrayForDialogueWaves)
	{
		// Verify that the found asset is a dialogue wave
		if (AssetData.GetClass() != UDialogueWave::StaticClass())
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Asset registry found asset '%ls', but it's not actually a dialogue wave.", *AssetData.GetSoftObjectPath().ToString());
			continue;
		}

		// Get the dialogue wave
		UDialogueWave* const DialogueWave = Cast<UDialogueWave>(AssetData.GetAsset());

		// Verify that the dialogue wave was loaded
		if (!DialogueWave)
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Asset registry found asset '%ls', but the dialogue wave could not be loaded.", *AssetData.GetSoftObjectPath().ToString());
			continue;
		}

		FString _Unused_DialogueWaveRoot, DialogueWaveSubPath, _Unused_DialogueWaveAssetName;
		if (!FPackageName::SplitLongPackageName(AssetData.PackageName.ToString(), _Unused_DialogueWaveRoot, DialogueWaveSubPath, _Unused_DialogueWaveAssetName))
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Failed to split dialogue wave package name '%ls'.", *AssetData.PackageName.ToString());
			continue;
		}

		// If we're importing native as source, then we import using a special culture import info
		if (bImportNativeAsSource && CultureImportInfoMap.Contains(NativeCulture))
		{
			FCultureImportInfo SourceCultureImportInfo = CultureImportInfoMap.FindRef(NativeCulture);
			SourceCultureImportInfo.LocalizedRootContentPath = RootContentDir;
			SourceCultureImportInfo.LocalizedRootPackagePath = RootAssetPath;
			SourceCultureImportInfo.LocalizedImportedDialoguePackagePath = SourceCultureImportInfo.LocalizedRootPackagePath / ImportedDialogueFolder;

			ImportDialogueForCulture(LocTextHelper, DialogueWave, DialogueWaveSubPath, SourceCultureImportInfo, /*bImportAsSource*/true);
		}

		// Iterate over each context looking for new audio to import
		for (const auto& CultureImportInfoPair : CultureImportInfoMap)
		{
			const FCultureImportInfo& CultureImportInfo = CultureImportInfoPair.Value;

			// Skip the native culture if importing native as source, as we'll have imported it above
			if (bImportNativeAsSource && CultureImportInfo.bIsNativeCulture)
			{
				continue;
			}

			ImportDialogueForCulture(LocTextHelper, DialogueWave, DialogueWaveSubPath, CultureImportInfo, /*bImportAsSource*/false);
		}
	}

	// Remove any left over assets that we no longer need
	for (const FAssetData& LocalizedAssetData : LocalizedAssetsToPotentiallyDelete)
	{
		// Has this asset already keen marked as "keep"?
		if (AssetsToKeep.Contains(LocalizedAssetData.GetSoftObjectPath()))
		{
			continue;
		}

		UObject* LocalizedAsset = LocalizedAssetData.GetAsset();
		if (!LocalizedAsset)
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Asset registry found localized asset '%ls' to delete, but the asset could not be loaded.", *LocalizedAssetData.GetSoftObjectPath().ToString());
			continue;
		}

		// Check the package meta-data to make sure that we only delete packages that we own
		const FString AssetGeneratedByCommandletMetaDataValue = LocalizedAsset->GetPackage()->GetMetaData().GetValue(LocalizedAsset, *GeneratedByCommandletMetaDataKey);
		if (AssetGeneratedByCommandletMetaDataValue != GeneratedByCommandletMetaDataValue)
		{
			continue;
		}
		
		FLocalizedAssetSCCUtil::DeleteAssetWithSCC(SourceControlInfo, LocalizedAsset);
	}

	return 0;
}

bool UImportLocalizedDialogueCommandlet::ImportDialogueForCulture(FLocTextHelper& InLocTextHelper, UDialogueWave* const DialogueWave, const FString& DialogueWaveSubPath, const FCultureImportInfo& InCultureImportInfo, const bool bImportAsSource)
{
	UDialogueWave* LocalizedDialogueWave = nullptr;
	FString LocalizedDialogueWaveFileName;

	if (bImportAsSource)
	{
		LocalizedDialogueWave = DialogueWave;
		LocalizedDialogueWaveFileName = FPackageName::LongPackageNameToFilename(DialogueWave->GetOutermost()->GetPathName(), FPackageName::GetAssetPackageExtension());
	}
	else
	{
		LocalizedDialogueWaveFileName = (InCultureImportInfo.LocalizedRootContentPath / DialogueWaveSubPath / DialogueWave->GetName()) + FPackageName::GetAssetPackageExtension();

		// Clone the source dialogue wave into the localized folder, replacing any existing asset to ensure that we're up-to-date with the source data
		{
			if (!FLocalizedAssetSCCUtil::SaveAssetWithSCC(SourceControlInfo, DialogueWave, LocalizedDialogueWaveFileName))
			{
				return false;
			}

			// Load up the newly saved asset
			const FString LocalizedDialogueWavePackagePath = (InCultureImportInfo.LocalizedRootPackagePath / DialogueWaveSubPath / TEXT("")) + DialogueWave->GetName();
			const FString LocalizedDialogueWaveAssetPath = FString::Printf(TEXT("%s.%s"), *LocalizedDialogueWavePackagePath, *DialogueWave->GetName());
			LocalizedDialogueWave = LoadObject<UDialogueWave>(nullptr, *LocalizedDialogueWaveAssetPath);
		}

		if (!LocalizedDialogueWave)
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Failed to create a localized dialogue wave '%ls' for culture '%ls'. No dialogue will be imported for this culture.", *DialogueWave->GetName(), *InCultureImportInfo.Name);
			return false;
		}

		// Mark this localized dialogue wave as used so it doesn't get deleted later
		AssetsToKeep.Add(FSoftObjectPath(LocalizedDialogueWave));
	}

	// First pass, handle any contexts that have an exact mapping to their audio file
	TArray<FDialogueContextMapping*> ContextMappingsWithMissingAudio;
	for (FDialogueContextMapping& ContextMapping : LocalizedDialogueWave->ContextMappings)
	{
		const FString ContextLocalizationKey = LocalizedDialogueWave->GetContextLocalizationKey(ContextMapping);

		// Check that this entry exists in the manifest file, as we want to skip over dialogue that we aren't gathering
		TSharedPtr<FManifestEntry> ContextManifestEntry = InLocTextHelper.FindSourceText(FDialogueConstants::DialogueNamespace, ContextLocalizationKey, &LocalizedDialogueWave->SpokenText);
		if (!ContextManifestEntry.IsValid())
		{
			// We're skipping this context entry due to our manifest, but we don't want the sound wave it's using to be deleted
			if (ContextMapping.SoundWave)
			{
				AssetsToKeep.Add(FSoftObjectPath(ContextMapping.SoundWave));
			}

			UE_LOGF(LogImportLocalizedDialogueCommandlet, Log, "No internationalization manifest entry was found for context '%ls' in culture '%ls'. This context will be skipped.", *ContextLocalizationKey, *InCultureImportInfo.Name);
			continue;
		}

		const FString ContextAudioFilename = InCultureImportInfo.AudioPath / LocalizedDialogueWave->GetContextRecordedAudioFilename(ContextMapping);
		if (!FPaths::FileExists(ContextAudioFilename))
		{
			// No specific audio file exists for this context, however that means we may use a different audio file if we have another context with the same speaker (to share sound waves where possible)
			// Flag this context as needing a second pass
			ContextMappingsWithMissingAudio.Add(&ContextMapping);
			continue;
		}

		// Import the WAV file as a sound wave asset, potentially overwriting any existing asset
		// The WAV file will only be imported if it has been changed since the last time it was imported
		USoundWave* const SoundWave = ConditionalImportSoundWave(InCultureImportInfo.LocalizedImportedDialoguePackagePath / ContextLocalizationKey, ContextLocalizationKey, ContextAudioFilename);
		if (SoundWave)
		{
			// Set this context to use the newly imported sound wave
			ContextMapping.SoundWave = SoundWave;
		}

		// This sound wave is in use, so shouldn't be deleted
		if (ContextMapping.SoundWave)
		{
			AssetsToKeep.Add(FSoftObjectPath(ContextMapping.SoundWave));
		}
	}

	// Second pass, handle any contexts that should share sound data with another context
	for (FDialogueContextMapping* ContextMappingPtr : ContextMappingsWithMissingAudio)
	{
		auto GetTranslatedTextForContext = [&](const FDialogueContextMapping& InContextMapping) -> FString
		{
			const FString ContextLocalizationKey = LocalizedDialogueWave->GetContextLocalizationKey(InContextMapping);

			// Find the manifest entry for our context
			TSharedPtr<FManifestEntry> ContextManifestEntry = InLocTextHelper.FindSourceText(FDialogueConstants::DialogueNamespace, ContextLocalizationKey, &LocalizedDialogueWave->SpokenText);
			if (!ContextManifestEntry.IsValid())
			{
				return FString();
			}

			// Find the correct entry for our context
			const FManifestContext* ContextManifestEntryContext = ContextManifestEntry->FindContextByKey(ContextLocalizationKey);
			check(ContextManifestEntryContext); // This should never fail as we pass in the key to FindSourceText

			// Get the localized text to export
			FLocItem ExportedSource;
			FLocItem ExportedTranslation;
			InLocTextHelper.GetExportText(InCultureImportInfo.Name, FDialogueConstants::DialogueNamespace, ContextManifestEntryContext->Key, ContextManifestEntryContext->KeyMetadataObj, ELocTextExportSourceMethod::NativeText, ContextManifestEntry->Source, ExportedSource, ExportedTranslation);

			return ExportedTranslation.Text;
		};

		// Find the correct localized dialogue for this context
		const FString ContextLocalizedDialogue = GetTranslatedTextForContext(*ContextMappingPtr);
		if (ContextLocalizedDialogue.IsEmpty())
		{
			UE_LOGF(LogImportLocalizedDialogueCommandlet, Warning, "No dialogue was imported for context '%ls' in culture '%ls' as it has an empty translation.", *LocalizedDialogueWave->GetContextLocalizationKey(*ContextMappingPtr), *InCultureImportInfo.Name);
		}
		else
		{
			// Try and find another context using the same speaking voice and localized dialogue that does have audio to import - we'll share its sound wave
			const FDialogueContextMapping* FoundContextMapping = LocalizedDialogueWave->ContextMappings.FindByPredicate([&](const FDialogueContextMapping& PotentialContextMapping) -> bool
			{
				// Can't match ourself
				if (ContextMappingPtr == &PotentialContextMapping)
				{
					return false;
				}

				// Need to be using the same speaking voice
				if (ContextMappingPtr->Context.Speaker != PotentialContextMapping.Context.Speaker)
				{
					return false;
				}

				// Need to be saying the same dialogue
				const FString PotentialContextLocalizedDialogue = GetTranslatedTextForContext(PotentialContextMapping);
				if (!ContextLocalizedDialogue.Equals(PotentialContextLocalizedDialogue, ESearchCase::CaseSensitive))
				{
					return false;
				}

				// Needs to actually have a valid audio file to import
				const FString PotentialContextAudioFilename = InCultureImportInfo.AudioPath / LocalizedDialogueWave->GetContextRecordedAudioFilename(PotentialContextMapping);
				return FPaths::FileExists(PotentialContextAudioFilename);
			});

			if (FoundContextMapping)
			{
				// Set this context to use the same sound wave as the found context
				ContextMappingPtr->SoundWave = FoundContextMapping->SoundWave;
			}
			else
			{
				UE_LOGF(LogImportLocalizedDialogueCommandlet, Warning, "No dialogue was imported for context '%ls' in culture '%ls' as no suitable audio file could be found to import.", *LocalizedDialogueWave->GetContextLocalizationKey(*ContextMappingPtr), *InCultureImportInfo.Name);
			}
		}

		// This sound wave is in use, so shouldn't be deleted
		if (ContextMappingPtr->SoundWave)
		{
			AssetsToKeep.Add(FSoftObjectPath(ContextMappingPtr->SoundWave));
		}
	}

	LocalizedDialogueWave->MarkPackageDirty();

	// Add meta-data stating that this asset is owned by this commandlet
	LocalizedDialogueWave->GetPackage()->GetMetaData().SetValue(LocalizedDialogueWave, *GeneratedByCommandletMetaDataKey, *GeneratedByCommandletMetaDataValue);

	return FLocalizedAssetSCCUtil::SaveAssetWithSCC(SourceControlInfo, LocalizedDialogueWave, LocalizedDialogueWaveFileName);
}

USoundWave* UImportLocalizedDialogueCommandlet::ConditionalImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const
{
	FString PackageFileName;
	if (!FPackageName::TryConvertLongPackageNameToFilename(InSoundWavePackageName, PackageFileName, FPackageName::GetAssetPackageExtension()) || !FPaths::FileExists(PackageFileName))
	{
		// No existing asset, we need to perform the import
		return ImportSoundWave(InSoundWavePackageName, InSoundWaveAssetName, InWavFilename);
	}

	USoundWave* const ExistingSoundWave = LoadObject<USoundWave>(nullptr, *FString::Printf(TEXT("%s.%s"), *InSoundWavePackageName, *InSoundWaveAssetName));
	if (!ExistingSoundWave)
	{
		// No existing asset, we need to perform the import
		return ImportSoundWave(InSoundWavePackageName, InSoundWaveAssetName, InWavFilename);
	}

	// Find the import data that matches the file we're going to import
	FMD5Hash OldFileHash;
	{
		const FString WavLeafname = FPaths::GetCleanFilename(InWavFilename);
		const FAssetImportInfo::FSourceFile* const FoundSourceFile = ExistingSoundWave->AssetImportData->SourceData.SourceFiles.FindByPredicate([&](const FAssetImportInfo::FSourceFile& InSourceFile) -> bool
		{
			const FString SourceFileLeafname = FPaths::GetCleanFilename(InSourceFile.RelativeFilename);
			return SourceFileLeafname == WavLeafname;
		});

		if (FoundSourceFile)
		{
			OldFileHash = FoundSourceFile->FileHash;
		}
	}

	// We only need to import the sound wave if the file hash has changed, or the source hash is invalid
	const FMD5Hash CurrentFileHash = FMD5Hash::HashFile(*InWavFilename);
	if (!OldFileHash.IsValid() || CurrentFileHash != OldFileHash)
	{
		return ImportSoundWave(InSoundWavePackageName, InSoundWaveAssetName, InWavFilename);
	}

	return ExistingSoundWave;
}

USoundWave* UImportLocalizedDialogueCommandlet::ImportSoundWave(const FString& InSoundWavePackageName, const FString& InSoundWaveAssetName, const FString& InWavFilename) const
{
	// Find or create the package to host the sound wave
	UPackage* const SoundWavePackage = CreatePackage( *InSoundWavePackageName);
	if (!SoundWavePackage)
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Failed to create a sound wave package '%ls'.", *InSoundWavePackageName);
		return nullptr;
	}

	// Make sure the destination package is loaded
	SoundWavePackage->FullyLoad();

	IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
	USoundWave* const SoundWave = AudioEditorModule->ImportSoundWave(SoundWavePackage, *InSoundWaveAssetName, *InWavFilename);
	if (!SoundWave)
	{
		UE_LOGF(LogImportLocalizedDialogueCommandlet, Error, "Failed to import the sound wave asset '%ls.%ls' from '%ls'", *InSoundWavePackageName, *InSoundWaveAssetName, *InWavFilename);
		return nullptr;
	}

	// Compress to whatever formats the active target platforms want prior to saving the asset
	{
		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		if (TPM)
		{
			const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
			for (ITargetPlatform* Platform : Platforms)
			{
				SoundWave->GetCompressedData(Platform->GetWaveFormat(SoundWave), FPlatformCompressionUtilities::GetCookOverrides(*Platform->IniPlatformName()));
			}
		}
	}

	// Add meta-data stating that this asset is owned by this commandlet
	SoundWavePackage->GetMetaData().SetValue(SoundWave, *GeneratedByCommandletMetaDataKey, *GeneratedByCommandletMetaDataValue);

	// Write out the updated sound wave asset
	if (!FLocalizedAssetSCCUtil::SavePackageWithSCC(SourceControlInfo, SoundWavePackage))
	{
		return nullptr;
	}

	return SoundWave;
}
