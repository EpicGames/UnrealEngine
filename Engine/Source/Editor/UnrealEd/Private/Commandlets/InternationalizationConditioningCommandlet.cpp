// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/InternationalizationConditioningCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "Containers/Map.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/InternationalizationArchive.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/LocKeyFuncs.h"
#include "LocalizedAssetUtil.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/JsonInternationalizationArchiveSerializer.h"
#include "Serialization/JsonInternationalizationManifestSerializer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternationalizationConditioningCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationConditioningCommandlet, Log, All);


/**
*	UInternationalizationConditioningCommandlet::FLocalizationFile
*/

UInternationalizationConditioningCommandlet::FLocalizationFile::FLocalizationFile( const FString& InPath, FConfigFile* const InConfigFile )
	: LocFilename(InPath)
	, LocFile(InConfigFile)
{
}

UInternationalizationConditioningCommandlet::FLocalizationFile::FLocalizationFile( const TSharedRef<FLocalizationFile> Other )
	: LocFilename(Other->GetFullName())
	, LocFile(Other->GetFile())
{
}

UInternationalizationConditioningCommandlet::FLocalizationFile::~FLocalizationFile()
{
	LocFile = NULL;
}

void UInternationalizationConditioningCommandlet::FLocalizationFile::GetIdenticalProperties( TArray<FLocalizationFileEntry>& OutProperties ) const
{
	OutProperties += IdenticalProperties;
}

void UInternationalizationConditioningCommandlet::FLocalizationFile::GetTranslatedProperties( TArray<FLocalizationFileEntry>& OutProperties ) const
{
	OutProperties += TranslatedProperties;
}

void UInternationalizationConditioningCommandlet::FLocalizationFile::CompareToCounterpart( TSharedPtr<FLocalizationFile> Other )
{
	check(Other.IsValid());

	FConfigFile* OtherFile = Other->GetFile();
	check(Other.IsValid());
	check(LocFile != NULL);

	// Iterate through all sections in the loc file
	for ( const auto& Pair : *(const FConfigFile*)OtherFile )
	{
		const FString& LocSectionName = Pair.Key;
		const FConfigSection& MySection = Pair.Value;

		// Skip the [Language] and [Public] sections
		if( LocSectionName == TEXT("Language") || LocSectionName == TEXT("Public") )
		{
			continue;
		}

		// Find this section in the counterpart loc file
		const FConfigSection* OtherSection = OtherFile->FindSection(LocSectionName);
		if ( OtherSection != NULL )
		{
			// Iterate through all keys in this section
			for ( FConfigSection::TConstIterator It(MySection); It; ++It )
			{
				const FName Propname = It.Key();
				const FString& PropValue = It.Value().GetValue();

				FString EscapedPropValue = PropValue.ReplaceQuotesWithEscapedQuotes();

				// Find this key in the counterpart loc file
				const FConfigValue* OtherValue = OtherSection->Find(Propname);
				if ( OtherValue != NULL )
				{
					FString EscapedOtherValue = OtherValue->GetValue().ReplaceQuotesWithEscapedQuotes();

					// If the counterpart has the same value as we do or is empty, the value is untranslated
					if( OtherValue->GetValue().IsEmpty() )
					{
						// If the entry is empty we do nothing for the time being.
					}
					else if ( PropValue == OtherValue->GetValue() )
					{
						IdenticalProperties.Emplace( Other->GetFilename(), LocSectionName, Propname.ToString(), EscapedPropValue, EscapedPropValue );
					}
					else
					{
						TranslatedProperties.Emplace( Other->GetFilename(), LocSectionName, Propname.ToString(), EscapedPropValue, EscapedOtherValue );
					}
				}
				else
				{
					// The counterpart didn't contain this key
					UnmatchedProperties.Add(LocSectionName + TEXT(".") + Propname.ToString());
				}
			}
		}
		else
		{
			// The counterpart didn't contain this section
			UnmatchedSections.Add(FPaths::GetBaseFilename(LocFilename) + TEXT(".") + LocSectionName);
		}
	}
}


/**
 *	UInternationalizationConditioningCommandlet::FLocalizationFilePair
 */

UInternationalizationConditioningCommandlet::FLocalizationFilePair::~FLocalizationFilePair()
{
	NativeFile.Reset();
	ForeignFile.Reset();
}


void UInternationalizationConditioningCommandlet::FLocalizationFilePair::CompareFiles()
{
	verify( HasNativeFile() || HasForeignFile() );

	if ( HasNativeFile() && HasForeignFile() )
	{
		NativeFile->CompareToCounterpart(ForeignFile);
	}
}


void UInternationalizationConditioningCommandlet::FLocalizationFilePair::GetTranslatedProperties( TArray<FLocalizationFileEntry>& Properties )
{
	if ( HasNativeFile() && HasForeignFile() )
	{
		NativeFile->GetTranslatedProperties(Properties);
	}
}


void UInternationalizationConditioningCommandlet::FLocalizationFilePair::GetIdenticalProperties( TArray<FLocalizationFileEntry>& Properties )
{
	if ( HasNativeFile() && HasForeignFile() )
	{
		NativeFile->GetIdenticalProperties(Properties);
	}
}


bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::SetNativeFile( const FString& NativeFilename, FConfigFile* const NativeConfigFile )
{
	if ( NativeFilename.Len() == 0 )
	{
		return false;
	}

	NativeFile.Reset();

	NativeFile = MakeShareable( new FLocalizationFile( NativeFilename, NativeConfigFile ) );
	return NativeFile.IsValid() && NativeFile->GetFile() != NULL;
}


bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::SetForeignFile( const FString& ForeignFilename, FConfigFile* const ForeignConfigFile )
{
	if ( ForeignFilename.Len() == 0 )
	{
		return false;
	}

	ForeignFile.Reset();

	ForeignFile = MakeShareable( new FLocalizationFile( ForeignFilename, ForeignConfigFile ) );
	return ForeignFile.IsValid() && ForeignFile->GetFile() != NULL;
}

const FString UInternationalizationConditioningCommandlet::FLocalizationFilePair::GetFilename()
{
	return HasNativeFile()
		? NativeFile->GetFilename()
		: ForeignFile->GetFilename();
}

bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::HasNativeFile()
{
	return NativeFile.IsValid() && NativeFile->GetFile() != NULL;
}
bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::HasForeignFile()
{
	return ForeignFile.IsValid() && ForeignFile->GetFile() != NULL;
}
bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::HasNativeFile( const FString& Filename )
{
	return HasNativeFile() && Filename == NativeFile->GetFilename();
}
bool UInternationalizationConditioningCommandlet::FLocalizationFilePair::HasForeignFile( const FString& Filename )
{
	return HasForeignFile() && Filename == ForeignFile->GetFilename();
}


/**
 *	UGatherTextCommandlet
 */

UInternationalizationConditioningCommandlet::UInternationalizationConditioningCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, LegacyLocalizationCacheIni(EConfigCacheType::Temporary)
{
}

int32 UInternationalizationConditioningCommandlet::FindNativeIndex( const FString& Filename )
{
	int32 Result = INDEX_NONE;
	if ( Filename.Len() > 0 )
	{
		for ( int32 i = 0; i < LocPairs.Num(); i++ )
		{
			if ( LocPairs[i].HasNativeFile(Filename) )
			{
				Result = i;
				break;
			}
		}
	}
	return Result;
}


int32 UInternationalizationConditioningCommandlet::FindForeignIndex( const FString& Filename )
{
	int32 Result = INDEX_NONE;
	if ( Filename.Len() > 0 )
	{
		for ( int32 i = 0; i < LocPairs.Num(); i++ )
		{
			if ( LocPairs[i].HasForeignFile(Filename) )
			{
				Result = i;
				break;
			}
		}
	}

	return Result;
}


void UInternationalizationConditioningCommandlet::AddNativeFile( const FString& Filename )
{
	if ( Filename.Len() > 0 )
	{

		// Attempt to find the matching foreign file for this native file
		int32 Index = FindForeignIndex( FPaths::GetBaseFilename(Filename) );
		if ( Index == INDEX_NONE )
		{
			Index = LocPairs.AddZeroed();
		}

		LocPairs[Index].SetNativeFile( Filename, LegacyLocalizationCacheIni.FindConfigFile(Filename) );
	}
}


void UInternationalizationConditioningCommandlet::AddForeignFile( const FString& Filename )
{
	if ( Filename.Len() > 0 )
	{
		// Attempt to find the matching foreign file for this native file
		int32 Index = FindNativeIndex( FPaths::GetBaseFilename(Filename) );
		if ( Index == INDEX_NONE )
		{
			Index = LocPairs.AddZeroed();
		}

		LocPairs[Index].SetForeignFile( Filename, LegacyLocalizationCacheIni.FindConfigFile(Filename) );
	}
}


void UInternationalizationConditioningCommandlet::ReadLocFiles( const TArray<FString>& NativeFilenames, const TArray<FString>& ForeignFilenames )
{
	for ( int32 i = 0; i < NativeFilenames.Num(); i++ )
	{
		AddNativeFile(*NativeFilenames[i]);
	}

	for ( int32 i = 0; i < ForeignFilenames.Num(); i++ )
	{
		AddForeignFile(*ForeignFilenames[i]);
	}
}


bool UInternationalizationConditioningCommandlet::ProcessManifest( const FString& PrimaryLangExt, const FString& SourcePath, const FString& DestinationPath )
{
	FString ManifestName = TEXT("Manifest.txt");

	GetStringFromConfig( *SectionName, TEXT("ManifestName"), ManifestName, GatherTextConfigPath );

	// Build info about the primary language
	TArray<FString> PrimaryFilenames;
	TArray<FString> PathPrimaryFilenames;
	FString PrimaryLocDirectory = SourcePath / PrimaryLangExt + TEXT("/");
	FString PrimaryWildcardName = PrimaryLocDirectory + TEXT("*.") + PrimaryLangExt;
	// Grab the list of primary language loc files
	IFileManager::Get().FindFiles(PathPrimaryFilenames, *PrimaryWildcardName, true, false);
	for ( int32 FileIndex = 0; FileIndex < PathPrimaryFilenames.Num(); FileIndex++ )
	{
		PrimaryFilenames.Add(PrimaryLocDirectory + PathPrimaryFilenames[FileIndex]);
	}

	if ( PrimaryFilenames.Num() == 0 )
	{
		UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "No primary language(%ls) loc files found!", *PrimaryLangExt);
		return false;
	}

	// Here we cheat a bit and use the primary language as the foreign language, some inefficiency here but it will let us leverage an
	//  existing system to get the entries we are after
	ReadLocFiles(PrimaryFilenames, PrimaryFilenames);

	// Instead of extracting the translated properties, we will pull out the identical properties which will be all the entries
	//  in the localization files since we are comparing the primary language with itself.
	TArray<FLocalizationFileEntry> IdenticalProperties;
	
	for ( int32 i = 0; i < LocPairs.Num(); i++ )
	{
		FLocalizationFilePair& Pair = LocPairs[i];
		Pair.CompareFiles();

		Pair.GetIdenticalProperties( IdenticalProperties );
	}

	// First we want to see if there is an existing manifest.  If so we will load it up and add our entries there
	TSharedRef< FInternationalizationManifest > InternationalizationManifest = MakeShareable( new FInternationalizationManifest );

	FString ExistingManifestFileName = DestinationPath / ManifestName;

	if( FPaths::FileExists(ExistingManifestFileName) )
	{
		FJsonInternationalizationManifestSerializer::DeserializeManifestFromFile( ExistingManifestFileName, InternationalizationManifest );
	}

	// Now we add our properties to the manifest. 
	for( int PropIndex = 0; PropIndex < IdenticalProperties.Num(); PropIndex++ )
	{
		FLocalizationFileEntry& Prop = IdenticalProperties[PropIndex];

		// We use the file(package) name and the namespace for the manifest namespace so we avoid potential collisions when multiple ini files have entries where the KEY and namespace are the same but the source text is different
		FString NewNamespace = Prop.Namespace;
		FManifestContext PropContext;
		PropContext.Key = Prop.Key;
		PropContext.SourceLocation = NewNamespace;
		FLocItem Source( Prop.SourceText );
		bool bAddSuccessful = InternationalizationManifest->AddSource( NewNamespace, Source, PropContext );
		if(!bAddSuccessful)
		{
			UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "Could not add manifest entry %ls.", *PropContext.SourceLocation );
		}
	}

	FString DestinationManifestFileName = DestinationPath / ManifestName;
	const bool bDidWriteManifest = FLocalizedAssetSCCUtil::SaveFileWithSCC(SourceControlInfo, DestinationManifestFileName, [&InternationalizationManifest](const FString& InSaveFileName) -> bool
	{
		return FJsonInternationalizationManifestSerializer::SerializeManifestToFile(InternationalizationManifest, InSaveFileName);
	});

	if (!bDidWriteManifest)
	{
		UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "Could not save file %ls", *DestinationManifestFileName);
		return false;
	}

	LocPairs.Empty();

	return true;
}

bool UInternationalizationConditioningCommandlet::ProcessArchive( const FString& PrimaryLangExt, const FString& SourcePath, const FString& DestinationPath )
{
	FString ArchiveName = TEXT("Archive.txt");
	TArray<FString> LanguagesToProcess;
	TArray<FString> TargetCultures;
	bool bAppendToExistingArchive = true;

	GetStringFromConfig( *SectionName, TEXT("ArchiveName"), ArchiveName, GatherTextConfigPath );
	GetStringArrayFromConfig( *SectionName, TEXT("ProcessLanguage"), LanguagesToProcess, GatherTextConfigPath );
	GetStringArrayFromConfig( *SectionName, TEXT("TargetCulture"), TargetCultures, GatherTextConfigPath );
	GetBoolFromConfig( *SectionName, TEXT("bAppendToExistingArchive"), bAppendToExistingArchive, GatherTextConfigPath );

	// Build info about the primary language
	TArray<FString> PrimaryFilenames;
	TArray<FString> PathPrimaryFilenames;
	FString PrimaryLocDirectory = SourcePath / PrimaryLangExt + TEXT("/");
	FString PrimaryWildcardName = PrimaryLocDirectory + TEXT("*.") + PrimaryLangExt;
	// Grab the list of primary language loc files
	IFileManager::Get().FindFiles(PathPrimaryFilenames, *PrimaryWildcardName, true, false);
	for ( int32 FileIndex = 0; FileIndex < PathPrimaryFilenames.Num(); FileIndex++ )
	{
		PrimaryFilenames.Add(PrimaryLocDirectory + PathPrimaryFilenames[FileIndex]);
	}

	if ( PrimaryFilenames.Num() == 0 )
	{
		UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "No primary language(%ls) loc files found!", *PrimaryLangExt);
		return false;
	}

	for( int32 LanguageIndex = 0; LanguageIndex < LanguagesToProcess.Num(); LanguageIndex++ )
	{
		FString ForeignLangExt = LanguagesToProcess[LanguageIndex];
		TArray<FString> ForeignFilenames;
		TArray<FString> PathForeignFilenames;
		FString ForeignLocDirectory = SourcePath / ForeignLangExt + TEXT("/");
		FString ForeignWildcardName = ForeignLocDirectory + TEXT("*.") + ForeignLangExt;
		FString TargetSubfolder = TargetCultures.Num() > LanguageIndex ? TargetCultures[LanguageIndex] : ForeignLangExt;

		// Get a list of foreign loc files
		IFileManager::Get().FindFiles(PathForeignFilenames, *ForeignWildcardName, true, false);

		for ( int32 FileIndex = 0; FileIndex < PathForeignFilenames.Num(); FileIndex++ )
		{
			ForeignFilenames.Add(ForeignLocDirectory + PathForeignFilenames[FileIndex]);
		}

		if ( ForeignFilenames.Num() == 0 )
		{
			UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "No foreign loc files found using language extension '%ls'", *ForeignLangExt);
			continue;
		}

		ReadLocFiles(PrimaryFilenames, ForeignFilenames);

		TArray<FLocalizationFileEntry> ArchiveProperties;

		// FSor each file in the list, 
		for ( int32 i = 0; i < LocPairs.Num(); i++ )
		{
			FLocalizationFilePair& Pair = LocPairs[i];
			Pair.CompareFiles();

			Pair.GetTranslatedProperties( ArchiveProperties );
			Pair.GetIdenticalProperties( ArchiveProperties );

		}

		TSharedRef< FInternationalizationArchive > InternationalizationArchive = MakeShareable( new FInternationalizationArchive );

		const FString DestinationArchiveFileName = DestinationPath / TargetSubfolder / ArchiveName;

		// If we want to append to an existing archive, we first read it into our data structure
		if( bAppendToExistingArchive )
		{
			FString ExistingArchiveFileName = DestinationArchiveFileName;

			if( FPaths::FileExists(ExistingArchiveFileName) )
			{
				FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromFile( ExistingArchiveFileName, InternationalizationArchive, nullptr, nullptr );
			}
		}

		for( int PropIndex = 0; PropIndex < ArchiveProperties.Num(); PropIndex++ )
		{
			FLocalizationFileEntry& Prop = ArchiveProperties[PropIndex];
			FString NewNamespace = Prop.Namespace;
			FString NewKey = Prop.Key;

			FLocItem Source( Prop.SourceText );
			FLocItem Translation( Prop.TranslatedText );

			if( !InternationalizationArchive->AddEntry( NewNamespace, NewKey, Source, Translation, NULL, false ) )
			{
				TSharedPtr<FArchiveEntry> ExistingConflictEntry = InternationalizationArchive->FindEntryByKey( NewNamespace, NewKey, NULL );

				if( !ExistingConflictEntry.IsValid() )
				{
					// Looks like we failed to add for a reason beyond conflicting translation, display an error and continue.
					UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "Failed to add entry to archive Namespace [%ls]: (DEFAULT TEXT): %ls (EXISTING TRANSLATION): %ls", 
						*NewNamespace, *Prop.SourceText, *ExistingConflictEntry->Translation.Text );
					continue;
				}

				// If we can't add the entry, we find the existing conflicting entry and see if the translation is empty.  If it is empty we will
				// just overwrite the translation.  If it is not empty we will display info about the conflict.
				if( ExistingConflictEntry->Translation.Text.IsEmpty() )
				{
					ExistingConflictEntry->Translation.Text =  Prop.TranslatedText;
				}
				else
				{
					UE_LOGF(LogInternationalizationConditioningCommandlet, Warning, "Conflicting translation ignored in Namespace [%ls]: (DEFAULT TEXT): %ls (EXISTING TRANSLATION): %ls  (REJECTED TRANSLATION): %ls", 
						*NewNamespace, *Prop.SourceText, *ExistingConflictEntry->Translation.Text, *Prop.TranslatedText );
				}
			}
		}

		const bool bDidWriteArchive = FLocalizedAssetSCCUtil::SaveFileWithSCC(SourceControlInfo, DestinationArchiveFileName, [&InternationalizationArchive](const FString& InSaveFileName) -> bool
		{
			return FJsonInternationalizationArchiveSerializer::SerializeArchiveToFile(InternationalizationArchive, InSaveFileName);
		});

		if (!bDidWriteArchive)
		{
			UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "Could not save file %ls", *DestinationArchiveFileName);
			return false;
		}

		LocPairs.Empty();
	}
	return true;
}

void UInternationalizationConditioningCommandlet::LoadLegacyLocalizationFiles( const FString& SourcePath, const FString& NativeLanguage, const TArray<FString>& LanguagesToProcess )
{
	TArray<FString> AllLanguages = LanguagesToProcess;
	AllLanguages.AddUnique(NativeLanguage); // Is this going to crash or just not add if it's not unique?

	for(int32 i = 0; i < AllLanguages.Num(); ++i)
	{
		const FString LanguageName = AllLanguages[i];
		const FString LanguageDirectory = SourcePath + TEXT("/") + LanguageName;
		const FString Wildcard = FString(TEXT("*.")) + LanguageName;

		TArray<FString> LegacyLocalizationFileNames;
		IFileManager::Get().FindFiles(LegacyLocalizationFileNames, *(LanguageDirectory + TEXT("/") + Wildcard), true, false);

		for(int32 j = 0; j < LegacyLocalizationFileNames.Num(); ++j)
		{
			const FString LegacyLocalizationPath = LanguageDirectory + TEXT("/") + LegacyLocalizationFileNames[j];
			LegacyLocalizationCacheIni.Find(LegacyLocalizationPath); // GConfigCacheIni::Find(const FString&, bool) Causes the file to load if not loaded.
		}
	}
}

int32 UInternationalizationConditioningCommandlet::Main( const FString& Params )
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);
	
	const FString* ParamVal = ParamVals.Find(FString(TEXT("Config")));

	if ( ParamVal )
	{
		GatherTextConfigPath = *ParamVal;
	}
	else
	{
		UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "No config specified.");
		return -1;
	}

	//Set config section
	ParamVal = ParamVals.Find(FString(TEXT("Section")));
	

	if ( ParamVal )
	{
		SectionName = *ParamVal;
	}
	else
	{
		UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "No config section specified.");
		return -1;
	}

	// Common settings
	FString SourcePath; // Source path to the root folder that manifest/archive files live in
	FString DestinationPath; // Destination path that we will write conditioned archive and manifest files to.  Language specific info will be appended to this path for archives.
	FString PrimaryLangExt;
	TArray<FString> LanguagesToProcess;

	// Settings for generating/appending to archive files from legacy localization files
	bool bGenerateArchiveFromLocIni = false;


	// Settings for generating or appending entries to manifest from legacy localization files
	bool bGenerateManifestFromLocIni = false;
	
	// Get the common settings from config
	GetStringFromConfig( *SectionName, TEXT("SourcePath"), SourcePath, GatherTextConfigPath );
	GetStringFromConfig( *SectionName, TEXT("DestinationPath"), DestinationPath, GatherTextConfigPath );
	GetStringFromConfig( *SectionName, TEXT("PrimaryLanguage"), PrimaryLangExt, GatherTextConfigPath );
	GetStringArrayFromConfig( *SectionName, TEXT("ProcessLanguage"), LanguagesToProcess, GatherTextConfigPath );

	GetBoolFromConfig( *SectionName, TEXT("bGenerateManifestFromLocIni"), bGenerateManifestFromLocIni, GatherTextConfigPath );
	GetBoolFromConfig( *SectionName, TEXT("bGenerateArchiveFromLocIni"), bGenerateArchiveFromLocIni, GatherTextConfigPath );

	// Load legacy localization files.
	LoadLegacyLocalizationFiles(SourcePath, PrimaryLangExt, LanguagesToProcess);

	// If features are enabled, we'll do those in order here
	if(  bGenerateManifestFromLocIni )
	{
		// Add to or create a manifest if desired
		if( !ProcessManifest( PrimaryLangExt, SourcePath, DestinationPath ) )
		{
			UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "Failed to generate manifest file from ini files.");
			return -1;
		}
	}

	if( bGenerateArchiveFromLocIni )
	{
		if( !ProcessArchive( PrimaryLangExt, SourcePath, DestinationPath ) )
		{
			UE_LOGF(LogInternationalizationConditioningCommandlet, Error, "Failed to generate manifest file from ini files.");
			return -1;
		}
	}

	return 0;
}
