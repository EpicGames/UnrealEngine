// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/InternationalizationExportCommandlet.h"

#include "Commandlets/Commandlet.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "LocTextHelper.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "PortableObjectPipeline.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InternationalizationExportCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogInternationalizationExportCommandlet, Log, All);
namespace InternationalizationExportCommandlet
{
	static constexpr int32 LocalizationLogIdentifier = 304;
}

/**
*	UInternationalizationExportCommandlet
*/
int32 UInternationalizationExportCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config
	FString ConfigPath;
	if (const FString* ConfigParamVal = ParamVals.Find(FString(TEXT("Config"))))
	{
		ConfigPath = *ConfigParamVal;
	}
	else
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No config specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Set config section
	FString SectionName;
	if (const FString* ConfigSectionParamVal = ParamVals.Find(FString(TEXT("Section"))))
	{
		SectionName = *ConfigSectionParamVal;
	}
	else
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No config section specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get native culture.
	FString NativeCultureName;
	if (!GetStringFromConfig(*SectionName, TEXT("NativeCulture"), NativeCultureName, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No native culture specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get manifest name.
	FString ManifestName;
	if (!GetStringFromConfig(*SectionName, TEXT("ManifestName"), ManifestName, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No manifest name specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Get archive name.
	FString ArchiveName;
	if (!GetStringFromConfig(*SectionName, TEXT("ArchiveName"), ArchiveName, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No archive name specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Source path to the root folder that manifest/archive files live in.
	FString SourcePath;
	if (!GetPathFromConfig(*SectionName, TEXT("SourcePath"), SourcePath, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No source path specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Destination path that we will write files to.
	FString DestinationPath;
	if (!GetPathFromConfig(*SectionName, TEXT("DestinationPath"), DestinationPath, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No destination path specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Name of the file to read or write from.
	FString Filename;
	if (!GetStringFromConfig(*SectionName, TEXT("PortableObjectName"), Filename, ConfigPath))
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No portable object name specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get cultures to generate.
	TArray<FString> CulturesToGenerate;
	if (GetStringArrayFromConfig(*SectionName, TEXT("CulturesToGenerate"), CulturesToGenerate, ConfigPath) == 0)
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No cultures specified for generation.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	// Get culture directory setting, default to true if not specified (used to allow picking of import directory with file open dialog from Translation Editor)
	bool bUseCultureDirectory = true;
	if (!GetBoolFromConfig(*SectionName, TEXT("bUseCultureDirectory"), bUseCultureDirectory, ConfigPath))
	{
		bUseCultureDirectory = true;
	}

	// Read in the text collapse mode to use
	ELocalizedTextCollapseMode TextCollapseMode = ELocalizedTextCollapseMode::IdenticalTextIdAndSource;
	{
		FString TextCollapseModeName;
		if (GetStringFromConfig(*SectionName, TEXT("LocalizedTextCollapseMode"), TextCollapseModeName, ConfigPath))
		{
			UEnum* LocalizedTextCollapseModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.ELocalizedTextCollapseMode"));
			const int64 TextCollapseModeInt = LocalizedTextCollapseModeEnum->GetValueByName(*TextCollapseModeName);
			if (TextCollapseModeInt != INDEX_NONE)
			{
				TextCollapseMode = (ELocalizedTextCollapseMode)TextCollapseModeInt;
			}
		}
	}

	// Read in the PO format to use
	EPortableObjectFormat POFormat = EPortableObjectFormat::Unreal;
	{
		FString POFormatName;
		if (GetStringFromConfig(*SectionName, TEXT("POFormat"), POFormatName, ConfigPath))
		{
			UEnum* POFormatEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Localization.EPortableObjectFormat"));
			const int64 POFormatInt = POFormatEnum->GetValueByName(*POFormatName);
			if (POFormatInt != INDEX_NONE)
			{
				POFormat = (EPortableObjectFormat)POFormatInt;
			}
		}
	}

	GetBoolFromConfig(*SectionName, TEXT("bImportLoc"), bDoImport, ConfigPath);
	GetBoolFromConfig(*SectionName, TEXT("bExportLoc"), bDoExport, ConfigPath);

	if (!bDoImport && !bDoExport)
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "Import/Export operation not detected. Use bExportLoc or bImportLoc in config section.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return -1;
	}

	if (bDoImport)
	{
		UE_SCOPED_TIMER(TEXT("UInternationalizationExportCommandlet::Main Import"), LogInternationalizationExportCommandlet, Display);
		// Load the manifest and all archives
		FLocTextHelper LocTextHelper(DestinationPath, ManifestName, ArchiveName, NativeCultureName, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
		LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
		{
			FText LoadError;
			if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
			{
				UE_LOGFMT(LogInternationalizationExportCommandlet, Warning, "Load error: {error}",
					("error", *LoadError.ToString()),
					("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
				);
				return 0;
			}
		}

		// Import all PO files
		if (!PortableObjectPipeline::ImportAll(LocTextHelper, SourcePath, Filename, TextCollapseMode, POFormat, bUseCultureDirectory))
		{
			UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "Failed to import localization files.",
				("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	if (bDoExport)
	{
		UE_SCOPED_TIMER(TEXT("UInternationalizationExportCommandlet::Main (Export)"), LogInternationalizationExportCommandlet, Display);
		bool bShouldPersistComments = false;
		GetBoolFromConfig(*SectionName, TEXT("ShouldPersistCommentsOnExport"), bShouldPersistComments, ConfigPath);

		// Load the manifest and all archives
		FLocTextHelper LocTextHelper(SourcePath, ManifestName, ArchiveName, NativeCultureName, CulturesToGenerate, GatherManifestHelper->GetLocFileNotifies(), GatherManifestHelper->GetPlatformSplitMode());
		LocTextHelper.SetCopyrightNotice(GatherManifestHelper->GetCopyrightNotice());
		{
			FText LoadError;
			if (!LocTextHelper.LoadAll(ELocTextHelperLoadFlags::Load, &LoadError))
			{
				UE_LOGFMT(LogInternationalizationExportCommandlet, Warning, "Load error: {error}",
					("error", *LoadError.ToString()),
					("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
				);
				return 0;
			}
		}

		// Export all PO files
		if (!PortableObjectPipeline::ExportAll(LocTextHelper, DestinationPath, Filename, TextCollapseMode, POFormat, bShouldPersistComments, bUseCultureDirectory))
		{
			UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "Failed to export localization files.",
				("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
			);
			return -1;
		}
	}

	return 0;
}

bool UInternationalizationExportCommandlet::ConfigurePhase(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Set config
	FString ConfigPath;
	if (const FString* ConfigParamVal = ParamVals.Find(FString(TEXT("Config"))))
	{
		ConfigPath = *ConfigParamVal;
	}
	else
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No config specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	// Set config section
	FString SectionName;
	if (const FString* ConfigSectionParamVal = ParamVals.Find(FString(TEXT("Section"))))
	{
		SectionName = *ConfigSectionParamVal;
	}
	else
	{
		UE_LOGFMT(LogInternationalizationExportCommandlet, Error, "No config section specified.",
			("id", InternationalizationExportCommandlet::LocalizationLogIdentifier)
		);
		return false;
	}

	GetBoolFromConfig(*SectionName, TEXT("bImportLoc"), bDoImport, ConfigPath);
	GetBoolFromConfig(*SectionName, TEXT("bExportLoc"), bDoExport, ConfigPath);

	return true;
}

EGatherTextCommandletPhase UInternationalizationExportCommandlet::GetPhase() const
{
	if (bDoImport)
	{
		return EGatherTextCommandletPhase::Import;
	}
	if (bDoExport)
	{
		return EGatherTextCommandletPhase::Export;
	}
	return EGatherTextCommandletPhase::Undefined;
}
