// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/IoStoreCommandlet.h"
#include "IoStoreUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IoStoreCommandlet)

UIoStoreCommandlet::UIoStoreCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UIoStoreCommandlet::Main(const FString& Params)
{
#if UE_BUILD_SHIPPING
	return 0;
#else
	return CreateIoStoreContainerFiles(*Params);
#endif
}
