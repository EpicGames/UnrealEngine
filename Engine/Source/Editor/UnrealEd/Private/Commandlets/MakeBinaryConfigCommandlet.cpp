// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/MakeBinaryConfigCommandlet.h"
#include "CoreGlobals.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MakeBinaryConfigCommandlet)


UMakeBinaryConfigCommandlet::UMakeBinaryConfigCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UMakeBinaryConfigCommandlet::Main(const FString& Params)
{
	UE_LOGF(LogInit, Fatal, "This commandlet is no longer supported. Use UnrealPak instead, which now has the BinaryConfig functionality");
	return 0;
}
