// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/MaterialLayerUsageCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialLayerUsageCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogMaterialLayerUsageCommandlet, Log, All);

static bool VisitExpressions(TConstArrayView<UMaterialExpression*> Expressions)
{
	for (UMaterialExpression* Expression : Expressions)
	{
		if (!Expression)
		{
			continue;
		}

		if (UMaterialExpressionMaterialAttributeLayers* LayersExpression = Cast<UMaterialExpressionMaterialAttributeLayers>(Expression))
		{
			return true;
		}
		else if (UMaterialExpressionMaterialFunctionCall* FunctionCall = Cast<UMaterialExpressionMaterialFunctionCall>(Expression))
		{
			if (FunctionCall->MaterialFunction && VisitExpressions(FunctionCall->MaterialFunction->GetExpressions()))
			{
				return true;
			}
		}
	}

	return false;
}

UMaterialLayerUsageCommandlet::UMaterialLayerUsageCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UMaterialLayerUsageCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMaterialLayerUsageCommandlet::Main);

	StaticExec(nullptr, TEXT("log LogMaterial Verbose"));

	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogMaterialLayerUsageCommandlet, Log, "MaterialLayerUsageCommandlet");
		UE_LOGF(LogMaterialLayerUsageCommandlet, Log, "This commandlet searches for all materials using material layers (i.e. materials containing the MaterialExpressionMaterialAttributeLayers node).");
		UE_LOGF(LogMaterialLayerUsageCommandlet, Log, " Optional: -collection=<name>                (You can also specify a collection of assets to narrow down the results e.g. if you maintain a collection that represents the actually used in-game assets).");
		UE_LOGF(LogMaterialLayerUsageCommandlet, Log, " Optional: -materials=<path1>+<path2>        (You can also specify a list of material asset paths separated by a '+' to narrow down the results.");
		return 0;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.SearchAllAssets(true);

	// Optional list of materials to compile.
	TArray<FAssetData> MaterialList;

	FARFilter Filter;

	// Parse collection
	FString Collection;
	if (FParse::Value(*Params, TEXT("collection="), Collection, true))
	{
		if (!Collection.IsEmpty())
		{
			// Get the list of materials from a collection
			Filter.PackagePaths.Add(FName(TEXT("/Game")));
			Filter.bRecursivePaths = true;
			Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());

			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
			TSharedPtr<ICollectionContainer> CollectionContainer;
			FName CollectionName;
			ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
			if (CollectionManager.TryParseCollectionPath(Collection, &CollectionContainer, &CollectionName, &ShareType))
			{
				CollectionContainer->GetObjectsInCollection(CollectionName, ShareType, Filter.SoftObjectPaths, ECollectionRecursionFlags::SelfAndChildren);
			}

			AssetRegistry.GetAssets(Filter, MaterialList);
		}
	}
	else
	{
		// Get the list of all materials
		Filter.PackagePaths.Add(FName(TEXT("/Game")));
		Filter.bRecursivePaths = true;
		Filter.ClassPaths.Add(UMaterial::StaticClass()->GetClassPathName());

		AssetRegistry.GetAssets(Filter, MaterialList);
	}

	UE_LOGF(LogMaterialLayerUsageCommandlet, Log, "Found %d/%d Materials.", MaterialList.Num(), Filter.SoftObjectPaths.Num());

	// Process -materials= switches separated by a '+'
	TArray<FString> CmdLineMaterialEntries;
	const TCHAR* MaterialsSwitchName = TEXT("Materials");
	if (const FString* MaterialsSwitches = ParamVals.Find(MaterialsSwitchName))
	{
		MaterialsSwitches->ParseIntoArray(CmdLineMaterialEntries, TEXT("+"));
	}

	if (CmdLineMaterialEntries.Num())
	{
		// re-use the filter and only filter based on the passed in objects.
		Filter.ClassPaths.Empty();
		Filter.SoftObjectPaths.Empty();
		for (const FString& MaterialPathString : CmdLineMaterialEntries)
		{
			const FSoftObjectPath MaterialPath(MaterialPathString);
			if (!Filter.SoftObjectPaths.Contains(MaterialPath))
			{
				Filter.SoftObjectPaths.Add(MaterialPath);
			}
		}

		AssetRegistry.GetAssets(Filter, MaterialList);
	}

	// Look for materials containing UMaterialExpressionMaterialAttributeLayers nodes
	UE_LOGF(LogMaterialLayerUsageCommandlet, Display, "%d input materials found.", MaterialList.Num());

	TArray<UMaterial*> PositiveMaterials;
	for (int i = 0; i < MaterialList.Num(); ++i)
	{
		UMaterial* Material = Cast<UMaterial>(MaterialList[i].GetAsset());
		check(Material);

		if (VisitExpressions(Material->GetExpressions()))
		{
			PositiveMaterials.Add(Material);
		}

		// Report an update message
		if ((i + 1) % 50 == 0)
		{
			UE_LOGF(LogMaterialLayerUsageCommandlet, Display, "Inspected %d/%d materials (%d positives).", i + 1, MaterialList.Num(), PositiveMaterials.Num());
		}
	}

	// Done. Report which materials containing layers were found.
	UE_LOGF(LogMaterialLayerUsageCommandlet, Display, "The following Materials contain MaterialExpressionMaterialAttributeLayers:");
	for (UMaterial* Material : PositiveMaterials)
	{
		UE_LOGF(LogMaterialLayerUsageCommandlet, Display, "%ls", *Material->GetFullName());
	}

	UE_LOGF(LogMaterialLayerUsageCommandlet, Display, "(Total count: %d materials)", PositiveMaterials.Num());

	return 0;
}

