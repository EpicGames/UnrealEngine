// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/MergeShaderPipelineCachesCommandlet.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "PipelineFileCache.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MergeShaderPipelineCachesCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogMergeShaderPipelineCachesCommandlet, Log, All);

UMergeShaderPipelineCachesCommandlet::UMergeShaderPipelineCachesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UMergeShaderPipelineCachesCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	bool bOK = (Tokens.Num() >= 3);
	
	FPipelineFileCacheManager::PSOOrder Order = FPipelineFileCacheManager::PSOOrder::Default;
	if (const FString* OrderVal = ParamVals.Find(FString(TEXT("Sort"))))
	{
		if(*OrderVal == TEXT("Default"))
		{
			Order = FPipelineFileCacheManager::PSOOrder::Default;
		}
		else if(*OrderVal == TEXT("FirstUsed"))
		{
			Order = FPipelineFileCacheManager::PSOOrder::FirstToLatestUsed;
		}
		else if(*OrderVal == TEXT("MostUsed"))
		{
			Order = FPipelineFileCacheManager::PSOOrder::MostToLeastUsed;
		}
		else
		{
			bOK = false;
		}
	}
	else
	{
		bOK = false;
	}
	
	FString A, B, Output;
	if (bOK)
	{
		A = Tokens[0];
		B = Tokens[1];
		Output = Tokens[2];
	}
	
	if (bOK == false)
	{
		UE_LOGF(LogMergeShaderPipelineCachesCommandlet, Warning, "Usage: FilePathA FilePathB -Sort=<Order> OutputPath.\n\tOrder Values:Default/FirstUsed/MostUsed.\n\tCache files must be have the same game version, shader platform and format version.");
        UE_LOGF(LogMergeShaderPipelineCachesCommandlet, Warning, "Provided arguments: %ls\n\tA: %ls\n\tB: %ls\n\tOutput: %ls.", *Params, *A, *B, *Output);
		return 0;
	}
	
	return !FPipelineFileCacheManager::MergePipelineFileCaches(A, B, Order, Output);
}
