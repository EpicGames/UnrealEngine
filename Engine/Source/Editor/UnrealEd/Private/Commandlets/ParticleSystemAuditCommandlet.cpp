// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ParticleSystemAuditCommandlet.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Particles/ParticleSystem.h"
#include "Distributions/DistributionFloatConstant.h"
#include "AssetRegistry/AssetData.h"
#include "Particles/ParticleModuleRequired.h"
#include "Particles/Light/ParticleModuleLight.h"
#include "Particles/Spawn/ParticleModuleSpawn.h"
#include "Particles/Spawn/ParticleModuleSpawnPerUnit.h"
#include "Particles/TypeData/ParticleModuleTypeDataRibbon.h"
#include "Particles/TypeData/ParticleModuleTypeDataBeam2.h"
#include "Particles/TypeData/ParticleModuleTypeDataAnimTrail.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/Location/ParticleModuleLocationBoneSocket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParticleSystemAuditCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogParticleSystemAuditCommandlet, Log, All);

UParticleSystemAuditCommandlet::UParticleSystemAuditCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	HighSpawnRateOrBurstThreshold = 35.f;
	FarLODDistanceTheshold = 3000.f;
}

int32 UParticleSystemAuditCommandlet::Main(const FString& Params)
{
	if (!FParse::Value(*Params, TEXT("AuditOutputFolder="), AuditOutputFolder))
	{
		// No output folder specified. Use the default folder.
		AuditOutputFolder = FPaths::ProjectSavedDir() / TEXT("Audit");
	}

	// Add a timestamp to the folder
	AuditOutputFolder /= FDateTime::Now().ToString();

	FParse::Value(*Params, TEXT("FilterCollection="), FilterCollection);

	FString PackagePathsString;
	if (FParse::Value(*Params, TEXT("PackagePaths="), PackagePathsString))
	{
		TArray<FString> PackagePathsStrings;
		PackagePathsString.ParseIntoArray(PackagePathsStrings, TEXT(","));
		for (const FString& v : PackagePathsStrings)
		{
			PackagePaths.Add(FName(v));
		}
	}

	if (PackagePaths.Num() == 0)
	{
		PackagePaths.Add(FName(TEXT("/Game")));
	}

	ProcessParticleSystems();
	DumpResults();

	return 0;
}

bool UParticleSystemAuditCommandlet::ProcessParticleSystems()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	FARFilter Filter;
	Filter.PackagePaths = PackagePaths;
	Filter.bRecursivePaths = true;

	Filter.ClassPaths.Add(UParticleSystem::StaticClass()->GetClassPathName());
	if (!FilterCollection.IsEmpty())
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
		if (CollectionManager.TryParseCollectionPath(FilterCollection, &CollectionContainer, &CollectionName, &ShareType))
		{
			CollectionContainer->GetObjectsInCollection(
				FName(*FilterCollection), ShareType, Filter.SoftObjectPaths,
				ECollectionRecursionFlags::SelfAndChildren);
		}
	}

	TArray<FAssetData> AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	double StartProcessParticleSystemsTime = FPlatformTime::Seconds();

	// Find all level placed particle systems with:
	//	- Single LOD level
	//	- No fixed bounds
	//	- LODLevel Mismatch 
	//	- Kismet referenced & auto-activate set
	// Iterate over the list and check each system for *no* lod
	// 
	const FString DevelopersFolder = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir().LeftChop(1));
	FString LastPackageName = TEXT("");
	int32 PackageSwitches = 0;
	UPackage* CurrentPackage = NULL;
	for (const FAssetData& AssetIt : AssetList)
	{
		const FString PSysName = AssetIt.GetObjectPathString();
		const FString PackageName = AssetIt.PackageName.ToString();

		if (PackageName.StartsWith(DevelopersFolder))
		{
			// Skip developer folders
			continue;
		}

		if (PackageName != LastPackageName)
		{
			UPackage* Package = ::LoadPackage(NULL, *PackageName, LOAD_None);
			if (Package != NULL)
			{
				LastPackageName = PackageName;
				Package->FullyLoad();
				CurrentPackage = Package;
			}
			else
			{
				UE_LOGF(LogParticleSystemAuditCommandlet, Warning, "Failed to load package %ls processing %ls", *PackageName, *PSysName);
				CurrentPackage = NULL;
			}
		}

		const FString ShorterPSysName = AssetIt.AssetName.ToString();
		UParticleSystem* PSys = FindObject<UParticleSystem>(CurrentPackage, *ShorterPSysName);
		if (PSys != NULL)
		{
			bool bInvalidLOD = false;
			bool bSingleLOD = false;
			bool bFoundEmitter = false;
			bool bMissingMaterial = false;
			bool bHasHighSpawnRateOrBurst = false;
			bool bHasRibbonTrailOrBeam = false;
			bool bHasOnlyBeamsOrHasNoEmitters = true;
			bool bHasSpawnPerUnit = false;
			bool bMismatchedLODBoneModules = false;
			bool bHasLights = false;
			bool bHasShadowCastingLights = false;
			bool bHasHiQualityLights = false;
			for (int32 EmitterIdx = 0; EmitterIdx < PSys->Emitters.Num(); EmitterIdx++)
			{
				UParticleEmitter* Emitter = PSys->Emitters[EmitterIdx];
				if (Emitter != NULL)
				{
					if (Emitter->LODLevels.Num() == 0)
					{
						bInvalidLOD = true;
					}
					else if (Emitter->LODLevels.Num() == 1)
					{
						bSingleLOD = true;
					}
					bFoundEmitter = true;
					int32 BoneLocationArraySize = -1;
					for (int32 LODIdx = 0; LODIdx < Emitter->LODLevels.Num(); LODIdx++)
					{
						UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
						if (LODLevel != NULL)
						{
							if (LODLevel->RequiredModule != NULL)
							{
								if (LODLevel->RequiredModule->Material == NULL)
								{
									bMissingMaterial = true;
								}
							}

							if (Cast<UParticleModuleTypeDataRibbon>(LODLevel->TypeDataModule) ||
								Cast<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule) ||
								Cast<UParticleModuleTypeDataAnimTrail>(LODLevel->TypeDataModule))
							{
								bHasRibbonTrailOrBeam = true;
							}

							if (!Cast<UParticleModuleTypeDataBeam2>(LODLevel->TypeDataModule))
							{
								bHasOnlyBeamsOrHasNoEmitters = false;
							}

							for (int32 ModuleIdx = 0; ModuleIdx < LODLevel->Modules.Num(); ModuleIdx++)
							{
								UParticleModule* Module = LODLevel->Modules[ModuleIdx];

								if (UParticleModuleSpawn* SpawnModule = Cast<UParticleModuleSpawn>(Module))
								{
									if (!bHasHighSpawnRateOrBurst)
									{
										if (UDistributionFloatConstant* ConstantDistribution = Cast<UDistributionFloatConstant>(SpawnModule->Rate.Distribution))
										{
											if (ConstantDistribution->Constant > HighSpawnRateOrBurstThreshold)
											{
												bHasHighSpawnRateOrBurst = true;
											}
										}

										for (const FParticleBurst& Burst : SpawnModule->BurstList)
										{
											if (Burst.Count > HighSpawnRateOrBurstThreshold)
											{
												bHasHighSpawnRateOrBurst = true;
											}
										}
									}
								}
								else if (Cast<UParticleModuleSpawnPerUnit>(Module) != nullptr)
								{
									bHasSpawnPerUnit = true;
								}
								else if (UParticleModuleLight* LightModule = Cast<UParticleModuleLight>(Module))
								{
									bHasLights = true;
									bHasShadowCastingLights |= LightModule->bShadowCastingLights;
									bHasHiQualityLights |= LightModule->bHighQualityLights;
								}
							}
						}
					}

					// Cross LOD validation for things where indices are per particle and used to index into module specific data for example
					if ( !bSingleLOD )
					{
						UParticleLODLevel* HighestLODLevel = Emitter->LODLevels[0];
						check(HighestLODLevel != nullptr);

						for ( int32 ModuleIdx=0; ModuleIdx < HighestLODLevel->Modules.Num(); ++ModuleIdx )
						{
							UParticleModule* HighestLODModule = HighestLODLevel->Modules[ModuleIdx];
							if ( UParticleModuleLocationBoneSocket* HighestBoneModule = Cast<UParticleModuleLocationBoneSocket>(HighestLODModule) )
							{
								bool bRequiresSourceLocationValidation = HighestBoneModule->bInheritBoneVelocity || HighestBoneModule->bUpdatePositionEachFrame;
								if (!bRequiresSourceLocationValidation )
								{
									for (int32 LODIdx = 1; LODIdx < Emitter->LODLevels.Num(); ++LODIdx)
									{
										UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
										if (LODLevel == nullptr)
											continue;

										UParticleModule* LODModule = LODLevel->Modules[ModuleIdx];
										UParticleModuleLocationBoneSocket* LODBoneModule = Cast<UParticleModuleLocationBoneSocket>(LODModule);
										if (LODBoneModule == nullptr)
										{
											UE_LOGF(LogParticleSystemAuditCommandlet, Warning, "ParticleSystem '%ls' Emitter '%ls' LOD index '%d' has incorrect module type '%ls' when should be 'UParticleModuleLocationBoneSocket'", *PSys->GetPathName(), *Emitter->GetName(), LODIdx, LODModule == nullptr ? TEXT("nullptr") : *LODModule->GetClass()->GetName());
											continue;
										}

										if (LODBoneModule->bInheritBoneVelocity || LODBoneModule->bUpdatePositionEachFrame)
										{
											bRequiresSourceLocationValidation = true;
											break;
										}
									}
								}

								if ( bRequiresSourceLocationValidation )
								{
									for (int32 LODIdx = 1; LODIdx < Emitter->LODLevels.Num(); ++LODIdx)
									{
										UParticleLODLevel* LODLevel = Emitter->LODLevels[LODIdx];
										if (LODLevel == nullptr)
											continue;

										UParticleModule* LODModule = LODLevel->Modules[ModuleIdx];
										UParticleModuleLocationBoneSocket* LODBoneModule = Cast<UParticleModuleLocationBoneSocket>(LODModule);
										if (LODBoneModule == nullptr)
											continue;

										if ( HighestBoneModule->SourceLocations.Num() != LODBoneModule->SourceLocations.Num() )
										{
											bMismatchedLODBoneModules = true;
											UE_LOGF(LogParticleSystemAuditCommandlet, Log, "ParticleSystem '%ls' Emitter '%ls' LOD index '%d' SourceLocations does not match HighestLOD, this should have been fixed on load", *PSys->GetPathName(), *Emitter->GetName(), LODIdx);
											continue;
										}

										for ( int32 SourceLocationIdx=0; SourceLocationIdx < HighestBoneModule->SourceLocations.Num(); ++SourceLocationIdx )
										{
											if ( HighestBoneModule->SourceLocations[SourceLocationIdx].BoneSocketName != LODBoneModule->SourceLocations[SourceLocationIdx].BoneSocketName)
											{
												bMismatchedLODBoneModules = true;
												UE_LOGF(LogParticleSystemAuditCommandlet, Log, "ParticleSystem '%ls' Emitter '%ls' LOD index '%d' BoneSocketNames do not match between LOD levels, this is not fatal but may be an error", *PSys->GetPathName(), *Emitter->GetName(), LODIdx);
												break;
											}
										}
									}
								}
							}
						}
					}
				}
			}

			// Note all PSystems w/ a high constant spawn rate or burst count...
			if (bHasHighSpawnRateOrBurst)
			{
				ParticleSystemsWithHighSpawnRateOrBurst.Add(PSys->GetPathName());
			}

			// Note all PSystems w/ a far LOD distance...
			bool bAtLeastOneLODUnderFarDistanceThresholdOrEmpty = (PSys->LODDistances.Num() == 0);
			for (float LODDistance : PSys->LODDistances)
			{
				if (LODDistance <= FarLODDistanceTheshold)
				{
					bAtLeastOneLODUnderFarDistanceThresholdOrEmpty = true;
					break;
				}
			}

			if (!bAtLeastOneLODUnderFarDistanceThresholdOrEmpty)
			{
				ParticleSystemsWithFarLODDistance.Add(PSys->GetPathName());
			}

			// Note all PSystems w/ no emitters...
			if (PSys->Emitters.Num() == 0)
			{
				ParticleSystemsWithNoEmitters.Add(PSys->GetPathName());
			}

			// Note all missing material case PSystems...
			if (bMissingMaterial == true)
			{
				ParticleSystemsWithMissingMaterials.Add(PSys->GetPathName());
			}

			// Note all 0 LOD case PSystems...
			if (bInvalidLOD == true)
			{
				ParticleSystemsWithNoLODs.Add(PSys->GetPathName());
			}
			// Note all single LOD case PSystems...
			if (bSingleLOD == true && !bHasOnlyBeamsOrHasNoEmitters)
			{
				ParticleSystemsWithSingleLOD.Add(PSys->GetPathName());
			}

			// Note all non-fixed bound PSystems, unless there is a ribbon, trail, or beam emitter, OR if we have a SpawnPerUnit module since it is often used in tail effects...
			if (PSys->bUseFixedRelativeBoundingBox == false && !bHasRibbonTrailOrBeam && !bHasSpawnPerUnit)
			{
				ParticleSystemsWithoutFixedBounds.Add(PSys->GetPathName());
			}

			// Note all bOrientZAxisTowardCamera systems
			if (PSys->bOrientZAxisTowardCamera == true)
			{
				ParticleSystemsWithOrientZAxisTowardCamera.Add(PSys->GetPathName());
			}

			// Note all systems with problematic bone location source arrays
			if (bMismatchedLODBoneModules)
			{
				ParticleSystemsWithBoneLocationMismatches.Add(PSys->GetPathName());
			}

			if (PSys->WarmupTime > 0.0f)
			{
				ParticleSystemsWithWarmupTime.Add(PSys->GetPathName());
			}

			if (bHasLights)
			{
				FString SystemString = FString::Printf(
					TEXT("%s,%s,%s"),
					*PSys->GetPathName(),
					bHasHiQualityLights ? TEXT("true") : TEXT("false"),
					bHasShadowCastingLights ? TEXT("true") : TEXT("false")
				);
				ParticleSystemsWithLights.Add(SystemString);
			}

			if (PSys->SystemUpdateMode == EParticleSystemUpdateMode::EPSUM_FixedTime)
			{
				ParticleSystemsWithFixedDeltaTime.Add(PSys->GetPathName());
			}

			if ((PSys->LODMethod == PARTICLESYSTEMLODMETHOD_Automatic) &&
				(bInvalidLOD == false) && (bSingleLOD == false) &&
				(PSys->LODDistanceCheckTime == 0.0f))
			{
				ParticleSystemsWithBadLODCheckTimes.Add(PSys->GetPathName());
			}

			if (LastPackageName.Len() > 0)
			{
				if (LastPackageName != PSys->GetOutermost()->GetName())
				{
					LastPackageName = PSys->GetOutermost()->GetName();
					PackageSwitches++;
				}
			}
			else
			{
				LastPackageName = PSys->GetOutermost()->GetName();
			}

			if (PackageSwitches > 10)
			{
				::CollectGarbage(RF_NoFlags);
				PackageSwitches = 0;
			}

		}
		else
		{
			UE_LOGF(LogParticleSystemAuditCommandlet, Warning, "Failed to load particle system %ls", *PSysName);
		}
	}

	// Probably don't need to do this, but just in case we have any 'hanging' packages 
	// and more processing steps are added later, let's clean up everything...
	::CollectGarbage(RF_NoFlags);

	double ProcessParticleSystemsTime = FPlatformTime::Seconds() - StartProcessParticleSystemsTime;
	UE_LOGF(LogParticleSystemAuditCommandlet, Log, "Took %5.3f seconds to process referenced particle systems...", ProcessParticleSystemsTime);

	return true;
}

/** Dump the results of the audit */
void UParticleSystemAuditCommandlet::DumpResults()
{
	// Dump all the simple mappings...
	DumpSimplePSysSet(ParticleSystemsWithNoLODs, TEXT("PSysNoLOD"));
	DumpSimplePSysSet(ParticleSystemsWithSingleLOD, TEXT("PSysSingleLOD"));
	DumpSimplePSysSet(ParticleSystemsWithoutFixedBounds, TEXT("PSysNoFixedBounds"));
	DumpSimplePSysSet(ParticleSystemsWithBadLODCheckTimes, TEXT("PSysBadLODCheckTimes"));
	DumpSimplePSysSet(ParticleSystemsWithMissingMaterials, TEXT("PSysMissingMaterial"));
	DumpSimplePSysSet(ParticleSystemsWithNoEmitters, TEXT("PSysNoEmitters"));
	DumpSimplePSysSet(ParticleSystemsWithOrientZAxisTowardCamera, TEXT("PSysOrientZTowardsCamera"));
	DumpSimplePSysSet(ParticleSystemsWithHighSpawnRateOrBurst, TEXT("PSysHighSpawnRateOrBurst"));
	DumpSimplePSysSet(ParticleSystemsWithFarLODDistance, TEXT("PSysFarLODDistance"));
	DumpSimplePSysSet(ParticleSystemsWithBoneLocationMismatches, TEXT("PSysBoneLocationLODMismatches"));
	DumpSimplePSysSet(ParticleSystemsWithWarmupTime, TEXT("PSysWithWarmupTime"));
	DumpSimplePSysSet(ParticleSystemsWithLights, TEXT("PSysWithLights"), TEXT("Name,HiQuality,ShadowCasting"));
	DumpSimplePSysSet(ParticleSystemsWithFixedDeltaTime, TEXT("PSysWithFixedDeltaTime"), TEXT("Name"));
}

/**
 *	Dump the give list of particle systems to an audit CSV file...
 *
 *	@param	InPSysMap		The particle system map to dump
 *	@param	InFilename		The name for the output file (short name)
 *
 *	@return	bool			true if successful, false if not
 */
bool UParticleSystemAuditCommandlet::DumpSimplePSysSet(TSet<FString>& InPSysSet, const TCHAR* InShortFilename, const TCHAR* OptionalHeader)
{
	return DumpSimpleSet(InPSysSet, InShortFilename, TEXT("ParticleSystem"), OptionalHeader);
}

bool UParticleSystemAuditCommandlet::DumpSimpleSet(TSet<FString>& InSet, const TCHAR* InShortFilename, const TCHAR* InObjectClassName, const TCHAR* OptionalHeader)
{
	if (InSet.Num() > 0)
	{
		check(InShortFilename != NULL);
		check(InObjectClassName != NULL);

		FArchive* OutputStream = GetOutputFile(InShortFilename);
		if (OutputStream != NULL)
		{
			UE_LOGF(LogParticleSystemAuditCommandlet, Log, "Dumping '%ls' results...", InShortFilename);
			if (OptionalHeader != nullptr)
			{
				OutputStream->Logf(TEXT("%s"), OptionalHeader);
			}
			else
			{
				OutputStream->Logf(TEXT("%s,..."), InObjectClassName);
			}
			for (TSet<FString>::TIterator DumpIt(InSet); DumpIt; ++DumpIt)
			{
				FString ObjName = *DumpIt;
				OutputStream->Logf(TEXT("%s"), *ObjName);
			}

			OutputStream->Close();
			delete OutputStream;
		}
		else
		{
			return false;
		}
	}
	return true;
}

FArchive* UParticleSystemAuditCommandlet::GetOutputFile(const TCHAR* InShortFilename)
{
	const FString Filename = FString::Printf(TEXT("%s/%s.csv"), *AuditOutputFolder, InShortFilename);
	FArchive* OutputStream = IFileManager::Get().CreateDebugFileWriter(*Filename);
	if (OutputStream == NULL)
	{
		UE_LOGF(LogParticleSystemAuditCommandlet, Warning, "Failed to create output stream %ls", *Filename);
	}
	return OutputStream;
}
