// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/PopulateDialogueWaveFromCharacterSheetCommandlet.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetData.h"
#include "Sound/SoundWave.h"
#include "Sound/DialogueWave.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "LocalizedAssetUtil.h"
#include "LocalizationSourceControlUtil.h"
#include "Serialization/Csv/CsvParser.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PopulateDialogueWaveFromCharacterSheetCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Log, All);

namespace CharacterDialogueScript
{
	const uint32 HeaderRowIndex = 12 - 1;
	const uint32 DialogLineColumn = 5 - 1;
	const uint32 VoiceInflectionColumn = 6 - 1;
	const uint32 VoicePowerColumn = 7 - 1;
	const uint32 AudioFileNameColumn = 8 - 1;
	const uint32 NotesColumn = 10 - 1;
}

int32 UPopulateDialogueWaveFromCharacterSheetCommandlet::Main(const FString& Params)
{
	// Prepare asset registry.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, Parameters);

	// Get file name from parameters.
	FString DialogTextFileName;
	{
		FString* const DialogTextFileParameterValue = Parameters.Find(TEXT("DialogTextFile"));
		if (!DialogTextFileParameterValue)
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Missing argument DialogTextFile.");
			return -1;
		}
		DialogTextFileName = *DialogTextFileParameterValue;
	}

	// Load file contents to string.
	FString DialogTextFileContents;
	if (!FFileHelper::LoadFileToString(DialogTextFileContents, *DialogTextFileName))
	{
		UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Failed to load contents of dialog text file (%ls).", *DialogTextFileName);
		return -1;
	}

	UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Log, "Using dialog text file (%ls).", *DialogTextFileName);

	TSharedPtr<FLocalizationSCC> SourceControlInfo;
	const bool bEnableSourceControl = Switches.Contains(TEXT("EnableSCC"));
	if (bEnableSourceControl)
	{
		SourceControlInfo = MakeShareable(new FLocalizationSCC());

		FText SCCErrorStr;
		if (!SourceControlInfo->IsReady(SCCErrorStr))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Revision Control error: %ls", *SCCErrorStr.ToString());
			return -1;
		}
	}

	// Parse file contents.
	FCsvParser DialogTextFileParser(DialogTextFileContents);
	const FCsvParser::FRows& Rows = DialogTextFileParser.GetRows();

	// Validate row count.
	if (!Rows.IsValidIndex(CharacterDialogueScript::HeaderRowIndex))
	{
		UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file has insufficient rows to be properly formed, expected at least %u rows.", CharacterDialogueScript::HeaderRowIndex + 1);
		return -1;
	}

	// Validate header row?
	// Character,Section Type,Category,Dialog Line (English),Voice Inflection,Voice Power,Audio File Name,Heard By,Notes,Video Name,Status

	// We only want dialogue wave assets that exist within the Game content directory.
	TArray<FAssetData> AllDialogueWaves;
	if (!FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, FName("/Game"), UDialogueWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/false, AllDialogueWaves))
	{
		UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Unable to get dialogue wave asset data from asset registry.");
		return -1;
	}

	// We only want sound wave assets that exist within the Game content directory.
	TArray<FAssetData> AllSoundWaves;
	if (!FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, FName("/Game"), USoundWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/false, AllSoundWaves))
	{
		UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Unable to get sound wave asset data from asset registry.");
		return -1;
	}

	// Iterate over rows of dialogue data.
	for (int32 i = CharacterDialogueScript::HeaderRowIndex + 1; i < Rows.Num(); ++i)
	{
		const TArray<const TCHAR*>& ColumnsInRow = Rows[i];

		// Validate Dialog Line column.
		if (!ColumnsInRow.IsValidIndex(CharacterDialogueScript::DialogLineColumn))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file row (%d) has insufficient columns to be properly formed, missing dialog file name column (%u).", i, CharacterDialogueScript::DialogLineColumn + 1);
			continue;
		}

		// Validate Voice Inflection column.
		if (!ColumnsInRow.IsValidIndex(CharacterDialogueScript::VoiceInflectionColumn))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file row (%d) has insufficient columns to be properly formed, missing dialog file name column (%u).", i, CharacterDialogueScript::VoiceInflectionColumn + 1);
			continue;
		}

		// Validate Voice Power column.
		if (!ColumnsInRow.IsValidIndex(CharacterDialogueScript::VoicePowerColumn))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file row (%d) has insufficient columns to be properly formed, missing dialog file name column (%u).", i, CharacterDialogueScript::VoicePowerColumn + 1);
			continue;
		}

		// Validate Audio File Name column.
		if (!ColumnsInRow.IsValidIndex(CharacterDialogueScript::AudioFileNameColumn))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file row (%d) has insufficient columns to be properly formed, missing audio file name column (%u).", i, CharacterDialogueScript::AudioFileNameColumn + 1);
			continue;
		}

		// Validate Notes column.
		if (!ColumnsInRow.IsValidIndex(CharacterDialogueScript::NotesColumn))
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file row (%d) has insufficient columns to be properly formed, missing dialog file name column (%u).", i, CharacterDialogueScript::NotesColumn + 1);
			continue;
		}

		// Find reference audio file by its file name.
		const FName AudioFileName = ColumnsInRow[CharacterDialogueScript::AudioFileNameColumn];
		TArray<FAssetData> AssetDataArrayForPackageName = AllDialogueWaves.FilterByPredicate([&](const FAssetData& AssetData) -> bool
		{
			return AssetData.AssetName == AudioFileName;
		});

		// If we couldn't find a dialogue wave with this name, try searching for a sound wave instead.
		if (AssetDataArrayForPackageName.Num() == 0)
		{
			TArray<FAssetData> SoundWaveArrayForPackageName = AllSoundWaves.FilterByPredicate([&](const FAssetData& AssetData) -> bool
			{
				return AssetData.AssetName == AudioFileName;
			});

			// If we found a single sound wave with this name, try and find an auto-converted dialogue wave that's using it.
			if (SoundWaveArrayForPackageName.Num() == 1)
			{
				const FName AutoConvertedAudioFileName = *(AudioFileName.ToString() + TEXT("_Dialogue"));
				TArray<FAssetData> DialogueWaveArrayForAutoConvertedPackageName = AllDialogueWaves.FilterByPredicate([&](const FAssetData& AssetData) -> bool
				{
					return AssetData.AssetName == AutoConvertedAudioFileName;
				});

				if (DialogueWaveArrayForAutoConvertedPackageName.Num() == 1)
				{
					UDialogueWave* const AutoConvertedDialogueWave = Cast<UDialogueWave>(DialogueWaveArrayForAutoConvertedPackageName[0].GetAsset());
					if (AutoConvertedDialogueWave)
					{
						const bool bIsValidDialogueWave = AutoConvertedDialogueWave->ContextMappings.ContainsByPredicate([&](const FDialogueContextMapping& Mapping) -> bool
						{
							return Mapping.SoundWave && Mapping.SoundWave->GetFName() == AudioFileName;
						});

						if (bIsValidDialogueWave)
						{
							AssetDataArrayForPackageName = DialogueWaveArrayForAutoConvertedPackageName;
						}
					}
				}
			}
		}

		// Verify that the number of matching dialogue waves is singular.
		if (AssetDataArrayForPackageName.Num() != 1)
		{
			if (AssetDataArrayForPackageName.Num() < 1)
			{
				UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file references absent audio file name (%ls) at (%d, %d). No dialogue waves use this name.", *AudioFileName.ToString(), i + 1, CharacterDialogueScript::AudioFileNameColumn + 1);
			}
			if (AssetDataArrayForPackageName.Num() > 1)
			{
				UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file references ambiguous audio file name (%ls) at (%d, %d). Multiple dialogue waves use this name.", *AudioFileName.ToString(), i + 1, CharacterDialogueScript::AudioFileNameColumn + 1);
			}
			continue;
		}

		const FAssetData& AssetData = AssetDataArrayForPackageName[0];

		// Verify that the found asset is a dialogue wave.
		if (AssetData.GetClass() != UDialogueWave::StaticClass())
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file references audio file name (%ls) at (%u, %u), but the asset with this name is not actually a dialogue wave.", *AudioFileName.ToString(), i + 1, CharacterDialogueScript::AudioFileNameColumn + 1);
			continue;
		}

		// Get the dialogue wave to populate with subtitles.
		UDialogueWave* const DialogueWave = Cast<UDialogueWave>(AssetData.GetAsset());

		// Verify that the dialogue wave was loaded.
		if (!DialogueWave)
		{
			UE_LOGF(LogPopulateDialogueWaveFromCharacterSheetCommandlet, Error, "Dialog text file references audio file name (%ls) at (%u, %u), but the dialogue wave could not be accessed.", *AudioFileName.ToString(), i + 1, CharacterDialogueScript::AudioFileNameColumn + 1);
			continue;
		}

		bool bHasChanged = false;

		// Set dialogue wave spoken text to dialogue line if not identical.
		{
			const TCHAR* const DialogueLine = ColumnsInRow[CharacterDialogueScript::DialogLineColumn];
			if (!DialogueWave->SpokenText.Equals(DialogueLine, ESearchCase::CaseSensitive))
			{
				bHasChanged = true;
				DialogueWave->SpokenText = DialogueLine;
				DialogueWave->MarkPackageDirty();
			}
		}

		// Set voice actor notes to the inflection, power, and notes if not identical.
		{
			const TCHAR* const VoiceInflection = ColumnsInRow[CharacterDialogueScript::VoiceInflectionColumn];
			const TCHAR* const VoicePower = ColumnsInRow[CharacterDialogueScript::VoicePowerColumn];
			const TCHAR* const Notes = ColumnsInRow[CharacterDialogueScript::NotesColumn];

			FString VoiceActorDirection = VoiceInflection;
			if (FCString::Strlen(VoicePower) > 0)
			{
				if (VoiceActorDirection.Len() > 0)
				{
					VoiceActorDirection += TEXT(". ");
				}
				VoiceActorDirection += VoicePower;
			}
			if (FCString::Strlen(Notes) > 0)
			{
				if (VoiceActorDirection.Len() > 0)
				{
					VoiceActorDirection += TEXT(". ");
				}
				VoiceActorDirection += Notes;
			}
			if (VoiceActorDirection.Len() > 0 && !VoiceActorDirection.EndsWith(TEXT(".")))
			{
				VoiceActorDirection += TEXT(".");
			}

			if (!DialogueWave->VoiceActorDirection.Equals(VoiceActorDirection, ESearchCase::CaseSensitive))
			{
				bHasChanged = true;
				DialogueWave->VoiceActorDirection = VoiceActorDirection;
				DialogueWave->MarkPackageDirty();
			}
		}

		if (!bHasChanged)
		{
			continue;
		}

		// Save package for dialogue wave.
		if (!FLocalizedAssetSCCUtil::SaveAssetWithSCC(SourceControlInfo, DialogueWave))
		{
			continue;
		}
	}

	return 0;
}
