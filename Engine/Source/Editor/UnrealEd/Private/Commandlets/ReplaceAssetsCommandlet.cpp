// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReplaceAssetsCommandlet.cpp: Commandlet for replacing assets with those
	from another location (intended use is replacing with cooked assets)
=============================================================================*/

#include "Commandlets/ReplaceAssetsCommandlet.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "AssetRegistry/ARFilter.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReplaceAssetsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogReplaceAssetsCommandlet, Log, All);

UReplaceAssetsCommandlet::UReplaceAssetsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

int32 UReplaceAssetsCommandlet::Main(const FString& InParams)
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*InParams, Tokens, Switches);

	// Support standard and BuildGraph style delimeters
	static const TCHAR* ParamDelims[] =
	{
		TEXT(";"),
		TEXT("+"),
	};

	const FString AssetSourcePathSwitch = TEXT("AssetSourcePath=");
	const FString ReplacedPathsSwitch = TEXT("ReplacedPaths=");
	const FString ReplacedClassesSwitch = TEXT("ReplacedClasses=");
	const FString ExcludedPathsSwitch = TEXT("ExcludedPaths=");
	const FString ExcludedClassesSwitch = TEXT("ExcludedClasses=");
	FString AssetSourcePath;
	TArray<FString> ReplacedPaths;
	TArray<FString> ReplacedClasses;
	TArray<FString> ExcludedPaths;
	TArray<FString> ExcludedClasses;

	// Parse parameters
	for (int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); ++SwitchIdx)
	{
		const FString& Switch = Switches[SwitchIdx];
		FString SwitchValue;
		if (FParse::Value(*Switch, *AssetSourcePathSwitch, SwitchValue))
		{
			AssetSourcePath = SwitchValue;
		}
		else if (FParse::Value(*Switch, *ReplacedPathsSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ReplacedPaths, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ReplacedClassesSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ReplacedClasses, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ExcludedPathsSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ExcludedPaths, ParamDelims, 2);
		}
		else if (FParse::Value(*Switch, *ExcludedClassesSwitch, SwitchValue))
		{
			SwitchValue.ParseIntoArray(ExcludedClasses, ParamDelims, 2);
		}
	}

	// Check that replacement asset folder exists
	bool bHasReplacementAssets = !AssetSourcePath.IsEmpty() && IFileManager::Get().DirectoryExists(*AssetSourcePath);
	if (!bHasReplacementAssets)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Error, "Source Path for replacement assets does not exist - please specify a valid location for -AssetSourcePath on the commandline");
		return 1;
	}

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Update Registry Module
	UE_LOGF(LogReplaceAssetsCommandlet, Display, "Searching Asset Registry");
	AssetRegistryModule.Get().SearchAllAssets(true);

	TArray<FAssetData> FinalAssetList;

	// Get assets from paths and classes that we want to replace
	if (ReplacedPaths.Num() > 0)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Getting Assets from specified paths");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.bRecursivePaths = true;
		for (const FString& ReplacedPath : ReplacedPaths)
		{
			Filter.PackagePaths.AddUnique(*ReplacedPath);
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		for (FAssetData& Asset : AssetList)
		{
			FinalAssetList.AddUnique(Asset);
		}
	}
	if (ReplacedClasses.Num() > 0)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Getting Assets of specified classes");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths.Add(TEXT("/Game"));
		Filter.bRecursivePaths = true;
		for (const FString& ReplacedClass : ReplacedClasses)
		{
			FTopLevelAssetPath ReplacedClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ReplacedClass, ELogVerbosity::Error, TEXT("UReplaceAssetsCommandlet"));
			if (ReplacedClassPathName.IsNull())
			{
				UE_LOGF(LogReplaceAssetsCommandlet, Error, "Failed to convert short class name \"%ls\" to path name. Please use class path names for ReplacedClasses.", *ReplacedClass);
			}
			else
			{
				Filter.ClassPaths.AddUnique(ReplacedClassPathName);
			}
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		for (FAssetData& Asset : AssetList)
		{
			FinalAssetList.AddUnique(Asset);
		}
	}

	// Run through paths and classes that should be excluded
	if (FinalAssetList.Num() > 0 && ExcludedPaths.Num() > 0)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Excluding Assets from specified paths");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.bRecursivePaths = true;
		for (const FString& ExcludedPath : ExcludedPaths)
		{
			Filter.PackagePaths.AddUnique(*ExcludedPath);
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		FinalAssetList.RemoveAll([&AssetList](const FAssetData& Asset) {return AssetList.Contains(Asset); });
	}
	if (FinalAssetList.Num() > 0 && ExcludedClasses.Num() > 0)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Excluding Assets of specified classes");
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.PackagePaths.Add(TEXT("/Game"));
		Filter.bRecursivePaths = true;
		for (const FString& ExcludedClass : ExcludedClasses)
		{
			FTopLevelAssetPath ExcludedClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ExcludedClass, ELogVerbosity::Error, TEXT("UReplaceAssetsCommandlet"));
			if (ExcludedClassPathName.IsNull())
			{
				UE_LOGF(LogReplaceAssetsCommandlet, Error, "Failed to convert short class name \"%ls\" to path name. Please use class path names for ExcludedClasses.", *ExcludedClass);
			}
			else
			{
				Filter.ClassPaths.AddUnique(ExcludedClassPathName);
			}
		}
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);
		FinalAssetList.RemoveAll([&AssetList](const FAssetData& Asset) {return AssetList.Contains(Asset); });
	}

	TArray<FString> FinalFileList;
	if (FinalAssetList.Num() > 0)
	{
		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Converting Package Names to File Paths");
		for (FAssetData& RemovedAsset : FinalAssetList)
		{
			bool bIsMap = RemovedAsset.AssetClassPath == UWorld::StaticClass()->GetClassPathName();
			FinalFileList.AddUnique(FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(RemovedAsset.PackageName.ToString(), bIsMap ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension())));
		}

		UE_LOGF(LogReplaceAssetsCommandlet, Display, "Replacing files...");
		for (const FString& ReplacedPath : FinalFileList)
		{
			UE_LOGF(LogReplaceAssetsCommandlet, Log, "Replacing asset: %ls", *ReplacedPath);
			if (!IFileManager::Get().Delete(*ReplacedPath, false, true))
			{
				UE_LOGF(LogReplaceAssetsCommandlet, Error, "Failed to delete asset: %ls", *ReplacedPath);
			}
			FString ReplacementPath = ReplacedPath;
			FPaths::MakePathRelativeTo(ReplacementPath, *FPaths::RootDir());
			ReplacementPath = FPaths::Combine(AssetSourcePath, ReplacementPath);
			if (IFileManager::Get().FileExists(*ReplacementPath))
			{
				if (IFileManager::Get().Copy(*ReplacedPath, *ReplacementPath) != COPY_OK)
				{
					UE_LOGF(LogReplaceAssetsCommandlet, Error, "Failed to copy asset: %ls", *ReplacementPath);
				}
			}
		}
	}

	return 0;
}
