// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/SavePackageUtilitiesCommandlet.h"

#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SavePackageUtilitiesCommandlet)

USavePackageUtilitiesCommandlet::USavePackageUtilitiesCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 USavePackageUtilitiesCommandlet::Main(const FString& Params)
{
	UE_LOGF(LogCore, Error, "SavePackageUtilitiesCommandlet is no longer needed. Contact Epic if you need this functionality.");
	return 0;
}
