// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ShaderCodeLibraryToolsCommandlet.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "ShaderCodeLibrary.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShaderCodeLibraryToolsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogShaderCodeLibraryTools, Log, All);

UShaderCodeLibraryToolsCommandlet::UShaderCodeLibraryToolsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UShaderCodeLibraryToolsCommandlet::Main(const FString& Params)
{
	return StaticMain(Params);
}

int32 UShaderCodeLibraryToolsCommandlet::StaticMain(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	if (Tokens.Num() >= 3)
	{
		FString const& Left = Tokens[0];
		FString const& Right = Tokens[1];
		FString const& Output = Tokens[2];
		bool bNativeFormat = Switches.Contains(TEXT("PreferNativeArchives")) || Switches.Contains(TEXT("-PreferNativeArchives"));
		
		bool bArchive = false;
		if (GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bSharedMaterialNativeLibraries"), bArchive, GGameIni))
		{
			bNativeFormat |= bArchive;
		}

		bool bNeedsDeterministicOrder = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bDeterministicShaderCodeOrder"), bNeedsDeterministicOrder, GGameIni);
		
		TArray<FString> OldMetaDataDirs;
		OldMetaDataDirs.Add(Left);
		
		return FShaderLibraryCooker::CreatePatchLibrary(OldMetaDataDirs, Right, Output, bNativeFormat, bNeedsDeterministicOrder) ? 0 : 1;
	}
	
	UE_LOGF(LogShaderCodeLibraryTools, Warning, "Usage: <Path-To-Old-MetaData> <Path-To-New-MetaData> <Output-Path> [-PreferNativeArchives]\n");
	return 0;
}
