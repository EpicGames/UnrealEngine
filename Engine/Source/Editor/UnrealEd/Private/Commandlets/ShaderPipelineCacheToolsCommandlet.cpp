// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/ShaderPipelineCacheToolsCommandlet.h"

#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/StringConv.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Compression.h"
#include "Misc/Crc.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "PipelineCacheUtilities.h"
#include "PipelineFileCache.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCodeLibrary.h"
#include "ShaderPipelineCache.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "ShaderCodeArchive.h"
#include "Serialization/LargeMemoryReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ShaderPipelineCacheToolsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogShaderPipelineCacheTools, Log, All);

const TCHAR* STABLE_CSV_EXT = TEXT("stablepc.csv");
const TCHAR* STABLE_CSV_COMPRESSED_EXT = TEXT("stablepc.csv.compressed");
const TCHAR* STABLE_COMPRESSED_EXT = TEXT(".compressed");
const int32  STABLE_COMPRESSED_EXT_LEN = 11; // len of ".compressed";
const int32  STABLE_COMPRESSED_VER = 3;
const int64  STABLE_MAX_CHUNK_SIZE = MAX_int32 - 100 * 1024 * 1024;
const TCHAR* ShaderStableKeysFileExt = TEXT("shk");
const TCHAR* ShaderStableKeysFileExtWildcard = TEXT("*.shk");
const TCHAR* ShaderStablePipelineFileExt = TEXT("spc");
const TCHAR* ShaderStablePipelineFileExtWildcard = TEXT("*.spc");

int32 GShaderPipelineCacheTools_ComputePSOInclusionMode = 2;
static FAutoConsoleVariableRef CVarShaderPipelineCacheDoNotPrecompileComputePSO(
	TEXT("r.ShaderPipelineCacheTools.IncludeComputePSODuringCook"),
	GShaderPipelineCacheTools_ComputePSOInclusionMode,
	TEXT("0 disables cook-time addition, 1 enables cook-time addition, 2 adds only Niagara PSOs."),
	ECVF_Default
);

int32 GShaderPipelineCacheTools_IgnoreObsoleteStableCacheFiles = 0;
static FAutoConsoleVariableRef CVarShaderPipelineCacheIgnoreObsoleteStableCacheFiles(
	TEXT("r.ShaderPipelineCacheTools.IgnoreObsoleteStableCacheFiles"),
	GShaderPipelineCacheTools_IgnoreObsoleteStableCacheFiles,
	TEXT("When set to the default value of 0, building the cache (and usually the whole cook) will fail if any .spc file can't be loaded, to prevent further testing.\n")
	TEXT("By setting to 1, a project may choose to ignore this instead (warning will still be issued)."),
	ECVF_Default
);


struct FSCDataChunk
{
	FSCDataChunk() : UncomressedOutputLines(), OutputLinesAr(UncomressedOutputLines) {}

	TArray<uint8> UncomressedOutputLines;
	FMemoryWriter OutputLinesAr;
};


void ExpandWildcards(TArray<FString>& Parts)
{
	TArray<FString> NewParts;
	for (const FString& OldPart : Parts)
	{
		if (OldPart.Contains(TEXT("*")) || OldPart.Contains(TEXT("?")))
		{
			FString CleanPath = FPaths::GetPath(OldPart);
			FString CleanFilename = FPaths::GetCleanFilename(OldPart);
			
			TArray<FString> ExpandedFiles;
			IFileManager::Get().FindFilesRecursive(ExpandedFiles, *CleanPath, *CleanFilename, true, false);
			
			if (CleanFilename.EndsWith(STABLE_CSV_EXT))
			{
				// look for stablepc.csv.compressed as well
				CleanFilename.Append(STABLE_COMPRESSED_EXT);
				IFileManager::Get().FindFilesRecursive(ExpandedFiles, *CleanPath, *CleanFilename, true, false, false);
			}
			
			UE_CLOGF(!ExpandedFiles.Num(), LogShaderPipelineCacheTools, Log, "Expanding %ls....did not match anything.", *OldPart);
			UE_CLOGF(ExpandedFiles.Num(), LogShaderPipelineCacheTools, Log, "Expanding matched %4d files: %ls", ExpandedFiles.Num(), *OldPart);
			for (const FString& Item : ExpandedFiles)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Log, "                             : %ls", *Item);
				NewParts.Add(Item);
			}
		}
		else
		{
			NewParts.Add(OldPart);
		}
	}
	Parts = NewParts;
}

static void LoadStableShaderKeys(TArray<FStableShaderKeyAndValue>& StableArray, const FStringView& FileName)
{
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loading %.*ls...", FileName.Len(), FileName.GetData());

	const int32 StableArrayOffset = StableArray.Num();

	if (!UE::PipelineCacheUtilities::LoadStableKeysFile(FileName, StableArray))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Could not load stable shader keys from %.*ls.", FileName.Len(), FileName.GetData());
	}

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d shader info lines from %.*ls.", (StableArray.Num() - StableArrayOffset), FileName.Len(), FileName.GetData());
}

static void LoadStableShaderKeysMultiple(FStableShaderSet& StableMap, TArrayView<const FStringView> FileNames)
{
	// This is now available in PipelineCacheUtilities
	UE::PipelineCacheUtilities::LoadMultipleStableKeysFiles(FileNames, StableMap);
}

// Version optimized for ExpandPSOSC
static void LoadStableShaderKeysMultiple(TMultiMap<int32, FShaderHash>& StableMap, TArray<FStableShaderKeyAndValue>& StableShaderKeyIndexTable, TArrayView<const FStringView> FileNames)
{
	TArray<TArray<FStableShaderKeyAndValue>> StableArrays;
	StableArrays.AddDefaulted(FileNames.Num());
	ParallelFor(FileNames.Num(), [&StableArrays, &FileNames](int32 Index) { LoadStableShaderKeys(StableArrays[Index], FileNames[Index]); });

	const int32 StableArrayCount = Algo::TransformAccumulate(StableArrays, &TArray<FStableShaderKeyAndValue>::Num, 0);
	StableMap.Reserve(StableMap.Num() + StableArrayCount);
	for (const TArray<FStableShaderKeyAndValue>& StableArray : StableArrays)
	{
		for (const FStableShaderKeyAndValue& Item : StableArray)
		{
			int32 ItemIndex = StableShaderKeyIndexTable.Add(Item);
			StableMap.AddUnique(ItemIndex, Item.OutputHash);
		}
	}
}

static bool LoadAndDecompressStableCSV(const FString& Filename, TArray<FString>& OutputLines)
{
	bool bResult = false;
	FArchive* Ar = IFileManager::Get().CreateFileReader(*Filename);
	if (Ar)
	{
		if (Ar->TotalSize() > 8)
		{
			int32 CompressedVersion = 0;
			int32 NumChunks = 1;

			Ar->Serialize(&CompressedVersion, sizeof(int32));
			if (CompressedVersion >= STABLE_COMPRESSED_VER)
			{
				Ar->Serialize(&NumChunks, sizeof(int32));

				for (int32 Index = 0; Index < NumChunks; ++Index)
				{
					int32 UncompressedSize = 0;
					int32 CompressedSize = 0;

					Ar->Serialize(&UncompressedSize, sizeof(int32));
					Ar->Serialize(&CompressedSize, sizeof(int32));

					TArray<uint8> CompressedData;
					CompressedData.SetNumUninitialized(CompressedSize);
					Ar->Serialize(CompressedData.GetData(), CompressedSize);

					TArray<uint8> UncompressedData;
					UncompressedData.SetNumUninitialized(UncompressedSize);
					bResult = FCompression::UncompressMemory(NAME_Zlib, UncompressedData.GetData(), UncompressedSize, CompressedData.GetData(), CompressedSize);
					if (!bResult)
					{
						UE_LOGF(LogShaderPipelineCacheTools, Display, "Failed to decompress file %ls", *Filename);
					}

					FMemoryReader MemArchive(UncompressedData);
					FString LineCSV;
					while (!MemArchive.AtEnd())
					{
						MemArchive << LineCSV;
						OutputLines.Add(LineCSV);
					}
				}
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Warning, "File %ls is too old (version %d, we need at least %d), rejecting.", *Filename, CompressedVersion, STABLE_COMPRESSED_VER);			
			}
		}
		else
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Corrupted file %ls", *Filename);
		}

		delete Ar;
	}
	else
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "Failed to open file %ls", *Filename);
	}

	return bResult;
}

static void ReadStableCSV(const TArray<FString>& CSVLines, const TFunctionRef<void(FStringView)>& LineVisitor)
{
	for (const FString& LineCSV : CSVLines)
	{
		LineVisitor(LineCSV);
	}
}

static bool LoadStableCSV(const FString& Filename, TArray<FString>& OutputLines)
{
	bool bResult = false;
	if (Filename.EndsWith(STABLE_CSV_COMPRESSED_EXT))
	{
		if (LoadAndDecompressStableCSV(Filename, OutputLines))
		{
			bResult = true;
		}
	}
	else
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "Uncompressed CSV files are no longer supported, rejecting %ls.", *Filename);
	}

	return bResult;
}

static int64 SaveStableCSV(const FString& Filename, const FSCDataChunk* DataChunks, int32 NumChunks)
{
	if (Filename.EndsWith(STABLE_CSV_COMPRESSED_EXT))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "Compressing output, %d chunks", NumChunks);

		struct FSCCompressedChunk
		{
			FSCCompressedChunk(int32 UncompressedSize)
			{
				CompressedSize = FCompression::CompressMemoryBound(NAME_Zlib, UncompressedSize);
				CompressedData.SetNumZeroed(CompressedSize);
			}

			TArray<uint8> CompressedData;
			int32 CompressedSize;
		};

		TArray<FSCCompressedChunk> CompressedChunks;

		for (int32 Index = 0; Index < NumChunks; ++Index)
		{
			const FSCDataChunk& Chunk = DataChunks[Index];
			CompressedChunks.Add(FSCCompressedChunk(Chunk.UncomressedOutputLines.Num()));

			UE_LOGF(LogShaderPipelineCacheTools, Display, "Compressing chunk %d, size = %.1fKB", Index, Chunk.UncomressedOutputLines.Num() / 1024.f);
			if (FCompression::CompressMemory(NAME_Zlib, CompressedChunks[Index].CompressedData.GetData(), CompressedChunks[Index].CompressedSize, Chunk.UncomressedOutputLines.GetData(), Chunk.UncomressedOutputLines.Num()) == false)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Failed to compress chunk %d (%.1f KB)", Index, Chunk.UncomressedOutputLines.Num() / 1024.f);
			}
		}

		FArchive* Ar = IFileManager::Get().CreateFileWriter(*Filename);
		if (!Ar)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Failed to open %ls", *Filename);
			return -1;
		}

		int32 CompressedVersion = STABLE_COMPRESSED_VER;

		Ar->Serialize(&CompressedVersion, sizeof(int32));
		Ar->Serialize(&NumChunks, sizeof(int32));

		for (int32 Index = 0; Index < NumChunks; ++Index)
		{
			int32 UncompressedSize = DataChunks[Index].UncomressedOutputLines.Num();
			int32 CompressedSize = CompressedChunks[Index].CompressedSize;
			Ar->Serialize(&UncompressedSize, sizeof(int32));
			Ar->Serialize(&CompressedSize, sizeof(int32));
			Ar->Serialize(CompressedChunks[Index].CompressedData.GetData(), CompressedSize);
		}

		delete Ar;
	}
	else
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "SaveStableCSV does not support saving uncompressed files.");
	}

	int64 Size = IFileManager::Get().FileSize(*Filename);
	if (Size < 1)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Failed to write %ls", *Filename);
	}

	return Size;
}

static void PrintShaders(const TMap<FShaderHash, TArray<FString>>& InverseMap, const FShaderHash& Shader)
{
	if (Shader == FShaderHash())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    null");
		return;
	}
	const TArray<FString>* Out = InverseMap.Find(Shader);
	if (!Out)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    No shaders found with hash %ls", *Shader.ToString());
		return;
	}

	for (const FString& Item : *Out)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *Item);
	}
}

bool CheckPSOStringInveribility(const FPipelineCacheFileFormatPSO& Item)
{
	FPipelineCacheFileFormatPSO TempItem(Item);

	FString StringRep;
	switch (Item.Type)
	{
	case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		StringRep = TempItem.ComputeDesc.ToString();
		break;
	case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		StringRep = TempItem.GraphicsDesc.ToString();
		break;
	case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		StringRep = TempItem.RayTracingDesc.ToString();
		break;
	default:
		return false;
	}

	FPipelineCacheFileFormatPSO DupItem;
	FMemory::Memzero(DupItem.GraphicsDesc);
	DupItem.Type = Item.Type;
	DupItem.UsageMask = Item.UsageMask;

	switch (Item.Type)
	{
	case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		DupItem.ComputeDesc.FromString(StringRep);
		break;
	case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		DupItem.GraphicsDesc.FromString(StringRep);
		break;
	case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		DupItem.RayTracingDesc.FromString(StringRep);
		break;
	default:
		return false;
	}

	UE_LOGF(LogShaderPipelineCacheTools, Verbose, "CheckPSOStringInveribility: %ls", *StringRep);

	return (DupItem == TempItem) && (GetTypeHash(DupItem) == GetTypeHash(TempItem));
}

int32 DumpPSOSC(FString& Token, const FString& StableKeyFileDir)
{
	TMap<FShaderHash, TArray<FString>> InverseMap;
	if (StableKeyFileDir.Len())
	{
		TArray<FString> StableKeyFiles;
		IFileManager::Get().FindFilesRecursive(StableKeyFiles, *StableKeyFileDir, ShaderStableKeysFileExtWildcard, true, false);
		TArray<FStringView, TInlineAllocator<16>> StableKeyFilesViews;
		for (const FString& StableKeyFile : StableKeyFiles)
		{
			StableKeyFilesViews.Add(StableKeyFile);
		}

		FStableShaderSet StableMap;
		LoadStableShaderKeysMultiple(StableMap, StableKeyFilesViews);
		for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)
		{
			InverseMap.FindOrAdd(ShaderKeyPtr->OutputHash).Add(ShaderKeyPtr->ToString());
		}
	}

	TSet<FPipelineCacheFileFormatPSO> PSOs;

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loading %ls....", *Token);
	if (!FPipelineFileCacheManager::LoadPipelineFileCacheInto(Token, PSOs))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Could not load %ls or it was empty.", *Token);
		return 1;
	}

	int32 Count = 0;
	for (FPipelineCacheFileFormatPSO& Item : PSOs)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "--- Entry %d --------------------------------", Count);
		FString ReadablePSODesc = Item.ToStringReadable();
		UE_LOGF(LogShaderPipelineCacheTools, Display, "%ls", *ReadablePSODesc);

		if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
		{
			check(!(Item.ComputeDesc.ComputeShader == FShaderHash()));
			UE_LOGF(LogShaderPipelineCacheTools, Display, "%ls", *Item.ComputeDesc.ToString());
		}
		else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			check(Item.GraphicsDesc.VertexShader != FShaderHash() || Item.GraphicsDesc.MeshShader != FShaderHash());
			UE_LOGF(LogShaderPipelineCacheTools, Display, "%ls", *Item.GraphicsDesc.ToString());

			if (InverseMap.Num())
			{
				UE_LOGF(LogShaderPipelineCacheTools, Display, "VertexShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.VertexShader);
				UE_LOGF(LogShaderPipelineCacheTools, Display, "FragmentShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.FragmentShader);
				UE_LOGF(LogShaderPipelineCacheTools, Display, "GeometryShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.GeometryShader);
			}
		}
		else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "%ls", *Item.RayTracingDesc.ToString());
		}
		else
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.Type));
		}
		++Count;
	}
	UE_LOGF(LogShaderPipelineCacheTools, Display, "%ls", *FPipelineCacheFileFormatPSO::GraphicsDescriptor::HeaderLine());
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Total PSOs logged: %d", PSOs.Num());

	for (const FPipelineCacheFileFormatPSO& Item : PSOs)
	{
		CheckPSOStringInveribility(Item);
	}

	return 0;
}

static void PrintShaders(const TMap<FShaderHash, TArray<int32>>& InverseMap, TArray<FStableShaderKeyAndValue>& StableArray, const FShaderHash& Shader, const TCHAR *Label)
{
	UE_LOGF(LogShaderPipelineCacheTools, Display, " -- %ls", Label);

	if (Shader == FShaderHash())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    null");
		return;
	}
	const TArray<int32>* Out = InverseMap.Find(Shader);
	if (!Out)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    No shaders found with hash %ls", *Shader.ToString());
		return;
	}
	for (const int32& Item : *Out)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *StableArray[Item].ToString());
	}
}

static bool GetStableShaders(const TMap<FShaderHash, TArray<int32>>& InverseMap, TArray<FStableShaderKeyAndValue>& StableArray, const FShaderHash& Shader, TArray<int32>& StableShaders, bool& bOutAnyActiveButMissing)
{
	if (Shader == FShaderHash())
	{
		return false;
	}
	const TArray<int32>* Out = InverseMap.Find(Shader);
	if (!Out)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "No shaders found with hash %ls", *Shader.ToString());
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "If you can find the old %ls file for this build, adding it will allow these PSOs to be usable.", ShaderStableKeysFileExt);
		bOutAnyActiveButMissing = true;
		return false;
	}
	StableShaders.Reserve(Out->Num());
	for (const int32& Item : *Out)
	{
		if (StableShaders.Contains(Item))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Duplicate stable shader. This is bad because it means our stable key is not exhaustive.");
			UE_LOGF(LogShaderPipelineCacheTools, Error, " %ls", *StableArray[Item].ToString());
			continue;
		}
		StableShaders.Add(Item);
	}
	return true;
}

static void StableShadersSerializationSelfTest(const FStableShaderSet& StableMap)
{
	TAnsiStringBuilder<384> TestString;
	for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)
	{
		TestString.Reset();
		check(ShaderKeyPtr->OutputHash != FShaderHash());
		ShaderKeyPtr->AppendString(TestString);
		FStableShaderKeyAndValue TestItem;
		TestItem.ParseFromString(UTF8_TO_TCHAR(TestString.ToString()));
		check(*ShaderKeyPtr == TestItem);
		check(GetTypeHash(*ShaderKeyPtr) == GetTypeHash(TestItem));
		check(ShaderKeyPtr->OutputHash == TestItem.OutputHash);
	}
}

// Version optimized for ExpandPSOSC
static void StableShadersSerializationSelfTest(const TMultiMap<int32, FShaderHash>& StableMap, const TArray<FStableShaderKeyAndValue>& StableArray)
{
	TAnsiStringBuilder<384> TestString;
	for (const auto& Pair : StableMap)
	{
		TestString.Reset();
		FStableShaderKeyAndValue Item(StableArray[Pair.Key]);
		Item.OutputHash = Pair.Value;
		check(Pair.Value != FShaderHash());
		Item.AppendString(TestString);
		FStableShaderKeyAndValue TestItem;
		TestItem.ParseFromString(UTF8_TO_TCHAR(TestString.ToString()));
		check(Item == TestItem);
		check(GetTypeHash(Item) == GetTypeHash(TestItem));
		check(Item.OutputHash == TestItem.OutputHash);
	}
}

// return true if these two shaders could be part of the same stable PSO
// for example, if they come from two different vertex factories, we return false because that situation cannot occur
bool CouldBeUsedTogether(const FStableShaderKeyAndValue& A, const FStableShaderKeyAndValue& B)
{
	// if the shaders don't belong to the same FShaderPipeline, they cannot be used together
	if ((A.PipelineHash != FShaderHash()) || (B.PipelineHash != FShaderHash()))
	{
		if (A.PipelineHash != B.PipelineHash)
		{
			return false;
		}
	}

	static FName NAME_FDeferredDecalVS("FDeferredDecalVS");
	static FName NAME_FDeferredLightVS("FDeferredLightVS");
	static FName NAME_FWriteToSliceVS("FWriteToSliceVS");
	static FName NAME_FScreenPassVS("FScreenPassVS");
	static FName NAME_FWriteToSliceGS("FWriteToSliceGS");
	if (
		A.ShaderType == NAME_FDeferredDecalVS || B.ShaderType == NAME_FDeferredDecalVS ||
		A.ShaderType == NAME_FDeferredLightVS || B.ShaderType == NAME_FDeferredLightVS ||
		A.ShaderType == NAME_FWriteToSliceVS || B.ShaderType == NAME_FWriteToSliceVS ||
		A.ShaderType == NAME_FScreenPassVS || B.ShaderType == NAME_FScreenPassVS ||
		A.ShaderType == NAME_FWriteToSliceGS || B.ShaderType == NAME_FWriteToSliceGS
		)
	{
		// oddball mix and match with any material shader.
		return true;
	}
	if (A.ShaderClass != B.ShaderClass)
	{
		return false;
	}
	if (A.VFType != B.VFType)
	{
		return false;
	}
	if (A.FeatureLevel != B.FeatureLevel)
	{
		return false;
	}
	if (A.TargetPlatform != B.TargetPlatform)
	{
		return false;
	}
	static FName NAME_FHWRasterizeVS("FHWRasterizeVS");
	static FName NAME_FHWRasterizeMS("FHWRasterizeMS");
	static FName NAME_FHWRasterizePS("FHWRasterizePS");
	if ((A.ShaderType == NAME_FHWRasterizePS && (B.ShaderType == NAME_FHWRasterizeVS || B.ShaderType == NAME_FHWRasterizeMS)) ||
		(B.ShaderType == NAME_FHWRasterizePS && (A.ShaderType == NAME_FHWRasterizeVS || A.ShaderType == NAME_FHWRasterizeMS)))
	{
		// skip quality level and ClassNameAndObjectPath because either vertex/mesh shader or pixel shader could be from WorldGridMaterial 
		// and then quality level could be Num and Epic, and different material name

		if (A.QualityLevel != B.QualityLevel)
		{
			static FName NAME_NumQualityLevel("Num");
			if (A.QualityLevel != NAME_NumQualityLevel && B.QualityLevel != NAME_NumQualityLevel)
			{
				return false;
			}
		}

		if (!(A.ClassNameAndObjectPath == B.ClassNameAndObjectPath))
		{
			static FName NAME_WorldGridMaterial("WorldGridMaterial");
			if (A.ClassNameAndObjectPath.ObjectClassAndPath.Num() < 3 || B.ClassNameAndObjectPath.ObjectClassAndPath.Num() < 3 ||
				(A.ClassNameAndObjectPath.ObjectClassAndPath[2] != NAME_WorldGridMaterial && B.ClassNameAndObjectPath.ObjectClassAndPath[2] != NAME_WorldGridMaterial))
			{
				return false;
			}
		}
	}
	else
	{
		if (A.QualityLevel != B.QualityLevel)
		{
			return false;
		}
		if (!(A.ClassNameAndObjectPath == B.ClassNameAndObjectPath))
		{
			return false;
		}
	}
	return true;
}

static TArray<FString> ParseParamStringList(const TMap<FString, FString>& ParamVals, const TCHAR* ParamName)
{
	TArray<FString> StrList;

	const FString* Param = ParamVals.Find(ParamName);
	if (!Param)
	{
		return StrList;
	}

	Param->ToLower().ParseIntoArray(StrList, TEXT(","));
	for (FString& Str : StrList)
	{
		Str.TrimStartAndEndInline();
	}
	return StrList;
}

int32 DumpStableKeysFile(const FString& Token, const TMap<FString, FString>& ParamVals)
{
	const FStringView File = Token;
	
	TArray<FString> OutputHashesToSearch = ParseParamStringList(ParamVals, TEXT("outputhash"));
	TArray<FString> NamesToSearch = ParseParamStringList(ParamVals, TEXT("objname"));

	FStableShaderSet StableMap;
	LoadStableShaderKeysMultiple(StableMap, MakeArrayView(&File, 1));
	UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *FStableShaderKeyAndValue::HeaderLine());
	for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)
	{
		if (!OutputHashesToSearch.IsEmpty() && !OutputHashesToSearch.Contains(ShaderKeyPtr->OutputHash.ToString().ToLower()))
		{
			continue;
		}

		if (!NamesToSearch.IsEmpty())
		{
			const FString* Found = Algo::FindByPredicate(NamesToSearch, [ObjName = ShaderKeyPtr->ClassNameAndObjectPath.ToString().ToLower()](const FString& PartialName) { return ObjName.Contains(PartialName); });
			if (!Found)
			{
				continue;
			}
		}

		UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *ShaderKeyPtr->ToString());
	}
	return 0;
}

int32 ExtractShaderFromUshaderbytecode(const FString& UshaderbytecodeName, const FString& ShaderHash)
{
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*UshaderbytecodeName));
	if (!FileReader)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Cannot open ushaderbytecode file at %ls", *UshaderbytecodeName);
		return 1;
	}

	TArray64<uint8> Data;
	Data.SetNumUninitialized(FileReader->TotalSize());
	FileReader->Serialize(Data.GetData(), Data.Num());
	if (FileReader->IsError())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Error reading ushaderbytecode file at %ls", *UshaderbytecodeName);
		return 1;
	}

	FLargeMemoryReader LibraryAr(Data.GetData(), Data.Num());
	uint32 Version;
	LibraryAr << Version;

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Reading ushaderbytecode file of version %d", Version);

	FSerializedShaderArchive SerializedShaders;
	LibraryAr << SerializedShaders;
	int64 OffsetToShaderCode = LibraryAr.Tell();

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Number of shaders %d", SerializedShaders.GetNumShaders());

	FShaderHash HashToLookFor;
	HashToLookFor.FromString(ShaderHash);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Looking for shader hash %ls", *HashToLookFor.ToString());	// print so we know the conversion was successful

	bool bHaveAtLeastOneMatch = false;
	for (int IdxShader = 0, NumShaders = SerializedShaders.GetNumShaders(); IdxShader < NumShaders; ++IdxShader)
	{
		if (SerializedShaders.ShaderHashes[IdxShader] == HashToLookFor)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Shader %d matches hash %ls", IdxShader, *SerializedShaders.ShaderHashes[IdxShader].ToString());	// print so we know the conversion was successful
			bHaveAtLeastOneMatch = true;
			const FShaderCodeEntry& Entry = SerializedShaders.ShaderEntries[IdxShader];

			// extract compressed first
			FString SavePath = FPaths::ProjectSavedDir() / TEXT("ExtractedShaders") / FPaths::GetBaseFilename(UshaderbytecodeName);
			FString ShaderName = FString::Printf(TEXT("%s_Shader_%d.bin"), *SerializedShaders.ShaderHashes[IdxShader].ToString(), IdxShader);
			FString FinalSavePath = SavePath / ShaderName;
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Extracting to %ls", *FinalSavePath);
			TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*FinalSavePath));

			if (FileWriter)
			{
				uint8* PtrToShader = (static_cast<uint64>(OffsetToShaderCode) + Entry.Offset) < static_cast<uint64>(Data.Num()) ? (Data.GetData() + OffsetToShaderCode + Entry.Offset) : nullptr;

				if (PtrToShader)
				{
					// small shaders might be stored without compression, handle them here
					if (!Entry.IsCompressed())
					{
						FileWriter->Serialize(PtrToShader, Entry.Size);
					}
					else
					{
						uint32 UncompressedSize = Entry.GetUncompressedSize();
						TUniquePtr<uint8[]> Buffer = MakeUnique<uint8[]>(UncompressedSize);
						ShaderCodeArchive::DecompressShaderWithOodle(Buffer.Get(), UncompressedSize, PtrToShader, Entry.Size);
						FileWriter->Serialize(Buffer.Get(), UncompressedSize);
					}
				}
			}
		}
	}

	if (!bHaveAtLeastOneMatch)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "Could find a shader with hash %ls among all %d shaders", *HashToLookFor.ToString(), SerializedShaders.GetNumShaders());	// print so we know the conversion was successful
	}
	return 0;
}

int32 CheckStableKeyAliasing(const TArray<FString>& Tokens)
{
	if (Tokens.Num() < 2)
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "At least two shk files need to be given to check hash aliases between them.");
		return 1;
	}

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loading %d stable shader key files...", Tokens.Num());

	TArray<FStableShaderSet> StableMaps;
	StableMaps.AddDefaulted(Tokens.Num());

	for (int32 IdxFile = 0; IdxFile < Tokens.Num(); ++IdxFile)
	{
		const FStringView File = Tokens[IdxFile];
		LoadStableShaderKeysMultiple(StableMaps[IdxFile], MakeArrayView(&File, 1));
	}

	TArray<TMultiMap<FShaderHash, const FStableShaderKeyAndValue*>> InverseStableMaps;
	InverseStableMaps.AddDefaulted(StableMaps.Num());

	for (int32 IdxFile = 0; IdxFile < Tokens.Num(); ++IdxFile)
	{
		for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMaps[IdxFile])
		{
			InverseStableMaps[IdxFile].Add(ShaderKeyPtr->OutputHash, ShaderKeyPtr.Get());
		}
	}

	auto LogAllInFile = [](const FString& FileName, const TMultiMap<FShaderHash, const FStableShaderKeyAndValue*>& InverseMap, const FShaderHash& Hash)
	{
		TArray<const FStableShaderKeyAndValue*> Values;
		InverseMap.MultiFind(Hash, Values, false);
		checkf(!Values.IsEmpty(), TEXT("This function should only be called with a hash known to exist in the inverse map"));

		UE_LOGF(LogShaderPipelineCacheTools, Display, "In file %ls, it maps to %d shader(s):", *FileName, Values.Num());
		int32 ValuesToLog = 10;
		for (const FStableShaderKeyAndValue* Value : Values)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *Value->ToString());
			if (ValuesToLog-- <= 0)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Display, "    etc (%d total)", Values.Num());
				break;
			}
		}
	};


	TSet<FShaderHash> ReportedHashes;
	// check for hashes being the same across files
	for (int32 IdxFile = 0; IdxFile < Tokens.Num(); ++IdxFile)
	{
		for (const auto& Pair : InverseStableMaps[IdxFile])
		{
			TSet<FShaderHash> AliasedHashesInThisFile;	// to allow checking for more than one pair of files

			for (int32 IdxOtherFile = IdxFile + 1; IdxOtherFile < Tokens.Num(); ++IdxOtherFile)
			{
				if (InverseStableMaps[IdxOtherFile].Contains(Pair.Key) && !ReportedHashes.Contains(Pair.Key))
				{
					UE_LOGF(LogShaderPipelineCacheTools, Warning, "Shaderhash %ls is contained in both %ls and %ls files", *Pair.Key.ToString(), *Tokens[IdxFile], *Tokens[IdxOtherFile]);

					// log it for the current file only if seeing for the first time
					if (!AliasedHashesInThisFile.Contains(Pair.Key))
					{
						// find all and log
						LogAllInFile(Tokens[IdxFile], InverseStableMaps[IdxFile], Pair.Key);
					}

					AliasedHashesInThisFile.Add(Pair.Key);		

					LogAllInFile(Tokens[IdxOtherFile], InverseStableMaps[IdxOtherFile], Pair.Key);
				}
			}

			ReportedHashes.Append(AliasedHashesInThisFile);
		}
	}

	if (ReportedHashes.IsEmpty())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "No hash aliases between the files!");
	}

	return ReportedHashes.IsEmpty() ? 0 : 1;
}

void IntersectSets(TSet<FCompactFullName>& Intersect, const TSet<FCompactFullName>& ShaderAssets)
{
	if (!Intersect.Num() && ShaderAssets.Num())
	{
		Intersect = ShaderAssets;
	}
	else if (Intersect.Num() && ShaderAssets.Num())
	{
		Intersect  = Intersect.Intersect(ShaderAssets);
	}
}

void GeneratePermutations(TArray<UE::PipelineCacheUtilities::FPermutation>& Permutations, UE::PipelineCacheUtilities::FPermutation& WorkingPerm, int32 SlotIndex , const TArray<int32> StableShadersPerSlot[SF_NumFrequencies], const TArray<FStableShaderKeyAndValue>& StableArray, const bool ActivePerSlot[SF_NumFrequencies])
{
	check(SlotIndex >= 0 && SlotIndex <= SF_NumFrequencies);
	while (SlotIndex < SF_NumFrequencies && !ActivePerSlot[SlotIndex])
	{
		SlotIndex++;
	}
	if (SlotIndex >= SF_NumFrequencies)
	{
		Permutations.Add(WorkingPerm);
		return;
	}
	for (int32 StableIndex = 0; StableIndex < StableShadersPerSlot[SlotIndex].Num(); StableIndex++)
	{
		bool bKeep = true;
		// check compatibility with shaders in the working perm
		for (int32 SlotIndexInner = 0; SlotIndexInner < SlotIndex; SlotIndexInner++)
		{
			if (SlotIndex == SlotIndexInner || !ActivePerSlot[SlotIndexInner])
			{
				continue;
			}
			check(SlotIndex != SF_Compute && SlotIndexInner != SF_Compute); // there is never any matching with compute shaders
			if (!CouldBeUsedTogether(StableArray[StableShadersPerSlot[SlotIndex][StableIndex]], StableArray[WorkingPerm.Slots[SlotIndexInner]]))
			{
				bKeep = false;
				break;
			}
		}
		if (!bKeep)
		{
			continue;
		}
		WorkingPerm.Slots[SlotIndex] = StableShadersPerSlot[SlotIndex][StableIndex];
		GeneratePermutations(Permutations, WorkingPerm, SlotIndex + 1, StableShadersPerSlot, StableArray, ActivePerSlot);
	}
}

/** Saves stable pipeline cache in a deprecated text format, returns false if failed */
bool SaveStablePipelineCacheDeprecated(const FString& OutputFilename, const TArray<UE::PipelineCacheUtilities::FPermsPerPSO>& StableResults, const TArray<FStableShaderKeyAndValue>& StableShaderKeyIndexTable)
{
	int32 NumLines = 0;
	FSCDataChunk DataChunks[16];
	int32 CurrentChunk = 0;
	TSet<uint32> DeDup;

	{
		FString PSOLine = FString::Printf(TEXT("\"%s\""), *FPipelineCacheFileFormatPSO::CommonHeaderLine());
		PSOLine += FString::Printf(TEXT(",\"%s\""), *FPipelineCacheFileFormatPSO::GraphicsDescriptor::StateHeaderLine());
		for (int32 SlotIndex = 0; SlotIndex < SF_Compute; SlotIndex++) // SF_Compute here because the stablepc.csv file format does not have a compute slot
		{
			PSOLine += FString::Printf(TEXT(",\"shaderslot%d: %s\""), SlotIndex, *FStableShaderKeyAndValue::HeaderLine());
		}

		DataChunks[CurrentChunk].OutputLinesAr << PSOLine;
		NumLines++;
	}

	for (const UE::PipelineCacheUtilities::FPermsPerPSO& Item : StableResults)
	{
		if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
		{
			if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, " Compute");
			}
			else if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, " %ls", *Item.PSO->GraphicsDesc.StateToString());
			}
			else if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, " RayTracing");
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.PSO->Type));
			}
			int32 PermIndex = 0;
			for (const UE::PipelineCacheUtilities::FPermutation& Perm : Item.Permutations)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "  ----- perm %d", PermIndex);
				for (int32 SlotIndex = 0; SlotIndex < SF_NumFrequencies; SlotIndex++)
				{
					if (!Item.ActivePerSlot[SlotIndex])
					{
						continue;
					}
					FStableShaderKeyAndValue ShaderKeyAndValue = StableShaderKeyIndexTable[Perm.Slots[SlotIndex]];
					ShaderKeyAndValue.OutputHash = FShaderHash(); // Saved output hash needs to be zeroed so that BuildPSOSC can use this entry even if shaders code changes in future builds
					UE_LOGF(LogShaderPipelineCacheTools, Verbose, "   %ls", *ShaderKeyAndValue.ToString());
				}
				PermIndex++;
			}

			UE_LOGF(LogShaderPipelineCacheTools, Verbose, "-----");
		}
		for (const UE::PipelineCacheUtilities::FPermutation& Perm : Item.Permutations)
		{
			// because it is a CSV, and for backward compat, compute shaders will just be a zeroed graphics desc with the shader in the hull shader slot.
			FString PSOLine = Item.PSO->CommonToString();
			PSOLine += TEXT(",");
			if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
			{
				FPipelineCacheFileFormatPSO::GraphicsDescriptor Zero;
				FMemory::Memzero(Zero);
				PSOLine += FString::Printf(TEXT("\"%s\""), *Zero.StateToString());
				for (int32 SlotIndex = 0; SlotIndex < SF_Compute; SlotIndex++)  // SF_Compute here because the stablepc.csv file format does not have a compute slot
				{
					check(!Item.ActivePerSlot[SlotIndex]); // none of these should be active for a compute shader
					if (SlotIndex == SF_Mesh)
					{
						FStableShaderKeyAndValue ShaderKeyAndValue = StableShaderKeyIndexTable[Perm.Slots[SF_Compute]];
						ShaderKeyAndValue.OutputHash = FShaderHash(); // Saved output hash needs to be zeroed so that BuildPSOSC can use this entry even if shaders code changes in future builds
						PSOLine += FString::Printf(TEXT(",\"%s\""), *ShaderKeyAndValue.ToString());
					}
					else
					{
						PSOLine += FString::Printf(TEXT(",\"\""));
					}
				}
			}
			else if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
			{
				PSOLine += FString::Printf(TEXT("\"%s\""), *Item.PSO->GraphicsDesc.StateToString());
				for (int32 SlotIndex = 0; SlotIndex < SF_Compute; SlotIndex++) // SF_Compute here because the stablepc.csv file format does not have a compute slot
				{
					if (!Item.ActivePerSlot[SlotIndex])
					{
						PSOLine += FString::Printf(TEXT(",\"\""));
						continue;
					}
					FStableShaderKeyAndValue ShaderKeyAndValue = StableShaderKeyIndexTable[Perm.Slots[SlotIndex]];
					ShaderKeyAndValue.OutputHash = FShaderHash(); // Saved output hash needs to be zeroed so that BuildPSOSC can use this entry even if shaders code changes in future builds
					PSOLine += FString::Printf(TEXT(",\"%s\""), *ShaderKeyAndValue.ToString());
				}
			}
			else if (Item.PSO->Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
			{
				FPipelineCacheFileFormatPSO::GraphicsDescriptor Desc;
				FMemory::Memzero(Desc);

				PSOLine += FString::Printf(TEXT("\"%s\""), *Desc.StateToString());

				for (int32 SlotIndex = 0; SlotIndex < SF_Compute; SlotIndex++)
				{
					static_assert(SF_RayGen > SF_Compute, "Unexpected shader frequency enum order");
					static_assert(SF_RayMiss > SF_Compute, "Unexpected shader frequency enum order");
					static_assert(SF_RayHitGroup > SF_Compute, "Unexpected shader frequency enum order");
					static_assert(SF_RayCallable > SF_Compute, "Unexpected shader frequency enum order");

					EShaderFrequency RayTracingSlotIndex = EShaderFrequency(SF_RayGen + SlotIndex);

					if (RayTracingSlotIndex >= SF_RayGen &&
						RayTracingSlotIndex <= SF_RayCallable &&
						Item.ActivePerSlot[RayTracingSlotIndex])
					{
						FStableShaderKeyAndValue ShaderKeyAndValue = StableShaderKeyIndexTable[Perm.Slots[RayTracingSlotIndex]];
						ShaderKeyAndValue.OutputHash = FShaderHash(); // Saved output hash needs to be zeroed so that BuildPSOSC can use this entry even if shaders code changes in future builds
						PSOLine += FString::Printf(TEXT(",\"%s\""), *ShaderKeyAndValue.ToString());
					}
					else
					{
						PSOLine += FString::Printf(TEXT(",\"\""));
					}
				}
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.PSO->Type));
			}

			const uint32 PSOLineHash = FCrc::MemCrc32(PSOLine.GetCharArray().GetData(), sizeof(TCHAR) * PSOLine.Len());
			if (!DeDup.Contains(PSOLineHash))
			{
				DeDup.Add(PSOLineHash);
				if (DataChunks[CurrentChunk].OutputLinesAr.TotalSize() + (int64)((PSOLine.Len() + 1) * sizeof(TCHAR)) >= STABLE_MAX_CHUNK_SIZE)
				{
					++CurrentChunk;
				}
				DataChunks[CurrentChunk].OutputLinesAr << PSOLine;
				NumLines++;
			}
		}
	}

	const bool bCompressed = OutputFilename.EndsWith(STABLE_CSV_COMPRESSED_EXT);

	FString CompressedFilename;
	FString UncompressedFilename;
	if (bCompressed)
	{
		CompressedFilename = OutputFilename;
		UncompressedFilename = CompressedFilename.LeftChop(STABLE_COMPRESSED_EXT_LEN); // remove the ".compressed"
	}
	else
	{
		UncompressedFilename = OutputFilename;
		CompressedFilename = UncompressedFilename + STABLE_COMPRESSED_EXT;  // add the ".compressed"
	}

	// delete both compressed and uncompressed files
	if (IFileManager::Get().FileExists(*UncompressedFilename))
	{
		IFileManager::Get().Delete(*UncompressedFilename, false, true);
		if (IFileManager::Get().FileExists(*UncompressedFilename))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Could not delete %ls", *UncompressedFilename);
			return false;
		}
	}
	if (IFileManager::Get().FileExists(*CompressedFilename))
	{
		IFileManager::Get().Delete(*CompressedFilename, false, true);
		if (IFileManager::Get().FileExists(*CompressedFilename))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Could not delete %ls", *CompressedFilename);
			return false;
		}
	}

	int64 FileSize = SaveStableCSV(OutputFilename, DataChunks, CurrentChunk + 1);
	if (FileSize < 1)
	{
		return false;
	}

	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Wrote stable PSOs in a deprecated text format, %d lines (%.1f KB) to %ls", NumLines, FileSize / 1024.f, *OutputFilename);
	return true;
}

int32 ExpandPSOSC(const TArray<FString>& Tokens)
{
	if (!Tokens.Last().EndsWith(ShaderStablePipelineFileExt) && !Tokens.Last().EndsWith(STABLE_CSV_EXT) && !Tokens.Last().EndsWith(STABLE_CSV_COMPRESSED_EXT))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "Pipeline cache filename '%ls' must end with '%ls' (or deprecated '%ls'/'%ls').",
			*Tokens.Last(), ShaderStablePipelineFileExt, STABLE_CSV_EXT, STABLE_CSV_COMPRESSED_EXT);
		return 0;
	}

	TArray<FStringView, TInlineAllocator<16>> StableCSVs;
	for (int32 Index = 0; Index < Tokens.Num() - 1; Index++)
	{
		if (Tokens[Index].EndsWith(ShaderStableKeysFileExt))
		{
			StableCSVs.Add(Tokens[Index]);
		}
	}

	// To save memory and make operations on the stable map faster, all the stable shader keys are stored in StableShaderKeyIndexTable array and shader map keys
	// and permutation slots use indices to this array instead of storing their own copies of FStableShaderKeyAndValue objects
	TArray<FStableShaderKeyAndValue> StableShaderKeyIndexTable;
	TMultiMap<int32, FShaderHash> StableMap;
	LoadStableShaderKeysMultiple(StableMap, StableShaderKeyIndexTable, StableCSVs);
	if (!StableMap.Num())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "No %ls found or they were all empty. Nothing to do.", ShaderStableKeysFileExt);
		return 0;
	}
	if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Verbose, "    %ls", *FStableShaderKeyAndValue::HeaderLine());
		for (const auto& Pair : StableMap)
		{
			FStableShaderKeyAndValue Temp(StableShaderKeyIndexTable[Pair.Key]);
			Temp.OutputHash = Pair.Value;
			UE_LOGF(LogShaderPipelineCacheTools, Verbose, "    %ls", *Temp.ToString());
		}
		StableShadersSerializationSelfTest(StableMap, StableShaderKeyIndexTable);
	}
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d unique shader info lines total.", StableMap.Num());

	TSet<FPipelineCacheFileFormatPSO> PSOs;
	
	uint32 MergeCount = 0;

	for (int32 Index = 0; Index < Tokens.Num() - 1; Index++)
	{
		if (Tokens[Index].EndsWith(TEXT(".upipelinecache")))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Loading %ls....", *Tokens[Index]);
			TSet<FPipelineCacheFileFormatPSO> TempPSOs;
			if (!FPipelineFileCacheManager::LoadPipelineFileCacheInto(Tokens[Index], TempPSOs))
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Could not load %ls or it was empty.", *Tokens[Index]);
				continue;
			}
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d PSOs", TempPSOs.Num());

			// We need to merge otherwise we'll lose usage masks on exact same PSO but in different files
			for(auto& TempPSO : TempPSOs)
			{
				auto* ExistingPSO = PSOs.Find(TempPSO);
				if(ExistingPSO != nullptr)
				{
					// Existing PSO must have already gone through verify and invertibility checks
					check(*ExistingPSO == TempPSO);
					
					// Get More accurate stats by testing for diff - we could just merge and be done
					if((ExistingPSO->UsageMask & TempPSO.UsageMask) != TempPSO.UsageMask)
					{
						ExistingPSO->UsageMask |= TempPSO.UsageMask;
						++MergeCount;
					}
					// Raw data files are not bind count averaged - just ensure we have captured max value
					ExistingPSO->BindCount = FMath::Max(ExistingPSO->BindCount, TempPSO.BindCount);
				}
				else
				{
					// as of UE 5.1, we do not support storing PSOs in CSV so disable the string invertibility test, as that code path isn't updated
					bool bInvertibilityResult = true; // CheckPSOStringInveribility(TempPSO);
					bool bVerifyResult = TempPSO.Verify();
					if(bInvertibilityResult && bVerifyResult)
					{
						PSOs.Add(TempPSO);
					}
					else
					{
						// Log Found Bad PSO, this is in the context of the logged current file above so we can see where this has come from
						UE_LOGF(LogShaderPipelineCacheTools, Warning, "Bad PSO found discarding [Invertibility=%ls Verify=%ls in: %ls]", bInvertibilityResult ? TEXT("PASS") :  TEXT("FAIL") , bVerifyResult ? TEXT("PASS") :  TEXT("FAIL"), *Tokens[Index]);
					}
				}
			}
		}
		else
		{
			check(Tokens[Index].EndsWith(ShaderStableKeysFileExt));
		}
	}
	if (!PSOs.Num())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "No .upipelinecache files found or they were all empty. Nothing to do.");
		return 0;
	}
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d PSOs total [Usage Mask Merged = %d].", PSOs.Num(), MergeCount);

	if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
	{
		TMap<FShaderHash, TArray<FString>> InverseMap;

		for (const auto& Pair : StableMap)
		{
			FStableShaderKeyAndValue Temp(StableShaderKeyIndexTable[Pair.Key]);
			Temp.OutputHash = Pair.Value;
			InverseMap.FindOrAdd(Pair.Value).Add(Temp.ToString());
		}

		for (const FPipelineCacheFileFormatPSO& Item : PSOs)
		{
			switch (Item.Type)
			{
			case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "ComputeShader");
				PrintShaders(InverseMap, Item.ComputeDesc.ComputeShader);
				break;
			case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "VertexShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.VertexShader);
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "MeshShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.MeshShader);
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "AmplificationShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.AmplificationShader);
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "FragmentShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.FragmentShader);
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "GeometryShader");
				PrintShaders(InverseMap, Item.GraphicsDesc.GeometryShader);
				break;
			case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "RayTracingShader");
				PrintShaders(InverseMap, Item.RayTracingDesc.ShaderHash);
				break;
			default:
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.Type));
				break;
			}
		}
	}
	TMap<FShaderHash, TArray<int32>> InverseMap;

	for (const auto& Pair : StableMap)
	{
		InverseMap.FindOrAdd(Pair.Value).AddUnique(Pair.Key);
	}

	int32 TotalStablePSOs = 0;

	TArray<UE::PipelineCacheUtilities::FPermsPerPSO> StableResults;
	StableResults.Reserve(PSOs.Num());
	int32 NumSkipped = 0;
	int32 NumExamined = PSOs.Num();
	FCriticalSection StableResultsAdditionGuard;	// guards addition to StableResults table
	FCriticalSection ConsoleOutputGuard;			// so the printouts from various PSOs aren't interleaved. Also guards NumSkipped, TotalStablePSO and other stat vars

	// we cannot run ParallelFor on a TSet, so linearize it
	TArray<const FPipelineCacheFileFormatPSO*> PSOPtrs;
	PSOPtrs.Reserve(PSOs.Num());
	for (const FPipelineCacheFileFormatPSO& Item : PSOs)
	{
		PSOPtrs.Add(&Item);
	}

	static_assert(SF_Vertex == 0 && SF_Compute == 5, "Shader Frequencies have changed, please update");
	ParallelFor(
		PSOPtrs.Num(),
		[&StableResults, &StableResultsAdditionGuard, &ConsoleOutputGuard, &PSOPtrs,
		&TotalStablePSOs, &NumSkipped, &InverseMap, &StableShaderKeyIndexTable
		](int32 PSOIndex)
	{
		const FPipelineCacheFileFormatPSO& Item = *PSOPtrs[PSOIndex];
		
		TArray<int32> StableShadersPerSlot[SF_NumFrequencies];
		bool ActivePerSlot[SF_NumFrequencies] = { false };

		bool OutAnyActiveButMissing = false;

		if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
		{
			ActivePerSlot[SF_Compute] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.ComputeDesc.ComputeShader, StableShadersPerSlot[SF_Compute], OutAnyActiveButMissing);
		}
		else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			ActivePerSlot[SF_Vertex] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.VertexShader, StableShadersPerSlot[SF_Vertex], OutAnyActiveButMissing);
			ActivePerSlot[SF_Mesh] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.MeshShader, StableShadersPerSlot[SF_Mesh], OutAnyActiveButMissing);
			ActivePerSlot[SF_Amplification] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.AmplificationShader, StableShadersPerSlot[SF_Amplification], OutAnyActiveButMissing);
			ActivePerSlot[SF_Pixel] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.FragmentShader, StableShadersPerSlot[SF_Pixel], OutAnyActiveButMissing);
			ActivePerSlot[SF_Geometry] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.GeometryShader, StableShadersPerSlot[SF_Geometry], OutAnyActiveButMissing);
		}
		else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
		{
			EShaderFrequency Frequency = Item.RayTracingDesc.Frequency;
			ActivePerSlot[Frequency] = GetStableShaders(InverseMap, StableShaderKeyIndexTable, Item.RayTracingDesc.ShaderHash, StableShadersPerSlot[Frequency], OutAnyActiveButMissing);
		}
		else
		{
			FScopeLock Lock(&ConsoleOutputGuard);
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.Type));
		}

		if (OutAnyActiveButMissing)
		{
			FScopeLock Lock(&ConsoleOutputGuard);
			UE_LOGF(LogShaderPipelineCacheTools, Display, "PSO had an active shader slot that did not match any current shaders, ignored.");
			if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
			{
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.ComputeDesc.ComputeShader, TEXT("ComputeShader"));
			}
			else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Display, "   %ls", *Item.GraphicsDesc.StateToString());
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.VertexShader, TEXT("VertexShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.MeshShader, TEXT("MeshShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.AmplificationShader, TEXT("AmplificationShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.FragmentShader, TEXT("FragmentShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.GeometryShader, TEXT("GeometryShader"));
			}
			else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
			{
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.RayTracingDesc.ShaderHash, TEXT("RayTracingShader"));
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.Type));
			}
			return;
		}

		if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			check(!ActivePerSlot[SF_Compute]); // this is NOT a compute shader
			bool bRemovedAll = false;
			bool bAnyActive = false;
			// Quite the nested loop. It isn't clear if this could be made faster, but the thing to realize is that the same set of shaders will be used in multiple PSOs we could take advantage of that...we don't.
			for (int32 SlotIndex = 0; SlotIndex < SF_NumFrequencies; SlotIndex++)
			{
				if (!ActivePerSlot[SlotIndex])
				{
					check(!StableShadersPerSlot[SlotIndex].Num());
					continue;
				}
				bAnyActive = true;
				for (int32 StableIndex = 0; StableIndex < StableShadersPerSlot[SlotIndex].Num(); StableIndex++)
				{
					bool bKeep = true;
					for (int32 SlotIndexInner = 0; SlotIndexInner < SF_Compute; SlotIndexInner++) //SF_Compute here because this is NOT a compute shader
					{
						if (SlotIndex == SlotIndexInner || !ActivePerSlot[SlotIndexInner])
						{
							continue;
						}
						bool bFoundCompat = false;
						for (int32 StableIndexInner = 0; StableIndexInner < StableShadersPerSlot[SlotIndexInner].Num(); StableIndexInner++)
						{
							if (CouldBeUsedTogether(StableShaderKeyIndexTable[StableShadersPerSlot[SlotIndex][StableIndex]], StableShaderKeyIndexTable[StableShadersPerSlot[SlotIndexInner][StableIndexInner]]))
							{
								bFoundCompat = true;
								break;
							}
						}
						if (!bFoundCompat)
						{
							bKeep = false;
							break;
						}
					}
					if (!bKeep)
					{
						StableShadersPerSlot[SlotIndex].RemoveAt(StableIndex--);
					}
				}
				if (!StableShadersPerSlot[SlotIndex].Num())
				{
					bRemovedAll = true;
				}
			}
			if (!bAnyActive)
			{
				FScopeLock Lock(&ConsoleOutputGuard);
				NumSkipped++;
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "PSO did not create any stable PSOs! (no active shader slots)");
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "   %ls", *Item.GraphicsDesc.StateToString());
				return;
			}
			if (bRemovedAll)
			{
				FScopeLock Lock(&ConsoleOutputGuard);
				UE_LOGF(LogShaderPipelineCacheTools, Warning, "PSO did not create any stable PSOs! (no cross shader slot compatibility)");
				UE_LOGF(LogShaderPipelineCacheTools, Warning, "   %ls", *Item.GraphicsDesc.StateToString());

				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.VertexShader, TEXT("VertexShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.MeshShader, TEXT("MeshShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.AmplificationShader, TEXT("AmplificationShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.FragmentShader, TEXT("FragmentShader"));
				PrintShaders(InverseMap, StableShaderKeyIndexTable, Item.GraphicsDesc.GeometryShader, TEXT("GeometryShader"));

				return;
			}
			// We could have done this on the fly, but that loop was already pretty complicated. Here we generate all plausible permutations and write them out
		}

		UE::PipelineCacheUtilities::FPermsPerPSO Current;
		Current.PSO = &Item;

		for (int32 Index = 0; Index < SF_NumFrequencies; Index++)
		{
			Current.ActivePerSlot[Index] = ActivePerSlot[Index];
		}

		TArray<UE::PipelineCacheUtilities::FPermutation>& Permutations(Current.Permutations);
		UE::PipelineCacheUtilities::FPermutation WorkingPerm = {};
		GeneratePermutations(Permutations, WorkingPerm, 0, StableShadersPerSlot, StableShaderKeyIndexTable, ActivePerSlot);
		if (!Permutations.Num())
		{
			FScopeLock Lock(&ConsoleOutputGuard);
			UE_LOGF(LogShaderPipelineCacheTools, Error, "PSO did not create any stable PSOs! (somehow)");
			// this is fatal because now we have a bogus thing in the list
			UE_LOGF(LogShaderPipelineCacheTools, Fatal, "   %ls", *Item.GraphicsDesc.StateToString());
			return;
		}

		{
			FScopeLock Lock(&StableResultsAdditionGuard);
			StableResults.Add(Current);
		}

		FScopeLock Lock(&ConsoleOutputGuard);
		UE_LOGF(LogShaderPipelineCacheTools, Verbose, "----- PSO created %d stable permutations --------------", Permutations.Num());
		TotalStablePSOs += Permutations.Num();
	},
	EParallelForFlags::Unbalanced
	);

	UE_CLOGF(NumSkipped > 0, LogShaderPipelineCacheTools, Warning, "%d/%d PSO did not create any stable PSOs! (no active shader slots)", NumSkipped, NumExamined);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Generated %d stable PSOs total", TotalStablePSOs);
	if (!TotalStablePSOs || !StableResults.Num())
	{
		UE_LOGF(LogShaderPipelineCacheTools, Error, "No stable PSOs created.");
		return 1;
	}

	const FString& OutputFilename = Tokens.Last();

	if (OutputFilename.EndsWith(STABLE_CSV_COMPRESSED_EXT) || OutputFilename.EndsWith(STABLE_CSV_EXT))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Warning, "Using a deprecated stablepc format %ls, please replace with %ls", *OutputFilename, ShaderStablePipelineFileExtWildcard);

		return SaveStablePipelineCacheDeprecated(OutputFilename, StableResults, StableShaderKeyIndexTable) ? 0 : 1;
	}

	return UE::PipelineCacheUtilities::SaveStablePipelineCacheFile(OutputFilename, StableResults, StableShaderKeyIndexTable) ? 0 : 1;
}

template <uint32 InlineSize>
static void ParseQuoteComma(const FStringView& InLine, TArray<FStringView, TInlineAllocator<InlineSize>>& OutParts)
{
	FStringView Line = InLine;
	while (true)
	{
		int32 QuoteLoc = 0;
		if (!Line.FindChar(TCHAR('\"'), QuoteLoc))
		{
			break;
		}
		Line.RightChopInline(QuoteLoc + 1);
		if (!Line.FindChar(TCHAR('\"'), QuoteLoc))
		{
			break;
		}
		OutParts.Add(Line.Left(QuoteLoc));
		Line.RightChopInline(QuoteLoc + 1);
	}
}

static TSet<FPipelineCacheFileFormatPSO> ParseStableCSV(const FString& FileName, const TArray<FString>& CSVLines, const FStableShaderSet& StableMap, FName& TargetPlatform, int32& PSOsRejected, int32& PSOsMerged)
{
	TSet<FPipelineCacheFileFormatPSO> PSOs;

	int32 LineIndex = 0;
	bool bParsed = true;
	ReadStableCSV(CSVLines, [&FileName, &StableMap, &TargetPlatform, &PSOs, &LineIndex, &bParsed, &PSOsRejected, &PSOsMerged](FStringView Line)
	{
		// Skip the header line.
		if (LineIndex++ == 0)
		{
			return;
		}

		// Only attempt to parse the current line if previous lines succeeded.
		if (!bParsed)
		{
			return;
		}

		TArray<FStringView, TInlineAllocator<2 + SF_NumFrequencies>> Parts;
		ParseQuoteComma(Line, Parts);

		FPipelineCacheFileFormatPSO PSO;
		FMemory::Memzero(PSO);
		PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::Graphics; // we will change this to compute later if needed
		PSO.CommonFromString(Parts[0]);
		bool bValidGraphicsDesc = PSO.GraphicsDesc.StateFromString(Parts[1]);
		if (!bValidGraphicsDesc)
		{
			// Failed to parse graphics descriptor, most likely format was changed, skip whole file.
			UE_LOGF(LogShaderPipelineCacheTools, Warning, "File %ls is not in the correct format (GraphicsDesc) ignoring the rest of its contents.", *FileName);
			bParsed = false;
			return;
		}

		// For backward compatibility, compute shaders are stored as a zeroed graphics desc with the shader in the mesh shader slot.
		static FName NAME_SF_Vertex("SF_Vertex");
		static FName NAME_SF_Mesh("SF_Mesh");
		static FName NAME_SF_Amplification("SF_Amplification");
		static FName NAME_SF_Pixel("SF_Pixel");
		static FName NAME_SF_Geometry("SF_Geometry");
		static FName NAME_SF_Compute("SF_Compute");
		static FName NAME_SF_RayGen("SF_RayGen");
		static FName NAME_SF_RayMiss("SF_RayMiss");
		static FName NAME_SF_RayHitGroup("SF_RayHitGroup");
		static FName NAME_SF_RayCallable("SF_RayCallable");
		for (int32 PartIndex = 2; PartIndex < Parts.Num(); ++PartIndex)
		{
			if (Parts[PartIndex].IsEmpty())
			{
				continue;
			}

			FStableShaderKeyAndValue Shader;
			Shader.ParseFromString(Parts[PartIndex]);

			int32 AdjustedSlotIndex = SF_Vertex;

			if (Shader.TargetFrequency == NAME_SF_Vertex)
			{
				AdjustedSlotIndex = SF_Vertex;
			}
			else if (Shader.TargetFrequency == NAME_SF_Mesh)
			{
				AdjustedSlotIndex = SF_Mesh;
			}
			else if (Shader.TargetFrequency == NAME_SF_Amplification)
			{
				AdjustedSlotIndex = SF_Amplification;
			}
			else if (Shader.TargetFrequency == NAME_SF_Pixel)
			{
				AdjustedSlotIndex = SF_Pixel;
			}
			else if (Shader.TargetFrequency == NAME_SF_Geometry)
			{
				AdjustedSlotIndex = SF_Geometry;
			}
			else if (Shader.TargetFrequency == NAME_SF_Compute)
			{
				PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::Compute;
				AdjustedSlotIndex = SF_Compute;
			}
			else if (Shader.TargetFrequency == NAME_SF_RayGen)
			{
				PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::RayTracing;
				AdjustedSlotIndex = SF_RayGen;
			}
			else if (Shader.TargetFrequency == NAME_SF_RayMiss)
			{
				PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::RayTracing;
				AdjustedSlotIndex = SF_RayMiss;
			}
			else if (Shader.TargetFrequency == NAME_SF_RayHitGroup)
			{
				PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::RayTracing;
				AdjustedSlotIndex = SF_RayHitGroup;
			}
			else if (Shader.TargetFrequency == NAME_SF_RayCallable)
			{
				PSO.Type = FPipelineCacheFileFormatPSO::DescriptorType::RayTracing;
				AdjustedSlotIndex = SF_RayCallable;
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Warning, "File %ls is not in the correct format (GraphicsDesc) ignoring the rest of its contents.", *FileName);
				bParsed = false;
				return;
			}

			FShaderHash Match;

			if (const TUniquePtr<FStableShaderKeyAndValue>* ShaderKeyPtr = StableMap.Find(Shader))
			{
				Match = (*ShaderKeyPtr)->OutputHash;
				check(Match != FShaderHash());

				TargetPlatform = (*ShaderKeyPtr)->TargetPlatform;
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "Stable PSO not found, rejecting %ls", *Shader.ToString());
				++PSOsRejected;
				return;
			}

			switch (AdjustedSlotIndex)
			{
			case SF_Vertex:
				PSO.GraphicsDesc.VertexShader = Match;
				break;
			case SF_Mesh:
				PSO.GraphicsDesc.MeshShader = Match;
				break;
			case SF_Amplification:
				PSO.GraphicsDesc.AmplificationShader = Match;
				break;
			case SF_Pixel:
				PSO.GraphicsDesc.FragmentShader = Match;
				break;
			case SF_Geometry:
				PSO.GraphicsDesc.GeometryShader = Match;
				break;
			case SF_Compute:
				PSO.ComputeDesc.ComputeShader = Match;
				break;
			case SF_RayGen:
			case SF_RayMiss:
			case SF_RayHitGroup:
			case SF_RayCallable:
				PSO.RayTracingDesc.ShaderHash = Match;
				// See corresponding serialization code in ExpandPSOSC()
				PSO.RayTracingDesc.Frequency = EShaderFrequency(AdjustedSlotIndex);
				break;
			default:
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected shader frequency");
			}
		}

		if (PSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
		{
			check(PSO.ComputeDesc.ComputeShader != FShaderHash() &&
				PSO.GraphicsDesc.VertexShader == FShaderHash() &&
				PSO.GraphicsDesc.MeshShader == FShaderHash() &&
				PSO.GraphicsDesc.AmplificationShader == FShaderHash() &&
				PSO.GraphicsDesc.FragmentShader == FShaderHash() &&
				PSO.GraphicsDesc.GeometryShader == FShaderHash());
		}
		else if (PSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			check(PSO.ComputeDesc.ComputeShader == FShaderHash());
		}
		else if (PSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
		{
			check(PSO.RayTracingDesc.ShaderHash != FShaderHash());
		}
		else
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(PSO.Type));
		}

		if (!PSO.Verify())
		{
			UE_LOGF(LogShaderPipelineCacheTools, Warning, "Bad PSO found. Verify failed. PSO discarded [Line %d in: %ls]", LineIndex, *FileName);
			++PSOsRejected;
			return;
		}

		// Merge duplicate PSO lines together.
		if (FPipelineCacheFileFormatPSO* ExistingPSO = PSOs.Find(PSO))
		{
			check(*ExistingPSO == PSO);
			ExistingPSO->UsageMask |= PSO.UsageMask;
			ExistingPSO->BindCount = FMath::Max(ExistingPSO->BindCount, PSO.BindCount);

			++PSOsMerged;
		}
		else
		{
			PSOs.Add(PSO);
		}
	});

	return PSOs;
}

typedef TFunction<bool(const FString&)> FilenameFilterFN;

void BuildDateSortedListOfFiles(const TArray<FString>& TokenList, FilenameFilterFN FilterFn, TArray<FString>& Result)
{
	struct FDateSortableFileRef
	{
		FDateTime SortTime;
		FString FileName;
	};
	
	TArray<FDateSortableFileRef> DateFileList;
	for (int32 TokenIndex = 0; TokenIndex < TokenList.Num() - 1; TokenIndex++)
	{
		if (FilterFn(TokenList[TokenIndex]))
		{
			FDateSortableFileRef DateSortEntry;
			DateSortEntry.SortTime = FDateTime::Now();
			DateSortEntry.FileName = TokenList[TokenIndex];
			
			FFileStatData StatData = IFileManager::Get().GetStatData(*TokenList[TokenIndex]);
			if(StatData.bIsValid && StatData.CreationTime != FDateTime::MinValue())
			{
				DateSortEntry.SortTime = StatData.CreationTime;
			}
			
			DateFileList.Add(DateSortEntry);
		}
	}
	
	DateFileList.Sort([](const FDateSortableFileRef& A, const FDateSortableFileRef& B) {return A.SortTime > B.SortTime;});
	
	for(auto& FileRef : DateFileList)
	{
		Result.Add(FileRef.FileName);
	}
}

const TCHAR* VertexElementToString(EVertexElementType Type)
{
	switch (Type)
	{
#define VES_STRINGIFY(T)   case T: return TEXT(#T);

		VES_STRINGIFY(VET_None)
		VES_STRINGIFY(VET_Float1)
		VES_STRINGIFY(VET_Float2)
		VES_STRINGIFY(VET_Float3)
		VES_STRINGIFY(VET_Float4)
		VES_STRINGIFY(VET_PackedNormal)
		VES_STRINGIFY(VET_UByte4)
		VES_STRINGIFY(VET_UByte4N)
		VES_STRINGIFY(VET_Color)
		VES_STRINGIFY(VET_Short2)
		VES_STRINGIFY(VET_Short4)
		VES_STRINGIFY(VET_Short2N)
		VES_STRINGIFY(VET_Half2)
		VES_STRINGIFY(VET_Half4)
		VES_STRINGIFY(VET_Short4N)
		VES_STRINGIFY(VET_UShort2)
		VES_STRINGIFY(VET_UShort4)
		VES_STRINGIFY(VET_UShort2N)
		VES_STRINGIFY(VET_UShort4N)
		VES_STRINGIFY(VET_URGB10A2N)
		VES_STRINGIFY(VET_UInt)
		VES_STRINGIFY(VET_MAX)

#undef VES_STRINGIFY
	}

	return TEXT("Unknown");
}


void FilterInvalidPSOs(TSet<FPipelineCacheFileFormatPSO>& InOutPSOs, const FStableShaderSet& StableMap)
{
	// list of Vertex Shaders known to be usable with empty vertex declaration without taking VF into consideration
	const TCHAR* VShadersUsableWithEmptyVertexDecl_Table[] =
	{
		TEXT("FHairFollicleMaskVS"),
		TEXT("FDiaphragmDOFHybridScatterVS"),
		TEXT("FLensFlareBlurVS"),
		TEXT("FMotionBlurVelocityDilateScatterVS"),
		TEXT("FScreenSpaceReflectionsTileVS"),
		TEXT("FWaterTileVS"),
		TEXT("FRenderSkyAtmosphereVS"),
		TEXT("TPageTableUpdateVS<true>"),
		TEXT("TPageTableUpdateVS<false>"),
		TEXT("FShaderDrawDebugVS"),
		TEXT("FHWRasterizeVS"),
		TEXT("FRenderRealTimeReflectionHeightFogVS")
	};

	TSet<FName> VShadersUsableWithEmptyVertexDecl;
	for (const TCHAR* VSType : VShadersUsableWithEmptyVertexDecl_Table)
	{
		VShadersUsableWithEmptyVertexDecl.Add(FName(VSType));
	}

	// list of Vertex Factories known to have empty vertex declaration
	const TCHAR* VFactoriesUsableWithEmptyVertexDecl_Table[] =
	{
		TEXT("FNiagaraRibbonVertexFactory"),
		TEXT("FLocalVertexFactory")
	};

	TSet<FName> VFactoriesUsableWithEmptyVertexDecl;
	for (const TCHAR* VFType : VFactoriesUsableWithEmptyVertexDecl_Table)
	{
		VFactoriesUsableWithEmptyVertexDecl.Add(FName(VFType));
	}

	// list of Shaders which are using RHI features which are not available on all systems and could fail to compile
	const TCHAR* ShadersUsingPossibleUnsupportedRHIFeatures_Table[] =
	{
		TEXT("FTSRRejectShadingCS")
	};
	
	TSet<FName> ShadersUsingPossibleUnsupportedRHIFeatures;
	for (const TCHAR* ShaderType : ShadersUsingPossibleUnsupportedRHIFeatures_Table)
	{
		ShadersUsingPossibleUnsupportedRHIFeatures.Add(FName(ShaderType));
	}

	// This may be too strict, but we cannot know the VS signature.
	auto IsInputLayoutCompatible = [](const FVertexDeclarationElementList& A, const FVertexDeclarationElementList& B, TMap<TTuple<EVertexElementType, EVertexElementType>, int32>& MismatchStats) -> bool
	{
		auto NumElements = [](EVertexElementType Type) -> int
		{
			switch (Type)
			{
				case VET_Float4:
				case VET_Half4:
				case VET_Short4:
				case VET_Short4N:
				case VET_UShort4:
				case VET_UShort4N:
				case VET_PackedNormal:
				case VET_UByte4:
				case VET_UByte4N:
				case VET_Color:
					return 4;

				case VET_Float3:
					return 3;

				case VET_Float2:
				case VET_Half2:
				case VET_Short2:
				case VET_Short2N:
				case VET_UShort2:
				case VET_UShort2N:
					return 2;

				default:
					break;
			}

			return 1;
		};

		auto IsFloatOrTuple = [](EVertexElementType Type)
		{
			// halves can also be promoted to float
			return Type == VET_Float1 || Type == VET_Float2 || Type == VET_Float3 || Type == VET_Float4 || Type == VET_Half2 || Type == VET_Half4;
		};

		auto IsShortOrTuple = [](EVertexElementType Type)
		{
			return Type == VET_Short2 || Type == VET_Short4;
		};

		auto IsShortNOrTuple = [](EVertexElementType Type)
		{
			return Type == VET_Short2N || Type == VET_Short4N;
		};

		auto IsUShortOrTuple = [](EVertexElementType Type)
		{
			return Type == VET_UShort2 || Type == VET_UShort4;
		};

		auto IsUShortNOrTuple = [](EVertexElementType Type)
		{
			return Type == VET_UShort2N || Type == VET_UShort4N;
		};

		// it's Okay for this number to be zero, there's a separate check for empty vs non-empty mismatch
		int32 NumElementsToCheck = FMath::Min(A.Num(), B.Num());

		for (int32 Idx = 0, Num = NumElementsToCheck; Idx < Num; ++Idx)
		{
			if (A[Idx].Type != B[Idx].Type)
			{
				// When we see float2 vs float4 mismatch, we cannot know which one the vertex shader expects.
				// Alas we cannot err on a safe side here because it's a very frequent case that would filter out a lot of valid PSOs
				//if (NumElements(A[Idx].Type) == NumElements(B[Idx].Type))
				{
					if (IsFloatOrTuple(A[Idx].Type) && IsFloatOrTuple(B[Idx].Type))
					{
						continue;
					}

					if (IsShortOrTuple(A[Idx].Type) && IsShortOrTuple(B[Idx].Type))
					{
						continue;
					}

					if (IsShortNOrTuple(A[Idx].Type) && IsShortNOrTuple(B[Idx].Type))
					{
						continue;
					}

					if (IsUShortOrTuple(A[Idx].Type) && IsUShortOrTuple(B[Idx].Type))
					{
						continue;
					}

					if (IsUShortNOrTuple(A[Idx].Type) && IsUShortNOrTuple(B[Idx].Type))
					{
						continue;
					}

					// also blindly allow any types that agree on the number of elements
					if (NumElements(A[Idx].Type) == NumElements(B[Idx].Type))
					{
						continue;
					}
				}

				// found a mismatch. Collect the stats about it.
				TTuple<EVertexElementType, EVertexElementType> Pair;
				// to avoid A,B vs B,A tuples, make sure that the first is always lower or equal
				if (A[Idx].Type < B[Idx].Type)
				{
					Pair.Key = A[Idx].Type;
					Pair.Value = B[Idx].Type;
				}
				else
				{
					Pair.Key = B[Idx].Type;
					Pair.Value = A[Idx].Type;
				}

				if (int32* ExistingCount = MismatchStats.Find(Pair))
				{
					++(*ExistingCount);
				}
				else
				{
					MismatchStats.Add(Pair, 1);
				}

				return false;
			}
		}

		return true;
	};

	UE_LOGF(LogShaderPipelineCacheTools, Display, "Running sanity check (consistency of vertex format).");

	// inverse map is needed for VS checking
	TMap<FShaderHash, TArray<FStableShaderKeyAndValue>> InverseMap;
	for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)
	{
		InverseMap.FindOrAdd(ShaderKeyPtr->OutputHash).Add(*ShaderKeyPtr);
	}

	// At this point we cannot really know what is the correct vertex format (input layout) for a given vertex shader. Instead, we're looking if we see the same VS used in multiple PSOs with incompatible vertex descriptors.
	// If we find that some of them are suspect, we'll remove all such PSOs from the cache. That may be aggressive but it's better to have hitches than hangs and crashes.
	TMap<FShaderHash, FVertexDeclarationElementList> VSToVertexDescriptor;
	TSet<FShaderHash> SuspiciousVertexShaders;
	TMap<TTuple<EVertexElementType, EVertexElementType>, int32> MismatchStats;

	TSet<FStableShaderKeyAndValue> PossiblyIncorrectUsageWithEmptyDeclaration;
	int32 NumPSOsFilteredDueToEmptyDecls = 0;
	int32 NumPSOsFilteredDueToInconsistentDecls = 0;
	int32 NumPSOsFilteredDueToUsingPossibleUnsupportedRHIFeatures = 0;
	int32 NumPSOsOriginal = InOutPSOs.Num();

	for (const FPipelineCacheFileFormatPSO& CurPSO : InOutPSOs)
	{
		if (CurPSO.Type != FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
		{
			continue;
		}

		if (CurPSO.GraphicsDesc.MeshShader != FShaderHash())
		{
			continue;
		}

		if (FVertexDeclarationElementList* Existing = VSToVertexDescriptor.Find(CurPSO.GraphicsDesc.VertexShader))
		{
			// check if current is the same or compatible
			if (!IsInputLayoutCompatible(CurPSO.GraphicsDesc.VertexDescriptor, *Existing, MismatchStats))
			{
				SuspiciousVertexShaders.Add(CurPSO.GraphicsDesc.VertexShader);
			}
		}
		else
		{
			VSToVertexDescriptor.Add(CurPSO.GraphicsDesc.VertexShader, CurPSO.GraphicsDesc.VertexDescriptor);
		}
	}

	UE_LOGF(LogShaderPipelineCacheTools, Display, "%d vertex shaders are used with an inconsistent vertex format", SuspiciousVertexShaders.Num());

	// remove all PSOs that have of those vertex shaders
	if (SuspiciousVertexShaders.Num() > 0)
	{
		// print what was not compatible
		UE_LOGF(LogShaderPipelineCacheTools, Display, "The following inconsistencies were noticed:");
		for (const TTuple< TTuple<EVertexElementType, EVertexElementType>, int32>& Stat : MismatchStats)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "%d times one PSO used the vertex shader with %ls (%d), another %ls (%d) (we don't know VS signature so assume it needs the larger type)", Stat.Value, VertexElementToString(Stat.Key.Key), Stat.Key.Key, VertexElementToString(Stat.Key.Value), Stat.Key.Value);
		}

		// print the shaders themselves
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "These vertex shaders are used with an inconsistent vertex format:");
			int32 SuspectVSIdx = 0;
			const int32 kMaxSuspectToPrint = 50;
			for (const FShaderHash& SuspectVS : SuspiciousVertexShaders)
			{
				const TArray<FStableShaderKeyAndValue>* Out = InverseMap.Find(SuspectVS);
				if (Out && Out->Num() > 0)
				{
					if (Out->Num() > 1)
					{
						UE_LOGF(LogShaderPipelineCacheTools, Display, "%d: %d shaders matching hash %ls", SuspectVSIdx, Out->Num(), *SuspectVS.ToString());

						if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
						{
							int32 SubIdx = 0;
							for (const FStableShaderKeyAndValue& Item : *Out)
							{
								UE_LOGF(LogShaderPipelineCacheTools, Verbose, "    %d: %ls", SubIdx, *Item.ToString());
								++SubIdx;
							}
						}
						else
						{
							UE_LOGF(LogShaderPipelineCacheTools, Display, "    Example: %ls", *((*Out)[0].ToString()));
						}
					}
					else
					{
						UE_LOGF(LogShaderPipelineCacheTools, Display, "%d: %ls", SuspectVSIdx, *((*Out)[0].ToString()));
					}
				}
				else
				{
					UE_LOGF(LogShaderPipelineCacheTools, Warning, "Unknown shader with a hash %ls", *SuspectVS.ToString());
				}
				++SuspectVSIdx;

				if (SuspectVSIdx > kMaxSuspectToPrint)
				{
					UE_LOGF(LogShaderPipelineCacheTools, Display, "... and %d more VS hashes", SuspiciousVertexShaders.Num() - SuspectVSIdx - 1);
					break;
				}
			}
		}
	}

	FName UnknownVFType(TEXT("null"));

	// Filter the PSOs using possible unsupported RHI features
	auto ContainsShaderWithPossibleUnsupportedRHIFeatures = [InverseMap, ShadersUsingPossibleUnsupportedRHIFeatures](const FShaderHash& ShaderHash) -> bool
	{
		if (ShaderHash != FShaderHash())
		{
			const TArray<FStableShaderKeyAndValue>* Shaders = InverseMap.Find(ShaderHash);
			if (Shaders != nullptr)
			{
				for (const FStableShaderKeyAndValue& Shader : *Shaders)
				{
					if (ShadersUsingPossibleUnsupportedRHIFeatures.Contains(Shader.ShaderType))
					{
						UE_LOGF(LogShaderPipelineCacheTools, Display, "Filtering out PSO using shader with possible unsupported RHI feature:\n %ls", *Shader.ToString());
						return true;
					}
				}
			}
		}

		return false;
	};

	// filter the PSOs
	TSet<FPipelineCacheFileFormatPSO> RetainedPSOs;
	for (const FPipelineCacheFileFormatPSO& CurPSO : InOutPSOs)
	{
		switch (CurPSO.Type)
		{
		case FPipelineCacheFileFormatPSO::DescriptorType::Compute:
		{
			if (ContainsShaderWithPossibleUnsupportedRHIFeatures(CurPSO.ComputeDesc.ComputeShader))
			{
				++NumPSOsFilteredDueToUsingPossibleUnsupportedRHIFeatures;
				continue;
			}

			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::Graphics:
		{
			if (CurPSO.GraphicsDesc.MeshShader != FShaderHash())
			{
				RetainedPSOs.Add(CurPSO);
				continue;
			}

			if (SuspiciousVertexShaders.Contains(CurPSO.GraphicsDesc.VertexShader))
			{
				++NumPSOsFilteredDueToInconsistentDecls;
				continue;
			}

			// check if the vertex shader is known to be used with an empty declaration - this is the largest source of driver crashes
			if (CurPSO.GraphicsDesc.VertexDescriptor.Num() == 0)
			{
				// check against the list
				const TArray<FStableShaderKeyAndValue>* OriginalShaders = InverseMap.Find(CurPSO.GraphicsDesc.VertexShader);
				if (OriginalShaders == nullptr)
				{
					UE_LOGF(LogShaderPipelineCacheTools, Warning, "PSO with an empty vertex declaration and unknown VS %ls encountered, filtering out", *CurPSO.GraphicsDesc.VertexShader.ToString());
					++NumPSOsFilteredDueToEmptyDecls;
					continue;
				}

				// all shader classes need to be usabe with empty declarations for this to pass
				bool bAllShadersAllowed = true;
				for (const FStableShaderKeyAndValue& OriginalShader : *OriginalShaders)
				{
					if (!VShadersUsableWithEmptyVertexDecl.Contains(OriginalShader.ShaderType))
					{
						// if this shader has a vertex factory type associated, check if VF is known to have empty decl
						if (OriginalShader.VFType != UnknownVFType)
						{
							if (VFactoriesUsableWithEmptyVertexDecl.Contains(OriginalShader.VFType))
							{
								// allow, vertex factory can have an empty declaration
								continue;
							}

							// found an incompatible (possibly, but we will err on the side of caution) usage. Log it
							PossiblyIncorrectUsageWithEmptyDeclaration.Add(OriginalShader);
						}
						bAllShadersAllowed = false;
						break;
					}
				}

				if (!bAllShadersAllowed)
				{
					// skip this PSO
					++NumPSOsFilteredDueToEmptyDecls;
					continue;
				}
			}

			break;
		}
		case FPipelineCacheFileFormatPSO::DescriptorType::RayTracing:
		{
			break;
		}
		}

		// still used
		RetainedPSOs.Add(CurPSO);
	}

	InOutPSOs = RetainedPSOs;

	if (NumPSOsFilteredDueToEmptyDecls)
	{
		if (PossiblyIncorrectUsageWithEmptyDeclaration.Num())
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "");
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Also, PSOs with the following vertex shaders were filtered out because VS were not known to be used with an empty declaration. ");
			UE_LOGF(LogShaderPipelineCacheTools, Display, "Check compatibility in the code and possibly add to the allowed list if a known safe usage:");

			for (const FStableShaderKeyAndValue& Shader : PossiblyIncorrectUsageWithEmptyDeclaration)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Display, "  %ls", *Shader.ToString());
			}
		}
	}

	UE_LOGF(LogShaderPipelineCacheTools, Display, "=== Sanitizing results ===");
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Before sanitization: .................................................................... %6d PSOs", NumPSOsOriginal);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Filtered out due to inconsistent vertex declaration for the same vertex shader:.......... %6d PSOs", NumPSOsFilteredDueToInconsistentDecls);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Filtered out due to VS being possibly incompatible with an empty vertex declaration:..... %6d PSOs", NumPSOsFilteredDueToEmptyDecls);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Filtered out due to using possible unsupported RHI features:............................. %6d PSOs", NumPSOsFilteredDueToUsingPossibleUnsupportedRHIFeatures);
	UE_LOGF(LogShaderPipelineCacheTools, Display, "-----");
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Number of PSOs after sanity checks:...................................................... %6d PSOs", InOutPSOs.Num());
}

/** Adds compute PSOs directly from the stable shader map of this build */
void AddComputePSOs(TSet<FPipelineCacheFileFormatPSO>& OutPSOs, const FStableShaderSet& StableShaderMap)
{
	if (GShaderPipelineCacheTools_ComputePSOInclusionMode < 1)
	{
		return;
	}

	static FName NAME_SF_Compute("SF_Compute");
	static FName NAME_NiagaraShader("FNiagaraShader");

	for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableShaderMap)
	{
		if (ShaderKeyPtr->TargetFrequency == NAME_SF_Compute)
		{
			// add a new Compute PSO
			// Check if we are only allowed to add Niagara PSOs
			if (GShaderPipelineCacheTools_ComputePSOInclusionMode == 2 &&ShaderKeyPtr->ShaderType != NAME_NiagaraShader)
			{
				continue;
			}

			FPipelineCacheFileFormatPSO NewPso;
			NewPso.Type = FPipelineCacheFileFormatPSO::DescriptorType::Compute;
			NewPso.ComputeDesc.ComputeShader = ShaderKeyPtr->OutputHash;
			NewPso.UsageMask = uint64(-1);
			NewPso.BindCount = 0;
			OutPSOs.Add(NewPso);
		}
	}
}

/** Function that gets the target platform name from the stable map. Shouldn't exist, and we should be passing the target platform explicitly. */
FName GetTargetPlatformFromStableShaderKeys(const FStableShaderSet& StableShaderMap)
{
	if (const TUniquePtr<FStableShaderKeyAndValue>* ShaderKeyPtr = StableShaderMap.FindArbitraryElement())
	{
		return (*ShaderKeyPtr)->TargetPlatform;
	}

	return NAME_None;
}

/** 
 * Saves the cache file to be bundled with the game. If it finds chunk description infos (on disk), it splits the file into per-chunk ones.
 * 
 * @return commandlet return (0 is everything Ok, otherwise can return error codes 1-255, currently only 1 is used)
 */
int32 SaveBinaryPipelineCacheFile(const FString& OutputFilename, const EShaderPlatform ShaderPlatform, const FString& ShaderFormat, const FString& ChunkInfoFilesPath, const FString& AssociatedShaderLibraryName, const FString& TargetPlatformName, const TSet<FPipelineCacheFileFormatPSO>& PSOs, const FStableShaderSet& StableMap)
{
	auto SaveSingleCacheFile = [](const FString& OutputFilename, const EShaderPlatform ShaderPlatform, const TSet<FPipelineCacheFileFormatPSO>& PSOs) -> int32
	{
		if (IFileManager::Get().FileExists(*OutputFilename))
		{
			IFileManager::Get().Delete(*OutputFilename, false, true);
		}
		if (IFileManager::Get().FileExists(*OutputFilename))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Could not delete %ls", *OutputFilename);
		}
		if (!FPipelineFileCacheManager::SavePipelineFileCacheFrom(FShaderPipelineCache::GetGameVersionForPSOFileCache(), ShaderPlatform, OutputFilename, PSOs))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Error, "Failed to save %ls", *OutputFilename);
			return 1;
		}
		int64 Size = IFileManager::Get().FileSize(*OutputFilename);
		if (Size < 1)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Failed to write %ls", *OutputFilename);
		}

		// count PSOs
		const int32 NumGraphicsPSOs = Algo::Accumulate(PSOs, 0, [](int32 Acc, const FPipelineCacheFileFormatPSO& PSO) { return (PSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics) ? Acc + 1 : Acc; });
		const int32 NumComputePSOs = Algo::Accumulate(PSOs, 0, [](int32 Acc, const FPipelineCacheFileFormatPSO& PSO) { return (PSO.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute) ? Acc + 1 : Acc; });
		const int32 NumRTPSOs = PSOs.Num() - NumGraphicsPSOs - NumComputePSOs;

		UE_LOGF(LogShaderPipelineCacheTools, Display, "Wrote %d binary PSOs (graphics: %d compute: %d RT: %d), (%lldKB) to %ls",
			PSOs.Num(), NumGraphicsPSOs, NumComputePSOs, NumRTPSOs,
			(Size + 1023) / 1024, *OutputFilename);
		return 0;
	};

	// first, attempt to find chunk info files to determine if we need to split the archive
	TArray<FString> ChunkInfoFilenames;
	UE::PipelineCacheUtilities::FindAllChunkInfos(AssociatedShaderLibraryName, TargetPlatformName, ChunkInfoFilesPath, ChunkInfoFilenames);

	if (ChunkInfoFilenames.IsEmpty())
	{
		// monolithic cache, save and exit
		return SaveSingleCacheFile(OutputFilename, ShaderPlatform, PSOs);
	}
	else
	{
		// chunked cache, load chunk infos, split the cache and save

		// first, kick off a task to prepare new StableMap that is easier to compare against
		TMultiMap<FName, FShaderHash> StableNameMap;
		FGraphEventRef StableMapConvTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&StableNameMap, &StableMap]
			{
				for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)	// could be parallelized (skip first N*ThreadIdx iterations on each thread?)
				{
					//Ensure we extract only the package name from ClassNameAndObjectPath, Compute PSOs may include subobject paths (e.g. “.../Core_0.GPUComputeScript”), which previously
					//caused TSet<FName> Packages lookups to fail because they were using normal package paths (see ShadersInChunk)
					FName PackageName(*ShaderKeyPtr->ClassNameAndObjectPath.GetPackageNameOnly());
					StableNameMap.Add(PackageName, ShaderKeyPtr->OutputHash);
				}
			}, TStatId());

		// proceed with reading the chunk info files
		FCriticalSection ChunkIdsAndResultAccessLock;	// we don't expect collisions, but we cannot rule out some weirdness on disk like two chunk infos pointing at the same chunk. This is to detect this gracefully.
		TArray<int32> ChunkIds;	// which chunk info file references which chunk id. Protected by ChunkIdsAndResultAccessLock. This array begins filled with invalid ids, but once we read all info files it should have only valid ones.
		int32 OverallResult = 0;	// used to communicate errors back from worker threads. Also protected by ChunkIdsAndResultAccessLock.

		// fill chunkids with invalid one
		ChunkIds.Reserve(ChunkInfoFilenames.Num());
		const int32 kInvalidChunkId = MIN_int32;
		for (int32 Idx = 0, Num = ChunkInfoFilenames.Num(); Idx < Num; ++Idx)
		{
			ChunkIds.Add(kInvalidChunkId);
		}

		// This should be passed in.
		FString ChunkDir = FPaths::GetPath(OutputFilename);

		// prepare everything necessary to split the PSOs
		ParallelFor(ChunkInfoFilenames.Num(),
			[&ChunkDir, &ChunkInfoFilenames, &ChunkIds, &ChunkIdsAndResultAccessLock, &OverallResult, &StableNameMap, &StableMapConvTask, &ShaderPlatform, &ShaderFormat, &PSOs, &SaveSingleCacheFile](int32 Index)
			{
				int32 ChunkId;
				FString OutputFilename;
				TSet<FName> Packages;
				if (!UE::PipelineCacheUtilities::LoadChunkInfo(ChunkInfoFilenames[Index], ShaderFormat, ChunkId, OutputFilename, Packages))
				{
					FScopeLock Locker(&ChunkIdsAndResultAccessLock);
					UE_LOGF(LogShaderPipelineCacheTools, Error, "Error loading chunk info file %ls",
						*ChunkInfoFilenames[Index]);

					// refuse to process such file
					OverallResult = 1;
					return;
				}

				{
					FScopeLock Locker(&ChunkIdsAndResultAccessLock);
					// find out if any other file referenced the same chunkid
					for (int32 Idx = 0, Num = ChunkInfoFilenames.Num(); Idx < Num; ++Idx)
					{
						if (ChunkIds[Idx] == ChunkId)
						{
							UE_LOGF(LogShaderPipelineCacheTools, Error, "Error processing chunk info files: chunk info file %ls (%d-th) and %ls (%d-th) reference the same chunk Id %d",
								*ChunkInfoFilenames[Index], Index, *ChunkInfoFilenames[Idx], Idx, ChunkId);

							// refuse to process such file
							OverallResult = 1;
							return;
						}
					}

					ChunkIds[Index] = ChunkId;
				}

				// go through whole stablemap and filter by the package id
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(StableMapConvTask);

				TSet<FShaderHash> ShadersInChunk;
				for (const TPair<FName, FShaderHash>& Pair : StableNameMap)
				{	
					if (Packages.Contains(Pair.Key))
					{
						ShadersInChunk.Add(Pair.Value);
					}
				}
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "Shaders in chunk %d: %d, not in chunk: %d (not counting deduplicated)", ChunkId, ShadersInChunk.Num(), StableNameMap.Num() - ShadersInChunk.Num());

				// now go through all PSOs
				TSet<FPipelineCacheFileFormatPSO> PSOsInChunk;
				for (const FPipelineCacheFileFormatPSO& Item : PSOs)
				{
					if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
					{
						if (ShadersInChunk.Contains(Item.ComputeDesc.ComputeShader))
						{
							PSOsInChunk.Add(Item);
						}
					}
					else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
					{
						if ((Item.GraphicsDesc.VertexShader != FShaderHash() && ShadersInChunk.Contains(Item.GraphicsDesc.VertexShader))
							|| (Item.GraphicsDesc.MeshShader != FShaderHash() && ShadersInChunk.Contains(Item.GraphicsDesc.MeshShader))
							|| (Item.GraphicsDesc.FragmentShader != FShaderHash() && ShadersInChunk.Contains(Item.GraphicsDesc.FragmentShader))
							|| (Item.GraphicsDesc.GeometryShader != FShaderHash() && ShadersInChunk.Contains(Item.GraphicsDesc.GeometryShader))
							|| (Item.GraphicsDesc.AmplificationShader != FShaderHash() && ShadersInChunk.Contains(Item.GraphicsDesc.AmplificationShader))
							)
						{
							PSOsInChunk.Add(Item);
						}
					}
					else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
					{
						if (Item.RayTracingDesc.ShaderHash != FShaderHash() && ShadersInChunk.Contains(Item.RayTracingDesc.ShaderHash))
						{
							PSOsInChunk.Add(Item);
						}
					}
				}
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "PSOs in chunk %d: %d, not in chunk: %d", ChunkId, PSOsInChunk.Num(), PSOs.Num() - PSOsInChunk.Num());


				if (PSOsInChunk.Num())
				{
					FString FinalPath = FPaths::Combine(ChunkDir, OutputFilename);
					int32 Result = SaveSingleCacheFile(FinalPath, ShaderPlatform, PSOsInChunk);
					if (Result != 0)
					{
						FScopeLock Locker(&ChunkIdsAndResultAccessLock);
						UE_LOGF(LogShaderPipelineCacheTools, Error, "Couldn't write chunked cache for chunk %d (info %ls)",
							ChunkId, *ChunkInfoFilenames[Index]);
						OverallResult = 1;
						return;
					}
				}
			},
			EParallelForFlags::Unbalanced
		);

		// last check: all chunk info files should have resulted in proper chunk ids
		for (int32 Idx = 0, Num = ChunkInfoFilenames.Num(); Idx < Num; ++Idx)
		{
			if (ChunkIds[Idx] == kInvalidChunkId)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Chunk info %ls somehow didn't yield a valid chunk id",
					*ChunkInfoFilenames[Idx]);
				OverallResult = 1;
			}
		}

		return OverallResult;
	}
}

int32 BuildPSOSC(const TArray<FString>& Tokens, const TMap<FString, FString>& ParamVals)
{
	check(Tokens.Last().EndsWith(TEXT(".upipelinecache")));

	TArray<FStringView, TInlineAllocator<16>> StableShaderFiles;
	TArray<FString> StablePipelineCacheFiles;
	bool bHaveBinaryStableCacheFormat = false;
	bool bHaveDeprecatedCSVFormat = false;
	FString ChunkInfoFilesPath, AssociatedShaderLibraryName, TargetPlatformName;

	if (const FString* Param = ParamVals.Find(TEXT("chunkinfodir")))
	{
		ChunkInfoFilesPath = *Param;
	}

	if (const FString* Param = ParamVals.Find(TEXT("library")))
	{
		AssociatedShaderLibraryName = *Param;
	}

	if (const FString* Param = ParamVals.Find(TEXT("platform")))
	{
		TargetPlatformName = *Param;
	}

	for (int32 Index = 0; Index < Tokens.Num() - 1; Index++)
	{
		if (Tokens[Index].EndsWith(ShaderStableKeysFileExt))
		{
			StableShaderFiles.Add(Tokens[Index]);
		}
		else if (!bHaveBinaryStableCacheFormat && Tokens[Index].EndsWith(ShaderStablePipelineFileExt))
		{
			bHaveBinaryStableCacheFormat = true;
		}
		else if (Tokens[Index].EndsWith(STABLE_CSV_EXT) || Tokens[Index].EndsWith(STABLE_CSV_COMPRESSED_EXT))
		{
			bHaveDeprecatedCSVFormat = true;
			UE_LOGF(LogShaderPipelineCacheTools, Warning, "Using stable pipeline cache in a deprecated text format: %ls", *Tokens[Index]);
		}
	}

	// Get the stable PC files in date order - least to most important(!?)
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Sorting input stable cache files into chronological order for merge processing...");
	FilenameFilterFN ExtensionFilterFn = [](const FString& Filename)
	{
		return Filename.EndsWith(ShaderStablePipelineFileExt) || Filename.EndsWith(STABLE_CSV_EXT) || Filename.EndsWith(STABLE_CSV_COMPRESSED_EXT);
	};
	BuildDateSortedListOfFiles(Tokens, ExtensionFilterFn, StablePipelineCacheFiles);

	// Start populating the files with stable keys in a task.
	FStableShaderSet StableMap;
	FGraphEventRef StableMapTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&StableShaderFiles, &StableMap]
	{
		LoadStableShaderKeysMultiple(StableMap, StableShaderFiles);
		if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
		{
			UE_LOGF(LogShaderPipelineCacheTools, Verbose, "    %ls", *FStableShaderKeyAndValue::HeaderLine());
			for (const TUniquePtr<FStableShaderKeyAndValue>& ShaderKeyPtr : StableMap)
			{
				UE_LOGF(LogShaderPipelineCacheTools, Verbose, "    %ls", *ShaderKeyPtr->ToString());
			}
			StableShadersSerializationSelfTest(StableMap);
		}
		UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d unique shader info lines total.", StableMap.Num());
	}, TStatId());

	// Read the stable PSO sets in parallel with the stable shaders.
	TArray<TSet<FPipelineCacheFileFormatPSO>> PSOsByFile;
	PSOsByFile.AddDefaulted(StablePipelineCacheFiles.Num());

	FGraphEventArray LoadPSOTasks;
	LoadPSOTasks.AddDefaulted(StablePipelineCacheFiles.Num());

	TArray<TArray<FString>> StableCSVs;
	StableCSVs.AddDefaulted(StablePipelineCacheFiles.Num());

	FGraphEventArray ParsePSOTasks;
	ParsePSOTasks.AddDefaulted(StablePipelineCacheFiles.Num());

	TArray<FName> TargetShaderFormatByFile;
	TargetShaderFormatByFile.AddDefaulted(StablePipelineCacheFiles.Num());

	// Check if we had any of the stable caches in the old textual format and process them the old way
	if (bHaveDeprecatedCSVFormat)
	{
		for (int32 FileIndex = 0; FileIndex < StablePipelineCacheFiles.Num(); ++FileIndex)
		{
			if (StablePipelineCacheFiles[FileIndex].EndsWith(STABLE_CSV_EXT) || StablePipelineCacheFiles[FileIndex].EndsWith(STABLE_CSV_COMPRESSED_EXT))
			{
				LoadPSOTasks[FileIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady([&StableCSV = StableCSVs[FileIndex], &FileName = StablePipelineCacheFiles[FileIndex]]
					{
						if (!LoadStableCSV(FileName, StableCSV))
						{
							UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Could not load %ls", *FileName);
						}
					}, TStatId());
			}
		}

		// Parse the stable PSO sets in parallel once both the stable shaders and the corresponding read are complete.
		for (int32 FileIndex = 0; FileIndex < StablePipelineCacheFiles.Num(); ++FileIndex)
		{
			if (StablePipelineCacheFiles[FileIndex].EndsWith(STABLE_CSV_EXT) || StablePipelineCacheFiles[FileIndex].EndsWith(STABLE_CSV_COMPRESSED_EXT))
			{
				const FGraphEventArray PreReqs{ StableMapTask, LoadPSOTasks[FileIndex] };
				ParsePSOTasks[FileIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady(
					[&PSOs = PSOsByFile[FileIndex],
					&FileName = StablePipelineCacheFiles[FileIndex],
					&StableCSV = StableCSVs[FileIndex],
					&StableMap,
					&TargetShaderFormat = TargetShaderFormatByFile[FileIndex]]
					{
						int32 PSOsRejected = 0, PSOsMerged = 0;
						PSOs = ParseStableCSV(FileName, StableCSV, StableMap, TargetShaderFormat, PSOsRejected, PSOsMerged);
						StableCSV.Empty();
						UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d PSO lines from %ls. %d lines rejected, %d lines merged", PSOs.Num(), *FileName, PSOsRejected, PSOsMerged);
					}, TStatId(), & PreReqs);
			}
		}
	}

	if (bHaveBinaryStableCacheFormat)
	{
		for (int32 FileIndex = 0; FileIndex < StablePipelineCacheFiles.Num(); ++FileIndex)
		{
			if (StablePipelineCacheFiles[FileIndex].EndsWith(ShaderStablePipelineFileExt))
			{
				const FGraphEventArray PreReqs{ StableMapTask };
				ParsePSOTasks[FileIndex] = FFunctionGraphTask::CreateAndDispatchWhenReady(
					[&PSOs = PSOsByFile[FileIndex],
					&FileName = StablePipelineCacheFiles[FileIndex],
					&StableMap,
					&TargetShaderFormat = TargetShaderFormatByFile[FileIndex]]
					{
						int32 PSOsRejected = 0, PSOsMerged = 0;
						if (UE::PipelineCacheUtilities::LoadStablePipelineCacheFile(FileName, StableMap, PSOs, TargetShaderFormat, PSOsRejected, PSOsMerged))
						{
							UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d stable PSOs from %ls. %d PSOs rejected, %d PSOs merged", PSOs.Num(), *FileName, PSOsRejected, PSOsMerged);
						}
					}, TStatId(), &PreReqs);
			}
		}
	}

	// Always wait for these tasks before returning from this function.
	// This is necessary if there is an error or if nothing consumes the stable map.
	ON_SCOPE_EXIT
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(StableMapTask);
		FTaskGraphInterface::Get().WaitUntilTasksComplete(ParsePSOTasks);
	};

	// Validate and merge the stable PSO sets sequentially as they finish.
	TSet<FPipelineCacheFileFormatPSO> PSOs;
	TMap<uint32,int64> PSOAvgIterations;
	uint32 MergeCount = 0;
	FName TargetShaderFormat;

	for (int32 FileIndex = 0; FileIndex < StablePipelineCacheFiles.Num(); ++FileIndex)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(ParsePSOTasks[FileIndex]);

		if (!PSOsByFile[FileIndex].Num())
		{
			if (GShaderPipelineCacheTools_IgnoreObsoleteStableCacheFiles)
			{
				continue;
			}
			else
			{
				return 1;
			}
		}

		check(TargetShaderFormat == NAME_None || TargetShaderFormat == TargetShaderFormatByFile[FileIndex]);
		TargetShaderFormat = TargetShaderFormatByFile[FileIndex];

		TSet<FPipelineCacheFileFormatPSO>& CurrentFilePSOs = PSOsByFile[FileIndex];

		// Now merge this file PSO set with main PSO set (this is going to be slow as we need to incrementally reprocess each existing PSO per file to get reasonable bindcount averages).
		// Can't sum all and avg: A) Overflow and B) Later ones want to remain high so only start to get averaged from the point they are added onwards:
		// 1) New PSO goes in with it's bindcount intact for this iteration - if it's the last file then it keeps it bindcount
		// 2) Existing PSO from older file gets incrementally averaged with PSO bindcount from new file
		// 3) Existing PSO from older file not in new file set gets incrementally averaged with zero - now less important
		// 4) PSOs are incrementally averaged from the point they are seen - i.e. a PSO seen in an earler file will get averaged more times than one
		//		seen in a later file using:  NewAvg = OldAvg + (NewValue - OldAvg) / CountFromPSOSeen
		//
		// Proof for incremental averaging:
		//	DataSet = {25 65 95 128}; Standard Average = (sum(25, 65, 95, 128) / 4) = 78.25
		//	Incremental:
		//	=> 25
		//	=> 25 + (65 - 25) / 2 = A 		==> 25 + (65 - 25) / 2 		= 45
		//	=>  A + (95 -  A) / 3 = B 		==> 45 + (95 - 45) / 3 		= 61 2/3
		//	=>  B + (128 - B) / 4 = Answer 	==> 61 2/3 + (128 - B) / 4 	= 78.25

		for (FPipelineCacheFileFormatPSO& PSO : PSOs)
		{
			// Already existing PSO in the next file round - increase its average iteration
			int64& PSOAvgIteration = PSOAvgIterations.FindChecked(GetTypeHash(PSO));
			++PSOAvgIteration;

			// Default the bindcount
			int64 NewBindCount = 0ll;

			// If you have the same PSO in the new file set
			if (FPipelineCacheFileFormatPSO* NewFilePSO = CurrentFilePSOs.Find(PSO))
			{
				// Sanity check!
				check(*NewFilePSO == PSO);

				// Get More accurate stats by testing for diff - we could just merge and be done
				if ((PSO.UsageMask & NewFilePSO->UsageMask) != NewFilePSO->UsageMask)
				{
					PSO.UsageMask |= NewFilePSO->UsageMask;
					++MergeCount;
				}

				NewBindCount = NewFilePSO->BindCount;

				// Remove from current file set - it's already there and we don't want any 'overwrites'
				CurrentFilePSOs.Remove(*NewFilePSO);
			}

			// Incrementally average this PSO bindcount - if not found in this set then avg will be pulled down
			PSO.BindCount += (NewBindCount - PSO.BindCount) / PSOAvgIteration;
		}

		// Add the leftover PSOs from the current file and initialize their iteration count.
		for (const FPipelineCacheFileFormatPSO& PSO : CurrentFilePSOs)
		{
			PSOAvgIterations.Add(GetTypeHash(PSO), 1ll);
		}
		PSOs.Append(MoveTemp(CurrentFilePSOs));
	}
	UE_LOGF(LogShaderPipelineCacheTools, Display, "Re-deduplicated into %d binary PSOs [Usage Mask Merged = %d].", PSOs.Num(), MergeCount);

	// need to make sure that the stable map task is done at this point (if there are no graphics PSOs it may not yet be)
	FTaskGraphInterface::Get().WaitUntilTaskCompletes(StableMapTask);

	AddComputePSOs(PSOs, StableMap);

	if (PSOs.Num() < 1)
	{
		// Previously, this commandlet was run only when we had stable cache files, so not creating any PSO was definitely a warning.
		// Now, we add some PSOs cook-time, so it is run pretty much always, including when there are no recorded stable files. Do not
		// issue a warning if cook-time addition resulted in no PSOs, and continue to issue in the case we had recorded files.
		if (StablePipelineCacheFiles.Num() > 0)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Warning, "No PSOs were created!");
		}
		else
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "No PSOs were created.");
		}
		return 0;
	}

	FilterInvalidPSOs(PSOs, StableMap);

	if (UE_LOG_ACTIVE(LogShaderPipelineCacheTools, Verbose))
	{
		for (const FPipelineCacheFileFormatPSO& Item : PSOs)
		{
			FString StringRep;
			if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Compute)
			{
				check(!(Item.ComputeDesc.ComputeShader == FShaderHash()));
				StringRep = Item.ComputeDesc.ToString();
			}
			else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
			{
				check(Item.GraphicsDesc.VertexShader != FShaderHash() || Item.GraphicsDesc.MeshShader != FShaderHash());
				StringRep = Item.GraphicsDesc.ToString();
			}
			else if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::RayTracing)
			{
				check(Item.RayTracingDesc.ShaderHash != FShaderHash());
				StringRep = Item.RayTracingDesc.ToString();
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Error, "Unexpected pipeline cache descriptor type %d", int32(Item.Type));
			}
			UE_LOGF(LogShaderPipelineCacheTools, Verbose, "%ls", *StringRep);
		}
	}

	if (TargetShaderFormat == NAME_None)
	{
		// get it from the StableMap
		TargetShaderFormat = GetTargetPlatformFromStableShaderKeys(StableMap);
	}
	check(TargetShaderFormat != NAME_None);
	EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(TargetShaderFormat);
	check(Platform != SP_NumPlatforms);

	if (IsOpenGLPlatform(Platform))
	{
		UE_LOGF(LogShaderPipelineCacheTools, Display, "OpenGL detected, reducing PSOs to be BSS only as OpenGL doesn't care about the state at all when compiling shaders.");

		TSet<FPipelineCacheFileFormatPSO> KeptPSOs;

		// N^2 not good. 
		for (const FPipelineCacheFileFormatPSO& Item : PSOs)
		{
			bool bMatchedKept = false;
			if (Item.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
			{
				for (const FPipelineCacheFileFormatPSO& TestItem : KeptPSOs)
				{
					if (TestItem.Type == FPipelineCacheFileFormatPSO::DescriptorType::Graphics)
					{
						if (
							TestItem.GraphicsDesc.VertexShader == Item.GraphicsDesc.VertexShader &&
							TestItem.GraphicsDesc.MeshShader == Item.GraphicsDesc.MeshShader &&
							TestItem.GraphicsDesc.AmplificationShader == Item.GraphicsDesc.AmplificationShader &&
							TestItem.GraphicsDesc.FragmentShader == Item.GraphicsDesc.FragmentShader &&
							TestItem.GraphicsDesc.GeometryShader == Item.GraphicsDesc.GeometryShader
							)
						{
							bMatchedKept = true;
							break;
						}
					}
				}
			}
			if (!bMatchedKept)
			{
				KeptPSOs.Add(Item);
			}
		}
		Exchange(PSOs, KeptPSOs);
		UE_LOGF(LogShaderPipelineCacheTools, Display, "BSS only reduction produced %d binary PSOs.", PSOs.Num());

		if (PSOs.Num() < 1)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Warning, "No PSOs were created!");
			return 0;
		}

	}

	return SaveBinaryPipelineCacheFile(Tokens.Last(), Platform, TargetShaderFormat.ToString(), ChunkInfoFilesPath, AssociatedShaderLibraryName, TargetPlatformName, PSOs, StableMap);
}


int32 DiffStable(const TArray<FString>& Tokens)
{
	TArray<TSet<FString>> Sets;
	for (int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++)
	{
		const FString& Filename = Tokens[TokenIndex];
		bool bCompressed = Filename.EndsWith(STABLE_CSV_COMPRESSED_EXT);
		if (!bCompressed && !Filename.EndsWith(STABLE_CSV_EXT))
		{
			check(0);
			continue;
		}
			   
		UE_LOGF(LogShaderPipelineCacheTools, Display, "Loading %ls....", *Filename);
		TArray<FString> SourceFileContents;
		if (LoadStableCSV(Filename, SourceFileContents) || SourceFileContents.Num() < 2)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Fatal, "Could not load %ls", *Filename);
			return 1;
		}

		UE_LOGF(LogShaderPipelineCacheTools, Display, "Loaded %d stable PSO lines.", SourceFileContents.Num() - 1);

		Sets.AddDefaulted();

		for (int32 Index = 1; Index < SourceFileContents.Num(); Index++)
		{
			Sets.Last().Add(SourceFileContents[Index]);
		}
	}
	TSet<FString> Inter;
	for (int32 TokenIndex = 0; TokenIndex < Sets.Num(); TokenIndex++)
	{
		if (TokenIndex)
		{
			Inter = Sets[TokenIndex];
		}
		else
		{
			Inter = Inter.Intersect(Sets[TokenIndex]);
		}
	}

	for (int32 TokenIndex = 0; TokenIndex < Sets.Num(); TokenIndex++)
	{
		TSet<FString> InterSet = Sets[TokenIndex].Difference(Inter);

		UE_LOGF(LogShaderPipelineCacheTools, Display, "********************* Loaded %d not in others %ls", InterSet.Num(), *Tokens[TokenIndex]);
		for (const FString& Item : InterSet)
		{
			UE_LOGF(LogShaderPipelineCacheTools, Display, "    %ls", *Item);
		}
	}
	return 0;
}

int32 DecompressCSV(const TArray<FString>& Tokens)
{
	TArray<FString> DecompressedData;
	for (int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++)
	{
		const FString& CompressedFilename = Tokens[TokenIndex];
		if (!CompressedFilename.EndsWith(STABLE_CSV_COMPRESSED_EXT))
		{
			continue;
		}

		FString CombinedCSV;
		DecompressedData.Reset();
		if (LoadAndDecompressStableCSV(CompressedFilename, DecompressedData))
		{
			FString FilenameCSV = CompressedFilename.LeftChop(STABLE_COMPRESSED_EXT_LEN);
			FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*FilenameCSV);

			for (const FString& LineCSV : DecompressedData)
			{
				CombinedCSV.Append(LineCSV);
				CombinedCSV.Append(LINE_TERMINATOR);

				if ((int64)(CombinedCSV.Len() * sizeof(TCHAR)) >= (int64)(MAX_int32 - 1024 * 1024))
				{
					FFileHelper::SaveStringToFile(CombinedCSV, *FilenameCSV, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), FILEWRITE_Append);
					CombinedCSV.Empty();
				}
			}

			FFileHelper::SaveStringToFile(CombinedCSV, *FilenameCSV, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), FILEWRITE_Append);
		}
	}

	return 0;
}

UShaderPipelineCacheToolsCommandlet::UShaderPipelineCacheToolsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UShaderPipelineCacheToolsCommandlet::Main(const FString& Params)
{
	return StaticMain(Params);
}

int32 UShaderPipelineCacheToolsCommandlet::StaticMain(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	if (Tokens.Num() >= 1)
	{
		ExpandWildcards(Tokens);
		if (Tokens[0] == TEXT("Expand") && Tokens.Num() >= 4)
		{
			Tokens.RemoveAt(0);
			return ExpandPSOSC(Tokens);
		}
		else if (Tokens[0] == TEXT("Build"))
		{
			// 3 tokens at a minimum, as stablepc is optional: build [stablepc] stablekey outputfile
			if (Tokens.Num() >= 3)
			{
				Tokens.RemoveAt(0);
				return BuildPSOSC(Tokens, ParamVals);
			}
		}
		else if (Tokens[0] == TEXT("Diff") && Tokens.Num() >= 3)
		{
			Tokens.RemoveAt(0);
			return DiffStable(Tokens);
		}
		else if (Tokens[0] == TEXT("Dump") && Tokens.Num() >= 2)
		{
			Tokens.RemoveAt(0);
			for (int32 Index = 0; Index < Tokens.Num(); Index++)
			{
				if (Tokens[Index].EndsWith(TEXT(".upipelinecache")))
				{
					// check if there's more arguments and assume it's the directory for stable keys
					FString StableKeysDir;
					if (Index < Tokens.Num() - 1)
					{
						StableKeysDir = Tokens[Index + 1];
					}
					return DumpPSOSC(Tokens[Index], StableKeysDir);
				}
				if (Tokens[Index].EndsWith(ShaderStableKeysFileExt))
				{
					return DumpStableKeysFile(Tokens[Index], ParamVals);
				}
			}
		}
		else if (Tokens[0] == TEXT("CheckShkAlias") && Tokens.Num() >= 2)
		{
			Tokens.RemoveAt(0);
			return CheckStableKeyAliasing(Tokens);
		}
		else if (Tokens[0] == TEXT("Decompress") && Tokens.Num() >= 2)
		{
			Tokens.RemoveAt(0);
			return DecompressCSV(Tokens);
		}
		else if (Tokens[0] == TEXT("Extract") && Tokens.Num() >= 3)
		{
			Tokens.RemoveAt(0);
			if (Tokens[0].EndsWith(TEXT(".ushaderbytecode")))
			{
				return ExtractShaderFromUshaderbytecode(Tokens[0], Tokens[1]);
			}
			else
			{
				UE_LOGF(LogShaderPipelineCacheTools, Warning, "Extract command expects a ushaderbytecode parameters as its first argument.\n");
			}
		}
	}
	
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Extract Foo.ushaderbytecode 123ABC (shader hash) - extracts all shaders that match the hash from this ushaderbytecode (and decompresses them).\n");
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Dump SCLInfo.%ls [...] [-outputhash=LIST] [-objname=LIST] - dumps stable keys file, optionally only dumping entries that match the specified output hashes and/or object names (hash and name lists are comma-separated).\n", ShaderStableKeysFileExt);
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Dump PSOCache.upipelinecache [SCLInfo.%ls] - dumps recorded PSOs. If optional stable keys file, converts shader hashes to readable stable shader descriptions.\n", ShaderStableKeysFileExt);
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Dump StablePSOCache.csv[.compressed] - dumps content of a stable PSO.\n");
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Diff ShaderCache1.stablepc.csv ShaderCache1.stablepc.csv [...]]\n");
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Expand Input1.upipelinecache Dir2/*.upipelinecache InputSCLInfo1.%ls Dir2/*.%ls InputSCLInfo3.%ls [...] Output.stablepc.csv\n", ShaderStableKeysFileExt, ShaderStableKeysFileExt, ShaderStableKeysFileExt);
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Build Input.stablepc.csv InputDir2/*.stablepc.csv InputSCLInfo1.%ls Dir2/*.%ls InputSCLInfo3.%ls [...] Output.upipelinecache\n", ShaderStableKeysFileExt, ShaderStableKeysFileExt, ShaderStableKeysFileExt);
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: Decompress Input1.stablepc.csv.compressed Input2.stablepc.csv.compressed [...]\n");
	UE_LOGF(LogShaderPipelineCacheTools, Warning, "Usage: All commands accept stablepc.csv.compressed instead of stablepc.csv for compressing output\n");
	return 0;
}
