// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/StabilizeLocalizationKeys.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "Serialization/ArchiveUObject.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Misc/PackageName.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/Linker.h"
#include "Internationalization/InternationalizationManifest.h"
#include "Internationalization/InternationalizationArchive.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Serialization/JsonInternationalizationManifestSerializer.h"
#include "Serialization/JsonInternationalizationArchiveSerializer.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Templates/UniquePtr.h"
#include "LocalizedAssetUtil.h"
#include "LocalizationSourceControlUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StabilizeLocalizationKeys)

DEFINE_LOG_CATEGORY_STATIC(LogStabilizeLocalizationKeys, Log, All);

#if USE_STABLE_LOCALIZATION_KEYS

struct FLocTextIdentity
{
public:
	FLocTextIdentity(FString InNamespace, FString InKey)
		: Namespace(MoveTemp(InNamespace))
		, Key(MoveTemp(InKey))
		, Hash(0)
	{
		Hash = FCrc::StrCrc32(*Namespace, Hash);
		Hash = FCrc::StrCrc32(*Key, Hash);
	}

	FORCEINLINE const FString& GetNamespace() const
	{
		return Namespace;
	}

	FORCEINLINE const FString& GetKey() const
	{
		return Key;
	}

	FORCEINLINE bool operator==(const FLocTextIdentity& Other) const
	{
		return Namespace.Equals(Other.Namespace, ESearchCase::CaseSensitive)
			&& Key.Equals(Other.Key, ESearchCase::CaseSensitive);
	}

	FORCEINLINE bool operator!=(const FLocTextIdentity& Other) const
	{
		return !Namespace.Equals(Other.Namespace, ESearchCase::CaseSensitive)
			|| !Key.Equals(Other.Key, ESearchCase::CaseSensitive);
	}

	friend inline uint32 GetTypeHash(const FLocTextIdentity& Id)
	{
		return Id.Hash;
	}

private:
	FString Namespace;
	FString Key;
	uint32 Hash;
};

class FTextKeyingArchive : public FArchiveUObject
{
public:
	FTextKeyingArchive(UPackage* InPackage, TMap<FLocTextIdentity, FLocTextIdentity>& InOutPackageTextKeyMap)
		: PackageTextKeyMap(&InOutPackageTextKeyMap)
		, PackageNamespace(TextNamespaceUtil::EnsurePackageNamespace(InPackage))
	{
		this->SetIsSaving(true);

		TArray<UObject*> AllObjectsInPackage;
		GetObjectsWithOuter(InPackage, AllObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);

		for (UObject* Obj : AllObjectsInPackage)
		{
			ProcessObject(Obj);
		}
	}

	void ProcessObject(UObject* Obj)
	{
		// User Defined Structs need some special handling as they store their default data in a way that serialize doesn't pick up
		if (UUserDefinedStruct* const UserDefinedStruct = Cast<UUserDefinedStruct>(Obj))
		{
			if (UUserDefinedStructEditorData* UDSEditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData))
			{
				for (FStructVariableDescription& StructVariableDesc : UDSEditorData->VariablesDescriptions)
				{
					static const FName TextCategory = TEXT("text"); // Must match UEdGraphSchema_K2::PC_Text
					if (StructVariableDesc.Category == TextCategory)
					{
						FText StructVariableValue;
						if (FTextStringHelper::ReadFromBuffer(*StructVariableDesc.DefaultValue, StructVariableValue) && KeyText(StructVariableValue))
						{
							FTextStringHelper::WriteToBuffer(StructVariableDesc.DefaultValue, StructVariableValue);
						}
					}
				}
			}
		}

		Obj->Serialize(*this);
	}

	bool KeyText(FText& InOutText)
	{
		if (!InOutText.ShouldGatherForLocalization())
		{
			return false;
		}

		const FTextId TextId = FTextInspector::GetTextId(InOutText);
		if (TextId.IsEmpty())
		{
			return false;
		}

		const FString Namespace = TextId.GetNamespace().ToString();
		const FString Key = TextId.GetKey().ToString();

		const FString CurrentPackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(Namespace);
		if (CurrentPackageNamespace.Equals(PackageNamespace, ESearchCase::CaseSensitive))
		{
			return false;
		}

		FString NewNamespace;
		FString NewKey;

		const FLocTextIdentity* ExistingMapping = PackageTextKeyMap->Find(FLocTextIdentity(Namespace, Key));
		if (ExistingMapping)
		{
			NewNamespace = ExistingMapping->GetNamespace();
			NewKey = ExistingMapping->GetKey();
		}
		else
		{
			// We only want to stabilize actual asset content - these all have GUID based keys, as prior to stable keys, you could never set a non-GUID based key in an asset (it must have come from C++)
			{
				FGuid KeyGuid;
				if (!FGuid::Parse(Key, KeyGuid))
				{
					return false;
				}
			}

			NewNamespace = TextNamespaceUtil::BuildFullNamespace(Namespace, PackageNamespace, /*bAlwaysApplyPackageNamespace*/true);
			NewKey = Key;
			
			PackageTextKeyMap->Add(FLocTextIdentity(Namespace, Key), FLocTextIdentity(NewNamespace, NewKey));
		}

		InOutText = FText::ChangeKey(NewNamespace, NewKey, InOutText);
		return true;
	}

	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<(FText& Value) override
	{
		KeyText(Value);
		return *this;
	}

private:
	TMap<FLocTextIdentity, FLocTextIdentity>* PackageTextKeyMap;
	FString PackageNamespace;
};

struct FLocArchiveInfo
{
	FLocArchiveInfo(FString InFilename, TSharedRef<FInternationalizationArchive> InArchive)
		: Filename(MoveTemp(InFilename))
		, Archive(MoveTemp(InArchive))
		, bHasArchiveChanged(false)
	{
	}

	FString Filename;
	TSharedRef<FInternationalizationArchive> Archive;
	bool bHasArchiveChanged;
};

int32 UStabilizeLocalizationKeysCommandlet::Main(const FString& Params)
{
	// Parse command line
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, Parameters);

	TSharedPtr<FLocalizationSCC> SourceControlInfo;
	const bool bEnableSourceControl = Switches.Contains(TEXT("EnableSCC"));
	if (bEnableSourceControl)
	{
		SourceControlInfo = MakeShareable(new FLocalizationSCC());

		FText SCCErrorStr;
		if (!SourceControlInfo->IsReady(SCCErrorStr))
		{
			UE_LOGF(LogStabilizeLocalizationKeys, Error, "Revision Control error: %ls", *SCCErrorStr.ToString());
			return -1;
		}
	}

	const bool bIncludeEngineContent = Switches.Contains(TEXT("IncludeEngine"));
	const bool bIncludeGameContent = Switches.Contains(TEXT("IncludeGame"));
	const bool bIncludePluginContent = Switches.Contains(TEXT("IncludePlugins"));
	const FString NativeCulture = Parameters.FindRef(TEXT("NativeCulture"));

	TArray<FString> RootContentPaths;
	FPackageName::QueryRootContentPaths(RootContentPaths);

	TArray<FString> AllPackages;
	for (const FString& RootContentPath : RootContentPaths)
	{
		// Passes path filter?
		const bool bIsEnginePath = RootContentPath.Equals(TEXT("/Engine/"));
		const bool bIsGamePath = RootContentPath.Equals(TEXT("/Game/"));
		const bool bIsPluginPath = !bIsEnginePath && !bIsGamePath;
		if ((bIsEnginePath && !bIncludeEngineContent) || (bIsGamePath && !bIncludeGameContent) || (bIsPluginPath && !bIncludePluginContent))
		{
			UE_LOGF(LogStabilizeLocalizationKeys, Display, "Skipping path '%ls' as it doesn't pass the filter.", *RootContentPath);
			continue;
		}

		FString RootContentFilePath;
		if (!FPackageName::TryConvertLongPackageNameToFilename(RootContentPath, RootContentFilePath))
		{
			UE_LOGF(LogStabilizeLocalizationKeys, Display, "Skipping path '%ls' as it failed to convert to a file path.", *RootContentPath);
			continue;
		}

		FPackageName::FindPackagesInDirectory(AllPackages, RootContentFilePath);
	}

	// Work out which packages need to be stabilized
	TArray<FString> UnstablePackages;
	for (const FString& PackageFilename : AllPackages)
	{
		if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*PackageFilename)))
		{
			// Read package file summary from the file so we can test the version
			FPackageFileSummary PackageFileSummary;
			(*FileReader) << PackageFileSummary;

			const bool bRequiresKeyStabilization = !!(PackageFileSummary.GetPackageFlags() & PKG_RequiresLocalizationGather);
			if (bRequiresKeyStabilization)
			{
				UnstablePackages.Add(PackageFilename);
			}
		}
	}

	// Re-key the unstable packages (in batches so we can GC at reasonable points)
	TMultiMap<FLocTextIdentity, FLocTextIdentity> TextKeyMap;
	{
		static const int32 PackagesPerBatch = 100;

		const int32 NumPackages = UnstablePackages.Num();
		for (int32 PackageIndex = 0; PackageIndex < NumPackages;)
		{
			const int32 PackageLimitForBatch = FMath::Min(PackageIndex + PackagesPerBatch, NumPackages);
			for (; PackageIndex < PackageLimitForBatch; ++PackageIndex)
			{
				const FString& PackageFilename = UnstablePackages[PackageIndex];
				UE_LOGF(LogStabilizeLocalizationKeys, Display, "Loading package %d of %d: '%ls'.", PackageIndex + 1, NumPackages, *PackageFilename);

				UPackage* Package = LoadPackage(nullptr, *PackageFilename, LOAD_NoWarn | LOAD_Quiet);
				if (!Package)
				{
					UE_LOGF(LogStabilizeLocalizationKeys, Error, "Failed to load package from: '%ls'.", *PackageFilename);
				}
				else if (Package->RequiresLocalizationGather())
				{
					// Re-key the texts in the package
					TMap<FLocTextIdentity, FLocTextIdentity> PackageTextKeyMap;
					FTextKeyingArchive(Package, PackageTextKeyMap);

					if (PackageTextKeyMap.Num() > 0)
					{
						UE_LOGF(LogStabilizeLocalizationKeys, Display, "\t%d texts stabilized in: '%ls'.", PackageTextKeyMap.Num(), *PackageFilename);

						for (const auto& IdPair : PackageTextKeyMap)
						{
							TextKeyMap.Add(IdPair.Key, IdPair.Value);
						}

						FLocalizedAssetSCCUtil::SavePackageWithSCC(SourceControlInfo, Package, PackageFilename);
					}
				}
			}

			CollectGarbage(RF_NoFlags);
		}
	}

	if (TextKeyMap.Num() > 0 && !NativeCulture.IsEmpty())
	{
		// Load up the manifests and archives - the native archives are used to find the correct source text for the foreign archives
		TArray<TSharedRef<FInternationalizationManifest>> Manifests;
		TArray<FLocArchiveInfo> NativeLocArchives;
		TArray<FLocArchiveInfo> ForeignLocArchives;
		{
			TArray<FString> LocalizationPaths;
			if (bIncludeEngineContent)
			{
				LocalizationPaths += FPaths::GetEngineLocalizationPaths();
				LocalizationPaths += FPaths::GetEditorLocalizationPaths();
			}
			if (bIncludeGameContent)
			{
				LocalizationPaths += FPaths::GetGameLocalizationPaths();
			}

			TArray<FString> ManifestFilenames;
			for (const FString& LocalizationPath : LocalizationPaths)
			{
				IFileManager::Get().FindFilesRecursive(ManifestFilenames, *LocalizationPath, TEXT("*.manifest"), /*Files*/true, /*Directories*/false, /*bClearFileNames*/false);
			}

			for (const FString& ManifestFilename : ManifestFilenames)
			{
				TSharedRef<FInternationalizationManifest> InternationalizationManifest = MakeShareable(new FInternationalizationManifest());
				if (FJsonInternationalizationManifestSerializer::DeserializeManifestFromFile(ManifestFilename, InternationalizationManifest))
				{
					Manifests.Add(InternationalizationManifest);
				}
			}

			TArray<FString> ArchiveFilenames;
			for (const FString& LocalizationPath : LocalizationPaths)
			{
				IFileManager::Get().FindFilesRecursive(ArchiveFilenames, *LocalizationPath, TEXT("*.archive"), /*Files*/true, /*Directories*/false, /*bClearFileNames*/false);
			}

			for (const FString& ArchiveFilename : ArchiveFilenames)
			{
				TSharedRef<FInternationalizationArchive> InternationalizationArchive = MakeShareable(new FInternationalizationArchive());
				if (FJsonInternationalizationArchiveSerializer::DeserializeArchiveFromFile(ArchiveFilename, InternationalizationArchive, nullptr, nullptr))
				{
					const FString ArchivePath = FPaths::GetPath(ArchiveFilename);
					const bool bIsNativeArchive = ArchivePath.EndsWith(NativeCulture);
					if (bIsNativeArchive)
					{
						NativeLocArchives.Add(FLocArchiveInfo(ArchiveFilename, InternationalizationArchive));
					}
					else
					{
						ForeignLocArchives.Add(FLocArchiveInfo(ArchiveFilename, InternationalizationArchive));
					}
				}
			}
		}

		// Update archives to preserve the translations from the old keys
		for (const auto& IdPair : TextKeyMap)
		{
			for (FLocArchiveInfo& LocArchive : ForeignLocArchives)
			{
				TSharedPtr<FArchiveEntry> FoundArchiveEntry = LocArchive.Archive->FindEntryByKey(IdPair.Key.GetNamespace(), IdPair.Key.GetKey(), nullptr);
				if (FoundArchiveEntry.IsValid() && !FoundArchiveEntry->Translation.Text.IsEmpty())
				{
					LocArchive.bHasArchiveChanged = true;
					if (!LocArchive.Archive->SetTranslation(IdPair.Value.GetNamespace(), IdPair.Value.GetKey(), FoundArchiveEntry->Source, FoundArchiveEntry->Translation, FoundArchiveEntry->KeyMetadataObj))
					{
						LocArchive.Archive->AddEntry(IdPair.Value.GetNamespace(), IdPair.Value.GetKey(), FoundArchiveEntry->Source, FoundArchiveEntry->Translation, FoundArchiveEntry->KeyMetadataObj, FoundArchiveEntry->bIsOptional);
					}
				}
			}
		}

		// Re-save any updated archives
		for (const FLocArchiveInfo& LocArchive : ForeignLocArchives)
		{
			if (LocArchive.bHasArchiveChanged)
			{
				const bool bDidWriteArchive = FLocalizedAssetSCCUtil::SaveFileWithSCC(SourceControlInfo, LocArchive.Filename, [&LocArchive](const FString& InSaveFileName) -> bool
				{
					return FJsonInternationalizationArchiveSerializer::SerializeArchiveToFile(LocArchive.Archive, InSaveFileName);
				});

				if (!bDidWriteArchive)
				{
					UE_LOGF(LogStabilizeLocalizationKeys, Error, "Failed to write archive to %ls.", *LocArchive.Filename);
					return false;
				}
			}
		}
	}

	return 0;
}

#else	// USE_STABLE_LOCALIZATION_KEYS

int32 UStabilizeLocalizationKeysCommandlet::Main(const FString& Params)
{
	UE_LOGF(LogStabilizeLocalizationKeys, Fatal, "UStabilizeLocalizationKeysCommandlet requires a build with USE_STABLE_LOCALIZATION_KEYS enabled!");
	return 0;
}

#endif	// USE_STABLE_LOCALIZATION_KEYS
