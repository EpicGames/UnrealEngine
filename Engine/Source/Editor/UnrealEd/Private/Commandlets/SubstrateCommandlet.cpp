// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/SubstrateCommandlet.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "MaterialDomain.h"
#include "ShaderCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubstrateCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogSubstrateCommandlet, Log, All);

#define SUBSTRATER_COMMANDLET_SHADER_COMPILATION 0

USubstrateCommandlet::USubstrateCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 USubstrateCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogSubstrateCommandlet, Log, "SubstrateCommandlet");
		UE_LOGF(LogSubstrateCommandlet, Log, "This commandlet will dump information about Substrate materials.");
		UE_LOGF(LogSubstrateCommandlet, Log, "A typical way to invoke it is: <YourProject> -run=Substrate -DumpSubstrateMaterials -targetplatform=Windows ,");
		// Note in case later we want to compile material to look at the result of the compilation (relevance, Substrate Closure Count etc.):
		//  - Check See DumpLightFunctionMaterialInfo.cpp or CompileShadersTestBedCommandlet.cpp on how to do that. Right now, the code is hidden behind behind SUBSTRATER_COMMANDLET_SKIP_SHADER_COMPILATION
		//  - We will need extra parameters such as " -sm6 -unattended -allowcommandletrendering -nomaterialshaderddc".
		//  - We also might want to recommend DebugViewModeHelpers.Enable=0 to avoid compiling those expenssive shaders.
		return 0;
	}

	if (!Switches.Contains("DumpSubstrateMaterials"))
	{
		return 0;	// Nothing to do
	}



	UE_LOGF(LogSubstrateCommandlet, Log, "Searching for materials within the project...");

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	TArray<FAssetData> MaterialAssets;
	AssetRegistry.GetAssetsByClass(UMaterial::StaticClass()->GetClassPathName(), MaterialAssets, true);
	UE_LOGF(LogSubstrateCommandlet, Log, "Found %d materials", MaterialAssets.Num());

	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	const TArray<ITargetPlatform*>& Platforms = TPM->GetActiveTargetPlatforms();
	const EShaderPlatform ShaderPlatform = EShaderPlatform::SP_PCD3D_SM6;

	TArray<UMaterialInterface*> SubstrateMaterials;
#if SUBSTRATER_COMMANDLET_SHADER_COMPILATION
	TSet<UMaterialInterface*> MaterialsToCompile;
#endif

	// only run for a single platform as this is enough to know if material is substrate or not
	if (Platforms.Num() > 0)
	{
		ITargetPlatform* Platform = Platforms[0];
		UE_LOGF(LogSubstrateCommandlet, Display, "Compiling shaders for %ls...", *Platform->PlatformName());
		for (const FAssetData& AssetData : MaterialAssets)
		{
			if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
			{
				UMaterial* Material = MaterialInterface->GetMaterial();
				if (Material)
				{
					if (Material->HasSubstrateFrontMaterialConnected())
					{
						SubstrateMaterials.Add(MaterialInterface);
					}
#if SUBSTRATER_COMMANDLET_SHADER_COMPILATION
					UE_LOGF(LogSubstrateCommandlet, Display, "BeginCache for %ls", *MaterialInterface->GetFullName());
					MaterialInterface->BeginCacheForCookedPlatformData(Platform);
					// need to call this once for all objects before any calls to ProcessAsyncResults as otherwise we'll potentially upload
					// incremental/incomplete shadermaps to DDC (as this function actually triggers compilation, some compiles for a particular
					// material may finish before we've even started others - if we call ProcessAsyncResults in that case the associated shader
					// maps will think they are "finished" due to having no outstanding dependencies).
					if (!MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						MaterialsToCompile.Add(MaterialInterface);
					}
#endif
				}
			}
		}


#if SUBSTRATER_COMMANDLET_SHADER_COMPILATION
		SubstrateMaterials.Reserve(MaterialsToCompile.Num());

		UE_LOGF(LogSubstrateCommandlet, Log, "Found %d materials to compile and check.", MaterialsToCompile.Num());

		static constexpr bool bLimitExecutationTime = false;
		int32 PreviousOutstandingJobs = 0;
		constexpr int32 MaxOutstandingJobs = 20000; // Having a max is a way to try to reduce memory usage.. otherwise outstanding jobs can reach 100k+ and use up 300gb committed memory
		// Submit all the jobs.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SubmitJobs);

			UE_LOGF(LogSubstrateCommandlet, Display, "Submit Jobs");

			while (MaterialsToCompile.Num())
			{
				for (auto It = MaterialsToCompile.CreateIterator(); It; ++It)
				{
					UMaterialInterface* MaterialInterface = *It;
					if (MaterialInterface->IsCachedCookedPlatformDataLoaded(Platform))
					{
						It.RemoveCurrent();
						UE_LOGF(LogSubstrateCommandlet, Display, "Finished cache for %ls.", *MaterialInterface->GetFullName());
						UE_LOGF(LogSubstrateCommandlet, Display, "Materials remaining: %d", MaterialsToCompile.Num());
					}

					GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

					while (true)
					{
						const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
						if (CurrentOutstandingJobs != PreviousOutstandingJobs)
						{
							UE_LOGF(LogSubstrateCommandlet, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
							PreviousOutstandingJobs = CurrentOutstandingJobs;
						}

						// Flush rendering commands to release any RHI resources (shaders and shader maps).
						// Delete any FPendingCleanupObjects (shader maps).
						FlushRenderingCommands();

						if (CurrentOutstandingJobs < MaxOutstandingJobs)
						{
							break;
						}
						FPlatformProcess::Sleep(1);
					}
				}
			}
		}



		// Process the shader maps and save to the DDC.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessShaderCompileResults);

			UE_LOGF(LogSubstrateCommandlet, Log, "ProcessAsyncResults");

			while (GShaderCompilingManager->IsCompiling())
			{
				GShaderCompilingManager->ProcessAsyncResults(bLimitExecutationTime, false /* bBlockOnGlobalShaderCompilation */);

				while (true)
				{
					const int32 CurrentOutstandingJobs = GShaderCompilingManager->GetNumOutstandingJobs();
					if (CurrentOutstandingJobs != PreviousOutstandingJobs)
					{
						UE_LOGF(LogSubstrateCommandlet, Display, "Outstanding Jobs: %d", CurrentOutstandingJobs);
						PreviousOutstandingJobs = CurrentOutstandingJobs;
					}

					// Flush rendering commands to release any RHI resources (shaders and shader maps).
					// Delete any FPendingCleanupObjects (shader maps).
					FlushRenderingCommands();

					if (CurrentOutstandingJobs < MaxOutstandingJobs)
					{
						break;
					}
					FPlatformProcess::Sleep(1);
				}
			}
		}



		// Look up all material that are Substrate
		if (Switches.Contains("DumpSubstrateMaterials"))
		{
			for (UMaterialInterface* MaterialInterface : MaterialsToCompile)
			{
				UMaterial* Material = MaterialInterface->GetMaterial();
				if (Material)
				{
					TArray<FMaterialResource*> ResourcesToCache;
					FMaterialResource* CurrentResource = FindOrCreateMaterialResource(ResourcesToCache, Material, nullptr, ShaderPlatform, EMaterialQualityLevel::High);
					check(CurrentResource);

					FMaterialRelevance MaterialRelevance = CurrentResource->GetMaterialInterface()->GetRelevance(ERHIFeatureLevel::SM6);
					// use material relenave to get the number of closure count per pixel for instance.

					FMaterial::DeferredDeleteArray(ResourcesToCache);
				}
			}
		}



		// Perform cleanup and clear cached data for cooking.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ClearCachedCookedPlatformData);

			UE_LOGF(LogSubstrateCommandlet, Display, "Clear Cached Cooked Platform Data");

			for (const FAssetData& AssetData : MaterialAssets)
			{
				if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetData.GetAsset()))
				{
					MaterialInterface->ClearAllCachedCookedPlatformData();
				}
			}
		}
#endif
	} // Platforms
	

	if (Switches.Contains("DumpSubstrateMaterials"))
	{
		UE_LOGF(LogSubstrateCommandlet, Log, "***********************");
		UE_LOGF(LogSubstrateCommandlet, Log, "* Substrate Materials *");
		UE_LOGF(LogSubstrateCommandlet, Log, "***********************");
		for (UMaterialInterface* MaterialInterface : SubstrateMaterials)
		{
			UE_LOGF(LogSubstrateCommandlet, Log, "  - %ls", *MaterialInterface->GetPathName());
		}

		UE_LOGF(LogSubstrateCommandlet, Log, "Substrate Material Count:.........%d.", SubstrateMaterials.Num());
		UE_LOGF(LogSubstrateCommandlet, Log, "Non Substrate Material Count:.....%d.", MaterialAssets.Num() - SubstrateMaterials.Num());
	}
	return 0;
}

