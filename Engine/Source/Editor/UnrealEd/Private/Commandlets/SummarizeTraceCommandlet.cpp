// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SummarizeTraceCommandlet.cpp: Commandlet for summarizing a utrace
=============================================================================*/

#include "Commandlets/SummarizeTraceCommandlet.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "String/ParseTokens.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Analysis.h"
#include "Trace/DataStream.h"
#include "TraceServices/AnalyzerFactories.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Utils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SummarizeTraceCommandlet)

/**
 * Summarizes matched CPU scopes, excluding time consumed by immediate children if any. The analyzer uses pattern matching to
 * selects the scopes of interest and detects parent/child relationship. Once a parent/child relationship is established in a
 * scope tree, the analyzer can substract the time consumed by its immediate children if any.
 *
 * Such analysis is often meaningful for reentrant/recursive scope. For example, the UE modules can be loaded recursively and
 * it is useful to know how much time a module used to load itself vs how much time it use to recursively load its dependent
 * modules. In that example, we need to know which scope timers are actually the recursion vs the other intermediate scopes.
 * So, pattern-matching scope names is used to deduce a relationship between scopes in a tree of scopes.
 *
 * For the LoadModule example described above, if the analyzer gets this scope tree as input:
 *
 * |-LoadModule(Module1)---------------------------------------------------------|
 *    |- StartupModule -------------------------------------------------------|
 *      |-LoadModule(Module1Dep1)-----------|  |-LoadModule(Module1Dep2)-----|
 *        |-StartupModule-----------------|      |-StartupModule----------|
 *
 * It would turn it into the one below if the REGEX to match was "LoadModule_.*"
 * 
 * |-LoadModule(Module1)---------------------------------------------------------|
 *      |-LoadModule(Module1Dep1)-----------|  |-LoadModule(Module1Dep2)-----|
 *
 * And it would compute the exclusive time required to load Module1 by substracting the time consumed to
 * load Module1Dep1 and Module1Dep2.
 *
 * @note If the matching expression was to match all scopes, the analyser would summarize all scopes,
 *       accounting for their exclusive time.
 */

class FSummarizeCpuScopeHierarchyAnalyzer
	: public FSummarizeCpuScopeAnalyzer
{
public:
	/**
	 * Constructs the analyzer
	 * @param InAnalyzerName    The name of this analyzer. Some output statistics will also be derived from this name.
	 * @param InMatchFn         Invoked by the analyzer to determine if a scope should be accounted by the analyzer. If it returns true, the scope is kept, otherwise, it is ignored.
	 * @param InPublishFn       Invoked at the end of the analysis to post process the scopes summaries procuded by this analysis, possibly eliminating or renaming them then to publish them.
	 *                          The first parameter is the summary of all matched scopes together while the array contains summary of scopes that matched the expression by grouped by exact name match.
	 * 
	 * @note This analyzer publishes summarized scope names with a suffix ".excl" as it computes exclusive time to  prevent name collisions with other analyzers. The summary of all scopes
	 *       suffixed with ".excl.all" and gets its base name from the analyzer name. The scopes in the array gets their names from the scope name themselves, but suffixed with .excl.
	 */
	FSummarizeCpuScopeHierarchyAnalyzer(const FString& InAnalyzerName, TFunction<bool(const FString&)> InMatchFn, TFunction<void(const FSummarizeScope&, TArray<FSummarizeScope>&&)> InPublishFn);

	/**
	 * Runs analysis on a collection of scopes events. The scopes are expected to be from the same thread and form a 'tree', meaning
	 * it has the root scope events on that thread as well as all children below the root down to the leaves.
	 * @param ThreadId The thread on which the scopes were recorded.
	 * @param ScopeEvents The scopes events containing one root event along with its hierarchy.
	 * @param InScopeNameLookup Callback function to lookup scope names from scope ID.
	 */
	virtual void OnCpuScopeTree(uint32 ThreadId, const TArray64<FSummarizeCpuScopeAnalyzer::FScopeEvent>& ScopeEvents, const TFunction<const FString*(uint32 /*ScopeId*/)>& InScopeNameLookup) override;

	/**
	 * Invoked to notify that the trace session ended and that the analyzer can publish the statistics gathered.
	 * The analyzer calls the publishing function passed at construction time to publish the analysis results.
	 */
	virtual void OnCpuScopeAnalysisEnd() override;

private:
	// The function invoked to filter (by name) the scopes of interest. A scopes is kept if this function returns true.
	TFunction<bool(const FString&)> MatchesFn;

	// Aggregate all scopes matching the filter function, accounting for the parent/child relationship, so the duration stats will be from
	// the 'exclusive' time (itself - duration of immediate children) of the matched scope.
	FSummarizeScope MatchedScopesSummary;

	// Among the scopes matching the filter function, grouped by scope name summaries, accounting for the parent/child relationship. So the duration stats will be
	// from the 'exclusive' time (itself - duration of immediate children).
	TMap<FString, FSummarizeScope> ExactNameMatchScopesSummaries;

	// Invoked at the end of the analysis to publish scope summaries.
	TFunction<void(const FSummarizeScope&, TArray<FSummarizeScope>&&)> PublishFn;
};

FSummarizeCpuScopeHierarchyAnalyzer::FSummarizeCpuScopeHierarchyAnalyzer(const FString& InAnalyzerName, TFunction<bool(const FString&)> InMatchFn, TFunction<void(const FSummarizeScope&, TArray<FSummarizeScope>&&)> InPublishFn)
 : MatchesFn(MoveTemp(InMatchFn))
 , PublishFn(MoveTemp(InPublishFn))
{
	MatchedScopesSummary.Name = InAnalyzerName;
}

void FSummarizeCpuScopeHierarchyAnalyzer::OnCpuScopeTree(uint32 ThreadId, const TArray64<FSummarizeCpuScopeAnalyzer::FScopeEvent>& ScopeEvents, const TFunction<const FString*(uint32 /*ScopeId*/)>& InScopeNameLookup)
{
	// Scope matching the pattern.
	struct FMatchScopeEnter
	{
		FMatchScopeEnter(uint32 InScopeId, double InTimestamp) : ScopeId(InScopeId), Timestamp(InTimestamp) {}
		uint32 ScopeId;
		double Timestamp;
		FTimespan ChildrenDuration;
	};

	TArray<FMatchScopeEnter> ScopeStack;

	// Replay and filter this scope hierarchy to only keep the ones matching the condition/regex. (See class documentation for a visual example)
	for (const FSummarizeCpuScopeAnalyzer::FScopeEvent& ScopeEvent : ScopeEvents)
	{
		const FString* ScopeName = InScopeNameLookup(ScopeEvent.ScopeId);
		if (!ScopeName || !MatchesFn(*ScopeName))
		{
			continue;
		}

		if (ScopeEvent.ScopeEventType == FSummarizeCpuScopeAnalyzer::EScopeEventType::Enter)
		{
			ScopeStack.Emplace(ScopeEvent.ScopeId, ScopeEvent.Timestamp);
		}
		else // Scope Exit
		{
			FMatchScopeEnter EnterScope = ScopeStack.Pop();
			double EnterTimestampSecs = EnterScope.Timestamp;
			uint32 ScopeId = EnterScope.ScopeId;

			// Total time consumed by this scope.
			FTimespan InclusiveDuration = FTimespan::FromSeconds(ScopeEvent.Timestamp - EnterTimestampSecs);

			// Total time consumed by this scope, excluding the time consumed by matched 'children' scopes.
			FTimespan ExclusiveDuration = InclusiveDuration - EnterScope.ChildrenDuration;

			if (ScopeStack.Num() > 0)
			{
				// Track how much time this 'child' consumed inside its parent.
				ScopeStack.Last().ChildrenDuration += InclusiveDuration;
			}

			// Aggregate this scope with all other scopes of the exact same name, excluding children duration, so that we have the 'self only' starts.
			FSummarizeScope& ExactNameScopeSummary = ExactNameMatchScopesSummaries.FindOrAdd(*ScopeName);
			ExactNameScopeSummary.Name = *ScopeName;
			ExactNameScopeSummary.AddDuration(EnterTimestampSecs, EnterTimestampSecs + ExclusiveDuration.GetTotalSeconds());

			// Aggregate this scope with all other scopes matching the pattern, but excluding the children time, so that we have the stats of 'self only'.
			MatchedScopesSummary.AddDuration(EnterTimestampSecs, EnterTimestampSecs + ExclusiveDuration.GetTotalSeconds());
		}
	}
}

void FSummarizeCpuScopeHierarchyAnalyzer::OnCpuScopeAnalysisEnd()
{
	MatchedScopesSummary.Name += TEXT(".excl.all");

	TArray<FSummarizeScope> ScopeSummaries;
	for (TPair<FString, FSummarizeScope>& Pair : ExactNameMatchScopesSummaries)
	{
		Pair.Value.Name += TEXT(".excl");
		ScopeSummaries.Add(Pair.Value);
	}

	// Publish the scopes.
	PublishFn(MatchedScopesSummary, MoveTemp(ScopeSummaries));
}

//
// FSummarizeCountersAnalyzer - Tally Counters from counter set/increment events
//

struct FSummarizeCounter
	: public TraceServices::IEditableCounter
{
	FString Name;
	ETraceCounterType Type = TraceCounterType_Int;

	union FValue
	{
		int64 Int;
		double Float;
	};

	const FValue Zero = { 0 };
	FValue First;
	FValue Last;
	FValue Minimum;
	FValue Maximum;

	FIncrementalVariance Variance;

	double FirstSeconds = 0.0;
	double LastSeconds = 0.0;

	FSummarizeCounter()
	{
		Type = TraceCounterType_Int;
		First.Int = 0;
		Last.Int = 0;
		Minimum.Int = TNumericLimits<int64>::Max();
		Maximum.Int = TNumericLimits<int64>::Min();
	}

	virtual void SetName(const TCHAR* InName) override
	{
		Name = InName;
	}

	virtual void SetGroup(const TCHAR* Group) override
	{}

	virtual void SetDescription(const TCHAR* Description) override
	{}

	virtual void SetIsFloatingPoint(bool bIsFloatingPoint) override
	{
		ETraceCounterType Previous = Type;

		if (bIsFloatingPoint)
		{
			Type = TraceCounterType_Float;
			First.Float = 0.0;
			Last.Float = 0.0;
			Minimum.Float = TNumericLimits<double>::Max();
			Maximum.Float = TNumericLimits<double>::Min();
		}
		else
		{
			Type = TraceCounterType_Int;
			First.Int = 0;
			Last.Int = 0;
			Minimum.Int = TNumericLimits<int64>::Max();
			Maximum.Int = TNumericLimits<int64>::Min();
		}

		if (Previous != Type)
		{
			Variance.Reset();
			FirstSeconds = 0.0;
			LastSeconds = 0.0;
		}
	}

	virtual void SetIsResetEveryFrame(bool bInIsResetEveryFrame) override
	{}

	virtual void SetDisplayHint(TraceServices::ECounterDisplayHint DisplayHint) override
	{}

	virtual void AddValue(double Time, int64 Value) override
	{
		int64 Current;
		if (Get(Current))
		{
			Set(Current + Value, Time);
		}
	}

	virtual void AddValue(double Time, double Value) override
	{
		double Current;
		if (Get(Current))
		{
			Set(Current + Value, Time);
		}
	}

	virtual void SetValue(double Time, int64 Value) override
	{
		Set(Value, Time);
	}

	virtual void SetValue(double Time, double Value) override
	{
		Set(Value, Time);
	}

	bool Get(int64& Value)
	{
		ensure(Type == TraceCounterType_Int);
		if (Type == TraceCounterType_Int)
		{
			Value = Last.Int;
			return true;
		}

		return false;
	}

	void Set(int64 InValue, double InTimestamp)
	{
		ensure(Type == TraceCounterType_Int);
		if (Type == TraceCounterType_Int)
		{
			Variance.Increment(double(InValue));

			if (Variance.GetCount() == 1)
			{
				First.Int = InValue;
				FirstSeconds = InTimestamp;
			}

			Last.Int = InValue;
			LastSeconds = InTimestamp;

			Minimum.Int = FMath::Min(Minimum.Int, InValue);
			Maximum.Int = FMath::Max(Maximum.Int, InValue);
		}
	}

	bool Get(double& Value)
	{
		ensure(Type == TraceCounterType_Float);
		if (Type == TraceCounterType_Float)
		{
			Value = Last.Float;
			return true;
		}

		return false;
	}

	void Set(double InValue, double InTimestamp)
	{
		ensure(Type == TraceCounterType_Float);
		if (Type == TraceCounterType_Float)
		{
			Variance.Increment(InValue);

			if (Variance.GetCount() == 1)
			{
				First.Float = InValue;
				FirstSeconds = InTimestamp;
			}

			Last.Float = InValue;
			LastSeconds = InTimestamp;

			Minimum.Float = FMath::Min(Minimum.Float, InValue);
			Maximum.Float = FMath::Max(Maximum.Float, InValue);
		}
	}
	
	uint64 GetCount() const
	{
		return Variance.GetCount();
	}

	double GetMean() const
	{
		return Variance.GetMean();
	}

	double GetDeviation() const
	{
		return Variance.Deviation();
	}

	double GetCountPerSecond() const
	{
		double CountPerSecond = 0.0;
		const uint64 Count = Variance.GetCount();
		if (Count)
		{
			if (Count == 1)
			{
				CountPerSecond = 1.0;
			}
			else
			{
				CountPerSecond = Count / (LastSeconds - FirstSeconds);
			}
		}
		return CountPerSecond;
	}

	FString PrintValue(const FValue& InValue) const
	{
		switch (Type)
		{
			case TraceCounterType_Int:
				return FString::Printf(TEXT("%lld"), InValue.Int);

			case TraceCounterType_Float:
				return FString::Printf(TEXT("%f"), InValue.Float);
		}

		ensure(false);
		return TEXT("");
	}

	FString GetValue(const FStringView& Statistic) const
	{
		if (Statistic == TEXT("Name"))
		{
			return Name;
		}
		else if (Statistic == TEXT("Count"))
		{
			return FString::Printf(TEXT("%llu"), Variance.GetCount());
		}
		else if (Statistic == TEXT("First"))
		{
			return PrintValue(First);
		}
		else if (Statistic == TEXT("FirstSeconds"))
		{
			return FString::Printf(TEXT("%f"), FirstSeconds);
		}
		else if (Statistic == TEXT("Last"))
		{
			return PrintValue(Last);
		}
		else if (Statistic == TEXT("LastSeconds"))
		{
			return FString::Printf(TEXT("%f"), LastSeconds);
		}
		else if (Statistic == TEXT("Minimum"))
		{
			return PrintValue(Variance.GetCount() ? Minimum : Zero);
		}
		else if (Statistic == TEXT("Maximum"))
		{
			return PrintValue(Variance.GetCount() ? Maximum : Zero);
		}
		else if (Statistic == TEXT("Mean"))
		{
			return FString::Printf(TEXT("%f"), Variance.GetMean());
		}
		else if (Statistic == TEXT("Deviation"))
		{
			return FString::Printf(TEXT("%f"), Variance.Deviation());
		}
		else if (Statistic == TEXT("CountPerSecond"))
		{
			return FString::Printf(TEXT("%f"), GetCountPerSecond());
		}

		ensure(false);
		return TEXT("");
	}
};

class FSummarizeCountersProvider
	: public TraceServices::IEditableCounterProvider
{
public:
	const TraceServices::ICounter* GetCounter(TraceServices::IEditableCounter* EditableCounter)
	{
		// we don't want derived counters to be created by the analyzer
		return nullptr;
	}

	virtual TraceServices::IEditableCounter* CreateEditableCounter()
	{
		Counters.Add(MakeUnique<FSummarizeCounter>());
		return Counters.Last().Get();
	}

	virtual void AddCounter(const TraceServices::ICounter* Counter)
	{
		// we don't use custom counter objects
	}

	TArray<TUniquePtr<FSummarizeCounter>> Counters;
};

class FSummarizeBookmarksProvider
	: public TraceServices::IEditableBookmarkProvider
{
	virtual void UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatString) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text) override;

	struct FBookmarkSpec
	{
		const TCHAR* File = nullptr;
		const TCHAR* FormatString = nullptr;
		int32 Line = 0;
	};

	FBookmarkSpec& GetSpec(uint64 BookmarkPoint);
	FSummarizeBookmark* FindStartBookmarkForEndBookmark(const FString& Name);

public:
	// Keyed by a unique memory address
	TMap<uint64, FBookmarkSpec> BookmarkSpecs;

	// Keyed by name
	TMap<FString, FSummarizeBookmark> Bookmarks;

	// Bookmarks named formed to scopes, see FindStartBookmarkForEndBookmark
	TMap<FString, FSummarizeScope> Scopes;

private:
	enum
	{
		FormatBufferSize = 65536
	};
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];
};

FSummarizeBookmarksProvider::FBookmarkSpec& FSummarizeBookmarksProvider::GetSpec(uint64 BookmarkPoint)
{
	FBookmarkSpec* Found = BookmarkSpecs.Find(BookmarkPoint);
	if (Found)
	{
		return *Found;
	}

	FBookmarkSpec& Spec = BookmarkSpecs.Add(BookmarkPoint, FBookmarkSpec());
	Spec.File = TEXT("<unknown>");
	Spec.FormatString = TEXT("<unknown>");
	return Spec;
}

void FSummarizeBookmarksProvider::UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line)
{
	FBookmarkSpec& BookmarkSpec = GetSpec(BookmarkPoint);
	BookmarkSpec.FormatString = FormatString;
	BookmarkSpec.File = File;
	BookmarkSpec.Line = Line;
}

void FSummarizeBookmarksProvider::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatArgs)
{
	FBookmarkSpec& BookmarkSpec = GetSpec(BookmarkPoint);
	TraceServices::StringFormat(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, BookmarkSpec.FormatString, FormatArgs);
	FSummarizeBookmarksProvider::AppendBookmark(BookmarkPoint, Time, CallstackId, FormatBuffer);
}

void FSummarizeBookmarksProvider::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text)
{
	FString Name(Text);

	FSummarizeBookmark* FoundBookmark = Bookmarks.Find(Name);
	if (!FoundBookmark)
	{
		FoundBookmark = &Bookmarks.Add(Name, FSummarizeBookmark());
		FoundBookmark->Name = Name;
	}

	FoundBookmark->AddTimestamp(Time);

	FSummarizeBookmark* StartBookmark = FindStartBookmarkForEndBookmark(Name);
	if (StartBookmark)
	{
		FString ScopeName = FString(TEXT("Generated Scope for ")) + StartBookmark->Name;
		FSummarizeScope* FoundScope = Scopes.Find(ScopeName);
		if (!FoundScope)
		{
			FoundScope = &Scopes.Add(ScopeName, FSummarizeScope());
			FoundScope->Name = ScopeName;
		}

		FoundScope->AddDuration(StartBookmark->LastSeconds, Time);
	}
}

FSummarizeBookmark* FSummarizeBookmarksProvider::FindStartBookmarkForEndBookmark(const FString& Name)
{
	int32 Index = Name.Find(TEXT("Complete"));
	if (Index != -1)
	{
		FString StartName = Name;
		StartName.RemoveAt(Index, TCString<TCHAR>::Strlen(TEXT("Complete")));
		return Bookmarks.Find(StartName);
	}

	return nullptr;
}

/*
* Begin SummarizeTrace commandlet implementation
*/

/*
* Helpers for the csv files
*/

static bool IsCsvSafeString(const FString& String)
{
	static struct DisallowedCharacter
	{
		const TCHAR Character;
		bool First;
	}
	DisallowedCharacters[] =
	{
		// breaks simple csv files
		{ TEXT('\n'), true },
		{ TEXT('\r'), true },
		{ TEXT(','), true },
	};

	// sanitize strings for a bog-simple csv file
	bool bDisallowed = false;
	int32 Index = 0;
	for (struct DisallowedCharacter& DisallowedCharacter : DisallowedCharacters)
	{
		if (String.FindChar(DisallowedCharacter.Character, Index))
		{
			if (DisallowedCharacter.First)
			{
				UE_LOGF(LogSummarizeTrace, Display, "A string contains disallowed character '%c'. See log for full list.", DisallowedCharacter.Character);
				DisallowedCharacter.First = false;
			}

			UE_LOGF(LogSummarizeTrace, Verbose, "String '%ls' contains disallowed character '%c', skipping...", *String, DisallowedCharacter.Character);
			bDisallowed = true;
		}

		if (bDisallowed)
		{
			break;
		}
	}

	return !bDisallowed;
}

struct StatisticDefinition
{
	StatisticDefinition()
	{}

	StatisticDefinition(const FString& InName, const FString& InStatistic,
		const FString& InTelemetryContext, const FString& InTelemetryDataPoint, const FString& InTelemetryUnit,
		const FString& InBaselineWarningThreshold, const FString& InBaselineErrorThreshold)
		: Name(InName)
		, Statistic(InStatistic)
		, TelemetryContext(InTelemetryContext)
		, TelemetryDataPoint(InTelemetryDataPoint)
		, TelemetryUnit(InTelemetryUnit)
		, BaselineWarningThreshold(InBaselineWarningThreshold)
		, BaselineErrorThreshold(InBaselineErrorThreshold)
	{}

	StatisticDefinition(const StatisticDefinition& InStatistic)
		: Name(InStatistic.Name)
		, Statistic(InStatistic.Statistic)
		, TelemetryContext(InStatistic.TelemetryContext)
		, TelemetryDataPoint(InStatistic.TelemetryDataPoint)
		, TelemetryUnit(InStatistic.TelemetryUnit)
		, BaselineWarningThreshold(InStatistic.BaselineWarningThreshold)
		, BaselineErrorThreshold(InStatistic.BaselineErrorThreshold)
	{}

	bool operator==(const StatisticDefinition& InStatistic) const
	{
		return Name == InStatistic.Name
			&& Statistic == InStatistic.Statistic
			&& TelemetryContext == InStatistic.TelemetryContext
			&& TelemetryDataPoint == InStatistic.TelemetryDataPoint
			&& TelemetryUnit == InStatistic.TelemetryUnit
			&& BaselineWarningThreshold == InStatistic.BaselineWarningThreshold
			&& BaselineErrorThreshold == InStatistic.BaselineErrorThreshold;
	}

	static bool LoadFromCSV(const FString& FilePath, TMultiMap<FString, StatisticDefinition>& NameToDefinitionMap, TSet<FString>& OutWildcardNames);

	FString Name;
	FString Statistic;
	FString TelemetryContext;
	FString TelemetryDataPoint;
	FString TelemetryUnit;
	FString BaselineWarningThreshold;
	FString BaselineErrorThreshold;
};

bool StatisticDefinition::LoadFromCSV(const FString& FilePath, TMultiMap<FString, StatisticDefinition>& NameToDefinitionMap, TSet<FString>& OutWildcardNames)
{
	TArray<FString> ParsedCSVFile;
	FFileHelper::LoadFileToStringArray(ParsedCSVFile, *FilePath);

	int NameColumn = -1;
	int StatisticColumn = -1;
	int TelemetryContextColumn = -1;
	int TelemetryDataPointColumn = -1;
	int TelemetryUnitColumn = -1;
	int BaselineWarningThresholdColumn = -1;
	int BaselineErrorThresholdColumn = -1;
	struct Column
	{
		const TCHAR* Name = nullptr;
		int* Index = nullptr;
	}
	Columns[] =
	{
		{ TEXT("Name"), &NameColumn },
		{ TEXT("Statistic"), &StatisticColumn },
		{ TEXT("TelemetryContext"), &TelemetryContextColumn },
		{ TEXT("TelemetryDataPoint"), &TelemetryDataPointColumn },
		{ TEXT("TelemetryUnit"), &TelemetryUnitColumn },
		{ TEXT("BaselineWarningThreshold"), &BaselineWarningThresholdColumn },
		{ TEXT("BaselineErrorThreshold"), &BaselineErrorThresholdColumn },
	};

	bool bValidColumns = true;
	for (int CSVIndex = 0; CSVIndex < ParsedCSVFile.Num() && bValidColumns; ++CSVIndex)
	{
		const FString& CSVEntry = ParsedCSVFile[CSVIndex];
		TArray<FString> Fields;
		UE::String::ParseTokens(CSVEntry.TrimStartAndEnd(), TEXT(','),
			[&Fields](FStringView Field)
			{
				Fields.Add(FString(Field));
			});

		if (CSVIndex == 0) // is this the header row?
		{
			for (struct Column& Column : Columns)
			{
				for (int FieldIndex = 0; FieldIndex < Fields.Num(); ++FieldIndex)
				{
					if (Fields[FieldIndex] == Column.Name)
					{
						(*Column.Index) = FieldIndex;
						break;
					}
				}

				if (*Column.Index == -1)
				{
					bValidColumns = false;
				}
			}
		}
		else // else it is a data row, pull each element from appropriate column
		{
			const FString& Name(Fields[NameColumn]);
			const FString& Statistic(Fields[StatisticColumn]);
			const FString& TelemetryContext(Fields[TelemetryContextColumn]);
			const FString& TelemetryDataPoint(Fields[TelemetryDataPointColumn]);
			const FString& TelemetryUnit(Fields[TelemetryUnitColumn]);
			const FString& BaselineWarningThreshold(Fields[BaselineWarningThresholdColumn]);
			const FString& BaselineErrorThreshold(Fields[BaselineErrorThresholdColumn]);

			if (Name.Contains("*") || Name.Contains("?")) // Wildcards.
			{
				OutWildcardNames.Add(Name);
			}
			NameToDefinitionMap.AddUnique(Name, StatisticDefinition(Name, Statistic, TelemetryContext, TelemetryDataPoint, TelemetryUnit, BaselineWarningThreshold, BaselineErrorThreshold));
		}
	}

	return bValidColumns;
}

/*
* Helper class for the telemetry csv file
*/

struct TelemetryDefinition
{
	TelemetryDefinition()
	{}

	TelemetryDefinition(const FString& InTestName, const FString& InContext, const FString& InDataPoint, const FString& InUnit,
		const FString& InMeasurement, const FString* InBaseline = nullptr)
		: TestName(InTestName)
		, Context(InContext)
		, DataPoint(InDataPoint)
		, Unit(InUnit)
		, Measurement(InMeasurement)
		, Baseline(InBaseline ? *InBaseline : FString ())
	{}

	TelemetryDefinition(const TelemetryDefinition& InStatistic)
		: TestName(InStatistic.TestName)
		, Context(InStatistic.Context)
		, DataPoint(InStatistic.DataPoint)
		, Unit(InStatistic.Unit)
		, Measurement(InStatistic.Measurement)
		, Baseline(InStatistic.Baseline)
	{}

	bool operator==(const TelemetryDefinition& InStatistic) const
	{
		return TestName == InStatistic.TestName
			&& Context == InStatistic.Context
			&& DataPoint == InStatistic.DataPoint
			&& Measurement == InStatistic.Measurement
			&& Baseline == InStatistic.Baseline
			&& Unit == InStatistic.Unit;
	}

	static bool LoadFromCSV(const FString& FilePath, TMap<TPair<FString,FString>, TelemetryDefinition>& ContextAndDataPointToDefinitionMap);
	static bool MeasurementWithinThreshold(const FString& Value, const FString& BaselineValue, const FString& Threshold);
	static FString SignFlipThreshold(const FString& Threshold);

	FString TestName;
	FString Context;
	FString DataPoint;
	FString Unit;
	FString Measurement;
	FString Baseline;
};

bool TelemetryDefinition::LoadFromCSV(const FString& FilePath, TMap<TPair<FString, FString>, TelemetryDefinition>& ContextAndDataPointToDefinitionMap)
{
	TArray<FString> ParsedCSVFile;
	FFileHelper::LoadFileToStringArray(ParsedCSVFile, *FilePath);

	int TestNameColumn = -1;
	int ContextColumn = -1;
	int DataPointColumn = -1;
	int UnitColumn = -1;
	int MeasurementColumn = -1;
	int BaselineColumn = -1;
	struct Column
	{
		const TCHAR* Name = nullptr;
		int* Index = nullptr;
		bool bRequired = true;
	}
	Columns[] =
	{
		{ TEXT("TestName"), &TestNameColumn },
		{ TEXT("Context"), &ContextColumn },
		{ TEXT("DataPoint"), &DataPointColumn },
		{ TEXT("Unit"), &UnitColumn },
		{ TEXT("Measurement"), &MeasurementColumn },
		{ TEXT("Baseline"), &BaselineColumn, false },
	};

	bool bValidColumns = true;
	for (int CSVIndex = 0; CSVIndex < ParsedCSVFile.Num() && bValidColumns; ++CSVIndex)
	{
		const FString& CSVEntry = ParsedCSVFile[CSVIndex];
		TArray<FString> Fields;
		UE::String::ParseTokens(CSVEntry.TrimStartAndEnd(), TEXT(','),
			[&Fields](FStringView Field)
			{
				Fields.Add(FString(Field));
			});

		if (CSVIndex == 0) // is this the header row?
		{
			for (struct Column& Column : Columns)
			{
				for (int FieldIndex = 0; FieldIndex < Fields.Num(); ++FieldIndex)
				{
					if (Fields[FieldIndex] == Column.Name)
					{
						(*Column.Index) = FieldIndex;
						break;
					}
				}

				if (*Column.Index == -1 && Column.bRequired)
				{
					bValidColumns = false;
				}
			}
		}
		else // else it is a data row, pull each element from appropriate column
		{
			const FString& TestName(Fields[TestNameColumn]);
			const FString& Context(Fields[ContextColumn]);
			const FString& DataPoint(Fields[DataPointColumn]);
			const FString& Unit(Fields[UnitColumn]);
			const FString& Measurement(Fields[MeasurementColumn]);

			FString Baseline;
			if (BaselineColumn != -1)
			{
				Baseline = Fields[BaselineColumn];
			}

			ContextAndDataPointToDefinitionMap.Add(TPair<FString, FString>(Context, DataPoint), TelemetryDefinition(TestName, Context, DataPoint, Unit, Measurement, &Baseline));
		}
	}

	return bValidColumns;
}

bool TelemetryDefinition::MeasurementWithinThreshold(const FString& MeasurementValue, const FString& BaselineValue, const FString& Threshold)
{
	if (Threshold.IsEmpty())
	{
		return true;
	}

	// detect threshold as delta percentage
	int32 PercentIndex = INDEX_NONE;
	if (Threshold.FindChar(TEXT('%'), PercentIndex))
	{
		FString ThresholdWithoutPercentSign = Threshold;
		ThresholdWithoutPercentSign.RemoveAt(PercentIndex);

		double Factor = 1.0 + (FCString::Atod(*ThresholdWithoutPercentSign) / 100.0);
		double RationalValue = FCString::Atod(*MeasurementValue);
		double RationalBaselineValue = FCString::Atod(*BaselineValue);
		if (Factor >= 1.0)
		{
			return RationalValue < (RationalBaselineValue * Factor);
		}
		else
		{
			return RationalValue > (RationalBaselineValue * Factor);
		}
	}
	else // threshold as delta cardinal value
	{
		// rational number, use float math
		if (Threshold.Contains(TEXT(".")))
		{
			double Delta = FCString::Atod(*Threshold);
			double RationalValue = FCString::Atod(*MeasurementValue);
			double RationalBaselineValue = FCString::Atod(*BaselineValue);
			if (Delta > 0.0)
			{
				return RationalValue <= (RationalBaselineValue + Delta);
			}
			else if (Delta < 0.0)
			{
				return RationalValue >= (RationalBaselineValue + Delta);
			}
			else
			{
				return fabs(RationalBaselineValue - RationalValue) < FLT_EPSILON;
			}
		}
		else // natural number, use int math
		{
			int64 Delta = FCString::Strtoi64(*Threshold, nullptr, 10);
			int64 NaturalValue = FCString::Strtoi64(*MeasurementValue, nullptr, 10);
			int64 NaturalBaselineValue = FCString::Strtoi64(*BaselineValue, nullptr, 10);
			if (Delta > 0)
			{
				return NaturalValue <= (NaturalBaselineValue + Delta);
			}
			else if (Delta < 0)
			{
				return NaturalValue >= (NaturalBaselineValue + Delta);
			}
			else
			{
				return NaturalValue == NaturalBaselineValue;
			}
		}
	}
}

FString TelemetryDefinition::SignFlipThreshold(const FString& Threshold)
{
	FString SignFlipped;

	if (Threshold.StartsWith(TEXT("-")))
	{
		SignFlipped = Threshold.RightChop(1);
	}
	else
	{
		SignFlipped = FString(TEXT("-")) + Threshold;
	}

	return SignFlipped;
}

/*
 * SummarizeTrace commandlet ingests a utrace file and summarizes the
 * cpu scope events within it, and summarizes each event to a csv. It
 * also can generate a telemetry file given statistics csv about what
 * events and what statistics you would like to track.
 */

USummarizeTraceCommandlet::USummarizeTraceCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 USummarizeTraceCommandlet::Main(const FString& CmdLineParams)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*CmdLineParams, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogSummarizeTrace, Log, "SummarizeTrace");
		UE_LOGF(LogSummarizeTrace, Log, "This commandlet will summarize a utrace into something more easily ingestable by a reporting tool (csv).");
		UE_LOGF(LogSummarizeTrace, Log, "Options:");
		UE_LOGF(LogSummarizeTrace, Log, " Required: -inputfile=<utrace path>   (The utrace you wish to process)");
		UE_LOGF(LogSummarizeTrace, Log, " Optional: -testname=<string>         (Test name to use in telemetry csv)");
		UE_LOGF(LogSummarizeTrace, Log, " Optional: -alltelemetry              (Dump all data to telemetry csv)");
		return 0;
	}

	FString TraceFileName;
	if (FParse::Value(*CmdLineParams, TEXT("inputfile="), TraceFileName, true))
	{
		UE_LOGF(LogSummarizeTrace, Display, "Loading trace from %ls", *TraceFileName);
	}
	else
	{
		UE_LOGF(LogSummarizeTrace, Error, "You must specify a utrace file using -inputfile=<path>");
		return 1;
	}

	
	bool bFound;
	if (FPaths::FileExists(TraceFileName))
	{
		bFound = true;
	}
	else
	{
		bFound = false;
		TArray<FString> SearchPaths;
		SearchPaths.Add(FPaths::Combine(FPaths::EngineDir(), TEXT("Programs"), TEXT("UnrealInsights"), TEXT("Saved"), TEXT("TraceSessions")));
		SearchPaths.Add(FPaths::EngineDir());
		SearchPaths.Add(FPaths::ProjectDir());
		for (const FString& SearchPath : SearchPaths)
		{
			FString PossibleTraceFileName = FPaths::Combine(SearchPath, TraceFileName);
			if (FPaths::FileExists(PossibleTraceFileName))
			{
				TraceFileName = PossibleTraceFileName;
				bFound = true;
				break;
			}
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogSummarizeTrace, Error, "Trace file '%ls' was not found", *TraceFileName);
		return 1;
	}

	
	UE::Trace::FFileDataStream* DataStream = new UE::Trace::FFileDataStream();
	if (!DataStream->Open(*TraceFileName))
	{
		UE_LOGF(LogSummarizeTrace, Error, "Unable to open trace file '%ls' for read", *TraceFileName);
		delete DataStream;
		return 1;
	}

	// setup analysis context with analyzers
	UE::Trace::FAnalysisContext AnalysisContext;

	// List of summarized scopes.
	TArray<FSummarizeScope> CollectedScopeSummaries;

	// Analyze CPU scope timer individually.
	TSharedPtr<FSummarizeCpuScopeDurationAnalyzer> IndividualScopeAnalyzer = MakeShared<FSummarizeCpuScopeDurationAnalyzer>(
		[&CollectedScopeSummaries](const TArray<FSummarizeScope>& ScopeSummaries)
		{
			// Collect all individual scopes summary from this analyzer.
			CollectedScopeSummaries.Append(ScopeSummaries);
		});

	// Analyze 'LoadModule' scope timer hierarchically to account individual load time only (substracting time consumed to load dependent module(s)).
	TSharedPtr<FSummarizeCpuScopeHierarchyAnalyzer> HierarchicalScopeAnalyzer = MakeShared<FSummarizeCpuScopeHierarchyAnalyzer>(
		TEXT("LoadModule"), // Analyzer Name.
		[](const FString& ScopeName)
		{
			return ScopeName.Equals("LoadModule"); // When analyzing a tree of scopes, only keeps scope with name 'LoadModule'.
		},
		[&CollectedScopeSummaries](const FSummarizeScope& AllModulesStats, TArray<FSummarizeScope>&& ModuleStats)
		{
			// Module should be loaded only once and the check below should be true but the load function can start loading X, process some dependencies which could
			// trigger loading X again within the first scope. The engine code gracefully handle this case and don't load twice, but we end up with two 'load x' scope.
			// Both scope times be added together, providing the correct sum for that module though.
			//check(AllModulesStats.Count == ModuleStats.Num())

			// Publish the total nb. of module loaded, total time to the modules, avg time per module, etc (all module load times exclude the time to load sub-modules)
			CollectedScopeSummaries.Add(AllModulesStats);

			// Sort the summaries descending.
			ModuleStats.Sort([](const FSummarizeScope& Lhs, const FSummarizeScope& Rhs) { return Lhs.TotalDurationSeconds >= Rhs.TotalDurationSeconds; });

			// Publish top N longuest load module. The ModuleStats are pre-sorted from the longest to the shorted timer.
			CollectedScopeSummaries.Append(ModuleStats.GetData(), FMath::Min(10, ModuleStats.Num()));
		});

	TSharedPtr<TraceServices::IAnalysisSession> Session = TraceServices::CreateAnalysisSession(0, nullptr, TUniquePtr<UE::Trace::IInDataStream>(DataStream));

	FSummarizeBookmarksProvider BookmarksProvider;
	TSharedPtr<UE::Trace::IAnalyzer> BookmarksAnalyzer = TraceServices::CreateBookmarksAnalyzer(*Session, BookmarksProvider);
	AnalysisContext.AddAnalyzer(*BookmarksAnalyzer);

	FSummarizeCountersProvider CountersProvider;
	TSharedPtr<UE::Trace::IAnalyzer> CountersAnalyzer = TraceServices::CreateCountersAnalyzer(*Session, CountersProvider);
	AnalysisContext.AddAnalyzer(*CountersAnalyzer);

	FSummarizeCpuProfilerProvider CpuProfilerProvider;
	CpuProfilerProvider.AddCpuScopeAnalyzer(IndividualScopeAnalyzer);
	CpuProfilerProvider.AddCpuScopeAnalyzer(HierarchicalScopeAnalyzer);
	TSharedPtr<UE::Trace::IAnalyzer> CpuProfilerAnalyzer = TraceServices::CreateCpuProfilerAnalyzer(*Session, CpuProfilerProvider, CpuProfilerProvider);
	AnalysisContext.AddAnalyzer(*CpuProfilerAnalyzer);

	// kick processing on a thread
	UE::Trace::FAnalysisProcessor AnalysisProcessor = AnalysisContext.Process(*DataStream);

	// sync on completion
	AnalysisProcessor.Wait();

	CpuProfilerProvider.AnalysisComplete();

	TSet<FSummarizeScope> DeduplicatedScopes;
	auto IngestScope = [](TSet<FSummarizeScope>& DeduplicatedScopes, const FSummarizeScope& Scope)
	{
		if (Scope.Name.IsEmpty())
		{
			return;
		}

		if (Scope.GetCount() == 0)
		{
			return;
		}

		FSummarizeScope* FoundScope = DeduplicatedScopes.Find(Scope);
		if (FoundScope)
		{
			FoundScope->Merge(Scope);
		}
		else
		{
			DeduplicatedScopes.Add(Scope);
		}
	};
	for (const FSummarizeScope& Scope : CollectedScopeSummaries)
	{
		IngestScope(DeduplicatedScopes, Scope);
	}
	for (const TMap<FString, FSummarizeScope>::ElementType& ScopeItem : BookmarksProvider.Scopes)
	{
		IngestScope(DeduplicatedScopes, ScopeItem.Value);
	}

	UE_LOGF(LogSummarizeTrace, Display, "Sorting %d events by total time accumulated...", DeduplicatedScopes.Num());
	TArray<FSummarizeScope> SortedScopes;
	for (const FSummarizeScope& Scope : DeduplicatedScopes)
	{
		SortedScopes.Add(Scope);
	}
	SortedScopes.Sort();
	

	// some locals to help with all the derived files we are about to generate
	TracePath = FPaths::GetPath(TraceFileName);
	TraceFileBasename = FPaths::GetBaseFilename(TraceFileName);

	// generate a summary csv files, always
	if (!GenerateScopesCSV(SortedScopes))
	{
		return 1;
	}		
	if (!GenerateCountersCSV(CountersProvider))
	{
		return 1;
	}		
	if (!GenerateBookmarksCSV(BookmarksProvider))
	{
		return 1;
	}		

		
	// override the test name
	FString TestName = TraceFileBasename;
	FParse::Value(*CmdLineParams, TEXT("testname="), TestName, true);
	
	bool bAllTelemetry = FParse::Param(*CmdLineParams, TEXT("alltelemetry"));
	bool SkipBaseline = FParse::Param(*CmdLineParams, TEXT("skipbaseline"));
	
	if (!GenerateTelemetryCSV(TestName, bAllTelemetry, SortedScopes, CountersProvider, BookmarksProvider, SkipBaseline))
	{
		return 1;
	}

	return 0;
}

TUniquePtr<IFileHandle> USummarizeTraceCommandlet::OpenCSVFile(const FString& Name)
{
	FString CsvFileName = TraceFileBasename + Name;
    CsvFileName = FPaths::Combine(TracePath, FPaths::SetExtension(CsvFileName, "csv"));
    UE_LOGF(LogSummarizeTrace, Display, "Writing %ls...", *CsvFileName);
    TUniquePtr<IFileHandle> CsvHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*CsvFileName));

    if (!CsvHandle)
    {
    	UE_LOGF(LogSummarizeTrace, Error, "Unable to open csv '%ls' for write", *CsvFileName);
    }

	return CsvHandle;
}

bool USummarizeTraceCommandlet::GenerateScopesCSV(const TArray<FSummarizeScope>& SortedScopes)
{
	TUniquePtr<IFileHandle> CsvHandle = OpenCSVFile(TEXT("Scopes"));
	if (!CsvHandle)
	{
		return false;
	}
	
	// no newline, see row printfs
	CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("Name,Count,CountPerSecond,TotalDurationSeconds,FirstStartSeconds,FirstFinishSeconds,FirstDurationSeconds,LastStartSeconds,LastFinishSeconds,LastDurationSeconds,MinDurationSeconds,MaxDurationSeconds,MeanDurationSeconds,DeviationDurationSeconds,")));
	for (const FSummarizeScope& Scope : SortedScopes)
	{
		if (!CsvUtils::IsCsvSafeString(Scope.Name))
		{
			continue;
		}

		// note newline is at the front of every data line to prevent final extraneous newline, per customary for csv
		CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("\n%s,%llu,%f,%f,%f,%f,%f,%f,%f,%f,%f,%f,%f,%f,"),
			*Scope.Name, Scope.GetCount(), Scope.GetCountPerSecond(),
			Scope.TotalDurationSeconds,
			Scope.FirstStartSeconds, Scope.FirstFinishSeconds, Scope.FirstDurationSeconds,
			Scope.LastStartSeconds, Scope.LastFinishSeconds, Scope.LastDurationSeconds,
			Scope.MinDurationSeconds, Scope.MaxDurationSeconds,
			Scope.GetMeanDurationSeconds(), Scope.GetDeviationDurationSeconds()));
	}
	CsvHandle->Flush();

	return true;
}

bool USummarizeTraceCommandlet::GenerateCountersCSV(const FSummarizeCountersProvider& CountersProvider)
{
	TUniquePtr<IFileHandle> CsvHandle = OpenCSVFile(TEXT("Counters"));
	if (!CsvHandle)
	{
		return false;
	}
	
	// no newline, see row printfs
	CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("Name,Count,CountPerSecond,First,FirstSeconds,Last,LastSeconds,Minimum,Maximum,Mean,Deviation,")));
	for (const TUniquePtr<FSummarizeCounter>& Counter : CountersProvider.Counters)
	{
		if (!CsvUtils::IsCsvSafeString(Counter->Name))
		{
			continue;
		}

		// note newline is at the front of every data line to prevent final extraneous newline, per customary for csv
		CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("\n%s,%llu,%f,%s,%f,%s,%f,%s,%s,%f,%f,"),
			*Counter->Name, Counter->GetCount(), Counter->GetCountPerSecond(),
			*Counter->GetValue(TEXT("First")), Counter->FirstSeconds,
			*Counter->GetValue(TEXT("Last")), Counter->LastSeconds,
			*Counter->GetValue(TEXT("Minimum")), *Counter->GetValue(TEXT("Maximum")),
			Counter->GetMean(), Counter->GetDeviation()));
	}
	CsvHandle->Flush();

	return true;
}

bool USummarizeTraceCommandlet::GenerateBookmarksCSV(const FSummarizeBookmarksProvider& BookmarksProvider)
{
	TUniquePtr<IFileHandle> CsvHandle = OpenCSVFile(TEXT("Bookmarks"));
	if (!CsvHandle)
	{
		return false;
	}
	
	// no newline, see row printfs
	CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("Name,Count,FirstSeconds,LastSeconds,")));
	for (const TMap<FString, FSummarizeBookmark>::ElementType& Bookmark : BookmarksProvider.Bookmarks)
	{
		if (!CsvUtils::IsCsvSafeString(Bookmark.Value.Name))
		{
			continue;
		}

		// note newline is at the front of every data line to prevent final extraneous newline, per customary for csv
		CsvUtils::WriteAsUTF8String(CsvHandle.Get(), FString::Printf(TEXT("\n%s,%" UINT64_FMT ",%f,%f,"),
			*Bookmark.Value.Name, Bookmark.Value.Count,
			Bookmark.Value.FirstSeconds, Bookmark.Value.LastSeconds));
	}
	CsvHandle->Flush();

	return true;
}

bool USummarizeTraceCommandlet::GenerateTelemetryCSV(const FString& TestName,
	bool bAllTelemetry,
	const TArray<FSummarizeScope>& SortedScopes,
	const FSummarizeCountersProvider& CountersProvider,
	const FSummarizeBookmarksProvider& BookmarksProvider,
	bool SkipBaseline)
{
	// load the stats file to know which event name and statistic name to generate in the telemetry csv
	// the telemetry csv is ingested completely, so this just highlights specific data elements we want to track
	TMultiMap<FString, StatisticDefinition> NameToDefinitionMap;
	TSet<FString> CpuScopeNamesWithWildcards;
	if (!bAllTelemetry)
	{
		FString GlobalStatisticsFileName = FPaths::RootDir() / TEXT("Engine") / TEXT("Build") / TEXT("EditorPerfStats.csv");
		if (FPaths::FileExists(GlobalStatisticsFileName))
		{
			UE_LOGF(LogSummarizeTrace, Display, "Loading global statistics from %ls", *GlobalStatisticsFileName);
			bool bCSVOk = StatisticDefinition::LoadFromCSV(GlobalStatisticsFileName, NameToDefinitionMap, CpuScopeNamesWithWildcards);
			check(bCSVOk);
		}
		FString ProjectStatisticsFileName = FPaths::ProjectDir() / TEXT("Build") / TEXT("EditorPerfStats.csv");
		if (FPaths::FileExists(ProjectStatisticsFileName))
		{
			UE_LOGF(LogSummarizeTrace, Display, "Loading project statistics from %ls", *ProjectStatisticsFileName);
			bool bCSVOk = StatisticDefinition::LoadFromCSV(ProjectStatisticsFileName, NameToDefinitionMap, CpuScopeNamesWithWildcards);
			check(bCSVOk);
		}
	}


	TArray<TelemetryDefinition> TelemetryData;
	TSet<FString> ResolvedStatistics;
	{
		TArray<StatisticDefinition> Statistics;

		// resolve scopes to telemetry
		for (const FSummarizeScope& Scope : SortedScopes)
		{
			if (!CsvUtils::IsCsvSafeString(Scope.Name))
			{
				continue;
			}

			if (bAllTelemetry)
			{
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("Count"), Scope.GetValue(TEXT("Count")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("TotalDurationSeconds"), Scope.GetValue(TEXT("TotalDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("MinDurationSeconds"), Scope.GetValue(TEXT("MinDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("MaxDurationSeconds"), Scope.GetValue(TEXT("MaxDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("MeanDurationSeconds"), Scope.GetValue(TEXT("MeanDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Scope.Name, TEXT("DeviationDurationSeconds"), Scope.GetValue(TEXT("DeviationDurationSeconds")), TEXT("Seconds")));
			}
			else
			{
				// Is that scope summary desired in the output, using an exact name match?
				NameToDefinitionMap.MultiFind(Scope.Name, Statistics, true);
				for (const StatisticDefinition& Statistic : Statistics)
				{
					TelemetryData.Add(TelemetryDefinition(TestName, Statistic.TelemetryContext, Statistic.TelemetryDataPoint, Statistic.TelemetryUnit, Scope.GetValue(Statistic.Statistic)));
					ResolvedStatistics.Add(Statistic.Name);
				}
				Statistics.Reset();

				// If the configuration file contains scope names with wildcard characters * and ?
				for (const FString& Pattern : CpuScopeNamesWithWildcards)
				{
					// Check if the current scope names matches the pattern.
					if (Scope.Name.MatchesWildcard(Pattern))
					{
						// Find the statistic definition for this pattern.
						NameToDefinitionMap.MultiFind(Pattern, Statistics, true);
						for (const StatisticDefinition& Statistic : Statistics)
						{
							// Use the scope name as data point. Normally, the data point is configured in the .csv as a 1:1 match (1 scopeName=> 1 DataPoint) but in this
							// case, it is a one to many relationship (1 pattern => * matches).
							const FString& DataPoint = Scope.Name;
							TelemetryData.Add(TelemetryDefinition(TestName, Statistic.TelemetryContext, DataPoint, Statistic.TelemetryUnit, Scope.GetValue(Statistic.Statistic)));
							ResolvedStatistics.Add(Statistic.Name);
						}
						Statistics.Reset();
					}
				}
			}
		}

		// resolve counters to telemetry
		for (const TUniquePtr<FSummarizeCounter>& Counter : CountersProvider.Counters)
		{
			if (!CsvUtils::IsCsvSafeString(Counter->Name))
			{
				continue;
			}

			if (bAllTelemetry)
			{
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Count"), Counter->GetValue(TEXT("Count")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("First"), Counter->GetValue(TEXT("First")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("FirstSeconds"), Counter->GetValue(TEXT("FirstSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Last"), Counter->GetValue(TEXT("Last")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("LastSeconds"), Counter->GetValue(TEXT("LastSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Minimum"), Counter->GetValue(TEXT("Minimum")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Maximum"), Counter->GetValue(TEXT("Maximum")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Mean"), Counter->GetValue(TEXT("Mean")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Counter->Name, TEXT("Deviation"), Counter->GetValue(TEXT("Deviation")), TEXT("Count")));
			}
			else
			{
				NameToDefinitionMap.MultiFind(Counter->Name, Statistics, true);
				for (const StatisticDefinition& Statistic : Statistics)
				{
					TelemetryData.Add(TelemetryDefinition(TestName, Statistic.TelemetryContext, Statistic.TelemetryDataPoint, Statistic.TelemetryUnit, Counter->GetValue(Statistic.Statistic)));
					ResolvedStatistics.Add(Statistic.Name);
				}
				Statistics.Reset();			
			}
		}

		// resolve bookmarks to telemetry
		for (const TMap<FString, FSummarizeBookmark>::ElementType& Bookmark : BookmarksProvider.Bookmarks)
		{
			if (!CsvUtils::IsCsvSafeString(Bookmark.Value.Name))
			{
				continue;
			}

			if (bAllTelemetry)
			{
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("Count"), Bookmark.Value.GetValue(TEXT("Count")), TEXT("Count")));
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("TotalDurationSeconds"), Bookmark.Value.GetValue(TEXT("TotalDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("MinDurationSeconds"), Bookmark.Value.GetValue(TEXT("MinDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("MaxDurationSeconds"), Bookmark.Value.GetValue(TEXT("MaxDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("MeanDurationSeconds"), Bookmark.Value.GetValue(TEXT("MeanDurationSeconds")), TEXT("Seconds")));
				TelemetryData.Add(TelemetryDefinition(TestName, Bookmark.Value.Name, TEXT("DeviationDurationSeconds"), Bookmark.Value.GetValue(TEXT("DeviationDurationSeconds")), TEXT("Seconds")));
			}
			else
			{
				NameToDefinitionMap.MultiFind(Bookmark.Value.Name, Statistics, true);
				ensure(Statistics.Num() <= 1); // there should only be one, the bookmark itself
				for (const StatisticDefinition& Statistic : Statistics)
				{
					TelemetryData.Add(TelemetryDefinition(TestName, Statistic.TelemetryContext, Statistic.TelemetryDataPoint, Statistic.TelemetryUnit, Bookmark.Value.GetValue(Statistic.Statistic)));
					ResolvedStatistics.Add(Statistic.Name);
				}
				Statistics.Reset();			
			}
		}
	}

	if (!bAllTelemetry)
	{
		// compare vs. baseline telemetry file, if it exists
		// note this does assume that the tracefile basename is directly comparable to a file in the baseline folder
		FString BaselineTelemetryCsvFilePath = FPaths::Combine(FPaths::EngineDir(), TEXT("Build"), TEXT("Baseline"), FPaths::SetExtension(TraceFileBasename + TEXT("Telemetry"), "csv"));
		if (SkipBaseline)
		{
			BaselineTelemetryCsvFilePath.Empty();
		}
		if (FPaths::FileExists(BaselineTelemetryCsvFilePath))
		{
			UE_LOGF(LogSummarizeTrace, Display, "Comparing telemetry to baseline telemetry %ls...", *BaselineTelemetryCsvFilePath);

			// each context (scope name or coutner name) and data point (statistic name) pair form a key, an item to check
			TMap<TPair<FString, FString>, TelemetryDefinition> ContextAndDataPointToDefinitionMap;
			bool bCSVOk = TelemetryDefinition::LoadFromCSV(*BaselineTelemetryCsvFilePath, ContextAndDataPointToDefinitionMap);
			check(bCSVOk);

			// for every telemetry item we wrote for this trace...
			for (TelemetryDefinition& Telemetry : TelemetryData)
			{
				// the threshold is defined along with the original statistic map
				const StatisticDefinition* RelatedStatistic = nullptr;

				// find the statistic definition
				TArray<StatisticDefinition> Statistics;
				NameToDefinitionMap.MultiFind(Telemetry.Context, Statistics, true);
				for (const StatisticDefinition& Statistic : Statistics)
				{
					// the find will match on name, here we just need to find the right statistic for that named item
					if (Statistic.Statistic == Telemetry.DataPoint)
					{
						// we found it!
						RelatedStatistic = &Statistic;
						break;
					}
				}

				// do we still have the statistic definition in our current stats file? (if we don't that's fine, we don't care about it anymore)
				if (RelatedStatistic)
				{
					// find the corresponding keyed telemetry item in the baseline telemetry file...
					TelemetryDefinition* BaselineTelemetry = ContextAndDataPointToDefinitionMap.Find(TPair<FString, FString>(Telemetry.Context, Telemetry.DataPoint));
					if (BaselineTelemetry)
					{
						Telemetry.Baseline = BaselineTelemetry->Measurement;

						// let's only report on statistics that have an assigned threshold, to keep things concise
						if (!RelatedStatistic->BaselineWarningThreshold.IsEmpty() || !RelatedStatistic->BaselineErrorThreshold.IsEmpty())
						{
							// verify that this telemetry measurement is within the allowed threshold as defined in the current stats file
							if (TelemetryDefinition::MeasurementWithinThreshold(Telemetry.Measurement, BaselineTelemetry->Measurement, RelatedStatistic->BaselineWarningThreshold))
							{
								FString SignFlippedWarningThreshold = TelemetryDefinition::SignFlipThreshold(RelatedStatistic->BaselineWarningThreshold);

								// check if it's beyond the threshold the other way and needs lowering in the stats csv
								if (!TelemetryDefinition::MeasurementWithinThreshold(Telemetry.Measurement, BaselineTelemetry->Measurement, SignFlippedWarningThreshold))
								{
									FString BaselineRelPath = FPaths::ConvertRelativePathToFull(BaselineTelemetryCsvFilePath);
									FPaths::MakePathRelativeTo(BaselineRelPath, *FPaths::RootDir());

									UE_LOGF(LogSummarizeTrace, Warning, "Telemetry %ls,%ls,%ls,%ls significantly within baseline value %ls using warning threshold %ls. Please submit a new baseline to %ls or adjust the threshold in the statistics file.",
										*Telemetry.TestName, *Telemetry.Context, *Telemetry.DataPoint, *Telemetry.Measurement,
										*BaselineTelemetry->Measurement, *RelatedStatistic->BaselineWarningThreshold,
										*BaselineRelPath);
								}
								else // it's within tolerance, just report that it's ok
								{
									UE_LOGF(LogSummarizeTrace, Verbose, "Telemetry %ls,%ls,%ls,%ls within baseline value %ls using warning threshold %ls",
										*Telemetry.TestName, *Telemetry.Context, *Telemetry.DataPoint, *Telemetry.Measurement,
										*BaselineTelemetry->Measurement, *RelatedStatistic->BaselineWarningThreshold);
								}
							}
							else
							{
								// it's outside warning threshold, check if it's inside the error threshold to just issue a warning
								if (TelemetryDefinition::MeasurementWithinThreshold(Telemetry.Measurement, BaselineTelemetry->Measurement, RelatedStatistic->BaselineErrorThreshold))
								{
									UE_LOGF(LogSummarizeTrace, Warning, "Telemetry %ls,%ls,%ls,%ls beyond baseline value %ls using warning threshold %ls. This could be a performance regression!",
										*Telemetry.TestName, *Telemetry.Context, *Telemetry.DataPoint, *Telemetry.Measurement,
										*BaselineTelemetry->Measurement, *RelatedStatistic->BaselineWarningThreshold);
								}
								else // it's outside the error threshold, hard error
								{
									UE_LOGF(LogSummarizeTrace, Error, "Telemetry %ls,%ls,%ls,%ls beyond baseline value %ls using error threshold %ls. This could be a performance regression!",
										*Telemetry.TestName, *Telemetry.Context, *Telemetry.DataPoint, *Telemetry.Measurement,
										*BaselineTelemetry->Measurement, *RelatedStatistic->BaselineErrorThreshold);
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogSummarizeTrace, Display, "Telemetry for %ls,%ls has no baseline measurement, skipping...", *Telemetry.Context, *Telemetry.DataPoint);
					}
				}
			}
		}

		// check for references to statistics desired for telemetry that are unresolved
		for (const FString& StatisticName : ResolvedStatistics)
		{
			NameToDefinitionMap.Remove(StatisticName);
		}

		for (const TPair<FString, StatisticDefinition>& Statistic : NameToDefinitionMap)
		{
			UE_LOGF(LogSummarizeTrace, Error, "Failed to find resolve telemety data for statistic \"%ls\"", *Statistic.Value.Name);
		}

		if (!NameToDefinitionMap.IsEmpty())
		{
			UE_LOGF(LogSummarizeTrace, Error, "Exiting...");
			return false;
		}
	}


	TUniquePtr<IFileHandle> TelemetryCsvHandle = OpenCSVFile(TEXT("Telemetry"));
	if (!TelemetryCsvHandle)
	{
		return false;
	}
	
	// no newline, see row printfs
	CsvUtils::WriteAsUTF8String(TelemetryCsvHandle.Get(), FString::Printf(TEXT("TestName,Context,DataPoint,Unit,Measurement,Baseline,")));
	for (const TelemetryDefinition& Telemetry : TelemetryData)
	{
		// note newline is at the front of every data line to prevent final extraneous newline, per customary for csv
		CsvUtils::WriteAsUTF8String(TelemetryCsvHandle.Get(), FString::Printf(TEXT("\n%s,%s,%s,%s,%s,%s,"), *Telemetry.TestName, *Telemetry.Context, *Telemetry.DataPoint, *Telemetry.Unit, *Telemetry.Measurement, *Telemetry.Baseline));
	}

	TelemetryCsvHandle->Flush();
	return true;
}
