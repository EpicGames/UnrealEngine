// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/SummarizeTraceEditorWorkflowsCommandlet.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "String/ParseTokens.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Analysis.h"
#include "Trace/DataStream.h"
#include "TraceServices/AnalyzerFactories.h"
#include "TraceServices/Model/Bookmarks.h"
#include "TraceServices/Model/Counters.h"
#include "TraceServices/Utils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SummarizeTraceEditorWorkflowsCommandlet)

class FSummarizeBookmarksProviderEditorWorkflows
	: public TraceServices::IEditableBookmarkProvider
{
public:
	virtual void UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatString) override;
	virtual void AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text) override;
	struct FBookmarkSpec
	{
		const TCHAR* File = nullptr;
		const TCHAR* FormatString = nullptr;
		int32 Line = 0;
	};
	FBookmarkSpec& GetSpec(uint64 BookmarkPoint);
	FSummarizeBookmark* FindStartBookmarkForEndBookmark(const FString& Name);

	// Keyed by a unique memory address
	TMap<uint64, FBookmarkSpec> BookmarkSpecs;
	// Keyed by name
	TMap<FString, FSummarizeBookmark> Bookmarks;

private:
	enum
	{
		FormatBufferSize = 65536
	};
	TCHAR FormatBuffer[FormatBufferSize];
	TCHAR TempBuffer[FormatBufferSize];
};

FSummarizeBookmarksProviderEditorWorkflows::FBookmarkSpec& FSummarizeBookmarksProviderEditorWorkflows::GetSpec(uint64 BookmarkPoint)
{
	FBookmarkSpec* Found = BookmarkSpecs.Find(BookmarkPoint);
	if (Found)
	{
		return *Found;
	}

	FBookmarkSpec& Spec = BookmarkSpecs.Add(BookmarkPoint, FBookmarkSpec());
	Spec.File = TEXT("<unknown>");
	Spec.FormatString = TEXT("<unknown>");
	return Spec;
}

void FSummarizeBookmarksProviderEditorWorkflows::UpdateBookmarkSpec(uint64 BookmarkPoint, const TCHAR* FormatString, const TCHAR* File, int32 Line)
{
	FBookmarkSpec& BookmarkSpec = GetSpec(BookmarkPoint);
	BookmarkSpec.FormatString = FormatString;
	BookmarkSpec.File = File;
	BookmarkSpec.Line = Line;
}

void FSummarizeBookmarksProviderEditorWorkflows::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const uint8* FormatArgs)
{
	FBookmarkSpec& BookmarkSpec = GetSpec(BookmarkPoint);
	TraceServices::StringFormat(FormatBuffer, FormatBufferSize - 1, TempBuffer, FormatBufferSize - 1, BookmarkSpec.FormatString, FormatArgs);
	FSummarizeBookmarksProviderEditorWorkflows::AppendBookmark(BookmarkPoint, Time, CallstackId, FormatBuffer);
}

void FSummarizeBookmarksProviderEditorWorkflows::AppendBookmark(uint64 BookmarkPoint, double Time, uint32 CallstackId, const TCHAR* Text)
{
	FString Name(Text);

	FSummarizeBookmark* FoundBookmark = Bookmarks.Find(Name);
	if (!FoundBookmark)
	{
		FoundBookmark = &Bookmarks.Add(Name, FSummarizeBookmark());
		FoundBookmark->Name = Name;
	}

	FoundBookmark->AddTimestamp(Time);
}

FSummarizeBookmark* FSummarizeBookmarksProviderEditorWorkflows::FindStartBookmarkForEndBookmark(const FString& Name)
{
	int32 Index = Name.Find(TEXT("Complete"));
	if (Index != -1)
	{
		FString StartName = Name;
		StartName.RemoveAt(Index, TCString<TCHAR>::Strlen(TEXT("Complete")));
		return Bookmarks.Find(StartName);
	}

	return nullptr;
}

USummarizeTraceEditorWorkflowsCommandlet::USummarizeTraceEditorWorkflowsCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 USummarizeTraceEditorWorkflowsCommandlet::Main(const FString& CmdLineParams)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*CmdLineParams, Tokens, Switches, ParamVals);

	// Display help
	if (Switches.Contains("help"))
	{
		UE_LOGF(LogSummarizeTrace, Log, "SummarizeTraceEditorWorkflows");
		UE_LOGF(LogSummarizeTrace, Log, "This commandlet will summarize a utrace into a csv to ease of parsing by reporting tool.");
		UE_LOGF(LogSummarizeTrace, Log, "Options:");
		UE_LOGF(LogSummarizeTrace, Log, " Required: -inputfile=<utrace path>   (The utrace you wish to process)");
		return 0;
	}

	FString TraceFileName;
	if (FParse::Value(*CmdLineParams, TEXT("inputfile="), TraceFileName, true))
	{
		UE_LOGF(LogSummarizeTrace, Display, "Loading trace from %ls", *TraceFileName);
	}
	else
	{
		UE_LOGF(LogSummarizeTrace, Error, "You must specify a utrace file using -inputfile=<path>");
		return 1;
	}

	if (!FPaths::FileExists(TraceFileName))
	{
		UE_LOGF(LogSummarizeTrace, Error, "Trace file '%ls' was not found", *TraceFileName);
		return 1;
	}

	UE::Trace::FFileDataStream* DataStream = new UE::Trace::FFileDataStream();
	if (!DataStream->Open(*TraceFileName))
	{
		UE_LOGF(LogSummarizeTrace, Error, "Unable to open trace file '%ls' for read", *TraceFileName);
		delete DataStream;
		return 1;
	}

	// setup analysis context with analyzers
	UE::Trace::FAnalysisContext AnalysisContext;

	// List of summarized scopes.
	TArray<FSummarizeScope> CollectedScopeSummaries;

	// Analyze CPU scope timer individually.
	TSharedPtr<FSummarizeCpuScopeDurationAnalyzer> IndividualScopeAnalyzer = MakeShared<FSummarizeCpuScopeDurationAnalyzer>(
		[&CollectedScopeSummaries](const TArray<FSummarizeScope>& ScopeSummaries)
		{
			// Collect all individual scopes summary from this analyzer.
			CollectedScopeSummaries.Append(ScopeSummaries);
		});

	TSharedPtr<TraceServices::IAnalysisSession> Session = TraceServices::CreateAnalysisSession(0, nullptr, TUniquePtr<UE::Trace::IInDataStream>(DataStream));

	FSummarizeBookmarksProviderEditorWorkflows BookmarksProvider;
	TSharedPtr<UE::Trace::IAnalyzer> BookmarksAnalyzer = TraceServices::CreateBookmarksAnalyzer(*Session, BookmarksProvider);
	AnalysisContext.AddAnalyzer(*BookmarksAnalyzer);

	FSummarizeCpuProfilerProvider CpuProfilerProvider;
	CpuProfilerProvider.AddCpuScopeAnalyzer(IndividualScopeAnalyzer);
	TSharedPtr<UE::Trace::IAnalyzer> CpuProfilerAnalyzer = TraceServices::CreateCpuProfilerAnalyzer(*Session, CpuProfilerProvider, CpuProfilerProvider);
	AnalysisContext.AddAnalyzer(*CpuProfilerAnalyzer);

	// kick processing on a thread
	UE::Trace::FAnalysisProcessor AnalysisProcessor = AnalysisContext.Process(*DataStream);

	// sync on completion
	AnalysisProcessor.Wait();

	CpuProfilerProvider.AnalysisComplete();

	// some locals to help with all the derived files we are about to generate
	FString TracePath = FPaths::GetPath(TraceFileName);

	// Define a map of test names to a pair of their begin and end times.
	TMap<FString, TPair<double, double>> TestNameAndTimes;
	for (const TMap<FString, FSummarizeBookmark>::ElementType& Bookmark : BookmarksProvider.Bookmarks)
	{
		if (!CsvUtils::IsCsvSafeString(Bookmark.Value.Name))
		{
			continue;
		}

		FString TestName;
		FString BookmarkAnnotation;
		Bookmark.Value.Name.Split(" ", &TestName, &BookmarkAnnotation);
		const TCHAR* ProfileBeginBookmark = TEXT("ProfileBegin");
		const TCHAR* ProfileEndBookmark = TEXT("ProfileEnd");

		if (BookmarkAnnotation != ProfileBeginBookmark && BookmarkAnnotation != ProfileEndBookmark)
		{
			continue;
		}

		TPair<double, double>& TestTime = TestNameAndTimes.FindOrAdd(TestName);

		if (BookmarkAnnotation == ProfileBeginBookmark)
		{
			TestTime.Key = Bookmark.Value.FirstSeconds;
		}
		else if (BookmarkAnnotation == ProfileEndBookmark)
		{
			TestTime.Value = Bookmark.Value.FirstSeconds;
		}
		else 
		{
			UE_LOGF(LogSummarizeTrace, Error, "Unrecognized bookmark: '%ls', bookmark should end with either 'ProfileBegin' or 'ProfileEnd'", *BookmarkAnnotation);
		}
	}

	// Loop through our map of test times and generate the CSV files.
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	for (const TMap<FString, TPair<double, double>>::ElementType& TestNameAndTime : TestNameAndTimes)
	{
		FString DirectoryPath = TracePath;
		DirectoryPath.Append(TEXT("/"));
		DirectoryPath.Append(TestNameAndTime.Key);

		// Create directory for each individual test
		if (!PlatformFile.DirectoryExists(*DirectoryPath))
		{
			PlatformFile.CreateDirectory(*DirectoryPath);

			if (!PlatformFile.DirectoryExists(*DirectoryPath))
			{
				UE_LOGF(LogSummarizeTrace, Error, "Failed to create directory: '%ls'", *DirectoryPath);
				return 1;
			}
		}

		if (!GenerateTimeCSV(DirectoryPath, CollectedScopeSummaries, TestNameAndTime.Value.Key, TestNameAndTime.Value.Value))
		{
			return 1;
		}
	}

	return 0;
}

TPair<int32, TArray<int32>> USummarizeTraceEditorWorkflowsCommandlet::ClipScopes(const TArray<const FSummarizeScope*>& Scopes, double BeginTime, double EndTime)
{
	TArray<int32> BeginIndices;
	int32 Size = TNumericLimits<int32>::Max();

	for (const FSummarizeScope* Scope : Scopes)
	{
		int32 BeginIdx = Algo::LowerBound(Scope->BeginTimeArray, BeginTime);
		int32 EndIdx = Algo::UpperBound(Scope->EndTimeArray, EndTime);

		// Ensure the EndIndex is not less than BeginIndex
		if (EndIdx < BeginIdx)
		{
			UE_LOGF(LogSummarizeTrace, Error, "End index is smaller than begin index detected during clipping for ScopeName: '%ls'", *(Scope->Name));
			EndIdx = BeginIdx;
		}

		BeginIndices.Add(BeginIdx);
		Size = FMath::Min(Size, EndIdx - BeginIdx);
	}

	// Return the size and the array of Begin Indices
	return { Size, BeginIndices };
}

TUniquePtr<IFileHandle> USummarizeTraceEditorWorkflowsCommandlet::OpenCSVFile(const FString& Path)
{
	FString CsvFileName = FPaths::Combine(Path, FPaths::SetExtension("Result", "csv"));
	UE_LOGF(LogSummarizeTrace, Display, "Writing %ls...", *CsvFileName);
	TUniquePtr<IFileHandle> CsvHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*CsvFileName));

	if (!CsvHandle)
	{
		UE_LOGF(LogSummarizeTrace, Error, "Unable to open csv '%ls' for write", *CsvFileName);
	}

	return CsvHandle;
}

bool USummarizeTraceEditorWorkflowsCommandlet::GenerateTimeCSV(const FString& Path, const TArray<FSummarizeScope>& Scopes, double BeginTime, double EndTime)
{
	TUniquePtr<IFileHandle> CsvHandle = OpenCSVFile(Path);
	if (!CsvHandle)
	{
		return false;
	}

	const TSet<FString> ScopeNamesToTrack = { TEXT("FEngineLoop::Tick"), TEXT("Slate::Tick (Time and Widgets)") };
	TArray<const FSummarizeScope*> TrackedScopes;
	TrackedScopes.Reserve(ScopeNamesToTrack.Num());
	for (const FSummarizeScope& Scope : Scopes)
	{
		if (ScopeNamesToTrack.Contains(Scope.Name))
		{
			TrackedScopes.Add(&Scope);
		}
	}

	// Write the header
	FString header = "FrameNumber";
	for (const FString& name : ScopeNamesToTrack)
	{
		header.Append(",");
		header.Append(name);
	}
	CsvUtils::WriteAsUTF8String(CsvHandle.Get(), header);

	// Clip
	TPair<int32, TArray<int32>> TimeRange = ClipScopes(TrackedScopes, BeginTime, EndTime);

	// Write the data
	for (int32 FrameNum = 0; FrameNum < TimeRange.Key; ++FrameNum)
	{
		FString DataLine = FString::Printf(TEXT("\n%d"), FrameNum);
		for (int32 i = 0; i < TrackedScopes.Num(); ++i)
		{
			const FSummarizeScope* Scope = TrackedScopes[i];
			int32 Idx = TimeRange.Value[i] + FrameNum;

			DataLine.Append(FString::Printf(TEXT(",%f"), Scope->EndTimeArray[Idx] - Scope->BeginTimeArray[Idx]));
		}

		CsvUtils::WriteAsUTF8String(CsvHandle.Get(), DataLine);
	}

	CsvHandle->Flush();
	return true;
}
