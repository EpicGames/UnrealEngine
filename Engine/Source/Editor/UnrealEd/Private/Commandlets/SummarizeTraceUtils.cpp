// Copyright Epic Games, Inc. All Rights Reserved.

#include "SummarizeTraceUtils.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringConv.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "GenericPlatform/GenericPlatformAffinity.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/Crc.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/Timespan.h"
#include "Model/MonotonicTimeline.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"

DEFINE_LOG_CATEGORY(LogSummarizeTrace);

uint64 FIncrementalVariance::GetCount() const
{
	return Count;
}

double FIncrementalVariance::GetMean() const
{
	return Mean;
}

double FIncrementalVariance::Variance() const
{
	double Result = 0.0;

	if (Count > 1)
	{
		// Welford's final step, dependent on sample count
		Result = VarianceAccumulator / double(Count - 1);
	}

	return Result;
}

double FIncrementalVariance::Deviation() const
{
	double Result = 0.0;

	if (Count > 1)
	{
		// Welford's final step, dependent on sample count
		double DeviationSqrd = VarianceAccumulator / double(Count - 1);

		// stddev is sqrt of variance, to restore to units (vs. units squared)
		Result = sqrt(DeviationSqrd);
	}

	return Result;
}

void FIncrementalVariance::Increment(const double InSample)
{
	Count++;
	const double OldMean = Mean;
	Mean += ((InSample - Mean) / double(Count));
	VarianceAccumulator += ((InSample - Mean) * (InSample - OldMean));
}

void FIncrementalVariance::Merge(const FIncrementalVariance& Other)
{
	// empty other, nothing to do
	if (Other.Count == 0)
	{
		return;
	}

	// empty this, just copy other
	if (Count == 0)
	{
		Count = Other.Count;
		Mean = Other.Mean;
		VarianceAccumulator = Other.VarianceAccumulator;
		return;
	}

	const double TotalPopulation = static_cast<double>(Count + Other.Count);
	const double MeanDifference = Mean - Other.Mean;
	const double A = ((Count - 1) * Variance()) + ((Other.Count - 1) * Other.Variance());
	const double B = (MeanDifference) * (MeanDifference) * (Count * Other.Count / TotalPopulation);
	const double MergedVariance = (A + B) / (TotalPopulation - 1);

	const uint64 NewCount = Count + Other.Count;
	const double NewMean = ((Mean * double(Count)) + (Other.Mean * double(Other.Count))) / double(NewCount);
	const double NewVarianceAccumulator = MergedVariance * (NewCount - 1);

	Count = NewCount;
	Mean = NewMean;
	VarianceAccumulator = NewVarianceAccumulator;
}

void FIncrementalVariance::Reset()
{
	Count = 0;
	Mean = 0.0;
	VarianceAccumulator = 0.0;
}

void FSummarizeScope::AddDuration(double StartSeconds, double FinishSeconds)
{
	// compute the duration
	double DurationSeconds = FinishSeconds - StartSeconds;

	DurationVariance.Increment(DurationSeconds);

	// only set first for the first sample
	if (DurationVariance.GetCount() == 1)
	{
		FirstStartSeconds = StartSeconds;
		FirstFinishSeconds = FinishSeconds;
		FirstDurationSeconds = DurationSeconds;
	}

	LastStartSeconds = StartSeconds;
	LastFinishSeconds = FinishSeconds;
	LastDurationSeconds = DurationSeconds;

	// set duration statistics
	TotalDurationSeconds += DurationSeconds;
	MinDurationSeconds = FMath::Min(MinDurationSeconds, DurationSeconds);
	MaxDurationSeconds = FMath::Max(MaxDurationSeconds, DurationSeconds);

	BeginTimeArray.Add(StartSeconds);
	EndTimeArray.Add(FinishSeconds);
}

uint64 FSummarizeScope::GetCount() const
{
	return DurationVariance.GetCount();
}

double FSummarizeScope::GetMeanDurationSeconds() const
{
	return DurationVariance.GetMean();
}

double FSummarizeScope::GetDeviationDurationSeconds() const
{
	return DurationVariance.Deviation();
}

double FSummarizeScope::GetCountPerSecond() const
{
	double CountPerSecond = 0.0;
	const uint64 Count = DurationVariance.GetCount();
	if (Count)
	{
		CountPerSecond = Count / (LastFinishSeconds - FirstStartSeconds);
	}
	return CountPerSecond;
}

void FSummarizeScope::Merge(const FSummarizeScope& Other)
{
	check(Name == Other.Name);
	DurationVariance.Merge(Other.DurationVariance);

	if (FirstStartSeconds > Other.FirstStartSeconds)
	{
		FirstStartSeconds = Other.FirstStartSeconds;
		FirstFinishSeconds = Other.FirstFinishSeconds;
		FirstDurationSeconds = Other.FirstDurationSeconds;
	}

	if (LastStartSeconds < Other.LastStartSeconds)
	{
		LastStartSeconds = Other.LastStartSeconds;
		LastFinishSeconds = Other.LastFinishSeconds;
		LastDurationSeconds = Other.LastDurationSeconds;
	}

	TotalDurationSeconds += Other.TotalDurationSeconds;
	MinDurationSeconds = FMath::Min(MinDurationSeconds, Other.MinDurationSeconds);
	MaxDurationSeconds = FMath::Max(MaxDurationSeconds, Other.MaxDurationSeconds);
}

FString FSummarizeScope::GetValue(const FStringView& Statistic) const
{
	if (Statistic == TEXT("Name"))
	{
		return Name;
	}
	else if (Statistic == TEXT("Count"))
	{
		return FString::Printf(TEXT("%llu"), DurationVariance.GetCount());
	}
	else if (Statistic == TEXT("TotalDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), TotalDurationSeconds);
	}
	else if (Statistic == TEXT("FirstStartSeconds"))
	{
		return FString::Printf(TEXT("%f"), FirstStartSeconds);
	}
	else if (Statistic == TEXT("FirstFinishSeconds"))
	{
		return FString::Printf(TEXT("%f"), FirstFinishSeconds);
	}
	else if (Statistic == TEXT("FirstDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), FirstDurationSeconds);
	}
	else if (Statistic == TEXT("LastStartSeconds"))
	{
		return FString::Printf(TEXT("%f"), LastStartSeconds);
	}
	else if (Statistic == TEXT("LastFinishSeconds"))
	{
		return FString::Printf(TEXT("%f"), LastFinishSeconds);
	}
	else if (Statistic == TEXT("LastDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), LastDurationSeconds);
	}
	else if (Statistic == TEXT("MinDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), MinDurationSeconds);
	}
	else if (Statistic == TEXT("MaxDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), MaxDurationSeconds);
	}
	else if (Statistic == TEXT("MeanDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), DurationVariance.GetMean());
	}
	else if (Statistic == TEXT("DeviationDurationSeconds"))
	{
		return FString::Printf(TEXT("%f"), DurationVariance.Deviation());
	}
	else if (Statistic == TEXT("CountPerSecond"))
	{
		return FString::Printf(TEXT("%f"), GetCountPerSecond());
	}
	return FString();
}

// for deduplication
bool FSummarizeScope::operator==(const FSummarizeScope& Scope) const
{
	return Name == Scope.Name;
}

// for sorting descending
bool FSummarizeScope::operator<(const FSummarizeScope& Scope) const
{
	return TotalDurationSeconds > Scope.TotalDurationSeconds;
}

FSummarizeCpuProfilerProvider::FSummarizeCpuProfilerProvider()
	: LookupScopeNameFn([this](uint32 ScopeId) { return LookupScopeName(ScopeId); })
{
}

void FSummarizeCpuProfilerProvider::AddCpuScopeAnalyzer(TSharedPtr<FSummarizeCpuScopeAnalyzer> Analyzer)
{
	ScopeAnalyzers.Add(MoveTemp(Analyzer));
}

void FSummarizeCpuProfilerProvider::AnalysisComplete()
{
	// Analyze scope trees that contained 'nameless' context when they were captured. Unless the trace was truncated,
	// all scope names should be known now.
	for (const auto& Thread : Threads)
	{
		for (FScopeTreeInfo& DelayedScopeTree : Thread.Value->DelayedScopeTreeInfo)
		{
			// Run summary analysis for this delayed hierarchy.
			OnCpuScopeTree(Thread.Key, DelayedScopeTree);
		}
	}

	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeAnalysisEnd();
	}
}

void FSummarizeCpuProfilerProvider::AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority)
{
	TUniquePtr<FThread>* Found = Threads.Find(Id);
	if (!Found)
	{
		Threads.Add(Id, MakeUnique<FThread>(Id, this));
	}
}

uint32 FSummarizeCpuProfilerProvider::AddTimer(TraceServices::ETimingProfilerTimerType Type)
{
	if (Type == TraceServices::ETimingProfilerTimerType::CPU)
	{
		return AddCpuTimerInternal(FStringView());
	}
	return 0;
}

uint32 FSummarizeCpuProfilerProvider::AddCpuTimer(FStringView Name, const TCHAR* File, uint32 Line)
{
	return AddCpuTimerInternal(Name);
}

uint32 FSummarizeCpuProfilerProvider::AddCpuTimerInternal(FStringView Name)
{
	TOptional<FString> ScopeName;

	if (!Name.IsEmpty())
	{
		ScopeName.Emplace(FString(Name));
	}

	uint32 TimerId = ScopeNames.Add(ScopeName);

	// Notify the analyzers.
	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeDiscovered(TimerId);

		if (!Name.IsEmpty())
		{
			Analyzer->OnCpuScopeName(TimerId, Name);
		}
	}

	return TimerId;
}

void FSummarizeCpuProfilerProvider::SetTimerName(uint32 TimerId, FStringView Name)
{
	check(TimerId < uint32(ScopeNames.Num()));
	check(!Name.IsEmpty());

	ScopeNames[TimerId].Emplace(FString(Name));

	// Notify the registered scope analyzers.
	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeName(TimerId, Name);
	}
}

uint32 FSummarizeCpuProfilerProvider::AddMetadata(uint32 MasterTimerId, TArray<uint8>&& Metadata)
{
	uint32 MetadataId = Metadatas.Num();
	Metadatas.Add({ MoveTemp(Metadata), MasterTimerId });
	return ~MetadataId;
}

TArrayView<uint8> FSummarizeCpuProfilerProvider::GetEditableMetadata(uint32 TimerId)
{
	if (int32(TimerId) >= 0)
	{
		return TArrayView<uint8>();
	}

	TimerId = ~TimerId;
	if (TimerId >= uint32(Metadatas.Num()))
	{
		return TArrayView<uint8>();
	}

	FMetadata& Metadata = Metadatas[TimerId];
	return Metadata.Payload;
}

TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>& FSummarizeCpuProfilerProvider::GetCpuThreadEditableTimeline(uint32 ThreadId)
{
	TUniquePtr<FThread>* Found = Threads.Find(ThreadId);
	if (Found)
	{
		return *(Found->Get());
	}

	return *Threads.Add(ThreadId, MakeUnique<FThread>(ThreadId, this));
}

void FSummarizeCpuProfilerProvider::FThread::AppendBeginEvent(double StartTime, const TraceServices::FTimingProfilerEvent& Event)
{
	FScopeEnter ScopeEnter{ Event.TimerIndex, StartTime };
	ScopeStack.Add(ScopeEnter);

	FSummarizeCpuScopeAnalyzer::FScopeEvent ScopeEvent{ FSummarizeCpuScopeAnalyzer::EScopeEventType::Enter, Event.TimerIndex, ThreadId, StartTime };
	ScopeTreeInfo.ScopeEvents.Add(ScopeEvent);

	Provider->AppendBeginEvent(ScopeEvent);
}

void FSummarizeCpuProfilerProvider::FThread::AppendEndEvent(double EndTime)
{
	if (ScopeStack.IsEmpty())
	{
		return;
	}

	FScopeEnter ScopeEnter = ScopeStack.Pop();

	FSummarizeCpuScopeAnalyzer::FScopeEvent ScopeEvent{ FSummarizeCpuScopeAnalyzer::EScopeEventType::Exit, ScopeEnter.ScopeId, ThreadId, EndTime };
	ScopeTreeInfo.ScopeEvents.Add(ScopeEvent);

	// Check if at this point if the scope has a name
	const FString* ScopeName = Provider->LookupScopeName(ScopeEnter.ScopeId);
	ScopeTreeInfo.bHasNamelessScopes |= ScopeName == nullptr || ScopeName->IsEmpty();

	FSummarizeCpuScopeAnalyzer::FScope Scope{ ScopeEnter.ScopeId, ThreadId, ScopeEnter.Timestamp, EndTime };
	Provider->AppendEndEvent(Scope, ScopeName);

	// The root scope on this thread just popped out.
	if (ScopeStack.IsEmpty())
	{
		if (ScopeTreeInfo.bHasNamelessScopes)
		{
			// Delay the analysis until all the scope names are known.
			DelayedScopeTreeInfo.Add(MoveTemp(ScopeTreeInfo));
		}
		else
		{
			// Run analysis for this scope tree.
			Provider->OnCpuScopeTree(ThreadId, ScopeTreeInfo);
		}

		ScopeTreeInfo.Reset();
	}
}

void FSummarizeCpuProfilerProvider::AppendBeginEvent(const FSummarizeCpuScopeAnalyzer::FScopeEvent& ScopeEvent)
{
	// Notify the registered scope analyzers.
	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeEnter(ScopeEvent, LookupScopeName(ScopeEvent.ScopeId));
	}
}

void FSummarizeCpuProfilerProvider::AppendEndEvent(const FSummarizeCpuScopeAnalyzer::FScope& Scope, const FString* ScopeName)
{
	// Notify the registered scope analyzers.
	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeExit(Scope, ScopeName);
	}
}

void FSummarizeCpuProfilerProvider::OnCpuScopeTree(uint32 ThreadId, const FScopeTreeInfo& ScopeTreeInfo)
{
	// Notify the registered scope analyzers.
	for (TSharedPtr<FSummarizeCpuScopeAnalyzer>& Analyzer : ScopeAnalyzers)
	{
		Analyzer->OnCpuScopeTree(ThreadId, ScopeTreeInfo.ScopeEvents, LookupScopeNameFn);
	}
}

const FString* FSummarizeCpuProfilerProvider::LookupScopeName(uint32 ScopeId)
{
	if (int32(ScopeId) < 0)
	{
		ScopeId = Metadatas[~ScopeId].TimerId;
	}

	if (ScopeId < static_cast<uint32>(ScopeNames.Num()) && ScopeNames[ScopeId])
	{
		return &ScopeNames[ScopeId].GetValue();
	}

	return nullptr;
}

FSummarizeCpuScopeDurationAnalyzer::FSummarizeCpuScopeDurationAnalyzer(TFunction<void(const TArray<FSummarizeScope>&)> InPublishFn)
	: PublishFn(MoveTemp(InPublishFn))
{
	OnCpuScopeDiscovered(FSummarizeCpuScopeAnalyzer::CoroutineSpecId);
	OnCpuScopeDiscovered(FSummarizeCpuScopeAnalyzer::CoroutineUnknownSpecId);
	OnCpuScopeName(FSummarizeCpuScopeAnalyzer::CoroutineSpecId, TEXT("Coroutine"));
	OnCpuScopeName(FSummarizeCpuScopeAnalyzer::CoroutineUnknownSpecId, TEXT("<unknown>"));
}

void FSummarizeCpuScopeDurationAnalyzer::OnCpuScopeDiscovered(uint32 ScopeId)
{
	if (!Scopes.Find(ScopeId))
	{
		Scopes.Add(ScopeId, FSummarizeScope());
	}
}

void FSummarizeCpuScopeDurationAnalyzer::OnCpuScopeName(uint32 ScopeId, const FStringView& ScopeName)
{
	Scopes[ScopeId].Name = ScopeName;
}

void FSummarizeCpuScopeDurationAnalyzer::OnCpuScopeExit(const FScope& Scope, const FString* ScopeName)
{
	// This can miss if we are given a metadata timer's index, which is negative signed
	FSummarizeScope* Found = Scopes.Find(Scope.ScopeId);
	if (Found)
	{
		Found->AddDuration(Scope.EnterTimestamp, Scope.ExitTimestamp);
	}
}

void FSummarizeCpuScopeDurationAnalyzer::OnCpuScopeAnalysisEnd()
{
	TArray<FSummarizeScope> LocalScopes;
	// Eliminates scopes that don't have a name. (On scope discovery, the array is expended to creates blank scopes that may never be filled).
	for (TMap<uint32, FSummarizeScope>::TIterator Iter = Scopes.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter.Value().Name.IsEmpty())
		{
			LocalScopes.Add(Iter.Value());
		}
	}

	// Publish the scopes.
	PublishFn(LocalScopes);
}

FString FSummarizeBookmark::GetValue(const FStringView& Statistic) const
{
	if (Statistic == TEXT("Name"))
	{
		return Name;
	}
	else if (Statistic == TEXT("Count"))
	{
		return FString::Printf(TEXT("%llu"), Count);
	}
	else if (Statistic == TEXT("FirstSeconds"))
	{
		return FString::Printf(TEXT("%f"), FirstSeconds);
	}
	else if (Statistic == TEXT("LastSeconds"))
	{
		return FString::Printf(TEXT("%f"), LastSeconds);
	}
	return FString();
}

void FSummarizeBookmark::AddTimestamp(double Seconds)
{
	Count += 1;

	// only set first for the first sample, compare exact zero
	if (FirstSeconds == 0.0)
	{
		FirstSeconds = Seconds;
	}

	LastSeconds = Seconds;
}

bool CsvUtils::IsCsvSafeString(const FString& String)
{
	static struct DisallowedCharacter
	{
		const TCHAR Character;
		bool First;
	}
	DisallowedCharacters[] =
	{
		// breaks simple csv files
		{ TEXT('\n'), true },
		{ TEXT('\r'), true },
		{ TEXT(','), true },
	};

	// sanitize strings for a bog-simple csv file
	bool bDisallowed = false;
	int32 Index = 0;
	for (struct DisallowedCharacter& DisallowedCharacter : DisallowedCharacters)
	{
		if (String.FindChar(DisallowedCharacter.Character, Index))
		{
			if (DisallowedCharacter.First)
			{
				UE_LOGF(LogSummarizeTrace, Display, "A string contains disallowed character '%c'. See log for full list.", DisallowedCharacter.Character);
				DisallowedCharacter.First = false;
			}

			UE_LOGF(LogSummarizeTrace, Verbose, "String '%ls' contains disallowed character '%c', skipping...", *String, DisallowedCharacter.Character);				bDisallowed = true;
		}

		if (bDisallowed)
		{
			break;
		}
	}

	return !bDisallowed;
}

void CsvUtils::WriteAsUTF8String(IFileHandle* Handle, const FString& String)
{
	const auto& UTF8String = StringCast<ANSICHAR>(*String);
	Handle->Write(reinterpret_cast<const uint8*>(UTF8String.Get()), UTF8String.Length());
}