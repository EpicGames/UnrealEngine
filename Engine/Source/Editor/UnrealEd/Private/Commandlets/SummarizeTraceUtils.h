// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceServices/Model/Threads.h"
#include "TraceServices/Model/TimingProfiler.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Trace { class IAnalyzer; }

// Defined here to prevent adding logs in the code above, which will likely be moved elsewhere.
DECLARE_LOG_CATEGORY_EXTERN(LogSummarizeTrace, Log, All);

/*
*
* Helper classes for the SummarizeTrace commandlet. Aggregates statistics about a trace.
* Code preceding this comment should eventually be relocated to the Trace/Insights modules.
* Everything succeeding this comment should stay in this commandlet.
*
*/

class FIncrementalVariance
{
public:
	FIncrementalVariance()
		: Count(0)
		, Mean(0.0)
		, VarianceAccumulator(0.0)
	{

	}

	uint64 GetCount() const;

	double GetMean() const;

	/**
	* Compute the variance given Welford's accumulator and the overall count
	*
	* @return The variance in sample units squared
	*/
	double Variance() const;

	/**
	* Compute the standard deviation given Welford's accumulator and the overall count
	*
	* @return The standard deviation in sample units
	*/
	double Deviation() const;

	/**
	* Perform an increment of work for Welford's variance, from which we can compute variation and standard deviation
	*
	* @param InSample	The new sample value to operate on
	*/
	void Increment(const double InSample);

	/**
	* Merge with another IncrementalVariance series in progress
	*
	* @param Other	The other variance incremented from another mutually exclusive population of analogous data.
	*/
	void Merge(const FIncrementalVariance& Other);

	/**
	* Reset state back to initialized.
	*/
	void Reset();

private:
	uint64 Count;
	double Mean;
	double VarianceAccumulator;
};

/**
 * Base class to extend for CPU scope analysis. Derived class instances are meant to be registered as
 * delegates to the FSummarizeCpuProfilerProvider to handle scope events and perform meaningful analysis.
 */
class FSummarizeCpuScopeAnalyzer
{
public:
	enum class EScopeEventType : uint32
	{
		Enter,
		Exit
	};

	struct FScopeEvent
	{
		EScopeEventType ScopeEventType;
		uint32 ScopeId;
		uint32 ThreadId;
		double Timestamp; // As Seconds
	};

	struct FScope
	{
		uint32 ScopeId;
		uint32 ThreadId;
		double EnterTimestamp; // As Seconds
		double ExitTimestamp;  // As Seconds
	};

public:
	virtual ~FSummarizeCpuScopeAnalyzer() = default;

	/** Invoked when a CPU scope is discovered. This function is always invoked first when a CPU scope is encountered for the first time.*/
	virtual void OnCpuScopeDiscovered(uint32 ScopeId) {}

	/** Invoked when CPU scope specification is encountered in the trace stream. */
	virtual void OnCpuScopeName(uint32 ScopeId, const FStringView& ScopeName) {};

	/** Invoked when a scope is entered. The scope name might not be known yet. */
	virtual void OnCpuScopeEnter(const FScopeEvent& ScopeEnter, const FString* ScopeName) {};

	/** Invoked when a scope is exited. The scope name might not be known yet. */
	virtual void OnCpuScopeExit(const FScope& Scope, const FString* ScopeName) {};

	/** Invoked when a root event on the specified thread along with all child events down to the leaves are known. */
	virtual void OnCpuScopeTree(uint32 ThreadId, const TArray64<FSummarizeCpuScopeAnalyzer::FScopeEvent>& ScopeEvents, const TFunction<const FString* (uint32)>& ScopeLookupNameFn) {};

	/** Invoked when the trace stream has been fully consumed/processed. */
	virtual void OnCpuScopeAnalysisEnd() {};

	static constexpr uint32 CoroutineSpecId = (1u << 31u) - 1u;
	static constexpr uint32 CoroutineUnknownSpecId = (1u << 31u) - 2u;
};

/**
 * Decodes, format and routes CPU scope events embedded into a trace stream to specialized CPU scope analyzers. This processor
 * decodes the low level events, keeps a small state and publishes higher level events to registered analyzers. The purpose
 * is to decode the stream only once and let several registered analyzers reuse the small state built up by this processor. This
 * matters when processing very large traces with possibly billions of scope events.
 */
class FSummarizeCpuProfilerProvider
	: public TraceServices::IEditableThreadProvider
	, public TraceServices::IEditableTimingProfilerProvider
{
public:
	FSummarizeCpuProfilerProvider();

	/** Register an analyzer with this processor. The processor decodes the trace stream and invokes the registered analyzers when a CPU scope event occurs.*/
	void AddCpuScopeAnalyzer(TSharedPtr<FSummarizeCpuScopeAnalyzer> Analyzer);

	void AnalysisComplete();

private:
	struct FScopeEnter
	{
		uint32 ScopeId;
		double Timestamp; // As Seconds
	};

	// Contains scope events for a root scope and its children along with extra info to analyze that tree at once.
	struct FScopeTreeInfo
	{
		// Records the current root scope and its children to run analysis that needs to know the parent/child relationship.
		TArray64<FSummarizeCpuScopeAnalyzer::FScopeEvent> ScopeEvents;

		// Indicates if one of the scope in the current hierarchy is nameless. (Its names specs hasn't been received yet).
		bool bHasNamelessScopes = false;

		void Reset()
		{
			ScopeEvents.Reset();
			bHasNamelessScopes = false;
		}
	};

	// For each thread we track what the stack of scopes are, for matching end-to-start
	struct FThread
		: public TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>
	{
		FThread(uint32 InThreadId, FSummarizeCpuProfilerProvider* InProvider)
			: ThreadId(InThreadId)
			, Provider(InProvider)
		{

		}

		virtual void AppendBeginEvent(double StartTime, const TraceServices::FTimingProfilerEvent& Event) override;
		virtual void AppendEndEvent(double EndTime) override;

		// The ThreadId of this thread
		uint32 ThreadId;

		// The provider to forward calls to
		FSummarizeCpuProfilerProvider* Provider;

		// The current stack state
		TArray<FScopeEnter> ScopeStack;

		// The events recorded for the current root scope and its children to run analysis that needs to know the parent/child relationship, for example to compute time including childreen and time
		// excluding childreen.
		FScopeTreeInfo ScopeTreeInfo;

		// Scope trees for which as least one scope name was unknown. Some analysis need scope names, but scope names/event can be emitted out of order by the engine depending on thread scheduling.
		// Some scope tree cannot be analyzed right away and need to be delayed until all scope names are discovered.
		TArray<FScopeTreeInfo> DelayedScopeTreeInfo;
	};

private:
	// TraceServices::IEditableThreadProvider
	virtual void AddThread(uint32 Id, const TCHAR* Name, EThreadPriority Priority) override;

	// TraceServices::IEditableTimingProfilerProvider
	virtual uint32 AddTimer(TraceServices::ETimingProfilerTimerType Type) override;
	virtual uint32 AddCpuTimer(FStringView Name, const TCHAR* File, uint32 Line) override;
	virtual void SetTimerName(uint32 TimerId, FStringView Name) override;
	virtual uint32 AddMetadata(uint32 MasterTimerId, TArray<uint8>&& Metadata) override;
	virtual TArrayView<uint8> GetEditableMetadata(uint32 TimerId) override;
	virtual TraceServices::IEditableTimeline<TraceServices::FTimingProfilerEvent>& GetCpuThreadEditableTimeline(uint32 ThreadId) override;

	// Callbacks from FThread to forward calls to Analyzers
	virtual void AppendBeginEvent(const FSummarizeCpuScopeAnalyzer::FScopeEvent& ScopeEvent);
	virtual void AppendEndEvent(const FSummarizeCpuScopeAnalyzer::FScope& ScopeEvent, const FString* ScopeName);

	// Processing the ScopeTree once it's complete
	void OnCpuScopeTree(uint32 ThreadId, const FScopeTreeInfo& ScopeTreeInfo);

	// Resolve a ScopeId to string
	const FString* LookupScopeName(uint32 ScopeId);

	uint32 AddCpuTimerInternal(FStringView Name);

private:
	// The state at any moment of the threads
	TMap<uint32, TUniquePtr<FThread>> Threads;

	// The scope names, the array index correspond to the scope Id. If the optional is not set, the scope hasn't been encountered yet.
	TArray<TOptional<FString>> ScopeNames;

	// List of analyzers to invoke when a scope event is decoded.
	TArray<TSharedPtr<FSummarizeCpuScopeAnalyzer>> ScopeAnalyzers;

	// Scope name lookup function, cached for efficiency.
	TFunction<const FString* (uint32 ScopeId)> LookupScopeNameFn;

	struct FMetadata
	{
		TArray<uint8> Payload;
		uint32 TimerId;
	};

	TArray<FMetadata> Metadatas;
};

struct FSummarizeScope
{
	FString Name;
	FIncrementalVariance DurationVariance;
	double TotalDurationSeconds = 0.0;

	double FirstStartSeconds = 0.0;
	double FirstFinishSeconds = 0.0;
	double FirstDurationSeconds = 0.0;

	double LastStartSeconds = 0.0;
	double LastFinishSeconds = 0.0;
	double LastDurationSeconds = 0.0;

	double MinDurationSeconds = TNumericLimits<double>::Max();
	double MaxDurationSeconds = TNumericLimits<double>::Min();

	TArray<double> EndTimeArray;
	TArray<double> BeginTimeArray;

	void AddDuration(double StartSeconds, double FinishSeconds);

	uint64 GetCount() const;

	double GetMeanDurationSeconds() const;

	double GetDeviationDurationSeconds() const;

	double GetCountPerSecond() const;

	void Merge(const FSummarizeScope& Other);

	FString GetValue(const FStringView& Statistic) const;

	// for deduplication
	bool operator==(const FSummarizeScope& Scope) const;

	// for sorting descending
	bool operator<(const FSummarizeScope& Scope) const;
};

static uint32 GetTypeHash(const FSummarizeScope& Scope)
{
	return FCrc::StrCrc32(*Scope.Name);
}

/**
 * This analyzer aggregates CPU scopes having the same name together, counting the number of occurrences, total duration,
 * average occurrence duration, for each scope name.
 */
class FSummarizeCpuScopeDurationAnalyzer
	: public FSummarizeCpuScopeAnalyzer
{
public:
	FSummarizeCpuScopeDurationAnalyzer(TFunction<void(const TArray<FSummarizeScope>&)> InPublishFn);

protected:
	virtual void OnCpuScopeDiscovered(uint32 ScopeId) override;
	virtual void OnCpuScopeName(uint32 ScopeId, const FStringView& ScopeName) override;
	virtual void OnCpuScopeExit(const FScope& Scope, const FString* ScopeName) override;
	virtual void OnCpuScopeAnalysisEnd() override;

private:
	TMap<uint32, FSummarizeScope> Scopes;

	// Function invoked to publish the list of scopes.
	TFunction<void(const TArray<FSummarizeScope>&)> PublishFn;
};

//
// FSummarizeBookmarksProvider - Tally Bookmarks from bookmark events
//
struct FSummarizeBookmark
{
	FString Name;
	uint64 Count = 0;

	double FirstSeconds = 0.0;
	double LastSeconds = 0.0;

	void AddTimestamp(double Seconds);

	FString GetValue(const FStringView& Statistic) const;

	// for deduplication
	bool operator==(const FSummarizeBookmark& Bookmark) const
	{
		return Name == Bookmark.Name;
	}
};

static uint32 GetTypeHash(const FSummarizeBookmark& Bookmark)
{
	return FCrc::StrCrc32(*Bookmark.Name);
}

namespace CsvUtils
{

	/*
	* Helpers for the csv files
	*/

	bool IsCsvSafeString(const FString& String);

	void WriteAsUTF8String(IFileHandle* Handle, const FString& String);
}