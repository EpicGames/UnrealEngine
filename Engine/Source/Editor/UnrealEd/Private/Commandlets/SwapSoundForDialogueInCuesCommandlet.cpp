// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/SwapSoundForDialogueInCuesCommandlet.h"
#include "Modules/ModuleManager.h"
#include "SoundCueGraph/SoundCueGraph.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "AssetRegistry/AssetData.h"
#include "Sound/SoundWave.h"
#include "Sound/DialogueWave.h"
#include "Sound/SoundCue.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Sound/SoundNodeDialoguePlayer.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "AudioEditorModule.h"
#include "LocalizedAssetUtil.h"
#include "LocalizationSourceControlUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SwapSoundForDialogueInCuesCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogSwapSoundForDialogueInCuesCommandlet, Log, All);

int32 USwapSoundForDialogueInCuesCommandlet::Main(const FString& Params)
{
	// Prepare asset registry.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.SearchAllAssets(true);

	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Parameters;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, Parameters);

	TSharedPtr<FLocalizationSCC> SourceControlInfo;
	const bool bEnableSourceControl = Switches.Contains(TEXT("EnableSCC"));
	if (bEnableSourceControl)
	{
		SourceControlInfo = MakeShareable(new FLocalizationSCC());

		FText SCCErrorStr;
		if (!SourceControlInfo->IsReady(SCCErrorStr))
		{
			UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Revision Control error: %ls", *SCCErrorStr.ToString());
			return -1;
		}
	}

	// We only want dialogue wave assets that exist within the Game content directory.
	TArray<FAssetData> AssetDataArrayForDialogueWaves;
	if (!FLocalizedAssetUtil::GetAssetsByPathAndClass(AssetRegistry, FName("/Game"), UDialogueWave::StaticClass()->GetClassPathName(), /*bIncludeLocalizedAssets*/false, AssetDataArrayForDialogueWaves))
	{
		UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Unable to get dialogue wave asset data from asset registry.");
		return -1;
	}

	for (const FAssetData& AssetData : AssetDataArrayForDialogueWaves)
	{
		// Verify that the found asset is a dialogue wave.
		if (AssetData.GetClass() != UDialogueWave::StaticClass())
		{
			UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found asset (%ls), but the asset with this name is not actually a dialogue wave.", *AssetData.AssetName.ToString());
			continue;
		}

		// Get the dialogue wave.
		UDialogueWave* const DialogueWave = Cast<UDialogueWave>(AssetData.GetAsset());

		// Verify that the dialogue wave was loaded.
		if (!DialogueWave)
		{
			UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found asset (%ls), but the dialogue wave could not be accessed.", *AssetData.AssetName.ToString());
			continue;
		}

		// Iterate over each of the contexts and fix up the sound cue nodes referencing this sound wave.
		for (const FDialogueContextMapping& ContextMapping : DialogueWave->ContextMappings)
		{
			// Skip contexts without sound waves.
			const USoundWave* const SoundWave = ContextMapping.SoundWave;
			if (!SoundWave)
			{
				continue;
			}

			// Verify that the sound wave has a package.
			const UPackage* const SoundWavePackage = SoundWave->GetOutermost();
			if (!SoundWavePackage)
			{
				UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found dialogue wave (%ls) with a context referencing sound wave (%ls) but no package exists for this sound wave.", *AssetData.AssetName.ToString(), *SoundWave->GetName());
				continue;
			}
			
			// Find referencers of the context's sound wave.
			TArray<FName> SoundWaveReferencerNames;
			if (!AssetRegistry.GetReferencers(SoundWavePackage->GetFName(), SoundWaveReferencerNames))
			{
				UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found dialogue wave (%ls) with a context referencing sound wave (%ls) but failed to search for referencers of the sound wave.", *AssetData.AssetName.ToString(), *SoundWave->GetName());
				continue;
			}

			// Skip further searching if there are no sound wave referencers.
			if (SoundWaveReferencerNames.Num() == 0)
			{
				continue;
			}

			// Get sound cue assets that reference the context's sound wave.
			FARFilter Filter;
			Filter.ClassPaths.Add(USoundCue::StaticClass()->GetClassPathName());
			Filter.bRecursiveClasses = true;
			Filter.PackageNames = SoundWaveReferencerNames;
			TArray<FAssetData> ReferencingSoundCueAssetDataArray;
			if (!AssetRegistry.GetAssets(Filter, ReferencingSoundCueAssetDataArray))
			{
				UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found dialogue wave (%ls) with a context referencing sound wave (%ls) but failed to search for sound cues referencing the sound wave.", *AssetData.AssetName.ToString(), *SoundWave->GetName());
				continue;
			}

			// Iterate through referencing sound cues, finding sound node wave players and replacing them
			for (const FAssetData& SoundCueAssetData : ReferencingSoundCueAssetDataArray)
			{
				USoundCue* const SoundCue = Cast<USoundCue>(SoundCueAssetData.GetAsset());

				// Verify that the sound cue exists.
				if (!SoundCue)
				{
					UE_LOGF(LogSwapSoundForDialogueInCuesCommandlet, Error, "Asset registry found dialogue wave (%ls) with a context referencing sound wave (%ls) but failed to access the referencing sound cue (%ls).", *AssetData.AssetName.ToString(), *SoundWave->GetName(), *SoundCueAssetData.AssetName.ToString());
					continue;
				}

				// Iterate through sound nodes in this sound cue and find those that need replacing.
				TArray<USoundNode*> NodesToReplace;
				for (USoundNode* const SoundNode : SoundCue->AllNodes)
				{
					USoundNodeWavePlayer* const SoundNodeWavePlayer = Cast<USoundNodeWavePlayer>(SoundNode);
					
					// Skip nodes that are not sound wave players or not referencing the sound wave in question.
					if (SoundNodeWavePlayer && SoundNodeWavePlayer->GetSoundWave() == SoundWave)
					{
						NodesToReplace.Add(SoundNode);
					}
				}
				if (NodesToReplace.Num() == 0)
				{
					continue;
				}

				IAudioEditorModule* AudioEditorModule = &FModuleManager::LoadModuleChecked<IAudioEditorModule>("AudioEditor");
				AudioEditorModule->ReplaceSoundNodesInGraph(SoundCue, DialogueWave, NodesToReplace, ContextMapping);

				// Execute save.
				if (!FLocalizedAssetSCCUtil::SaveAssetWithSCC(SourceControlInfo, SoundCue))
				{
					continue;
				}
			}
		}
	}

	return 0;
}
