// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextAssetCommandlet.cpp: Commandlet for batch conversion and testing of
	text asset formats
=============================================================================*/

#include "Commandlets/TextAssetCommandlet.h"
#include "PackageHelperFunctions.h"
#include "Engine/Texture.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Linker.h"
#include "UObject/UObjectIterator.h"
#include "Stats/StatsMisc.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextAssetCommandlet)

DEFINE_LOG_CATEGORY(LogTextAsset);

UTextAssetCommandlet::UTextAssetCommandlet( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
}

bool HashFile(const TCHAR* InFilename, FSHAHash& OutHash)
{
	TArray<uint8> Bytes;
	if (FFileHelper::LoadFileToArray(Bytes, InFilename))
	{
		FSHA1::HashBuffer(&Bytes[0], Bytes.Num(), OutHash.Hash);
	}
	return false;
}

void FindMismatchedSerializers()
{
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if (!It->HasAnyClassFlags(CLASS_MatchedSerializers))
		{
			UE_LOGF(LogTextAsset, Display, "Class Mismatched Serializers: %ls", *It->GetName());
		}
	}
}

namespace 
{
	static const FString BackupExtension = TEXT("textassetbackup");
	static const FString BackupRoundtripExtension = TEXT("textassetbackup_roundtrip");
	static const FString BackupExtension_WithDot = TEXT(".") + BackupExtension;
	static const FString BackupRoundtripExtension_WithDot = TEXT(".") + BackupRoundtripExtension;
}

static void RepairDamagedFiles()
{
	// Repair any damage caused by a failed run of this commandlet
	struct FVisitor : public IPlatformFile::FDirectoryVisitor
	{
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FString Extension = FPaths::GetExtension(FilenameOrDirectory);
			if (!bIsDirectory && (Extension == BackupExtension))
			{
				UE_LOGF(LogTextAsset, Display, "Cleaning up old intermediate file %ls", FilenameOrDirectory);

				FString BinaryFilename = FPaths::GetPath(FilenameOrDirectory) / FPaths::GetBaseFilename(FilenameOrDirectory);
				FString TextFilename = FPaths::ChangeExtension(BinaryFilename, FPackageName::GetTextAssetPackageExtension());
				FString RoundtripBackup = BinaryFilename + BackupRoundtripExtension_WithDot;

				IFileManager::Get().Delete(*BinaryFilename);
				IFileManager::Get().Delete(*TextFilename);
				IFileManager::Get().Delete(*RoundtripBackup);

				IFileManager::Get().Move(*BinaryFilename, FilenameOrDirectory);
			}
			return true;
		}
	} RepairVisitor;

	IFileManager::Get().IterateDirectoryRecursively(*FPaths::ProjectContentDir(), RepairVisitor);
	IFileManager::Get().IterateDirectoryRecursively(*FPaths::EngineContentDir(), RepairVisitor);
}

typedef TFunction<void(FStructuredArchiveRecord, TArray<FString>&)> FSimpleSchemaFieldPropertyGenerator;

namespace StringConstants
{
	static FString Object(TEXT("object"));
	static FString String(TEXT("string"));
	static FString Number(TEXT("number"));
	static FString Array(TEXT("array"));
	static FString Boolean(TEXT("boolean"));
	static FString Properties(TEXT("properties"));
	static FString Type(TEXT("type"));
}

inline void WriteSimpleSchemaField(FStructuredArchiveRecord Record, const TCHAR* FieldName, FString& Type, FSimpleSchemaFieldPropertyGenerator PropertiesCallback = FSimpleSchemaFieldPropertyGenerator())
{
	FStructuredArchiveRecord FieldRecord = Record.EnterField(FieldName).EnterRecord();
	FieldRecord << SA_VALUE(TEXT("type"), Type);

	if (PropertiesCallback)
	{
		FStructuredArchiveRecord PropertiesRecord = FieldRecord.EnterField(TEXT("properties")).EnterRecord();
		TArray<FString> Required;
		PropertiesCallback(PropertiesRecord, Required);
		if (Required.Num() > 0)
		{
			int32 NumRequired = Required.Num();
			FStructuredArchiveArray RequiredArray = FieldRecord.EnterField(TEXT("required")).EnterArray(NumRequired);
			for (FString& RequiredProperty : Required)
			{
				RequiredArray.EnterElement() << RequiredProperty;
			}
		}
	}
}

TSet<FName> GMissingThings;

void GeneratePropertySchema(FProperty* Property, FStructuredArchiveRecord Record, TArray<FString>& Required)
{
	static const FName NAME_ClassProperty(TEXT("ClassProperty"));

	const FFieldClass* PropertyClass = Property->GetClass();
	const FName PropertyClassName = PropertyClass->GetFName();

	if (PropertyClassName == NAME_ArrayProperty)
	{
		WriteSimpleSchemaField(Record, TEXT("__Type"), StringConstants::String);
		WriteSimpleSchemaField(Record, TEXT("__InnerType"), StringConstants::String);
		WriteSimpleSchemaField(Record, TEXT("__InnerStructName"), StringConstants::String);
		WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Array);

		FStructuredArchiveRecord ItemsRecord = Record.EnterRecord(TEXT("items"));
	}
	else
	{
		WriteSimpleSchemaField(Record, TEXT("__Type"), StringConstants::String);
		
		// We need to describe the data that this property writes out, which only the derived property class knows. We'll have to add something to the FProperty API to do that
		// but for now I'm just going to hardcode things here
		if (PropertyClassName == NAME_StrProperty
 		||  PropertyClassName == NAME_ObjectProperty
		||  PropertyClassName == NAME_NameProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::String);
		}
		else if (PropertyClassName == NAME_EnumProperty || (PropertyClassName == NAME_ByteProperty && ((const FByteProperty*)(Property))->Enum != nullptr))
		{
			WriteSimpleSchemaField(Record, TEXT("__EnumName"), StringConstants::String);
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::String);
		}
		else if (PropertyClass->IsChildOf(FNumericProperty::StaticClass()))
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Number);
		}
		else if (PropertyClassName == NAME_StructProperty || PropertyClassName == NAME_ClassProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__StructName"), StringConstants::String);
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else if (PropertyClassName == NAME_TextProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else if (PropertyClassName == NAME_BoolProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Boolean);
		}
		else if (PropertyClassName == NAME_SetProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__InnerType"), StringConstants::String);
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else if (PropertyClassName == NAME_InterfaceProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else if (PropertyClassName == NAME_WeakObjectProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else if (PropertyClassName == NAME_MapProperty)
		{
			WriteSimpleSchemaField(Record, TEXT("__InnerType"), StringConstants::String);
			WriteSimpleSchemaField(Record, TEXT("__ValueType"), StringConstants::String);
			WriteSimpleSchemaField(Record, TEXT("__Value"), StringConstants::Object);
		}
		else
		{
			if (!GMissingThings.Contains(PropertyClassName))
			{
				UE_LOGF(LogTextAsset, Warning, "Unhandled property type: %ls", *PropertyClassName.ToString());
				GMissingThings.Add(PropertyClassName);
			}
		}
	}
}

void GenerateClassSchema(UClass* Class, FStructuredArchiveRecord Record, TArray<FString>& Required)
{
	FProperty* CurProperty = Class->PropertyLink;
	while (CurProperty != nullptr)
	{
		WriteSimpleSchemaField(Record, *CurProperty->GetName(), StringConstants::Object, [CurProperty](FStructuredArchiveRecord InRecord, TArray<FString>& InRequired) { GeneratePropertySchema(CurProperty, InRecord, InRequired); });
		CurProperty = CurProperty->PropertyLinkNext;
	}
}

#if WITH_TEXT_ARCHIVE_SUPPORT
void GenerateSchema()
{
	//static const FName NAME_SpecificClass(TEXT("TextAssetTestObject"));
	static const FName NAME_SpecificClass(NAME_None);

	FString OutputFilename;
	if (!FParse::Value(FCommandLine::Get(), TEXT("-schemaoutput="), OutputFilename))
	{
		OutputFilename = FPaths::ProjectConfigDir() / TEXT("Schemas/TextAssetExports.json");
	}

	TUniquePtr<FArchive> OutputAr(IFileManager::Get().CreateFileWriter(*OutputFilename));
	FJsonArchiveOutputFormatter JsonFormatter(*OutputAr.Get());
	FStructuredArchive StructuredArchive(JsonFormatter);
	FStructuredArchiveRecord RootRecord = StructuredArchive.Open().EnterRecord();

	for (FThreadSafeObjectIterator It(UClass::StaticClass()); It; ++It)
	{
		UClass* Class = Cast<UClass>(*It);
		if (Class)
		{
			if (NAME_SpecificClass == NAME_None || Class->GetFName() == NAME_SpecificClass)
			{
				FStructuredArchiveRecord ClassRecord = RootRecord.EnterRecord(*Class->GetFullName());
				ClassRecord << SA_VALUE(*StringConstants::Type, StringConstants::Object);

				WriteSimpleSchemaField(ClassRecord, TEXT("__Class"), StringConstants::Object);
				WriteSimpleSchemaField(ClassRecord, TEXT("__Outer"), StringConstants::String);
				WriteSimpleSchemaField(ClassRecord, TEXT("__bNotAlwaysLoadedForEditorGame"), StringConstants::Boolean);
				WriteSimpleSchemaField(ClassRecord, TEXT("__Value"), StringConstants::Object, [Class](FStructuredArchiveRecord Record, TArray<FString>& Required) { GenerateClassSchema(Class, Record, Required); });
			}
		}
	}

	StructuredArchive.Close();
}
#endif

bool UTextAssetCommandlet::DoTextAssetProcessing(const FString& InCommandLine)
{
	FProcessingArgs Args;

	FString ModeString = TEXT("ResaveText");
	FString IterationsString = TEXT("1");

	FString Filename, FilenameFilter;

	FParse::Value(*InCommandLine, TEXT("mode="), ModeString);
	FParse::Value(*InCommandLine, TEXT("filename="), Filename);
	FParse::Value(*InCommandLine, TEXT("filter="), FilenameFilter);
	FParse::Value(*InCommandLine, TEXT("csv="), Args.CSVFilename);
	FParse::Value(*InCommandLine, TEXT("outputpath="), Args.OutputPath);

	if (Filename.Len() > 0 && FilenameFilter.Len() > 0)
	{
		UE_LOGF(LogTextAsset, Error, "Cannot specify a filename and a filter at the same time when processing text assets");
		return false;
	}

	if (Filename.Len() > 0)
	{
		Args.Filename = Filename;
		Args.bFilenameIsFilter = false;
	}
	else if (FilenameFilter.Len() > 0)
	{
		Args.Filename = FilenameFilter;
		Args.bFilenameIsFilter = true;
	}
	else
	{
		Args.bFilenameIsFilter = true;	// do everything
	}

	Args.bVerifyJson = !FParse::Param(*InCommandLine, TEXT("noverifyjson"));
	Args.ProcessingMode = (ETextAssetCommandletMode)StaticEnum<ETextAssetCommandletMode>()->GetValueByNameString(ModeString);

	FParse::Value(*InCommandLine, TEXT("iterations="), Args.NumSaveIterations);

	Args.bIncludeEngineContent = FParse::Param(*InCommandLine, TEXT("includeenginecontent"));

	return DoTextAssetProcessing(Args);
}

bool UTextAssetCommandlet::DoTextAssetProcessing(const FProcessingArgs& InArgs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::Main);

	RepairDamagedFiles();

	switch (InArgs.ProcessingMode)
	{
	case ETextAssetCommandletMode::FindMismatchedSerializers:
		FindMismatchedSerializers();
		return true;

	case ETextAssetCommandletMode::GenerateSchema:
#if WITH_TEXT_ARCHIVE_SUPPORT
		GenerateSchema();
#else 
		UE_LOGF(LogTextAsset, Error, "Unable to generate schema when compiled with WITH_TEXT_ARCHIVE_SUPPORT=0");
#endif
		break;

	default:
		break;
	}

	TArray<UObject*> Objects;		
	TArray<FString> InputAssetFilenames;

	FString ProjectContentDir = *FPaths::ProjectContentDir();
	FString EngineContentDir = *FPaths::EngineContentDir();
	const FString Wildcard = TEXT("*");

	switch (InArgs.ProcessingMode)
	{
	case ETextAssetCommandletMode::ResaveBinary:
	case ETextAssetCommandletMode::ResaveText:
	case ETextAssetCommandletMode::RoundTrip:
	{
		IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *ProjectContentDir, *(Wildcard + FPackageName::GetAssetPackageExtension()), true, false, true);
		IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *ProjectContentDir, *(Wildcard + FPackageName::GetMapPackageExtension()), true, false, false);

		if (InArgs.bIncludeEngineContent)
		{
			IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *EngineContentDir, *(Wildcard + FPackageName::GetAssetPackageExtension()), true, false, false);
			IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *EngineContentDir, *(Wildcard + FPackageName::GetMapPackageExtension()), true, false, false);
		}

		break;
	}

	case ETextAssetCommandletMode::LoadText:
	{
		IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *ProjectContentDir, *(Wildcard + FPackageName::GetTextAssetPackageExtension()), true, false, true);
		//IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *BasePath, *(Wildcard + FPackageName::GetTextMapPackageExtension()), true, false, false);
		break;
	}

	case ETextAssetCommandletMode::LoadBinary:
	{
		IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *ProjectContentDir, *(Wildcard + FPackageName::GetAssetPackageExtension()), true, false, true);
		//IFileManager::Get().FindFilesRecursive(InputAssetFilenames, *BasePath, *(Wildcard + FPackageName::GetTextMapPackageExtension()), true, false, false);
		break;
	}
	}

	FString FilenameFilter = InArgs.Filename;
	if (!InArgs.bFilenameIsFilter)
	{
		FString PotentialFilenames[] = {
			InArgs.Filename + FPackageName::GetAssetPackageExtension(),
			InArgs.Filename + FPackageName::GetMapPackageExtension(),
			InArgs.Filename + FPackageName::GetTextAssetPackageExtension(),
			InArgs.Filename + FPackageName::GetTextMapPackageExtension()
		};

		for (const FString& Filename : PotentialFilenames)
		{
			if (FPaths::FileExists(Filename))
			{
				FilenameFilter = Filename;
				break;
			}
		}
	}

	TArray<TTuple<FString, FString>> FilesToProcess;

	for (const FString& InputAssetFilename : InputAssetFilenames)
	{
		bool bIgnore = false;

		if (FilenameFilter.Len() > 0 && !InputAssetFilename.Contains(FilenameFilter))
		{
			bIgnore = true;
		}

		bIgnore = bIgnore || (InputAssetFilename.Contains(TEXT("_BuiltData")));

		if (bIgnore)
		{
			continue;
		}

		bool bShouldProcess = true;
		FString DestinationFilename = InputAssetFilename;

		switch (InArgs.ProcessingMode)
		{
		case ETextAssetCommandletMode::ResaveBinary:
		{
			DestinationFilename = InputAssetFilename + TEXT(".tmp");
			break;
		}

		case ETextAssetCommandletMode::ResaveText:
		{
			if (InputAssetFilename.EndsWith(FPackageName::GetAssetPackageExtension())) DestinationFilename = FPaths::ChangeExtension(InputAssetFilename, FPackageName::GetTextAssetPackageExtension());;
			if (InputAssetFilename.EndsWith(FPackageName::GetMapPackageExtension())) DestinationFilename = FPaths::ChangeExtension(InputAssetFilename, FPackageName::GetTextMapPackageExtension());;
			
			break;
		}

		case ETextAssetCommandletMode::LoadText:
		case ETextAssetCommandletMode::LoadBinary:
		{
			break;
		}
		}

		if (bShouldProcess)
		{
			FilesToProcess.Add(TTuple<FString, FString>(InputAssetFilename, DestinationFilename));
		}
	}

	const FString TempFailedDiffsPath = FPaths::ProjectSavedDir() / TEXT(".roundtrip");
	const FString FailedDiffsPath = FPaths::ProjectSavedDir() / TEXT("FailedDiffs");
	IFileManager::Get().DeleteDirectory(*FailedDiffsPath, false, true);

	double TotalPackageLoadTime = 0.0;
	double TotalPackageSaveTime = 0.0;

	FArchive* CSVWriter = nullptr;
	if (InArgs.CSVFilename.Len() > 0)
	{
		CSVWriter = IFileManager::Get().CreateFileWriter(*InArgs.CSVFilename);
		if (CSVWriter != nullptr)
		{
			FString CSVLine = FString::Printf(TEXT("Total Time,Num Files,AvgFileTime,MinFileTime,MaxFileTime,TotalLoadTime\n"));
			CSVWriter->Serialize(TCHAR_TO_ANSI(*CSVLine), CSVLine.Len());
		}
	}

	for (int32 Iteration = 0; Iteration < InArgs.NumSaveIterations; ++Iteration)
	{
		if (InArgs.NumSaveIterations > 1)
		{
			UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------");
			UE_LOGF(LogTextAsset, Display, "Iteration %i/%i", Iteration + 1, InArgs.NumSaveIterations);
		}

		double MaxTime = FLT_MIN;
		double MinTime = FLT_MAX;
		double TotalTime = 0;
		int64 NumFiles = 0;
		FString MaxTimePackage;
		FString MinTimePackage;
		double IterationPackageLoadTime = 0.0;
		double IterationPackageSaveTime = 0.0;
		double ThisPackageLoadTime = 0.0;

		TArray<FString> PhaseSuccess;
		TArray<TArray<FString>> PhaseFails;
		PhaseFails.AddDefaulted(3);

		for (const TTuple<FString, FString>& FileToProcess : FilesToProcess)
		{
			FString SourceFilename = FileToProcess.Get<0>();
			FString SourceLongPackageName = FPackageName::FilenameToLongPackageName(SourceFilename);
			FString DestinationFilename = FileToProcess.Get<1>();

			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*SourceFilename);

			double StartTime = FPlatformTime::Seconds();

			switch (InArgs.ProcessingMode)
			{
			case ETextAssetCommandletMode::RoundTrip:
			{
				UE_LOGF(LogTextAsset, Display, "Starting roundtrip test for '%ls' [%lld/%d]", *SourceLongPackageName, NumFiles + 1, FilesToProcess.Num());
				UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------------------------------------------");

				const FString WorkingFilenames[2] = { SourceFilename, FPaths::ChangeExtension(SourceFilename, FPackageName::GetTextAssetPackageExtension()) };
				

				IFileManager::Get().Delete(*WorkingFilenames[1], false, false, true);

				FString SourceBackupFilename = SourceFilename + BackupExtension_WithDot;
				if (IFileManager::Get().FileExists(*SourceBackupFilename))
				{
					IFileManager::Get().Delete(*SourceFilename, false, false, true);
					IFileManager::Get().Move(*SourceFilename, *SourceBackupFilename, true);
				}
				IFileManager::Get().Copy(*SourceBackupFilename, *SourceFilename, true);
				
				// Firstly, do a resave of the package
				UPackage* OriginalPackage = LoadPackage(nullptr, *SourceLongPackageName, LOAD_None);
				IFileManager::Get().Delete(*SourceFilename, false, true, true);
				SavePackageHelper(OriginalPackage, SourceFilename, RF_Standalone, GWarn, SAVE_None);
				CollectGarbage(RF_NoFlags, true);

				// Make a copy of the resaved source package which we can use as the base revision for each test
				FString BaseBinaryPackageBackup = SourceFilename + BackupRoundtripExtension_WithDot;
				IFileManager::Get().Copy(*BaseBinaryPackageBackup, *SourceFilename, true);

				FSHAHash SourceHash;
				HashFile(*SourceBackupFilename, SourceHash);
				
				static const int32 NumPhases = 3;
				static const int32 NumTests = 3;

				static const TCHAR* PhaseNames[] = { TEXT("Binary Only"), TEXT("Text Only"), TEXT("Alternating Binary/Text") };

				#if CPUPROFILERTRACE_ENABLED
				static const TCHAR* PhaseEventTypes[3] = {
						TEXT("BinaryOnly"),
						TEXT("TextOnly"),
						TEXT("Alternating"),
				};

				static const TCHAR* TestEventTypes[6] = {
						TEXT("Test1"),
						TEXT("Test2"),
						TEXT("Test3"),
						TEXT("Test4"),
						TEXT("Test5"),
						TEXT("Test6"),
				};
				#endif // CPUPROFILERTRACE_ENABLED

				TArray<TArray<FSHAHash>> Hashes;

				CollectGarbage(RF_NoFlags, true);

				bool bPhasesMatched[NumPhases] = { true, true, true };
				TArray<TPair<FString,FString>> DiffFilenames;

				for (int32 Phase = 0; Phase < NumPhases; ++Phase)
				{
					IFileManager::Get().Delete(*SourceFilename, false, false, true);
					IFileManager::Get().Copy(*SourceFilename, *BaseBinaryPackageBackup, true);

					TArray<FSHAHash> PhaseHashes = Hashes[Hashes.AddDefaulted()];
					
					#if CPUPROFILERTRACE_ENABLED
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(PhaseEventTypes[Phase]);
					#endif

					for (int32 i = 0; i < ((Phase == 2) ? NumTests * 2 : NumTests); ++i)
					{
						int32 Bucket;

						#if CPUPROFILERTRACE_ENABLED
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TestEventTypes[i]);
						#endif

						switch (Phase)
						{
						case 0: // binary only
						{
							Bucket = 0;
							break;
						}

						case 1: // text only
						{
							Bucket = 1;

							if (i > 0)
							{
								IFileManager::Get().Delete(*WorkingFilenames[0]);
							}

							break;
						}

						case 2: // alternate
						{
							Bucket = i % 2;

							if ((i > 0) && Bucket == 0)
							{
								// We're doing alternating text/binary saves, so we need to delete the text version as we have no way of forcing the load to choose between text and binary
								IFileManager::Get().Delete(*WorkingFilenames[0]);
							}

							break;
						}

						default:
						{
							checkNoEntry();
							Bucket = 0;
						}
						};

						UPackage* Package = nullptr;
						
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(LoadPackage);
							Package = LoadPackage(nullptr, *SourceLongPackageName, LOAD_None);
						}
						
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(SavePackage); 
							SavePackageHelper(Package, *WorkingFilenames[Bucket], RF_Standalone, GWarn, SAVE_None);
						}
						
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(ResetLoaders); 
							ResetLoaders(Package);
						}

						{
							TRACE_CPUPROFILER_EVENT_SCOPE(CollectGarbage);
							CollectGarbage(RF_NoFlags, true);
						}

						{
							TRACE_CPUPROFILER_EVENT_SCOPE(RoundtripTestCleanup);
							FSHAHash& Hash = PhaseHashes[PhaseHashes.AddDefaulted()];
							HashFile(*WorkingFilenames[Bucket], Hash);

							FString TargetPath = WorkingFilenames[Bucket];
							FPaths::MakePathRelativeTo(TargetPath, *FPaths::ProjectContentDir());
							FString IntermediateTargetPath = TempFailedDiffsPath / TargetPath;
							FString FinalTargetPath = FailedDiffsPath / TargetPath;

							FString IntermediateFilename = FString::Printf(TEXT("%s_Phase%i_%03i%s"), *FPaths::ChangeExtension(IntermediateTargetPath, TEXT("")), Phase, i + 1, *FPaths::GetExtension(WorkingFilenames[Bucket], true));
							FString FinalFilename = FString::Printf(TEXT("%s_Phase%i_%03i%s"), *FPaths::ChangeExtension(FinalTargetPath, TEXT("")), Phase, i + 1, *FPaths::GetExtension(WorkingFilenames[Bucket], true));
							IFileManager::Get().Copy(*IntermediateFilename, *WorkingFilenames[Bucket]);

							DiffFilenames.Add(TPair<FString, FString>(IntermediateFilename, FinalFilename));
						}
					}

					UE_LOGF(LogTextAsset, Display, "Phase %i (%ls) Results", Phase + 1, PhaseNames[Phase]);
					int32 Pass = 1;
					FSHAHash Refs[2] = { PhaseHashes[0], PhaseHashes[1] };
					bool bTotalSuccess = true;
					for (const FSHAHash& Hash : PhaseHashes)
					{
						if (Phase == 2)
						{
							bPhasesMatched[Phase] = bPhasesMatched[Phase] && Hash == Refs[(Pass + 1) % 2];
						}
						else
						{
							bPhasesMatched[Phase] = bPhasesMatched[Phase] && Hash == Refs[0];
						}

						UE_LOGF(LogTextAsset, Display, "\tPass %i [%ls] %ls", Pass, *Hash.ToString(), bPhasesMatched[Phase] ? TEXT("OK") : TEXT("FAILED"));
						Pass++;
					}

					if (!bPhasesMatched[Phase])
					{
						UE_LOGF(LogTextAsset, Display, "\tPhase %i (%ls) failed for asset '%ls'", Phase + 1, PhaseNames[Phase], *SourceLongPackageName);
						bTotalSuccess = false;
					}

					if (Phase == 1)
					{
						IFileManager::Get().Delete(*WorkingFilenames[1], false, false, true);
					}

					if (!bTotalSuccess)
					{
						for (const TPair<FString, FString>& DiffPair : DiffFilenames)
						{
							IFileManager::Get().MakeDirectory(*FPaths::GetPath(DiffPair.Value));
							IFileManager::Get().Move(*DiffPair.Value, *DiffPair.Key);
						}
					}

					DiffFilenames.Empty();
					IFileManager::Get().DeleteDirectory(*TempFailedDiffsPath, false, true);
				}

				static const bool bDisableCleanup = FParse::Param(FCommandLine::Get(), TEXT("disablecleanup"));
				CollectGarbage(RF_NoFlags, true);
				IFileManager::Get().Delete(*WorkingFilenames[1], false, true, true);
				IFileManager::Get().Delete(*BaseBinaryPackageBackup, false, true, true);
				IFileManager::Get().Delete(*SourceFilename, false, true, true);
				IFileManager::Get().Move(*SourceFilename, *SourceBackupFilename);

				if (!bPhasesMatched[0])
				{
					UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------------------------------------------");
					UE_LOGF(LogTextAsset, Warning, "Binary determinism tests failed, so we can't determine meaningful results for '%ls'", *SourceLongPackageName);
				}
				else if (!bPhasesMatched[1] || !bPhasesMatched[2])
				{
					UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------------------------------------------");
					UE_LOGF(LogTextAsset, Error, "Binary determinism tests succeeded, but text and/or alternating tests failed for asset '%ls'", *SourceLongPackageName);
				}

					bool bSuccess = true;
				for (int32 PhaseIndex = 0; PhaseIndex < NumPhases; ++PhaseIndex)
				{
					if (!bPhasesMatched[PhaseIndex])
					{
						bSuccess = false;
						PhaseFails[PhaseIndex].Add(SourceLongPackageName);
					}
				}

				if (bSuccess)
				{
					PhaseSuccess.Add(SourceLongPackageName);
				}

				UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------------------------------------------");
				UE_LOGF(LogTextAsset, Display, "Completed roundtrip test for '%ls'", *SourceLongPackageName);
				UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------------------------------------------");
				
				break;
			}

			case ETextAssetCommandletMode::ResaveBinary:
			case ETextAssetCommandletMode::ResaveText:
			{
				UPackage* Package = nullptr;

				UE_LOGF(LogTextAsset, Display, "Resaving asset %ls", *SourceFilename);
				TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::Resave);

				double Timer = 0.0;
				{
					SCOPE_SECONDS_COUNTER(Timer);
					TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::LoadPackage);
					Package = LoadPackage(nullptr, *SourceFilename, 0);
				}
				IterationPackageLoadTime += Timer;
				TotalPackageLoadTime += Timer;

				bool bSaveSuccessful = false;

				if (Package)
				{
					{
						SCOPE_SECONDS_COUNTER(Timer);
						TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::SavePackage);
						IFileManager::Get().Delete(*DestinationFilename, false, true, true);
						bSaveSuccessful = SavePackageHelper(Package, *DestinationFilename, RF_Standalone, GWarn, SAVE_None);
					}
					TotalPackageSaveTime += Timer;
					IterationPackageSaveTime += Timer;
				}

				if (bSaveSuccessful)
				{
					if (InArgs.bVerifyJson && InArgs.ProcessingMode == ETextAssetCommandletMode::ResaveText)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::VerifyJson);
						FArchive* File = IFileManager::Get().CreateFileReader(*DestinationFilename);
						TSharedPtr< FJsonObject > RootObject;
						TSharedRef< TJsonReader<UTF8CHAR> > Reader = TJsonReaderFactory<UTF8CHAR>::Create(File);
						ensure(FJsonSerializer::Deserialize(Reader, RootObject));
						delete File;
					}

					if (InArgs.OutputPath.Len() > 0)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UTextAssetCommandlet::CopyToExternalOutput);
						FString CopyFilename = DestinationFilename;
						FPaths::MakePathRelativeTo(CopyFilename, *FPaths::RootDir());
						CopyFilename = InArgs.OutputPath / CopyFilename;
						CopyFilename.RemoveFromEnd(TEXT(".tmp"));
						IFileManager::Get().MakeDirectory(*FPaths::GetPath(CopyFilename));
						IFileManager::Get().Move(*CopyFilename, *DestinationFilename);
					}
				}

				break;
			}

			case ETextAssetCommandletMode::LoadText:
			{
				UPackage* Package = nullptr;
				UE_LOGF(LogTextAsset, Display, "Loading Text Asset '%ls'", *SourceFilename);

				CollectGarbage(RF_NoFlags, true);
				ThisPackageLoadTime = 0.0;
				{
					SCOPE_SECONDS_COUNTER(ThisPackageLoadTime);
					Package = LoadPackage(nullptr, *SourceFilename, 0);
				}
				CollectGarbage(RF_NoFlags, true);
				IterationPackageLoadTime += ThisPackageLoadTime;
				TotalPackageLoadTime += ThisPackageLoadTime;

				Package = nullptr;

				break;
			}

			case ETextAssetCommandletMode::LoadBinary:
			{
				UPackage* Package = nullptr;
				UE_LOGF(LogTextAsset, Display, "Loading Binary Asset '%ls'", *SourceFilename);
				CollectGarbage(RF_NoFlags, true);

				ThisPackageLoadTime = 0.0;
				{
					SCOPE_SECONDS_COUNTER(ThisPackageLoadTime);
					Package = LoadPackage(nullptr, *SourceFilename, 0);
				}
				CollectGarbage(RF_NoFlags, true);
				IterationPackageLoadTime += ThisPackageLoadTime;
				TotalPackageLoadTime += ThisPackageLoadTime;

				Package = nullptr;

				break;
			}
			}

			double EndTime = FPlatformTime::Seconds();
			double Time = EndTime - StartTime;

			if (InArgs.ProcessingMode == ETextAssetCommandletMode::LoadBinary || InArgs.ProcessingMode == ETextAssetCommandletMode::LoadText)
			{
				if (ThisPackageLoadTime > MaxTime)
				{
					MaxTime = ThisPackageLoadTime;
					MaxTimePackage = SourceFilename;
				}

				if (ThisPackageLoadTime < MinTime)
				{
					MinTime = ThisPackageLoadTime;
					MinTimePackage = SourceFilename;
				}
			}
			else
			{
				if (Time > MaxTime)
				{
					MaxTime = Time;
					MaxTimePackage = SourceFilename;
				}

				if (Time < MinTime)
				{
					MinTime = Time;
					MinTimePackage = SourceFilename;
				}
			}

			TotalTime += Time;
			NumFiles++;
		}

		if (InArgs.ProcessingMode == ETextAssetCommandletMode::RoundTrip)
		{
			UE_LOGF(LogTextAsset, Display, "\t-----------------------------------------------------");
			UE_LOGF(LogTextAsset, Display, "\tRoundTrip Results");
			UE_LOGF(LogTextAsset, Display, "\tTotal Packages: %i", FilesToProcess.Num());
			UE_LOGF(LogTextAsset, Display, "\tNum Successful Packages: %i", PhaseSuccess.Num());
			UE_LOGF(LogTextAsset, Display, "\tPhase 0 Fails: %i (Binary Package Determinism Fails)", PhaseFails[0].Num());
			UE_LOGF(LogTextAsset, Display, "\tPhase 1 Fails: %i (Text Package Determinism Fails)", PhaseFails[1].Num());
			UE_LOGF(LogTextAsset, Display, "\tPhase 2 Fails: %i (Mixed Package Determinism Fails)", PhaseFails[2].Num());
			UE_LOGF(LogTextAsset, Display, "\t-----------------------------------------------------");

			for (int32 PhaseIndex = 1; PhaseIndex < PhaseFails.Num(); ++PhaseIndex)
			{
				if (PhaseFails[PhaseIndex].Num() > 0)
				{
					UE_LOGF(LogTextAsset, Display, "\tPhase %i Fails:", PhaseIndex);
					for (const FString& PhaseFail : PhaseFails[PhaseIndex])
					{
						if (!PhaseFails[0].Contains(PhaseFail))
						{
							UE_LOGF(LogTextAsset, Display, "\t\t%ls", *PhaseFail);
						}
					}
					UE_LOGF(LogTextAsset, Display, "\t-----------------------------------------------------");
				}
			}
		}

		double AvgFileTime, MinFileTime, MaxFileTime;

		if (InArgs.ProcessingMode == ETextAssetCommandletMode::LoadBinary || InArgs.ProcessingMode == ETextAssetCommandletMode::LoadText)
		{
			AvgFileTime = IterationPackageLoadTime;
		}
		else
		{
			AvgFileTime = TotalTime;
		}

		AvgFileTime /= (double)NumFiles;
		MinFileTime = MinTime;
		MaxFileTime = MaxTime;

		UE_LOGF(LogTextAsset, Display, "\tTotal Time:\t%.2fs", TotalTime);
		UE_LOGF(LogTextAsset, Display, "\tTotal Files:\t%lli", NumFiles);
		UE_LOGF(LogTextAsset, Display, "\tAvg File Time:  \t%.2fms", AvgFileTime * 1000.0);
		UE_LOGF(LogTextAsset, Display, "\tMin File Time:  \t%.2fms (%ls)", MinFileTime * 1000.0, *MinTimePackage);
		UE_LOGF(LogTextAsset, Display, "\tMax File Time:  \t%.2fms (%ls)", MaxFileTime * 1000.0, *MaxTimePackage);
		UE_LOGF(LogTextAsset, Display, "\tTotal Package Load Time:  \t%.2fs", IterationPackageLoadTime);

		if (CSVWriter != nullptr)
		{
			FString CSVLine = FString::Printf(TEXT("%f,%" INT64_FMT ",%f,%f,%f,%f\n"), TotalTime, NumFiles, AvgFileTime, MinFileTime, MaxFileTime, IterationPackageLoadTime);
			CSVWriter->Serialize(TCHAR_TO_ANSI(*CSVLine), CSVLine.Len());
		}

		if (InArgs.ProcessingMode != ETextAssetCommandletMode::LoadText && InArgs.ProcessingMode != ETextAssetCommandletMode::ResaveText)
		{
			UE_LOGF(LogTextAsset, Display, "\tTotal Package Save Time:  \t%.2fs", IterationPackageSaveTime);
		}

		CollectGarbage(RF_NoFlags, true);
	}

	UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------");
	UE_LOGF(LogTextAsset, Display, "Text Asset Commandlet Completed!");
	UE_LOGF(LogTextAsset, Display, "\tTotal Files Processed:  \t%i", FilesToProcess.Num());
	UE_LOGF(LogTextAsset, Display, "\tAvg Iteration Package Load Time:  \t%.2fs", TotalPackageLoadTime / (float)InArgs.NumSaveIterations);

	if (CSVWriter != nullptr)
	{
		delete CSVWriter;
	}

	if (InArgs.ProcessingMode != ETextAssetCommandletMode::LoadText)
	{
		UE_LOGF(LogTextAsset, Display, "\tAvg Iteration Save Time:  \t%.2fs", TotalPackageSaveTime / (float)InArgs.NumSaveIterations);
	}
	
	UE_LOGF(LogTextAsset, Display, "-----------------------------------------------------");
	
	return true;
}

int32 UTextAssetCommandlet::Main(const FString& CmdLineParams)
{
	return DoTextAssetProcessing(CmdLineParams) ? 0 : 1;
}

static void TextAssetToolCVarCommand(const TArray<FString>& Args)
{
	FString JoinedArgs;
	for (const FString& Arg : Args) 
	{
		JoinedArgs += Arg;
		JoinedArgs += TEXT(" ");
	}
	UTextAssetCommandlet::DoTextAssetProcessing(JoinedArgs);
}

static FAutoConsoleCommand CVar_TextAssetTool(
	TEXT("TextAssetTool"),
	TEXT("--"),
	FConsoleCommandWithArgsDelegate::CreateStatic(TextAssetToolCVarCommand));
