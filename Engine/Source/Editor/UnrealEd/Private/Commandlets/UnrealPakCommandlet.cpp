// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/UnrealPakCommandlet.h"
#include "PakFileUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnrealPakCommandlet)

UUnrealPakCommandlet::UUnrealPakCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UUnrealPakCommandlet::Main(const FString& Params)
{
	return ExecuteUnrealPak(*Params)? 0 : 1;
}
