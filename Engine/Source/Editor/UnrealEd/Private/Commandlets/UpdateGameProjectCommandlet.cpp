// Copyright Epic Games, Inc. All Rights Reserved.


#include "Commandlets/UpdateGameProjectCommandlet.h"

#include "Containers/Array.h"
#include "GameProjectGenerationModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Interfaces/IProjectManager.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineVersionBase.h"
#include "Misc/Paths.h"
#include "SourceControlOperations.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UpdateGameProjectCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogUpdateGameProjectCommandlet, Log, All);

UUpdateGameProjectCommandlet::UUpdateGameProjectCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
}

int32 UUpdateGameProjectCommandlet::Main( const FString& InParams )
{
	// Parse command line.
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*InParams, Tokens, Switches);

	FString Category;
	FText ChangelistDescription = NSLOCTEXT("UpdateGameProjectCmdlet", "ChangelistDescription", "Updated game project");
	bool bAutoCheckout = false;
	bool bAutoSubmit = false;
	bool bSignSampleProject = false;

	const FString CategorySwitch = TEXT("Category=");
	const FString ChangelistDescriptionSwitch = TEXT("ChangelistDescription=");
	for ( int32 SwitchIdx = 0; SwitchIdx < Switches.Num(); ++SwitchIdx )
	{
		const FString& Switch = Switches[SwitchIdx];
		if ( Switch == TEXT("AutoCheckout") )
		{
			bAutoCheckout = true;
		}
		else if ( Switch == TEXT("AutoSubmit") )
		{
			bAutoSubmit = true;
		}
		else if ( Switch == TEXT("SignSampleProject") )
		{
			bSignSampleProject = true;
		}
		else if ( Switch.StartsWith(CategorySwitch) )
		{
			Category = Switch.Mid( CategorySwitch.Len() );
		}
		else if ( Switch.StartsWith(ChangelistDescriptionSwitch) )
		{
			ChangelistDescription = FText::FromString( Switch.Mid( ChangelistDescriptionSwitch.Len() ) );
		}
	}

	if ( !FPaths::IsProjectFilePathSet() )
	{
		UE_LOGF(LogUpdateGameProjectCommandlet, Error, "You must launch with a project file to be able to update it");
		return 1;
	}

	const FString ProjectFilePath = FPaths::GetProjectFilePath();

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( bAutoCheckout )
	{
		SourceControlProvider.Init();
	}

	FString EngineIdentifier = FEngineVersion::Current().ToString(EVersionComponent::Minor);

	UE_LOGF(LogUpdateGameProjectCommandlet, Display, "Updating project file %ls to %ls...", *ProjectFilePath, *EngineIdentifier);

	FText FailReason;
	if ( !FGameProjectGenerationModule::Get().UpdateGameProject(ProjectFilePath, EngineIdentifier, FailReason) )
	{
		UE_LOGF(LogUpdateGameProjectCommandlet, Error, "Couldn't update game project: %ls", *FailReason.ToString());
		return 1;
	}

	if ( bSignSampleProject )
	{
		UE_LOGF(LogUpdateGameProjectCommandlet, Display, "Attempting to sign project file %ls...", *ProjectFilePath);

		FText LocalFailReason;
		if ( IProjectManager::Get().SignSampleProject(ProjectFilePath, Category, LocalFailReason) )
		{
			UE_LOGF(LogUpdateGameProjectCommandlet, Display, "Signed project file %ls saved.", *ProjectFilePath);
		}
		else
		{
			UE_LOGF(LogUpdateGameProjectCommandlet, Warning, "%ls", *LocalFailReason.ToString());
		}
	}

	if ( bAutoSubmit )
	{
		if ( !bAutoCheckout )
		{
			// We didn't init SCC above so do it now
			SourceControlProvider.Init();
		}

		if ( ISourceControlModule::Get().IsEnabled() )
		{
			const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(AbsoluteFilename, EStateCacheUsage::ForceUpdate);
			if ( SourceControlState.IsValid() && SourceControlState->IsCheckedOut() )
			{
				TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
				CheckInOperation->SetDescription(ChangelistDescription);
				SourceControlProvider.Execute(CheckInOperation, AbsoluteFilename);
			}
		}
	}

	return 0;
}
