// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/WorldPartitionBuilderCommandlet.h"

#include "CoreMinimal.h"
#include "EngineUtils.h"
#include "EditorWorldUtils.h"
#include "FileHelpers.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/EngineVersion.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Trace/Trace.h"

#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"

#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionBuilderCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionBuilderCommandlet, All, All);

UWorldPartitionBuilderCommandlet::UWorldPartitionBuilderCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

int32 UWorldPartitionBuilderCommandlet::Main(const FString& Params)
{
	FPackageSourceControlHelper PackageHelper;

	// Use the commandlet parameters as it may differ from FCommandline::Get()
	// Provided through this scope as most WP builders are retrieving their arguments from their
	// constructors which can't receive parameters.
	FWorldPartitionBuilderArgsScope BuilderArgsScope(Params);

	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionBuilderCommandlet::Main);

	UE_SCOPED_TIMER(TEXT("Execution"), LogWorldPartitionBuilderCommandlet, Display);

	TArray<FString> Tokens, Switches;
	ParseCommandLine(*Params, Tokens, Switches);

	if (Tokens.Num() != 1)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Missing world name");
		return 1;
	}

	bAutoSubmit = Switches.Contains(TEXT("AutoSubmit"));
	if (bAutoSubmit)
	{
		if (!ISourceControlModule::Get().GetProvider().IsEnabled())
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "-AutoSubmit requires that a valid revision control provider is enabled, exiting...");
			return 0;
		}

		FParse::Value(*Params, TEXT("AutoSubmitTags="), AutoSubmitTags);
	}

	if (Switches.Contains(TEXT("Verbose")))
	{
		LogWorldPartitionBuilderCommandlet.SetVerbosity(ELogVerbosity::Verbose);
	}

	if (Switches.Contains(TEXT("RunningFromUnrealEd")))
	{
		UseCommandletResultAsExitCode = true;	// The process return code will match the return code of the commandlet
		FastExit = true;						// Faster exit which avoids crash during shutdown. The engine isn't shutdown cleanly.
	}

	ICollectionManager& CollectionManager = FModuleManager::LoadModuleChecked<FCollectionManagerModule>("CollectionManager").Get();
	TArray<FString> MapPackagesNames;

	// Parse map name or maps collection
	TSharedPtr<ICollectionContainer> CollectionContainer;
	FName CollectionName;
	ECollectionShareType::Type ShareType = ECollectionShareType::CST_All;
	if (CollectionManager.TryParseCollectionPath(Tokens[0], &CollectionContainer, &CollectionName, &ShareType) &&
		CollectionContainer->CollectionExists(CollectionName, ShareType))
	{
		MapPackagesNames = GatherMapsFromCollection(*CollectionContainer, CollectionName, ShareType);
		if (MapPackagesNames.IsEmpty())
	    {
		    UE_LOGF(LogWorldPartitionBuilderCommandlet, Warning, "Found no maps to process in collection %ls, exiting", *Tokens[0]);
		    return 0;
	    }
	}
	else if (Tokens[0].StartsWith(TEXT("*")))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.WaitForCompletion();

		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true;
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		
		TArray<FAssetData> WorldAssets;
		AssetRegistry.GetAssets(Filter, WorldAssets);

		Algo::Transform(WorldAssets, MapPackagesNames, [](const FAssetData& AssetData) { return AssetData.GetSoftObjectPath().ToString(); });
		
		// Sort maps gathered from the Asset Registry in case the commandlet crashes, so we can specify *,100 to start at a specific index
		MapPackagesNames.Sort();

		TArray<FString> SubTokens;
		Tokens[0].ParseIntoArray(SubTokens, TEXT(","));

		if (SubTokens.IsValidIndex(1))
		{
			MapPackagesNames.RemoveAt(0, FCString::Atoi(*SubTokens[1]));
		}
	}
	else
	{
		TArray<FString> MapList;
		Tokens[0].ParseIntoArray(MapList, TEXT(","));

		for (const FString& Map : MapList)
		{
			FString MapLongPackageName;
			if (FPackageName::SearchForPackageOnDisk(Map, &MapLongPackageName))
			{
				MapPackagesNames.Add(MapLongPackageName);
			}
			else
			{
				FString PackageFilename;
				if (FPackageName::TryConvertLongPackageNameToFilename(Map, PackageFilename, TEXT("")))
				{
					IFileManager::Get().IterateDirectoryRecursively(*PackageFilename, [&MapLongPackageName, &MapPackagesNames](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
					{
						if (!bIsDirectory && FString(FilenameOrDirectory).EndsWith(FPackageName::GetMapPackageExtension()))
						{
							if (FPackageName::TryConvertFilenameToLongPackageName(FilenameOrDirectory, MapLongPackageName))
							{
								MapPackagesNames.Add(MapLongPackageName);
							}
						}
						return true;
					});
				}
			}
		}
	}

	if (MapPackagesNames.IsEmpty())
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Missing world(s) as the first argument to the commandlet. Either supply the world name directly (WorldName or /Path/To/WorldName), or provide a collection name to have the builder operate on a set of maps.");
		return 1;
	}

	// Parse builder class name
	FString BuilderClassName;
	if (!FParse::Value(*Params, TEXT("Builder="), BuilderClassName, false))
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Invalid builder name.");
		return 1;
	}

	// Find builder class
	TSubclassOf<UWorldPartitionBuilder> BuilderClass = FindFirstObject<UClass>(*BuilderClassName, EFindFirstObjectOptions::EnsureIfAmbiguous);
	if (!BuilderClass)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Unknown/invalid world partition builder class: %ls.", *BuilderClassName);
		return 1;
	}	

	// Run the builder on the provided map(s)
	int32 Result = 0;
	uint32 PackageIndex = 0;
	const uint32 PackageCount = MapPackagesNames.Num();
	for (const FString& MapPackageName : MapPackagesNames)
	{
		if (PackageCount > 1)
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "##################################################");
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "[%d / %d] Executing %ls on map %ls...", ++PackageIndex, PackageCount, *BuilderClassName, *MapPackageName);
		}

		if (!RunBuilder(BuilderClass, MapPackageName))
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Warning, "Failed to execute %ls on map %ls.", *BuilderClassName, *MapPackageName);
			Result = 1;
		}

		for (int32 AdditionalPackageIndex = 0; AdditionalPackageIndex < AdditionalWorldPackagesToProcess.Num(); AdditionalPackageIndex++)
		{
			const FString& AdditionalPackageName = AdditionalWorldPackagesToProcess[AdditionalPackageIndex];
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "[%d.%d / %d] Executing %ls on additional map %ls...", PackageIndex, AdditionalPackageIndex + 1, PackageCount, *BuilderClassName, *AdditionalPackageName);

			if (!RunBuilder(BuilderClass, AdditionalPackageName))
			{
				UE_LOGF(LogWorldPartitionBuilderCommandlet, Warning, "Failed to execute %ls on additional map %ls.", *BuilderClassName, *AdditionalPackageName);
				Result = 1;
			}
		}
		AdditionalWorldPackagesToProcess.Empty();

		//Collect Garbage between each levels to avoid problems when reusing levels.
		FWorldPartitionHelpers::DoCollectGarbage();
	}

	// Autosubmit
	if (!Result && !AutoSubmitModifiedFiles())
	{
		return 1;
	}

	return Result;
}

TArray<FString> UWorldPartitionBuilderCommandlet::GatherMapsFromCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType) const
{
	TSet<FString> MapPackagesNames;

	TArray<FSoftObjectPath> AssetsPaths;
	CollectionContainer.GetAssetsInCollection(CollectionName, ShareType, AssetsPaths, ECollectionRecursionFlags::SelfAndChildren);

	UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "Processing collection %ls (%d items)", *CollectionName.ToString(), AssetsPaths.Num());
	for (FSoftObjectPath& AssetPath : AssetsPaths)
	{
		UAssetRegistryHelpers::FixupRedirectedAssetPath(AssetPath);

		FString PackageName = AssetPath.GetLongPackageName();

		if (FEditorFileUtils::IsMapPackageAsset(PackageName))
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "* %ls", *PackageName);
			MapPackagesNames.Add(PackageName);
		}
		else
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Log, "%ls was not found or is not a map package", *PackageName);
		}
	}

	return MapPackagesNames.Array();
}

bool UWorldPartitionBuilderCommandlet::RunBuilder(TSubclassOf<UWorldPartitionBuilder> InBuilderClass, const FString& InWorldPackageName)
{
	// This will convert incomplete package name to a fully qualified path
	FString WorldLongPackageName;
	FString WorldFilename;
	if (!FPackageName::SearchForPackageOnDisk(InWorldPackageName, &WorldLongPackageName, &WorldFilename))
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Package '%ls' not found", *InWorldPackageName);
		return false;
	}

	// Load the world package
	UPackage* WorldPackage = LoadWorldPackageForEditor(WorldLongPackageName);
	if (!WorldPackage)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Couldn't load package %ls.", *WorldLongPackageName);
		return false;
	}

	// Find the world in the given package
	UWorld* World = UWorld::FindWorldInPackage(WorldPackage);
	if (!World)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "No world in specified package %ls.", *WorldLongPackageName);
		return false;
	}

	// Load configuration file
	FString WorldConfigFilename = FPackageName::LongPackageNameToFilename(World->GetPackage()->GetName(), TEXT(".ini"));
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*WorldConfigFilename))
	{
		LoadConfig(GetClass(), *WorldConfigFilename);
	}

	// Create builder instance
	UWorldPartitionBuilder* Builder = NewObject<UWorldPartitionBuilder>(GetTransientPackage(), InBuilderClass);
	if (!Builder)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Failed to create builder.");
		return false;
	}

	Builder->SetModifiedFilesHandler(UWorldPartitionBuilder::FModifiedFilesHandler::CreateUObject(this, &UWorldPartitionBuilderCommandlet::OnFilesModified));

	bool bResult;
	{
		FGCObjectScopeGuard BuilderGuard(Builder);
		bResult = Builder->RunBuilder(World);
	}

	// Save configuration file
	if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*WorldConfigFilename) ||
		!FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*WorldConfigFilename))
	{
		SaveConfig(CPF_Config, *WorldConfigFilename);
	}

	TArray<FString> WorldPackages;
	if (Builder->ShouldProcessAdditionalWorlds(World, WorldPackages))
	{
		AdditionalWorldPackagesToProcess.Append(WorldPackages);
	}

	return bResult;
}

bool UWorldPartitionBuilderCommandlet::OnFilesModified(const TArray<FString>& InModifiedFiles, const FString& InChangeDescription)
{
	if (!InModifiedFiles.IsEmpty())
	{
		AutoSubmitFiles.Emplace(InChangeDescription, InModifiedFiles);
	}

	return true;
}

bool UWorldPartitionBuilderCommandlet::AutoSubmitModifiedFiles() const
{
	bool bSucceeded = true;

	if (bAutoSubmit)
	{
		UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "Submitting changes to revision control...");

		if (!AutoSubmitFiles.IsEmpty())
		{
			FString AllChanges;
			TArray<FString> AllModifiedFiles;
			for (const auto& [Description, Files] : AutoSubmitFiles)
			{
				AllChanges += Description + TEXT("\n");
				AllModifiedFiles.Append(Files);
			}

			FText ChangelistDescription = FText::FromString(FString::Printf(TEXT("%s\nBased on CL %d\n%s"), *AllChanges, FEngineVersion::Current().GetChangelist(), *AutoSubmitTags));

			TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
			CheckInOperation->SetDescription(ChangelistDescription);
			if (ISourceControlModule::Get().GetProvider().Execute(CheckInOperation, AllModifiedFiles) != ECommandResult::Succeeded)
			{
				UE_LOGF(LogWorldPartitionBuilderCommandlet, Error, "Failed to submit changes to revision control.");
				bSucceeded = false;
			}
			else
			{
				UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "Submitted changes to revision control");
			}
		}
		else
		{
			UE_LOGF(LogWorldPartitionBuilderCommandlet, Display, "No files to submit!");
		}
	}

	return bSucceeded;
}
