// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/WorldPartitionCommandletHelpers.h"

#include "Engine/Level.h"
#include "Engine/World.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PackageName.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

namespace WorldPartitionCommandletHelpers
{
	DEFINE_LOG_CATEGORY(LogWorldPartitionCommandletUtils)

	UWorld* LoadAndInitWorld(const FString& LevelToLoad)
	{
		UWorld* World = LoadWorld(LevelToLoad);
		if (World != nullptr)
		{
			if (InitLevel(World) != nullptr)
			{
				return World;
			}
		}

		return nullptr;
	}

	UWorld* LoadWorld(const FString& LevelToLoad)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionConvertCommandlet::LoadWorld);

		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogWorldPartitionCommandletUtils, Log, "Loading level %ls.", *LevelToLoad);
		CLEAR_WARN_COLOR();

		// This will convert incomplete package name to a fully qualified path, avoiding calling it several times (takes ~50s)
		FString FullLevelToLoadPath;
		if (!FPackageName::SearchForPackageOnDisk(LevelToLoad, &FullLevelToLoadPath))
		{
			UE_LOGF(LogWorldPartitionCommandletUtils, Error, "Unknown level '%ls'", *LevelToLoad);
			return nullptr;
		}

		UPackage* MapPackage = LoadPackage(nullptr, *FullLevelToLoadPath, LOAD_None);
		if (!MapPackage)
		{
			UE_LOGF(LogWorldPartitionCommandletUtils, Error, "Error loading %ls.", *FullLevelToLoadPath);
			return nullptr;
		}

		return UWorld::FindWorldInPackage(MapPackage);
	}

	ULevel* InitLevel(UWorld* World)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionConvertCommandlet::InitWorld);

		SET_WARN_COLOR(COLOR_WHITE);
		UE_LOGF(LogWorldPartitionCommandletUtils, Log, "Initializing level %ls.", *World->GetName());
		CLEAR_WARN_COLOR();

		// Setup the world.
		World->WorldType = EWorldType::Editor;
		World->AddToRoot();
		if (!World->bIsWorldInitialized)
		{
			UWorld::InitializationValues IVS;
			IVS.RequiresHitProxies(false);
			IVS.ShouldSimulatePhysics(false);
			IVS.EnableTraceCollision(false);
			IVS.CreateNavigation(false);
			IVS.CreateAISystem(false);
			IVS.AllowAudioPlayback(false);
			IVS.CreatePhysicsScene(true);

			World->InitWorld(IVS);
			World->PersistentLevel->UpdateModelComponents();
			World->UpdateWorldComponents(true, false);

			World->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		}

		return World->PersistentLevel;
	}

	bool Checkout(const TArray<UPackage*>& PackagesToCheckout, FPackageSourceControlHelper& SCHelper)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CheckoutPackages);

		UE_LOGF(LogWorldPartitionCommandletUtils, Log, "Checking out %d Packages.", PackagesToCheckout.Num());
		for (UPackage* Package : PackagesToCheckout)
		{
			if (!Checkout(Package, SCHelper))
			{
				return false;
			}
		}

		return true;
	}

	bool Checkout(UPackage* PackagesToCheckout, FPackageSourceControlHelper& SCHelper)
	{
		UE_LOGF(LogWorldPartitionCommandletUtils, Verbose, "Checking out Package %ls.", *SourceControlHelpers::PackageFilename(PackagesToCheckout));

		FString PackageFileName = SourceControlHelpers::PackageFilename(PackagesToCheckout);
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*PackageFileName))
		{
			if (!SCHelper.Checkout(PackagesToCheckout))
			{
				return false;
			}
		}

		return true;
	}

	bool Save(const TArray<UPackage*>& PackagesToSave)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SavePackages);

		UE_LOGF(LogWorldPartitionCommandletUtils, Log, "Saving %d packages.", PackagesToSave.Num());
		for (UPackage* PackageToSave : PackagesToSave)
		{
			if (!Save(PackagesToSave))
			{
				return false;
			}
		}	
		
		return true;
	}

	bool Save(UPackage* PackageToSave)
	{
		FString PackageFileName = SourceControlHelpers::PackageFilename(PackageToSave);
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;

		UE_LOGF(LogWorldPartitionCommandletUtils, Verbose, "Saving Package %ls.", *PackageFileName);
		return UPackage::SavePackage(PackageToSave, nullptr, *PackageFileName, SaveArgs);
	}

	bool AddToSourceControl(const TArray<UPackage*>& PackagesToAdd, FPackageSourceControlHelper& SCHelper)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AddPackagesToSourceControl);

		for (UPackage* PackageToAdd : PackagesToAdd)
		{
			if (!AddToSourceControl(PackageToAdd, SCHelper))
			{
				return false;
			}
		}

		return true;
	}

	bool AddToSourceControl(UPackage* PackageToAdd, FPackageSourceControlHelper& SCHelper)
	{
		UE_LOGF(LogWorldPartitionCommandletUtils, Verbose, "Adding Package %ls.", *SourceControlHelpers::PackageFilename(PackageToAdd));

		return SCHelper.AddToSourceControl(PackageToAdd);
	}

	bool Delete(const TArray<UPackage*>& PackagesToDelete, FPackageSourceControlHelper& SCHelper)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DeletePackages);

		return SCHelper.Delete(PackagesToDelete);
	}

	bool Delete(UPackage* PackageToDelete, FPackageSourceControlHelper& SCHelper)
	{
		UE_LOGF(LogWorldPartitionCommandletUtils, Verbose, "Deleting Package %ls.", *SourceControlHelpers::PackageFilename(PackageToDelete));

		return SCHelper.Delete(PackageToDelete);
	}
}
