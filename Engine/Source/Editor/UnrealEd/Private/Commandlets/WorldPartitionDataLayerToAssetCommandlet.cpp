// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 UDataLayerToAssetCommandlet.cpp: Commandlet used to convert a partitioned ULevel's data layers to assets
=============================================================================*/

#include "Commandlets/WorldPartitionDataLayerToAssetCommandLet.h"

#include "Algo/Copy.h"
#include "Algo/Accumulate.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "DataLayer/DataLayerFactory.h"
#include "Engine/World.h"
#include "Logging/LogVerbosity.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHelpers.h"

DEFINE_LOG_CATEGORY(LogDataLayerToAssetCommandlet);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDataLayerConversionInfo::SetDataLayerToConvert(const UDeprecatedDataLayerInstance* InDataLayerToConvert)
{
	DataLayerToConvert = InDataLayerToConvert;

	DataLayerAsset->DataLayerType = DataLayerToConvert->GetType();
	DataLayerAsset->DebugColor = DataLayerToConvert->GetDebugColor();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDataLayerConversionInfo::SetDataLayerInstance(UDataLayerInstanceWithAsset* InDataLayerInstance)
{
	DataLayerInstance = InDataLayerInstance;

	if (DataLayerToConvert != nullptr)
	{
		DataLayerInstance->DataLayerAsset = DataLayerAsset;
		DataLayerInstance->bIsVisible = DataLayerToConvert->bIsVisible;
		DataLayerInstance->bIsInitiallyVisible = DataLayerToConvert->bIsInitiallyVisible;
		DataLayerInstance->bIsInitiallyLoadedInEditor = DataLayerToConvert->bIsInitiallyLoadedInEditor;
		DataLayerInstance->bIsLocked = DataLayerToConvert->bIsLocked;
		DataLayerInstance->InitialRuntimeState = DataLayerToConvert->InitialRuntimeState;
	}
	else
	{
		// Check the DataLayerInstance was already properly converted
		check(DataLayerInstance->DataLayerAsset == DataLayerAsset)
	}
}

UDataLayerToAssetCommandletContext::UDataLayerToAssetCommandletContext(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::GetDataLayerConversionInfo(const UDeprecatedDataLayerInstance* DataLayer) const
{
	const TObjectPtr<UDataLayerConversionInfo>* Entry = DataLayerConversionInfo.FindByPredicate([&DataLayer](const TObjectPtr<UDataLayerConversionInfo>& Other) { return DataLayer == Other->DataLayerToConvert; });
	return Entry != nullptr ? Entry->Get() : nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::GetDataLayerConversionInfo(const UDataLayerAsset* DataLayerAsset) const
{
	const TObjectPtr<UDataLayerConversionInfo>* Entry = DataLayerConversionInfo.FindByPredicate([&DataLayerAsset](const TObjectPtr<UDataLayerConversionInfo>& Other) { return DataLayerAsset == Other->DataLayerAsset; });
	return Entry != nullptr ? Entry->Get() : nullptr;
}

UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::GetDataLayerConversionInfo(const UDataLayerInstanceWithAsset* DataLayerInstance) const
{
	const TObjectPtr<UDataLayerConversionInfo>* Entry = DataLayerConversionInfo.FindByPredicate([&DataLayerInstance](const TObjectPtr<UDataLayerConversionInfo>& Other) { return DataLayerInstance == Other->DataLayerInstance; });
	return Entry != nullptr ? Entry->Get() : nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::GetDataLayerConversionInfo(const FActorDataLayer& ActorDataLayer) const
{
	const TObjectPtr<UDataLayerConversionInfo>* Entry = DataLayerConversionInfo.FindByPredicate([&ActorDataLayer](const TObjectPtr<UDataLayerConversionInfo>& Other)
	{ 
		return Other->DataLayerToConvert != nullptr && ActorDataLayer.Name == Other->DataLayerToConvert->GetDataLayerFName();
	});
	
	return Entry != nullptr ? Entry->Get() : nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::StoreExistingDataLayer(FAssetData& AssetData)
{
	UDataLayerConversionInfo* ConversionInfo = NewObject<UDataLayerConversionInfo>();
	ConversionInfo->DataLayerAsset = CastChecked<UDataLayerAsset>(AssetData.GetAsset());

	UE_LOGF(LogDataLayerToAssetCommandlet, Verbose, "Data Layer Asset %ls discovered.",
		*ConversionInfo->DataLayerAsset->GetFullName());
	return DataLayerConversionInfo.Add_GetRef(ConversionInfo).Get();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::StoreDataLayerAssetConversion(const UDeprecatedDataLayerInstance* DataLayerToConvert, UDataLayerAsset* NewDataLayerAsset)
{
	if (UDataLayerConversionInfo* ConversionInfo = GetDataLayerConversionInfo(DataLayerToConvert))
	{
		UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to assign asset %ls to data Layer %ls. The data layer is already associated to asset %ls.", 
			*NewDataLayerAsset->GetFullName() , *DataLayerToConvert->GetDataLayerShortName() , *ConversionInfo->DataLayerAsset->GetFullName());
		return nullptr;
	}

	UDataLayerConversionInfo* ConversionInfo = GetDataLayerConversionInfo(NewDataLayerAsset);
	if (ConversionInfo != nullptr)
	{
		if (ConversionInfo->DataLayerToConvert != nullptr)
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to assign asset %ls to data Layer %ls. The asset is already associated to data layer %ls.",
				*NewDataLayerAsset->GetFullName(), *DataLayerToConvert->GetDataLayerShortName(), *ConversionInfo->DataLayerToConvert->GetDataLayerShortName());
			return nullptr;
		}
	}
	else
	{
		ConversionInfo = NewObject<UDataLayerConversionInfo>();
		ConversionInfo->DataLayerAsset = NewDataLayerAsset;
		DataLayerConversionInfo.Add(ConversionInfo);
	}

	ConversionInfo->SetDataLayerToConvert(DataLayerToConvert);

	UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Data Layer Asset %ls is associated to Data Layer %ls for conversion",
		*ConversionInfo->DataLayerAsset->GetFullName(), *ConversionInfo->DataLayerToConvert->GetDataLayerShortName());
	return ConvertingDataLayerInfo.Add_GetRef(ConversionInfo).Get();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UDataLayerConversionInfo* UDataLayerToAssetCommandletContext::StoreDataLayerInstanceConversion(const UDataLayerAsset* DataLayerAsset, UDataLayerInstanceWithAsset* NewDataLayerInstance)
{
	if (UDataLayerConversionInfo* ConversionInfo = GetDataLayerConversionInfo(NewDataLayerInstance))
	{
		UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to assign asset %ls to data layer instance %ls. The instance is already associated to asset %ls.",
			*DataLayerAsset->GetFullName(), *NewDataLayerInstance->GetFName().ToString(), *ConversionInfo->DataLayerAsset->GetFullName());
		return nullptr;
	}

	if (UDataLayerConversionInfo* ConversionInfo = GetDataLayerConversionInfo(DataLayerAsset)) 
	{
		check(ConversionInfo->DataLayerInstance == nullptr);
		ConversionInfo->SetDataLayerInstance(NewDataLayerInstance);

		UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Data Layer Instance %ls is associated to Data Layer Asset %ls for conversion",
			*ConversionInfo->DataLayerInstance->GetFName().ToString(), *ConversionInfo->DataLayerAsset->GetFullName());
		return ConversionInfo;
	}

	UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to assign asset %ls to data layer instance %ls. The asset was not retrieved in conversion data.",
		*DataLayerAsset->GetFullName(), *NewDataLayerInstance->GetFName().ToString());
	return nullptr;
}

bool UDataLayerToAssetCommandletContext::SetPreviousConversions(UDataLayerConversionInfo* CurrentConversion, TArray<TWeakObjectPtr<UDataLayerConversionInfo>>&& PreviousConversions)
{
	check(CurrentConversion->CurrentConvertingInfo == nullptr);
	check(CurrentConversion->PreviousConversionsInfo.IsEmpty());

	CurrentConversion->PreviousConversionsInfo = MoveTemp(PreviousConversions);

	uint32 ErrorCount = 0;
	for (TWeakObjectPtr<UDataLayerConversionInfo>& PreviousConversion : CurrentConversion->PreviousConversionsInfo)
	{
		check(PreviousConversion->CurrentConvertingInfo == nullptr);
		check(PreviousConversion->PreviousConversionsInfo.IsEmpty());

		if (PreviousConversion->DataLayerInstance != nullptr)
		{
			ErrorCount++;
			UE_LOGF(LogDataLayerToAssetCommandlet, Error,
				"DataLayer %ls was already converted but is still to be converted. Re-Sync Data to a clean conversion or pre-conversion state and re-run the commandlet",
				*CurrentConversion->DataLayerAsset->GetFullName());
		}

		PreviousConversion->CurrentConvertingInfo = CurrentConversion;
	}

	ConvertingDataLayerInfo.AddUnique(CurrentConversion);

	return ErrorCount == 0;
}

bool UDataLayerToAssetCommandletContext::FindDataLayerConversionInfos(FName DataLayerAssetName, TArray<TWeakObjectPtr<UDataLayerConversionInfo>>& OutConversionInfos) const
{
	OutConversionInfos.Empty();

	FString SanitizedAssetName = DataLayerAssetName.ToString();
	const TCHAR* InvalidPackageChar = INVALID_LONGPACKAGE_CHARACTERS;
	while (*InvalidPackageChar)
	{
		SanitizedAssetName.ReplaceCharInline(*InvalidPackageChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidPackageChar;
	}

	for (TObjectPtr<UDataLayerConversionInfo> const&  ConversionInfo : DataLayerConversionInfo)
	{
		if (ConversionInfo->DataLayerAsset->GetFName() == FName(SanitizedAssetName))
		{
			OutConversionInfos.Add(ConversionInfo.Get());
		}
	}

	return !OutConversionInfos.IsEmpty();
}

void UDataLayerToAssetCommandletContext::LogConversionInfos() const
{
	if (LogDataLayerToAssetCommandlet.GetVerbosity() >= ELogVerbosity::Verbose)
	{
		for (const TObjectPtr<UDataLayerConversionInfo>& info : DataLayerConversionInfo)
		{
			FString ConflictingConversionString = TEXT("");
			if(!info->PreviousConversionsInfo.IsEmpty())
			{
				ConflictingConversionString  = FString::JoinBy(info->PreviousConversionsInfo, TEXT(", "), [](const TWeakObjectPtr<UDataLayerConversionInfo>& ConflictingInfo)
				{ 
						return ConflictingInfo->DataLayerAsset->GetFullName(); 
				});
			}
			
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UE_LOGF(LogDataLayerToAssetCommandlet, Verbose, "[Conversion Info] Data Layer %ls\t\tData Layer Asset: %ls\t\t\t\t\t\tData Layer Instance: %ls\t\tConverting By: %ls\t\t\t\t\t\tConflicting Previous Conversion: %ls",
				info->DataLayerToConvert != nullptr ? *info->DataLayerToConvert->GetDataLayerShortName() : TEXT("None"),
				info->DataLayerAsset != nullptr ? *info->DataLayerAsset->GetFullName() : TEXT("None"),
				info->DataLayerInstance != nullptr ? *info->DataLayerInstance->GetFName().ToString() : TEXT("None"),
				info->CurrentConvertingInfo != nullptr ? *info->CurrentConvertingInfo->DataLayerAsset->GetFullName() : TEXT("None"),
				*ConflictingConversionString);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

UDataLayerToAssetCommandlet::UDataLayerToAssetCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	DataLayerFactory(NewObject<UDataLayerFactory>())
{}

int32 UDataLayerToAssetCommandlet::Main(const FString& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionConvertCommandlet::Main);

	UE_SCOPED_TIMER(TEXT("Data Layer Conversion"), LogDataLayerToAssetCommandlet, Display);

	FPackageSourceControlHelper PackageHelper;

	ON_SCOPE_EXIT
	{
		if (MainWorld != nullptr)
		{
			const bool bBroadcastWorldDestroyedEvent = false;
			MainWorld->DestroyWorld(bBroadcastWorldDestroyedEvent);
		}
	};

	TArray<FString> Tokens, Switches;
	TMap<FString, FString> CommandLineParams;
	ParseCommandLine(*Params, Tokens, Switches, CommandLineParams);
	if (!InitializeFromCommandLine(Tokens, Switches, CommandLineParams))
	{
		return EReturnCode::CommandletInitializationError;
	}

	ConversionFolder = DestinationFolder + "/" + MainWorld->GetName() + "/";

	TStrongObjectPtr<UDataLayerToAssetCommandletContext> Context(NewObject<UDataLayerToAssetCommandletContext>());

	if (!BuildConversionInfos(Context, PackageHelper))
	{
		return EReturnCode::DataLayerConversionError;
	}

	if (!ResolvePreviousConversionsToCurrent(Context))
	{
		return EReturnCode::DataLayerConversionError;
	}

	Context->LogConversionInfos();

	if (!CreateDataLayerInstances(Context))
	{
		return EReturnCode::DataLayerConversionError;
	}

	if (!RemapActorDataLayersToAssets(Context, PackageHelper))
	{
		return EReturnCode::ActorDataLayerRemappingError;
	}

	if (!PerformProjectSpecificConversions(Context))
	{
		return EReturnCode::ProjectSpecificConversionError;
	}

	if (!DeletePreviousConversionsData(Context, PackageHelper))
	{
		return EReturnCode::DataLayerConversionError;
	}

	if (!CommitConversion(Context, PackageHelper))
	{
		return EReturnCode::DataLayerConversionError;
	}

	return EReturnCode::Success;
}

bool UDataLayerToAssetCommandlet::InitializeFromCommandLine(TArray<FString>& Tokens, TArray<FString> const& Switches, TMap<FString, FString> const& Params)
{
	constexpr TCHAR DestinationFolderSwitch[] = TEXT("DestinationFolder");
	if (FString const* DestFolderPtr = Params.Find(DestinationFolderSwitch))
	{
		DestinationFolder = *DestFolderPtr;
	}
	else
	{
		UE_LOGF(LogDataLayerToAssetCommandlet, Error, "No \"%ls\" switch specified", DestinationFolderSwitch);
		return false;
	}

	if (Switches.Contains(TEXT("Verbose")))
	{
		LogDataLayerToAssetCommandlet.SetVerbosity(ELogVerbosity::Verbose);
		WorldPartitionCommandletHelpers::LogWorldPartitionCommandletUtils.SetVerbosity(ELogVerbosity::Verbose);
	}

	bPerformSavePackages = Switches.Contains(TEXT("NoSave")) == false;
	bIgnoreActorLoadingErrors = Switches.Contains(TEXT("IgnoreActorLoadingErrors"));

	MainWorld = WorldPartitionCommandletHelpers::LoadAndInitWorld(Tokens[0]);
	if (!MainWorld)
	{
		UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to Load %ls, Conversion will abort", *Tokens[0]);
		return false;
	}

	return true;
}

bool UDataLayerToAssetCommandlet::BuildConversionInfos(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper)
{
	UE_SCOPED_TIMER(TEXT("Retrieving Already Converted Data Layers"), LogDataLayerToAssetCommandlet, Display);

	TArray<FAssetData> ExistingDataLayerAssets;
	IAssetRegistry& AssetRegistry = FAssetRegistryModule::GetRegistry();
	AssetRegistry.GetAssetsByClass(FTopLevelAssetPath(UDataLayerAsset::StaticClass()), ExistingDataLayerAssets);
	for (FAssetData& AssetData : ExistingDataLayerAssets)
	{
		if (IsAssetInConversionFolder(AssetData.GetSoftObjectPath()))
		{
			CommandletContext->StoreExistingDataLayer(AssetData);
		}
	}

	AWorldDataLayers* WorldDataLayers = MainWorld->GetWorldDataLayers();
	TArray<UDataLayerInstance*> DataLayerInstances;
	Algo::Copy(WorldDataLayers->DataLayerInstances, DataLayerInstances);
	for (int32 i = DataLayerInstances.Num() -1; i >= 0; --i)
	{
		if(UDataLayerInstanceWithAsset* DataLayerInstance = Cast<UDataLayerInstanceWithAsset>(DataLayerInstances[i]))
		{
			if (const UDataLayerAsset* DataLayerAsset = DataLayerInstance->GetAsset())
			{
				if (UDataLayerConversionInfo* ConversionInfo = CommandletContext->GetDataLayerConversionInfo(DataLayerAsset))
				{
					CommandletContext->StoreDataLayerInstanceConversion(ConversionInfo->DataLayerAsset, DataLayerInstance);
				}
			}
		}
	}

	uint32 ErrorCount = 0;
	for (const UDataLayerInstance* DataLayerInstance : WorldDataLayers->DataLayerInstances)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (const UDeprecatedDataLayerInstance* DataLayerToConvert = Cast<UDeprecatedDataLayerInstance>(DataLayerInstance))
		{
			if (!CreateConversionFromDataLayer(CommandletContext, DataLayerToConvert, PackageHelper))
			{
				ErrorCount++;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return ErrorCount == 0;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDataLayerToAssetCommandlet::CreateConversionFromDataLayer(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, const UDeprecatedDataLayerInstance* DataLayer, FPackageSourceControlHelper &PackageHelper)
{
	if(TObjectPtr<UDataLayerAsset> DataLayerAsset = GetOrCreateDataLayerAssetForConversion(CommandletContext, FName(DataLayer->GetDataLayerShortName())))
	{
		if (UDataLayerConversionInfo* ConversionInfo = CommandletContext->StoreDataLayerAssetConversion(DataLayer, DataLayerAsset))
		{
			if (bPerformSavePackages)
			{
				return WorldPartitionCommandletHelpers::CheckoutSaveAdd(ConversionInfo->DataLayerAsset->GetPackage(), PackageHelper);
			}

			return true;
		}
	}
	
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TObjectPtr<UDataLayerAsset> UDataLayerToAssetCommandlet::GetOrCreateDataLayerAssetForConversion(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FName DataLayerAssetName)
{
	TObjectPtr<UDataLayerAsset> DataLayerAsset = nullptr;

	TArray<TWeakObjectPtr<UDataLayerConversionInfo>> ConversionInfos;
	if (CommandletContext->FindDataLayerConversionInfos(DataLayerAssetName, ConversionInfos))
	{
		for (TWeakObjectPtr<UDataLayerConversionInfo>& ConversionInfo : ConversionInfos)
		{
			if (IsAssetInConversionFolder(ConversionInfo->DataLayerAsset))
			{
				check(DataLayerAsset == nullptr);
				return CastChecked<UDataLayerAsset>(ConversionInfo->DataLayerAsset);
			}
		}
	}

	UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Creating new Data Layer Asset %ls in folder %ls",
		*DataLayerAssetName.ToString(), *ConversionFolder);

	FString SanitizedAssetName = DataLayerAssetName.ToString();
	const TCHAR* InvalidPackageChar = INVALID_LONGPACKAGE_CHARACTERS;
	while (*InvalidPackageChar)
	{
		SanitizedAssetName.ReplaceCharInline(*InvalidPackageChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidPackageChar;
	}

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	if (UObject* Asset = AssetTools.CreateAsset(SanitizedAssetName, ConversionFolder, UDataLayerAsset::StaticClass(), DataLayerFactory))
	{
		DataLayerAsset = CastChecked<UDataLayerAsset>(Asset);
	}
	else
	{
		UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to create asset for %ls in folder %ls. Consult log for more details",
			*DataLayerAssetName.ToString(), *ConversionFolder);
	}

	return DataLayerAsset;
}

bool UDataLayerToAssetCommandlet::ResolvePreviousConversionsToCurrent(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext)
{
	UE_SCOPED_TIMER(TEXT("Resolving Conflicting Conversions"), LogDataLayerToAssetCommandlet, Display);
	UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Resolving Conflicting Conversions");

	uint32 ErrorCount = 0;
	for (const TObjectPtr<UDataLayerConversionInfo>& ConversionInfo : CommandletContext->GetDataLayerConversionInfos())
	{
		if(IsAssetInConversionFolder(ConversionInfo->DataLayerAsset))
		{
			TArray<TWeakObjectPtr<UDataLayerConversionInfo>> PreviousConversionInfos;
			if (CommandletContext->FindDataLayerConversionInfos(ConversionInfo->DataLayerAsset->GetFName(), PreviousConversionInfos))
			{
				PreviousConversionInfos.Remove(ConversionInfo);

				if (!PreviousConversionInfos.IsEmpty())
				{
					if (!CommandletContext->SetPreviousConversions(ConversionInfo, MoveTemp(PreviousConversionInfos)))
					{
						ErrorCount++;
					}
				}
			}
		}
	}

	return ErrorCount == 0;
}

bool UDataLayerToAssetCommandlet::RemapActorDataLayersToAssets(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper)
{
	UE_SCOPED_TIMER(TEXT("Remapping Actors Data Layers"), LogDataLayerToAssetCommandlet, Display);
	UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Starting Actor Data Layer Remapping To Data Layer Asset. This can take a while.");

	uint32 ErrorCount = 0;
	FWorldPartitionHelpers::ForEachActorWithLoading(MainWorld->GetWorldPartition(), [&ErrorCount, &CommandletContext, this, &PackageHelper](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		uint32 ActorConversionErrors = 0;
		if (AActor* Actor = ActorDescInstance->GetActor())
		{
			ActorConversionErrors += RemapDataLayersAssetsFromPreviousConversions(CommandletContext, Actor);
			ActorConversionErrors += RemapActorDataLayers(CommandletContext, Actor);

			if (!PerformAdditionalActorConversions(CommandletContext, Actor))
			{
				ActorConversionErrors++;
			}

			if (ActorConversionErrors == 0 && bPerformSavePackages)
			{
				if (bPerformSavePackages && Actor->GetExternalPackage()->IsDirty())
				{
					if (!WorldPartitionCommandletHelpers::CheckoutSaveAdd(Actor->GetExternalPackage(), PackageHelper))
					{
						ActorConversionErrors++;
					}
				}
			}
		}
		else
		{
			const FDataLayerInstanceNames& ActDescDataLayers = ActorDescInstance->GetDataLayerInstanceNames();
			if (!ActDescDataLayers.IsEmpty())
			{
				FString DataLayerString = FString::JoinBy(ActDescDataLayers.ToArray(), TEXT(", "), [](const FName& DataLayerName) { return DataLayerName.ToString(); });

				UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Actor %ls failed to load. Its data layers %ls will not be remapped to a data layer asset.",
					*ActorDescInstance->GetActorNameString(), *DataLayerString);
				if (!bIgnoreActorLoadingErrors)
				{
					ActorConversionErrors++;
				}
			}
		}

		ErrorCount += ActorConversionErrors;
		return true;
	});

	UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Actor Data Layer Remapping To Data Layer Asset Done.");

	return ErrorCount == 0;
}

uint32 UDataLayerToAssetCommandlet::RemapActorDataLayers(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, AActor* Actor)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	uint32 ErrorCount = 0;

	const TArray<FActorDataLayer>& ActorDataLayers = Actor->GetActorDataLayers();
	for (int32 i = ActorDataLayers.Num() - 1; i >= 0; --i)
	{
		const FActorDataLayer& ActorDataLayer = ActorDataLayers[i];
		if (UDataLayerConversionInfo* DataLayerConversionInfo = CommandletContext->GetDataLayerConversionInfo(ActorDataLayer))
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Verbose, "Data layer %ls in Actor %ls remapped to data layer asset %ls",
				*ActorDataLayer.Name.ToString(), *Actor->GetName(), *DataLayerConversionInfo->DataLayerAsset->GetName());

			if (Actor->AddDataLayer(DataLayerConversionInfo->DataLayerInstance))
			{
				if (!Actor->RemoveDataLayer(DataLayerConversionInfo->DataLayerToConvert))
				{
					ErrorCount++;
					UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to remove data layer %ls in Actor %ls",
						*DataLayerConversionInfo->DataLayerToConvert->GetDataLayerShortName(), *Actor->GetName());
				}
			}
			else
			{
				ErrorCount++;
				UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to add data layer asset %ls in Actor %ls",
					*DataLayerConversionInfo->DataLayerAsset->GetFullName(), *Actor->GetName());
			}

		}
		else
		{
			ErrorCount++;
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Actor %ls is referencing data layer %ls but it does not match any data layer asset.",
				*Actor->GetName(), *ActorDataLayer.Name.ToString());
		}
	}

	return ErrorCount;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

// Multiple run of the commandlet can lead to Actors referencing Data Layer Assets in different folders. Always remap to the newly created asset.
uint32 UDataLayerToAssetCommandlet::RemapDataLayersAssetsFromPreviousConversions(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, AActor* Actor)
{
	uint32 ErrorCount = 0;

	TArray<const UDataLayerAsset*> ActorDataLayerAssets = Actor->GetDataLayerAssets();
	for (int32 i = ActorDataLayerAssets.Num() - 1; i >= 0; --i)
	{
		const TObjectPtr<const UDataLayerAsset>& ActorDataLayerAsset = ActorDataLayerAssets[i];
		if (UDataLayerConversionInfo* DataLayerConversionInfo = CommandletContext->GetDataLayerConversionInfo(ActorDataLayerAsset.Get()))
		{
			if (DataLayerConversionInfo->IsAPreviousConversion())
			{
				if (!DataLayerConversionInfo->GetCurrentConversion()->DataLayerInstance->AddActor(Actor))
				{
					ErrorCount++;
					UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to replace data layer asset %ls in Actor %ls",
						*DataLayerConversionInfo->GetCurrentConversion()->DataLayerAsset->GetFullName(), *Actor->GetName());
				}
			}
		}
	}

	return ErrorCount;
}

bool UDataLayerToAssetCommandlet::CreateDataLayerInstances(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext)
{
	UE_SCOPED_TIMER(TEXT("Creating Data Layer Instances"), LogDataLayerToAssetCommandlet, Display);

	AWorldDataLayers* WorldDataLayers = MainWorld->GetWorldDataLayers();
	for (const TWeakObjectPtr<UDataLayerConversionInfo>& ConvertingInfo : CommandletContext->GetConvertingDataLayerConversionInfo())
	{
		if (ConvertingInfo->DataLayerInstance == nullptr)
		{
			UDataLayerInstanceWithAsset* DataLayerInstance = WorldDataLayers->CreateDataLayer<UDataLayerInstanceWithAsset>(ConvertingInfo->DataLayerAsset);
			UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Created new Data Layer Instance %ls",
				*DataLayerInstance->GetFName().ToString());

			CommandletContext->StoreDataLayerInstanceConversion(ConvertingInfo->DataLayerAsset, DataLayerInstance);
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		WorldDataLayers->DeprecatedDataLayerNameToDataLayerInstance.Add(ConvertingInfo->DataLayerToConvert->GetDataLayerFName(), ConvertingInfo->DataLayerInstance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	uint32 ErrorCount = 0;

	if (!RebuildDataLayerHierarchies(CommandletContext))
	{
		ErrorCount++;
	}

	return ErrorCount == 0;
}

bool UDataLayerToAssetCommandlet::RebuildDataLayerHierarchies(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext)
{
	UE_SCOPED_TIMER(TEXT("Creating Data Layer Instances Hierarchy"), LogDataLayerToAssetCommandlet, Display);

	uint32 ErrorCount = 0;
	for (const TWeakObjectPtr<UDataLayerConversionInfo>& ConvertingInfo : CommandletContext->GetConvertingDataLayerConversionInfo())
	{
		UDataLayerInstance* Child = ConvertingInfo->DataLayerInstance;
		if (Child == nullptr)
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to retrieve data layer instance when re-building hiearchy for %ls",
				*ConvertingInfo->DataLayerAsset->GetFullName());
			ErrorCount++;
			continue;
		}

		const UDataLayerInstance* OldParent = ConvertingInfo->DataLayerToConvert->GetParent();
		if (OldParent == nullptr)
		{
			// No Parent, continue
			continue;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const UDeprecatedDataLayerInstance* DeprecatedParent = Cast<UDeprecatedDataLayerInstance>(OldParent);
		if (DeprecatedParent == nullptr)
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Deprecated Data Layer Instance %ls has %ls as a parent. But %ls is not depcrated. This is not permitted",
				*Child->GetDataLayerFullName(), *OldParent->GetDataLayerShortName(), *OldParent->GetDataLayerShortName());
			ErrorCount++;
			continue;
		}

		UDataLayerConversionInfo* ParentConversionInfo = CommandletContext->GetDataLayerConversionInfo(DeprecatedParent);
		if (ParentConversionInfo == nullptr)
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to find conversion info for parent %ls of %ls while rebuilding data layer hierarchy",
				*DeprecatedParent->GetDataLayerFullName(), *Child->GetDataLayerShortName());
			ErrorCount++;
			continue;

		}

		if (UDataLayerInstance* NewParent = ParentConversionInfo->DataLayerInstance)
		{
			if (!Child->SetParent(NewParent))
			{
				UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to set %ls as the parent of %ls",
					*NewParent->GetDataLayerFullName(), *Child->GetDataLayerShortName());
				ErrorCount++;
			}
		}
		else
		{
			UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to retrieve data layer instance for parent %ls when re-building hiearchy for %ls",
				*DeprecatedParent->GetDataLayerFullName(), *Child->GetDataLayerShortName());
			ErrorCount++;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return ErrorCount == 0;
}

bool UDataLayerToAssetCommandlet::DeletePreviousConversionsData(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper)
{
	UE_SCOPED_TIMER(TEXT("Delete Conflicting Assets"), LogDataLayerToAssetCommandlet, Display);

	uint32 ErrorCount = 0;
	for (const TWeakObjectPtr<UDataLayerConversionInfo>& ConvertingInfo : CommandletContext->GetConvertingDataLayerConversionInfo())
	{
		for (const TWeakObjectPtr<UDataLayerConversionInfo>& PreviousConversion : ConvertingInfo->GetPreviousConversions())
		{
			if (bPerformSavePackages)
			{
				if (WorldPartitionCommandletHelpers::Delete(PreviousConversion->DataLayerAsset->GetPackage(), PackageHelper))
				{
					PreviousConversion->DataLayerAsset = nullptr;
				}
				else
				{
					UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to delete Data Layer Asset %ls from previous conversion. (Conflicting with %ls)",
						*PreviousConversion->DataLayerAsset->GetFullName(), *ConvertingInfo->DataLayerAsset->GetFullName());
					ErrorCount++;
				}
			}
		}
	}

	return ErrorCount == 0;
}

bool UDataLayerToAssetCommandlet::CommitConversion(TStrongObjectPtr<UDataLayerToAssetCommandletContext>& CommandletContext, FPackageSourceControlHelper& PackageHelper)
{
	uint32 ErrorCount = 0;
	AWorldDataLayers* WorldDataLayers = MainWorld->GetWorldDataLayers();
	for (const TWeakObjectPtr<UDataLayerConversionInfo>& ConversionInfo : CommandletContext->GetConvertingDataLayerConversionInfo())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ConversionInfo->DataLayerToConvert != nullptr)
		{
			// Remove directly from DataLayerInstances as RemoveDataLayer method also cleans DeprecatedDataLayerNameToDataLayerInstance which is used for runtime conversion
			if (WorldDataLayers->DataLayerInstances.Remove(ConversionInfo->DataLayerToConvert))
			{
				UE_LOGF(LogDataLayerToAssetCommandlet, Log, "Deleted old data layer %ls, it is now converted to Data Layer Asset %ls and Data Layer Instance %ls",
					*ConversionInfo->DataLayerToConvert->GetDataLayerShortName(), *ConversionInfo->DataLayerAsset->GetFullName(), *ConversionInfo->DataLayerInstance->GetFName().ToString());
			}
			else
			{
				UE_LOGF(LogDataLayerToAssetCommandlet, Error, "Failed to delete converting data layer %ls.",
					*ConversionInfo->DataLayerToConvert->GetDataLayerShortName());
				ErrorCount++;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (ErrorCount == 0 && bPerformSavePackages)
	{
		if (!WorldPartitionCommandletHelpers::CheckoutSaveAdd(WorldDataLayers->GetExternalPackage(), PackageHelper))
		{
			return false;
		}
	}

	return true;
}

bool UDataLayerToAssetCommandlet::IsAssetInConversionFolder(const FSoftObjectPath& DataLayerAsset)
{
	return DataLayerAsset.GetAssetPathString().StartsWith(ConversionFolder);
}