// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentAssetBroker.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Particles/ParticleSystem.h"
#include "Components/AudioComponent.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundBase.h"
#include "Components/ChildActorComponent.h"

//////////////////////////////////////////////////////////////////////////
// FStaticMeshComponentBroker

class FStaticMeshComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UStaticMesh::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(InComponent))
		{
			UStaticMesh* StaticMesh = Cast<UStaticMesh>(InAsset);

			if ((StaticMesh != NULL) || (InAsset == NULL))
			{
				StaticMeshComp->SetStaticMesh(StaticMesh);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(InComponent))
		{
			return StaticMeshComp->GetStaticMesh();
		}
		return NULL;
	}
};

//////////////////////////////////////////////////////////////////////////
// FSkeletalMeshComponentBroker

class FSkeletalMeshComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return USkeletalMesh::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (USkeletalMeshComponent* SkeletalComp = Cast<USkeletalMeshComponent>(InComponent))
		{
			USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InAsset);

			if ((SkeletalMesh != NULL) || (InAsset == NULL))
			{
				SkeletalComp->SetSkeletalMesh(SkeletalMesh);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(InComponent))
		{
			return SkelMeshComp->GetSkeletalMeshAsset();
		}
		return NULL;
	}
};

//////////////////////////////////////////////////////////////////////////
// FParticleSystemComponentBroker

class FParticleSystemComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UParticleSystem::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UParticleSystemComponent* ParticleComp = Cast<UParticleSystemComponent>(InComponent))
		{
			UParticleSystem* ParticleSystem = Cast<UParticleSystem>(InAsset);

			if ((ParticleSystem != NULL) || (InAsset == NULL))
			{
				ParticleComp->SetTemplate(ParticleSystem);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UParticleSystemComponent* ParticleComp = Cast<UParticleSystemComponent>(InComponent))
		{
			return ParticleComp->Template;
		}
		return NULL;
	}
};

//////////////////////////////////////////////////////////////////////////
// FChildActorComponentBroker

class FChildActorComponentBroker : public IComponentAssetBroker
{
public:
	UClass* GetSupportedAssetClass() override
	{
		return UBlueprint::StaticClass();
	}

	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override
	{
		if (UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>(InComponent))
		{
			UClass* Class = Cast<UClass>(InAsset);
			if (Class == nullptr)
			{
				if (UBlueprint* BP = Cast<UBlueprint>(InAsset))
				{
					Class = *(BP->GeneratedClass);
				}
			}
			if (Class && Class->IsChildOf<AActor>())
			{
				ChildActorComp->SetChildActorClass(Class);
				return true;
			}
		}

		return false;
	}

	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override
	{
		if (UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>(InComponent))
		{
			return UBlueprint::GetBlueprintFromClass(*(ChildActorComp->GetChildActorClass()));
		}
		return NULL;
	}
};

//////////////////////////////////////////////////////////////////////////
// FComponentAssetBrokerageage statics

TMap<UClass*, FComponentClassList> FComponentAssetBrokerage::AssetToComponentClassMap;
TMap< TSubclassOf<UActorComponent>, TSharedPtr<IComponentAssetBroker> > FComponentAssetBrokerage::ComponentToBrokerMap;
TMap< UClass*, TArray< TSharedPtr<IComponentAssetBroker> > > FComponentAssetBrokerage::AssetToBrokerMap;

bool FComponentAssetBrokerage::bInitializedBuiltinMap = false;
bool FComponentAssetBrokerage::bShutSystemDown = false;

//////////////////////////////////////////////////////////////////////////
// FComponentAssetBrokerageage

/** Find set of components that support this asset */
FComponentClassList FComponentAssetBrokerage::GetComponentsForAsset(const UObject* InAsset)
{
	InitializeMap();
	FComponentClassList OutClasses;

	if (InAsset != NULL)
	{
		for (UClass* Class = InAsset->GetClass(); Class != UObject::StaticClass(); Class = Class->GetSuperClass())
		{
			if (FComponentClassList* pTypesForClass = AssetToComponentClassMap.Find(Class))
			{
				OutClasses.Append(*pTypesForClass);
			}
		}
	}

	return OutClasses;
}

TSubclassOf<UActorComponent> FComponentAssetBrokerage::GetPrimaryComponentForAsset(UClass* InAssetClass)
{
	InitializeMap();

	if (InAssetClass != NULL)
	{
		for (UClass* Class = InAssetClass; Class != UObject::StaticClass(); Class = Class->GetSuperClass())
		{
			if (FComponentClassList* pTypesForClass = AssetToComponentClassMap.Find(Class))
			{
				if (pTypesForClass->Num() > 0)
				{
					return (*pTypesForClass)[0];
				}
			}
		}
	}

	return NULL;
}

/** Assign the assigned asset to the supplied component */
bool FComponentAssetBrokerage::AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset)
{
	InitializeMap();

	if (InComponent != NULL)
	{
		TSharedPtr<IComponentAssetBroker> Broker = FindBrokerByComponentType(InComponent->GetClass());
		if (Broker.IsValid())
		{
			return Broker->AssignAssetToComponent(InComponent, InAsset);
		}
	}

	return false;
}

UObject* FComponentAssetBrokerage::GetAssetFromComponent(UActorComponent* InComponent)
{
	InitializeMap();

	if (InComponent != NULL)
	{
		TSharedPtr<IComponentAssetBroker> Broker = FindBrokerByComponentType(InComponent->GetClass());
		if (Broker.IsValid())
		{
			return Broker->GetAssetFromComponent(InComponent);
		}
	}

	return NULL;
}

/** See if this component supports assets of any type */
bool FComponentAssetBrokerage::SupportsAssets(UActorComponent* InComponent)
{
	InitializeMap();

	if (InComponent == NULL)
	{
		return false;
	}
	else
	{
		TSharedPtr<IComponentAssetBroker> Broker = FindBrokerByComponentType(InComponent->GetClass());
		return Broker.IsValid();
	}
}

void FComponentAssetBrokerage::PRIVATE_ShutdownBrokerage()
{
	check(!bShutSystemDown);
	bShutSystemDown = true;

	AssetToComponentClassMap.Empty();
	AssetToBrokerMap.Empty();
	ComponentToBrokerMap.Empty();
}

void FComponentAssetBrokerage::InitializeMap()
{
	check(!bShutSystemDown);

	if (!bInitializedBuiltinMap)
	{
		bInitializedBuiltinMap = true;

		RegisterBroker(MakeShareable(new FStaticMeshComponentBroker), UStaticMeshComponent::StaticClass(), true, true);
		RegisterBroker(MakeShareable(new FSkeletalMeshComponentBroker), USkeletalMeshComponent::StaticClass(), true, true);
		RegisterBroker(MakeShareable(new FParticleSystemComponentBroker), UParticleSystemComponent::StaticClass(), true, true);
		RegisterBroker(MakeShareable(new FChildActorComponentBroker), UChildActorComponent::StaticClass(), true, false);
	}
}

void FComponentAssetBrokerage::RegisterBroker(TSharedPtr<IComponentAssetBroker> Broker, TSubclassOf<UActorComponent> InComponentClass, bool bSetAsPrimary, bool bMapComponentForAssets)
{
	InitializeMap();

	check(Broker.IsValid());
	
	UClass* AssetClass = Broker->GetSupportedAssetClass();
	check((AssetClass != NULL) && (AssetClass != UObject::StaticClass()));

	checkf(!ComponentToBrokerMap.Contains(InComponentClass), TEXT("Component class already has a registered broker; you have to chain them yourself."));
	ComponentToBrokerMap.Add(InComponentClass, Broker);

	if (bSetAsPrimary)
	{
		AssetToBrokerMap.FindOrAdd(AssetClass).Insert(Broker, 0);
	}
	else
	{
		AssetToBrokerMap.FindOrAdd(AssetClass).Add(Broker);
	}

	if (bMapComponentForAssets)
	{
		FComponentClassList& ValidComponentTypes = AssetToComponentClassMap.FindOrAdd(AssetClass);
		if (bSetAsPrimary)
		{
			ValidComponentTypes.Insert(InComponentClass, 0);
		}
		else 
		{
			ValidComponentTypes.Add(InComponentClass);
		}
	}
}

void FComponentAssetBrokerage::UnregisterBroker(TSharedPtr<IComponentAssetBroker> Broker)
{
	UClass* AssetClass = Broker->GetSupportedAssetClass();

	if (TArray< TSharedPtr<IComponentAssetBroker> >* pBrokerList = AssetToBrokerMap.Find(AssetClass))
	{
		pBrokerList->Remove(Broker);
	}

	if (FComponentClassList* pTypesForClass = AssetToComponentClassMap.Find(AssetClass))
	{
		for (auto ComponentTypeIt = ComponentToBrokerMap.CreateIterator(); ComponentTypeIt; ++ComponentTypeIt)
		{
			TSubclassOf<UActorComponent> ComponentClass = ComponentTypeIt.Key();

			if (ComponentTypeIt.Value() == Broker)
			{
				ComponentTypeIt.RemoveCurrent();
				pTypesForClass->Remove(ComponentClass);
			}
		}

		if (pTypesForClass->Num() == 0)
		{
			AssetToComponentClassMap.Remove(AssetClass);
		}
	}
}

TSharedPtr<IComponentAssetBroker> FComponentAssetBrokerage::FindBrokerByComponentType(TSubclassOf<UActorComponent> InComponentClass)
{
	InitializeMap();

	return ComponentToBrokerMap.FindRef(InComponentClass);
}


TSharedPtr<IComponentAssetBroker> FComponentAssetBrokerage::FindBrokerByAssetType(UClass* InAssetClass)
{
	InitializeMap();

	TArray< TSharedPtr<IComponentAssetBroker> >* pBrokerList = AssetToBrokerMap.Find(InAssetClass);
	return ((pBrokerList != NULL) && (pBrokerList->Num() > 0)) ? (*pBrokerList)[0] : NULL;
}

TArray<UClass*> FComponentAssetBrokerage::GetSupportedAssets(UClass* InFilterComponentClass)
{
	InitializeMap();

	TArray< UClass* > SupportedAssets;

	for (auto ComponentTypeIt = ComponentToBrokerMap.CreateIterator(); ComponentTypeIt; ++ComponentTypeIt)
	{
		TSubclassOf<UActorComponent> Component = ComponentTypeIt.Key();

		if(InFilterComponentClass == NULL || Component->IsChildOf(InFilterComponentClass))
		{
			SupportedAssets.Add(ComponentTypeIt.Value()->GetSupportedAssetClass());
		}
	}
	
	return SupportedAssets;
}
