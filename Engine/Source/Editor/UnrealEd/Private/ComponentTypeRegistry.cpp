// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentTypeRegistry.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Engine/Blueprint.h"
#include "Components/StaticMeshComponent.h"
#include "TickableEditorObject.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "Materials/Material.h"
#include "Engine/StaticMesh.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ClassIconFinder.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "SComponentClassCombo.h"
#include "Settings/ClassViewerSettings.h"
#include "ClassViewerFilter.h"
#include "EditorClassUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "ComponentTypeRegistry"

namespace UE::Editor::ComponentTypeRegistry::Private
{
	class FUnloadedBlueprintData : public IUnloadedBlueprintData
	{
	public:
		FUnloadedBlueprintData(const FAssetData& InAssetData)
			:ClassPath()
			,ParentClassPath()
			,ClassFlags(CLASS_None)
			,bIsNormalBlueprintType(false)
		{
			ClassName = MakeShared<FString>(InAssetData.AssetName.ToString());

			FString GeneratedClassPath;
			const UClass* AssetClass = InAssetData.GetClass();
			if (AssetClass && AssetClass->IsChildOf(UBlueprintGeneratedClass::StaticClass()))
			{
				ClassPath = InAssetData.GetSoftObjectPath().GetAssetPath();
			}
			else if (InAssetData.GetTagValue(FBlueprintTags::GeneratedClassPath, GeneratedClassPath))
			{
				ClassPath = FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(GeneratedClassPath));
			}

			FString ParentClassPathString;
			if (InAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassPathString))
			{
				ParentClassPath = FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(ParentClassPathString));
			}

			FEditorClassUtils::GetImplementedInterfaceClassPathsFromAsset(InAssetData, ImplementedInterfaces);
		}

		virtual ~FUnloadedBlueprintData()
		{
		}

		// Begin IUnloadedBlueprintData interface
		virtual bool HasAnyClassFlags(uint32 InFlagsToCheck) const
		{
			return (ClassFlags & InFlagsToCheck) != 0;
		}

		virtual bool HasAllClassFlags(uint32 InFlagsToCheck) const
		{
			return ((ClassFlags & InFlagsToCheck) == InFlagsToCheck);
		}

		virtual void SetClassFlags(uint32 InFlags)
		{
			ClassFlags = InFlags;
		}

		virtual bool ImplementsInterface(const UClass* InInterface) const
		{
			FString InterfacePath = InInterface->GetPathName();
			for (const FString& ImplementedInterface : ImplementedInterfaces)
			{
				if (ImplementedInterface == InterfacePath)
				{
					return true;
				}
			}

			FComponentClassComboEntryPtr CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(ParentClassPath);
			while (CurrentEntry.IsValid())
			{
				if (const UClass* CurrentClass = CurrentEntry->GetComponentClass())
				{
					return CurrentClass->ImplementsInterface(InInterface);
				}

				TSharedPtr<FUnloadedBlueprintData> UnloadedBlueprintData = StaticCastSharedPtr<FUnloadedBlueprintData>(CurrentEntry->GetUnloadedBlueprintData());
				if (UnloadedBlueprintData.IsValid())
				{
					for (const FString& ImplementedInterface : UnloadedBlueprintData->ImplementedInterfaces)
					{
						if (ImplementedInterface == InterfacePath)
						{
							return true;
						}
					}

					CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(UnloadedBlueprintData->ParentClassPath);
				}
				else
				{
					CurrentEntry.Reset();
				}
			}

			return false;
		}

		virtual bool IsChildOf(const UClass* InClass) const
		{
			FComponentClassComboEntryPtr CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(ParentClassPath);
			while (CurrentEntry.IsValid())
			{
				if (const UClass* CurrentClass = CurrentEntry->GetComponentClass())
				{
					return CurrentClass->IsChildOf(InClass);
				}

				TSharedPtr<FUnloadedBlueprintData> UnloadedBlueprintData = StaticCastSharedPtr<FUnloadedBlueprintData>(CurrentEntry->GetUnloadedBlueprintData());
				if (UnloadedBlueprintData.IsValid())
				{
					CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(UnloadedBlueprintData->ParentClassPath);
				}
				else
				{
					CurrentEntry.Reset();
				}
			}

			return false;
		}

		virtual bool IsA(const UClass* InClass) const
		{
			// Unloaded blueprint classes should always be a BPGC, so this just checks against the expected type.
			return UBlueprintGeneratedClass::StaticClass()->UObject::IsA(InClass);
		}

		virtual const UClass* GetClassWithin() const
		{
			FComponentClassComboEntryPtr CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(ParentClassPath);
			while (CurrentEntry.IsValid())
			{
				if (const UClass* CurrentClass = CurrentEntry->GetComponentClass())
				{
					return CurrentClass->ClassWithin;
				}

				TSharedPtr<FUnloadedBlueprintData> UnloadedBlueprintData = StaticCastSharedPtr<FUnloadedBlueprintData>(CurrentEntry->GetUnloadedBlueprintData());
				if (UnloadedBlueprintData.IsValid())
				{
					CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(UnloadedBlueprintData->ParentClassPath);
				}
				else
				{
					CurrentEntry.Reset();
				}
			}

			return nullptr;
		}

		virtual const UClass* GetNativeParent() const
		{
			const UClass* CurrentClass = nullptr;
			FComponentClassComboEntryPtr CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(ParentClassPath);
			while (CurrentEntry.IsValid() || CurrentClass)
			{
				if (!CurrentClass)
				{
					CurrentClass = CurrentEntry->GetComponentClass();
				}

				if (CurrentClass)
				{
					if (CurrentClass->HasAnyClassFlags(CLASS_Native))
					{
						return CurrentClass;
					}
					else
					{
						CurrentClass = CurrentClass->GetSuperClass();
					}
				}
				else
				{
					TSharedPtr<FUnloadedBlueprintData> UnloadedBlueprintData = StaticCastSharedPtr<FUnloadedBlueprintData>(CurrentEntry->GetUnloadedBlueprintData());
					if (UnloadedBlueprintData.IsValid())
					{
						CurrentEntry = FComponentTypeRegistry::Get().FindClassEntryForObjectPath(UnloadedBlueprintData->ParentClassPath);
					}
					else
					{
						CurrentEntry.Reset();
					}
				}
			}

			return nullptr;
		}

		virtual void SetNormalBlueprintType(bool bInNormalBPType)
		{
			bIsNormalBlueprintType = bInNormalBPType;
		}

		virtual bool IsNormalBlueprintType() const
		{
			return bIsNormalBlueprintType;
		}

		virtual TSharedPtr<FString> GetClassName() const
		{
			return ClassName;
		}

		virtual FTopLevelAssetPath GetClassPathName() const override
		{
			return ClassPath;
		}
		// End IUnloadedBlueprintData interface

	private:
		TSharedPtr<FString> ClassName;
		FTopLevelAssetPath ClassPath;
		FTopLevelAssetPath ParentClassPath;
		uint32 ClassFlags;
		TArray<FString> ImplementedInterfaces;
		bool bIsNormalBlueprintType;
	};
}

//////////////////////////////////////////////////////////////////////////
// FComponentTypeRegistryData

struct FComponentTypeRegistryData
	: public FTickableEditorObject
	, public FGCObject
{
	FComponentTypeRegistryData();

	// Force a refresh of the components list right now (also calls the ComponentListChanged delegate to notify watchers)
	void ForceRefreshComponentList();

	static void AddBasicShapeComponents(TArray<FComponentClassComboEntryPtr>& SortedClassList);

	/** Implementation of FTickableEditorObject */
	virtual void Tick(float) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual bool IsTickable() const override 
	{ 
		// Only necessary to tick when in editor. However,
		// avoid ticking until the AssetRegistry is done gathering otherwise 
		// we can introduce excessive amounts of contention and cache updates when ticking.
		return GUnrealEd != nullptr && !AssetRegistryModule.Get().IsGathering();
	}
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FTypeDatabaseUpdater, STATGROUP_Tickables); }
	
	/** Implementation of FGCObject */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return "FComponentTypeRegistryData";
	}
	
	// Request a refresh of the components list next frame
	void Invalidate()
	{
		bNeedsRefreshNextTick = true;
	}
public:
	/** End implementation of FTickableEditorObject */
	TArray<FComponentClassComboEntryPtr> ComponentClassList;
	TArray<FComponentTypeEntry> ComponentTypeList;
	TArray<FAssetData> PendingAssetData;
	TMap<FTopLevelAssetPath, int32> ClassPathToClassListIndexMap;
	FOnComponentTypeListChanged ComponentListChanged;
	FAssetRegistryModule& AssetRegistryModule;
	TSAN_ATOMIC(bool) bNeedsRefreshNextTick;
};

static const FString CommonClassGroup(TEXT("Common"));
// This has to stay in sync with logic in FKismetCompilerContext::FinishCompilingClass
static const FString BlueprintComponents(TEXT("Custom"));
static const FName BPParentClassName(GET_MEMBER_NAME_CHECKED(UBlueprint, ParentClass));

template <typename ObjectType>
static ObjectType* FindOrLoadObject( const FString& ObjectPath )
{
	ObjectType* Object = FindObject<ObjectType>( nullptr, *ObjectPath );
	if( !Object )
	{
		Object = LoadObject<ObjectType>( nullptr, *ObjectPath );
	}

	return Object;
}

void FComponentTypeRegistryData::AddBasicShapeComponents(TArray<FComponentClassComboEntryPtr>& SortedClassList)
{
	FString BasicShapesHeading = LOCTEXT("BasicShapesHeading", "Basic Shapes").ToString();

	const auto OnBasicShapeCreated = [](FSubobjectDataHandle ComponentHandle)
	{
		FSubobjectData* Data = ComponentHandle.GetData();
		// TODO const cast is bad practice, but until the subobject refactor it is only way for internal stuff to get mutable components
		UStaticMeshComponent* SMC = const_cast<UStaticMeshComponent*>(Cast<UStaticMeshComponent>(Data->GetComponentTemplate()));
		if (SMC)
		{
			const FString MaterialName = TEXT("/Engine/BasicShapes/BasicShapeMaterial.BasicShapeMaterial");
			UMaterial* MaterialAsset = FindOrLoadObject<UMaterial>(MaterialName);
			SMC->SetMaterial(0, MaterialAsset);

			// If the component object is an archetype (template), propagate the material setting to any instances, as instances
			// of the archetype will end up being created BEFORE we are able to set the override material on the template object.
			if (SMC->HasAnyFlags(RF_ArchetypeObject))
			{
				TArray<UObject*> ArchetypeInstances;
				SMC->GetArchetypeInstances(ArchetypeInstances);
				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					CastChecked<UStaticMeshComponent>(ArchetypeInstance)->SetMaterial(0, MaterialAsset);
				}
			}
		}
	};

	{
		FComponentEntryCustomizationArgs Args;
		Args.AssetOverride = FindOrLoadObject<UStaticMesh>(UActorFactoryBasicShape::BasicCube.ToString());
		Args.OnSubobjectCreated = FOnSubobjectCreated::CreateStatic(OnBasicShapeCreated);
		Args.ComponentNameOverride = LOCTEXT("BasicCubeShapeDisplayName", "Cube").ToString();
		Args.IconOverrideBrushName = FName("ClassIcon.Cube");
		Args.SortPriority = 2;

		{
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(BasicShapesHeading, UStaticMeshComponent::StaticClass(), true, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}

		{
			//Cube also goes in the common group
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(CommonClassGroup, UStaticMeshComponent::StaticClass(), false, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}
	}

	{
		FComponentEntryCustomizationArgs Args;
		Args.AssetOverride = FindOrLoadObject<UStaticMesh>(UActorFactoryBasicShape::BasicPlane.ToString());
		Args.OnSubobjectCreated = FOnSubobjectCreated::CreateStatic(OnBasicShapeCreated);
		Args.ComponentNameOverride = LOCTEXT("BasicPlaneShapeDisplayName", "Plane").ToString();
		Args.IconOverrideBrushName = FName("ClassIcon.Plane");
		Args.SortPriority = 2;

		{
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(BasicShapesHeading, UStaticMeshComponent::StaticClass(), true, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}

		{
			//Quad also goes in the common group
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(CommonClassGroup, UStaticMeshComponent::StaticClass(), false, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}
	}

	{
		FComponentEntryCustomizationArgs Args;
		Args.AssetOverride = FindOrLoadObject<UStaticMesh>(UActorFactoryBasicShape::BasicSphere.ToString());
		Args.OnSubobjectCreated = FOnSubobjectCreated::CreateStatic(OnBasicShapeCreated);
		Args.ComponentNameOverride = LOCTEXT("BasicSphereShapeDisplayName", "Sphere").ToString();
		Args.IconOverrideBrushName = FName("ClassIcon.Sphere");
		Args.SortPriority = 2;
		{
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(BasicShapesHeading, UStaticMeshComponent::StaticClass(), true, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}

		{
			// Sphere also goes in the common group
			FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(CommonClassGroup, UStaticMeshComponent::StaticClass(), false, EComponentCreateAction::SpawnExistingClass, Args));
			SortedClassList.Add(NewShape);
		}
	}

	{
		FComponentEntryCustomizationArgs Args;
		Args.AssetOverride = FindOrLoadObject<UStaticMesh>(UActorFactoryBasicShape::BasicCylinder.ToString());
		Args.OnSubobjectCreated = FOnSubobjectCreated::CreateStatic(OnBasicShapeCreated);
		Args.ComponentNameOverride = LOCTEXT("BasicCylinderShapeDisplayName", "Cylinder").ToString();
		Args.IconOverrideBrushName = FName("ClassIcon.Cylinder");
		Args.SortPriority = 3;
		FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(BasicShapesHeading, UStaticMeshComponent::StaticClass(), true, EComponentCreateAction::SpawnExistingClass, Args));
		SortedClassList.Add(NewShape);
	}

	{
		FComponentEntryCustomizationArgs Args;
		Args.AssetOverride = FindOrLoadObject<UStaticMesh>(UActorFactoryBasicShape::BasicCone.ToString());
		Args.OnSubobjectCreated = FOnSubobjectCreated::CreateStatic(OnBasicShapeCreated);
		Args.ComponentNameOverride = LOCTEXT("BasicConeShapeDisplayName", "Cone").ToString();
		Args.IconOverrideBrushName = FName("ClassIcon.Cone");
		Args.SortPriority = 4;
		FComponentClassComboEntryPtr NewShape = MakeShareable(new FComponentClassComboEntry(BasicShapesHeading, UStaticMeshComponent::StaticClass(), true, EComponentCreateAction::SpawnExistingClass, Args));
		SortedClassList.Add(NewShape);
	}
}

FComponentTypeRegistryData::FComponentTypeRegistryData()
	: AssetRegistryModule(FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")))
	, bNeedsRefreshNextTick(false)
{
	const auto HandleAdded = [](const FAssetData& Data, FComponentTypeRegistryData* Parent)
	{
		// Only add to pending array if this might actually be a blueprint
		if (Data.FindTag(BPParentClassName))
		{
			Parent->PendingAssetData.Push(Data);
		}
	};

	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	AssetRegistry.OnAssetAdded().AddStatic(HandleAdded, this);
	AssetRegistry.OnAssetRemoved().AddStatic(HandleAdded, this);
	AssetRegistry.OnKnownGathersComplete().AddLambda([](FComponentTypeRegistryData* Data) { Data->Invalidate(); }, this);

	const auto HandleRenamed = [](const FAssetData& Data, const FString&, FComponentTypeRegistryData* Parent)
	{
		if (Data.FindTag(BPParentClassName))
		{
			Parent->PendingAssetData.Push(Data);
		}
	};
	AssetRegistry.OnAssetRenamed().AddStatic(HandleRenamed, this);
}

void FComponentTypeRegistryData::ForceRefreshComponentList()
{
	if (!IsTickable())
	{
		// If we aren't tickable, don't refresh for the same reasons why we don't tick. This early out is 
		// necessary should ForceRefreshComponentList be called explicitly rather than Invalidate().
		return;
	}

	bNeedsRefreshNextTick = false;
	ComponentClassList.Empty();
	ComponentTypeList.Empty();
	ClassPathToClassListIndexMap.Empty();

	struct SortComboEntry
	{
		bool operator () (const FComponentClassComboEntryPtr& A, const FComponentClassComboEntryPtr& B) const
		{
			bool bResult = false;

			// check headings first, if they are the same compare the individual entries
			int32 HeadingCompareResult = FCString::Stricmp(*A->GetHeadingText(), *B->GetHeadingText());
			if (HeadingCompareResult == 0)
			{
				if( A->GetSortPriority() == 0 && B->GetSortPriority() == 0 )
				{
					bResult = FCString::Stricmp(*A->GetClassDisplayName(), *B->GetClassDisplayName()) < 0;
				}
				else
				{
					bResult = A->GetSortPriority() < B->GetSortPriority();
				}
			}
			else if (CommonClassGroup == A->GetHeadingText())
			{
				bResult = true;
			}
			else if (CommonClassGroup == B->GetHeadingText())
			{
				bResult = false;
			}
			else
			{
				bResult = HeadingCompareResult < 0;
			}

			return bResult;
		}
	};

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	{
		FString NewComponentsHeading = LOCTEXT("NewComponentsHeading", "Scripting").ToString();
		// Add new C++ component class
		FComponentClassComboEntryPtr NewClassHeader = MakeShareable(new FComponentClassComboEntry(NewComponentsHeading));
		ComponentClassList.Add(NewClassHeader);

		FComponentClassComboEntryPtr NewBPClass = MakeShareable(new FComponentClassComboEntry(NewComponentsHeading, UActorComponent::StaticClass(), true, EComponentCreateAction::CreateNewBlueprintClass));
		ComponentClassList.Add(NewBPClass);

		if (GUnrealEd && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
		{
			FComponentClassComboEntryPtr NewCPPClass = MakeShareable(new FComponentClassComboEntry(NewComponentsHeading, UActorComponent::StaticClass(), true, EComponentCreateAction::CreateNewCPPClass));
			ComponentClassList.Add(NewCPPClass);
		}

		FComponentClassComboEntryPtr NewSeparator(new FComponentClassComboEntry());
		ComponentClassList.Add(NewSeparator);
	}

	TArray<FComponentClassComboEntryPtr> SortedClassList;

	AddBasicShapeComponents(SortedClassList);

	// Add loaded component classes
	TSet<FTopLevelAssetPath> InMemoryClassPaths;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		// If this is a subclass of Actor Component, not abstract, and tagged as spawnable from Kismet
		if (Class->IsChildOf(UActorComponent::StaticClass()))
		{
			InMemoryClassPaths.Add(Class->GetClassPathName());

			const bool bOutOfDateClass = Class->HasAnyClassFlags(CLASS_NewerVersionExists);
			const bool bBlueprintSkeletonClass = FKismetEditorUtilities::IsClassABlueprintSkeleton(Class);
			const bool bPackageForDiffing = Class->GetPackage()->HasAnyPackageFlags(PKG_ForDiffing);

			if (!bOutOfDateClass &&	!bBlueprintSkeletonClass && !bPackageForDiffing)
			{
				if (FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(Class))
				{
					TArray<FString> ClassGroupNames;
					Class->GetClassGroupNames(ClassGroupNames);

					if (ClassGroupNames.Contains(CommonClassGroup))
					{
						FString ClassGroup = CommonClassGroup;
						FComponentClassComboEntryPtr NewEntry(new FComponentClassComboEntry(ClassGroup, Class, ClassGroupNames.Num() <= 1, EComponentCreateAction::SpawnExistingClass));
						SortedClassList.Add(NewEntry);
					}
					if (ClassGroupNames.Num() && !ClassGroupNames[0].Equals(CommonClassGroup))
					{
						const bool bIncludeInFilter = true;

						FString ClassGroup = ClassGroupNames[0];
						FComponentClassComboEntryPtr NewEntry(new FComponentClassComboEntry(ClassGroup, Class, bIncludeInFilter, EComponentCreateAction::SpawnExistingClass));
						SortedClassList.Add(NewEntry);
					}
					else if (ClassGroupNames.Num() == 0)
					{
						// No class group name found. Just add it to a "custom" category

						const bool bIncludeInFilter = true;
						FString ClassGroup = LOCTEXT("CustomClassGroup", "Custom").ToString();
						FComponentClassComboEntryPtr NewEntry(new FComponentClassComboEntry(ClassGroup, Class, bIncludeInFilter, EComponentCreateAction::SpawnExistingClass));
						SortedClassList.Add(NewEntry);
					}
				}

				FComponentTypeEntry Entry = { Class->GetName(), FString(), ObjectPtrWrap(Class) };
				ComponentTypeList.Add(MoveTemp(Entry));
			}
		}
	}

	// Add unloaded component classes
	{
		auto AddUnloadedComponentClass = [this, &SortedClassList](const FTopLevelAssetPath& ClassPath, const FAssetData& AssetData)
		{
			// The blueprint is unloaded, so we need to work out which icon to use for it using its asset data
			const UClass* IconClass = FClassIconFinder::GetIconClassForAssetData(AssetData);

			FString ClassName = ClassPath.GetAssetName().ToString();
			ClassName.RemoveFromEnd(TEXT("_C"));

			FComponentClassComboEntryPtr NewEntry(new FComponentClassComboEntry(BlueprintComponents, ClassName, ClassPath, IconClass, /*bIncludeInFilter=*/true));
			SortedClassList.Add(NewEntry);

			// Create an unloaded blueprint data object to assist with class filtering
			{
				using namespace UE::Editor::ComponentTypeRegistry;
				TSharedPtr<IUnloadedBlueprintData> UnloadedBlueprintData = MakeShared<Private::FUnloadedBlueprintData>(AssetData);

				const uint32 ClassFlags = AssetData.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
				UnloadedBlueprintData->SetClassFlags(ClassFlags);

				const FString BlueprintType = AssetData.GetTagValueRef<FString>(FBlueprintTags::BlueprintType);
				UnloadedBlueprintData->SetNormalBlueprintType(BlueprintType == TEXT("BPType_Normal"));

				NewEntry->SetUnloadedBlueprintData(UnloadedBlueprintData);
			}

			FComponentTypeEntry Entry = { MoveTemp(ClassName), ClassPath.ToString(), nullptr };
			ComponentTypeList.Add(MoveTemp(Entry));
		};

		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		TSet<FTopLevelAssetPath> ActorComponentDerivedClassNames;
		{
			TArray<FTopLevelAssetPath> ActorComponentClassNames;
			ActorComponentClassNames.Add(UActorComponent::StaticClass()->GetClassPathName());
			AssetRegistry.GetDerivedClassNames(ActorComponentClassNames, TSet<FTopLevelAssetPath>(), ActorComponentDerivedClassNames);
		}

		// GetAssetsByClass call is a kludge to get the full asset paths for the blueprints we care about
		// Bob T. thinks that the Asset Registry could just keep asset paths
		TArray<FAssetData> Assets;

		AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), Assets, true);
		for (FAssetData& BPAsset : Assets)
		{
			constexpr bool bGenerateClassPathIfMissing = true;
			FTopLevelAssetPath ClassPath(FEditorClassUtils::GetClassPathNameFromAsset(BPAsset, bGenerateClassPathIfMissing));
			FSoftObjectPath SoftClassPath(ClassPath);
			if (SoftClassPath.FixupCoreRedirects() && SoftClassPath.GetSubPathString().IsEmpty())
			{
				ClassPath = FTopLevelAssetPath(SoftClassPath.GetLongPackageFName(), SoftClassPath.GetAssetFName());
			}
			if (!ClassPath.IsNull())
			{
				if (!InMemoryClassPaths.Contains(ClassPath)
					&& ActorComponentDerivedClassNames.Contains(ClassPath)
					&& !BPAsset.HasAnyPackageFlags(PKG_ForDiffing))
				{
					AddUnloadedComponentClass(ClassPath, BPAsset);
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Warning, "Blueprint %ls is missing %ls asset tag", *BPAsset.PackageName.ToString(), *FBlueprintTags::GeneratedClassPath.ToString());
			}

		}

		Assets.Reset();
		AssetRegistry.GetAssetsByClass(UBlueprintGeneratedClass::StaticClass()->GetClassPathName(), Assets, true);
		for (FAssetData& BPGCAsset : Assets)
		{
			FTopLevelAssetPath BPGCAssetClassPathName(BPGCAsset.PackagePath, BPGCAsset.AssetName);
			if (ActorComponentDerivedClassNames.Contains(BPGCAssetClassPathName) && !InMemoryClassPaths.Contains(BPGCAssetClassPathName)
				&& !BPGCAsset.HasAnyPackageFlags(PKG_ForDiffing))
			{
				AddUnloadedComponentClass(BPGCAssetClassPathName, BPGCAsset);
			}
		}
	}
	
	if (SortedClassList.Num() > 0)
	{
		Algo::Sort(SortedClassList, SortComboEntry());

		FString PreviousHeading;
		for (int32 ClassIndex = 0; ClassIndex < SortedClassList.Num(); ClassIndex++)
		{
			FComponentClassComboEntryPtr& CurrentEntry = SortedClassList[ClassIndex];

			const FString& CurrentHeadingText = CurrentEntry->GetHeadingText();
			if (CurrentHeadingText != PreviousHeading)
			{
				// This avoids a redundant separator being added to the very top of the list
				if (ClassIndex > 0)
				{
					FComponentClassComboEntryPtr NewSeparator(new FComponentClassComboEntry());
					ComponentClassList.Add(NewSeparator);
				}
				FComponentClassComboEntryPtr NewHeading(new FComponentClassComboEntry(CurrentHeadingText));
				ComponentClassList.Add(NewHeading);

				PreviousHeading = CurrentHeadingText;
			}

			int32 EntryIndex = ComponentClassList.Add(CurrentEntry);
			if (CurrentEntry->IsClass())
			{
				ClassPathToClassListIndexMap.FindOrAdd(FTopLevelAssetPath(CurrentEntry->GetComponentPath()), EntryIndex);
			}
		}
	}

	ComponentListChanged.Broadcast();
}

void FComponentTypeRegistryData::Tick(float)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FComponentTypeRegistryData::Tick);

	bool bRequiresRefresh = bNeedsRefreshNextTick;
	if (!bRequiresRefresh && PendingAssetData.Num() != 0)
	{
		TArray<FTopLevelAssetPath> ClassNames;
		ClassNames.Add(UActorComponent::StaticClass()->GetClassPathName());
		TSet<FTopLevelAssetPath> DerivedClassNames;
		AssetRegistryModule.Get().GetDerivedClassNames(ClassNames, TSet<FTopLevelAssetPath>(), DerivedClassNames);

		for (const FAssetData& Asset : PendingAssetData)
		{
			const FString TagValue = Asset.GetTagValueRef<FString>(BPParentClassName);
			const FTopLevelAssetPath ObjectPath(FPackageName::ExportTextPathToObjectPath(TagValue));
			if (DerivedClassNames.Contains(ObjectPath))
			{
				bRequiresRefresh = true;
				break;
			}
		}
		PendingAssetData.Empty();
	}

	if (bRequiresRefresh)
	{
		PendingAssetData.Empty();
		ForceRefreshComponentList();
	}
}

void FComponentTypeRegistryData::AddReferencedObjects(FReferenceCollector& Collector)
{
	for(FComponentClassComboEntryPtr& ComboEntry : ComponentClassList)
	{
		ComboEntry->AddReferencedObjects(Collector);
	}

	for(FComponentTypeEntry& TypeEntry : ComponentTypeList)
	{
		Collector.AddReferencedObject(TypeEntry.ComponentClass);
	}
}

//////////////////////////////////////////////////////////////////////////
// FComponentTypeRegistry

FComponentTypeRegistry& FComponentTypeRegistry::Get()
{
	static FComponentTypeRegistry ComponentRegistry;
	return ComponentRegistry;
}

FOnComponentTypeListChanged& FComponentTypeRegistry::SubscribeToComponentList(TArray<FComponentClassComboEntryPtr>*& OutComponentList)
{
	check(Data);
	OutComponentList = &Data->ComponentClassList;
	return Data->ComponentListChanged;
}

FOnComponentTypeListChanged& FComponentTypeRegistry::SubscribeToComponentList(const TArray<FComponentTypeEntry>*& OutComponentList)
{
	check(Data);
	OutComponentList = &Data->ComponentTypeList;
	return Data->ComponentListChanged;
}

FOnComponentTypeListChanged& FComponentTypeRegistry::GetOnComponentTypeListChanged()
{
	check(Data);
	return Data->ComponentListChanged;
}

FComponentTypeRegistry::FComponentTypeRegistry()
	: Data(nullptr)
{
	Data = new FComponentTypeRegistryData();

	// This will load the assets on next tick. It's not safe to do right now because we could be deep in a stack
	Data->Invalidate();

	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FComponentTypeRegistry::OnReloadComplete);

	// Clear references when unloaded otherwise unload will fail and leak the object
	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.AddRaw(this, &FComponentTypeRegistry::OnBlueprintGeneratedClassUnloaded);
}

FComponentTypeRegistry::~FComponentTypeRegistry()
{
	FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);
	FKismetEditorUtilities::OnBlueprintGeneratedClassUnloaded.RemoveAll(this);
}

void FComponentTypeRegistry::OnReloadComplete(EReloadCompleteReason Reason)
{
	Data->ForceRefreshComponentList();
}

void FComponentTypeRegistry::OnBlueprintGeneratedClassUnloaded(UBlueprintGeneratedClass* BlueprintGeneratedClass)
{
	if (!BlueprintGeneratedClass)
	{
		return;
	}

	if (!BlueprintGeneratedClass->IsChildOf(UActorComponent::StaticClass()))
	{
		return;
	}

	bool bModified = false;
	for (FComponentClassComboEntryPtr& ComboEntry : Data->ComponentClassList)
	{
		if (ComboEntry->OnBlueprintGeneratedClassUnloaded(BlueprintGeneratedClass))
		{
			bModified = true;
		}
	}

	for (FComponentTypeEntry& ComponentTypeEntry : Data->ComponentTypeList)
	{
		if (ComponentTypeEntry.ComponentClass == BlueprintGeneratedClass)
		{
			ComponentTypeEntry.ComponentClass = nullptr;
			bModified = true;
		}
	}

	if (bModified)
	{
		Data->bNeedsRefreshNextTick = true;
	}
}

void FComponentTypeRegistry::InvalidateClass(TSubclassOf<UActorComponent> /*ClassToUpdate*/)
{
	Data->Invalidate();
}

void FComponentTypeRegistry::Invalidate()
{
	Data->Invalidate();
}

FComponentClassComboEntryPtr FComponentTypeRegistry::FindClassEntryForObjectPath(FTopLevelAssetPath InObjectPath) const
{
	if (int32* ClassListIndexPtr = Data->ClassPathToClassListIndexMap.Find(InObjectPath))
	{
		const int32 ClassListIndex = *ClassListIndexPtr;
		if (Data->ComponentClassList.IsValidIndex(ClassListIndex))
		{
			return Data->ComponentClassList[ClassListIndex];
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
