// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentVisualizer.h"

#include "ActorEditorUtils.h"
#include "Components/ChildActorComponent.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentVisualizer)

IMPLEMENT_HIT_PROXY(HComponentVisProxy, HHitProxy);

namespace ComponentVisualizers
{
	static bool GAutoSelectComponentWithPointSelection = true;
	static FAutoConsoleVariableRef CVarAutoSelectComponentWithPointSelection(
		TEXT("Editor.ComponentVisualizer.AutoSelectComponent"),
		GAutoSelectComponentWithPointSelection,
		TEXT("Automatically adds the visualized component to the selection set if "
			"available when visualization handles a click (e.g. select spline component when a point is selected on the spline).")
	);

}; // namespace ComponentVisualizers

static FPropertyNameAndIndex GetActorPropertyNameAndIndexForComponent(const AActor* Actor, const UActorComponent* Component)
{
	if (Actor != nullptr && Component != nullptr)
	{
			// Iterate over UObject* fields of this actor
		UClass* ActorClass = Actor->GetClass();
		for (TFieldIterator<FObjectProperty> It(ActorClass); It; ++It)
		{
			// See if this property points to the component in question
			FObjectProperty* ObjectProp = *It;
			for (int32 Index = 0; Index < ObjectProp->ArrayDim; ++Index)
			{
				UObject* Object = ObjectProp->GetObjectPropertyValue(ObjectProp->ContainerPtrToValuePtr<void>(Actor, Index));
				if (Object == Component)
				{
					// It does! Return this name
					return FPropertyNameAndIndex(ObjectProp->GetFName(), Index);
				}
			}
		}

		// If nothing found, look in TArray<UObject*> fields
		for (TFieldIterator<FArrayProperty> It(ActorClass); It; ++It)
		{
			FArrayProperty* ArrayProp = *It;
			if (FObjectProperty* InnerProp = CastField<FObjectProperty>(It->Inner))
			{
				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Actor));
				for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
				{
					UObject* Object = InnerProp->GetObjectPropertyValue(ArrayHelper.GetRawPtr(Index));
					if (Object == Component)
					{
						return FPropertyNameAndIndex(ArrayProp->GetFName(), Index);
					}
				}
			}
		}
	}

	return FPropertyNameAndIndex();
}


void FComponentPropertyPath::Set(const UActorComponent* Component)
{
	// Determine which property on the component's actor owner references the component.
	AActor* Actor = Component->GetOwner();

	// If such a property were found, build a chain of such properties, recursing up actor parent components,
	// until we reach the top
	UChildActorComponent* ParentComponent = Actor->GetParentComponent();
	if (ParentComponent)
	{
		Set(ParentComponent);	// recurse to next parent
	}
	else
	{
		// If there are no further parents, store this one (the outermost)
		ParentOwningActor = Actor;

		// We have successfully arrived at the top of the actor/component tree, and have a valid property chain.
		// Hence we don't need to cache the current component ptr as a last resort.
		LastResortComponentPtr = nullptr;
	}

	// If a last resort component ptr has been set, no need to build the property chain
	if (LastResortComponentPtr.IsValid())
	{
		return;
	}

	FPropertyNameAndIndex PropertyNameAndIndex = GetActorPropertyNameAndIndexForComponent(Actor, Component);

	if (PropertyNameAndIndex.IsValid())
	{
		// If we found a property, add it to the chain after the recursion, so they are added outermost-first.
		PropertyChain.Add(PropertyNameAndIndex);
	}
	else
	{
		// If no such property were found, we set a "last resort" weak ptr to the component itself and get rid of the property chain.
		// This is not preferable as we can't recuperate the component if its address changes (e.g. on hot reload or BP reconstruction).
		// However it is valid to have an owned component without a UPROPERTY reference, so we need to handle this case.
		LastResortComponentPtr = const_cast<UActorComponent*>(Component);
		PropertyChain.Empty();
	}
}


UActorComponent* FComponentPropertyPath::GetComponent() const
{
	// If there's a valid "last resort" component ptr, use this. 
	if (LastResortComponentPtr.IsValid())
	{
		return LastResortComponentPtr.Get();
	}

	UActorComponent* Result = nullptr;
	const AActor* Actor = ParentOwningActor.Get();
	if (Actor)
	{
		int32 Level = 0;
		for (const FPropertyNameAndIndex& PropertyNameAndIndex : PropertyChain)
		{
			Result = nullptr;

			if (PropertyNameAndIndex.IsValid())
			{
				FName PropertyName = PropertyNameAndIndex.Name;
				int32 PropertyIndex = PropertyNameAndIndex.Index;

				UClass* ActorClass = Actor->GetClass();
				check(ActorClass);
				FProperty* Prop = FindFProperty<FProperty>(ActorClass, PropertyName);

				if (FObjectProperty* ObjectProp = CastField<FObjectProperty>(Prop))
				{
					UObject* Object = ObjectProp->GetObjectPropertyValue(ObjectProp->ContainerPtrToValuePtr<void>(Actor, PropertyIndex));
					Result = Cast<UActorComponent>(Object);
				}
				else if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
				{
					if (FObjectProperty* InnerProp = CastField<FObjectProperty>(ArrayProp->Inner))
					{
						FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Actor));
						if (ArrayHelper.IsValidIndex(PropertyIndex))
						{
							UObject* Object = InnerProp->GetObjectPropertyValue(ArrayHelper.GetRawPtr(PropertyIndex));
							Result = Cast<UActorComponent>(Object);
						}
					}
				}
			}

			Level++;

			if (Level < PropertyChain.Num())
			{
				UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(Result);
				if (ChildActorComponent == nullptr)
				{
					break;
				}

				Actor = ChildActorComponent->GetChildActor();
				if (Actor == nullptr)
				{
					break;
				}
			}
		}
	}

	return Result;
}


bool FComponentPropertyPath::IsValid() const
{
	// If there's a valid "last resort" component, this will always be valid.
	if (LastResortComponentPtr.IsValid())
	{
		return true;
	}

	if (!ParentOwningActor.IsValid())
	{
		return false;
	}

	for (const FPropertyNameAndIndex& PropertyNameAndIndex : PropertyChain)
	{
		if (!PropertyNameAndIndex.IsValid())
		{
			return false;
		}
	}

	return true;
}


bool FComponentVisualizer::ShouldAutoSelectElementOnHandleClick() const
{
	return ComponentVisualizers::GAutoSelectComponentWithPointSelection;
}

void FComponentVisualizer::NotifyPropertyModified(UActorComponent* Component, FProperty* Property, EPropertyChangeType::Type PropertyChangeType)
{
	TArray<FProperty*> Properties;
	Properties.Add(Property);
	NotifyPropertiesModified(Component, Properties, PropertyChangeType);
}

void FComponentVisualizer::NotifyPropertiesModified(UActorComponent* Component, const TArray<FProperty*>& Properties, EPropertyChangeType::Type PropertyChangeType)
{
	if (Component == nullptr)
	{
		return;
	}

	for (FProperty* Property : Properties)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, PropertyChangeType);
		Component->PostEditChangeProperty(PropertyChangedEvent);
	}

	AActor* Owner = Component->GetOwner();

	if (Owner && FActorEditorUtils::IsAPreviewOrInactiveActor(Owner))
	{
		// The component belongs to the preview actor in the BP editor, so we need to propagate the property change to the archetype.
		// Before this, we exploit the fact that the archetype and the preview actor have the old and new value of the property, respectively.
		// So we can go through all archetype instances, and if they hold the (old) archetype value, update it to the new value.

		// Get archetype
		UActorComponent* Archetype = Cast<UActorComponent>(Component->GetArchetype());
		check(Archetype);

		// Get all archetype instances (the preview actor passed in should be amongst them)
		TArray<UObject*> ArchetypeInstances;
		Archetype->GetArchetypeInstances(ArchetypeInstances);
		check(ArchetypeInstances.Contains(Component));

		// Identify which of the modified properties are at their default values in the archetype instances,
		// and thus need the new value to be propagated to them
		struct FInstanceDefaultProperties
		{
			UActorComponent* ArchetypeInstance;
			TArray<FProperty*, TInlineAllocator<8>> Properties;
		};

		TArray<FInstanceDefaultProperties> InstanceDefaultProperties;
		InstanceDefaultProperties.Reserve(ArchetypeInstances.Num());

		for (UObject* ArchetypeInstance : ArchetypeInstances)
		{
			UActorComponent* InstanceComp = Cast<UActorComponent>(ArchetypeInstance);
			if (InstanceComp != Component)
			{
				FInstanceDefaultProperties Entry;
				for (FProperty* Property : Properties)
				{
					uint8* ArchetypePtr = Property->ContainerPtrToValuePtr<uint8>(Archetype);
					uint8* InstancePtr = Property->ContainerPtrToValuePtr<uint8>(InstanceComp);
					if (Property->Identical(ArchetypePtr, InstancePtr))
					{
						Entry.Properties.Add(Property);
					}
				}

				if (Entry.Properties.Num() > 0)
				{
					Entry.ArchetypeInstance = InstanceComp;
					InstanceDefaultProperties.Add(MoveTemp(Entry));
				}
			}
		}

		// Propagate all modified properties to the archetype
		Archetype->SetFlags(RF_Transactional);
		Archetype->Modify();

		if (Archetype->GetOwner())
		{
			Archetype->GetOwner()->Modify();
		}

		for (FProperty* Property : Properties)
		{
			uint8* ArchetypePtr = Property->ContainerPtrToValuePtr<uint8>(Archetype);
			uint8* PreviewPtr = Property->ContainerPtrToValuePtr<uint8>(Component);
			Property->CopyCompleteValue(ArchetypePtr, PreviewPtr);

			FPropertyChangedEvent PropertyChangedEvent(Property);
			Archetype->PostEditChangeProperty(PropertyChangedEvent);
		}

		// Apply changes to each archetype instance
		for (const auto& Instance : InstanceDefaultProperties)
		{
			Instance.ArchetypeInstance->SetFlags(RF_Transactional);
			Instance.ArchetypeInstance->Modify();

			AActor* InstanceOwner = Instance.ArchetypeInstance->GetOwner();

			if (InstanceOwner)
			{
				InstanceOwner->Modify();
			}

			for (FProperty* Property : Instance.Properties)
			{
				uint8* InstancePtr = Property->ContainerPtrToValuePtr<uint8>(Instance.ArchetypeInstance);
				uint8* PreviewPtr = Property->ContainerPtrToValuePtr<uint8>(Component);
				Property->CopyCompleteValue(InstancePtr, PreviewPtr);

				FPropertyChangedEvent PropertyChangedEvent(Property);
				Instance.ArchetypeInstance->PostEditChangeProperty(PropertyChangedEvent);
			}

			// Rerun construction script on instance
			if (InstanceOwner)
			{
				InstanceOwner->PostEditMove(PropertyChangeType == EPropertyChangeType::ValueSet);
			}
		}

		// Rerun construction script on preview actor
		if (Owner)
		{
			Owner->PostEditMove(PropertyChangeType == EPropertyChangeType::ValueSet);
		}
	}
}
