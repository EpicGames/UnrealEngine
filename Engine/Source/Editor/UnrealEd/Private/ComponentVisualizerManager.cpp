// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentVisualizerManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "SEditorViewport.h"
#include "HAL/IConsoleManager.h"
#include "EditorModeManager.h"
#include "Elements/Component/ComponentElementData.h"
#include "Elements/Framework/TypedElementSelectionSet.h"


FComponentVisualizerManager::FComponentVisualizerManager()
	: EditedVisualizerViewportClient(nullptr)
{
}

/** Handle a click on the specified editor viewport client */
bool FComponentVisualizerManager::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	// Give current visualizer a chance to handle click if it has a modifier.
	if (EditedVisualizer.IsValid() && (Click.IsControlDown() || Click.IsAltDown() || Click.IsShiftDown()))
	{
		if (EditedVisualizer->HandleModifiedClick(InViewportClient, HitProxy, Click))
		{ 
			return true;
		}
	}

	bool bHandled = HandleProxyForComponentVis(InViewportClient, HitProxy, Click);
	if (bHandled && Click.GetKey() == EKeys::RightMouseButton)
	{
		TSharedPtr<SWidget> MenuWidget = GenerateContextMenuForComponentVis();
		if (MenuWidget.IsValid())
		{
			TSharedPtr<SEditorViewport> ViewportWidget = InViewportClient->GetEditorViewportWidget();
			if (ViewportWidget.IsValid())
			{
				FSlateApplication::Get().PushMenu(
					ViewportWidget.ToSharedRef(),
					FWidgetPath(),
					MenuWidget.ToSharedRef(),
					FSlateApplication::Get().GetCursorPos(),
					FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

				return true;
			}
		}
	}

	// Make sure we are not keeping any active component visualizer around when Visualizer Manager is not handling the click
	if (!bHandled)
	{
		ClearActiveComponentVis();
	}

	return bHandled;
}

bool FComponentVisualizerManager::HandleProxyForComponentVis(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (HitProxy && HitProxy->IsA(HComponentVisProxy::StaticGetType()))
	{
		HComponentVisProxy* VisProxy = (HComponentVisProxy*)HitProxy;
		const UActorComponent* ClickedComponent = VisProxy->Component.Get();
		if (ClickedComponent != NULL)
		{
			TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(ClickedComponent->GetClass());
			if (Visualizer.IsValid())
			{
				FTypedElementHandle SelectedComponentHandle = VisProxy->GetElementHandle();
				UTypedElementSelectionSet* ElementSelectionSet = (InViewportClient && InViewportClient->GetModeTools()) ? InViewportClient->GetModeTools()->GetEditorSelectionSet() : nullptr;

				bool bIsActive = Visualizer->VisProxyHandleClick(InViewportClient, VisProxy, Click);
				if (bIsActive)
				{
					// For spline components in particular, it is nice to select the component itself when manipulating the spline. Otherwise,
					//  if we have the actor selected but not the component, move a component point, and then select the component in the 
					//  component outliner, we lose the point selection state, which feels unexpected.
					bool bShouldSelectElement = Visualizer->ShouldAutoSelectElementOnHandleClick();

					if (bShouldSelectElement && ElementSelectionSet)
					{
						if (ElementSelectionSet->CanSelectElement(SelectedComponentHandle, FTypedElementSelectionOptions())
							&& !ElementSelectionSet->IsElementSelected(SelectedComponentHandle, FTypedElementIsSelectedOptions()))
						{
							// Ideally, we would just perform the selection via a ClearSelection followed by a SelectElement call.
							// Unfortunately, actors currently get notified of deselection immediately in ClearSelection
							// instead of in the subsequent NotifyPendingChanges call, which causes some visualizers (spline
							// visualizer in particular) to lose state, as they see their actor as deselected. So instead,
							// we select the component and deselect everything except what would normally be added to an
							// empty selection: the component and its parent actor.
							// That way the component/actor do not receive a spurious deselection notification.

							// We will remove Component and its Owner handles from this list.
							TArray<FTypedElementHandle> CachedSelection = ElementSelectionSet->GetSelectedElementHandles();

							// Select the Component
							ElementSelectionSet->SelectElement(SelectedComponentHandle, FTypedElementSelectionOptions());

							if (const UActorComponent* const Component = ComponentElementDataUtil::GetComponentFromHandle(SelectedComponentHandle))
							{
								if (const AActor* const Owner = Component->GetOwner())
								{
									FTypedElementHandle OwningActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Owner);
									CachedSelection.Remove(OwningActorHandle);
								}

								CachedSelection.Remove(SelectedComponentHandle);

								// Remove all element handles from list, except from Component and Owner handles.
								ElementSelectionSet->DeselectElements(CachedSelection, FTypedElementSelectionOptions());
							}
						}
					}

					SetActiveComponentVis(InViewportClient, Visualizer);

					if (bShouldSelectElement && ElementSelectionSet)
					{
						ElementSelectionSet->NotifyPendingChanges();
					}

					return true;
				}
			}
		}
	}

	// DO NOT call ClearActiveComponentVis() here. If a new actor is being selected, ClearActiveComponentVis() 
	// will eventually be called by UUnrealEdEngine::NoteSelectionChange().  If it were called here, 
	// it would be prior to the selection transaction and thus the previous state of the component visualizer 
	// would not be captured for undo/redo.

	return false;
}

TSharedPtr<FComponentVisualizer> FComponentVisualizerManager::GetActiveComponentVis()
{
	return EditedVisualizerPtr.Pin();
}

bool FComponentVisualizerManager::SetActiveComponentVis(FEditorViewportClient* InViewportClient, TSharedPtr<FComponentVisualizer>& InVisualizer)
{
	if (InViewportClient && InVisualizer.IsValid())
	{
		// call EndEditing on any currently edited visualizer, if we are going to change it
		TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();
		if (EditedVisualizer.IsValid() && InVisualizer.Get() != EditedVisualizer.Get())
		{
			EditedVisualizer->EndEditing();
		}

		EditedVisualizerPtr = InVisualizer;
		EditedVisualizerViewportClient = InViewportClient;
		return true;
	}

	return false;
}

void FComponentVisualizerManager::ClearActiveComponentVis()
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		EditedVisualizer->EndEditing();
	}

	EditedVisualizerPtr.Reset();
	EditedVisualizerViewportClient = nullptr;
}

bool FComponentVisualizerManager::HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->HandleInputKey(ViewportClient, Viewport, Key, Event);
	}

	return false;
}

bool FComponentVisualizerManager::HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid() && InViewportClient && InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
	{
		return EditedVisualizer->HandleInputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
	}

	return false;
}

bool FComponentVisualizerManager::HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->HandleFrustumSelect(InFrustum, InViewportClient, InViewport);
	}

	return false;
}

bool FComponentVisualizerManager::HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->HandleBoxSelect(InBox, InViewportClient, InViewport);
	}

	return false;
}

bool FComponentVisualizerManager::HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->HasFocusOnSelectionBoundingBox(OutBoundingBox);
	}

	return false;
}

bool FComponentVisualizerManager::HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination)
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->HandleSnapTo(bInAlign, bInUseLineTrace, bInUseBounds, bInUsePivot, InDestination);
	}

	return false;
}

bool FComponentVisualizerManager::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->GetWidgetLocation(ViewportClient, OutLocation);
	}

	return false;
}


bool FComponentVisualizerManager::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->GetCustomInputCoordinateSystem(ViewportClient, OutMatrix);
	}

	return false;
}

void FComponentVisualizerManager::TrackingStarted(FEditorViewportClient* InViewportClient)
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->TrackingStarted(InViewportClient);
	}
}

void FComponentVisualizerManager::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid() && EditedVisualizer->GetEditedComponent() != nullptr)
	{
		return EditedVisualizer->TrackingStopped(InViewportClient, bInDidMove);
	}
}

TSharedPtr<SWidget> FComponentVisualizerManager::GenerateContextMenuForComponentVis() const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();

	if (EditedVisualizer.IsValid())
	{
		return EditedVisualizer->GenerateContextMenu();
	}

	return TSharedPtr<SWidget>();
}


bool FComponentVisualizerManager::IsActive() const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();
	return EditedVisualizer.IsValid();
}


bool FComponentVisualizerManager::IsVisualizingArchetype() const
{
	TSharedPtr<FComponentVisualizer> EditedVisualizer = EditedVisualizerPtr.Pin();
	return EditedVisualizer.IsValid() && EditedVisualizer->IsVisualizingArchetype();
}
