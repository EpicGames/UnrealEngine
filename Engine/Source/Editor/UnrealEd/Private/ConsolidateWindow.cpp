// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsolidateWindow.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectRedirector.h"
#include "IAssetTools.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Materials/MaterialInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "ISourceControlModule.h"
#include "Engine/Texture.h"
#include "AssetRegistry/AssetData.h"
#include "FileHelpers.h"
#include "AssetSelection.h"
#include "ObjectTools.h"
#include "Interfaces/IMainFrameModule.h"
#include "Engine/DataTable.h"


#define LOCTEXT_NAMESPACE "SConsolidateWindow"

/**
 * The Consolidate Tool Widget Class
 */

class SConsolidateToolWidget : public SBorder
{
public:
	SLATE_BEGIN_ARGS( SConsolidateToolWidget )
	: _ParentWindow()
	{}

		SLATE_ATTRIBUTE( TSharedPtr<SWindow>, ParentWindow )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
	
	SConsolidateToolWidget()
	{
	}

	
	/** 
	 * Class to support our list box
	 */
	class FListItem
	{
	public:
		/** 
		 * Constructor
		 * @param	InParent	Parent Widget that holds the ListBox
		 * @param	InObject	the UObject this list item represents in the ListBox
		 */
		FListItem(SConsolidateToolWidget *InParent, UObject* InObject)
		{
			Parent = InParent;
			Object = InObject;
		};

		/** 
		 * Callback function used to tell the ListBox parent what item has been selected
		 */
		void OnAssetSelected( ECheckBoxState NewCheckedState )
		{
			check(Parent != NULL);
			Parent->SetSelectedListItem(this);
		};

		/**
		 * Callback function used to ensure only one item is highlighted (selected) at a time
		 */
		ECheckBoxState IsAssetSelected() const
		{
			return (Parent->GetSelectedListItem() == this) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		/** @return		The full name of the object this item represents */
		FText GetObjectName() const
		{
			return FText::FromString(IAssetTools::Get().GetUserFacingFullName(*Object));
		}
		/** @return		The UObject this item represents */	
		const UObject* GetObject() const
		{
			return Object;
		}

	private:
		/** Parent Widget that holds the ListBox */
		SConsolidateToolWidget* Parent;
		/** The UObject this item represents */
		UObject *Object;
	};

	/** @return		Selected Item in the listbox */
	FListItem* GetSelectedListItem()
	{
		return SelectedListItem;
	}


	/** 
	 * Used by the listbox to tell its parent what item is selected.
	 *
	 * @param	List item from the listbox to select
	 */
	void SetSelectedListItem(FListItem* ListItem)
	{
		SelectedListItem = ListItem;
	}

	/**
	* Used by the listbox to tell its parent what item is selected.
	*
	* @param	Item from the listbox to select
	*/
	void SetSelectedItem( UObject* Item )
	{
		for ( TSharedPtr<FListItem>& ListItem : ListViewItems )
		{
			if ( ListItem->GetObject() == Item )
			{
				SetSelectedListItem( ListItem.Get() );
				break;
			}
		}
	}

	/** @return	Return the index of the ConsolidationoOjects array of the actively selected ListItem */
	int32 GetSelectedListItemIndex();

	/**
	 * Attempt to add the provided objects to the consolidation panel; Only adds objects which are compatible with objects already existing within the panel, if any
	 *
	 * @param	InObjects			Objects to attempt to add to the panel
	 *
	 * @return	The number of objects successfully added to the consolidation panel
	 */
	int32 AddConsolidationObjects( const TArray<UObject*>& InObjects );

	/**
	 * Fills the provided array with all of the UObjects referenced by the consolidation panel, for the purpose of serialization
	 *
	 * @param	[out]OutSerializableObjects	Array to fill with all of the UObjects referenced by the consolidation panel
	 */
	void QuerySerializableObjects( TArray<UObject*>& OutSerializableObjects );

	/**
	 * Determine the compatibility of the passed in objects with the objects already present in the consolidation panel
	 *
	 * @param	InProposedObjects		Objects to check compatibility with vs. the objects already present in the consolidation panel
	 * @param	OutCompatibleObjects	[out]Objects from the passed in array which are compatible with those already present in the
	 *									consolidation panel, if any
	 *
	 * @return	true if all of the passed in objects are compatible, false otherwise
	 */
	bool DetermineAssetCompatibility( const TArray<UObject*>& InProposedObjects, TArray<UObject*>& OutCompatibleObjects );

	/** Removes all consolidation objects from the consolidation panel */
	void ClearConsolidationObjects();

private:
	/**
	 * Verifies if all of the consolidation objects in the panel are of the same class or not
	 *
	 * @return	true if all of the classes of the consolidation objects are the same; false otherwise
	 */
	bool AreObjClassesHomogeneous();
	
	/** Delete all of the dropped asset data for drag-drop support */
	void ClearDroppedAssets();
	
	/** Reset the consolidate panel's error panel to its default state */
	void ResetErrorPanel();

	/** Remove the currently selected object from the consolidation panel */
	void RemoveSelectedObject();

	/**
	 * Display a message in the consolidation panel's "error" panel; Naive implementation, wipes out any pre-existing message
	 *
	 * @param	bError			If true, change the error panel's styling to indicate the severity of the message; if false, use a lighter style
	 * @param	ErrorMessage	Message to display in the "error" panel
	 *
	 * @note	The current implementation is naive and will wipe out any pre-existing message when called. The current needs of the window don't require
	 *			anything more sophisticated, but in the future perhaps the messages should be appended, multiple panel types should exist, etc.
	 */
	void DisplayMessage( bool bError, const FText& ErrorMessage );

	/**  Closes the parent window and clears the consolidation objects, dropped assets and error panel. */
	void ClearAndCloseWindow( );

	// Button Responses

	/** Called in response to the user clicking the "X" button on the error panel; dismisses the error panel */
	FReply OnDismissErrorPanelButtonClicked( );

	/** Called in response to the user clicking the "Consolidate Objects"/OK button; performs asset consolidation */
	FReply OnConsolidateButtonClicked( );

	/** Called in response to the user clicking the cancel button; dismisses the panel w/o consolidating objects */
	FReply OnCancelButtonClicked( );
	
	// Drag-drop Support

	/** Called in response to the user beginning to drag something over the consolidation panel; parses the drop data into dropped assets, if possible */
	void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/** Called in response to the user's drag operation exiting the consolidation panel; deletes any dropped asset data */
	void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	
	/** Called in response to the user performing a drop operation in the consolidation panel; adds the dropped objects to the panel */
	FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/** Called while the user is dragging something over the consolidation panel; provides visual feedback on whether a drop is allowed or not */
	FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	// Input Responses

	/** Called in response to the user releasing a keyboard key while the consolidation panel has keyboard focus */
	FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Track if the panel has already warned the user about consolidating assets with different types, so as not to repeatedly (and annoyingly) warn */
	bool bAlreadyWarnedAboutTypes;

	/** if checked, signifies that after a consolidation operation, an attempt will be made to save the packages dirtied by the operation */ 
	bool bSavePackagesChecked;

	/** List of strings to appear in the list box */
	//using AssetInfo array instead
	TArray< FString > ConsolidationObjectNames;

	/** Array of consolidation objects */
	TArray< UObject* > ConsolidationObjects;

	/** Array of dropped asset data for supporting drag-and-drop */
	TArray< FAssetData > DroppedAssets;


private:

	/** A Pointer to our Parent Window */
	TWeakPtr<SWindow> ParentWindowPtr;

	typedef SListView< TSharedPtr<FListItem> > SListType;
	/** ListBox for selecting which object to consolidate */
	TSharedPtr< SListType > ListView;
	/** Collection of objects (Widgets) to display in the List View. */
	TArray< TSharedPtr<FListItem> > ListViewItems;
	/** List Box Item currently selected */
	FListItem *SelectedListItem;
	/** Error Text display for error/warning messages */
	TSharedPtr<SErrorText> ErrorPanel;

	/** Callback to generate ListBoxRows */
	TSharedRef<ITableRow> OnGenerateRowForList( TSharedPtr<FListItem> ListItemPtr, const TSharedRef<STableViewBase>& OwnerTable );

	/** Callback for enabling/disabling the Cosolidate Button */
	bool IsConsolidateButtonEnabled() const;

	/** @return		true if the user has elected to "Save Dirty Packages" */
	ECheckBoxState IsSavePackagesChecked() const;
	
	/** Callback	called when the user checks/unchecks the "Save Dirty Packages" button */
	void OnSavePackagesCheckStateChanged(ECheckBoxState NewCheckedState);

	/** Callback	Called to determin the error message visibility */
	EVisibility IsErrorPanelVisible() const;

	/** Refreshes the List Box and window */
	void RefreshListItems();
};


// Window/Interface Function...
TWeakPtr<SConsolidateToolWidget> FConsolidateToolWindow::WidgetInstance;

void FConsolidateToolWindow::AddConsolidationObjects( const TArray<UObject*>& InObjects, UObject* SelectedItem )
{
	if (WidgetInstance.IsValid())
	{
		//Use the existing widget
		WidgetInstance.Pin()->AddConsolidationObjects(InObjects);
	}
	else
	{
		//Create a new window
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title(LOCTEXT("Consolidate_Title", "Replace References"))
			.ClientSize( FVector2D(768,300) )
			.SupportsMinimize(false)
			.SupportsMaximize(false);

		TSharedRef<SConsolidateToolWidget> NewWidget = 
			SNew(SConsolidateToolWidget)
			.ParentWindow(NewWindow);

		NewWidget->AddConsolidationObjects(InObjects);

		if ( SelectedItem )
		{
			NewWidget->SetSelectedItem( SelectedItem );
		}

		NewWindow->SetContent(NewWidget);

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));

		if ( MainFrameModule.GetParentWindow().IsValid() )
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, MainFrameModule.GetParentWindow().ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow);
		}

		WidgetInstance = NewWidget;
	}
}

bool FConsolidateToolWindow::DetermineAssetCompatibility( const TArray<UObject*>& InProposedObjects, TArray<UObject*>& OutCompatibleObjects )
{
	if (WidgetInstance.IsValid())
	{
		//Compare with the existing widget
		return WidgetInstance.Pin()->DetermineAssetCompatibility(InProposedObjects,OutCompatibleObjects);
	}
	else
	{
		//create a temp widget to compare assets with
		TSharedRef<SConsolidateToolWidget> TempWidget = 
			SNew(SConsolidateToolWidget)
			.ParentWindow(TSharedPtr<SWindow>());
		return TempWidget->DetermineAssetCompatibility(InProposedObjects,OutCompatibleObjects);
	}
}




// Widget Function definitions...

void SConsolidateToolWidget::Construct( const FArguments& InArgs )
{
	ParentWindowPtr = InArgs._ParentWindow.Get();

	SelectedListItem = NULL;
	bSavePackagesChecked = ISourceControlModule::Get().IsEnabled();

	SetBorderImage(FAppStyle::GetBrush("NoBorder"));

	ChildSlot
	[
		
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5)
		[
			SNew( STextBlock )
			.AutoWrapText(true)
			.Text( LOCTEXT("Consolidate_Select", "Select an asset to serve as the asset to consolidate the non-selected assets to. This will replace all uses of the non-selected assets below with the selected asset.") )
		]

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(5)
		[
			SNew( SBorder )
			.Padding(5)
			[
				SAssignNew( ListView, SListType)
				.ListItemsSource( &ListViewItems )
				.OnGenerateRow( this, &SConsolidateToolWidget::OnGenerateRowForList )
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5)
		[
			SNew( SHorizontalBox )
			.Visibility(this, &SConsolidateToolWidget::IsErrorPanelVisible)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Fill)
			[
				SAssignNew( ErrorPanel, SErrorText )
				.Visibility(EVisibility::Hidden)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "Window.Buttons.Close" )
				.OnClicked(this, &SConsolidateToolWidget::OnDismissErrorPanelButtonClicked)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5)
			.HAlign(HAlign_Left)
			[
				SNew(SCheckBox)
				.IsChecked( this, &SConsolidateToolWidget::IsSavePackagesChecked )
				.OnCheckStateChanged( this, &SConsolidateToolWidget::OnSavePackagesCheckStateChanged )
				[
					SNew( STextBlock )
					.Text( LOCTEXT("Consolidate_SaveDirtyAssets", "Save dirtied assets") )
				]
			]
			+SHorizontalBox::Slot()
			.Padding(5)
			.HAlign(HAlign_Right)
			.FillWidth(1)
			[
				SNew(SButton) 
				.Text( LOCTEXT("ConsolidateAssetsButton", "Consolidate Assets") )
				.IsEnabled(this, &SConsolidateToolWidget::IsConsolidateButtonEnabled)
				.OnClicked(this, &SConsolidateToolWidget::OnConsolidateButtonClicked)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.Text( LOCTEXT("CancelConsolidateButton", "Cancel"))
				.OnClicked(this, &SConsolidateToolWidget::OnCancelButtonClicked)
			]
		]
	];
}

TSharedRef<ITableRow> SConsolidateToolWidget::OnGenerateRowForList( TSharedPtr<FListItem> ListItemPtr, const TSharedRef<STableViewBase>& OwnerTable )
{
	return
		SNew(STableRow< TSharedPtr<FName> >, OwnerTable)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "Menu.RadioButton")
			.IsChecked(ListItemPtr.ToSharedRef(), &FListItem::IsAssetSelected)
			.OnCheckStateChanged( ListItemPtr.ToSharedRef(), &FListItem::OnAssetSelected )
			[
				SNew (STextBlock)
				.Text( ListItemPtr.ToSharedRef(), &FListItem::GetObjectName )
			]
		];
}

bool SConsolidateToolWidget::IsConsolidateButtonEnabled() const
{
	return (ConsolidationObjects.Num() > 1 && SelectedListItem != NULL);
}

ECheckBoxState SConsolidateToolWidget::IsSavePackagesChecked() const
{
	return bSavePackagesChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SConsolidateToolWidget::OnSavePackagesCheckStateChanged(ECheckBoxState NewCheckedState)
{
	bSavePackagesChecked = (NewCheckedState == ECheckBoxState::Checked);
}

EVisibility SConsolidateToolWidget::IsErrorPanelVisible() const
{
	if (ErrorPanel.IsValid())
	{
		return ErrorPanel->GetVisibility();
	}

	return EVisibility::Hidden;
}

void SConsolidateToolWidget::RefreshListItems()
{
	ListViewItems.Empty();
	
	for ( TArray<UObject*>::TConstIterator ObjIter( ConsolidationObjects ); ObjIter; ++ObjIter )
	{
		ListViewItems.Add( MakeShareable( new FListItem(this, *ObjIter) ));
	}

	ListView->RequestListRefresh();

	SelectedListItem = NULL;
}

int32 SConsolidateToolWidget::GetSelectedListItemIndex()
{
	if (SelectedListItem)
	{
		for ( int32 Index=0; Index < ConsolidationObjects.Num(); Index++ )
		{
			if (SelectedListItem->GetObject() == ConsolidationObjects[Index])
			{
				return Index;
			}
		}
	}

	return INDEX_NONE;
}

/**
 * Attempt to add the provided objects to the consolidation panel; Only adds objects which are compatible with objects already existing within the panel, if any
 *
 * @param	InObjects			Objects to attempt to add to the panel
 *
 * @return	The number of objects successfully added to the consolidation panel
 */
int32 SConsolidateToolWidget::AddConsolidationObjects( const TArray<UObject*>& InObjects )
{
	// First check the passed in objects for compatibility; allowing cross-type consolidation would result in disaster
	TArray<UObject*> CompatibleObjects;
	DetermineAssetCompatibility( InObjects, CompatibleObjects );

	// Iterate over each compatible object, adding it to the panel if it's not already there 
	for ( TArray<UObject*>::TConstIterator CompatibleObjIter( CompatibleObjects ); CompatibleObjIter; ++CompatibleObjIter )
	{
		UObject* CurObj = *CompatibleObjIter;
		check( CurObj );

		// Don't allow an object to be added to the panel twice
		if ( !ConsolidationObjects.Contains( CurObj ) )
		{
			ConsolidationObjectNames.Add( CurObj->GetFullName() );
			ConsolidationObjects.Add( CurObj );
		}
	}

	// Refresh the list box, as new items have been added
	RefreshListItems();

	// Check if all of the consolidation objects share the same type. If they don't, and the user hasn't been prompted about it before,
	// display a warning message informing them of the potential danger.
	if ( !AreObjClassesHomogeneous() && !bAlreadyWarnedAboutTypes )
	{
		DisplayMessage( false, LOCTEXT("Consolidate_WarningSameClass", "The object to consolidate are not the same class"));
		bAlreadyWarnedAboutTypes = true;
	}

	return CompatibleObjects.Num();
}

/**
 * Determine the compatibility of the passed in objects with the objects already present in the consolidation panel
 *
 * @param	InProposedObjects		Objects to check compatibility with vs. the objects already present in the consolidation panel
 * @param	OutCompatibleObjects	[out]Objects from the passed in array which are compatible with those already present in the
 *									consolidation panel, if any
 *
 * @return	true if all of the passed in objects are compatible, false otherwise
 */
bool SConsolidateToolWidget::DetermineAssetCompatibility( const TArray<UObject*>& InProposedObjects, TArray<UObject*>& OutCompatibleObjects )
{
	bool bAllAssetsValid = true;

	OutCompatibleObjects.Empty();

	if ( InProposedObjects.Num() > 0 )
	{
		// If the consolidation panel is currently empty, use the first member of the proposed objects as the object whose class should be checked against.
		// Otherwise, use the first consolidation object.
		const UObject* ComparisonObject = ConsolidationObjects.Num() > 0 ? ConsolidationObjects[ 0 ] : InProposedObjects[ 0 ];
		check( ComparisonObject );

		const UClass* ComparisonClass = ComparisonObject->GetClass();
		check( ComparisonClass );

		// Iterate over each proposed consolidation object, checking if each shares a common class with the consolidation objects, or at least, a common base that
		// is allowed as an exception (currently only exceptions made for textures and materials).
		for ( TArray<UObject*>::TConstIterator ProposedObjIter( InProposedObjects ); ProposedObjIter; ++ProposedObjIter )
		{
			UObject* CurProposedObj = *ProposedObjIter;
			check( CurProposedObj );

			// You may not consolidate object redirectors
			if ( CurProposedObj->GetClass()->IsChildOf(UObjectRedirector::StaticClass()) )
			{
				bAllAssetsValid = false;
				continue;
			}

			if ( CurProposedObj->GetClass() != ComparisonClass )
			{
				const UClass* NearestCommonBase = CurProposedObj->FindNearestCommonBaseClass( ComparisonClass );

				// If the proposed object doesn't share a common class or a common base that is allowed as an exception, it is not a compatible object
				if ( !( NearestCommonBase->IsChildOf( UTexture::StaticClass() ) )  &&
					 !( NearestCommonBase->IsChildOf( UMaterialInterface::StaticClass() ) ) &&
					 !( NearestCommonBase->IsChildOf( UPhysicalMaterial::StaticClass() ) ) &&
					 !( NearestCommonBase->IsChildOf( UDataTable::StaticClass() ) ) )
				{
					bAllAssetsValid = false;
					continue;
				}
			}

			// If the proposed object is already in the panel, it is not a compatible object
			if ( ConsolidationObjects.Contains( CurProposedObj ) )
			{
				bAllAssetsValid = false;
				continue;
			}

			// If execution has gotten this far, the current proposed object is compatible
			OutCompatibleObjects.Add( CurProposedObj );
		}
	}

	return bAllAssetsValid;
}

/**
 * Fills the provided array with all of the UObjects referenced by the consolidation panel, for the purpose of serialization
 *
 * @param	[out]OutSerializableObjects	Array to fill with all of the UObjects referenced by the consolidation panel
 */
void SConsolidateToolWidget::QuerySerializableObjects( TArray<UObject*>& OutSerializableObjects )
{
	OutSerializableObjects.Empty( ConsolidationObjects.Num() );

	// Add all of the consolidation objects to the array
	OutSerializableObjects.Append( ConsolidationObjects );

	// Add each drop data info object to the array
	for ( TArray<FAssetData>::TConstIterator DroppedAssetsIter( DroppedAssets ); DroppedAssetsIter; ++DroppedAssetsIter )
	{
		const FAssetData& AssetData = *DroppedAssetsIter;

		UObject* Object = AssetData.GetAsset();
		if ( Object != NULL )
		{
			OutSerializableObjects.AddUnique( Object );
		}
	}
}

/** Removes all consolidation objects from the consolidation panel */
void SConsolidateToolWidget::ClearConsolidationObjects()
{

	ConsolidationObjectNames.Empty();
	ConsolidationObjects.Empty();
	RefreshListItems();
}

/**
 * Verifies if all of the consolidation objects in the panel are of the same class or not
 *
 * @return	true if all of the classes of the consolidation objects are the same; false otherwise
 */
bool SConsolidateToolWidget::AreObjClassesHomogeneous()
{
	bool bAllClassesSame = true;

	if ( ConsolidationObjects.Num() > 1 )
	{
		TArray<UObject*>::TConstIterator ConsolidationObjIter( ConsolidationObjects );
		const UObject* FirstObj = *ConsolidationObjIter;
		check( FirstObj );
		
		// Store the class of the first consolidation object for comparison purposes
		const UClass* FirstObjClass = FirstObj->GetClass();
		check( FirstObjClass );
		
		// Starting from the second consolidation object, iterate through all consolidation objects
		// to see if they all share a common class
		++ConsolidationObjIter;
		for ( ; ConsolidationObjIter; ++ConsolidationObjIter )
		{
			const UObject* CurObj = *ConsolidationObjIter;
			check( CurObj );

			const UClass* CurObjClass = CurObj->GetClass();
			check( CurObjClass );

			if ( CurObjClass != FirstObjClass )
			{
				bAllClassesSame = false;
				break;
			}
		}
	}

	return bAllClassesSame;
}

/** Delete all of the dropped asset data for drag-drop support */
void SConsolidateToolWidget::ClearDroppedAssets()
{
	DroppedAssets.Empty();
}

/** Reset the consolidate panel's error panel to its default state */
void SConsolidateToolWidget::ResetErrorPanel()
{
	bAlreadyWarnedAboutTypes = false;

	ErrorPanel->SetVisibility(EVisibility::Hidden);
	ErrorPanel->SetError( FText::GetEmpty() );
}


/** Remove the currently selected object from the consolidation panel */
void SConsolidateToolWidget::RemoveSelectedObject()
{
	const int32 SelectedIndex = GetSelectedListItemIndex();

	// Ensure there's currently a valid selection
	if ( ConsolidationObjects.IsValidIndex( SelectedIndex ) )
	{
		// If the selection was valid, remove the consolidation object from the panel
		ConsolidationObjects.RemoveAt( SelectedIndex );
		ConsolidationObjectNames.RemoveAt( SelectedIndex );

		// Refresh the list box to display the change in contents
		RefreshListItems();

		// If prior to the removal the consolidation objects contained multiple classes but now only
		// contain one, remove the warning about the presence of multiple classes
		// NOTE: This works because of the limited number of messages utilized by the window. If more errors are added,
		// simply resetting the error panel here might confuse the user.
		if ( bAlreadyWarnedAboutTypes && AreObjClassesHomogeneous() )
		{
			ResetErrorPanel();
		}
	}
}

/**
 * Display a message in the consolidation panel's "error" panel; Naive implementation, wipes out any pre-existing message
 *
 * @param	bError			If true, change the error panel's styling to indicate the severity of the message; if false, use a lighter style
 * @param	ErrorMessage	Message to display in the "error" panel
 *
 * @note	The current implementation is naive and will wipe out any pre-existing message when called. The current needs of the window don't require
 *			anything more sophisticated, but in the future perhaps the messages should be appended, multiple panel types should exist, etc.
 */
void SConsolidateToolWidget::DisplayMessage( bool bError, const FText& ErrorMessage )
{
	// Update the error text block to display the requested message
	ErrorPanel->SetError(ErrorMessage);

	// Show the error panel
	ErrorPanel->SetVisibility(EVisibility::Visible);

}

/** Closes the parent window and clears the consolidation objects, dropped assets and error panel.*/
void SConsolidateToolWidget::ClearAndCloseWindow()
{
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	ClearConsolidationObjects();
	ClearDroppedAssets();
	ResetErrorPanel();
}

/** Called in response to the user clicking the "X" button on the error panel; dismisses the error panel */
FReply SConsolidateToolWidget::OnDismissErrorPanelButtonClicked( )
{
	// Hide the error panel
	ErrorPanel->SetVisibility(EVisibility::Hidden);
	
	return FReply::Handled();
}


/** Called in response to the user clicking the "Consolidate Objects"/OK button; performs asset consolidation */
FReply SConsolidateToolWidget::OnConsolidateButtonClicked()
{
	const int32 SelectedIndex = GetSelectedListItemIndex();
	check( SelectedIndex >= 0 && ConsolidationObjects.Num() > 1 );

	// Find which object the user has elected to be the "object to consolidate to"
	UObject* ObjectToConsolidateTo = ConsolidationObjects[ SelectedIndex ];
	check( ObjectToConsolidateTo );

	// Compose an array of the objects to consolidate, removing the "object to consolidate to" from the array
	// NOTE: We cannot just use the array held on the panel, because the references need to be cleared prior to the consolidation
	// attempt or else they will interfere and cause problems.
	TArray<UObject*> FinalConsolidationObjects = ConsolidationObjects;
	FinalConsolidationObjects.RemoveSingle( ObjectToConsolidateTo );

	// Close the window while the consolidation operation occurs
	ParentWindowPtr.Pin()->HideWindow();

	// Reset the panel back to its default state so that post-consolidation the panel appears as it would from a fresh launch
	ResetErrorPanel();

	// The consolidation objects must be cleared from the panel, lest they interfere with the consolidation
	ClearConsolidationObjects();

	// Perform the object consolidation
	ObjectTools::FConsolidationResults ConsResults = ObjectTools::ConsolidateObjects( ObjectToConsolidateTo, FinalConsolidationObjects );

	// Check if the user has specified if they'd like to save the dirtied packages post-consolidation
	if ( bSavePackagesChecked )
	{
		// If the consolidation went off successfully with no failed objects, prompt the user to checkout/save the packages dirtied by the operation
		if ( ConsResults.DirtiedPackages.Num() > 0 && ConsResults.FailedConsolidationObjs.Num() == 0 && bSavePackagesChecked == true )
		{
			FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
			SaveParams.bCheckDirty = false;
			SaveParams.bPromptToSave = true;
			SaveParams.bIsExplicitSave = true;

			FEditorFileUtils::PromptForCheckoutAndSave( ObjectPtrDecay(ConsResults.DirtiedPackages), SaveParams);
		}
		// If the consolidation resulted in failed (partially consolidated) objects, do not save, and inform the user no save attempt was made
		else if ( ConsResults.FailedConsolidationObjs.Num() > 0 && bSavePackagesChecked == true )
		{
			DisplayMessage( true, LOCTEXT("Consolidate_WarningPartial", "Not all objects could be consolidated, no save has occurred") );
		}
	}

	RefreshListItems();
	// No point in showing the list again if it's empty
	if (ListViewItems.Num() > 0)
	{
		ParentWindowPtr.Pin()->ShowWindow();
	}
	else
	{
		ClearAndCloseWindow();
	}

	return FReply::Handled();
}

/** Called in response to the user clicking the cancel button; dismisses the panel w/o consolidating objects */
FReply SConsolidateToolWidget::OnCancelButtonClicked( )
{
	// Close the window and clear out all the consolidation assets/dropped assets/etc.
	ClearAndCloseWindow();

	return FReply::Handled();
}

/** Called in response to the user beginning to drag something over the consolidation panel; parses the drop data into dropped assets, if possible */
void SConsolidateToolWidget::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Assets being dropped from content browser should be parsable from a string format
	TArray<FAssetData> ExtractedDroppedAsset = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);

	// Construct drop data info for each parsed asset string
	DroppedAssets.Empty( ExtractedDroppedAsset.Num() );
	DroppedAssets.Append( ExtractedDroppedAsset );
}

/** Called in response to the user's drag operation exiting the consolidation panel; deletes any dropped asset data */
void SConsolidateToolWidget::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	ClearDroppedAssets();
}

/** Called in response to the user performing a drop operation in the consolidation panel; adds the dropped objects to the panel */
FReply SConsolidateToolWidget::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TArray<FAssetData> ExtractedDroppedAsset = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);

	TArray<UObject*> DroppedObjects;
	for (int Index = 0; Index < ExtractedDroppedAsset.Num(); Index++)
	{
		UObject* Object = ExtractedDroppedAsset[Index].GetAsset();

		if ( Object != NULL )
		{
			DroppedObjects.AddUnique( Object );
		}
	}

	AddConsolidationObjects( DroppedObjects );

	// Clear out the drop data, as the drop is over
	ClearDroppedAssets();

	return FReply::Handled();
}

/** Called while the user is dragging something over the consolidation panel; provides visual feedback on whether a drop is allowed or not */
FReply SConsolidateToolWidget::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Construct an array of objects that would be dropped upon the consolidation panel
	TArray<FAssetData> ExtractedDroppedAsset = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);

	TArray<UObject*> DroppedObjects;
	for (int Index = 0; Index < ExtractedDroppedAsset.Num(); Index++)
	{
		UObject* Object = ExtractedDroppedAsset[Index].GetAsset();

		if ( Object == NULL )
		{
			UClass* ObjectClass = ExtractedDroppedAsset[Index].GetClass();
			if (ObjectClass)
			{
				Object = ObjectClass->GetDefaultObject();
			}
		}

		if ( Object != NULL )
		{
			DroppedObjects.AddUnique( Object );
		}
	}

	// If all of the dragged over assets are compatible, update the mouse cursor to signify a drop is possible
	TArray<UObject*> CompatibleAssets;
	if ( DroppedObjects.Num() > 0 && DetermineAssetCompatibility( DroppedObjects, CompatibleAssets ) )
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called in response to the user releasing a keyboard key while the consolidation panel has keyboard focus */
FReply SConsolidateToolWidget::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	const FKey Key = InKeyEvent.GetKey();
	if ( Key == EKeys::Platform_Delete )
	{
		RemoveSelectedObject();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
