// Copyright Epic Games, Inc. All Rights Reserved.
#include "ConvexDecompositionNotification.h"

#include "GlobalEditorNotification.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Notifications/SNotificationList.h"

/** Pointer to global status object */
FConvexDecompositionNotificationState* GConvexDecompositionNotificationState = NULL;

/** Notification class for asynchronous convex decomposition. */
class FConvexDecompositionNotificationImpl : public FGlobalEditorNotification, public FConvexDecompositionNotificationState
{
public:
	FConvexDecompositionNotificationImpl(void)
	{
		GConvexDecompositionNotificationState = this;	// Initialize the pointer to the global state notification
	}
	~FConvexDecompositionNotificationImpl(void)
	{
		GConvexDecompositionNotificationState = nullptr; // Clear the pointer to the global state notification
	}
protected:
	/** FGlobalEditorNotification interface */
	virtual bool ShouldShowNotification(const bool bIsNotificationAlreadyActive) const override;
	virtual void SetNotificationText(const TSharedPtr<SNotificationItem>& InNotificationItem) const override;
};

/** Global notification object. */
FConvexDecompositionNotificationImpl GConvexDecompositionNotification;


bool FConvexDecompositionNotificationImpl::ShouldShowNotification(const bool bIsNotificationAlreadyActive) const
{
	return IsActive;
}

void FConvexDecompositionNotificationImpl::SetNotificationText(const TSharedPtr<SNotificationItem>& InNotificationItem) const
{
	if ( IsActive )
	{
		FText ProgressMessage = FText::AsCultureInvariant(Status); // Text from V-HACD status is no localized
		InNotificationItem->SetText(ProgressMessage);
	}
}

