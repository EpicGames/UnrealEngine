// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CookOnTheFlyServer.cpp: handles polite cook requests via network ;)
=============================================================================*/

#include "CookOnTheSide/CookOnTheFlyServer.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/RandomShuffle.h"
#include "Algo/Unique.h"
#include "Async/InheritedGuardValue.h"
#include "AssetCompilingManager.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "Commandlets/AssetRegistryGenerator.h"
#include "Containers/DirectoryTree.h"
#include "Containers/RingBuffer.h"
#include "Cooker/CookActivePackage.h"
#include "Cooker/AsyncIODelete.h"
#include "Cooker/CookAssetRegistryAccessTracker.h"
#include "Cooker/CookConfigAccessTracker.h"
#include "Cooker/CookDiagnostics.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookGarbageCollect.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookGlobalDependencies.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookArtifact.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookOnTheFlyServerInterface.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPackagePreloader.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookRequestCluster.h"
#include "Cooker/CookRequests.h"
#include "Cooker/CookSandbox.h"
#include "Cooker/CookSavePackage.h"
#include "Cooker/CookTypes.h"
#include "Cooker/CookWorkerClient.h"
#include "Cooker/DiffPackageWriter.h"
#include "Cooker/GlobalCookArtifact.h"
#include "LocalizationCookArtifact.h"
#include "Cooker/IncrementalValidatePackageWriter.h"
#include "Cooker/IoStoreCookOnTheFlyRequestManager.h"
#include "Cooker/LooseCookedPackageWriter.h"
#include "Cooker/MPCollector.h"
#include "Cooker/NetworkFileCookOnTheFlyRequestManager.h"
#include "Cooker/OnDemandShaderCompilation.h"
#include "Cooker/PackageTracker.h"
#include "Cooker/ShaderLibraryCooking.h"
#include "Cooker/StallDetector.h"
#include "Cooker/ValidationCookArtifact.h"
#include "Cooker/WorkerRequestsLocal.h"
#include "Cooker/WorkerRequestsRemote.h"
#include "CookerSettings.h"
#include "CookMetadata.h"
#include "CookOnTheFlyNetServer.h"
#include "CookPackageSplitter.h"
#include "DerivedDataCacheInterface.h"
#include "DistanceFieldAtlas.h"
#include "Dom/JsonObject.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorCommandLineUtils.h"
#include "EditorDomain/EditorDomain.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "Engine/AssetManager.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Texture.h"
#include "Engine/TextureLODSettings.h"
#include "Engine/WorldComposition.h"
#include "EngineGlobals.h"
#include "FileServerMessages.h"
#include "GameDelegates.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/MemoryMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Hash/xxhash.h"
#include "IMessageContext.h"
#include "INetworkFileServer.h"
#include "INetworkFileSystemModule.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITextureFormat.h"
#include "Internationalization/Culture.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "IPAddress.h"
#include "LayeredCookArtifactReader.h"
#include "LocalizationChunkDataGenerator.h"
#include "LockFile.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "LooseFilesCookArtifactReader.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "MeshCardRepresentation.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DataValidation.h"
#include "Misc/FileHelper.h"
#include "Misc/LocalTimestampDirectoryVisitor.h"
#include "Misc/NetworkVersion.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/RedirectCollector.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "PackageHelperFunctions.h"
#include "PlatformInfo.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/PlatformFileTrace.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "ProjectDescriptor.h"
#include "SceneUtils.h"
#include "Serialization/ArchiveCountMem.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CustomVersion.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Settings/ProjectPackagingSettings.h"
#include "ShaderCompiler.h"
#include "ShaderStats.h"
#include "Stats/StatsSystem.h"
#include "String/Find.h"
#include "String/ParseLines.h"
#include "String/ParseTokens.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "Templates/UnrealTemplate.h"
#include "TextureFormatManager.h"
#include "ThumbnailExternalCache.h"
#include "UnrealEdGlobals.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/Class.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/GarbageCollection.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectIterator.h"
#include "UserGeneratedContentLocalization.h"
#include "ZenCookArtifactReader.h"
#include "ZenStoreWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookOnTheFlyServer)

#define LOCTEXT_NAMESPACE "Cooker"

LLM_DEFINE_TAG(Cooker);

int32 GCookProgressDisplay = (int32)ECookProgressDisplayMode::RemainingPackages;
static FAutoConsoleVariableRef CVarCookDisplayMode(
	TEXT("cook.displaymode"),
	GCookProgressDisplay,
	TEXT("Controls the display for cooker logging of packages:\n")
	TEXT("  0: No display\n")
	TEXT("  1: Display the Count of packages remaining\n")
	TEXT("  2: Display each package by Name\n")
	TEXT("  3: Display Names and Count\n")
	TEXT("  4: Display the Instigator of each package\n")
	TEXT("  5: Display Instigators and Count\n")
	TEXT("  6: Display Instigators and Names\n")
	TEXT("  7: Display Instigators and Names and Count\n"),
	ECVF_Default);

float GCookProgressUpdateTime = 2.0f;
static FAutoConsoleVariableRef CVarCookDisplayUpdateTime(
	TEXT("cook.display.updatetime"),
	GCookProgressUpdateTime,
	TEXT("Controls the time before the cooker will send a new progress message.\n"),
	ECVF_Default);

float GCookProgressDiagnosticTime = 30.0f;
static FAutoConsoleVariableRef CVarCookDisplayDiagnosticTime(
	TEXT("Cook.display.diagnostictime"),
	GCookProgressDiagnosticTime,
	TEXT("Controls the time between cooker diagnostics messages.\n"),
	ECVF_Default);

float GCookProgressRepeatTime = 5.0f;
static FAutoConsoleVariableRef CVarCookDisplayRepeatTime(
	TEXT("cook.display.repeattime"),
	GCookProgressRepeatTime,
	TEXT("Controls the time before the cooker will repeat the same progress message.\n"),
	ECVF_Default);

float GCookProgressRetryBusyTime = 0.01f;
static FAutoConsoleVariableRef CVarCookRetryBusyTime(
	TEXT("Cook.retrybusytime"),
	GCookProgressRetryBusyTime,
	TEXT("Controls the time between retry attempts at save and load when the save and load queues are busy and there is no other work to do.\n"),
	ECVF_Default);

float GCookProgressWarnBusyTime = 120.0f;
static FAutoConsoleVariableRef CVarCookDisplayWarnBusyTime(
	TEXT("Cook.display.warnbusytime"),
	GCookProgressWarnBusyTime,
	TEXT("Controls the time before the cooker will issue a warning that there is a deadlock in a busy queue.\n"),
	ECVF_Default);

static int32 GCookTimeCVarControl = 0;
static FAutoConsoleVariableRef CVarCookTimeCVarControl(
	TEXT("cook.cvarcontrol"),
	GCookTimeCVarControl,
	TEXT("Controls how cvars for other platforms are managed during cooking:\n"
		"0: Completely disabled\n"
		"1: Redirects CVars for the cooking platform to the cooking DP\n"
		"2: Performs mode 1, and will also update the active value of any CVar flagged with ECVF_Preview to match the platform/DP value\n"
		"2: Same as mode 2, except it will update ALL CVars, not just ECVF_Preview CVars\n"
		"NOTE: CURRENTLY any non-zero mode will also enable redirecting a ShaderPlatform to the cooking platform"),
	ECVF_Default);


////////////////////////////////////////////////////////////////
/// Cook on the fly server
///////////////////////////////////////////////////////////////
UCookOnTheFlyServer* UCookOnTheFlyServer::ActiveCOTFS = nullptr;

static FName ScriptPackageNameEngine(TEXT("/Script/Engine"));

namespace UE::Cook
{

// Keep the old behavior of cooking all by default until we implement good feedback in the editor about the missing setting
static bool bCookAllByDefault = true;
static FAutoConsoleVariableRef CookAllByDefaultCVar(
	TEXT("Cook.CookAllByDefault"),
	bCookAllByDefault,
	TEXT("When FilesInPath is empty. Cook all packages by default."));

}

/* helper structs functions
 *****************************************************************************/

/**
 * Return the release asset registry filename for the release version supplied
 */
static FString GetReleaseVersionAssetRegistryPath(const FString& ReleaseVersion, const FString& PlatformName, const FString& RootOverride)
{
	// cache the part of the path which is static because getting the ProjectDir is really slow and also string manipulation
	const FString* ReleasesRoot;
	if (RootOverride.IsEmpty())
	{
		const static FString DefaultReleasesRoot = FPaths::ProjectDir() / FString(TEXT("Releases"));
		ReleasesRoot = &DefaultReleasesRoot;
	}
	else
	{
		ReleasesRoot = &RootOverride;
	}
	return (*ReleasesRoot) / ReleaseVersion / PlatformName;
}

template<typename T>
struct FOneTimeCommandlineReader
{
	T Value;
	FOneTimeCommandlineReader(const TCHAR* Match)
	{
		FParse::Value(FCommandLine::Get(), Match, Value);
	}
};

static FString GetCreateReleaseVersionAssetRegistryPath(const FString& ReleaseVersion, const FString& PlatformName)
{
	static FOneTimeCommandlineReader<FString> CreateReleaseVersionRoot(TEXT("-createreleaseversionroot="));
	return GetReleaseVersionAssetRegistryPath(ReleaseVersion, PlatformName, CreateReleaseVersionRoot.Value);
}

static FString GetBasedOnReleaseVersionAssetRegistryPath(const FString& ReleaseVersion, const FString& PlatformName)
{
	static FOneTimeCommandlineReader<FString> BasedOnReleaseVersionRoot(TEXT("-basedonreleaseversionroot="));
	return GetReleaseVersionAssetRegistryPath(ReleaseVersion, PlatformName, BasedOnReleaseVersionRoot.Value);
}

const FString& GetAssetRegistryFilename()
{
	static const FString AssetRegistryFilename = FString(TEXT("AssetRegistry.bin"));
	return AssetRegistryFilename;
}

static void ConditionalWaitOnCommandFile(FStringView GateName, TFunctionRef<void (FStringView)> CommandHandler = [](FStringView){});

/**
 * Uses the FMessageLog to log a message
 * 
 * @param Message to log
 * @param Severity of the message
 */
void LogCookerMessage( const FString& MessageText, EMessageSeverity::Type Severity)
{
	FMessageLog MessageLog("LogCook");

	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(Severity);

	Message->AddToken( FTextToken::Create( FText::FromString(MessageText) ) );
	MessageLog.AddMessage(Message);

	MessageLog.Notify(FText(), EMessageSeverity::Warning, false);
}

//////////////////////////////////////////////////////////////////////////
// Cook on the fly server interface adapter

class UCookOnTheFlyServer::FCookOnTheFlyServerInterface final
	: public UE::Cook::ICookOnTheFlyServer
{
public:
	FCookOnTheFlyServerInterface(UCookOnTheFlyServer& InCooker)
		: Cooker(InCooker)
	{
	}

	virtual ~FCookOnTheFlyServerInterface()
	{
	}

	virtual FString GetSandboxDirectory() const override
	{
		return Cooker.SandboxFile->GetSandboxDirectory();
	}

	virtual const ITargetPlatform* AddPlatform(FName PlatformName, bool& bOutAlreadyInitialized) override
	{
		UE::Cook::FPlatformManager::FReadScopeLock PlatformScopeLock(Cooker.PlatformManager->ReadLockPlatforms());
		const ITargetPlatform* TargetPlatform = AddPlatformInternal(PlatformName);
		if (!TargetPlatform)
		{
			UE_LOGF(LogCook, Warning, "Trying to add invalid platform '%ls' on the fly", *PlatformName.ToString());
			bOutAlreadyInitialized = false;
			return nullptr;
		}

		bOutAlreadyInitialized = Cooker.PlatformManager->HasSessionPlatform(TargetPlatform);
		Cooker.PlatformManager->AddRefCookOnTheFlyPlatform(PlatformName, Cooker);

		return TargetPlatform;
	}

	virtual void RemovePlatform(FName PlatformName) override
	{
		UE::Cook::FPlatformManager::FReadScopeLock PlatformScopeLock(Cooker.PlatformManager->ReadLockPlatforms());
		Cooker.PlatformManager->ReleaseCookOnTheFlyPlatform(PlatformName);
	}

	virtual bool IsSchedulerThread() const override
	{
		return IsInGameThread();
	}

	virtual void GetUnsolicitedFiles(const FName& PlatformName, const FString& Filename, const bool bIsCookable, TArray<FString>& OutUnsolicitedFiles) override
	{
		UE::Cook::FPlatformManager::FReadScopeLock PlatformsScopeLock(Cooker.PlatformManager->ReadLockPlatforms());
		const ITargetPlatform* TargetPlatform = AddPlatformInternal(PlatformName);
		if (!TargetPlatform)
		{
			UE_LOGF(LogCook, Warning, "Trying to get unsolicited files on the fly for an invalid platform '%ls'",
					*PlatformName.ToString());
			return;
		}
		Cooker.GetCookOnTheFlyUnsolicitedFiles(TargetPlatform, PlatformName.ToString(), OutUnsolicitedFiles, Filename, bIsCookable);
	}

	virtual bool EnqueueCookRequest(UE::Cook::FCookPackageRequest CookPackageRequest) override
	{
		using namespace UE::Cook;

		UE::Cook::FPlatformManager::FReadScopeLock PlatformsScopeLock(Cooker.PlatformManager->ReadLockPlatforms());
		const ITargetPlatform* TargetPlatform = AddPlatformInternal(CookPackageRequest.PlatformName);
		if (!TargetPlatform)
		{
			UE_LOGF(LogCook, Warning, "Trying to cook package on the fly for invalid platform '%ls'",
					*CookPackageRequest.PlatformName.ToString());
			return false;
		}

		FName StandardFileName(*FPaths::CreateStandardFilename(CookPackageRequest.Filename));
		UE_LOGF(LogCook, Verbose, "Enqueing cook request, Filename='%ls', Platform='%ls'", *CookPackageRequest.Filename, *CookPackageRequest.PlatformName.ToString());
		FFilePlatformRequest Request(StandardFileName, EInstigator::CookOnTheFly, TargetPlatform, MoveTemp(CookPackageRequest.CompletionCallback));
		Request.SetUrgent(true);
		Cooker.WorkerRequests->AddCookOnTheFlyRequest(MoveTemp(Request));

		return true;
	};

	virtual void MarkPackageDirty(const FName& PackageName) override
	{
		Cooker.WorkerRequests->AddCookOnTheFlyCallback([this, PackageName]()
			{
				UE::Cook::FPackageData* PackageData = Cooker.PackageDatas->FindPackageDataByPackageName(PackageName);
				if (!PackageData)
				{
					return;
				}
				if (PackageData->IsInProgress())
				{
					return;
				}
				if (!PackageData->HasAnyCookedPlatform())
				{
					return;
				}
				PackageData->ClearCookResults();
			});
	}

	virtual ICookedPackageWriter& GetPackageWriter(const ITargetPlatform* TargetPlatform) override
	{
		return GetPackageWriterInternal(TargetPlatform);
	}

private:

	const ITargetPlatform* AddPlatformInternal(const FName& PlatformName)
	{
		using namespace UE::Cook;

		const UE::Cook::FPlatformData* PlatformData = Cooker.PlatformManager->GetPlatformDataByName(PlatformName);
		if (!PlatformData)
		{
			UE_LOGF(LogCook, Warning, "Target platform %ls wasn't found.", *PlatformName.ToString());
			return nullptr;
		}

		ITargetPlatform* TargetPlatform = PlatformData->TargetPlatform;

		if (PlatformData->bIsSandboxInitialized)
		{
			return TargetPlatform;
		}

		if (IsInGameThread())
		{
			Cooker.AddCookOnTheFlyPlatformFromGameThread(TargetPlatform);
			return TargetPlatform;
		}

		FEventRef Event;
		Cooker.WorkerRequests->AddCookOnTheFlyCallback([this, &Event, &TargetPlatform]()
			{
				Cooker.AddCookOnTheFlyPlatformFromGameThread(TargetPlatform);
				Event->Trigger();
			});

		Event->Wait();
		return TargetPlatform;
	}

	ICookedPackageWriter& GetPackageWriterInternal(const ITargetPlatform* TargetPlatform)
	{
		if (IsInGameThread())
		{
			return Cooker.FindOrCreatePackageWriter(TargetPlatform);
		}

		FEventRef Event;
		ICookedPackageWriter* PackageWriter = nullptr;
		Cooker.WorkerRequests->AddCookOnTheFlyCallback([this, &Event, &TargetPlatform, &PackageWriter]()
			{
				PackageWriter = &Cooker.FindOrCreatePackageWriter(TargetPlatform);
				check(PackageWriter);
				Event->Trigger();
			});

		Event->Wait();
		return *PackageWriter;
	}
	UCookOnTheFlyServer& Cooker;
};

/* UCookOnTheFlyServer functions
 *****************************************************************************/

UCookOnTheFlyServer::UCookOnTheFlyServer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer),
	CurrentCookMode(ECookMode::CookOnTheFly),
	CookFlags(ECookInitializationFlags::None),
	bIsSavingPackage(false),
	AssetRegistry(nullptr)
{
}

UCookOnTheFlyServer::UCookOnTheFlyServer(FVTableHelper& Helper) :Super(Helper) {}

UCookOnTheFlyServer::~UCookOnTheFlyServer()
{
	ClearPackageStoreContexts();

	FCoreDelegates::TSOnFConfigCreated().RemoveAll(this);
	FCoreDelegates::TSOnFConfigDeleted().RemoveAll(this);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
	GetTargetPlatformManager()->GetOnTargetPlatformsInvalidatedDelegate().RemoveAll(this);
#if WITH_ADDITIONAL_CRASH_CONTEXTS
	FGenericCrashContext::OnAdditionalCrashContextDelegate().RemoveAll(this);
#endif

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ClearHierarchyTimers();
	}
}

// This tick only happens in the editor.  The cook commandlet directly calls tick on the side.
void UCookOnTheFlyServer::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCookOnTheFlyServer::Tick);
	LLM_SCOPE_BYTAG(Cooker);

	check(IsCookingInEditor());

	if (IsInSession())
	{
		// prevent autosave from happening until we are finished cooking
		// causes really bad hitches
		if (GUnrealEd)
		{
			constexpr float SecondsWarningTillAutosave = 10.0f;
			GUnrealEd->GetPackageAutoSaver().ForceMinimumTimeTillAutoSave(SecondsWarningTillAutosave);
		}
	}
	else
	{
		if (IsCookByTheBookMode() && !GIsSlowTask && IsCookFlagSet(ECookInitializationFlags::BuildDDCInBackground))
		{
			// if we are in the editor then precache some stuff ;)
			TArray<const ITargetPlatform*> CacheTargetPlatforms;
			const ULevelEditorPlaySettings* PlaySettings = GetDefault<ULevelEditorPlaySettings>();
			if (PlaySettings && (PlaySettings->LastExecutedLaunchModeType == LaunchMode_OnDevice))
			{
				FString DeviceName = PlaySettings->LastExecutedLaunchDevice.Left(PlaySettings->LastExecutedLaunchDevice.Find(TEXT("@")));
				CacheTargetPlatforms.Add(GetTargetPlatformManager()->FindTargetPlatform(DeviceName));
			}
			if (CacheTargetPlatforms.Num() > 0)
			{
				TickPrecacheObjectsForPlatforms(0.001f, CacheTargetPlatforms);
			}
		}
	}

	const float TickTimeSliceSeconds = 0.1f;
	TickCancels();
	if (IsCookOnTheFlyMode())
	{
		TickCookOnTheFly(TickTimeSliceSeconds);
	}
	else
	{
		check(IsCookByTheBookMode());
		TickCookByTheBook(TickTimeSliceSeconds);
	}
}

bool UCookOnTheFlyServer::IsTickable() const 
{ 
	return IsCookFlagSet(ECookInitializationFlags::AutoTick); 
}

TStatId UCookOnTheFlyServer::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UCookServer, STATGROUP_Tickables);
}

bool UCookOnTheFlyServer::StartCookOnTheFly(FCookOnTheFlyStartupOptions InCookOnTheFlyOptions)
{
	using namespace UE::Cook;

	GShaderCompilingManager->SkipShaderCompilation(bDisableShaderCompilationDuringCookOnTheFly);
	GShaderCompilingManager->SetAllowForIncompleteShaderMaps(bAllowIncompleteShaderMapsDuringCookOnTheFly);

	LLM_SCOPE_BYTAG(Cooker);
#if WITH_COTF
	check(IsCookOnTheFlyMode());
	//GetDerivedDataCacheRef().WaitForQuiescence(false);

#if PROFILE_NETWORK
	NetworkRequestEvent = FPlatformProcess::GetSynchEventFromPool();
#endif

	FBeginCookContext BeginContext = CreateBeginCookOnTheFlyContext(InCookOnTheFlyOptions);
	CreateSandboxFile(BeginContext);

	CookOnTheFlyServerInterface = MakeUnique<UCookOnTheFlyServer::FCookOnTheFlyServerInterface>(*this);
	WorkerRequests->InitializeCookOnTheFly();

	// Precreate the map of all possible target platforms so we can access the collection of existing platforms in a threadsafe manner
	// Each PlatformData in the map will be uninitialized until we call AddCookOnTheFlyPlatform for the platform
	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	for (const ITargetPlatform* TargetPlatform : TPM.GetTargetPlatforms())
	{
		PlatformManager->CreatePlatformData(*this, TargetPlatform);
	}
	PlatformManager->SetArePlatformsPrepopulated(true);

	LoadBeginCookConfigSettings(BeginContext);

	GRedirectCollector.OnStartupPackageLoadComplete();

	for (ITargetPlatform* TargetPlatform : InCookOnTheFlyOptions.TargetPlatforms)
	{
		AddCookOnTheFlyPlatformFromGameThread(TargetPlatform);
	}

	UE_LOGF(LogCook, Display, "Starting '%ls' cook-on-the-fly server",
		IsUsingZenStore() ? TEXT("Zen") : TEXT("Network File"));

	FCookOnTheFlyNetworkServerOptions NetworkServerOptions;
	NetworkServerOptions.Protocol = CookOnTheFlyOptions->bPlatformProtocol ? ECookOnTheFlyNetworkServerProtocol::Platform : ECookOnTheFlyNetworkServerProtocol::Tcp;
	NetworkServerOptions.Port = CookOnTheFlyOptions->Port;
	if (!InCookOnTheFlyOptions.TargetPlatforms.IsEmpty())
	{
		NetworkServerOptions.TargetPlatforms = InCookOnTheFlyOptions.TargetPlatforms;
	}
	else
	{
		NetworkServerOptions.TargetPlatforms = TPM.GetTargetPlatforms();
	}

	ICookOnTheFlyNetworkServerModule& CookOnTheFlyNetworkServerModule = FModuleManager::LoadModuleChecked<ICookOnTheFlyNetworkServerModule>(TEXT("CookOnTheFlyNetServer"));
	CookOnTheFlyNetworkServer = CookOnTheFlyNetworkServerModule.CreateServer(NetworkServerOptions);

	CookOnTheFlyNetworkServer->OnClientConnected().AddLambda([this](ICookOnTheFlyClientConnection& Connection)
		{
			if (Connection.GetTargetPlatform())
			{
				bool bAlreadyInitialized = false;
				CookOnTheFlyServerInterface->AddPlatform(Connection.GetPlatformName(), bAlreadyInitialized);
			}
			if (ODSCClientData)
			{
				ODSCClientData->OnClientConnected(&Connection);
			}
		});

	CookOnTheFlyNetworkServer->OnClientDisconnected().AddLambda([this](ICookOnTheFlyClientConnection& Connection)
		{
			if (ODSCClientData)
			{
				ODSCClientData->OnClientDisconnected(&Connection);
			}
		});

	CookOnTheFlyNetworkServer->OnRequest(ECookOnTheFlyMessage::RecompileShaders).BindLambda([this](ICookOnTheFlyClientConnection& Connection, const FCookOnTheFlyRequest& Request)
		{
			FCookOnTheFlyResponse Response(Request);

			if (!Connection.GetTargetPlatform())
			{
				UE_LOGF(LogCook, Warning, "RecompileShadersRequest from editor client");
				Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Error);
			}
			else
			{
				TArray<FString> RecompileModifiedFiles;
				TArray<uint8> MeshMaterialMaps;
				TArray<uint8> GlobalShaderMap;
				TArray<TStrongObjectPtr<UMaterialInterface>> LoadedMaterialsToRecompile;

				FShaderRecompileData RecompileData(Connection.GetTargetPlatform()->PlatformName(), &RecompileModifiedFiles, &MeshMaterialMaps, &GlobalShaderMap);
				{
					TUniquePtr<FArchive> Ar = Request.ReadBody();
					*Ar << RecompileData;
				}

				const void* ConnectionPtr = &Connection;
				RecompileData.LoadedMaterialsToRecompile = &LoadedMaterialsToRecompile;
				RecompileData.ODSCCustomLoadMaterial = &FODSCClientData::FindMaterial;

				FEventRef RecompileCompletedEvent;
				UE::Cook::FRecompileShaderCompletedCallback RecompileCompleted = [this, &RecompileCompletedEvent, ConnectionPtr, &LoadedMaterialsToRecompile, RecompileDataCommandType = RecompileData.CommandType]()
				{
					if (ODSCClientData)
					{
						if (RecompileDataCommandType == ODSCRecompileCommand::ResetMaterialCache)
						{
							ODSCClientData->FlushClientPersistentData(ConnectionPtr);
						}
						else
						{
							ODSCClientData->KeepClientPersistentData(ConnectionPtr, LoadedMaterialsToRecompile);
						}
					}
					LoadedMaterialsToRecompile.Empty();

					RecompileCompletedEvent->Trigger();
				};

				PackageTracker->RecompileRequests.Enqueue({ RecompileData, MoveTemp(RecompileCompleted) });
				RecompileRequestsPollable->Trigger(*this);

				RecompileCompletedEvent->Wait();

				{
					TUniquePtr<FArchive> Ar = Response.WriteBody();
					*Ar << MeshMaterialMaps;
					*Ar << GlobalShaderMap;
				}
			}
			return Connection.SendMessage(Response);
		});

	if (IsUsingZenStore())
	{
		CookOnTheFlyRequestManager = MakeIoStoreCookOnTheFlyRequestManager(*CookOnTheFlyServerInterface, AssetRegistry, CookOnTheFlyNetworkServer.ToSharedRef());
	}
	else
	{
		CookOnTheFlyRequestManager = MakeNetworkFileCookOnTheFlyRequestManager(*CookOnTheFlyServerInterface, CookOnTheFlyNetworkServer.ToSharedRef());
	}

	if (bRunningAsShaderServer)
	{
		BlockOnAssetRegistry(TConstArrayView<FString>());
	}

	if (CookOnTheFlyNetworkServer->Start())
	{
		TArray<TSharedPtr<FInternetAddr>> ListenAddresses;
		if (!CookOnTheFlyNetworkServer->GetAddressList(ListenAddresses))
		{
			UE_LOGF(LogCook, Fatal, "Unable to get any ListenAddresses for Unreal Network file server!");
		}

		if (ListenAddresses.Num() > 0)
		{
			UE_LOGF(LogCook, Display, "Unreal Network File Server is ready for client connections on %ls!", *ListenAddresses[0]->ToString(true));
		}
	}
	else
	{
		UE_LOGF(LogCook, Fatal, "Failed starting Unreal Network file server!");
	}
	BeginCookEditorSystems();
	InitializePollables();

	const bool bInitialized = CookOnTheFlyRequestManager->Initialize();

	BroadcastCookStarted();

	return bInitialized;
#else
	return false;
#endif
}

void UCookOnTheFlyServer::InitializeShadersForCookOnTheFly(const TArrayView<ITargetPlatform* const>& NewTargetPlatforms)
{
	UE_LOGF(LogCook, Display, "Initializing shaders for cook-on-the-fly");
	SaveGlobalShaderMapFiles(NewTargetPlatforms, ODSCRecompileCommand::Global);
}

void UCookOnTheFlyServer::AddCookOnTheFlyPlatformFromGameThread(ITargetPlatform* TargetPlatform)
{
	UE::Cook::FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
	check(PlatformData != nullptr); // should have been checked by the caller
	if (PlatformData->bIsSandboxInitialized || bRunningAsShaderServer)
	{
		return;
	}

	FBeginCookContext BeginContext = CreateAddPlatformContext(TargetPlatform);

	// Initialize systems and settings that the rest of AddCookOnTheFlyPlatformFromGameThread depends on
	// Functions in this section are ordered and can depend on the functions before them
	FindOrCreateSaveContexts(BeginContext.TargetPlatforms);
	LoadBeginCookIncrementalFlags(BeginContext);

	// Initialize systems referenced by later stages or that need to start early for async performance
	// Functions in this section must not need to read/write the SandboxDirectory or MemoryCookedPackages
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance
	RefreshPlatformAssetRegistries(BeginContext.TargetPlatforms);

	// Clear the sandbox directory, or preserve it and populate incremental cooks
	// Clear in-memory CookedPackages, or preserve them and cook incrementally in-process
	BeginCookSandbox(BeginContext);

	// Initialize systems that need to write files to the sandbox directory, for consumption later in AddCookOnTheFlyPlatformFromGameThread
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance
	InitializeShadersForCookOnTheFly(BeginContext.TargetPlatforms);
	// SaveAssetRegistry is done in CookByTheBookFinished for CBTB, but we need at the start of CookOnTheFly to send
	// as startup information to connecting clients
	// We copy the data from the global AssetRegistry since we don't have the up to date data from AssetRegistry
	// saves yet. 
	PlatformData->RegistryGenerator->InitializeAssetRegistryForCookOnTheFly(
		PlatformData->AssetRegistryDevelopmentSaveOptions);
	uint64 DevelopmentAssetRegistryHash = 0;
	PlatformData->RegistryGenerator->SaveAssetRegistry(GetSandboxAssetRegistryFilename(), true, false,
		DevelopmentAssetRegistryHash, TSet<FName>(), PlatformData->AssetRegistryDevelopmentSaveOptions);

	// Initialize systems that nothing in AddCookOnTheFlyPlatformFromGameThread references
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance
	BeginCookPackageWriters(BeginContext);

	// SaveCurrentIniSettings is done in CookByTheBookFinished for CBTB, but we don't have a definite end point in CookOnTheFly so we write it at the start
	// This will miss settings that are accessed during the cook
	// TODO: A better way of handling ini settings
	SaveCurrentIniSettings(TargetPlatform);
}

void UCookOnTheFlyServer::StartCookOnTheFlySessionFromGameThread(ITargetPlatform* TargetPlatform)
{
	const bool bIsFirstSession = PlatformManager->GetNumSessionPlatforms() == 0;
	if (bIsFirstSession)
	{
		InitializeSession();
	}
	PlatformManager->AddSessionPlatform(*this, TargetPlatform);

	CalculateGlobalDependenciesHashes();

	TargetPlatform->InitializeForCook();
	ResetCook({ TPair<const ITargetPlatform*,bool>{TargetPlatform, true /* bResetResults */} });

	// Blocking on the AssetRegistry needs to wait until the session starts because it needs all plugins loaded.
	// AddCookOnTheFlyPlatformFromGameThread can be called on cooker startup which occurs in UUnrealEdEngine::Init
	// before all plugins are loaded.
	BlockOnAssetRegistry(TConstArrayView<FString>());

	if (bIsFirstSession)
	{
		UE::Cook::FDelegates::ConfigureCookSession.Broadcast(*this);
	}

	if (CookOnTheFlyRequestManager)
	{
		FName PlatformName(*TargetPlatform->PlatformName());
		CookOnTheFlyRequestManager->OnSessionStarted(PlatformName, bFirstCookInThisProcess);
	}
}

void UCookOnTheFlyServer::OnTargetPlatformsInvalidated()
{
	using namespace UE::Cook;

	check(IsInGameThread());
	TMap<ITargetPlatform*, ITargetPlatform*> Remap = PlatformManager->RemapTargetPlatforms();

	PackageDatas->RemapTargetPlatforms(Remap);
	PackageTracker->RemapTargetPlatforms(Remap);
	WorkerRequests->RemapTargetPlatforms(Remap);
	for (TUniquePtr<FRequestCluster>& Cluster : PackageDatas->GetRequestQueue().GetRequestClusters())
	{
		Cluster->RemapTargetPlatforms(Remap);
	}
	for (FDiscoveryQueueElement& Element : PackageDatas->GetRequestQueue().GetDiscoveryQueue())
	{
		Element.ReachablePlatforms.RemapTargetPlatforms(Remap);
	}

	if (PlatformManager->GetArePlatformsPrepopulated())
	{
		for (const ITargetPlatform* TargetPlatform : GetTargetPlatformManager()->GetTargetPlatforms())
		{
			PlatformManager->CreatePlatformData(*this, TargetPlatform);
		}
	}
}

bool UCookOnTheFlyServer::BroadcastFileserverPresence( const FGuid &InstanceId )
{
	
	TArray<FString> AddressStringList;

	for ( int i = 0; i < NetworkFileServers.Num(); ++i )
	{
		TArray<TSharedPtr<FInternetAddr> > AddressList;
		INetworkFileServer *NetworkFileServer = NetworkFileServers[i];
		if ((NetworkFileServer == NULL || !NetworkFileServer->IsItReadyToAcceptConnections() || !NetworkFileServer->GetAddressList(AddressList)))
		{
			LogCookerMessage( FString(TEXT("Failed to create network file server")), EMessageSeverity::Error );
			continue;
		}

		// broadcast our presence
		if (InstanceId.IsValid())
		{
			for (int32 AddressIndex = 0; AddressIndex < AddressList.Num(); ++AddressIndex)
			{
				AddressStringList.Add(FString::Printf( TEXT("%s://%s"), *NetworkFileServer->GetSupportedProtocol(),  *AddressList[AddressIndex]->ToString(true)));
			}

		}
	}

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = FMessageEndpoint::Builder("UCookOnTheFlyServer").Build();

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FFileServerReady>(AddressStringList, InstanceId), EMessageScope::Network);
	}		
	
	return true;
}

/*----------------------------------------------------------------------------
	FArchiveFindReferences.
----------------------------------------------------------------------------*/
/**
 * Archive for gathering all the object references to other objects
 */
class FArchiveFindReferences : public FArchiveUObject
{
private:
	/**
	 * I/O function.  Called when an object reference is encountered.
	 *
	 * @param	Obj		a pointer to the object that was encountered
	 */
	FArchive& operator<<( UObject*& Obj ) override
	{
		if( Obj )
		{
			FoundObject( Obj );
		}
		return *this;
	}

	virtual FArchive& operator<< (struct FSoftObjectPtr& Value) override
	{
		if ( Value.Get() )
		{
			Value.Get()->Serialize( *this );
		}
		return *this;
	}
	virtual FArchive& operator<< (struct FSoftObjectPath& Value) override
	{
		if ( Value.ResolveObject() )
		{
			Value.ResolveObject()->Serialize( *this );
		}
		return *this;
	}


	void FoundObject( UObject* Object )
	{
		if ( RootSet.Find(Object) == NULL )
		{
			if ( Exclude.Find(Object) == INDEX_NONE )
			{
				// remove this check later because don't want this happening in development builds
				//check(RootSetArray.Find(Object)==INDEX_NONE);

				RootSetArray.Add( Object );
				RootSet.Add(Object);
				Found.Add(Object);
			}
		}
	}


	/**
	 * list of Outers to ignore;  any objects encountered that have one of
	 * these objects as an Outer will also be ignored
	 */
	TArray<UObject*> &Exclude;

	/** list of objects that have been found */
	TSet<UObject*> &Found;
	
	/** the objects to display references to */
	TArray<UObject*> RootSetArray;
	/** Reflection of the rootsetarray */
	TSet<UObject*> RootSet;

public:

	/**
	 * Constructor
	 * 
	 * @param	inOutputAr		archive to use for logging results
	 * @param	inOuter			only consider objects that do not have this object as its Outer
	 * @param	inSource		object to show references for
	 * @param	inExclude		list of objects that should be ignored if encountered while serializing SourceObject
	 */
	FArchiveFindReferences( TSet<UObject*> InRootSet, TSet<UObject*> &inFound, TArray<UObject*> &inExclude )
		: Exclude(inExclude)
		, Found(inFound)
		, RootSet(InRootSet)
	{
		ArIsObjectReferenceCollector = true;
		this->SetIsSaving(true);

		for ( UObject* Object : RootSet )
		{
			RootSetArray.Add( Object );
		}
		
		// loop through all the objects in the root set and serialize them
		for ( int RootIndex = 0; RootIndex < RootSetArray.Num(); ++RootIndex )
		{
			UObject* SourceObject = RootSetArray[RootIndex];

			// quick sanity check
			check(SourceObject);
			check(SourceObject->IsValidLowLevel());

			SourceObject->Serialize( *this );
		}

	}

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override { return TEXT("FArchiveFindReferences"); }
};

void UCookOnTheFlyServer::GetDependentPackages(const TSet<UPackage*>& RootPackages, TSet<FName>& FoundPackages)
{
	TSet<FName> RootPackageFNames;
	for (const UPackage* RootPackage : RootPackages)
	{
		RootPackageFNames.Add(RootPackage->GetFName());
	}


	GetDependentPackages(RootPackageFNames, FoundPackages);

}


void UCookOnTheFlyServer::GetDependentPackages( const TSet<FName>& RootPackages, TSet<FName>& FoundPackages )
{
	TArray<FName> FoundPackagesArray;
	for (const FName& RootPackage : RootPackages)
	{
		FoundPackagesArray.Add(RootPackage);
		FoundPackages.Add(RootPackage);
	}

	int FoundPackagesCounter = 0;
	while ( FoundPackagesCounter < FoundPackagesArray.Num() )
	{
		TArray<FName> PackageDependencies;
		if (AssetRegistry->GetDependencies(FoundPackagesArray[FoundPackagesCounter], PackageDependencies, UE::AssetRegistry::EDependencyCategory::Package) == false)
		{
			// this could happen if we are in the editor and the dependency list is not up to date

			if (IsCookingInEditor() == false)
			{
				UE_LOGF(LogCook, Fatal, "Unable to find package %ls in asset registry.  Can't generate cooked asset registry", *FoundPackagesArray[FoundPackagesCounter].ToString());
			}
			else
			{
				UE_LOGF(LogCook, Warning, "Unable to find package %ls in asset registry, cooked asset registry information may be invalid ", *FoundPackagesArray[FoundPackagesCounter].ToString());
			}
		}
		++FoundPackagesCounter;
		for ( const FName& OriginalPackageDependency : PackageDependencies )
		{
			// check(PackageDependency.ToString().StartsWith(TEXT("/")));
			FName PackageDependency = OriginalPackageDependency;
			FString PackageDependencyString = PackageDependency.ToString();

			FText OutReason;
			const bool bIncludeReadOnlyRoots = true; // Dependency packages are often script packages (read-only)
			if (!FPackageName::IsValidLongPackageName(PackageDependencyString, bIncludeReadOnlyRoots, &OutReason))
			{
				const FText FailMessage = FText::Format(LOCTEXT("UnableToGeneratePackageName", "Unable to generate long package name for {0}. {1}"),
					FText::FromString(PackageDependencyString), OutReason);

				LogCookerMessage(FailMessage.ToString(), EMessageSeverity::Warning);
				continue;
			}
			else if (FPackageName::IsScriptPackage(PackageDependencyString) || FPackageName::IsMemoryPackage(PackageDependencyString))
			{
				continue;
			}

			if ( FoundPackages.Contains(PackageDependency) == false )
			{
				FoundPackages.Add(PackageDependency);
				FoundPackagesArray.Add( PackageDependency );
			}
		}
	}

}

bool UCookOnTheFlyServer::ContainsMap(const FName& PackageName) const
{
	TArray<FAssetData> Assets;
	ensure(AssetRegistry->GetAssetsByPackageName(PackageName, Assets, true /* IncludeOnlyDiskAssets */));

	for (const FAssetData& Asset : Assets)
	{
		UClass* AssetClass = Asset.GetClass();
		if (AssetClass && (AssetClass->IsChildOf(UWorld::StaticClass()) || AssetClass->IsChildOf(ULevel::StaticClass())))
		{
			return true;
		}
	}
	return false;
}

bool UCookOnTheFlyServer::ContainsRedirector(const FName& PackageName, TMap<FSoftObjectPath, FSoftObjectPath>& RedirectedPaths) const
{
	bool bFoundRedirector = false;
	TArray<FAssetData> Assets;
	ensure(AssetRegistry->GetAssetsByPackageName(PackageName, Assets, true /* IncludeOnlyDiskAssets */));

	for (const FAssetData& Asset : Assets)
	{
		if (Asset.IsRedirector())
		{
			FSoftObjectPath RedirectedPath;
			FString RedirectedPathString;
			if (Asset.GetTagValue("DestinationObject", RedirectedPathString))
			{
				ConstructorHelpers::StripObjectClass(RedirectedPathString);
				RedirectedPath = FSoftObjectPath(RedirectedPathString);
				FAssetData DestinationData = AssetRegistry->GetAssetByObjectPath(RedirectedPath, true);
				TSet<FSoftObjectPath> SeenPaths;

				SeenPaths.Add(RedirectedPath);

				// Need to follow chain of redirectors
				while (DestinationData.IsRedirector())
				{
					if (DestinationData.GetTagValue("DestinationObject", RedirectedPathString))
					{
						ConstructorHelpers::StripObjectClass(RedirectedPathString);
						RedirectedPath = FSoftObjectPath(RedirectedPathString);

						if (SeenPaths.Contains(RedirectedPath))
						{
							// Recursive, bail
							DestinationData = FAssetData();
						}
						else
						{
							SeenPaths.Add(RedirectedPath);
							DestinationData = AssetRegistry->GetAssetByObjectPath(RedirectedPath, true);
						}
					}
					else
					{
						// Can't extract
						DestinationData = FAssetData();						
					}
				}

				// DestinationData may be invalid if this is a subobject, check package as well
				bool bDestinationValid = DestinationData.IsValid();

				if (!bDestinationValid)
				{
					if (RedirectedPath.IsValid())
					{
						FName StandardPackageName = PackageDatas->GetFileNameByPackageName(FName(*FPackageName::ObjectPathToPackageName(RedirectedPathString)));
						if (!StandardPackageName.IsNone())
						{
							bDestinationValid = true;
						}
					}
				}

				if (bDestinationValid)
				{
					RedirectedPaths.Add(Asset.GetSoftObjectPath(), RedirectedPath);
				}
				else
				{
					RedirectedPaths.Add(Asset.GetSoftObjectPath(), FSoftObjectPath{});
					UE_LOGF(LogCook, Log, "Found redirector in package %ls pointing to deleted object %ls", *PackageName.ToString(), *RedirectedPathString);
				}

				bFoundRedirector = true;
			}
		}
	}
	return bFoundRedirector;
}

bool UCookOnTheFlyServer::IsCookingInEditor() const
{
	return ::IsCookingInEditor(CurrentCookMode);
}

bool UCookOnTheFlyServer::IsRealtimeMode() const 
{
	return ::IsRealtimeMode(CurrentCookMode);
}

bool UCookOnTheFlyServer::IsCookByTheBookMode() const
{
	return ::IsCookByTheBookMode(CurrentCookMode);
}

bool UCookOnTheFlyServer::IsDirectorCookByTheBook() const
{
	return ::IsCookByTheBookMode(DirectorCookMode);
}

bool UCookOnTheFlyServer::IsUsingZenStore() const
{
	return bZenStore;
}

bool UCookOnTheFlyServer::IsUsingIoStore() const
{
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	return PackagingSettings->bUseIoStore;
}

bool UCookOnTheFlyServer::IsCookOnTheFlyMode() const
{
	return ::IsCookOnTheFlyMode(CurrentCookMode);
}

bool UCookOnTheFlyServer::IsDirectorCookOnTheFly() const
{
	return ::IsCookOnTheFlyMode(DirectorCookMode);
}

bool UCookOnTheFlyServer::IsCookWorkerMode() const
{
	return ::IsCookWorkerMode(CurrentCookMode);
}

UE::Cook::ECookPhase UCookOnTheFlyServer::GetCookPhase() const
{
	using namespace UE::Cook;
	return !bKickedBuildDependencies ? ECookPhase::Cook : ECookPhase::BuildDependencies;
}

void UCookOnTheFlyServer::SetCookPhase(UE::Cook::ECookPhase TargetPhase)
{
	using namespace UE::Cook;

	switch (TargetPhase)
	{
	case ECookPhase::Cook:
		bKickedBuildDependencies = false;
		break;
	case ECookPhase::BuildDependencies:
		bKickedBuildDependencies = true;
		break;
	default:
		checkNoEntry();
		break;
	}
}

bool UCookOnTheFlyServer::IsUsingLegacyCookOnTheFlyScheduling() const
{
	return CookOnTheFlyRequestManager && CookOnTheFlyRequestManager->ShouldUseLegacyScheduling();
}

bool UCookOnTheFlyServer::IsCreatingReleaseVersion()
{
	return !CookByTheBookOptions->CreateReleaseVersion.IsEmpty();
}

bool UCookOnTheFlyServer::IsCookingDLC() const
{
	// we are cooking DLC when the DLC name is setup
	return !CookByTheBookOptions->DlcName.IsEmpty();
}

bool UCookOnTheFlyServer::IsCookingDLCPluginOnly() const
{
	return IsCookingDLC() && CookByTheBookOptions->bDlcPluginOnly;
}

bool UCookOnTheFlyServer::IsCookingAgainstFixedBase() const
{
	return IsCookingDLC() && CookByTheBookOptions->bCookAgainstFixedBase;
}

bool UCookOnTheFlyServer::ShouldPopulateFullAssetRegistry() const
{
	return IsCookWorkerMode() || !IsCookingDLC() || CookByTheBookOptions->bDlcLoadMainAssetRegistry;
}

FString UCookOnTheFlyServer::GetBaseDirectoryForDLC() const
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(CookByTheBookOptions->DlcName);
	if (Plugin.IsValid())
	{
		return Plugin->GetBaseDir();
	}

	return FPaths::ProjectPluginsDir() / CookByTheBookOptions->DlcName;
}

FString UCookOnTheFlyServer::GetMountedAssetPathForDLC() const
{
	return GetMountedAssetPathForPlugin(CookByTheBookOptions->DlcName);
}

FString UCookOnTheFlyServer::GetMountedAssetPathForPlugin(const FString& InPluginName)
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(InPluginName);
	if (Plugin.IsValid())
	{
		return Plugin->GetMountedAssetPath();
	}

	return FString::Printf(TEXT("/%s/"), *InPluginName);
}

FString UCookOnTheFlyServer::GetContentDirectoryForDLC() const
{
	return GetBaseDirectoryForDLC() / TEXT("Content");
}

FString UCookOnTheFlyServer::GetMetadataDirectory() const
{
	FString ProjectOrPluginRoot = !IsCookingDLC() ? FPaths::ProjectDir() : GetBaseDirectoryForDLC();
	return ProjectOrPluginRoot / TEXT("Metadata");
}

// allow for a command line to start async preloading a Development AssetRegistry if requested
static FEventRef GPreloadAREvent(EEventMode::ManualReset);
static FEventRef GPreloadARInfoEvent(EEventMode::ManualReset);
static FAssetRegistryState GPreloadedARState;
static FString GPreloadedARPath;
static FDelayedAutoRegisterHelper GPreloadARHelper(EDelayedRegisterRunPhase::EarliestPossiblePluginsLoaded, []()
	{
		// if we don't want to preload, then do nothing here
		if (!FParse::Param(FCommandLine::Get(), TEXT("PreloadDevAR")))
		{
			GPreloadAREvent->Trigger();
			GPreloadARInfoEvent->Trigger();
			return;
		}

		// kick off a thread to preload the DevelopmentAssetRegistry
		Async(EAsyncExecution::Thread, []()
			{
				FString BasedOnReleaseVersion;
				FString DevelopmentAssetRegistryPlatformOverride;
				// some manual commandline processing - we don't have the cooker params set properly yet - but this is not a generic solution, it is opt-in
				if (FParse::Value(FCommandLine::Get(), TEXT("BasedOnReleaseVersion="), BasedOnReleaseVersion) &&
					FParse::Value(FCommandLine::Get(), TEXT("DevelopmentAssetRegistryPlatformOverride="), DevelopmentAssetRegistryPlatformOverride))
				{
					// get the AR file path and see if it exists
					GPreloadedARPath = GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, DevelopmentAssetRegistryPlatformOverride) / TEXT("Metadata") / GetDevelopmentAssetRegistryFilename();

					// now that the info has been set, we can allow the other side of this code to check the ARPath
					GPreloadARInfoEvent->Trigger();

					TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*GPreloadedARPath));
					if (Reader)
					{
						GPreloadedARState.Serialize(*Reader.Get(), FAssetRegistrySerializationOptions());
					}
				}
				else
				{
					GPreloadARInfoEvent->Trigger();
				}

				GPreloadAREvent->Trigger();
			}
		);
	}
);


COREUOBJECT_API extern void SetOutputCookingWarnings(bool bValue);
COREUOBJECT_API extern bool GetOutputCookingWarnings();

void UCookOnTheFlyServer::WaitForRequests(int TimeoutMs)
{
	WorkerRequests->WaitForCookOnTheFlyEvents(TimeoutMs);
}

bool UCookOnTheFlyServer::HasRemainingWork() const
{ 
	return WorkerRequests->HasExternalRequests() ||
		PackageDatas->GetMonitor().GetNumInProgress() > 0;
}

bool UCookOnTheFlyServer::RequestPackage(const FName& StandardFileName, const TArrayView<const ITargetPlatform* const>& TargetPlatforms, const bool bForceFrontOfQueue)
{
	using namespace UE::Cook;

	if (IsCookOnTheFlyMode())
	{
		bCookOnTheFlyExternalRequests = true;
		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			AddCookOnTheFlyPlatformFromGameThread(const_cast<ITargetPlatform*>(TargetPlatform));
			PlatformManager->AddRefCookOnTheFlyPlatform(FName(*TargetPlatform->PlatformName()), *this);
		}
	}

	FFilePlatformRequest Request(StandardFileName, EInstigator::RequestPackageFunction, TargetPlatforms);
	Request.SetUrgent(bForceFrontOfQueue);
	WorkerRequests->AddPublicInterfaceRequest(MoveTemp(Request), bForceFrontOfQueue);
	return true;
}

bool UCookOnTheFlyServer::RequestPackage(const FName& StandardPackageFName, const bool bForceFrontOfQueue)
{
	check(!IsCookOnTheFlyMode()); // Invalid to call RequestPackage without a list of TargetPlatforms if we are in CookOnTheFly
	return RequestPackage(StandardPackageFName, PlatformManager->GetSessionPlatforms(), bForceFrontOfQueue);
}

uint32 UCookOnTheFlyServer::TickCookByTheBook(const float TimeSlice, ECookTickFlags TickFlags)
{
	check(IsCookByTheBookMode());

	LLM_SCOPE_BYTAG(Cooker);
	COOK_STAT(FScopedDurationTimer TickTimer(DetailedCookStats::TickCookOnTheSideTimeSec));
	UE::Cook::FTickStackData StackData(TimeSlice, TickFlags);

	TickMainCookLoop(StackData);

	CookByTheBookOptions->CookTime += StackData.Timer.GetTickTimeTillNow();
	// Make sure no UE_SCOPED_HIERARCHICAL_COOKTIMERs are around CookByTheBookFinished or CancelCookByTheBook, as those functions delete memory for them
	if (StackData.bCookCancelled)
	{
		CancelCookByTheBook();
	}
	else if (IsInSession() && StackData.bCookComplete)
	{
		UpdateDisplay(StackData, true /* bForceDisplay */);
		CookByTheBookFinished();
	}
	return StackData.ResultFlags;
}

void UCookOnTheFlyServer::RunCookList(ECookListOptions CookListOptions)
{
	using namespace UE::Cook;

	TGuardValue<bool> SetRunCookListMode(bCookListMode, true);

	FTickStackData StackData(MAX_flt, ECookTickFlags::None);
	PumpExternalRequests(StackData.Timer);
	ProcessUnsolicitedPackages();
	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	while (RequestQueue.HasRequestsToExplore())
	{
		int32 NumPushed;
		PumpRequests(StackData, NumPushed);
	}

	TArray<FPackageData*> ReportedDatas;
	PackageDatas->LockAndEnumeratePackageDatas([&ReportedDatas, CookListOptions](FPackageData* PackageData)
	{
		bool bIncludePackage;
		if (EnumHasAnyFlags(CookListOptions, ECookListOptions::ShowRejected))
		{
			bIncludePackage = PackageData->HasInstigator(EReachability::Runtime);
			if (bIncludePackage)
			{
				// Skip printing out a message for external actors
				TStringBuilder<256> PackageNameStr(InPlace, PackageData->GetPackageName());
				if (INDEX_NONE != UE::String::FindFirst(PackageNameStr, ULevel::GetExternalActorsFolderName()) ||
					INDEX_NONE != UE::String::FindFirst(PackageNameStr, FPackagePath::GetExternalObjectsFolderName()))
				{
					bIncludePackage = false;
				}
			}
		}
		else
		{
			bIncludePackage = PackageData->IsInProgress() || PackageData->HasAnyCookedPlatform();
		}
		if (bIncludePackage)
		{
			ReportedDatas.Add(PackageData);
		}
	});
	Algo::Sort(ReportedDatas, [](FPackageData* A, FPackageData* B)
	{
		return A->GetPackageName().LexicalLess(B->GetPackageName());
	});

	bool bShowInstigators = (GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators) != 0;
	for (FPackageData* PackageData : ReportedDatas)
	{
		bool bRejected = !PackageData->IsInProgress() && !PackageData->HasAnyCookedPlatform();
		UE_LOGF(LogCookList, Display, "%ls%ls%ls%ls",
			bRejected ? TEXT("Rejected: ") : TEXT(""),
			*WriteToString<256>(PackageData->GetPackageName()),
			bShowInstigators ? TEXT(", Instigator: ") : TEXT(""),
			bShowInstigators ? *PackageData->GetInstigator(EReachability::Runtime).ToString() : TEXT(""));
	}
}

uint32 UCookOnTheFlyServer::TickCookOnTheFly(const float TimeSlice, ECookTickFlags TickFlags)
{
	check(IsCookOnTheFlyMode());

	LLM_SCOPE_BYTAG(Cooker);
	COOK_STAT(FScopedDurationTimer TickTimer(DetailedCookStats::TickCookOnTheSideTimeSec));
	UE::Cook::FTickStackData StackData(TimeSlice, TickFlags);

	TickNetwork();
	TickMainCookLoop(StackData);

	return StackData.ResultFlags;
}

uint32 UCookOnTheFlyServer::TickCookWorker()
{
	check(IsCookWorkerMode());

	LLM_SCOPE_BYTAG(Cooker);
	UE::Cook::FTickStackData StackData(MAX_flt, ECookTickFlags::None);

	TickMainCookLoop(StackData);
	if (StackData.bCookCancelled)
	{
		CancelAllQueues();
		// Make sure no UE_SCOPED_HIERARCHICAL_COOKTIMERs are around ShutdownCookSession, as ShutdownCookSession deletes memory for them
		ShutdownCookSession();
		SetIdleStatus(StackData, EIdleStatus::Done);
	}

	return StackData.ResultFlags;
}

void UCookOnTheFlyServer::TickMainCookLoop(UE::Cook::FTickStackData& StackData)
{
	if (!IsInSession())
	{
		return;
	}
	// Set the soft time limit to spend pumping any action at 30s, so we periodically check for pollables
	// This is useful on CookWorkers to poll the CookClientWorker and check for CookDirector shutdown.
	constexpr float MaxActionTimeSlice = 30.f;
	StackData.Timer.SetActionTimeSlice(FMath::Min(StackData.Timer.GetTickTimeSlice(), MaxActionTimeSlice));

	UE_SCOPED_HIERARCHICAL_COOKTIMER(TickMainCookLoop);
	bool bContinueTick = true;
	while (bContinueTick && (!IsEngineExitRequested() || (IsCookByTheBookMode() && !IsCookingInEditor())))
	{
		TickCookStatus(StackData);

		ECookAction CookAction = DecideNextCookAction(StackData);
		int32 NumPushed;
		bool bBusy;
		switch (CookAction)
		{
		case ECookAction::Request:
			PumpRequests(StackData, NumPushed);
			if (NumPushed > 0)
			{
				SetLoadBusy(false);
			}
			break;
		case ECookAction::Load: [[fallthrough]];
		case ECookAction::LoadLimited:
		{
			uint32 CurrentLoadQueueLength = CookAction == ECookAction::Load ? 0 : DesiredLoadQueueLength;
			PumpLoads(StackData, CurrentLoadQueueLength, NumPushed, bBusy);
			SetLoadBusy(bBusy && NumPushed == 0); // Mark as busy if pump was blocked and we did not make any progress
			if (NumPushed > 0)
			{
				SetSaveBusy(false, 0);
			}
			break;
		}
		case ECookAction::Save: [[fallthrough]];
		case ECookAction::SaveLimited:
		{
			uint32 CurrentSaveQueueLength = CookAction == ECookAction::Save ? 0 : DesiredSaveQueueLength;
			int32 NumAsyncWorkRetired;
			PumpSaves(StackData, CurrentSaveQueueLength, NumPushed, bBusy, NumAsyncWorkRetired);
			// Mark save as busy if pump was blocked and we have no more work we can do in pumpsaves.
			// Also report NumAsyncWorkRetired for status that cares about whether we are making some progress.
			SetSaveBusy(bBusy && NumPushed == 0, NumAsyncWorkRetired);
			break;
		}
		case ECookAction::Poll:
			PumpPollables(StackData, false /* bIsIdle */);
			break;
		case ECookAction::PollIdle:
			PumpPollables(StackData, true /* bIsIdle */);
			break;
		case ECookAction::KickBuildDependencies:
			KickBuildDependencies(StackData);
			break;
		case ECookAction::WaitForAsync:
			WaitForAsync(StackData);
			break;
		case ECookAction::YieldTick:
			bContinueTick = false;
			break;
		case ECookAction::Done:
			bContinueTick = false;
			StackData.bCookComplete = true;
			break;
		default:
			check(false);
			break;
		}
	}
}

void UCookOnTheFlyServer::TickCookStatus(UE::Cook::FTickStackData& StackData)
{
	UE_SCOPED_COOKTIMER(TickCookStatus);

	double CurrentTime = FPlatformTime::Seconds();
	StackData.LoopStartTime = CurrentTime;
	StackData.Timer.SetActionStartTime(CurrentTime);
	if (LastCookableObjectTickTime + TickCookableObjectsFrameTime <= CurrentTime)
	{
		UE_SCOPED_COOKTIMER(TickCookableObjects);
		FTickableCookObject::TickObjects(static_cast<float>(CurrentTime - LastCookableObjectTickTime), false /* bTickComplete */);
		LastCookableObjectTickTime = CurrentTime;
	}

	UpdateDisplay(StackData, false /* bForceDisplay */);
	ProcessAsyncLoading(false, false, 0.0f);
	ProcessUnsolicitedPackages();
	LogHandler->FlushIncrementalCookLogs();
	PumpExternalRequests(StackData.Timer);
}

void UCookOnTheFlyServer::SetSaveBusy(bool bInBusy, int32 NumAsyncWorkRetired)
{
	using namespace UE::Cook;

	if (!bInBusy)
	{
		if (bSaveBusy)
		{
			bSaveBusy = false;
			SaveBusyStartTimeSeconds = MAX_flt;
			SaveBusyRetryTimeSeconds = MAX_flt;
			SaveBusyWarnTimeSeconds = MAX_flt;
			// Whenever we set Save back to non-busy, reset the counter for how many busy reports with an
			// idle shadercompiler we need before we issue a warning
			bShaderCompilerWasActiveOnPreviousBusyReport = true;
		}
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();
	if (!bSaveBusy)
	{
		bSaveBusy = true;
		SaveBusyStartTimeSeconds = CurrentTime;
		SaveBusyWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
	}
	SaveBusyRetryTimeSeconds = CurrentTime + CookProgressRetryBusyPeriodSeconds;

	if (NumAsyncWorkRetired)
	{
		// Some progress was made so reset the warning time, and the start time in case we later issue a warning.
		SaveBusyStartTimeSeconds = CurrentTime;
		SaveBusyWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
	}

	if (CurrentTime >= SaveBusyWarnTimeSeconds)
	{
		// Compiler users - classes using the shader compiler - can take multiple minutes to be compiled due to long
		// queues and long compile times, so we do not issue a warning when they are the only objects holding us up,
		// so long as the shadercompiler reports it is working on them.
		bool bShaderCompilerIsActive = GShaderCompilingManager->IsCompiling();
		bool bBusyCompilationUsersAreExpected = bShaderCompilerIsActive ||
			// Even if the ShaderCompilerManager is not currently compiling, it might shortly begin or have recently finished.
			// Issue a warning for blocked compiler users only if there are two reports in a row where the compiler is not active
			bShaderCompilerWasActiveOnPreviousBusyReport;
		bShaderCompilerWasActiveOnPreviousBusyReport = bShaderCompilerIsActive;

		// Issue a status update. For each UObject we're still waiting on, check whether the long duration is expected using type-specific checks
		// Make the status update a warning if the long duration is not reported as expected.
		TArray<UObject*> NonExpectedObjects;
		TSet<UPackage*> NonExpectedPackages;
		TArray<UObject*> ExpectedObjects;
		TSet<UPackage*> ExpectedPackages;
		FPackageDataQueue& SaveQueue = PackageDatas->GetSaveQueue();
		TArray<UClass*> CompilationUsers({
			UMaterialInterface::StaticClass(),
			FindObject<UClass>(nullptr, TEXT("/Script/Niagara.NiagaraScript")),
			FindObject<UClass>(nullptr, TEXT("/Script/Niagara.NiagaraSystem"))
		});

		PackageDatas->ForEachPendingCookedPlatformData(
		[&CompilationUsers, &ExpectedObjects, &ExpectedPackages, &NonExpectedObjects, &NonExpectedPackages, bBusyCompilationUsersAreExpected]
		(const FPendingCookedPlatformData& Data)
		{
			UObject* Object = Data.Object.Get();
			if (!Object)
			{
				return;
			}
			bool bCompilationUser = false;
			for (UClass* CompilationUserClass : CompilationUsers)
			{
				if (Object->IsA(CompilationUserClass))
				{
					bCompilationUser = true;
					break;
				}
			}
			if (bCompilationUser && bBusyCompilationUsersAreExpected)
			{
				ExpectedObjects.Add(Object);
				ExpectedPackages.Add(Object->GetPackage());
			}
			else
			{
				NonExpectedObjects.Add(Object);
				NonExpectedPackages.Add(Object->GetPackage());
			}
		});
		TArray<UPackage*> RemovePackages;
		for (UPackage* Package : ExpectedPackages)
		{
			if (NonExpectedPackages.Contains(Package))
			{
				RemovePackages.Add(Package);
			}
		}
		for (UPackage* Package : RemovePackages)
		{
			ExpectedPackages.Remove(Package);
		}
			
		bool bExpectedDueToSlowBuildOperations = !ExpectedObjects.IsEmpty() && NonExpectedObjects.IsEmpty();
			
		FString Message = FString::Printf(TEXT("Cooker has been blocked from saving the current packages for %.0f seconds."),
			(float)CurrentTime - SaveBusyStartTimeSeconds);
		ELogVerbosity::Type MessageSeverity = ELogVerbosity::Display;
		if (!bExpectedDueToSlowBuildOperations)
		{
			MessageSeverity = CookerIdleWarningSeverity;
		}
#if !NO_LOGGING
		FMsg::Logf(__FILE__, __LINE__, LogCook.GetCategoryName(), MessageSeverity, TEXT("%s"), *Message);
#endif

		int DisplayCount = 0;
		const int DisplayMax = 10;
		if (NonExpectedPackages.Num() + ExpectedPackages.Num() > 0)
		{
			UE_LOGF(LogCook, Display, "%d packages in the savequeue: ", SaveQueue.Num());
			for (TSet<UPackage*>* PackageSet : { &NonExpectedPackages, &ExpectedPackages })
			{
				for (UPackage* Package : *PackageSet)
				{
					if (DisplayCount == DisplayMax)
					{
						UE_LOGF(LogCook, Display, "    ...");
						break;
					}
					UE_LOGF(LogCook, Display, "    %ls", *Package->GetName());
					++DisplayCount;
				}
			}
			if (DisplayCount == 0)
			{
				UE_LOGF(LogCook, Display, "    <None>");
			}

			UE_LOGF(LogCook, Display, "%d objects that have not yet returned true from IsCachedCookedPlatformDataLoaded:",
				PackageDatas->GetPendingCookedPlatformDataNum());
			DisplayCount = 0;
			for (TArray<UObject*>* ObjectArray : { &NonExpectedObjects, &ExpectedObjects })
			{
				for (UObject* Object : *ObjectArray)
				{
					if (DisplayCount == DisplayMax)
					{
						UE_LOGF(LogCook, Display, "    ...");
						break;
					}
					UE_LOGF(LogCook, Display, "    %ls", *Object->GetFullName());

					TStringBuilder<2048> AdditionalDebugInfo;
					FDelegates::PackageBlocked.Broadcast(Object, AdditionalDebugInfo);
					UE::String::ParseTokens(AdditionalDebugInfo.ToView(), TEXT('\n'),
						[](FStringView Line)
						{
							UE_LOGF(LogCook, Display, "        %.*ls", Line.Len(), Line.GetData());
						}, UE::String::EParseTokensOptions::SkipEmpty);

					++DisplayCount;
				}
			}
			if (DisplayCount == 0)
			{
				UE_LOGF(LogCook, Display, "    <None>");
			}
		}
		else if (SaveQueue.Num() > 0)
		{
			UE_LOGF(LogCook, Display, "%d packages in the savequeue: ", SaveQueue.Num());
			for (FPackageData* PackageData : SaveQueue)
			{
				if (DisplayCount == DisplayMax)
				{
					UE_LOGF(LogCook, Display, "    ...");
					break;
				}
				FString GeneratorText;
				if (PackageData->GetGenerationHelper())
				{
					GeneratorText = TEXT(" (Generator)");
				}
				else if (PackageData->GetParentGenerationHelper())
				{
					GeneratorText = TEXT(" (Generated)");
				}
				UE_LOGF(LogCook, Display, "    %ls%ls: %ls",
					*PackageData->GetPackageName().ToString(), *GeneratorText, LexToString(PackageData->GetSaveSubState()));
				++DisplayCount;
			}
		}

		SaveBusyWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
			
		UE::Cook::FDelegates::CookSaveIdle.Broadcast(
			*this, SaveQueue.Num(), PackageDatas->GetPendingCookedPlatformDataNum(), bExpectedDueToSlowBuildOperations);
	}
}

void UCookOnTheFlyServer::SetLoadBusy(bool bInLoadBusy)
{
	using namespace UE::Cook;

	if (bLoadBusy != bInLoadBusy)
	{
		bLoadBusy = bInLoadBusy;
		if (bLoadBusy)
		{
			const double CurrentTime = FPlatformTime::Seconds();
			LoadBusyStartTimeSeconds = CurrentTime;
			LoadBusyRetryTimeSeconds = CurrentTime + CookProgressRetryBusyPeriodSeconds;
			LoadBusyWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
		}
		else
		{
			LoadBusyStartTimeSeconds = MAX_flt;
			LoadBusyRetryTimeSeconds = MAX_flt;
			LoadBusyWarnTimeSeconds = MAX_flt;
		}
	}
	else if (bLoadBusy)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		LoadBusyRetryTimeSeconds = CurrentTime + CookProgressRetryBusyPeriodSeconds;
		if (CurrentTime >= LoadBusyWarnTimeSeconds)
		{
			int DisplayCount = 0;
			const int DisplayMax = 10;
			FLoadQueue& LoadQueue = PackageDatas->GetLoadQueue();
#if !NO_LOGGING
			FMsg::Logf(__FILE__, __LINE__, LogCook.GetCategoryName(), CookerIdleWarningSeverity,
				TEXT("Cooker has been blocked from loading the current packages for %.0f seconds. %d packages in the loadqueue."),
				(float)(CurrentTime - LoadBusyStartTimeSeconds), LoadQueue.Num());
#endif
			UE_LOGF(LogCook, Display, "%d packages in ReadyForLoads:", LoadQueue.ReadyForLoads.Num());
			DisplayCount = 0;
			for (TRefCountPtr<FPackagePreloader>& Preloader : LoadQueue.ReadyForLoads)
			{
				FPackageData& PackageData(Preloader->GetPackageData());
				if (DisplayCount++ == DisplayMax)
				{
					UE_LOGF(LogCook, Display, "    ...");
					break;
				}
				UE_LOGF(LogCook, Display, "    %ls: %ls", 
					*PackageData.GetPackageName().ToString(), LexToString(PackageData.GetState()));
			}

			UE_LOGF(LogCook, Display, "%d packages in Inbox:", LoadQueue.Inbox.Num());
			DisplayCount = 0;
			for (FPackageData* PackageData : LoadQueue.Inbox)
			{
				if (DisplayCount++ == DisplayMax)
				{
					UE_LOGF(LogCook, Display, "    ...");
					break;
				}
				UE_LOGF(LogCook, Display, "    %ls",
					*PackageData->GetPackageName().ToString());
			}

			TArray<TRefCountPtr<FPackagePreloader>> PoppedPendingKicks;
			while (!LoadQueue.PendingKicks.IsEmpty())
			{
				PoppedPendingKicks.Add(LoadQueue.PendingKicks.PopFront());
			}
			UE_LOGF(LogCook, Display, "%d packages in PendingKicks:", PoppedPendingKicks.Num());
			DisplayCount = 0;
			for (TRefCountPtr<FPackagePreloader>& Preloader : PoppedPendingKicks)
			{
				FPackageData& PackageData(Preloader->GetPackageData());
				if (DisplayCount++ == DisplayMax)
				{
					UE_LOGF(LogCook, Display, "    ...");
					break;
				}
				UE_LOGF(LogCook, Display, "    %ls",
					*PackageData.GetPackageName().ToString());
			}
			for (TRefCountPtr<FPackagePreloader>& Preloader : PoppedPendingKicks)
			{
				LoadQueue.PendingKicks.Add(Preloader);
			}
			
			UE_LOGF(LogCook, Display, "%d packages in ActivePreloads:", LoadQueue.ActivePreloads.Num());
			DisplayCount = 0;
			for (TRefCountPtr<FPackagePreloader>& Preloader : LoadQueue.ActivePreloads)
			{
				FPackageData& PackageData(Preloader->GetPackageData());
				if (DisplayCount++ == DisplayMax)
				{
					UE_LOGF(LogCook, Display, "    ...");
					break;
				}
				FString BlockerReason;
				TRefCountPtr<FPackagePreloader> Blocker;
				Preloader->PumpLoadsIsReadyToLeavePreload(&Blocker, &BlockerReason);
				if (!Blocker)
				{
					BlockerReason = TEXT("UnknownReasonStillInPreload: No Blocker found");
				}
				else
				{
					BlockerReason = FString::Printf(TEXT("Waiting on %s: %s"),
						*Blocker->GetPackageData().GetPackageName().ToString(), *BlockerReason);
				}

				UE_LOGF(LogCook, Display, "    %ls: %ls",
					*PackageData.GetPackageName().ToString(), *BlockerReason);
			}
			LoadBusyWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
			
			UE::Cook::FDelegates::CookLoadIdle.Broadcast(*this, LoadQueue.Num());
		}
	}
}

void UCookOnTheFlyServer::SetIdleStatus(UE::Cook::FTickStackData& StackData, EIdleStatus InStatus)
{
	if (InStatus == EIdleStatus::Active)
	{
		PhaseTransitionFence = -1;
	}

	if (InStatus == IdleStatus)
	{
		return;
	}

	IdleStatusStartTime = StackData.LoopStartTime;
	IdleStatusLastReportTime = IdleStatusStartTime;
	IdleStatus = InStatus;
}

void UCookOnTheFlyServer::UpdateDisplay(UE::Cook::FTickStackData& StackData, bool bForceDisplay)
{
	using namespace UE::Cook;

	const double CurrentTime = StackData.LoopStartTime;
	const double DeltaProgressDisplayTime = CurrentTime - LastProgressDisplayTime;
	if (!bForceDisplay && DeltaProgressDisplayTime < DisplayUpdatePeriodSeconds)
	{
		return;
	}

	int32 CookedCountWitness = PackageDatas->GetNumCooked();
	int32 PendingCountWitness = WorkerRequests->GetNumExternalRequests()
		+ PackageDatas->GetMonitor().GetNumInProgress();
	if (bForceDisplay ||
		(DeltaProgressDisplayTime >= GCookProgressUpdateTime && PendingCountWitness != 0 &&
			(LastCookedPackagesCount != CookedCountWitness || LastCookPendingCount != PendingCountWitness
				|| DeltaProgressDisplayTime > GCookProgressRepeatTime)))
	{
		const int32 CookedPackagesCount = PackageDatas->GetNumCooked()
			- PackageDatas->GetNumCooked(ECookResult::NeverCookPlaceholder) - PackageDataFromBaseGameNum;
		int32 CookPendingCount = WorkerRequests->GetNumExternalRequests()
			+ PackageDatas->GetMonitor().GetNumInProgress();
		// When a RequestCluster is doing a graph search, it marks uncookable packages as to-be-demoted, and
		// incrementally skippable packages as cooked, but those packages remain in the request state until the cluster
		// search is complete so we still count them as inprogress. Subtract them from the inprogress count.
		for (const TUniquePtr<FRequestCluster>& Cluster : PackageDatas->GetRequestQueue().GetRequestClusters())
		{
			CookPendingCount -= Cluster->GetPackagesToMarkNotInProgress();
		}
		UE_CLOGF(!(StackData.TickFlags & ECookTickFlags::HideProgressDisplay)
			&& (GCookProgressDisplay & (int32)ECookProgressDisplayMode::RemainingPackages),
			LogCook, Display,
			"Cooked packages %d Packages Remain %d Total %d",
			CookedPackagesCount, CookPendingCount, CookedPackagesCount + CookPendingCount);

		UE::Cook::FDelegates::CookUpdateDisplay.Broadcast(*this, CookedPackagesCount, CookPendingCount);

		LastCookedPackagesCount = CookedCountWitness;
		LastCookPendingCount = PendingCountWitness;
		LastProgressDisplayTime = CurrentTime;
	}
	const double DeltaDiagnosticsDisplayTime = CurrentTime - LastDiagnosticsDisplayTime;
	if (bForceDisplay || DeltaDiagnosticsDisplayTime > GCookProgressDiagnosticTime)
	{
		uint32 OpenFileHandles = 0;
#if PLATFORMFILETRACE_ENABLED
		OpenFileHandles = FPlatformFileTrace::GetOpenFileHandleCount();
#endif
		bool bCookOnTheFlyShouldDisplay = false;
		if (IsCookOnTheFlyMode() && (IsCookingInEditor() == false))
		{
			// Dump stats in CookOnTheFly, but only if there is new data
			static uint64 LastNumLoadedAndSaved = 0;
			if (StatLoadedPackageCount + StatSavedPackageCount != LastNumLoadedAndSaved)
			{
				bCookOnTheFlyShouldDisplay = true;
				LastNumLoadedAndSaved = StatLoadedPackageCount + StatSavedPackageCount;
			}
		}
		if (!IsCookOnTheFlyMode() || bCookOnTheFlyShouldDisplay)
		{
			if (!(StackData.TickFlags & ECookTickFlags::HideProgressDisplay) && (GCookProgressDisplay != (int32)ECookProgressDisplayMode::Nothing))
			{
				const FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
				UE_LOGF(LogCook, Display, "Cook Diagnostics: OpenFileHandles=%d, VirtualMemory=%lldMiB, VirtualMemoryAvailable=%lldMiB",
					OpenFileHandles,
					MemoryStats.UsedVirtual / 1024 / 1024,
					MemoryStats.AvailableVirtual / 1024 / 1024
				);
				if (CookDirector)
				{
					CookDirector->UpdateDisplayDiagnostics();
				}
			}
		}
		if (bCookOnTheFlyShouldDisplay)
		{
			DumpStats();
		}

		LastDiagnosticsDisplayTime = CurrentTime;

	}
}

FString UCookOnTheFlyServer::GetCookSettingsForMemoryLogText() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FString::Printf(
		TEXT("\n\tMemoryMaxUsedVirtual %lldMiB")
		TEXT("\n\tMemoryMaxUsedPhysical %lldMiB")
		TEXT("\n\tMemoryMinFreeVirtual %lldMiB")
		TEXT("\n\tMemoryMinFreePhysical %lldMiB")
		TEXT("\n\tMemoryTriggerGCAtPressureLevel %s")
		TEXT("\n\tSoftGCMemoryUseTrigger %s%s")
		TEXT("\n\tSoftGCTimeBudgetTrigger %s%s"),
		MemoryMaxUsedVirtual / 1024 / 1024, MemoryMaxUsedPhysical / 1024 / 1024,
		MemoryMinFreeVirtual / 1024 / 1024, MemoryMinFreePhysical / 1024 / 1024,
		*LexToString(MemoryTriggerGCAtPressureLevel),
		(bUseSoftGC && SoftGCStartNumerator > 0) ? TEXT("enabled") : TEXT("disabled"),
		(bUseSoftGC && SoftGCStartNumerator > 0) ? *FString::Printf(TEXT(" (%d/%d)"), SoftGCStartNumerator, SoftGCDenominator) : TEXT(""),
		(bUseSoftGC && SoftGCTimeFractionBudget > 0) ? TEXT("enabled") : TEXT("disabled"),
		(bUseSoftGC && SoftGCTimeFractionBudget > 0) ? *FString::Printf(TEXT(" (%.3g budget, %.0fs min period)"),
			SoftGCTimeFractionBudget, SoftGCMinimumPeriodSeconds) : TEXT("")
	);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

namespace UE::Cook::Pollable
{
constexpr double TimePeriodNever = MAX_flt / 2;
constexpr int32 ExpectedMaxNum = 10; // Used to size inline arrays

}

UCookOnTheFlyServer::FPollable::FPollable(const TCHAR* InDebugName, float InPeriodSeconds, float InPeriodIdleSeconds,
	UCookOnTheFlyServer::FPollFunction&& InFunction)
	: DebugName(InDebugName)
	, PollFunction(MoveTemp(InFunction))
	, NextTimeIdleSeconds(0)
	, PeriodSeconds(InPeriodSeconds)
	, PeriodIdleSeconds(InPeriodIdleSeconds)
{
	check(DebugName);
}

UCookOnTheFlyServer::FPollable::FPollable(const TCHAR* InDebugName, EManualTrigger,
	UCookOnTheFlyServer::FPollFunction&& InFunction)
	: DebugName(InDebugName)
	, PollFunction(MoveTemp(InFunction))
	, NextTimeIdleSeconds(MAX_flt)
	, PeriodSeconds(UE::Cook::Pollable::TimePeriodNever)
	, PeriodIdleSeconds(UE::Cook::Pollable::TimePeriodNever)
{
	check(DebugName);
}

UCookOnTheFlyServer::FPollableQueueKey::FPollableQueueKey(FPollable* InPollable)
	: FPollableQueueKey(TRefCountPtr<FPollable>(InPollable))
{
}
UCookOnTheFlyServer::FPollableQueueKey::FPollableQueueKey(const TRefCountPtr<FPollable>& InPollable)
	: FPollableQueueKey(TRefCountPtr<FPollable>(InPollable))
{
}
UCookOnTheFlyServer::FPollableQueueKey::FPollableQueueKey(TRefCountPtr<FPollable>&& InPollable)
	: Pollable(MoveTemp(InPollable))
{
	if (Pollable->PeriodSeconds < UE::Cook::Pollable::TimePeriodNever)
	{
		NextTimeSeconds = 0;
	}
	else
	{
		NextTimeSeconds = MAX_flt;
	}
}

void UCookOnTheFlyServer::FPollable::Trigger(UCookOnTheFlyServer& COTFS)
{
	FScopeLock PollablesScopeLock(&COTFS.PollablesLock);
	if (COTFS.bPollablesInTick)
	{
		FPollableQueueKey DeferredTrigger(this);
		DeferredTrigger.NextTimeSeconds = 0.;
		COTFS.PollablesDeferredTriggers.Add(MoveTemp(DeferredTrigger));
		return;
	}

	TriggerInternal(COTFS);
}

void UCookOnTheFlyServer::FPollable::TriggerInternal(UCookOnTheFlyServer& COTFS)
{
	FPollableQueueKey* KeyInQueue = COTFS.Pollables.FindByPredicate(
		[this](const FPollableQueueKey& Existing) { return Existing.Pollable.GetReference() == this; });
	if (ensure(KeyInQueue))
	{
		// CA_ASSUME is temporarily required because static analysis is not recognizing the if condition.
		CA_ASSUME(KeyInQueue != nullptr);
		FPollableQueueKey LocalQueueKey;
		LocalQueueKey.Pollable = MoveTemp(KeyInQueue->Pollable);
		// If the top of the heap is already triggered, put this after the top of the heap to
		// avoid excessive triggering causing starvation for other pollables
		// Note that the top of the heap might be this. Otherwise put this at the top of the
		// heap by setting its time to CurrentTime
		double CurrentTime = FPlatformTime::Seconds();
		double TimeAfterHeapTop = COTFS.Pollables.HeapTop().NextTimeSeconds + .001f;
		LocalQueueKey.NextTimeSeconds = FMath::Min(CurrentTime, TimeAfterHeapTop);
		this->NextTimeIdleSeconds = LocalQueueKey.NextTimeSeconds;

		int32 Index = UE_PTRDIFF_TO_INT32(KeyInQueue - COTFS.Pollables.GetData());
		COTFS.Pollables.HeapRemoveAt(Index, EAllowShrinking::No);
		COTFS.Pollables.HeapPush(MoveTemp(LocalQueueKey));
		COTFS.PollNextTimeSeconds = 0;
		COTFS.PollNextTimeIdleSeconds = 0;
	}
}

void UCookOnTheFlyServer::FPollable::RunNow(UCookOnTheFlyServer& COTFS)
{
	FScopeLock PollablesScopeLock(&COTFS.PollablesLock);

	UE::Cook::FTickStackData StackData(MAX_flt, ECookTickFlags::None);
	PollFunction(StackData);

	double CurrentTime = FPlatformTime::Seconds();
	if (COTFS.bPollablesInTick)
	{
		FPollableQueueKey DeferredTrigger(this);
		DeferredTrigger.NextTimeSeconds = CurrentTime;
		COTFS.PollablesDeferredTriggers.Add(MoveTemp(DeferredTrigger));
		return;
	}

	RunNowInternal(COTFS, CurrentTime);
}

void UCookOnTheFlyServer::FPollable::RunNowInternal(UCookOnTheFlyServer & COTFS, double TimeLastRun)
{
	FPollableQueueKey* KeyInQueue = COTFS.Pollables.FindByPredicate(
		[this](const FPollableQueueKey& Existing) { return Existing.Pollable.GetReference() == this; });
	if (ensure(KeyInQueue))
	{
		// CA_ASSUME is temporarily required because static analysis is not recognizing the if condition.
		CA_ASSUME(KeyInQueue != nullptr);
		FPollableQueueKey LocalQueueKey;
		LocalQueueKey.Pollable = MoveTemp(KeyInQueue->Pollable);
		LocalQueueKey.NextTimeSeconds = TimeLastRun + this->PeriodSeconds;
		this->NextTimeIdleSeconds = TimeLastRun + this->PeriodIdleSeconds;

		int32 Index = UE_PTRDIFF_TO_INT32(KeyInQueue - COTFS.Pollables.GetData());
		COTFS.PollNextTimeSeconds = FMath::Min(LocalQueueKey.NextTimeSeconds, COTFS.PollNextTimeSeconds);
		COTFS.PollNextTimeIdleSeconds = FMath::Min(this->NextTimeIdleSeconds, COTFS.PollNextTimeIdleSeconds);
		COTFS.Pollables.HeapRemoveAt(Index, EAllowShrinking::No);
		COTFS.Pollables.HeapPush(MoveTemp(LocalQueueKey));
	}
}

void UCookOnTheFlyServer::FPollable::RunDuringPump(UE::Cook::FTickStackData& StackData, double& OutNewCurrentTime, double& OutNextTimeSeconds)
{
	PollFunction(StackData);
	OutNewCurrentTime = FPlatformTime::Seconds();
	OutNextTimeSeconds = OutNewCurrentTime + PeriodSeconds;
	NextTimeIdleSeconds = OutNewCurrentTime + PeriodIdleSeconds;
}

void UCookOnTheFlyServer::PumpPollables(UE::Cook::FTickStackData& StackData, bool bIsIdle)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER(PumpPollables);
	{
		FScopeLock PollablesScopeLock(&PollablesLock);
		bPollablesInTick = true;
	}

	int32 NumPollables = Pollables.Num();
	if (NumPollables == 0)
	{
		PollNextTimeSeconds = MAX_flt;
		PollNextTimeIdleSeconds = MAX_flt;
		return;
	}

	double CurrentTime = StackData.LoopStartTime;
	if (!bIsIdle)
	{
		// To avoid an infinite loop, we keep the popped pollables in a separate list to readd afterwards
		// rather than readding them as soon as we know their new time
		TArray<FPollableQueueKey, TInlineAllocator<UE::Cook::Pollable::ExpectedMaxNum>> PoppedQueueKeys;
		while (!Pollables.IsEmpty() && Pollables.HeapTop().NextTimeSeconds <= CurrentTime)
		{
			FPollableQueueKey QueueKey;
			Pollables.HeapPop(QueueKey, EAllowShrinking::No);
			QueueKey.Pollable->RunDuringPump(StackData, CurrentTime, QueueKey.NextTimeSeconds);
			PoppedQueueKeys.Add(MoveTemp(QueueKey));
			if (StackData.Timer.IsActionTimeUp(CurrentTime))
			{
				break;
			}
		}
		for (FPollableQueueKey& QueueKey: PoppedQueueKeys)
		{
			Pollables.HeapPush(MoveTemp(QueueKey));
		}
		PollNextTimeSeconds = Pollables.HeapTop().NextTimeSeconds;
		// We don't know the real value of PollNextTimeIdleSeconds because we didn't look at the entire heap.
		// Mark that it needs to run next time we're idle, which will also make it recalculate PollNextTimeIdleSeconds
		PollNextTimeIdleSeconds = 0;
	}
	else
	{
		// Since Idle times are not heap sorted, we have to look at all elements in the heap.
		bool bUpdated = false;
		PollNextTimeSeconds = MAX_flt;
		PollNextTimeIdleSeconds = MAX_flt;
		int32 PollIndex = 0;
		for (; PollIndex < NumPollables; ++PollIndex)
		{
			FPollableQueueKey& QueueKey= Pollables[PollIndex];
			if (QueueKey.Pollable->NextTimeIdleSeconds <= CurrentTime)
			{
				QueueKey.Pollable->RunDuringPump(StackData, CurrentTime, QueueKey.NextTimeSeconds);
				bUpdated = true;
			}
			PollNextTimeSeconds = FMath::Min(QueueKey.NextTimeSeconds, PollNextTimeSeconds);
			PollNextTimeIdleSeconds = FMath::Min(QueueKey.Pollable->NextTimeIdleSeconds, PollNextTimeIdleSeconds);
			if (StackData.Timer.IsActionTimeUp(CurrentTime))
			{
				break;
			}
		}
		// If we early exited, finish calculating PollNextTimeSeconds from the remaining members we didn't reach
		for (;PollIndex < NumPollables; ++PollIndex)
		{
			FPollableQueueKey& QueueKey = Pollables[PollIndex];
			PollNextTimeSeconds = FMath::Min(QueueKey.NextTimeSeconds, PollNextTimeSeconds);
			PollNextTimeIdleSeconds = FMath::Min(QueueKey.Pollable->NextTimeIdleSeconds, PollNextTimeIdleSeconds);
		}
		if (bUpdated)
		{
			Pollables.Heapify();
		}
	}

	{
		FScopeLock PollablesScopeLock(&PollablesLock);
		for (FPollableQueueKey& QueueKey : PollablesDeferredTriggers)
		{
			if (QueueKey.NextTimeSeconds == 0)
			{
				QueueKey.Pollable->TriggerInternal(*this);
			}
			else
			{
				QueueKey.Pollable->RunNowInternal(*this, QueueKey.NextTimeSeconds);
			}
		}
		PollablesDeferredTriggers.Reset();
		bPollablesInTick = false;
	}
}

void UCookOnTheFlyServer::PollFlushRenderingCommands()
{
	UE_SCOPED_COOKTIMER_AND_DURATION(CookByTheBook_TickCommandletStats, DetailedCookStats::TickLoopFlushRenderingCommandsTimeSec);

	// Flush rendering commands to release any RHI resources (shaders and shader maps).
	// Delete any FPendingCleanupObjects (shader maps).
	FlushRenderingCommands();
}

TRefCountPtr<UCookOnTheFlyServer::FPollable> UCookOnTheFlyServer::CreatePollableLLM()
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (FLowLevelMemTracker::Get().IsEnabled())
	{
		float PeriodSeconds = 120.0f;
		FParse::Value(FCommandLine::Get(), TEXT("-CookLLMPeriod="), PeriodSeconds);
		return TRefCountPtr<FPollable>(new FPollable(TEXT("LLM"), PeriodSeconds, PeriodSeconds,
			[](UE::Cook::FTickStackData&) { FLowLevelMemTracker::Get().UpdateStatsPerFrame(); }));
	}
#endif
	return TRefCountPtr<FPollable>();
}

TRefCountPtr<UCookOnTheFlyServer::FPollable> UCookOnTheFlyServer::CreatePollableTriggerGC()
{
	bool bTestCook = IsCookFlagSet(ECookInitializationFlags::TestCook);

	// Collect statistics every 2 minutes even if we are not tracking time between garbage collects
	float PeriodSeconds = 120.f;
	float IdlePeriodSeconds = 120.f;
	constexpr float SecondsPerPackage = .01f;
	if (bTestCook)
	{
		PeriodSeconds = FMath::Min(PeriodSeconds, 50 * SecondsPerPackage);
	}
	if (PackagesPerGC > 0)
	{
		// PackagesPerGC is usually used only to debug; max memory counts are commonly used instead
		// Since it's not commonly used, we make a concession to support it: we check on a timer rather than checking after every saved package.
		// For large values, check less frequently.
		PeriodSeconds = FMath::Min(PeriodSeconds, PackagesPerGC * SecondsPerPackage);
	}
	if (IsCookOnTheFlyMode())
	{
		PeriodSeconds = FMath::Min(PeriodSeconds, 10.f);
		IdlePeriodSeconds = FMath::Min(IdlePeriodSeconds, 0.1f);
	}
	IdlePeriodSeconds = FMath::Min(IdlePeriodSeconds, PeriodSeconds);

	return TRefCountPtr<FPollable>(new FPollable(TEXT("TimeForGC"), PeriodSeconds, IdlePeriodSeconds,
		[this](UE::Cook::FTickStackData& StackData) { PollGarbageCollection(StackData); }));
}

namespace UE::Cook
{

void FStatHistoryInt::Initialize(int64 InitialValue)
{
	Maximum = Minimum = InitialValue;
}

void FStatHistoryInt::AddInstance(int64 CurrentValue)
{
	Maximum = FMath::Max(CurrentValue, Maximum);
	Minimum = FMath::Min(CurrentValue, Minimum);
}

static void ProcessDeferredCommands(UCookOnTheFlyServer& COTFS)
{
#if OUTPUT_COOKTIMING
	TOptional<FScopedDurationTimer> CBTBScopedDurationTimer;
	if (!COTFS.IsCookOnTheFlyMode())
	{
		CBTBScopedDurationTimer.Emplace(DetailedCookStats::TickLoopProcessDeferredCommandsTimeSec);
	}
#endif
	UE_SCOPED_COOKTIMER(ProcessDeferredCommands);

#if PLATFORM_MAC
	// On Mac we need to process Cocoa events so that the console window for CookOnTheFlyServer is interactive
	FPlatformApplicationMisc::PumpMessages(true);
#endif

	// update task graph
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

	// execute deferred commands
	for (const FString& DeferredCommand : GEngine->DeferredCommands)
	{
		GEngine->Exec(GWorld, *DeferredCommand, *GLog);
	}

	GEngine->DeferredCommands.Empty();
}

static void CBTBTickCommandletStats()
{
	UE_SCOPED_COOKTIMER_AND_DURATION(CookByTheBook_TickCommandletStats, DetailedCookStats::TickLoopTickCommandletStatsTimeSec);
	UE::Stats::FStats::TickCommandletStats();
}

static void TickAssetRegistry()
{
	UE_SCOPED_COOKTIMER(CookByTheBook_TickAssetRegistry);
	FAssetRegistryModule::TickAssetRegistry(-1.0f);
}

} // namespace UE::Cook

void UCookOnTheFlyServer::InitializePollables()
{
	UE_SCOPED_COOKTIMER(InitializePollables);
	using namespace UE::Cook;

	Pollables.Reset();

	QueuedCancelPollable = new FPollable(TEXT("QueuedCancel"), FPollable::EManualTrigger(), [this](FTickStackData& StackData) { PollQueuedCancel(StackData); });
	Pollables.Emplace(QueuedCancelPollable);
	if (!IsCookingInEditor())
	{
		Pollables.Emplace(new FPollable(TEXT("AssetRegistry"), 60.f, 5.f, [](FTickStackData&) { TickAssetRegistry(); }));
		if (TRefCountPtr<FPollable> Pollable = CreatePollableTriggerGC())
		{
			Pollables.Emplace(MoveTemp(Pollable));
		}
		Pollables.Emplace(new FPollable(TEXT("ProcessDeferredCommands"), 60.f, 5.f, [this](FTickStackData&) { ProcessDeferredCommands(*this); }));
		Pollables.Emplace(new FPollable(TEXT("ShaderCompilingManager"), 60.f, 5.f, [this](FTickStackData& StackData)
		{
			TickShaderCompilingManager(StackData);
		}));
		Pollables.Emplace(new FPollable(TEXT("FlushRenderingCommands"), 60.f, 5.f, [this](FTickStackData&) { PollFlushRenderingCommands(); }));
		if (TRefCountPtr<FPollable> Pollable = CreatePollableLLM())
		{
			Pollables.Emplace(MoveTemp(Pollable));
		}
	}
	if (!IsCookOnTheFlyMode())
	{
		if (!IsCookingInEditor())
		{
			Pollables.Emplace(new FPollable(TEXT("CommandletStats"), 60.f, 5.f, [](FTickStackData&) { CBTBTickCommandletStats(); }));
		}
	}
	else
	{
		RecompileRequestsPollable = new FPollable(TEXT("RecompileShaderRequests"), FPollable::EManualTrigger(), [this](FTickStackData& TickStackData) { TickRecompileShaderRequestsPrivate(TickStackData); });
		Pollables.Add(FPollableQueueKey(RecompileRequestsPollable));
		Pollables.Emplace(new FPollable(TEXT("RequestManager"), 0.5f, 0.5f, [this](FTickStackData&) { TickRequestManager(); }));

	}
	if (CookDirector)
	{
		DirectorPollable = new FPollable(TEXT("CookDirector"), 1.0f, 1.0f, [this](FTickStackData& StackData) { CookDirector->TickFromSchedulerThread(); });
		Pollables.Add(FPollableQueueKey(DirectorPollable));
	}
	if (CookWorkerClient)
	{
		Pollables.Emplace(new FPollable(TEXT("CookWorkerClient"), 1.0f, 1.0f, [this](FTickStackData& StackData) { CookWorkerClient->TickFromSchedulerThread(StackData); }));
	}
	Pollables.Heapify();

	PollNextTimeSeconds = 0.;
	PollNextTimeIdleSeconds = 0.;
}

void UCookOnTheFlyServer::WaitForAsync(UE::Cook::FTickStackData& StackData)
{
	// Sleep until the next time that DecideNextCookAction will find work to do, up to a maximum of WaitForAsyncSleepSeconds
	UE_SCOPED_HIERARCHICAL_COOKTIMER(WaitForAsync);
	double CurrentTime = FPlatformTime::Seconds();
	double SleepDuration = WaitForAsyncSleepSeconds;
	SleepDuration = FMath::Min(SleepDuration, StackData.Timer.GetActionEndTimeSeconds() - CurrentTime);
	SleepDuration = FMath::Min(SleepDuration, PollNextTimeIdleSeconds - CurrentTime);
	SleepDuration = FMath::Min(SleepDuration, SaveBusyRetryTimeSeconds - CurrentTime);
	SleepDuration = FMath::Min(SleepDuration, LoadBusyRetryTimeSeconds - CurrentTime);
	SleepDuration = FMath::Max(SleepDuration, 0);
	FPlatformProcess::Sleep(static_cast<float>(SleepDuration));
}

UCookOnTheFlyServer::ECookAction UCookOnTheFlyServer::DecideNextCookAction(UE::Cook::FTickStackData& StackData)
{
	using namespace UE::Cook;

	if (StackData.ResultFlags & COSR_YieldTick)
	{
		// Yielding on demand does not impact idle status
		return ECookAction::YieldTick;
	}

	double CurrentTime = StackData.LoopStartTime;
	if (StackData.Timer.IsTickTimeUp(CurrentTime))
	{
		// Timeup does not impact idle status
		return ECookAction::YieldTick;
	}
	else if (CurrentTime >= PollNextTimeSeconds)
	{
		// Polling does not impact idle status
		return ECookAction::Poll;
	}

	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	if (RequestQueue.HasRequestsToExplore())
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Request;
	}

	FPackageDataMonitor& Monitor = PackageDatas->GetMonitor();

	// If we have any packages with urgency higher than EUrgency::Normal, pump their states first, if not busy,
	// before pumping any lower-urgency states.
	for (EUrgency UrgencyLevel = EUrgency::Max; UrgencyLevel > EUrgency::Normal;
		UrgencyLevel = static_cast<EUrgency>(static_cast<uint32>(UrgencyLevel) - 1))
	{
		if (Monitor.GetNumUrgent(UrgencyLevel) > 0)
		{
			static_assert(static_cast<uint32>(EPackageState::Count) == 7, "Need to handle every state here.");
			bool bSaveHasUrgent = Monitor.GetNumUrgent(EPackageState::SaveActive, UrgencyLevel) > 0;
			if (!bSaveBusy && bSaveHasUrgent)
			{
				SetIdleStatus(StackData, EIdleStatus::Active);
				return ECookAction::Save;
			}
			bool bLoadHasUrgent = Monitor.GetNumUrgent(EPackageState::Load, UrgencyLevel) > 0;
			if (!bLoadBusy && bLoadHasUrgent)
			{
				SetIdleStatus(StackData, EIdleStatus::Active);
				return ECookAction::Load;
			}
			if (Monitor.GetNumUrgent(EPackageState::Request, UrgencyLevel) > 0)
			{
				SetIdleStatus(StackData, EIdleStatus::Active);
				return ECookAction::Request;
			}
			if (UrgencyLevel == EUrgency::Blocking && bSaveHasUrgent)
			{
				SetIdleStatus(StackData, EIdleStatus::Active);
				return ECookAction::Save;
			}
			if (UrgencyLevel == EUrgency::Blocking && bLoadHasUrgent)
			{
				SetIdleStatus(StackData, EIdleStatus::Active);
				return ECookAction::Load;
			}

			// For the the remaining states, nothing to do
			// EPackageState::AssignedToWorker
			// EPackageState::SaveStalledAssignedToWorker
			// EPackageState::SaveStalledRetracted

			// fall through and do the next lower level of urgency
		}
	}

	static_assert(static_cast<uint32>(EPackageState::Count) == 7, "Need to handle every state here.");
	int32 NumSaves = PackageDatas->GetSaveQueue().Num();
	bool bSaveAvailable = ((!bSaveBusy) & (NumSaves > 0)) != 0;
	if (bSaveAvailable & (NumSaves > static_cast<int32>(DesiredSaveQueueLength)))
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::SaveLimited;
	}

	int32 NumLoads = PackageDatas->GetLoadQueue().Num();
	bool bLoadAvailable = ((!bLoadBusy) & (NumLoads > 0)) != 0;
	if (bLoadAvailable & (NumLoads > static_cast<int32>(DesiredLoadQueueLength)))
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::LoadLimited;
	}

	if (!RequestQueue.IsReadyRequestsEmpty())
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Request;
	}

	if (bSaveAvailable)
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Save;
	}

	if (bLoadAvailable)
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Load;
	}

	if (NumSaves > 0 && CurrentTime >= SaveBusyRetryTimeSeconds)
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Save;
	}
	if (NumLoads > 0 && CurrentTime >= LoadBusyRetryTimeSeconds)
	{
		SetIdleStatus(StackData, EIdleStatus::Active);
		return ECookAction::Load;
	}

	if (PackageDatas->GetMonitor().GetNumInProgress() > 0)
	{
		if (CurrentTime >= PollNextTimeIdleSeconds)
		{
			// Polling does not impact idle status
			return ECookAction::PollIdle;
		}
		else if (IsRealtimeMode() || IsCookOnTheFlyMode())
		{
			SetIdleStatus(StackData, EIdleStatus::Idle);
			return ECookAction::YieldTick;
		}
		else
		{
			SetIdleStatus(StackData, EIdleStatus::Idle);
			return ECookAction::WaitForAsync;
		}
	}

	if (IsCookByTheBookMode() && GetCookPhase() != ECookPhase::BuildDependencies)
	{
		// In MPCook, we need to wait for any late-arrival discovery messages before we can know that we have finished
		// all work for the Cook phase. Send a fence to the CookWorkers and wait for their acknowledgement to be sure
		// that we received all messages they had in progress. We only need to insert a new fence if there have been
		// messages sent to CookWorkers since the last fence we created. (Note this also provides an optimization in
		// empty cooks: no messages have been sent so we do not need to wait, which lets us skip having to wait for
		// the CookWorkers to start up, which might take minutes.)
		if (CookDirector && CookDirector->HasSentActions())
		{
			InsertPhaseTransitionFence();
			CookDirector->ClearHasSentActions();
		}

		bool bCompleted;
		PumpPhaseTransitionFence(bCompleted);
		if (!bCompleted)
		{
			SetIdleStatus(StackData, EIdleStatus::Idle);
			return ECookAction::WaitForAsync;
		}

		return ECookAction::KickBuildDependencies;
	}

	if (IsCookOnTheFlyMode() || IsCookWorkerMode())
	{
		// These modes are not done until a manual trigger, so continue polling idle
		if (CurrentTime >= PollNextTimeIdleSeconds)
		{
			// Polling does not impact idle status
			return ECookAction::PollIdle;
		}
		if (IsCookOnTheFlyMode())
		{
			SetIdleStatus(StackData, EIdleStatus::Done);
			return ECookAction::Done;
		}
		else
		{
			SetIdleStatus(StackData, EIdleStatus::Idle);
			return ECookAction::WaitForAsync;
		}
	}

	// We're in the CookComplete phase, pump the special cases in this phase
	// and return WaitForAsync until they are complete
	if (CookDirector)
	{
		bool bCompleted;
		CookDirector->PumpCookComplete(bCompleted);
		if (!bCompleted)
		{
			// Continue polling idle
			if (CurrentTime >= PollNextTimeIdleSeconds)
			{
				// Polling does not impact idle status
				return ECookAction::PollIdle;
			}

			SetIdleStatus(StackData, EIdleStatus::Idle);
			return ECookAction::WaitForAsync;
		}
	}

	SetIdleStatus(StackData, EIdleStatus::Done);
	return ECookAction::Done;
}

int32 UCookOnTheFlyServer::NumMultiprocessLocalWorkerAssignments() const
{
	using namespace UE::Cook;

	if (!CookDirector.IsValid())
	{
		return 0;
	}
	UE::Cook::FPackageDataMonitor& Monitor = PackageDatas->GetMonitor();
	int32 Result = WorkerRequests->GetNumExternalRequests() +
		PackageDatas->GetRequestQueue().Num() +
		PackageDatas->GetLoadQueue().Num() +
		PackageDatas->GetSaveQueue().Num();
	for (const TUniquePtr<FRequestCluster>& Cluster : PackageDatas->GetRequestQueue().GetRequestClusters())
	{
		Result -= Cluster->GetPackagesToMarkNotInProgress();
	}
	return Result;
}

void UCookOnTheFlyServer::PumpExternalRequests(const UE::Cook::FCookerTimer& CookerTimer)
{
	using namespace UE::Cook;

	if (!WorkerRequests->HasExternalRequests())
	{
		return;
	}
	UE_SCOPED_COOKTIMER(PumpExternalRequests);

	TArray<FFilePlatformRequest> BuildRequests;
	TArray<FSchedulerCallback> SchedulerCallbacks;
	EExternalRequestType RequestType;
	while (!CookerTimer.IsActionTimeUp())
	{
		BuildRequests.Reset();
		SchedulerCallbacks.Reset();
		RequestType = WorkerRequests->DequeueNextCluster(SchedulerCallbacks, BuildRequests);
		if (RequestType == EExternalRequestType::None)
		{
			// No more requests to process
			break;
		}
		else if (RequestType == EExternalRequestType::Callback)
		{
			// An array of TickCommands to process; execute through them all
			for (FSchedulerCallback& SchedulerCallback : SchedulerCallbacks)
			{
				SchedulerCallback();
			}
		}
		else
		{
			check(RequestType == EExternalRequestType::Cook && BuildRequests.Num() > 0);
#if PROFILE_NETWORK
			if (NetworkRequestEvent)
			{
				NetworkRequestEvent->Trigger();
			}
#endif
			bool bRequestsAreUrgent = IsCookOnTheFlyMode() && IsUsingLegacyCookOnTheFlyScheduling();
			TRingBuffer<TUniquePtr<FRequestCluster>>& RequestClusters
				= PackageDatas->GetRequestQueue().GetRequestClusters();
			RequestClusters.Add(MakeUnique<FRequestCluster>(*this, MoveTemp(BuildRequests)));
		}
	}
}

bool UCookOnTheFlyServer::TryCreateRequestCluster(UE::Cook::FPackageData& PackageData)
{
	check(PackageData.IsInProgress()); // This should only be called from Pump functions, and only on in-progress Packages
	using namespace UE::Cook;
	const EReachability DesiredReachability = GetCookPhase() == ECookPhase::Cook
		? EReachability::Runtime : EReachability::Build;
	if (!PackageData.AreAllReachablePlatformsVisitedByCluster(DesiredReachability))
	{
		PackageData.SendToState(EPackageState::Request, ESendFlags::QueueAdd, EStateChangeReason::Discovered);
		return true;
	}
	return false;
}

void UCookOnTheFlyServer::PumpRequests(UE::Cook::FTickStackData& StackData, int32& OutNumPushed)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER(PumpRequests);
	using namespace UE::Cook;

	OutNumPushed = 0;
	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	TPackageDataMap<ESuppressCookReason>& RestartedRequests = RequestQueue.GetRestartedRequests();
	TRingBuffer<FDiscoveryQueueElement>& DiscoveryQueue = RequestQueue.GetDiscoveryQueue();
	TRingBuffer<FPackageData*>& BuildDependencyDiscoveryQueue = RequestQueue.GetBuildDependencyDiscoveryQueue();
	TRingBuffer<TUniquePtr<FRequestCluster>>& RequestClusters = RequestQueue.GetRequestClusters();
	const FCookerTimer& CookerTimer = StackData.Timer;

	// First pump all requestclusters and unclustered/discovered requests that need to create a requestcluster
	for (;;)
	{
		// We completely finish the first cluster before moving on to new or remaining clusters.
		// This prevents the problem of an infinite loop due to having two clusters steal a PackageData
		// back and forth from each other.
		if (!RequestClusters.IsEmpty())
		{
			FRequestCluster& RequestCluster = *RequestClusters.First();
			bool bComplete = false;
			RequestCluster.Process(CookerTimer, bComplete);
			if (bComplete)
			{
				TArray<FPackageData*> RequestsToLoad;
				TArray<TPair<FPackageData*, ESuppressCookReason>> RequestsToDemote;
				TMap<FPackageData*, TArray<FPackageData*>> RequestGraph;
				RequestCluster.ClearAndDetachOwnedPackageDatas(RequestsToLoad, RequestsToDemote, RequestGraph);
				// Some packages might be reachable only on the CookerLoadingPlatform, or on previous cooked packages,
				// because they are OnlyEditorOnly or are excluded for the newly reachable platform.
				// Demote any packages that do not have any platforms needing cooking.
				for (FPackageData*& PackageData : RequestsToLoad)
				{
					check(PackageData->GetState() == EPackageState::Request);
					if (PackageData->GetPlatformsNeedingCommitNum(GetCookPhase()) == 0)
					{
						ESuppressCookReason SuppressCookReason = PackageData->HasAnyCookedPlatform() ? ESuppressCookReason::AlreadyCooked :
							ESuppressCookReason::OnlyEditorOnly;
						RequestsToDemote.Add(TPair<FPackageData*,ESuppressCookReason>(PackageData, SuppressCookReason));
						PackageData = nullptr; // Do not RemoveSwap; need to maintain order of the other elements in the list
					}
				}
				RequestsToLoad.Remove(nullptr);
				AssignRequests(RequestsToLoad, RequestQueue, MoveTemp(RequestGraph));
				for (TPair<FPackageData*, ESuppressCookReason>& Pair : RequestsToDemote)
				{
					DemoteToIdle(*Pair.Key, ESendFlags::QueueAdd, Pair.Value);
				}
				RequestClusters.PopFront();
				OnRequestClusterCompleted(RequestCluster);
			}
		}
		else if (!RestartedRequests.IsEmpty())
		{
			EReachability ClusterReachability = GetCookPhase() == ECookPhase::Cook
				? EReachability::Runtime
				: EReachability::Build;
			RequestClusters.Add(MakeUnique<FRequestCluster>(*this, MoveTemp(RestartedRequests), ClusterReachability));
			RestartedRequests.Empty();
		}
		else if (!DiscoveryQueue.IsEmpty())
		{
			if (GetCookPhase() == ECookPhase::Cook)
			{
				TUniquePtr<FRequestCluster> Cluster = MakeUnique<FRequestCluster>(*this, DiscoveryQueue);
				if (Cluster->NeedsProcessing())
				{
					RequestClusters.Add(MoveTemp(Cluster));
				}
			}
			else
			{
				UE_LOGF(LogCook, Warning,
					"Packages were added to the runtime discovery queue after starting BuildDependencies phase. Ignoring them."
					" The first added package: %ls",
					*DiscoveryQueue.First().PackageData->GetPackageName().ToString());
				DiscoveryQueue.Empty();
			}
		}
		else if (!BuildDependencyDiscoveryQueue.IsEmpty())
		{
			if (GetCookPhase() == ECookPhase::Cook)
			{
				UE_LOGF(LogCook, Error,
					"Packages were added to the build dependency discovery queue before starting BuildDependencies phase. Ignoring them."
					" The first added package: %ls",
					*BuildDependencyDiscoveryQueue.First()->GetPackageName().ToString());
				BuildDependencyDiscoveryQueue.Empty();
			}
			else
			{
				TUniquePtr<FRequestCluster> Cluster = MakeUnique<FRequestCluster>(*this,
					FRequestCluster::BuildDependencyQueue, BuildDependencyDiscoveryQueue);
				if (Cluster->NeedsProcessing())
				{
					RequestClusters.Add(MoveTemp(Cluster));
				}
			}
		}
		else
		{
			break;
		}

		if (CookerTimer.IsActionTimeUp())
		{
			return;
		}
	}

	// After all clusters have been processed, pull a batch of readyrequests into the load state
	COOK_STAT(DetailedCookStats::PeakRequestQueueSize = FMath::Max(DetailedCookStats::PeakRequestQueueSize,
		static_cast<int32>(RequestQueue.ReadyRequestsNum())));
	uint32 NumInBatch = 0;
	while (!RequestQueue.IsReadyRequestsEmpty() && NumInBatch < RequestBatchSize)
	{
		FPackageData* PackageData = RequestQueue.PopReadyRequest();
		check(PackageData->GetState() == EPackageState::Request);
		FPoppedPackageDataScope Scope(*PackageData);
		if (TryCreateRequestCluster(*PackageData))
		{
			continue;
		}
		if (PackageData->IsGenerated() && !CookWorkerClient)
		{
			// We only need to check for this on the CookDirector (or in SPCook), because it is at the start of making
			// every package active. CookWorkers do not have the information they need to check for this.
			// 
			// Generated packages cannot be requested before their generator queues them for some generators,
			// because some generators require that they save on the same CooKWorker as the generator
			// (EGeneratedRequiresGenerator::Save) and we don't know that assignment yet, and some generators require
			// that the generator run BeginCacheCookedPlatformData before calling ShouldSplit
			// (RequiresCachedCookedPlatformDataBeforeSplit).
			// Generated packages being queued before their generator queues them can occur during IncrementalValidate
			// incremental cooks: if the generator is up to date, the incremental cook will explore the generator and
			// all of its generated packages, but IncrementalValidate will decide they need to be cooked and will queue
			// them for cooking all at the same time.
			TRefCountPtr<FGenerationHelper> ParentGenerationHelper = PackageData->GetOrFindParentGenerationHelper();
			if (!CookWorkerClient &&
				(!ParentGenerationHelper || !ParentGenerationHelper->GetDirectorAPI().HasStartedQueueGeneratedPackages()))
			{
				// If the generated package's generator has not yet called QueueGeneratedPackages, temporarily demote the
				// generated package back to Idle; it will be requeued during QueueGeneratedPackages. Also reset its
				// reachability, so that we don't early exit from that upcoming QueueDiscoveredPackage call.
				PackageData->ResetReachable(EReachability::Runtime);
				DemoteToIdle(*PackageData, ESendFlags::QueueAdd, ESuppressCookReason::NotYetReadyForRequest);
				continue;
			}
		}
		if (PackageData->GetPlatformsNeedingCommitNum(GetCookPhase()) == 0)
		{
			ESuppressCookReason SuppressCookReason = PackageData->HasAnyCommittedPlatforms()
				? ESuppressCookReason::AlreadyCooked : ESuppressCookReason::OnlyEditorOnly;
			DemoteToIdle(*PackageData, ESendFlags::QueueAdd, SuppressCookReason);
			continue;
		}
		PackageData->SendToState(EPackageState::Load, ESendFlags::QueueAdd, EStateChangeReason::Requested);
		++NumInBatch;
	}
	OutNumPushed += NumInBatch;
	if (DiscoveryQueue.IsEmpty() && RequestClusters.IsEmpty() && RestartedRequests.IsEmpty())
	{
		RequestQueue.NotifyRequestFencePassed(*PackageDatas);
	}
}

void UCookOnTheFlyServer::AssignRequests(TArrayView<UE::Cook::FPackageData*> Requests, UE::Cook::FRequestQueue& RequestQueue,
	TMap<UE::Cook::FPackageData*, TArray<UE::Cook::FPackageData*>>&& RequestGraph)
{
	using namespace UE::Cook;

	if (CookDirector)
	{
		int32 NumRequests = Requests.Num();
		if (NumRequests == 0)
		{
			return;
		}
		TArray<FWorkerId> Assignments;
		CookDirector->AssignRequests(Requests, Assignments, MoveTemp(RequestGraph));
		check(Assignments.Num() == NumRequests);

		// The Input RequestQueue is in LeafToRoot order, but we want to save in RootToLeaf order,
		// so reverse iterate. This is only important for the Assignment.IsLocal case; the other two
		// cases go into order-independent containers.
		for (int32 Index = NumRequests - 1; Index >= 0; --Index)
		{
			FPackageData* PackageData = Requests[Index];
			FWorkerId Assignment = Assignments[Index];
			if (Assignment.IsInvalid())
			{
				DemoteToIdle(*PackageData, ESendFlags::QueueAdd, ESuppressCookReason::MultiprocessAssignmentError);
			}
			else if (Assignment.IsLocal())
			{
				RequestQueue.AddReadyRequest(PackageData);
			}
			else
			{
				EPackageState NewState = PackageData->IsInStateProperty(EPackageStateProperty::Saving)
					? EPackageState::SaveStalledAssignedToWorker
					: EPackageState::AssignedToWorker;
				PackageData->SendToState(NewState, ESendFlags::QueueAdd, EStateChangeReason::Requested);
				PackageData->SetWorkerAssignment(Assignment);
			}
		}
	}
	else
	{
		TArray<FPackageData*> Shuffled;
		if (bRandomizeCookOrder)
		{
			Shuffled = Requests;
			Algo::RandomShuffle(Shuffled);
			Requests = Shuffled;
		}

		// The Input RequestQueue is in LeafToRoot order, but we want to save in RootToLeaf order,
		// so reverse iterate.
		for (FPackageData* PackageData : ReverseIterate(Requests))
		{
			RequestQueue.AddReadyRequest(PackageData);
		}
	}
}

void UCookOnTheFlyServer::NotifyRemovedFromWorker(UE::Cook::FPackageData& PackageData)
{
	check(CookDirector);
	CookDirector->RemoveFromWorker(PackageData);
}

void UCookOnTheFlyServer::DemoteToIdle(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags,
	UE::Cook::ESuppressCookReason Reason)
{
	using namespace UE::Cook;

	if (PackageData.IsInProgress())
	{
		WorkerRequests->ReportDemotion(PackageData, Reason);

		bool bHasCookResult = PackageData.HasAllCookedPlatforms(PlatformManager->GetSessionPlatforms(), true /* bIncludeFailed */);

		// If per-package display is on, write a log statement explaining that the package was reachable but skipped.
		bool bPrintDiagnostic = !bCookListMode &
			!!((GCookProgressDisplay & ((int32)ECookProgressDisplayMode::Instigators | (int32)ECookProgressDisplayMode::PackageNames)));

		// Suppress the message if it's a temporary demotion
		bPrintDiagnostic &= (Reason != ESuppressCookReason::NotYetReadyForRequest);
		// Suppress the message in cases that cause large spam like NotInCurrentPlugin for DLC cooks.
		bPrintDiagnostic &= (Reason != ESuppressCookReason::NotInCurrentPlugin);
		// Incremental cooks: suppress the diagnostic for packages that were incrementally skipped
		bPrintDiagnostic &= !bHasCookResult;
		if (bPrintDiagnostic && IsCookingDLC() && Reason == ESuppressCookReason::AlreadyCooked && LogCook.GetVerbosity() < ELogVerbosity::Verbose)
		{
			bPrintDiagnostic = false;
		}

		if (bPrintDiagnostic)
		{
			TStringBuilder<256> PackageNameStr(InPlace, PackageData.GetPackageName());

			// ExternalActors: Do not send a message for every NeverCook external Actor package; too much spam
			if ((Reason == ESuppressCookReason::NeverCook) || (Reason == ESuppressCookReason::OnlyEditorOnly))
			{
				bPrintDiagnostic &= UE::String::FindFirst(PackageNameStr.ToView(),
					ULevel::GetExternalActorsFolderName(), ESearchCase::IgnoreCase) == INDEX_NONE;
				bPrintDiagnostic &= UE::String::FindFirst(PackageNameStr.ToView(),
					FPackagePath::GetExternalObjectsFolderName(), ESearchCase::IgnoreCase) == INDEX_NONE;
			}

			// Reachability: Suppress the diagnostic that were found via cookload reference traversal but are not reachable on the target platforms
			bPrintDiagnostic &= (PackageData.HasInstigator(EReachability::Runtime) || Reason != ESuppressCookReason::OnlyEditorOnly);

			if (bPrintDiagnostic)
			{
				UE_CLOGF((GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators), LogCook, Display,
					"Cooking %ls, Instigator: { %ls } -> Rejected %ls", *PackageNameStr,
					*(PackageData.GetInstigator(EReachability::Runtime).ToString()), LexToString(Reason));
				UE_CLOGF(GCookProgressDisplay & (int32)ECookProgressDisplayMode::PackageNames, LogCook, Display,
					"Cooking %ls -> Rejected %ls", *PackageNameStr, LexToString(Reason));
			}
		}

		// If the package is demoted without a cookresult, store the suppressed reason
		if (!bHasCookResult)
		{
			PackageData.SetSuppressCookReason(Reason);
		}
	}
	PackageData.SendToState(UE::Cook::EPackageState::Idle, SendFlags, ConvertToStateChangeReason(Reason));
}

void UCookOnTheFlyServer::DemoteToRequest(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags,
	UE::Cook::ESuppressCookReason Reason)
{
	using namespace UE::Cook;

	if (!PackageData.IsInProgress())
	{
		return;
	}

	UE_LOGF(LogCook, Display, "DemoteToRequest: Package %ls was sent back to request state, with reason %ls.",
		*WriteToString<256>(PackageData.GetPackageName()), LexToString(Reason));
	WorkerRequests->ReportDemotion(PackageData, Reason);
	if (CookWorkerClient)
	{
		PackageData.SendToState(UE::Cook::EPackageState::Idle, SendFlags, ConvertToStateChangeReason(Reason));
	}
	else
	{
		EnumRemoveFlags(SendFlags, ESendFlags::QueueAdd);
		TPackageDataMap<ESuppressCookReason>& RestartedRequests = PackageDatas->GetRequestQueue().GetRestartedRequests();
		PackageData.SendToState(UE::Cook::EPackageState::Request, SendFlags, ConvertToStateChangeReason(Reason));
		RestartedRequests.Add(&PackageData, Reason);
	}
}

void UCookOnTheFlyServer::PromoteToSaveComplete(UE::Cook::FPackageData& PackageData, UE::Cook::ESendFlags SendFlags)
{
	if (!PackageData.IsInProgress())
	{
		UE_LOGF(LogCook, Error, "Package %ls is in PromoteToSaveComplete but is not in progress.",
			*PackageData.GetPackageName().ToString());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	WorkerRequests->ReportPromoteToSaveComplete(PackageData);
	PackageData.SendToState(UE::Cook::EPackageState::Idle, SendFlags, UE::Cook::EStateChangeReason::Saved);
}

void UCookOnTheFlyServer::PumpLoads(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength, int32& OutNumPushed, bool& bOutBusy)
{
	using namespace UE::Cook;
	FLoadQueue& LoadQueue = PackageDatas->GetLoadQueue();
	FPackageDataMonitor& Monitor = PackageDatas->GetMonitor();
	bool bIsBlockingUrgencyInProgress = Monitor.GetNumUrgent(EUrgency::Blocking) > 0;
	OutNumPushed = 0;
	bOutBusy = false;

	TSet<FPackageData*>& InProgress = LoadQueue.InProgress;
	TSet<TRefCountPtr<FPackagePreloader>>& ActivePreloads = LoadQueue.ActivePreloads;
	TRingBuffer<TRefCountPtr<FPackagePreloader>>& ReadyForLoads = LoadQueue.ReadyForLoads;

	if (bIsBlockingUrgencyInProgress && !Monitor.GetNumUrgent(EPackageState::Load, EUrgency::Blocking))
	{
		return;
	}

	// Process loads until we reduce the queue size down to the desired size or we hit the max number of loads per batch
	// We do not want to load too many packages without saving because if we hit the memory limit and GC every package
	// we load will have to be loaded again
	while (InProgress.Num() > static_cast<int32>(DesiredQueueLength) && OutNumPushed < LoadBatchSize)
	{
		if (StackData.Timer.IsActionTimeUp())
		{
			return;
		}
		if (bIsBlockingUrgencyInProgress && !Monitor.GetNumUrgent(EPackageState::Load, EUrgency::Blocking))
		{
			return;
		}
		COOK_STAT(DetailedCookStats::PeakLoadQueueSize = FMath::Max(DetailedCookStats::PeakLoadQueueSize, InProgress.Num()));

		if (!ReadyForLoads.IsEmpty())
		{
			TRefCountPtr<FPackagePreloader> Preloader = ReadyForLoads.PopFrontValue();
			FPackageData& PackageData(Preloader->GetPackageData());
			if (PackageData.GetState() == EPackageState::Load)
			{
				// A PackageData is in the load state, and we are done with preloading its imports
				// and are ready to load it.
				// Call extra code to add logging and state-transitioning the package out of load.
				int32 NumPushed;
				LoadPackageInQueue(PackageData, StackData.ResultFlags, NumPushed);
				OutNumPushed += NumPushed;
			}
			else
			{
				// The PackagePreloader is done preloading and needs to be loaded, but its PackageData
				// is in some other state. Just do minimum amount of work to load the package.
				// Note that generated packages do not come through here; they are never added because
				// they are not in any other package's import tree.
				if (PackageData.IsGenerated())
				{
					UE_LOGF(LogCook, Warning,
						"Package %ls is generated but is ReadyForLoad when not in load state. State == %ls, PreloaderState == %ls, CountFromRequestedLoads == %d.",
						*WriteToString<256>(PackageData.GetPackageName()), LexToString(PackageData.GetState()),
						LexToString(Preloader->GetState()), Preloader->GetCountFromRequestedLoads());
				}
				UPackage* UnusedPackage;
				LoadPackageForCooking(PackageData, UnusedPackage);
				Preloader->PumpLoadsMarkLoadAttemptComplete();
			}

			ProcessUnsolicitedPackages(); // May add new packages into LoadInbox
#if ENABLE_LOW_LEVEL_MEM_TRACKER
			FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif
			if (PumpHasExceededMaxMemory(StackData.ResultFlags))
			{
				return;
			}
			continue;
		}

		// Process all values in the inbox until it is empty or we run out of time. Adding values from the inbox may
		// change the front of the PendingKicks priority queue.
		while (FPackagePreloader::PumpLoadsTryStartInboxPackage(*this))
		{
			// Work was done in while condition
			if (StackData.Timer.IsActionTimeUp())
			{
				return;
			}
		}

		// Kick preloads until we run out of preload budget
		while (FPackagePreloader::PumpLoadsTryKickPreload(*this))
		{
			// Work was done in while condition
		}

		// Poll all active preloads
		for (TSet<TRefCountPtr<FPackagePreloader>>::TIterator Iter(ActivePreloads); Iter; ++Iter)
		{
			TRefCountPtr<FPackagePreloader>& IterPreloader = *Iter;
			if (IterPreloader->PumpLoadsIsReadyToLeavePreload())
			{
				TRefCountPtr<FPackagePreloader> Preloader(IterPreloader);
				Iter.RemoveCurrent();
				Preloader->SendToState(EPreloaderState::ReadyForLoad, ESendFlags::QueueAdd);
			}
		}

		// If we did not find any packages ready to load then report the load queue is busy waiting for preloads
		if (ReadyForLoads.IsEmpty())
		{
			bOutBusy = true;
			break;
		}
	}
}

void UCookOnTheFlyServer::LoadPackageInQueue(UE::Cook::FPackageData& PackageData, uint32& ResultFlags, int32& OutNumPushed)
{
	using namespace UE::Cook;

	UPackage* LoadedPackage = nullptr;
	OutNumPushed = 0;
	TRefCountPtr<FPackagePreloader> Preloader = PackageData.CreatePackagePreloader();
	check(Preloader->GetState() == EPreloaderState::ReadyForLoad);

	FName PackageFileName(PackageData.GetFileName());
	if (!PackageData.IsGenerated())
	{
		bool bLoadFullySuccessful = LoadPackageForCooking(PackageData, LoadedPackage);
		// Mark the load attempt complete before we do any state transition of the PackageData.
		Preloader->PumpLoadsMarkLoadAttemptComplete();
		if (!bLoadFullySuccessful)
		{
			ResultFlags |= COSR_ErrorLoadingPackage;
			UE_LOGF(LogCook, Verbose, "Not cooking package %ls", *PackageFileName.ToString());
			RejectPackageToLoad(PackageData, TEXT("failed to load"), ESuppressCookReason::LoadError);
			return;
		}
		check(LoadedPackage != nullptr && LoadedPackage->IsFullyLoaded());

		if (LoadedPackage->GetFName() != PackageData.GetPackageName())
		{
			// The PackageName is not the name that we loaded. This can happen due to CoreRedirects.
			// We refuse to cook requests for packages that no longer exist in PumpExternalRequests, but it is possible
			// that a CoreRedirect exists from a (externally requested or requested as a reference) package that still exists.
			// Mark the original PackageName as cooked for all platforms and send a request to cook the new FileName
			FPackageData& OtherPackageData = PackageDatas->AddPackageDataByPackageNameChecked(LoadedPackage->GetFName());
			UE_LOGF(LogCook, Verbose, "Request for %ls received going to save %ls", *PackageFileName.ToString(),
				*OtherPackageData.GetFileName().ToString());
			QueueDiscoveredPackage(OtherPackageData,
				FInstigator(EInstigator::ForceExplorableSaveTimeSoftDependency, PackageData.GetPackageName()),
				EDiscoveredPlatformSet::CopyFromInstigator);

			PackageData.SetPlatformsCooked(PlatformManager->GetSessionPlatforms(), ECookResult::Succeeded);
			RejectPackageToLoad(PackageData, TEXT("is redirected to another filename"), ESuppressCookReason::Redirected);
			return;
		}
	}
	else
	{
		// Generated packages do not use the preload, so go ahead and mark it complete now. As with regular packages,
		// we need to mark it complete before any state transitions of the PackageData.
		Preloader->PumpLoadsMarkLoadAttemptComplete();
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.TryCreateValidParentGenerationHelper();
		if (!GenerationHelper)
		{
			UE_LOGF(LogCook, Error,
				"Package %ls is a generated package, but ParentGenerator '%ls' is not a generator package. The generated package cannot be loaded.",
				*PackageFileName.ToString(), *PackageData.GetParentGenerator().ToString());
			RejectPackageToLoad(PackageData, TEXT("is an orphaned generated package"), ESuppressCookReason::OrphanedGenerated);
			return;
		}
		if (!GenerationHelper->TryReportGenerationManifest())
		{
			RejectPackageToLoad(PackageData, TEXT("is an orphaned generated package"), ESuppressCookReason::OrphanedGenerated);
			return;
		}
		FCookGenerationInfo* Info = GenerationHelper->FindInfo(PackageData);
		if (!Info)
		{
			UE_LOGF(LogCook, Error,
				"Package %ls is a generated package but its generator does not have a record of it. It can not be loaded.",
				*PackageFileName.ToString());
			TArray<FString> GeneratedNames;
			for (FCookGenerationInfo& ExistingInfo : GenerationHelper->GetPackagesToGenerate())
			{
				GeneratedNames.Add(ExistingInfo.GetPackageName());
			}
			GeneratedNames.Sort();
			TStringBuilder<1024> GeneratedNamesListStr;
			constexpr int32 MaxCount = 10;
			int32 Count = 0;
			for (const FString& GeneratedName : GeneratedNames)
			{
				if (Count++ >= MaxCount)
				{
					GeneratedNamesListStr << TEXT("\n\t...");
					break;
				}
				GeneratedNamesListStr << TEXT("\n\t") << GeneratedName;
			}
			UE_LOGF(LogCook, Display, "The generator has %d generated packages, but %ls is not one of them:%ls",
				GeneratedNames.Num(), *WriteToString<256>(PackageData.GetPackageName()), *GeneratedNamesListStr);
			RejectPackageToLoad(PackageData, TEXT("is an orphaned generated package"), ESuppressCookReason::OrphanedGenerated);
			return;
		}

		LoadedPackage = GenerationHelper->TryCreateGeneratedPackage(*Info, true /* bResetToEmpty */);
		if (!LoadedPackage)
		{
			RejectPackageToLoad(PackageData, TEXT("is a generated package which could not be populated"),
				ESuppressCookReason::LoadError);
			return;
		}
	}

	if (PackageData.GetPlatformsNeedingCommitNum(GetCookPhase()) == 0)
	{
		// Already cooked. This can happen if we needed to load a package that was previously cooked and garbage collected because it is a loaddependency of a new request.
		// Send the package back to idle, nothing further to do with it.
		DemoteToIdle(PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::AlreadyCooked);
		return;
	}

	if (ValidateSourcePackage(PackageData, LoadedPackage) == EDataValidationResult::Invalid)
	{
		if (ValidationArtifact)
		{
			ValidationArtifact->MarkPackageRequiresFullValidation(LoadedPackage->GetFName());
		}

		if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::ValidationErrorsAreFatal))
		{
			UE_LOGF(LogCook, Error, "%ls failed validation", *LoadedPackage->GetName());

			PackageData.SetPlatformsCooked(PlatformManager->GetSessionPlatforms(), ECookResult::Failed);
			RejectPackageToLoad(PackageData, TEXT("failed validation"), ESuppressCookReason::ValidationError);
			return;
		}

		UE_LOGF(LogCook, Warning, "%ls failed validation", *LoadedPackage->GetName());
	}

	PostLoadPackageFixup(PackageData, LoadedPackage);
	PackageData.SetPackage(LoadedPackage);
	PackageData.CreateLoadDependencies();
	PackageData.SendToState(EPackageState::SaveActive, ESendFlags::QueueAddAndRemove, EStateChangeReason::Loaded);
	++OutNumPushed;
}

void UCookOnTheFlyServer::RejectPackageToLoad(UE::Cook::FPackageData& PackageData, const TCHAR* ReasonText, UE::Cook::ESuppressCookReason Reason)
{
	// make sure this package doesn't exist
	for (const TPair<const ITargetPlatform*, UE::Cook::FPackagePlatformData>& Pair : PackageData.GetPlatformDatas())
	{
		if (Pair.Key == CookerLoadingPlatformKey || !Pair.Value.NeedsCommit(Pair.Key, GetCookPhase()))
		{
			continue;
		}
		const ITargetPlatform* TargetPlatform = Pair.Key;

		const FString SandboxFilename = ConvertToFullSandboxPath(PackageData.GetFileName().ToString(), true, TargetPlatform->PlatformName());
		if (IFileManager::Get().FileExists(*SandboxFilename))
		{
			// if we find the file this means it was cooked on a previous cook, however source package can't be found now. 
			// this could be because the source package was deleted or renamed, and we are using legacyiterative cooking
			// perhaps in this case we should delete it?
			UE_LOGF(LogCook, Warning, "Found cooked file '%ls' which shouldn't exist as it %ls.", *SandboxFilename, ReasonText);
			IFileManager::Get().Delete(*SandboxFilename);
		}
	}
	DemoteToIdle(PackageData, UE::Cook::ESendFlags::QueueAddAndRemove, Reason);
}

EDataValidationResult UCookOnTheFlyServer::ValidateSourcePackage(UE::Cook::FPackageData& PackageData, UPackage* Package)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(ValidateSourcePackage, DetailedCookStats::ValidationTimeSec);

	// Don't validate packages if validation is disabled
	if (!EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::RunAssetValidation | ECookByTheBookOptions::RunMapValidation))
	{
		return EDataValidationResult::NotValidated;
	}

	// Don't validate packages generated during cook
	if (PackageData.IsGenerated())
	{
		return EDataValidationResult::NotValidated;
	}

	// Don't validate packages that are already cooked
	if (Package->HasAnyPackageFlags(PKG_Cooked))
	{
		return EDataValidationResult::NotValidated;
	}

	FNameBuilder PackageName(Package->GetFName());
	const FStringView ContentRootName = FPackageName::SplitPackageNameRoot(PackageName.ToView(), nullptr);

	// Don't validate Verse packages, as the Verse compiler handles that
	if (FPackageName::IsVersePackage(PackageName.ToView()))
	{
		return EDataValidationResult::NotValidated;
	}

	// When cooking DLC, don't validate anything outside of the DLC plugin
	if (IsCookingDLC() && ContentRootName != CookByTheBookOptions->DlcName)
	{
		return EDataValidationResult::NotValidated;
	}

	// When cooking a project, don't validate any engine content as it may not pass the project specific validators
	if (FApp::HasProjectName())
	{
		if (ContentRootName == TEXTVIEW("Engine"))
		{
			return EDataValidationResult::NotValidated;
		}
		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(ContentRootName);
			Plugin && Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine)
		{
			return EDataValidationResult::NotValidated;
		}
	}

	// Don't validate packages that won't actually be cooked
	{
		UAssetManager& AssetManager = UAssetManager::Get();
		
		if (!AssetManager.VerifyCanCookPackage(this, Package->GetFName(), /*bLogError*/false))
		{
			return EDataValidationResult::NotValidated;
		}

		bool bShouldCookForAnyPlatform = false;
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			if (AssetManager.ShouldCookForPlatform(Package, TargetPlatform))
			{
				if (const TSet<FName>* NeverCookPackages = PackageTracker->PlatformSpecificNeverCookPackages.Find(TargetPlatform);
					!NeverCookPackages || !NeverCookPackages->Contains(Package->GetFName()))
				{
					bShouldCookForAnyPlatform = true;
					break;
				}
			}
		}
		if (!bShouldCookForAnyPlatform)
		{
			return EDataValidationResult::NotValidated;
		}
	}

#if DEBUG_COOKONTHEFLY 
	UE_LOGF(LogCook, Display, "Validating package %ls", *PackageName);
#endif

	const bool bLogErrorsAsWarnings = !EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::ValidationErrorsAreFatal);
	EDataValidationResult FinalValidationResult = EDataValidationResult::NotValidated;

	UWorld* World = nullptr;
	if (Package->HasAnyPackageFlags(PKG_ContainsMap))
	{
		World = UWorld::FindWorldInPackage(Package);
	}

	bool bRunCleanupWorld = false;
	if (World && !World->bIsWorldInitialized)
	{
		FWorldInitializationValues IVS;
		IVS.AllowAudioPlayback(false);
		IVS.RequiresHitProxies(false);
		IVS.ShouldSimulatePhysics(false);
		IVS.EnableTraceCollision(true);
		IVS.SetTransactional(false);
		IVS.CreateWorldPartition(true);
		IVS.CreateAISystem(false);
		IVS.CreateNavigation(false);

		World->InitWorld(IVS);
		World->UpdateWorldComponents(true, false);
		bRunCleanupWorld = true;
	}

	// Run asset validation if requested
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::RunAssetValidation) && UE::Cook::FDelegates::ValidateSourcePackage.IsBound())
	{
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		
		TArray<FAssetData> ExternalObjects = UE::Cook::GetExternalObjectsForPackageValidation(Package->GetFName(), World != nullptr);
		if (ValidationArtifact)
		{
			ValidationArtifact->FilterExternalPackagesToValidate(Package->GetFName(), ExternalObjects);
		}

		FMessageLogScopedOverride AssetCheckLogOverride(UE::DataValidation::MessageLogName);
		if (bLogErrorsAsWarnings)
		{
			AssetCheckLogOverride.RemapMessageSeverity(EMessageSeverity::Error, EMessageSeverity::Warning);
		}

		TGuardValue<bool> LogErrorsAsWarnings(GWarn->TreatErrorsAsWarnings, GWarn->TreatErrorsAsWarnings || bLogErrorsAsWarnings);

		FDataValidationContext ValidationContext(IsRunningCookCommandlet(), EDataValidationUsecase::Save, ExternalObjects);
		const EDataValidationResult ValidationResult = UE::Cook::FDelegates::ValidateSourcePackage.Execute(Package, ValidationContext);
		FinalValidationResult = CombineDataValidationResults(FinalValidationResult, ValidationResult);
	}

	// Run map validation if requested
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::RunMapValidation) && World)
	{
		static const FName NAME_MapCheck = "MapCheck";

		FMessageLogScopedOverride MapCheckLogOverride(NAME_MapCheck);
		if (bLogErrorsAsWarnings)
		{
			MapCheckLogOverride.RemapMessageSeverity(EMessageSeverity::Error, EMessageSeverity::Warning);
		}

		TGuardValue<bool> LogErrorsAsWarnings(GWarn->TreatErrorsAsWarnings, GWarn->TreatErrorsAsWarnings || bLogErrorsAsWarnings);

		GEditor->Exec(World, TEXT("MAP CHECK"));
		
		FMessageLog MapCheckLog(NAME_MapCheck);
		if (MapCheckLog.NumMessages(EMessageSeverity::Error) > 0)
		{
			FinalValidationResult = EDataValidationResult::Invalid;
		}
	}

	if (bRunCleanupWorld)
	{
		checkf(World, TEXT("bRunCleanupWorld was true but World was null!"));
		checkf(World->bIsWorldInitialized, TEXT("bRunCleanupWorld was true but World->bIsWorldInitialized was false!"));

		World->ClearWorldComponents();
		World->CleanupWorld();
		World->SetPhysicsScene(nullptr);
	}

	return FinalValidationResult;
}

void UCookOnTheFlyServer::QueueDiscoveredPackage(UE::Cook::FPackageData& PackageData,
	UE::Cook::FInstigator&& Instigator, UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms)
{
	QueueDiscoveredPackage(PackageData, MoveTemp(Instigator), MoveTemp(ReachablePlatforms),
		UE::Cook::EUrgency::Normal, nullptr /* ParentGenerationHelper */);
}

void UCookOnTheFlyServer::QueueDiscoveredPackage(UE::Cook::FPackageData& PackageData,
	UE::Cook::FInstigator&& Instigator, UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms,
	UE::Cook::EUrgency Urgency, UE::Cook::FGenerationHelper* ParentGenerationHelper)
{
	using namespace UE::Cook;

	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> BufferPlatforms;
	TConstArrayView<const ITargetPlatform*> DiscoveredPlatforms;
	EReachability DiscoveredReachability =
		Instigator.Category == EInstigator::BuildDependency ? EReachability::Build : EReachability::Runtime;
	if (!bSkipOnlyEditorOnly)
	{
		BufferPlatforms = PlatformManager->GetSessionPlatforms();
		BufferPlatforms.Add(CookerLoadingPlatformKey);
		DiscoveredPlatforms = BufferPlatforms;
	}
	else
	{
		DiscoveredPlatforms = ReachablePlatforms.GetPlatforms(*this, &Instigator,
			TConstArrayView<const ITargetPlatform*>(), DiscoveredReachability, BufferPlatforms);
	}
	if (Instigator.Category != EInstigator::ForceExplorableSaveTimeSoftDependency &&
		PackageData.HasReachablePlatforms(DiscoveredReachability, DiscoveredPlatforms))
	{
		// Not a new discovery; ignore
		return;
	}

	if (bHiddenDependenciesDebug)
	{
		OnDiscoveredPackageDebug(PackageData.GetPackageName(), Instigator);
	}
	WorkerRequests->QueueDiscoveredPackage(*this, PackageData, MoveTemp(Instigator), MoveTemp(ReachablePlatforms),
		Urgency, ParentGenerationHelper);
}

void UCookOnTheFlyServer::QueueDiscoveredPackageOnDirector(UE::Cook::FPackageData& PackageData,
	UE::Cook::FInstigator&& Instigator, UE::Cook::FDiscoveredPlatformSet&& ReachablePlatforms,
	UE::Cook::EUrgency Urgency)
{
	using namespace UE::Cook;

	if (CookOnTheFlyRequestManager)
	{
		if (PackageData.IsGenerated())
		{
			CookOnTheFlyRequestManager->OnPackageGenerated(PackageData.GetPackageName());
		}
		if (!CookOnTheFlyRequestManager->ShouldUseLegacyScheduling())
		{
			return;
		}
	}

	if (!CookByTheBookOptions->bSkipHardReferences ||
		(Instigator.Category == EInstigator::GeneratedPackage))
	{
		PackageData.QueueAsDiscovered(MoveTemp(Instigator), MoveTemp(ReachablePlatforms), Urgency);
	}
}

void UCookOnTheFlyServer::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform, int32 RemovedIndex)
{
	using namespace UE::Cook;

	for (FDiscoveryQueueElement& Element : PackageDatas->GetRequestQueue().GetDiscoveryQueue())
	{
		Element.ReachablePlatforms.OnRemoveSessionPlatform(TargetPlatform, RemovedIndex);
	}
	for (TUniquePtr<FRequestCluster>& Cluster : PackageDatas->GetRequestQueue().GetRequestClusters())
	{
		Cluster->OnRemoveSessionPlatform(TargetPlatform);
	}
	PackageDatas->OnRemoveSessionPlatform(TargetPlatform);
	WorkerRequests->OnRemoveSessionPlatform(TargetPlatform);
}

void UCookOnTheFlyServer::OnBeforePlatformAddedToSession(const ITargetPlatform* TargetPlatform)
{
	using namespace UE::Cook;

	for (TUniquePtr<FRequestCluster>& Cluster : PackageDatas->GetRequestQueue().GetRequestClusters())
	{
		Cluster->OnBeforePlatformAddedToSession(TargetPlatform);
	}
}

void UCookOnTheFlyServer::OnPlatformAddedToSession(const ITargetPlatform* TargetPlatform)
{
	using namespace UE::Cook;

	for (FDiscoveryQueueElement& Element : PackageDatas->GetRequestQueue().GetDiscoveryQueue())
	{
		Element.ReachablePlatforms.OnPlatformAddedToSession(TargetPlatform);
	}
}

void UCookOnTheFlyServer::TickNetwork()
{
	// Only CookOnTheFly handles network requests
	// It is not safe to call PruneUnreferencedSessionPlatforms in CookByTheBook because StartCookByTheBook does not AddRef its session platforms
	check(IsCookOnTheFlyMode())
	if (IsInSession())
	{
		if (!bCookOnTheFlyExternalRequests)
		{
			PlatformManager->PruneUnreferencedSessionPlatforms(*this);
		}
	}
	else
	{
		// Process callbacks in case there is a callback pending that needs to create a session
		TArray<UE::Cook::FSchedulerCallback> Callbacks;
		if (WorkerRequests->DequeueSchedulerCallbacks(Callbacks))
		{
			for (UE::Cook::FSchedulerCallback& Callback : Callbacks)
			{
				Callback();
			}
		}
	}
}

UE::Cook::EPollStatus UCookOnTheFlyServer::QueueGeneratedPackages(UE::Cook::FGenerationHelper& GenerationHelper,
	UE::Cook::FPackageData& PackageData)
{
	using namespace UE::Cook;

	FCookGenerationInfo& Info = GenerationHelper.GetOwnerInfo();

	UPackage* Owner = PackageData.GetPackage();
	FName OwnerName = Owner->GetFName();
	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_QueueGeneratedPackages)
	{
		GenerationHelper.StartQueueGeneratedPackages(*this);
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> ReachablePlatforms;
		PackageData.GetReachablePlatforms(EReachability::Runtime, ReachablePlatforms);
		for (const FCookGenerationInfo& ChildInfo: GenerationHelper.GetPackagesToGenerate())
		{
			FPackageData* ChildPackageData = ChildInfo.PackageData;
			// Set the Instigator now rather than delaying it until the discovery queue is processed.
			ChildPackageData->SetInstigator(GenerationHelper, EReachability::Runtime,
				FInstigator(EInstigator::GeneratedPackage, OwnerName));
			// The urgency of generated packages must be at least as high as the generator to satisfy the contract of 
			// making the generator urgent. By default they are High urgency rather than Normal so that they are saved 
			// quickly, so that we release the memory used by their generator for them.
			EUrgency Urgency = PackageData.GetUrgency() > EUrgency::High ? PackageData.GetUrgency() : EUrgency::High;

			// Queue the package for cooking
			QueueDiscoveredPackage(*ChildPackageData, FInstigator(ChildPackageData->GetInstigator(EReachability::Runtime)),
				EDiscoveredPlatformSet::CopyFromInstigator, Urgency, &GenerationHelper);
		}
		GenerationHelper.EndQueueGeneratedPackages(*this);
	}
	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::PrepareSaveGenerationPackage(UE::Cook::FGenerationHelper& GenerationHelper,
	UE::Cook::FPackageData& PackageData, UE::Cook::FCookerTimer& Timer, bool bPrecaching)
{
	using namespace UE::Cook;

	FCookGenerationInfo* InfoPtr = GenerationHelper.FindInfo(PackageData);
	if (!InfoPtr)
	{
		UE_LOGF(LogCook, Error, "Generated package %ls is missing its generation data and cannot be saved.",
			*PackageData.GetPackageName().ToString());
		return EPollStatus::Error;
	}
	FCookGenerationInfo& Info(*InfoPtr);

	if (PackageData.GetSaveSubState() <= ESaveSubState::CheckForIsGenerated)
	{
		// We cannot proceed with Populate,PreSave,Save,PostSave on the generator until after
		// QueuedGeneratedPackagesFencePassed, because we might discover during processing of that fence that we are
		// not allowed to save the generator package.
		if (Info.IsGenerator() && !GenerationHelper.IsQueuedGeneratedPackagesFencePassed())
		{
			return EPollStatus::Incomplete;
		}
	}

	// If we moved a FPendingCookedPlatformData object between generator and a generated package, wait for it
	// to complete before we allow the generator or moved generated package to proceed. For simplicity, we currently
	// block all generated packages from proceeding rather than keeping track of which ones have the pending objects.
	if (GenerationHelper.HasBlockingPendingCookedPlatformDataObjects())
	{
		return EPollStatus::Incomplete;
	}

	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_PreMoveCookedPlatformData_WaitingForIsLoaded)
	{
		// Both Generator packages and Generated packages should wait for all IsCachedCookedPlatformData
		// to finish before they start BeginCache calls on the objects to move.
		if (PackageData.GetNumPendingCookedPlatformData() > 0)
		{
			return EPollStatus::Incomplete;
		}

		// If generator should not save until after generated, stall it here
		if (Info.IsGenerator()
			&& FGenerationHelper::IsGeneratedSavedFirst()
			// Splitters that declare DoesGeneratedRequireGenerator=Save ignore the global setting and
			// never wait for generated to save
			&& GenerationHelper.DoesGeneratedRequireGenerator() <
				ICookPackageSplitter::EGeneratedRequiresGenerator::Save)
		{
			if (GenerationHelper.IsWaitingForQueueResults())
			{
				return EPollStatus::Incomplete;
			}
			for (FCookGenerationInfo& GeneratedInfo : GenerationHelper.GetPackagesToGenerate())
			{
				if (GeneratedInfo.PackageData->IsInProgress())
				{
					return EPollStatus::Incomplete;
				}
			}
		}

		// If generated should not save until after generator, stall it here
		if (!Info.IsGenerator()
			&& (FGenerationHelper::IsGeneratorSavedFirst()
			// Splitters that declare DoesGeneratedRequireGenerator=Save ignore the global setting and
			// always wait for the generator to save
				|| GenerationHelper.DoesGeneratedRequireGenerator() >= 
					ICookPackageSplitter::EGeneratedRequiresGenerator::Save))
		{
			if (GenerationHelper.GetOwner().IsInProgress())
			{
				return EPollStatus::Incomplete;
			}
		}
		PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_PreMoveCookedPlatformData_WaitingForIsLoaded);
	}

	// GeneratedPackagesForPopulate is used by multiple steps, recreate it when needed each time we come in to this function
	TArray<ICookPackageSplitter::FGeneratedPackageForPopulate> GeneratedPackagesForPopulate;
	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_FinishCacheObjectsToMove)
	{
		if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_BeginCacheObjectsToMove)
		{
			EPollStatus Result = BeginCacheObjectsToMove(GenerationHelper, Info, Timer, GeneratedPackagesForPopulate);
			if (Result != EPollStatus::Success)
			{
				return Result;
			}
			PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_BeginCacheObjectsToMove);
		}
		check(PackageData.GetSaveSubState() <= ESaveSubState::Generation_FinishCacheObjectsToMove);
		if (PackageData.GetNumPendingCookedPlatformData() > 0 ||
			GenerationHelper.HasBlockingPendingCookedPlatformDataObjects())
		{
			return EPollStatus::Incomplete;
		}
		bool bFoundNewObjects;
		EPollStatus Result = Info.RefreshPackageObjects(GenerationHelper, PackageData.GetPackage(), bFoundNewObjects,
			ESaveSubState::Generation_BeginCacheObjectsToMove);
		if (Result != EPollStatus::Success)
		{
			return Result;
		}
		if (bFoundNewObjects)
		{
			// Call this function recursively to reexecute CallBeginCacheOnObjects in BeginCacheObjectsToMove.
			// Note that RefreshPackageObjects checked for too many recursive calls and ErrorExited if so.
			return PrepareSaveGenerationPackage(GenerationHelper, PackageData, Timer, bPrecaching);
		}
		PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_FinishCacheObjectsToMove);
	}

	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_CallPreSave)
	{
		if (bPrecaching)
		{
			// We're not allowed to PreSave when precaching, because we want to avoid garbagecollection in between
			// PreSaving and PostSaving the package, so we need to not PreSave until we're ready to save.
			return EPollStatus::Incomplete;
		}

		EPollStatus Result;
		if (Info.IsGenerator())
		{
			Result = PreSaveGeneratorPackage(PackageData, GenerationHelper, Info, GeneratedPackagesForPopulate);
		}
		else
		{
			Result = TryPreSaveGeneratedPackage(GenerationHelper, Info);
		}
		if (Result != EPollStatus::Success)
		{
			return Result;
		}
		PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_CallPreSave);
	}

	if (PackageData.GetSaveSubState() <= ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded)
	{
		if (PackageData.GetSaveSubState() <= ESaveSubState::LastCookedPlatformData_CallingBegin)
		{
			EPollStatus Result = BeginCachePostMove(GenerationHelper, Info, Timer);
			if (Result != EPollStatus::Success)
			{
				return Result;
			}
			PackageData.SetSaveSubStateComplete(ESaveSubState::LastCookedPlatformData_CallingBegin);
		}
		check(PackageData.GetSaveSubState() <= ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded);
		if (PackageData.GetNumPendingCookedPlatformData() > 0)
		{
			return EPollStatus::Incomplete;
		}
		bool bFoundNewObjects;
		EPollStatus Result = Info.RefreshPackageObjects(GenerationHelper, PackageData.GetPackage(), bFoundNewObjects,
			ESaveSubState::LastCookedPlatformData_CallingBegin);
		if (Result != EPollStatus::Success)
		{
			return Result;
		}
		if (bFoundNewObjects)
		{
			// Call this function recursively to reexecute CallBeginCacheOnObjects in BeginCachePostMove
			// Note that RefreshPackageObjects checked for too many recursive calls and ErrorExited if so.
			return PrepareSaveGenerationPackage(GenerationHelper, PackageData, Timer, bPrecaching);
		}

		PackageData.SetSaveSubStateComplete(ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded);
	}
	check(PackageData.GetSaveSubState() == ESaveSubState::ReadyForSave);

	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::BeginCacheObjectsToMove(UE::Cook::FGenerationHelper& GenerationHelper,
	UE::Cook::FCookGenerationInfo& Info, UE::Cook::FCookerTimer& Timer,
	TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate)
{
	using namespace UE::Cook;

	FPackageData& PackageData(*Info.PackageData);
	UPackage* Package = PackageData.GetPackage();
	if (!Package)
	{
		UE_LOGF(LogCook, Error,
			"CookPackageSplitter is missing package during BeginCacheObjectsToMove. PackageName: %ls.",
			*PackageData.GetPackageName().ToString());
		return EPollStatus::Error;
	}

	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_CallObjectsToMove)
	{
		bool bGeneratedRequiresGeneratorPopulate = GenerationHelper.DoesGeneratedRequireGenerator()
			>= ICookPackageSplitter::EGeneratedRequiresGenerator::Populate;
		if (Info.IsGenerator() || bGeneratedRequiresGeneratorPopulate)
		{
			bool bPopulateCallAllowed = bGeneratedRequiresGeneratorPopulate ||
				(Info.IsGenerator() && GenerationHelper.IsSaveGenerator());
			if (!GenerationHelper.TryCallPopulateGeneratorPackage(GeneratedPackagesForPopulate, bPopulateCallAllowed))
			{
				return EPollStatus::Error;
			}
		}
		TArray<UObject*> ObjectsToMove;
		if (Info.IsGenerator())
		{
			ObjectsToMove.Reserve(GenerationHelper.GetOwnerObjectsToMove().Num());
			for (const FWeakObjectPtr& ObjectToMove : GenerationHelper.GetOwnerObjectsToMove())
			{
				UObject* Object = ObjectToMove.Get();
				if (Object)
				{
					ObjectsToMove.Add(Object);
				}
			}
		}
		else
		{
			if (!GenerationHelper.TryCallPopulateGeneratedPackage(Info, ObjectsToMove))
			{
				return EPollStatus::Error;
			}
		}

		Info.TakeOverCachedObjectsAndAddMoved(GenerationHelper, PackageData.GetCachedObjectsInOuter(), ObjectsToMove);
		PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_CallObjectsToMove);
	}

	EPollStatus Result = CallBeginCacheOnObjects(PackageData, Package, PackageData.GetCachedObjectsInOuter(),
		PackageData.GetCookedPlatformDataNextIndex(), Timer);
	if (Result != EPollStatus::Success)
	{
		return Result;
	}
	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::PreSaveGeneratorPackage(UE::Cook::FPackageData& PackageData,
	UE::Cook::FGenerationHelper& GenerationHelper, UE::Cook::FCookGenerationInfo& Info,
	TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate)
{
	using namespace UE::Cook;

	UPackage* Package = PackageData.GetPackage();
	ICookPackageSplitter* Splitter = GenerationHelper.GetCookPackageSplitterInstance();
	UObject* SplitDataObject = GenerationHelper.FindOrLoadSplitDataObject();
	if (!Package || !Splitter || !SplitDataObject)
	{
		UE_LOGF(LogCook, Error, "CookPackageSplitter is missing %ls during PreSaveGeneratorPackage. PackageName: %ls.",
			(!Package ? TEXT("Package") : (!Splitter ? TEXT("Splitter") : TEXT("SplitDataObject"))),
			*PackageData.GetPackageName().ToString());
		return EPollStatus::Error;
	}

	if (!TryConstructGeneratedPackagesForPopulate(PackageData, GenerationHelper, GeneratedPackagesForPopulate))
	{
		UE_LOGF(LogCook, Error, "PackageSplitter unexpected failure: could not Construct GeneratedPackagesForPopulate. Splitter=%ls",
			*GenerationHelper.GetSplitDataObjectName().ToString());
		return EPollStatus::Error;
	}

	UE::Cook::CookPackageSplitter::FPopulateContextData PopulateData;
	// By Contract we do not call PreSaveGeneratorPackage if not saving generator, but we execute the rest of this
	// function so that other GenerationHelper code can rely on the support tasks for the PreSave being done.
	if (GenerationHelper.IsSaveGenerator())
	{
		UE::Cook::FScopedCookActivePackage ScopedActivePackage(GenerationHelper.GetOwner().GetPackageName(),
#if UE_WITH_OBJECT_HANDLE_TRACKING
			PackageAccessTrackingOps::NAME_CookerBuildObject
#else
			FName()
#endif
		);
		TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

		ICookPackageSplitter::FPopulateContext PopulateContext(PopulateData);
		PopulateData.OwnerPackage = Package;
		PopulateData.OwnerObject = SplitDataObject;
		PopulateData.GeneratedPackages = GeneratedPackagesForPopulate;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		bool bPreSaveSucceeded = Splitter->PreSaveGeneratorPackage(PopulateData.OwnerPackage, PopulateData.OwnerObject,
			GeneratedPackagesForPopulate, PopulateData.KeepReferencedPackages);
		Splitter->WarnIfDeprecatedVirtualNotCalled(TEXT("PreSaveGeneratorPackage"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		bPreSaveSucceeded = Splitter->PreSaveGeneratorPackage(PopulateContext) && bPreSaveSucceeded;
		if (!bPreSaveSucceeded)
		{
			UE_LOGF(LogCook, Error, "PackageSplitter returned false from PreSaveGeneratorPackage. Splitter=%ls",
				*GenerationHelper.GetSplitDataObjectName().ToString());
			return EPollStatus::Error;
		}
	}
	Info.AddKeepReferencedPackages(GenerationHelper, PopulateData.KeepReferencedPackages);

	return EPollStatus::Success;
}

bool UCookOnTheFlyServer::TryConstructGeneratedPackagesForPopulate(UE::Cook::FPackageData& PackageData, UE::Cook::FGenerationHelper& GenerationHelper,
	TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackagesForPopulate)
{
	using namespace UE::Cook;

	if (GeneratedPackagesForPopulate.Num() > 0)
	{
		// Already constructed, save time by early exiting
		return true;
	}

	// We need to find or (create empty stub packages for) each of the PackagesToGenerate so that PreSaveGeneratorPackage
	// can refer to them to create hardlinks in the cooked Generator package.
	TArrayView<FCookGenerationInfo> PackagesToGenerate = GenerationHelper.GetPackagesToGenerate();
	GeneratedPackagesForPopulate.Reserve(PackagesToGenerate.Num());
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		ICookPackageSplitter::FGeneratedPackageForPopulate& SplitterData = GeneratedPackagesForPopulate.Emplace_GetRef();
		SplitterData.RelativePath = Info.RelativePath;
		SplitterData.GeneratedRootPath = Info.GeneratedRootPath;
		SplitterData.bCreatedAsMap = Info.IsCreateAsMap();
		SplitterData.Package = GenerationHelper.TryCreateGeneratedPackage(Info, false /* bResetToEmpty */);
		if (!SplitterData.Package)
		{
			return false;
		}
	}
	return true;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::BeginCachePostMove(UE::Cook::FGenerationHelper& GenerationHelper,
	UE::Cook::FCookGenerationInfo& Info, UE::Cook::FCookerTimer& Timer)
{
	using namespace UE::Cook;

	UE::Cook::FPackageData& PackageData(*Info.PackageData);
	UPackage* Package = PackageData.GetPackage();
	ICookPackageSplitter* Splitter = GenerationHelper.GetCookPackageSplitterInstance();
	UObject* SplitDataObject = GenerationHelper.FindOrLoadSplitDataObject();
	if (!Package || !Splitter || !SplitDataObject)
	{
		UE_LOGF(LogCook, Error, "CookPackageSplitter is missing %ls during BeginCachePostMove. PackageName: %ls.",
			(!Package ? TEXT("Package") : (!Splitter ? TEXT("Splitter") : TEXT("SplitDataObject"))),
			*PackageData.GetPackageName().ToString());
		return EPollStatus::Error;
	}

	if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_CallGetPostMoveObjects)
	{
		bool bFoundNewObjects;
		EPollStatus Result = Info.RefreshPackageObjects(GenerationHelper, Package, bFoundNewObjects,
			ESaveSubState::Last);
		if (Result != EPollStatus::Success)
		{
			return Result;
		}
		PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_CallGetPostMoveObjects);
	}

	EPollStatus Result = CallBeginCacheOnObjects(PackageData, Package, PackageData.GetCachedObjectsInOuter(),
		PackageData.GetCookedPlatformDataNextIndex(), Timer);
	if (PackageData.GetNumPendingCookedPlatformData() > 0 &&
		!GenerationHelper.GetCookPackageSplitterInstance()->UseInternalReferenceToAvoidGarbageCollect() &&
		!Info.HasIssuedUndeclaredMovedObjectsWarning())
	{
		UObject* FirstPendingObject = nullptr;
		FString FirstPendingObjectName;
		PackageDatas->ForEachPendingCookedPlatformData([&PackageData, &FirstPendingObject, &FirstPendingObjectName]
		(const FPendingCookedPlatformData& Pending)
		{
			if (&Pending.PackageData == &PackageData)
			{
				FString ObjectName = Pending.Object.IsValid() ? Pending.Object.Get()->GetPathName() : TEXT("");
				if (ObjectName.Len() && (!FirstPendingObject || ObjectName < FirstPendingObjectName))
				{
					FirstPendingObject = Pending.Object.Get();
					FirstPendingObjectName = MoveTemp(ObjectName);
				}
			}
		});
		UE_LOGF(LogCook, Warning,
			"CookPackageSplitter created or moved objects during %ls that are not yet ready to save. This will cause an error if garbage collection runs before the package is saved.\n"
			"Change the splitter's %ls to construct new objects and declare existing objects that will be moved from other packages.\n"
			"SplitterObject: %ls%ls\n"
			"NumPendingObjects: %d, FirstPendingObject: %ls",
			Info.IsGenerator() ? TEXT("PreSaveGeneratorPackage") : TEXT("PreSaveGeneratedPackage"),
			Info.IsGenerator() ? TEXT("PopulateGeneratorPackage") : TEXT("PopulateGeneratedPackage"),
			*SplitDataObject->GetFullName(),
			Info.IsGenerator() ? TEXT("") : *FString::Printf(TEXT("\nGeneratedPackage: %s"), *PackageData.GetPackageName().ToString()),
			PackageData.GetNumPendingCookedPlatformData(),
			FirstPendingObject ? *FirstPendingObject->GetFullName() : TEXT("<unknown>"));
		Info.SetHasIssuedUndeclaredMovedObjectsWarning(true);
	}
	if (Result != EPollStatus::Success)
	{
		return Result;
	}

	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::TryPreSaveGeneratedPackage(UE::Cook::FGenerationHelper& GenerationHelper,
	UE::Cook::FCookGenerationInfo& GeneratedInfo)
{
	using namespace UE::Cook;

	UE::Cook::FPackageData& GeneratedPackageData = *GeneratedInfo.PackageData;
	const FString GeneratedPackageName = GeneratedPackageData.GetPackageName().ToString();
	UPackage* OwnerPackage = GenerationHelper.FindOrLoadOwnerPackage(*this);
	if (!OwnerPackage)
	{
		UE_LOGF(LogCook, Error,
			"TryPreSaveGeneratedPackage: could not load ParentGeneratorPackage %ls for GeneratedPackage %ls",
			*GenerationHelper.GetOwner().GetPackageName().ToString(), *GeneratedPackageName);
		return EPollStatus::Error;
	}
	UPackage* GeneratedPackage = GeneratedPackageData.GetPackage();
	check(GeneratedPackage); // We would have been kicked out of save if the package were gone

	UObject* OwnerObject = GenerationHelper.FindOrLoadSplitDataObject();
	if (!OwnerObject)
	{
		UE_LOGF(LogCook, Error,
			"TryPreSaveGeneratedPackage could not find the original splitting object. Generated package can not be created. Splitter=%ls, Generated=%ls.",
			*GenerationHelper.GetSplitDataObjectName().ToString(), *GeneratedPackageName);
		return EPollStatus::Error;
	}

	ICookPackageSplitter* Splitter = GenerationHelper.GetCookPackageSplitterInstance();

	// Call the CookPackageSplitterInstance's PreSave hook for the package and pass name and other information about
	// the GeneratedPackage so the splitter can properly setup any internal reference to this package (e.g.
	// SoftObjectPath to the package).
	ICookPackageSplitter::FGeneratedPackageForPopulate GeneratedPackagePopulateData;
	GeneratedPackagePopulateData.RelativePath = GeneratedInfo.RelativePath;
	GeneratedPackagePopulateData.GeneratedRootPath = GeneratedInfo.GeneratedRootPath;
	GeneratedPackagePopulateData.Package = GeneratedPackage;
	GeneratedPackagePopulateData.bCreatedAsMap = GeneratedInfo.IsCreateAsMap();
	UE::Cook::CookPackageSplitter::FPopulateContextData PopulateData;
	{
		UE::Cook::FScopedCookActivePackage ScopedActivePackage(GenerationHelper.GetOwner().GetPackageName(),
#if UE_WITH_OBJECT_HANDLE_TRACKING
			PackageAccessTrackingOps::NAME_CookerBuildObject
#else
			FName()
#endif
		);
		TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

		ICookPackageSplitter::FPopulateContext PopulateContext(PopulateData);
		PopulateData.OwnerPackage = OwnerPackage;
		PopulateData.OwnerObject = OwnerObject;
		PopulateData.TargetGeneratedPackage = &GeneratedPackagePopulateData;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		bool bPreSaveSucceeded = Splitter->PreSaveGeneratedPackage(PopulateData.OwnerPackage, PopulateData.OwnerObject,
			*PopulateData.TargetGeneratedPackage, PopulateData.KeepReferencedPackages);
		Splitter->WarnIfDeprecatedVirtualNotCalled(TEXT("PreSaveGeneratedPackage"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		bPreSaveSucceeded = Splitter->PreSaveGeneratedPackage(PopulateContext) && bPreSaveSucceeded;
		if (!bPreSaveSucceeded)
		{
			UE_LOGF(LogCook, Error, "PackageSplitter returned false from PreSaveGeneratedPackage. Splitter=%ls, Generated=%ls.",
				*GenerationHelper.GetSplitDataObjectName().ToString(), *GeneratedPackageName);
			return EPollStatus::Error;
		}
	}
	GeneratedInfo.AddKeepReferencedPackages(GenerationHelper, PopulateData.KeepReferencedPackages);
	GeneratedInfo.BuildResultDependencies.Append(MoveTemp(PopulateData.BuildResultDependencies));

	bool bPackageIsMap = GeneratedPackage->ContainsMap();
	if (bPackageIsMap != GeneratedInfo.IsCreateAsMap())
	{
		UE_LOGF(LogCook, Error,
			"PackageSplitter specified generated package is %ls in ReportGenerationManifest, but then in PreSaveGeneratedPackage created it as %ls. Splitter=%ls, Generated=%ls.",
			(GeneratedInfo.IsCreateAsMap() ? TEXT("map") : TEXT("uasset")), (bPackageIsMap ? TEXT("map") : TEXT("uasset")),
			*GenerationHelper.GetSplitDataObjectName().ToString(), *GeneratedPackageName);
		return EPollStatus::Error;
	}

	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::PrepareSave(UE::Cook::FPackageData& PackageData,
	UE::Cook::FCookerTimer& Timer, bool bPrecaching, UE::Cook::ESuppressCookReason& OutDemotionRequestedReason)
{
	using namespace UE::Cook;

	EPollStatus Result = EPollStatus::Incomplete;
	OutDemotionRequestedReason = ESuppressCookReason::NotSuppressed;
	if (PackageData.GetSaveSubState() == ESaveSubState::ReadyForSave)
	{
		Result = EPollStatus::Success;
	}
	else if (PackageData.HasPrepareSaveFailed())
	{
		Result = EPollStatus::Error;
	}
	else
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(PrepareSave, DetailedCookStats::TickCookOnTheSidePrepareSaveTimeSec);
		Result = PrepareSaveInternal(PackageData, Timer, bPrecaching, OutDemotionRequestedReason);
		if (Result == EPollStatus::Error)
		{
			PackageData.SetHasPrepareSaveFailed(true);
		}
	}

	if (Result == EPollStatus::Success && PackageData.GetIsCookLast())
	{
		// No longer urgent
		PackageData.SetUrgency(EUrgency::Normal, ESendFlags::QueueAddAndRemove);
		// Mark it as still not ready if there are non-cook-last packages still in progress
		if (PackageDatas->GetMonitor().GetNumInProgress() - PackageDatas->GetMonitor().GetNumCookLast() > 0)
		{
			Result = EPollStatus::Incomplete;
		}
		else
		{
			UE_LOGF(LogCook, Display, "CookLast: All other packages cooked. Releasing %ls.", *PackageData.GetPackageName().ToString());
		}
	}

	return Result;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::PrepareSaveInternal(UE::Cook::FPackageData& PackageData,
	UE::Cook::FCookerTimer& Timer, bool bPrecaching, UE::Cook::ESuppressCookReason& OutDemotionRequestedReason)
{
	using namespace UE::Cook;

#if DEBUG_COOKONTHEFLY 
	UE_LOGF(LogCook, Display, "Caching objects for package %ls", *PackageData.GetPackageName().ToString());
#endif
	UPackage* Package = PackageData.GetPackage();
	check(Package && Package->IsFullyLoaded());
	check(PackageData.GetState() == EPackageState::SaveActive);
	TRefCountPtr<FGenerationHelper> GenerationHelper;

	if (PackageData.GetSaveSubState() < ESaveSubState::CheckForIsGenerated)
	{
		if (PackageData.GetSaveSubState() <= ESaveSubState::StartSave)
		{
			if (PackageData.GetNumPendingCookedPlatformData() > 0)
			{
				// A previous Save was started and demoted after some calls to BeginCacheForCookedPlatformData
				// occurred, and some of those objects have still not returned true for
				// IsCachedCookedPlatformDataLoaded. We were keeping them around to call Clear on them after they
				// return true from IsCachedCooked before we call Begin on them again. But depending on their state
				// after a garbage collect, they might now never return true. So rather than blocking the BeginCache
				// calls, clear the cancel manager and call BeginCache on them again even though they never returned
				// true from IsCached. The contract for BeginCacheCookedPlatformData and
				// IsCachedCookedPlatformDataLoaded includes the provision that is valid for the cooker to call
				// BeginCacheCookedPlatformData multiple times before IsCachedCookedPlatformData is returns true, and
				// the object should remain in a valid state (possibly reset to the beginning of its async work)
				// afterwards and still eventually return true from a future IsCachedCookedPlatformData call.
				PackageDatas->ClearCancelManager(PackageData);
				if (PackageData.GetNumPendingCookedPlatformData() > 0)
				{
					UE_LOGF(LogCook, Error,
						"CookerBug: Package %ls is blocked from entering save due to GetNumPendingCookedCookedPlatformData() == %d.",
						*PackageData.GetPackageName().ToString(), PackageData.GetNumPendingCookedPlatformData());
					return EPollStatus::Error;
				}
			}
			PackageData.SetSaveSubStateComplete(ESaveSubState::StartSave);
		}

		if (PackageData.GetSaveSubState() <= ESaveSubState::FirstCookedPlatformData_CreateObjectCache)
		{
			PackageData.CreateObjectCache();
			PackageData.SetSaveSubStateComplete(ESaveSubState::FirstCookedPlatformData_CreateObjectCache);
		}

		if (PackageData.GetSaveSubState() <= ESaveSubState::FirstCookedPlatformData_CallingBegin)
		{
			// Note that we cache cooked data for all requested platforms, rather than only for the requested platforms that have not cooked yet.  This allows
			// us to avoid the complexity of needing to cancel the Save and keep track of the old list of uncooked platforms whenever the cooked platforms change
			// while PrepareSave is active.
			// Currently this does not cause significant cost since saving new platforms with some platforms already saved is a rare operation.

			int32& CookedPlatformDataNextIndex = PackageData.GetCookedPlatformDataNextIndex();
			if (CookedPlatformDataNextIndex < 0)
			{
				if (!BuildDefinitions->TryRemovePendingBuilds(PackageData.GetPackageName()))
				{
					// Builds are in progress; wait for them to complete
					return EPollStatus::Incomplete;
				}
				CookedPlatformDataNextIndex = 0;
			}

			TArray<FCachedObjectInOuter>& CachedObjectsInOuter = PackageData.GetCachedObjectsInOuter();
			EPollStatus Result = CallBeginCacheOnObjects(PackageData, Package, CachedObjectsInOuter,
				CookedPlatformDataNextIndex, Timer);
			if (Result != EPollStatus::Success)
			{
				return Result;
			}

			PackageData.SetSaveSubStateComplete(ESaveSubState::FirstCookedPlatformData_CallingBegin);
		}

		if (PackageData.GetSaveSubState() <= ESaveSubState::FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded)
		{
			bool bCookedPlatformDataIsLoaded = PackageData.GetNumPendingCookedPlatformData() == 0;
			bool bWaitingForIsLoaded = PackageData.GetSaveSubState() > ESaveSubState::FirstCookedPlatformData_CheckForGenerator;
			if (bWaitingForIsLoaded && !bCookedPlatformDataIsLoaded)
			{
				return EPollStatus::Incomplete;
			}

			// Check for whether the Package has a Splitter and initialize its list if so
			// The GenerationHelper might have already been created by a child generated package;
			// or it might have been created and not initialized by incremental cook startup.
			// If not created or initialized, try looking for it
			bool bNeedWaitForIsLoaded = false;
			GenerationHelper = PackageData.TryCreateValidGenerationHelper(bCookedPlatformDataIsLoaded, bNeedWaitForIsLoaded);
			if (!GenerationHelper && bNeedWaitForIsLoaded)
			{
				// bNeedWaitForIsLoaded can only be set to true if we pass in !bCookedPlatformDataIsLoaded, and that can only happen
				// if !bWaitingForIsLoaded, due to the early exit above.
				check(!bWaitingForIsLoaded);
				PackageData.SetSaveSubState(ESaveSubState::FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded);
				return EPollStatus::Incomplete;
			}
			PackageData.SetSaveSubStateComplete(ESaveSubState::FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded);
		}
		else
		{
			GenerationHelper = PackageData.GetGenerationHelperIfValid();
		}

		if (GenerationHelper)
		{
			if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_TryReportGenerationManifest)
			{
				// Keep it referenced even if we are only precaching, so we do not recreate it
				GenerationHelper->SetKeepForGeneratorSaveAllPlatforms();
				if (bPrecaching)
				{
					// Do not proceed to ReportGenerationManifest when precaching; do that only when we're ready to save the package
					return EPollStatus::Incomplete;
				}
				else
				{
					// TODO: Add support for cooking in the editor. Possibly moot since we plan to deprecate cooking in the editor.
					if (IsCookingInEditor())
					{
						// CookPackageSplitters allow destructive changes to the generator package. e.g. moving UObjects out
						// of it into the streaming packages. To allow its use in the editor, we will need to make it non-destructive
						// (by e.g. copying to new packages), or restore the package after the changes have been made.
						UE_LOGF(LogCook, Error, "Can not cook package %ls: cooking in editor doesn't support Cook Package Splitters.",
							*PackageData.GetPackageName().ToString());
						return EPollStatus::Error;
					}
					// TODO_COOKGENERATIONHELPER: We don't currently support separate cooking for one platform but not
					// another for a generated package, see the class comment on FGenerationHelper. Therefore if any
					// platform is unreachable, send this package back to the request state to add the other platforms.
					TConstArrayView<const ITargetPlatform*> SessionPlatforms = PlatformManager->GetSessionPlatforms();
					EReachability CurrentReachability = GetCookPhase() == ECookPhase::Cook
						? EReachability::Runtime : EReachability::Build;
					if (!PackageData.HasReachablePlatforms(CurrentReachability, SessionPlatforms))
					{
						OutDemotionRequestedReason = ESuppressCookReason::GeneratedPackageNeedsRequestUpdate;
						return EPollStatus::Incomplete;
					}

					if (!GenerationHelper->TryReportGenerationManifest())
					{
						return EPollStatus::Error;
					}
					GenerationHelper->StartOwnerSave();
					PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_TryReportGenerationManifest);
				}
			}

			if (PackageData.GetSaveSubState() <= ESaveSubState::Generation_QueueGeneratedPackages)
			{
				EPollStatus Result = QueueGeneratedPackages(*GenerationHelper, PackageData);
				if (Result != EPollStatus::Success)
				{
					return Result;
				}
				PackageData.SetSaveSubStateComplete(ESaveSubState::Generation_QueueGeneratedPackages);
			}
		}
		else
		{
			PackageData.SetSaveSubState(ESaveSubState::CheckForIsGenerated);
		}
	}
	else
	{
		GenerationHelper = PackageData.GetGenerationHelperIfValid();
	}

	if (PackageData.GetSaveSubState() < ESaveSubState::ReadyForSave)
	{
		if (GenerationHelper)
		{
			EPollStatus Result = PrepareSaveGenerationPackage(*GenerationHelper, PackageData, Timer, bPrecaching);
			if (Result != EPollStatus::Success)
			{
				return Result;
			}
		}
		else if (PackageData.IsGenerated())
		{
			TRefCountPtr<FGenerationHelper> ParentGenerationHelper = PackageData.GetParentGenerationHelper();
			if (!ParentGenerationHelper || !ParentGenerationHelper->IsValid())
			{
				UE_LOGF(LogCook, Error, "Generated package %ls %ls ParentGenerator package %ls and cannot be saved.",
					(!ParentGenerationHelper ? TEXT("is missing its") : TEXT("has an invalid")),
					*PackageData.GetPackageName().ToString(), *PackageData.GetParentGenerator().ToString());
				return EPollStatus::Error;
			}

			EPollStatus Result = PrepareSaveGenerationPackage(*ParentGenerationHelper, PackageData, Timer, bPrecaching);
			if (Result != EPollStatus::Success)
			{
				return Result;
			}
		}
		else
		{
			if (PackageData.GetSaveSubState() <= ESaveSubState::CheckForIsGenerated)
			{
				// Skip over the LastCookedPlatformData_CallingBegin state; we only need to enter that state if
				// RefreshObjectCache finds some new objects, and for this block (non-generation packages that reach
				// CheckForIsGenerated for the first time) we have not called RefreshObjectCache yet.
				PackageData.SetSaveSubState(ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded);
			}

			if (PackageData.GetSaveSubState() <= ESaveSubState::LastCookedPlatformData_CallingBegin)
			{
				int32& CookedPlatformDataNextIndex = PackageData.GetCookedPlatformDataNextIndex();
				TArray<FCachedObjectInOuter>& CachedObjectsInOuter = PackageData.GetCachedObjectsInOuter();
				EPollStatus Result = CallBeginCacheOnObjects(PackageData, Package, CachedObjectsInOuter,
					CookedPlatformDataNextIndex, Timer);
				if (Result != EPollStatus::Success)
				{
					return Result;
				}
				PackageData.SetSaveSubStateComplete(ESaveSubState::LastCookedPlatformData_CallingBegin);
			}

			if (PackageData.GetSaveSubState() <= ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded)
			{
				if (PackageData.GetNumPendingCookedPlatformData() > 0)
				{
					return EPollStatus::Incomplete;
				}
				bool bFoundNewObjects;
				EPollStatus Result = PackageData.RefreshObjectCache(bFoundNewObjects);
				if (Result != EPollStatus::Success)
				{
					return Result;
				}
				if (bFoundNewObjects)
				{
					PackageData.SetSaveSubState(ESaveSubState::LastCookedPlatformData_CallingBegin);
					// Call this function recursively to immediately reexecute CallBeginCacheOnObjects.
					// Note that RefreshObjectCache checked for too many recursive calls and ErrorExited if so.
					return PrepareSaveInternal(PackageData, Timer, bPrecaching, OutDemotionRequestedReason);
				}
				else
				{
					PackageData.SetSaveSubStateComplete(ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded);
				}
			}
		}
	}

	check(PackageData.GetSaveSubState() == ESaveSubState::ReadyForSave);
	check(PackageData.GetNumPendingCookedPlatformData() == 0);
	return EPollStatus::Success;
}

UE::Cook::EPollStatus UCookOnTheFlyServer::CallBeginCacheOnObjects(UE::Cook::FPackageData& PackageData,
	UPackage* Package, TArray<UE::Cook::FCachedObjectInOuter>& Objects, int32& NextIndex, UE::Cook::FCookerTimer& Timer)
{
	using namespace UE::Cook;

	check(Package);

	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> TargetPlatforms;
	PackageData.GetCachedObjectsInOuterPlatforms(TargetPlatforms);

	FCachedObjectInOuter* ObjectsData = Objects.GetData();
	int NumObjects = Objects.Num();
	for (; NextIndex < NumObjects; ++NextIndex)
	{
		UObject* Obj = ObjectsData[NextIndex].Object.Get();
		if (!Obj)
		{
			// Objects can be marked as pending kill even without a garbage collect, and our weakptr.get will return
			// null for them, so we have to always check the WeakPtr before using it.
			// Treat objects that have been marked as pending kill or deleted as no-longer-required for
			// BeginCacheForCookedPlatformData and ClearAllCachedCookedPlatformData
			// In case the weakptr is merely pendingkill, set it to null explicitly so we don't think that we've called
			// BeginCacheForCookedPlatformData on it if it gets unmarked pendingkill later
			ObjectsData[NextIndex].Object = nullptr;
			continue;
		}
		FCachedCookedPlatformDataState& CCPDState = PackageDatas->GetCachedCookedPlatformDataObjects().FindOrAdd(Obj);
		CCPDState.AddRefFrom(&PackageData);

		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			ECachedCookedPlatformDataEvent& ExistingEvent =
				CCPDState.PlatformStates.FindOrAdd(TargetPlatform, ECachedCookedPlatformDataEvent::None);
			if (ExistingEvent != ECachedCookedPlatformDataEvent::None)
			{
				continue;
			}

			if (Obj->IsA(UMaterialInterface::StaticClass()))
			{
				if (GShaderCompilingManager->GetNumRemainingJobs() + 1 > MaxConcurrentShaderJobs)
				{
#if DEBUG_COOKONTHEFLY
					UE_LOGF(LogCook, Display, "Delaying shader compilation of material %ls", *Obj->GetFullName());
#endif
					return EPollStatus::Incomplete;
				}
			}

			const FName ClassFName = Obj->GetClass()->GetFName();
			int32* CurrentAsyncCache = CurrentAsyncCacheForType.Find(ClassFName);
			if (CurrentAsyncCache != nullptr)
			{
				if (*CurrentAsyncCache < 1)
				{
					return EPollStatus::Incomplete;
				}
				*CurrentAsyncCache -= 1;
			}

			RouteBeginCacheForCookedPlatformData(PackageData, Obj, TargetPlatform, &ExistingEvent);
			if (RouteIsCachedCookedPlatformDataLoaded(PackageData, Obj, TargetPlatform, &ExistingEvent))
			{
				if (CurrentAsyncCache)
				{
					*CurrentAsyncCache += 1;
				}
			}
			else
			{
				bool bNeedsResourceRelease = CurrentAsyncCache != nullptr;
				PackageDatas->AddPendingCookedPlatformData(FPendingCookedPlatformData(Obj, TargetPlatform,
					PackageData, bNeedsResourceRelease, *this));
				TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper();
				if (GenerationHelper)
				{
					GenerationHelper->OnAddPendingCookedPlatformData(*this, PackageData, Obj);
				}
			}

			if (Timer.IsActionTimeUp())
			{
#if DEBUG_COOKONTHEFLY
				UE_LOGF(LogCook, Display, "Object %ls took too long to cache", *Obj->GetFullName());
#endif
				return EPollStatus::Incomplete;
			}
		}
	}

	return EPollStatus::Success;
}

void UCookOnTheFlyServer::ReleaseCookedPlatformData(UE::Cook::FPackageData& PackageData,
	UE::Cook::EStateChangeReason ReleaseSaveReason, UE::Cook::EPackageState NewState)
{
	using namespace UE::Cook;

	TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper();
	if (!GenerationHelper)
	{
		GenerationHelper = PackageData.GetParentGenerationHelper();
	}
	FCookGenerationInfo* GenerationInfo = (GenerationHelper && GenerationHelper->IsInitialized()) ?
		GenerationHelper->FindInfo(PackageData) : nullptr;

	// For every BeginCacheForCookedPlatformData call we made we need to call ClearAllCachedCookedPlatformData
	// No need to check for CookedPlatformData if in the StartSave state; we can not have any in that case
	if (PackageData.GetSaveSubState() > ESaveSubState::StartSave)
	{
		if (ReleaseSaveReason == EStateChangeReason::Completed)
		{
			// Since we have completed CookedPlatformData, we know we called BeginCacheForCookedPlatformData on all
			// objects in the package, and none are pending
			UE_SCOPED_HIERARCHICAL_COOKTIMER(ClearAllCachedCookedPlatformData);
			for (FCachedObjectInOuter& CachedObjectInOuter : PackageData.GetCachedObjectsInOuter())
			{
				UObject* Object = CachedObjectInOuter.Object.Get();
				if (Object)
				{
					FPendingCookedPlatformData::ClearCachedCookedPlatformData(Object, PackageData,
						true /* bCompletedSuccesfully */);
				}
			}
		}
		else
		{
			// This is a slower but more general flow that can handle releasing whether or not we called SavePackage
			// Note that even after we return from this function, some objects with pending IsCachedCookedPlatformDataLoaded
			// calls may still exist for this Package in PendingCookedPlatformDatas
			// and this PackageData may therefore still have GetNumPendingCookedPlatformData > 0
			// We have only called BeginCacheForCookedPlatformData on Object,Platform pairs up to GetCookedPlatformDataNextIndex.
			// Further, some of those calls might still be pending.

			// Find all pending BeginCacheForCookedPlatformData for this FPackageData
			TMap<UObject*, TArray<FPendingCookedPlatformData*>> PendingObjects;
			PackageDatas->ForEachPendingCookedPlatformData(
				[&PendingObjects, &PackageData](FPendingCookedPlatformData& PendingCookedPlatformData)
				{
					if (&PendingCookedPlatformData.PackageData == &PackageData && !PendingCookedPlatformData.PollIsComplete())
					{
						UObject* Object = PendingCookedPlatformData.Object.Get();
						check(Object); // Otherwise PollIsComplete would have returned true
						check(!PendingCookedPlatformData.bHasReleased); // bHasReleased should be false since PollIsComplete returned false
						PendingObjects.FindOrAdd(Object).Add(&PendingCookedPlatformData);
					}
				});


			// Iterate over all objects in the FPackageData up to GetCookedPlatformDataNextIndex
			TArray<FCachedObjectInOuter>& CachedObjects = PackageData.GetCachedObjectsInOuter();
			for (int32 ObjectIndex = 0; ObjectIndex < PackageData.GetCookedPlatformDataNextIndex(); ++ObjectIndex)
			{
				UObject* Object = CachedObjects[ObjectIndex].Object.Get();
				if (!Object)
				{
					continue;
				}
				TArray<FPendingCookedPlatformData*>* PendingDatas = PendingObjects.Find(Object);
				if (!PendingDatas || PendingDatas->Num() == 0)
				{
					// No pending BeginCacheForCookedPlatformData calls for this object; clear it now.
					FPendingCookedPlatformData::ClearCachedCookedPlatformData(Object, PackageData,
						false /* bCompletedSuccesfully */);
				}
				else
				{
					// For any pending Objects, we add a CancelManager to the FPendingCookedPlatformData to call
					// ClearAllCachedCookedPlatformData when the pending Object,Platform pairs for that object completes.
					FPendingCookedPlatformDataCancelManager* CancelManager = new FPendingCookedPlatformDataCancelManager();
					CancelManager->NumPendingPlatforms = PendingDatas->Num();
					for (FPendingCookedPlatformData* PendingCookedPlatformData : *PendingDatas)
					{
						// We never start a new package until after clearing the previous cancels, so all of the
						// FPendingCookedPlatformData for the PlatformData we are cancelling can not have been cancelled before.
						// We would leak the CancelManager if we overwrote it here.
						check(PendingCookedPlatformData->CancelManager == nullptr);
						// If bHasReleased on the PendingCookedPlatformData were already true, we would leak the CancelManager
						// because the PendingCookedPlatformData would never call Release on it.
						check(!PendingCookedPlatformData->bHasReleased);
						PendingCookedPlatformData->CancelManager = CancelManager;
					}
				}
			}
		}

		PackageData.ClearCookedPlatformData();
	}

	if (GenerationInfo)
	{
		GenerationHelper->ResetSaveState(*GenerationInfo, PackageData.GetPackage(), ReleaseSaveReason, NewState);
	}

	if (ReleaseSaveReason != EStateChangeReason::RecreateObjectCache)
	{
		if (!IsCookOnTheFlyMode() && !IsCookingInEditor())
		{
			UPackage* Package = PackageData.GetPackage();
			if (Package && Package->GetLinker())
			{
				// Loaders and their handles can have large buffers held in process memory and in the system file cache from the
				// data that was loaded.  Keeping this for the lifetime of the cook is costly, so we try and unload it here.
				Package->GetLinker()->FlushCache();
			}
		}
	}

	PackageData.SetSaveSubState(ESaveSubState::StartSave);
}

void UCookOnTheFlyServer::TickCancels()
{
	int32 UnusedNumRetired;
	PackageDatas->PollPendingCookedPlatformDatas(false, LastCookableObjectTickTime, UnusedNumRetired);
}

bool UCookOnTheFlyServer::LoadPackageForCooking(UE::Cook::FPackageData& PackageData, UPackage*& OutPackage,
	UE::Cook::FPackageData* ReportingPackageData)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(LoadPackageForCooking, DetailedCookStats::TickCookOnTheSideLoadPackagesTimeSec);
	UE::Cook::FScopedCookActivePackage ScopedActivePackage(PackageData.GetPackageName(), NAME_None);

	FString PackageName = PackageData.GetPackageName().ToString();
	OutPackage = FindObject<UPackage>(nullptr, *PackageName);

	FString FileName(PackageData.GetFileName().ToString());
	FString ReportingFileName(ReportingPackageData ? ReportingPackageData->GetFileName().ToString() : FileName);
#if DEBUG_COOKONTHEFLY
	UE_LOGF(LogCook, Display, "Processing request %ls", *ReportingFileName);
#endif
	TOptional<TInheritedGuardValueAccessors<GetOutputCookingWarnings, SetOutputCookingWarnings>> OutputCookingWarningsGuard;
	if (!PackageData.HasBeenLoadedForCooking())
	{
		PackageData.SetHasBeenLoadedForCooking(true);
		OutputCookingWarningsGuard.Emplace(IsCookFlagSet(ECookInitializationFlags::OutputVerboseCookerWarnings));
	}

	bool bSuccess = true;
	//  if the package is not yet fully loaded then fully load it
	if (!IsValid(OutPackage) || !OutPackage->IsFullyLoaded())
	{
		bool bWasPartiallyLoaded = OutPackage != nullptr;
		GIsCookerLoadingPackage = true;
		UPackage* LoadedPackage;
		{
			LLM_SCOPE(ELLMTag::Untagged); // Reset the scope so that untagged memory in the package shows up as Untagged rather than Cooker
#if ENABLE_COOK_STATS
			++DetailedCookStats::NumRequestedLoads;
#endif
			// Declare the package as being referenced by the ScriptPackageNameEngine; we look for this in
			// ProcessUnsolicitedPackages so we can avoid adding an error for the package being loaded without a
			// referencer.
			UE_TRACK_REFERENCING_PACKAGE_SCOPED(ScriptPackageNameEngine, PackageAccessTrackingOps::NAME_Load);
			LoadedPackage = LoadPackage(nullptr, *FileName, LOAD_None);
		}
		if (IsValid(LoadedPackage) && LoadedPackage->IsFullyLoaded())
		{
			OutPackage = LoadedPackage;

			if (bWasPartiallyLoaded)
			{
				// If fully loading has caused a blueprint to be regenerated, make sure we eliminate all meta data outside the package
				FMetaData& MetaData = LoadedPackage->GetMetaData();
				MetaData.RemoveMetaDataOutsidePackage(LoadedPackage);
			}
		}
		else
		{
			bSuccess = false;
		}

		++this->StatLoadedPackageCount;

		GIsCookerLoadingPackage = false;
	}
#if DEBUG_COOKONTHEFLY
	else
	{
		UE_LOGF(LogCook, Display, "Package already loaded %ls avoiding reload", *ReportingFileName);
	}
#endif

	if (!bSuccess)
	{
		if ((!IsCookOnTheFlyMode()) || (!IsCookingInEditor()))
		{
			LogCookerMessage(FString::Printf(TEXT("Error loading %s!"), *ReportingFileName), EMessageSeverity::Error);
		}
	}
	return bSuccess;
}

void UCookOnTheFlyServer::ProcessUnsolicitedPackages(TArray<FName>* OutDiscoveredPackageNames,
	TMap<FName, UE::Cook::FInstigator>* OutInstigators)
{
	using namespace UE::Cook;

	if (bRunningAsShaderServer)
	{
		return;
	}

	if (GetCookPhase() != ECookPhase::Cook)
	{
		// We no longer add unsolicited packages to the cook when we have started committing build dependencies
		return;
	}

	auto AddToOutDiscovered = [OutDiscoveredPackageNames, OutInstigators](FPackageData* PackageData, const FInstigator& Instigator)
		{
			if (OutDiscoveredPackageNames)
			{
				if (!PackageData->IsInProgress())
				{
					FInstigator& Existing = OutInstigators->FindOrAdd(PackageData->GetPackageName());
					if (Existing.Category == EInstigator::InvalidCategory)
					{
						OutDiscoveredPackageNames->Add(PackageData->GetPackageName());
						Existing = Instigator;
					}
				}
			}
		};

	TArray<FPackageStreamEvent> PackageStream = PackageTracker->GetPackageStream();
	for (FPackageStreamEvent& PackageStreamEvent : PackageStream)
	{
		FName PackageName = PackageStreamEvent.PackageName;
		FInstigator& Instigator = PackageStreamEvent.Instigator;
		if (PackageStreamEvent.EventType == EPackageStreamEvent::InstancedPackageEndLoad)
		{
			continue;
		}
		check(PackageStreamEvent.EventType == EPackageStreamEvent::PackageLoad);

		if (Instigator.Referencer == ScriptPackageNameEngine)
		{
			// A load by the cooker, expected, no need to queue it as discovered.
			continue;
		}
		FPackageData* PackageData = PackageDatas->TryAddPackageDataByPackageName(PackageName);
		if (!PackageData)
		{
			// Some types of packages are non-cookable, and we ignore the loadevent of them: scriptpackages,
			// instanced load packages.
			continue;
		}
		if (PackageData->IsGenerated())
		{
			continue; // Generated packages are queued separately (with the correct instigator) in QueueGeneratedPackages
		}

		FPackageData* Referencer = nullptr;
		FName InstancedLeafLoadedName;
		if (!Instigator.Referencer.IsNone())
		{
			// Packages loaded by an instanced package referencer need to redirect the Instigator to be the Instigator of
			// the instanced package load, merged with the Instigator category based on the whether the loaded package is
			// a recorded dependency of the instanced package's LoadPath package.
			TRefCountPtr<const FPackageStreamInstancedPackage> Instance
				= PackageTracker->FindInstancedPackage(Instigator.Referencer);
			if (Instance)
			{
				InstancedLeafLoadedName = Instance->PackageName;
				Instigator.Referencer = Instance->Instigator.Referencer;
				if (Instigator.Category == EInstigator::Unsolicited)
				{
					using namespace UE::AssetRegistry;
					const EDependencyProperty* DependencyProperty = Instance->Dependencies.Find(PackageName);
					if (DependencyProperty)
					{
						Instigator.Category = EnumHasAnyFlags(*DependencyProperty, EDependencyProperty::Game)
							? EInstigator::HardDependency : EInstigator::EditorOnlyLoad;
					}
				}
				Instigator.Category = FPackageTracker::MergeReferenceCategories(
					Instance->Instigator.Category, Instigator.Category);
			}
			Referencer = PackageDatas->FindPackageDataByPackageName(Instigator.Referencer);
		}

		if (Instigator.Category == EInstigator::EditorOnlyLoad)
		{
			// This load was expected so we do not need to add a hidden dependency for it.
			// If we are using legacy WhatGetsCookedRules, queue the package for cooking because it was loaded.
			if (!bSkipOnlyEditorOnly)
			{
				QueueDiscoveredPackage(*PackageData, FInstigator(Instigator), EDiscoveredPlatformSet::CopyFromInstigator);
				AddToOutDiscovered(PackageData, Instigator);
			}
			continue;
		}

		AddToOutDiscovered(PackageData, Instigator);
		TStringBuilder<256> PackageNameStr(InPlace, PackageData->GetPackageName());

		if (!Referencer)
		{
			// Package loads after cook startup that were requested outside of a package operation are a bug unless
			// marked from code as EInstigator::EditorOnlyLoad using an
			// FCookLoadScope Scope(ECookLoadType::EditorOnly). Report it as a bug and add it to the cook.
			// TODO: Incremental cook will not include these packages; add it?
			if (Instigator.Category != EInstigator::StartupPackage &&
				Instigator.Category != EInstigator::StartupPackageCookLoadScope &&
				Instigator.Category != EInstigator::StartupSoftObjectPath)
			{
#if !NO_LOGGING
				if (!LogCook.IsSuppressed(UnexpectedLoadWarningSeverity))
				{
					FMsg::Logf(__FILE__, __LINE__, LogCook.GetCategoryName(), UnexpectedLoadWarningSeverity,
						TEXT("UnexpectedLoad: Package %s was loaded outside of any cook operation on a referencer package; we don't know why it was loaded. ")
						TEXT("Adding it to the current cook, but it will possibly not be found in future incremental cooks."),
						*PackageNameStr);
				}
#endif
			}
			QueueDiscoveredPackage(*PackageData, FInstigator(Instigator), EDiscoveredPlatformSet::CopyFromInstigator);
			continue;
		}

		if (bSkipOnlyEditorOnly
			&& Instigator.Category != EInstigator::ForceExplorableSaveTimeSoftDependency
			&& PackageTracker->NeverCookPackageList.Contains(PackageData->GetPackageName())
			&& (INDEX_NONE != UE::String::FindFirst(PackageNameStr,
				ULevel::GetExternalActorsFolderName(), ESearchCase::IgnoreCase)
				|| INDEX_NONE != UE::String::FindFirst(PackageNameStr,
					FPackagePath::GetExternalObjectsFolderName(), ESearchCase::IgnoreCase)))
		{
			// ONLYEDITORONLY_TODO: WorldPartition should mark these loads as ForceExplorableSaveTimeSoftDependency
			// rather than needing to use a naming convention. We should also mark them once, during Save, rather than
			// marking them and reexploring them every time they are loaded. 
			Instigator.Category = EInstigator::ForceExplorableSaveTimeSoftDependency;
		}

		// If we have already reported this discovery from the same referencer, at the same or lower edge priority,
		// then there is no need to report it again so early exit to skip some unnecessary work.
		TMap<FPackageData*, EInstigator>* DiscoveredDependencies = Referencer->GetDiscoveredDependencies(
			nullptr /* PlatformAgnosticTargetPlatform */);
		if (DiscoveredDependencies)
		{
			EInstigator* ExistingInstigator = DiscoveredDependencies->Find(PackageData);
			if (ExistingInstigator)
			{
				if (Instigator.Category != EInstigator::ForceExplorableSaveTimeSoftDependency 
					|| *ExistingInstigator == EInstigator::ForceExplorableSaveTimeSoftDependency)
				{
					continue;
				}
			}
		}

		if (Instigator.Category == EInstigator::Unsolicited)
		{
			// If it comes from an import that was declared to the assetregistry, then it is expected, and
			// CookRequestCluster already handled or will handle adding it if necessary, and we can ignore it here.
			if (AssetRegistry->ContainsDependency(Instigator.Referencer, PackageName,
				UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard))
			{
				continue;
			}

			// Otherwise, it is a bug; unsolicited loads are not allowed. Report it, and add it as a
			// discovered dependency.
			FString DescriptionOfReferencer;
			if (InstancedLeafLoadedName.IsNone())
			{
				DescriptionOfReferencer = Instigator.Referencer.ToString();
			}
			else
			{
				DescriptionOfReferencer = FString::Printf(TEXT("%s (an instanced package load triggered by %s)"),
					*InstancedLeafLoadedName.ToString(), *Instigator.Referencer.ToString());
			}
#if !NO_LOGGING
			if (!LogCook.IsSuppressed(UnexpectedLoadWarningSeverity))
			{
				FMsg::Logf(__FILE__, __LINE__, LogCook.GetCategoryName(), UnexpectedLoadWarningSeverity,
					TEXT("UnexpectedLoad: TargetPackage %s was unexpectededly loaded during the cook of SourcePackage %s; it was not declared as a hard dependency (aka Import) of the SourcePackage. ")
					TEXT("We have to conservatively add this package to the cook, but it might not be needed at runtime. ")
					TEXT("Declare the TargetPackage as an import during editor save of the SourcePackage, or mark up its load with FCookLoadScope to specify whether it is runtime or editoronly."),
					*PackageNameStr, *DescriptionOfReferencer);
			}
#endif
		}

		Referencer->AddDiscoveredDependency(EDiscoveredPlatformSet::CopyFromInstigator, PackageData, Instigator.Category);
		QueueDiscoveredPackage(*PackageData, MoveTemp(Instigator), EDiscoveredPlatformSet::CopyFromInstigator);
	}
}

void UCookOnTheFlyServer::PumpSaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength,
	int32& OutNumPushed, bool& bOutBusy, int32& OutNumAsyncWorkRetired)
{
	if (GetCookPhase() == UE::Cook::ECookPhase::Cook)
	{
		PumpRuntimeSaves(StackData, DesiredQueueLength, OutNumPushed, bOutBusy, OutNumAsyncWorkRetired);
	}
	else
	{
		// After we enter the build dependencies phase, we no longer save packages,
		// we just commit them with dependencies but no cooked data.
		PumpBuildDependencySaves(StackData, DesiredQueueLength, OutNumPushed, bOutBusy, OutNumAsyncWorkRetired);
	}
}

void UCookOnTheFlyServer::PumpRuntimeSaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength,
	int32& OutNumPushed, bool& bOutBusy, int32& OutNumAsyncWorkRetired)
{
	using namespace UE::Cook;
	OutNumPushed = 0;
	OutNumAsyncWorkRetired = 0;
	bOutBusy = false;

	UE_SCOPED_HIERARCHICAL_COOKTIMER(SavingPackages);
	check(IsInGameThread());
	ON_SCOPE_EXIT
	{
		PumpHasExceededMaxMemory(StackData.ResultFlags);
	};

	// save as many packages as we can during our time slice
	FPackageDataQueue& SaveQueue = PackageDatas->GetSaveQueue();
	const uint32 OriginalPackagesToSaveCount = SaveQueue.Num();
	uint32 HandledCount = 0;
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> PlatformsForPackage;
	COOK_STAT(DetailedCookStats::PeakSaveQueueSize = FMath::Max(DetailedCookStats::PeakSaveQueueSize, SaveQueue.Num()));
	while (SaveQueue.Num() > static_cast<int32>(DesiredQueueLength))
	{
		FPackageData& PackageData(*SaveQueue.PopFrontValue());
		if (TryCreateRequestCluster(PackageData))
		{
			continue;
		}

		FPoppedPackageDataScope PoppedScope(PackageData);
		UPackage* Package = PackageData.GetPackage();

		check(Package != nullptr);
		++HandledCount;

#if DEBUG_COOKONTHEFLY
		UE_LOGF(LogCook, Display, "Processing save for package %ls", *Package->GetName());
#endif

		// Cook only the session platforms that have not yet been cooked for the given package
		PackageData.GetPlatformsNeedingCommit(PlatformsForPackage, GetCookPhase());
		if (PlatformsForPackage.Num() == 0)
		{
			// We've already saved all possible platforms for this package; this should not be possible.
			// All places that add a package to the save queue check for existence of incomplete platforms before adding
			UE_LOGF(LogCook, Warning, "Package '%ls' in SaveQueue has no more platforms left to cook; this should not be possible!", *PackageData.GetFileName().ToString());
			DemoteToIdle(PackageData, ESendFlags::QueueAdd, ESuppressCookReason::AlreadyCooked);
			++OutNumPushed;
			continue;
		}

		bool bShouldExitPump = false;
		if (IsCookOnTheFlyMode())
		{
			if (IsUsingLegacyCookOnTheFlyScheduling() && PackageData.GetUrgency() != EUrgency::Blocking)
			{
				if (WorkerRequests->HasExternalRequests() || PackageDatas->GetMonitor().GetNumUrgent(EUrgency::Blocking) > 0)
				{
					bShouldExitPump = true;
				}
				if (StackData.Timer.IsActionTimeUp())
				{
					// our timeslice is up
					bShouldExitPump = true;
				}
			}
			else
			{
				if (IsRealtimeMode())
				{
					if (StackData.Timer.IsActionTimeUp())
					{
						// our timeslice is up
						bShouldExitPump = true;
					}
				}
				else
				{
					// if we are cook on the fly and not in the editor then save the requested package as fast as we can because the client is waiting on it
					// Until we are blocked on async work, ignore the timer
				}
			}
		}
		else // !IsCookOnTheFlyMode
		{
			if (StackData.Timer.IsActionTimeUp())
			{
				// our timeslice is up
				bShouldExitPump = true;
			}
		}
		if (bShouldExitPump)
		{
			SaveQueue.AddFront(&PackageData);
			return;
		}

		// Release any completed pending CookedPlatformDatas, so that slots in the per-class limits on calls to BeginCacheForCookedPlatformData are freed up for new objects to use
		bool bForce = IsCookOnTheFlyMode() && !IsRealtimeMode();
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(PollPendingCookedPlatformDatas);
			int32 NumAsyncWorkRetired;
			PackageDatas->PollPendingCookedPlatformDatas(bForce, LastCookableObjectTickTime, NumAsyncWorkRetired);
			OutNumAsyncWorkRetired += NumAsyncWorkRetired;
		}

		// If BeginCacheCookPlatformData is not ready then postpone the package, exit, or wait for it as appropriate
		ESuppressCookReason DemotionReason;
		EPollStatus PrepareSaveStatus = PrepareSave(PackageData, StackData.Timer, false /* bPrecaching */, DemotionReason);
		if (PrepareSaveStatus != EPollStatus::Success)
		{
			if (PrepareSaveStatus == EPollStatus::Error)
			{
				check(PackageData.HasPrepareSaveFailed()); // Should have been set by PrepareSave; we rely on this for cleanup
				ReleaseCookedPlatformData(PackageData, EStateChangeReason::SaveError, EPackageState::Idle);
				PackageData.SetPlatformsCooked(PlatformsForPackage, ECookResult::Failed);
				DemoteToIdle(PackageData, ESendFlags::QueueAdd, ESuppressCookReason::SaveError);
				++OutNumPushed;
				continue;
			}
			if (DemotionReason != ESuppressCookReason::NotSuppressed)
			{
				DemoteToRequest(PackageData, ESendFlags::QueueAdd, DemotionReason);
				++OutNumPushed;
				continue;
			}

			// GC is required
			if (PackageData.IsPrepareSaveRequiresGC())
			{
				// We consume the requiresGC; it will not trigger GC again unless set again
				PackageData.SetIsPrepareSaveRequiresGC(false);
				StackData.ResultFlags |= COSR_RequiresGC | COSR_YieldTick;
				SaveQueue.AddFront(&PackageData);
				return;
			}

			// Can we postpone?
			if (PackageData.GetUrgency() != EUrgency::Blocking)
			{
				bool HasCheckedAllPackagesAreCached = HandledCount >= OriginalPackagesToSaveCount;
				if (!HasCheckedAllPackagesAreCached)
				{
					SaveQueue.Add(&PackageData);
					continue;
				}
			}
			else
			{
				// Move the PackageData to the end of the urgent packages in the SaveQueue but before all of
				// the non-urgent packages.
				TArray<FPackageData*> UrgentSaves;
				TArray<FPackageData*> NonUrgentSaves;
				UrgentSaves.Reserve(SaveQueue.Num());
				NonUrgentSaves.Reserve(SaveQueue.Num());
				for (FPackageData* Next : SaveQueue)
				{
					((Next->GetUrgency() == EUrgency::Blocking) ? &UrgentSaves : &NonUrgentSaves)->Add(Next);
				}
				// We don't have a count of the Original number of urgent packages, so just keep cycling through all
				// blocked urgent packages up to OriginalPackagesToSaveCount times. But abort that cycling if there are
				// no other urgent packages left in the SaveQueue.
				bool HasCheckedAllPackagesAreCached = HandledCount >= OriginalPackagesToSaveCount || UrgentSaves.Num() == 0;
				if (!HasCheckedAllPackagesAreCached)
				{
					SaveQueue.Reset();
					for (FPackageData* Next : UrgentSaves)
					{
						SaveQueue.Add(Next);
					}
					SaveQueue.Add(&PackageData);
					for (FPackageData* Next : NonUrgentSaves)
					{
						SaveQueue.Add(Next);
					}
					continue;
				}
			}

			// Should we wait?
			if (PackageData.GetUrgency() == EUrgency::Blocking && !IsRealtimeMode())
			{
				UE_SCOPED_HIERARCHICAL_COOKTIMER(WaitingForCachedCookedPlatformData);
				do
				{
					// PrepareSave might block on pending CookedPlatformDatas, and it might block on resources held by other
					// CookedPlatformDatas. Calling PollPendingCookedPlatformDatas should handle pumping all of those.
					if (!PackageDatas->GetPendingCookedPlatformDataNum())
					{
						// We're waiting on something other than pendingcookedplatformdatas; this loop does not yet handle
						// updating anything else, so break out
						break;
					}
					// sleep for a bit
					FPlatformProcess::Sleep(0.0f);
					// Poll the results again and check whether we are now done
					int32 NumAsyncWorkRetired;
					PackageDatas->PollPendingCookedPlatformDatas(true, LastCookableObjectTickTime, NumAsyncWorkRetired);
					OutNumAsyncWorkRetired += NumAsyncWorkRetired;
					PrepareSaveStatus = PrepareSave(PackageData, StackData.Timer, false /* bPrecaching */,
						DemotionReason);
				} while (!StackData.Timer.IsActionTimeUp() && PrepareSaveStatus == EPollStatus::Incomplete
					&& DemotionReason == ESuppressCookReason::NotSuppressed
					&& PackageData.GetUrgency() == EUrgency::Blocking);
			}
			// If we couldn't postpone or wait, then we need to exit and try again later
			if (PrepareSaveStatus != EPollStatus::Success)
			{
				StackData.ResultFlags |= COSR_WaitingOnCache;
				bOutBusy = true;
				SaveQueue.AddFront(&PackageData);
				return;
			}
		}
		check(PrepareSaveStatus == EPollStatus::Success); // We are not allowed to save until PrepareSave succeeds.  We should have early exited above if it didn't

		// precache the next few packages
		if (!IsCookOnTheFlyMode() && SaveQueue.Num() != 0)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(PrecachePlatformDataForNextPackage);
			const int32 NumberToPrecache = 2;
			int32 LeftToPrecache = NumberToPrecache;
			for (FPackageData* NextData : SaveQueue)
			{
				if (LeftToPrecache == 0)
				{
					break;
				}

				--LeftToPrecache;
				PrepareSave(*NextData, StackData.Timer, /*bPrecaching*/ true, DemotionReason);
			}

			// If we're in RealTimeMode, check whether the precaching overflowed our timer and if so exit before we do the potentially expensive SavePackage
			// For non-realtime, overflowing the timer is not a critical issue.
			if (IsRealtimeMode() && StackData.Timer.IsActionTimeUp())
			{
				SaveQueue.AddFront(&PackageData);
				return;
			}
		}

		FSaveCookedPackageContext Context(*this, PackageData, PlatformsForPackage, StackData, EReachability::Runtime);
		SaveCookedPackage(Context);
		if (Context.bHasTimeOut)
		{
			// Timeouts can occur because of new objects created during the save, so we need to update our object cache,
			// so we call ReleaseCookedPlatformData and ClearObjectCache to clear it and recache on next attempt.
			check(PackageData.GetState() == EPackageState::SaveActive);
			// TODO: ReleaseCookedPlatformData is not valid for resetting the objectcache for a generator or generated
			// package; we need to add a function to handle it on the GenerationHelper
			check(!PackageData.GetGenerationHelper() && !PackageData.GetParentGenerationHelper());
			ReleaseCookedPlatformData(PackageData, EStateChangeReason::RecreateObjectCache, EPackageState::SaveActive);
			PackageData.ClearObjectCache();
			if (PackageData.GetUrgency() > EUrgency::Normal)
			{
				SaveQueue.AddFront(&PackageData);
			}
			else
			{
				SaveQueue.Add(&PackageData);
			}
			continue;
		}

		ReleaseCookedPlatformData(PackageData,
			!Context.bHasRetryErrorCode ? EStateChangeReason::Completed : EStateChangeReason::DoneForNow,
			EPackageState::Idle);
		PromoteToSaveComplete(PackageData, ESendFlags::QueueAdd);
		++OutNumPushed;
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif
	}
}

void UCookOnTheFlyServer::PumpBuildDependencySaves(UE::Cook::FTickStackData& StackData, uint32 DesiredQueueLength,
	int32& OutNumPushed, bool& bOutBusy, int32& OutNumAsyncWorkRetired)
{
	using namespace UE::Cook;
	OutNumPushed = 0;
	bOutBusy = false;
	OutNumAsyncWorkRetired = 0;

	UE_SCOPED_HIERARCHICAL_COOKTIMER(SavingPackages);
	check(IsInGameThread());
	ON_SCOPE_EXIT
	{
		PumpHasExceededMaxMemory(StackData.ResultFlags);
	};

	// Commit as many packages as we can during our time slice
	FPackageDataQueue& SaveQueue = PackageDatas->GetSaveQueue();
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> PlatformsForPackage;
	COOK_STAT(DetailedCookStats::PeakSaveQueueSize = FMath::Max(DetailedCookStats::PeakSaveQueueSize, SaveQueue.Num()));
	while (SaveQueue.Num() > static_cast<int32>(DesiredQueueLength))
	{
		FPackageData& PackageData(*SaveQueue.PopFrontValue());
		if (TryCreateRequestCluster(PackageData))
		{
			continue;
		}

		FPoppedPackageDataScope PoppedScope(PackageData);
		UPackage* Package = PackageData.GetPackage();

		check(Package != nullptr);

		// Commit only the session platforms that are requested and have not yet been committed
		PackageData.GetPlatformsNeedingCommit(PlatformsForPackage, GetCookPhase());
		if (PlatformsForPackage.Num() == 0)
		{
			// We've already committed all possible platforms for this package; this should not be possible.
			// All places that add a package to the save queue check for existence of incomplete platforms before adding
			UE_LOGF(LogCook, Warning, "Package '%ls' in SaveQueue has no more platforms left to commit; this should not be possible!",
				*PackageData.GetFileName().ToString());
			DemoteToIdle(PackageData, ESendFlags::QueueAdd, ESuppressCookReason::AlreadyCooked);
			++OutNumPushed;
			continue;
		}

		if (StackData.Timer.IsActionTimeUp())
		{
			SaveQueue.AddFront(&PackageData);
			return;
		}

		FSaveCookedPackageContext Context(*this, PackageData, PlatformsForPackage, StackData, EReachability::Build);
		CommitUncookedPackage(Context);
		if (Context.bHasTimeOut)
		{
			// Timeouts can occur because of new objects created during the save, so we need to update our object cache,
			// so we call ReleaseCookedPlatformData and ClearObjectCache to clear it and recache on next attempt.
			check(PackageData.GetState() == EPackageState::SaveActive);
			SaveQueue.Add(&PackageData);
			continue;
		}

		PromoteToSaveComplete(PackageData, ESendFlags::QueueAdd);
		++OutNumPushed;
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif
	}
}

void UCookOnTheFlyServer::InsertPhaseTransitionFence()
{
	if (!CookDirector)
	{
		return;
	}

	PhaseTransitionFence = CookDirector->InsertBroadcastFence();
}

void UCookOnTheFlyServer::PumpPhaseTransitionFence(bool& bOutComplete)
{
	using namespace UE::Cook;

	if (!CookDirector || PhaseTransitionFence == -1)
	{
		bOutComplete = true;
		return;
	}

	TArray<FWorkerId> PendingWorkers;
	if (CookDirector->IsBroadcastFencePassed(PhaseTransitionFence, &PendingWorkers))
	{
		bOutComplete = true;
		PhaseTransitionFence = -1;
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	constexpr float ReportPeriod = 60.f;
	if (IdleStatusLastReportTime + ReportPeriod < CurrentTime)
	{
		TStringBuilder<256> WorkerStr;
		for (FWorkerId WorkerId: PendingWorkers)
		{
			WorkerStr << CookDirector->GetDisplayName(WorkerId) << TEXT(", ");
		}
		check(WorkerStr.Len() >= 0);
		WorkerStr.RemoveSuffix(2); // Remove ", "

		UE_LOGF(LogCook, Display,
			"Waiting on MPCook fence, but { %ls } has not responded to a heartbeat request for %.1f seconds. Continuing to wait...",
			*WorkerStr, CurrentTime - IdleStatusStartTime);
		IdleStatusLastReportTime = CurrentTime;
	}

	bOutComplete = false;
}

void UCookOnTheFlyServer::KickBuildDependencies(UE::Cook::FTickStackData& StackData)
{
	using namespace UE::Cook;

	check(!IsCookWorkerMode()); // KickBuildDependencies should only be called on the Director or in SPCook.
	if (CookDirector)
	{
		CookDirector->BroadcastMessage(FDirectorEventMessage(EDirectorEvent::KickBuildDependencies));
	}
	SetCookPhase(ECookPhase::BuildDependencies);

	if (bLegacyBuildDependencies)
	{
		// Uncooked BuildDependencies are not committed in legacy cooks that are not using IncrementalCook
		return;
	}

	// Switch to turn off the commit of uncooked build dependencies, in case it causes a performance problem.
	constexpr bool bCommitUncookedBuildDependenciesEnabled = true;
	if (!bCommitUncookedBuildDependenciesEnabled)
	{
		return;
	}

	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	// KickBuildDependencies is only called when !RequestQueue.HasRequestsToExplore(). We rely on that and give a warning
	// during BuildDependencies phase if it is ever non-empty.
	check(RequestQueue.GetDiscoveryQueue().IsEmpty());

	TArray<FPackageData*> BuildPackages;
	PackageDatas->LockAndEnumeratePackageDatas([&BuildPackages](FPackageData* PackageData)
		{
			// KickBuildDependencies is only called when no packages are in progress.
			check(!PackageData->IsInProgress());
			for (const TPair<const ITargetPlatform*, FPackagePlatformData>& PlatformPair :
				PackageData->GetPlatformDatas())
			{
				const FPackagePlatformData& PlatformData = PlatformPair.Value;
				if (PlatformData.IsReachable(EReachability::Build) && !PlatformData.IsCommitted())
				{
					BuildPackages.Add(PackageData);
					break;
				}
			}
		});
	if (!BuildPackages.IsEmpty())
	{
		UE_LOGF(LogCook, Display, "UncookedBuildDependencies: Queueing %d packages for load-only commit.", BuildPackages.Num());
		PackageDatas->GetRequestQueue().GetBuildDependencyDiscoveryQueue().MoveAppendRange(BuildPackages.GetData(), BuildPackages.Num());
	}
}

void UCookOnTheFlyServer::PostLoadPackageFixup(UE::Cook::FPackageData& PackageData, UPackage* Package)
{
	if (Package->ContainsMap() == false)
	{
		return;
	}
	UWorld* World = UWorld::FindWorldInPackage(Package);
	if (!World)
	{
		return;
	}

	UE_SCOPED_HIERARCHICAL_COOKTIMER(PostLoadPackageFixup);
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(Package, PackageAccessTrackingOps::NAME_PostLoad);

	// Perform special processing for UWorld
	World->PersistentLevel->HandleLegacyMapBuildData();

	if (IsDirectorCookOnTheFly() || CookByTheBookOptions->bSkipSoftReferences)
	{
		return;
	}

	GIsCookerLoadingPackage = true;
	if (World->GetStreamingLevels().Num())
	{
		UE_SCOPED_COOKTIMER(PostLoadPackageFixup_LoadSecondaryLevels);
		TSet<FName> NeverCookPackageNames;
		PackageTracker->NeverCookPackageList.GetValues(NeverCookPackageNames);

		UE_LOGF(LogCook, Display, "Loading secondary levels for package '%ls'", *World->GetName());

		World->LoadSecondaryLevels(true, &NeverCookPackageNames);
	}
	GIsCookerLoadingPackage = false;

	TArray<FString> NewPackagesToCook;

	// Collect world composition tile packages to cook
	if (World->WorldComposition)
	{
		World->WorldComposition->CollectTilesToCook(NewPackagesToCook);
	}

	FName OwnerName = Package->GetFName();
	for (const FString& PackageName : NewPackagesToCook)
	{
		UE::Cook::FPackageData* NewPackageData = PackageDatas->TryAddPackageDataByPackageName(FName(*PackageName));
		if (NewPackageData)
		{
			QueueDiscoveredPackage(*NewPackageData, UE::Cook::FInstigator(UE::Cook::EInstigator::Dependency, OwnerName),
				UE::Cook::EDiscoveredPlatformSet::CopyFromInstigator);
		}
	}
}

void UCookOnTheFlyServer::TickPrecacheObjectsForPlatforms(const float TimeSlice, const TArray<const ITargetPlatform*>& TargetPlatforms) 
{
	using namespace UE::Cook;
	SCOPE_CYCLE_COUNTER(STAT_TickPrecacheCooking);


	FCookerTimer Timer(TimeSlice);

	if (LastUpdateTick > 50 ||
		((CachedMaterialsToCacheArray.Num() == 0) && (CachedTexturesToCacheArray.Num() == 0)))
	{
		CachedMaterialsToCacheArray.Reset();
		CachedTexturesToCacheArray.Reset();
		LastUpdateTick = 0;
		TArray<UObject*> Materials;
		GetObjectsOfClass(UMaterial::StaticClass(), Materials, true);
		for (UObject* Material : Materials)
		{
			if ( Material->GetOutermost() == GetTransientPackage())
				continue;

			CachedMaterialsToCacheArray.Add(Material);
		}
		TArray<UObject*> Textures;
		GetObjectsOfClass(UTexture::StaticClass(), Textures, true);
		for (UObject* Texture : Textures)
		{
			if (Texture->GetOutermost() == GetTransientPackage())
				continue;

			CachedTexturesToCacheArray.Add(Texture);
		}
	}
	++LastUpdateTick;

	if (Timer.IsActionTimeUp())
	{
		return;
	}

	bool AllMaterialsCompiled = true;
	// queue up some shaders for compilation

	while (CachedMaterialsToCacheArray.Num() > 0)
	{
		UMaterial* Material = (UMaterial*)(CachedMaterialsToCacheArray[0].Get());
		CachedMaterialsToCacheArray.RemoveAtSwap(0, EAllowShrinking::No);

		if (Material == nullptr)
		{
			continue;
		}

		FName PackageName = Material->GetPackage()->GetFName();
		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			if (!TargetPlatform)
			{
				continue;
			}
			if (!Material->IsCachedCookedPlatformDataLoaded(TargetPlatform))
			{
				Material->BeginCacheForCookedPlatformData(TargetPlatform);
				AllMaterialsCompiled = false;
			}
		}

		if (Timer.IsActionTimeUp())
		{
			return;
		}

		if (GShaderCompilingManager->GetNumRemainingJobs() > MaxPrecacheShaderJobs)
		{
			return;
		}
	}


	if (!AllMaterialsCompiled)
	{
		return;
	}

	while (CachedTexturesToCacheArray.Num() > 0)
	{
		UTexture* Texture = (UTexture*)(CachedTexturesToCacheArray[0].Get());
		CachedTexturesToCacheArray.RemoveAtSwap(0, EAllowShrinking::No);

		if (Texture == nullptr)
		{
			continue;
		}

		FName PackageName = Texture->GetPackage()->GetFName();
		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			if (!TargetPlatform)
			{
				continue;
			}
			if (!Texture->IsCachedCookedPlatformDataLoaded(TargetPlatform))
			{
				Texture->BeginCacheForCookedPlatformData(TargetPlatform);
			}
		}
		if (Timer.IsActionTimeUp())
		{
			return;
		}
	}
}

void UCookOnTheFlyServer::OnObjectModified( UObject *ObjectMoving )
{
	if (IsGarbageCollecting())
	{
		return;
	}
	OnObjectUpdated( ObjectMoving );
}

void UCookOnTheFlyServer::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (IsGarbageCollecting())
	{
		return;
	}
	if ( PropertyChangedEvent.Property == nullptr && 
		PropertyChangedEvent.MemberProperty == nullptr )
	{
		// probably nothing changed... 
		return;
	}

	OnObjectUpdated( ObjectBeingModified );
}

void UCookOnTheFlyServer::OnObjectSaved( UObject* ObjectSaved, FObjectPreSaveContext SaveContext)
{
	if (SaveContext.IsProceduralSave() )
	{
		// This is a procedural save (e.g. our own saving of the cooked package) rather than a user save, ignore
		return;
	}

	UPackage* Package = ObjectSaved->GetOutermost();
	if (Package == nullptr || Package == GetTransientPackage())
	{
		return;
	}

	MarkPackageDirtyForCooker(Package);

	// Register the package filename as modified. We don't use the cache because the file may not exist on disk yet at this point
	const FString PackageFilename = FPackageName::LongPackageNameToFilename(Package->GetName(), Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension());
	ModifiedAssetFilenames.Add(FName(*PackageFilename));
}

void UCookOnTheFlyServer::OnObjectUpdated( UObject *Object )
{
	// get the outer of the object
	UPackage *Package = Object->GetOutermost();

	MarkPackageDirtyForCooker( Package );
}

void UCookOnTheFlyServer::MarkPackageDirtyForCooker(UPackage* Package, bool bAllowInSession)
{
	if (Package->RootPackageHasAnyFlags(PKG_PlayInEditor))
	{
		return;
	}

	if (Package->HasAnyPackageFlags(PKG_PlayInEditor | PKG_ContainsScript | PKG_InMemoryOnly) == true && !GetClass()->HasAnyClassFlags(CLASS_DefaultConfig | CLASS_Config))
	{
		return;
	}

	if (Package == GetTransientPackage())
	{
		return;
	}

	if (Package->GetOuter() != nullptr)
	{
		return;
	}

	FName PackageName = Package->GetFName();
	if (FPackageName::IsMemoryPackage(PackageName.ToString()))
	{
		return;
	}

	if (bIsSavingPackage)
	{
		return;
	}

	if (IsInSession() && !bAllowInSession)
	{
		WorkerRequests->AddEditorActionCallback([this, PackageName]() { MarkPackageDirtyForCookerFromSchedulerThread(PackageName); });
	}
	else
	{
		MarkPackageDirtyForCookerFromSchedulerThread(PackageName);
	}
}

FName GInstigatorMarkPackageDirty(TEXT("MarkPackageDirtyForCooker"));
void UCookOnTheFlyServer::MarkPackageDirtyForCookerFromSchedulerThread(const FName& PackageName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MarkPackageDirtyForCooker);

	// could have just cooked a file which we might need to write
	UPackage::WaitForAsyncFileWrites();

	// Update the package's FileName if it has changed
	UE::Cook::FPackageData* PackageData = PackageDatas->UpdateFileName(PackageName);

	// force the package to be recooked
	UE_LOGF(LogCook, Verbose, "Modification detected to package %ls", *PackageName.ToString());
	if ( PackageData && IsCookingInEditor() )
	{
		check(IsInGameThread()); // We're editing scheduler data, which is only allowable from the scheduler thread
		bool bHadCookedPlatforms = PackageData->HasAnyCookedPlatform();
		PackageData->ClearCookResults();
		if (PackageData->IsInProgress())
		{
			PackageData->SendToState(UE::Cook::EPackageState::Request,
				UE::Cook::ESendFlags::QueueAddAndRemove, UE::Cook::EStateChangeReason::ForceRecook);
		}
		else if (IsCookByTheBookMode() && IsInSession() && bHadCookedPlatforms)
		{
			QueueDiscoveredPackage(*PackageData,
				UE::Cook::FInstigator(UE::Cook::EInstigator::Unspecified, GInstigatorMarkPackageDirty),
				UE::Cook::EDiscoveredPlatformSet::CopyFromInstigator);
		}

		if ( IsCookOnTheFlyMode() && FileModifiedDelegate.IsBound())
		{
			FString PackageFileNameString = PackageData->GetFileName().ToString();
			FileModifiedDelegate.Broadcast(PackageFileNameString);
			if (PackageFileNameString.EndsWith(".uasset") || PackageFileNameString.EndsWith(".umap"))
			{
				FileModifiedDelegate.Broadcast( FPaths::ChangeExtension(PackageFileNameString, TEXT(".uexp")) );
				FileModifiedDelegate.Broadcast( FPaths::ChangeExtension(PackageFileNameString, TEXT(".ubulk")) );
				FileModifiedDelegate.Broadcast( FPaths::ChangeExtension(PackageFileNameString, TEXT(".ufont")) );
			}
		}
	}
}

bool UCookOnTheFlyServer::IsInSession() const
{
	return bSessionRunning;
}

void UCookOnTheFlyServer::ShutdownCookOnTheFly()
{
	if (CookOnTheFlyRequestManager.IsValid())
	{
		UE_LOGF(LogCook, Display, "Shutting down cook on the fly server");
		CookOnTheFlyRequestManager->Shutdown();
		CookOnTheFlyRequestManager.Reset();

		ShutdownCookSession();

		if (bDisableShaderCompilationDuringCookOnTheFly)
		{
			GShaderCompilingManager->SkipShaderCompilation(false);
		}
		if (bAllowIncompleteShaderMapsDuringCookOnTheFly)
		{
			GShaderCompilingManager->SetAllowForIncompleteShaderMaps(false);
		}
	}
}

uint32 UCookOnTheFlyServer::GetPackagesPerGC() const
{
	return PackagesPerGC;
}

uint32 UCookOnTheFlyServer::GetPackagesPerPartialGC() const
{
	return MaxNumPackagesBeforePartialGC;
}

double UCookOnTheFlyServer::GetIdleTimeToGC() const
{
	if (IsCookOnTheFlyMode() && !IsCookingInEditor())
	{
		// For COTF outside of the editor we want to release open linker file handles promptly but still give some time for new requests to come in
		return 0.5;
	}
	else
	{
		return IdleTimeToGC;
	}
}

void UCookOnTheFlyServer::BeginDestroy()
{
	// BeginDestroy will be called for the CDO of UCookOnTheFlyServer and I shouldn't call anything in this case.
	// So check if Initialize was called using PackageDatas.IsValid() and only call Shutdown in this case.
	if (PackageDatas.IsValid())
	{
		UE::CookAssetRegistryAccessTracker::FCookAssetRegistryAccessTracker::Get().Shutdown();
	}

	ShutdownCookOnTheFly();

	Super::BeginDestroy();
}

void UCookOnTheFlyServer::TickRequestManager()
{
	if (CookOnTheFlyRequestManager)
	{
		CookOnTheFlyRequestManager->Tick();
	}
}

class FDiffModeCookServerUtils
{
public:
	enum class EDiffMode
	{
		None,
		DiffOnly,
		IncrementalValidate,
		IncrementalValidatePhase1,
		IncrementalValidatePhase2,
	};

	void InitializePackageWriter(UCookOnTheFlyServer& COTFS, ICookedPackageWriter*& CookedPackageWriter,
		const FString& ResolvedMetadataPath, UE::Cook::FDeterminismManager* InDeterminismManager)
	{
		Initialize();
		if (DiffMode == EDiffMode::None)
		{
			bAllowWrite.Emplace(true);
			return;
		}

		bool bCurrentWriterAllowWrite = true;
		ICookedPackageWriter::FCookCapabilities Capabilities = CookedPackageWriter->GetCookCapabilities();
		if (!Capabilities.bDiffModeSupported)
		{
			// All current PackageWriters support bDiffModeSupported; log a fatal error in case a new one is added.
			UE_LOGF(LogCook, Fatal, 
				"A DiffMode was enabled, but the current PackageWriter has bDiffModeSupported=false.");
		}

		// Wrap the incoming writer inside the feature-specific-functionality writer
		FIncrementalValidatePackageWriter* IncrementalValidatePackageWriter = nullptr;
		switch (DiffMode)
		{
		case EDiffMode::DiffOnly:
			CookedPackageWriter = new FDiffPackageWriter(COTFS, TUniquePtr<ICookedPackageWriter>(CookedPackageWriter),
				InDeterminismManager, FDiffPackageWriter::EReportingMode::OutputPackageSummary);
			bCurrentWriterAllowWrite = false;
			break;
		case EDiffMode::IncrementalValidate:
			IncrementalValidatePackageWriter = new FIncrementalValidatePackageWriter(COTFS,
				TUniquePtr<ICookedPackageWriter>(CookedPackageWriter), FIncrementalValidatePackageWriter::EPhase::AllInOnePhase,
				ResolvedMetadataPath, InDeterminismManager);
			break;
		case EDiffMode::IncrementalValidatePhase1:
			IncrementalValidatePackageWriter = new FIncrementalValidatePackageWriter(COTFS,
				TUniquePtr<ICookedPackageWriter>(CookedPackageWriter), FIncrementalValidatePackageWriter::EPhase::Phase1,
				ResolvedMetadataPath, InDeterminismManager);
			break;
		case EDiffMode::IncrementalValidatePhase2:
			IncrementalValidatePackageWriter = new FIncrementalValidatePackageWriter(COTFS,
				TUniquePtr<ICookedPackageWriter>(CookedPackageWriter), FIncrementalValidatePackageWriter::EPhase::Phase2,
				ResolvedMetadataPath, InDeterminismManager);
			break;
		default:
			checkNoEntry();
			break;
		}

		if (IncrementalValidatePackageWriter)
		{
			CookedPackageWriter = IncrementalValidatePackageWriter;
			bCurrentWriterAllowWrite = !IncrementalValidatePackageWriter->IsReadOnly();
		}
		if (!bAllowWrite.IsSet())
		{
			bAllowWrite.Emplace(bCurrentWriterAllowWrite);
		}
	}

	bool IsDeterminismDebug() const
	{
		switch (DiffMode)
		{
		case EDiffMode::None: return false;
		case EDiffMode::DiffOnly: return true;
		case EDiffMode::IncrementalValidate: return true;
		case EDiffMode::IncrementalValidatePhase1: return true;
		case EDiffMode::IncrementalValidatePhase2: return true;
		default:
			checkNoEntry();
			return false;
		}
	}

	bool IsFullBuildAllowed() const
	{
		// We require that bAllowWrite has been set - which is done during FindOrCreatePackageWriter - before
		// IsFullBuildAllowed is read.
		check(bAllowWrite.IsSet());
		return *bAllowWrite;
	}

	void Initialize()
	{
		if (bInitialized)
		{
			return;
		}

		DiffMode = EDiffMode::None;
		const TCHAR* CommandLine = FCommandLine::Get();
		auto EnsureMutualExclusion = [this]()
		{
			if (DiffMode != EDiffMode::None)
			{
				UE_LOGF(LogCook, Fatal, "-DiffOnly, and -IncrementalValidate* are mutually exclusive.");
			}
		};

		if (FParse::Param(CommandLine, TEXT("DIFFONLY")))
		{
			EnsureMutualExclusion();
			DiffMode = EDiffMode::DiffOnly;
		}
		if (FParse::Param(CommandLine, TEXT("IncrementalValidate")))
		{
			EnsureMutualExclusion();
			DiffMode = EDiffMode::IncrementalValidate;
		}
		if (FParse::Param(CommandLine, TEXT("IncrementalValidatePhase1")))
		{
			EnsureMutualExclusion();
			DiffMode = EDiffMode::IncrementalValidatePhase1;
		}
		if (FParse::Param(CommandLine, TEXT("IncrementalValidatePhase2")))
		{
			EnsureMutualExclusion();
			DiffMode = EDiffMode::IncrementalValidatePhase2;
		}
		bInitialized = true;
	}

	bool IsDiffModeActive() const
	{
		return DiffMode != EDiffMode::None;
	}

	bool IsIncrementallyModifiedDiagnostics() const
	{
		switch (DiffMode)
		{
		case EDiffMode::None: return false;
		case EDiffMode::DiffOnly: return false;
		case EDiffMode::IncrementalValidate: return true;
		case EDiffMode::IncrementalValidatePhase1: return true;
		case EDiffMode::IncrementalValidatePhase2: return true;
		default:
			checkNoEntry();
			return false;
		}
	}

private:
	bool bInitialized = false;
	EDiffMode DiffMode = EDiffMode::None;
	TOptional<bool> bAllowWrite;
};

bool UCookOnTheFlyServer::IsDebugRecordUnsolicited() const
{
	return bOnlyEditorOnlyDebug | bHiddenDependenciesDebug;
}

void UCookOnTheFlyServer::RecordExternalActorDependencies(TConstArrayView<FName> ExternalActorDependencies)
{
	using namespace UE::Cook;

	if (IsCookWorkerMode())
	{
		// The dependencies will be replicated to the CookDirector during ReportPromoteToSaveComplete
		return;
	}

	// External actors are a special case in the cooker, and they are only referenced through the
	// WorldPartitionCookPackageSplitter. They are marked as NeverCook, but we need to add them to the cook results so
	// we can detect whether they change in incremental cooks. The splitter has passed in its list of
	// ExternalActorDependencies; add them to the list of cooked packages stored in the AssetRegistry.
	for (FName DependencyName : ExternalActorDependencies)
	{
		FPackageData* DependencyData = PackageDatas->TryAddPackageDataByPackageName(DependencyName);
		if (DependencyData)
		{
			for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
			{
				IAssetRegistryReporter& Reporter = *PlatformManager->GetPlatformData(TargetPlatform)->RegistryReporter;

				DependencyData->SetPlatformCooked(TargetPlatform, ECookResult::NeverCookPlaceholder);
				Reporter.UpdateAssetRegistryData(DependencyName, nullptr /* Package */,
					ECookResult::NeverCookPlaceholder, nullptr /* SavePackageResult */,
					TOptional<TArray<FAssetData>>(), TOptional<FAssetPackageData>(), TOptional<TArray<FAssetDependency>>(),
					*this);
			}
		}
	}
}

void UCookOnTheFlyServer::Initialize( ECookMode::Type DesiredCookMode, ECookInitializationFlags InCookFlags, const FString &InOutputDirectoryOverride )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCookOnTheFlyServer::Initialize);
	LLM_SCOPE_BYTAG(Cooker);

	CurrentCookMode = DesiredCookMode;
	CookFlags = InCookFlags;

	PackageDatas = MakeUnique<UE::Cook::FPackageDatas>(*this);
	PlatformManager = MakeUnique<UE::Cook::FPlatformManager>();
	PackageTracker = MakeUnique<UE::Cook::FPackageTracker>(*this);
	DiffModeHelper = MakeUnique<FDiffModeCookServerUtils>();
	BuildDefinitions = MakeUnique<UE::Cook::FBuildDefinitions>();
	SharedLooseFilesCookArtifactReader = MakeShared<FLooseFilesCookArtifactReader>();
	AllContextArtifactReader = MakeUnique<FLayeredCookArtifactReader>();
	AllContextArtifactReader->AddLayer(SharedLooseFilesCookArtifactReader.ToSharedRef());
	CookByTheBookOptions = MakeUnique<UE::Cook::FCookByTheBookOptions>();
	CookOnTheFlyOptions = MakeUnique<UE::Cook::FCookOnTheFlyOptions>();
	AssetRegistry = IAssetRegistry::Get();
	GCDiagnosticContext = MakeUnique<UE::Cook::FCookGCDiagnosticContext>();
	StallDetector = MakeUnique<UE::Cook::FStallDetector>();
	LogHandler.Reset(UE::Cook::CreateLogHandler(*this));

	GlobalArtifact = new UE::Cook::FGlobalCookArtifact(*this);
	TRefCountPtr<UE::Cook::ICookArtifact> RefCounter(GlobalArtifact);
	RegisterArtifact(GlobalArtifact);
	// We keep a raw pointer and rely on RegisterArtifact holding the reference.
	check(RefCounter.GetRefCount() > 1);

	if (!IsCookWorkerMode())
	{
		WorkerRequests.Reset(new UE::Cook::FWorkerRequestsLocal());
	}
	else
	{
		check(WorkerRequests); // Caller should have constructed
	}
	DirectorCookMode = WorkerRequests->GetDirectorCookMode(*this);
	if (IsCookByTheBookMode() && !IsCookingInEditor())
	{
		bool bLaunchedByEditor = !FPlatformMisc::GetEnvironmentVariable(GEditorUIPidVariable).IsEmpty();
		int32 CookProcessCount=-1;
		bool bSetByCommandLine = FParse::Value(FCommandLine::Get(), TEXT("-CookProcessCount="), CookProcessCount);
		if (CookProcessCount < 0 && bLaunchedByEditor)
		{
			GConfig->GetInt(TEXT("CookSettings"), TEXT("CookProcessCountFromEditor"), CookProcessCount, GEditorIni);
		}
		if (CookProcessCount < 0)
		{
			GConfig->GetInt(TEXT("CookSettings"), TEXT("CookProcessCount"), CookProcessCount, GEditorIni);
		}
		CookProcessCount = FMath::Max(1, CookProcessCount);
		if (CookProcessCount > UE::Cook::FWorkerId::GetMaxCookWorkerCount())
		{
			// We could clamp it and continue on, but it's not clear what to clamp it to. If they ask for
			// 1 billion by accidental typo in the ini, what should we set it to?
			UE_LOGF(LogCook, Fatal, "Invalid CookProcessCount=%d, maximum value is %d.",
				CookProcessCount, UE::Cook::FWorkerId::GetMaxCookWorkerCount());
		}
		if (CookProcessCount > 1)
		{
			CookDirector = MakeUnique<UE::Cook::FCookDirector>(*this, CookProcessCount, bSetByCommandLine);
			if (!CookDirector->IsMultiprocessAvailable())
			{
				CookDirector.Reset();
			}
		}
		else
		{
			UE_LOGF(LogCook, Display, "CookProcessCount=%d. CookMultiprocess is disabled and the cooker is running as a single process.",
				CookProcessCount);
		}
	}

	UE::Cook::InitializeTls();
	UE::Cook::FPlatformManager::InitializeTls();

	LoadInitializeConfigSettings(InOutputDirectoryOverride);

	CookProgressRetryBusyPeriodSeconds = GCookProgressRetryBusyTime;
	if (IsCookOnTheFlyMode() && !IsRealtimeMode())
	{
		// Remove sleeps when waiting on async operations and otherwise idle; busy wait instead to minimize latency
		CookProgressRetryBusyPeriodSeconds = 0.0f;
	}
	DisplayUpdatePeriodSeconds = FMath::Min(GCookProgressRepeatTime, FMath::Min(GCookProgressUpdateTime, GCookProgressDiagnosticTime));

	PollNextTimeSeconds = MAX_flt;
	PollNextTimeIdleSeconds = MAX_flt;

	CurrentAsyncCacheForType = MaxAsyncCacheForType;
	// CookCommandlet CookWorker and CookByTheBook do not initialize startup packages until BlockOnAssetRegistry,
	// because systems that subscribe to the AssetRegistry's OnFilesLoaded can load further packages at that time.
	// But for the CookOnTheFlyServer in the editor, or CookOnTheFly this is the only opportunity it has.
	if (IsCookingInEditor() || (!IsCookByTheBookMode() && !IsCookWorkerMode()))
	{
		TSet<FName> StartupPackages;
		PackageTracker->InitializeTracking(StartupPackages);
		NumPackagesHistory.Initialize(PackageTracker->NumLoadedPackages());
		CookByTheBookOptions->StartupPackages = MoveTemp(StartupPackages);
	}

	IdleStatus = EIdleStatus::Done;
	IdleStatusStartTime = FPlatformTime::Seconds();
	IdleStatusLastReportTime = IdleStatusStartTime;

	// It is perfectly safe to allow preloading in any environment whenever async loading is active
	// since we won't try to deal with linkers directly, we'll use proper calls to LoadPackageAsync in the preloader.
	if (IsAsyncLoadingMultithreaded())
	{
		bPreloadingEnabled = true;
	}
	else 
	{
		if (!IsCookOnTheFlyMode() && !IsCookingInEditor() &&
			FPlatformMisc::SupportsMultithreadedFileHandles() && // Preloading moves file handles between threads
			!GAllowCookedDataInEditorBuilds // // Use of preloaded files is not yet implemented when GAllowCookedDataInEditorBuilds is on, see FLinkerLoad::CreateLoader
			)
		{
			bPreloadingEnabled = true;
			FLinkerLoad::SetPreloadingEnabled(true);
		}
	}

	// Prepare a map of SplitDataClass to FRegisteredCookPackageSplitter* for TryGetRegisteredCookPackageSplitter to use
	RegisteredSplitDataClasses.Reset();
	UE::Cook::Private::FRegisteredCookPackageSplitter::ForEach([this](UE::Cook::Private::FRegisteredCookPackageSplitter* RegisteredCookPackageSplitter)
	{
		UClass* SplitDataClass = RegisteredCookPackageSplitter->GetSplitDataClass();
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			if (ClassIt->IsChildOf(SplitDataClass) && !ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				RegisteredSplitDataClasses.Add(SplitDataClass, RegisteredCookPackageSplitter);
			}
		}
	});

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddUObject(this, &UCookOnTheFlyServer::PreGarbageCollect);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UCookOnTheFlyServer::PostGarbageCollect);

	if (IsCookingInEditor())
	{
		check(!IsCookWorkerMode()); // To allow in-editor callbacks on CookWorker, FWorkerRequestsRemote::AddEditorActionCallback will need to be updated to allow editor operations
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UCookOnTheFlyServer::OnObjectPropertyChanged);
		FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &UCookOnTheFlyServer::OnObjectModified);
		FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UCookOnTheFlyServer::OnObjectSaved);

		FCoreDelegates::OnTargetPlatformChangedSupportedFormats.AddUObject(this, &UCookOnTheFlyServer::OnTargetPlatformChangedSupportedFormats);
	}

	GetTargetPlatformManager()->GetOnTargetPlatformsInvalidatedDelegate().AddUObject(this, &UCookOnTheFlyServer::OnTargetPlatformsInvalidated);

	UE::CookAssetRegistryAccessTracker::FCookAssetRegistryAccessTracker::Get().Init();
}

void UCookOnTheFlyServer::InitializeAtFirstSession()
{
	UE::EditorDomain::UtilsCookInitialize();
}

void UCookOnTheFlyServer::LoadInitializeConfigSettings(const FString& InOutputDirectoryOverride)
{
	UE::Cook::FInitializeConfigSettings Settings;
	WorkerRequests->GetInitializeConfigSettings(*this, InOutputDirectoryOverride, Settings);
	SetInitializeConfigSettings(MoveTemp(Settings));
}

namespace UE::Cook
{

void FInitializeConfigSettings::LoadLocal(const FString& InOutputDirectoryOverride)
{
	using namespace UE::Cook;

	OutputDirectoryOverride = InOutputDirectoryOverride;

	MaxPrecacheShaderJobs = FPlatformMisc::NumberOfCores() - 1; // number of cores -1 is a good default allows the editor to still be responsive to other shader requests and allows cooker to take advantage of multiple processors while the editor is running
	GConfig->GetInt(TEXT("CookSettings"), TEXT("MaxPrecacheShaderJobs"), MaxPrecacheShaderJobs, GEditorIni);

	MaxConcurrentShaderJobs = FPlatformMisc::NumberOfCores() * 4; // TODO: document why number of cores * 4 is a good default
	GConfig->GetInt(TEXT("CookSettings"), TEXT("MaxConcurrentShaderJobs"), MaxConcurrentShaderJobs, GEditorIni);

	PackagesPerGC = 500;
	int32 ConfigPackagesPerGC = 0;
	if (GConfig->GetInt( TEXT("CookSettings"), TEXT("PackagesPerGC"), ConfigPackagesPerGC, GEditorIni ))
	{
		// Going unsigned. Make negative values 0
		PackagesPerGC = ConfigPackagesPerGC > 0 ? ConfigPackagesPerGC : 0;
	}

	IdleTimeToGC = 20.0;
	GConfig->GetDouble( TEXT("CookSettings"), TEXT("IdleTimeToGC"), IdleTimeToGC, GEditorIni );

	auto ReadMemorySetting = [](const TCHAR* SettingName, uint64& TargetVariable)
	{
		int32 ValueInMB = 0;
		if (GConfig->GetInt(TEXT("CookSettings"), SettingName, ValueInMB, GEditorIni))
		{
			ValueInMB = FMath::Max(ValueInMB, 0);
			TargetVariable = ValueInMB * 1024LL * 1024LL;
			return true;
		}
		return false;
	};

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MemoryMaxUsedVirtual = 0;
	MemoryMaxUsedPhysical = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MemoryMinFreeVirtual = 0;
	MemoryMinFreePhysical = 0;
	bUseSoftGC = false;
	SoftGCStartNumerator = 5;
	SoftGCDenominator = 10;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ReadMemorySetting(TEXT("MemoryMaxUsedVirtual"), MemoryMaxUsedVirtual);
	ReadMemorySetting(TEXT("MemoryMaxUsedPhysical"), MemoryMaxUsedPhysical);

	if (MemoryMaxUsedVirtual != 0)
	{
		UE_LOGF(LogCook, Warning, "Setting MemoryMaxUsedVirtual will be deprecated in future version. Please remove it from the settings files."
			"On systems with too little memory to load the minimum set required to cook some packages, frequent garbage collection will stall progress, "
			"we now detect this stall and terminate the cook with an assertion rather than the previous behavior of MemoryMaxUsedVirtual, "
			"which would terminate it with an OutOfMemory message.")
	}

	if (MemoryMaxUsedPhysical != 0)
	{
		UE_LOGF(LogCook, Warning, "Setting MemoryMaxUsedPhysical will be deprecated in future version. Please remove it from the settings files."
			"On systems with too little memory to load the minimum set required to cook some packages, frequent garbage collection will stall progress, "
			"we now detect this stall and terminate the cook with an assertion rather than the previous behavior of MemoryMaxUsedPhysical, "
			"which would terminate it with an OutOfMemory message.")
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ReadMemorySetting(TEXT("MemoryMinFreeVirtual"), MemoryMinFreeVirtual);
	ReadMemorySetting(TEXT("MemoryMinFreePhysical"), MemoryMinFreePhysical);
	FString ConfigText(TEXT("None"));
	GConfig->GetString(TEXT("CookSettings"), TEXT("MemoryTriggerGCAtPressureLevel"), ConfigText, GEditorIni);
	if (!LexTryParseString(MemoryTriggerGCAtPressureLevel, ConfigText))
	{
		UE_LOGF(LogCook, Error, "Unrecognized value \"%ls\" for MemoryTriggerGCAtPressureLevel. Expected None or Critical.",
			*ConfigText);
	}
	GConfig->GetBool(TEXT("CookSettings"), TEXT("bUseSoftGC"), bUseSoftGC, GEditorIni);
	GConfig->GetInt(TEXT("CookSettings"), TEXT("SoftGCStartNumerator"), SoftGCStartNumerator, GEditorIni);
	GConfig->GetInt(TEXT("CookSettings"), TEXT("SoftGCDenominator"), SoftGCDenominator, GEditorIni);
	GConfig->GetFloat(TEXT("CookSettings"), TEXT("SoftGCTimeFractionBudget"), SoftGCTimeFractionBudget, GEditorIni);
	GConfig->GetFloat(TEXT("CookSettings"), TEXT("SoftGCMinimumPeriodSeconds"), SoftGCMinimumPeriodSeconds, GEditorIni);

	MemoryExpectedFreedToSpreadRatio = 0.10f;
	GConfig->GetFloat(TEXT("CookSettings"), TEXT("MemoryExpectedFreedToSpreadRatio"),
		MemoryExpectedFreedToSpreadRatio, GEditorIni);

	MinFreeUObjectIndicesBeforeGC = 100000;
	GConfig->GetInt(TEXT("CookSettings"), TEXT("MinFreeUObjectIndicesBeforeGC"), MinFreeUObjectIndicesBeforeGC, GEditorIni);
	MinFreeUObjectIndicesBeforeGC = FMath::Max(MinFreeUObjectIndicesBeforeGC, 0);

	MaxNumPackagesBeforePartialGC = 400;
	GConfig->GetInt(TEXT("CookSettings"), TEXT("MaxNumPackagesBeforePartialGC"), MaxNumPackagesBeforePartialGC, GEditorIni);
	
	GConfig->GetArray(TEXT("CookSettings"), TEXT("CookOnTheFlyConfigSettingDenyList"), ConfigSettingDenyList, GEditorIni);

	const FConfigSection* CacheSettings = GConfig->GetSection(TEXT("CookPlatformDataCacheSettings"), false, GEditorIni);
	if (CacheSettings)
	{
		for (const auto& CacheSetting : *CacheSettings)
		{

			const FString& ReadString = CacheSetting.Value.GetValue();
			int32 ReadValue = FCString::Atoi(*ReadString);
			int32 Count = FMath::Max(2, ReadValue);
			MaxAsyncCacheForType.Add(CacheSetting.Key, Count);
		}
	}

	bRandomizeCookOrder = FParse::Param(FCommandLine::Get(), TEXT("RANDOMPACKAGEORDER")) ||
		(FParse::Param(FCommandLine::Get(), TEXT("DIFFONLY")) && !FParse::Param(FCommandLine::Get(), TEXT("DIFFNORANDCOOK")));
}

static EMPCookGeneratorSplit ParseMPCookGeneratorSplitFromString(const FString& Text)
{
	if (Text.IsEmpty() || Text == TEXTVIEW("AnyWorker"))
	{
		return EMPCookGeneratorSplit::AnyWorker;
	}
	else if (Text == TEXTVIEW("AllOnSameWorker"))
	{
		return EMPCookGeneratorSplit::AllOnSameWorker;
	}
	if (Text == TEXTVIEW("SomeOnSameWorker"))
	{
		return EMPCookGeneratorSplit::SomeOnSameWorker;
	}
	if (Text == TEXTVIEW("NoneOnSameWorker"))
	{
		return EMPCookGeneratorSplit::NoneOnSameWorker;
	}
	else
	{
		UE_LOGF(LogCook, Error,
			"Invalid value -MPCookGeneratorSplit=%ls. Valid values: { AnyWorker, AllOnSameWorker, SomeOnSameWorker, NoneOnSameWorker }.",
			*Text);
		return EMPCookGeneratorSplit::AnyWorker;
	}
}

}

namespace UE::Cook::CVarControl
{
	void UpdateCVars(FBeginCookContext& BeginContext, FName OverrideDeviceProfileName, int32 OverrideCookCVarControl)
	{
		static FName CookTimeCVarTag("CookTimeCVars");

		const int32 CookTimeCVarControl = (OverrideCookCVarControl >= 0 && OverrideCookCVarControl <= 3) ? OverrideCookCVarControl : GCookTimeCVarControl;
		if (CookTimeCVarControl == 2 || CookTimeCVarControl == 3 || !OverrideDeviceProfileName.IsNone())
		{
			checkf(BeginContext.TargetPlatforms.Num() == 1, TEXT("When using Cook.CVarControl in mode 2 or 3, or specifying a device profile override, only a single TargetPlatform may be cooked at once."));
		}

		// in case we had cooked before on this run, reset everything
		IConsoleManager::Get().UnsetAllConsoleVariablesWithTag(CookTimeCVarTag);

		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();

		// hook up some global mappings
		// @todo clear out existing mappings for each Invalidate?
		for (const ITargetPlatform* Platform : BeginContext.TargetPlatforms)
		{
			FName PlatformName = *Platform->IniPlatformName();
			FName DPName = OverrideDeviceProfileName.IsNone() ? *Platform->CookingDeviceProfileName() : OverrideDeviceProfileName;

			if (CookTimeCVarControl != 0)
			{
				// register that when we cook for this platform, we will want to use the given DP when looking up CVar values
				// if it matches the platform, we don't set it, because that's the default, so don't actually do anything special
				if (DPName != PlatformName)
				{
					ConsoleVariablePlatformMapping::RegisterPlatformToDeviceProfileMapping(PlatformName, DPName);
				}
			}
			if (CookTimeCVarControl == 2)
			{
				IConsoleManager::Get().StompPlatformCVars(PlatformName, DPName.ToString(), CookTimeCVarTag, ECVF_SetByCode, ECVF_Preview, ECVF_Cheat);
			}
			else if (CookTimeCVarControl == 3)
			{
				IConsoleManager::Get().StompPlatformCVars(PlatformName, DPName.ToString(), CookTimeCVarTag, ECVF_SetByCode, ECVF_Default, ECVF_Default);
			}

			UpdateShaderCookingCVars(TPM, CookTimeCVarControl, Platform, PlatformName);
		}
	}
}

void UCookOnTheFlyServer::SetInitializeConfigSettings(UE::Cook::FInitializeConfigSettings&& Settings)
{
	Settings.MoveToLocal(*this);

	// For preload to actually be able to pipeline with load/batch, we need both RequestBatchSize
	// and MaxPreloadAllocated to be bigger than LoadBatchSize so that we won't consume all preloads
	// for every iteration.
	DesiredSaveQueueLength = 8;
	
	// Best preloading performance is achieved when all the loading parameters have the same value
	// and that value is high enough to get multithreaded async loading using all cores.
	if (IsAsyncLoadingMultithreaded())
	{
		DesiredLoadQueueLength = 256;
		LoadBatchSize = DesiredLoadQueueLength;
		RequestBatchSize = DesiredLoadQueueLength;
		MaxPreloadAllocated = DesiredLoadQueueLength;
	}
	else
	{
		MaxPreloadAllocated = 32;
		DesiredLoadQueueLength = 8;
		LoadBatchSize = 16;
		RequestBatchSize = 32;
	}

	WaitForAsyncSleepSeconds = 1.0f;


	// See if there are any plugins that need to be remapped for the sandbox
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
	if (Project != nullptr)
	{
		PluginsToRemap = IPluginManager::Get().GetEnabledPlugins();
		TArray<FString> AdditionalPluginDirs = Project->GetAdditionalPluginDirectories();
		// Remove all plugins that are not in the additional directories. Plugins not in additional directories
		// are under ProjectRoot or EngineRoot and do not need remapping.
		for (int32 Index = PluginsToRemap.Num() - 1; Index >= 0; Index--)
		{
			bool bRemove = true;
			for (const FString& PluginDir : AdditionalPluginDirs)
			{
				// If this plugin is in a directory that needs remapping
				if (PluginsToRemap[Index]->GetBaseDir().StartsWith(PluginDir))
				{
					bRemove = false;
					break;
				}
			}
			if (bRemove)
			{
				PluginsToRemap.RemoveAt(Index);
			}
		}
	}

	if (SoftGCTimeFractionBudget > 0)
	{
		SoftGCHistory.Reset(new UE::Cook::FSoftGCHistory());
	}

	// The rest of this function parses config settings that are reparsed on every CookDirector and CookWorker rather than
	// being replicated from CookDirector to CookWorker


	const TCHAR* CommandLine = FCommandLine::Get();

	// Parse -cookshow... flags
	bool bShowRemaining = !!(GCookProgressDisplay & (int32)ECookProgressDisplayMode::RemainingPackages);
	bool bShowPackageNames = !!(GCookProgressDisplay & (int32)ECookProgressDisplayMode::PackageNames);
	bool bShowInstigators = !!(GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators);
	ParseBoolParamOrValue(CommandLine, TEXT("cookshowremaining"), bShowRemaining);
	ParseBoolParamOrValue(CommandLine, TEXT("cookshowpackagenames"), bShowPackageNames);
	ParseBoolParamOrValue(CommandLine, TEXT("cookshowinstigators"), bShowInstigators);
	GCookProgressDisplay = 0;
	GCookProgressDisplay |= bShowRemaining ? (int32)ECookProgressDisplayMode::RemainingPackages : 0;
	GCookProgressDisplay |= bShowPackageNames ? (int32)ECookProgressDisplayMode::PackageNames : 0;
	GCookProgressDisplay |= bShowInstigators ? (int32)ECookProgressDisplayMode::Instigators : 0;

	// Report the memory settings
	UE_LOGF(LogCook, Display, "CookSettings for Memory:%ls", *GetCookSettingsForMemoryLogText());

	// Debugging hidden dependencies
	bOnlyEditorOnlyDebug = FParse::Param(CommandLine, TEXT("OnlyEditorOnlyDebug"));
	bSkipOnlyEditorOnly = false;
	GConfig->GetBool(TEXT("CookSettings"), TEXT("SkipOnlyEditorOnly"), bSkipOnlyEditorOnly, GEditorIni);
	ParseBoolParamOrValue(CommandLine, TEXT("SkipOnlyEditorOnly"), bSkipOnlyEditorOnly);
	bSkipOnlyEditorOnly |= bOnlyEditorOnlyDebug;
	if (bSkipOnlyEditorOnly)
	{
		UE_LOGF(LogCook, Display,
			"SkipOnlyEditorOnly is enabled, unsolicited packages will not be cooked unless they are referenced from the cooked version of the instigator package.");
	}

	bHiddenDependenciesDebug = FParse::Param(CommandLine, TEXT("HiddenDependenciesDebug"));
	if (bHiddenDependenciesDebug)
	{
		UE_LOGF(LogCook, Display, "HiddenDependenciesDebug is enabled.");

		// HiddenDependencies diagnostics rely on using SkipOnlyEditorOnly
		bSkipOnlyEditorOnly = true;

		FScopeLock HiddenDependenciesScopeLock(&HiddenDependenciesLock);

		FString ClassPathListStr;
		TOptional<bool> bAllowList;
		if (FParse::Value(CommandLine, TEXT("-HiddenDependenciesIgnore="), ClassPathListStr))
		{
			bAllowList = false;
		}
		if (FParse::Value(CommandLine, TEXT("-HiddenDependenciesReport="), ClassPathListStr))
		{
			if (bAllowList.IsSet() && !*bAllowList)
			{
				UE_LOGF(LogCook, Error, "-HiddenDependenciesIgnore and HiddenDepenciesReport are mutually exclusive. HiddenDepenciesIgnore setting will be discarded.");
			}
			bAllowList = true;
		}
		bHiddenDependenciesClassPathFilterListIsAllowList = bAllowList.IsSet() ? *bAllowList : false;
		if (!bHiddenDependenciesClassPathFilterListIsAllowList)
		{
			TArray<FString> ClassPaths;
			GConfig->GetArray(TEXT("CookSettings"), TEXT("IncrementalClassDenyList"), ClassPaths, GEditorIni);
			for (const FString& ClassPathLine : ClassPaths)
			{
				FStringView ClassPath(UE::EditorDomain::RemoveConfigComment(ClassPathLine));
				FTopLevelAssetPath Path(ClassPath);
				if (!Path.IsValid())
				{
					UE_LOGF(LogCook, Error, "Invalid Editor:[CookSettings]:IncrementalClassDenyList entry %.*ls. Expected an array of fullpaths such as /Script/Engine.Material",
						ClassPath.Len(), ClassPath.GetData());
					continue;
				}
				HiddenDependenciesClassPathFilterList.Add(FName(Path.ToString()));
			}
		}
		if (!ClassPathListStr.IsEmpty())
		{
			UE::String::ParseTokensMultiple(ClassPathListStr, UE::Cook::GetCommandLineDelimiterChars(),
				[this](FStringView Token)
				{
					FTopLevelAssetPath Path(Token);
					if (!Path.IsValid())
					{
						UE_LOGF(LogCook, Error, "Invalid %ls=<ClassPath> setting. Expected a comma-delimited list of fullpaths such as /Script/Engine.Material",
							bHiddenDependenciesClassPathFilterListIsAllowList ? TEXT("-HiddenDependenciesReport") : TEXT("-HiddenDependenciesIgnore"));
						return;
					}
					HiddenDependenciesClassPathFilterList.Add(FName(Path.ToString()));
				});
		}
	}

	ParseCookFilters();

	bCallIsCachedOnSaveCreatedObjects = FParse::Param(CommandLine, TEXT("CallIsCachedOnSaveCreatedObjects"));

	bLegacyIterativeIgnoreIni = false;
	GConfig->GetBool(TEXT("CookSettings"), TEXT("IterativeIgnoreIni"), bLegacyIterativeIgnoreIni, GEditorIni);
	GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterativeIgnoreIni"), bLegacyIterativeIgnoreIni, GEditorIni);
	bLegacyIterativeIgnoreIni =
		!FParse::Param(CommandLine, TEXT("iteraterequireini")) &&
		!FParse::Param(CommandLine, TEXT("iterativerequireini")) &&
		!FParse::Param(CommandLine, TEXT("legacyiterativerequireini")) &&
		(bLegacyIterativeIgnoreIni ||
			IsCookFlagSet(ECookInitializationFlags::IgnoreIniSettingsOutOfDate) ||
			FParse::Param(CommandLine, TEXT("iterateignoreini")) ||
			FParse::Param(CommandLine, TEXT("iterativeignoreini")) ||
			FParse::Param(CommandLine, TEXT("legacyiterativeignoreini")));
	bLegacyIterativeCalculateExe = true;
	bool bConfigSettingSetLegacyIterativeIgnoreExe = false;
	GConfig->GetBool(TEXT("CookSettings"), TEXT("IterativeIgnoreExe"),
		bConfigSettingSetLegacyIterativeIgnoreExe, GEditorIni);
	GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterativeIgnoreExe"),
		bConfigSettingSetLegacyIterativeIgnoreExe, GEditorIni);
	bLegacyIterativeIgnoreExe =
		!FParse::Param(CommandLine, TEXT("iteraterequireexe")) &&
		!FParse::Param(CommandLine, TEXT("iterativerequireexe")) &&
		!FParse::Param(CommandLine, TEXT("legacyiterativerequireexe")) &&
		(bConfigSettingSetLegacyIterativeIgnoreExe ||
			FParse::Param(CommandLine, TEXT("iterateignoreexe")) ||
			FParse::Param(CommandLine, TEXT("iterativeignoreexe")) ||
			FParse::Param(CommandLine, TEXT("legacyiterativeignoreexe")));
	// Calculate the exe hash if LegacyIterativeExeInvalidation is required by ini OR required by commandline
	// It would be better to always calculate it, but we want to avoid the performance cost until it becomes more widely used
	bLegacyIterativeCalculateExe = !bLegacyIterativeIgnoreIni || !bConfigSettingSetLegacyIterativeIgnoreExe;

	UE::Cook::ProjectCookIncrementalVersion = FGuid(0, 0, 0, 0);
	FString GuidText;
	if (GConfig->GetString(TEXT("CookSettings"), TEXT("ProjectCookIncrementalVersion"), GuidText, GEditorIni))
	{
		if (!FGuid::Parse(GuidText, UE::Cook::ProjectCookIncrementalVersion))
		{
			UE_LOGF(LogCook, Error,
				"Invalid config setting Editor.ini:[CookSettings]:ProjectCookIncrementalVersion=%ls. Expected a hexadecimal guid string: ProjectCookIncrementalVersion=A0C50A4C-3305-4AB1-A401-66635AEFDD7F. FGuid(0,0,0,0) will be used instead.",
				*GuidText);
		}
	}

	bRunningAsShaderServer = FParse::Param(CommandLine, TEXT("odsc"));
    ODSCClientData = nullptr;
	if (bRunningAsShaderServer)
	{
		ODSCClientData = MakeUnique<UE::Cook::FODSCClientData>();
		bAllowIncompleteShaderMapsDuringCookOnTheFly = true;
	}

	bSkipSave = FParse::Param(CommandLine, TEXT("CookSkipSave"));

	FString Severity;
	GConfig->GetString(TEXT("CookSettings"), TEXT("CookerIdleWarningSeverity"), Severity, GEditorIni);
	CookerIdleWarningSeverity = ParseLogVerbosityFromString(Severity);
	GConfig->GetString(TEXT("CookSettings"), TEXT("UnexpectedLoadWarningSeverity"), Severity, GEditorIni);
	UnexpectedLoadWarningSeverity = ParseLogVerbosityFromString(Severity);

	bCookFastStartup = FParse::Param(CommandLine, TEXT("cookfaststartup"));

	const UProjectPackagingSettings* PackagingSettings = Cast<UProjectPackagingSettings>(UProjectPackagingSettings::StaticClass()->GetDefaultObject());
	if (PackagingSettings->bTreatWarningsAsErrorsOnCook)
	{
		GWarn->TreatWarningsAsErrors = true;
	}

	FString GeneratorSplit;
	GConfig->GetString(TEXT("CookSettings"), TEXT("MPCookGeneratorSplit"), GeneratorSplit, GEditorIni);
	FParse::Value(CommandLine, TEXT("-MPCookGeneratorSplit="), GeneratorSplit);
	MPCookGeneratorSplit = UE::Cook::ParseMPCookGeneratorSplitFromString(GeneratorSplit);

	bDeterminismDebug = FParse::Param(CommandLine, TEXT("cookdeterminism")) ||
		FParse::Param(CommandLine, TEXT("diffonlybase"));

	FParse::Value(CommandLine, TEXT("-CookCVarControl="), OverrideCookCVarControl);
	FParse::Value(CommandLine, TEXT("-DeviceProfile="), OverrideDeviceProfileName);

	bIncrementallyModifiedDiagnostics = FParse::Param(CommandLine, TEXT("CookIncrementallyModifiedDiagnostics"))
		|| FParse::Param(CommandLine, TEXT("Cook.Diagnostic.Modified"));

	bCookingDLCBaseGamePlugin = FParse::Param(CommandLine, TEXT("CookingDLCBaseGamePlugin"));
}

void UCookOnTheFlyServer::ParseCookFilters()
{
	CookFilterIncludedClasses.Empty();
	CookFilterIncludedAssetClasses.Empty();
	bCookFilter = false;
	if (!IsCookByTheBookMode() || IsCookingInEditor())
	{
		return;
	}

	ParseCookFilters(TEXT("cookincludeclass"), TEXT("contain an object"), CookFilterIncludedClasses);
	ParseCookFilters(TEXT("cookincludeassetclass"), TEXT("contain an asset"), CookFilterIncludedAssetClasses);
}

void UCookOnTheFlyServer::ParseCookFilters(const TCHAR* Parameter, const TCHAR* Message, TSet<FName>& OutFilterClasses)
{
	FString IncludeClassesString;
	TStringBuilder<256> FullParameter(InPlace, TEXT("-"), Parameter, TEXT("="));
	if (FParse::Value(FCommandLine::Get(), *FullParameter, IncludeClassesString))
	{
		TArray<FString> IncludeClasses;
		TConstArrayView<const TCHAR*> Delimiters = UE::Cook::GetCommandLineDelimiterStrs();
		IncludeClassesString.ParseIntoArray(IncludeClasses, Delimiters.GetData(), Delimiters.Num(),
			true /* bCullEmpty */);
		TArray<FTopLevelAssetPath> RootNames;
		for (FString& IncludeClassString : IncludeClasses)
		{
			FTopLevelAssetPath ClassPath;
			if (!UClass::IsShortTypeName(IncludeClassString))
			{
				ClassPath.TrySetPath(IncludeClassString);
			}
			else
			{
				ClassPath = UClass::TryConvertShortTypeNameToPathName<UClass>(IncludeClassString);
			}
			if (!ClassPath.IsValid())
			{
				UE_LOGF(LogCook, Error, "%ls: Could not convert string '%ls' into a class path. Ignoring it.",
					Parameter, *IncludeClassString);
				continue;
			}
			UClass* IncludedClass = FindObject<UClass>(nullptr, *ClassPath.ToString());
			if (!IncludedClass)
			{
				UE_LOGF(LogCook, Error, "%ls: Could not find class with ClassPath '%ls'. Ignoring it.",
					Parameter, *IncludeClassString);
				continue;
			}
			FTopLevelAssetPath NormalizedClassPath(IncludedClass);
			RootNames.Add(NormalizedClassPath);
			OutFilterClasses.Add(FName(*NormalizedClassPath.ToString()));
		}
		if (!RootNames.IsEmpty())
		{
			TSet<FTopLevelAssetPath> DerivedClassNames;
			AssetRegistry->GetDerivedClassNames(RootNames, TSet<FTopLevelAssetPath>() /* ExcludedClassNames */,
				DerivedClassNames);
			for (const FTopLevelAssetPath& NormalizedClassPath : DerivedClassNames)
			{
				OutFilterClasses.Add(FName(*NormalizedClassPath.ToString()));
			}

			UE_LOGF(LogCook, Display, "%ls: Only cooking packages that %ls with class in { %ls }",
				Parameter, Message, *TStringBuilder<256>().Join(RootNames, TEXTVIEW(", ")));
			bCookFilter = true;
		}
	}
}

bool UCookOnTheFlyServer::TryInitializeCookWorker()
{
	using namespace UE::Cook;

	FDirectorConnectionInfo ConnectInfo;
	if (!ConnectInfo.TryParseCommandLine())
	{
		return false;
	}
	CookWorkerClient = MakeUnique<FCookWorkerClient>(*this);
	TUniquePtr<FWorkerRequestsRemote> RemoteTasks = MakeUnique<FWorkerRequestsRemote>(*this);
	if (!CookWorkerClient->TryConnect(MoveTemp(ConnectInfo)))
	{
		return false;
	}
	WorkerRequests.Reset(RemoteTasks.Release());
	Initialize(ECookMode::CookWorker, CookWorkerClient->GetCookInitializationFlags(), FString());
	StartCookAsCookWorker();
	return true;
}

void UCookOnTheFlyServer::InitializeSession()
{
	UE_SCOPED_COOKTIMER(InitializeSession);
	if (!bFirstCookInThisProcessInitialized)
	{
		// This is the first cook; set bFirstCookInThisProcess=true for the entire cook until SetBeginCookConfigSettings is called to mark the second cook
		bFirstCookInThisProcessInitialized = true;
		bFirstCookInThisProcess = true;
	}
	else
	{
		// We have cooked before; set bFirstCookInThisProcess=false
		bFirstCookInThisProcess = false;
	}

	if (bFirstCookInThisProcess)
	{
		InitializeAtFirstSession();
	}

	NumObjectsHistory.Initialize(GUObjectArray.GetObjectArrayNumMinusAvailable());
	VirtualMemoryHistory.Initialize(FPlatformMemory::GetStats().UsedVirtual);
	SetCookPhase(UE::Cook::ECookPhase::Cook);
	InitialRequestCount = 0;
}

bool UCookOnTheFlyServer::Exec_Editor(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("package")))
	{
		FString PackageName;
		if (!FParse::Value(Cmd, TEXT("name="), PackageName))
		{
			Ar.Logf(TEXT("Required package name for cook package function. \"cook package name=<name> platform=<platform>\""));
			return true;
		}

		FString PlatformName;
		if (!FParse::Value(Cmd, TEXT("platform="), PlatformName))
		{
			Ar.Logf(TEXT("Required package name for cook package function. \"cook package name=<name> platform=<platform>\""));
			return true;
		}

		if (FPackageName::IsShortPackageName(PackageName))
		{
			TArray<FName> LongPackageNames;
			AssetRegistry->GetPackagesByName(PackageName, LongPackageNames);
			if (LongPackageNames.IsEmpty())
			{
				Ar.Logf(TEXT("No package found with leaf name %s."), *PackageName);
				return true;
			}
			if (LongPackageNames.Num() > 1)
			{
				Ar.Logf(TEXT("Multiple packages found with leaf name %s. Specify the full LongPackageName."), *PackageName);
				for (FName LongPackageName : LongPackageNames)
				{
					Ar.Logf(TEXT("\n\t%s"), *LongPackageName.ToString());
				}
				return true;
			}
			PackageName = LongPackageNames[0].ToString();
		}

		FName RawPackageName(*PackageName);
		TArray<FName> PackageNames;
		PackageNames.Add(RawPackageName);
		TMap<FName, UE::Cook::FInstigator> Instigators;
		Instigators.Add(RawPackageName, UE::Cook::EInstigator::ConsoleCommand);

		GenerateLongPackageNames(PackageNames, Instigators);

		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
		ITargetPlatform* TargetPlatform = TPM.FindTargetPlatform(PlatformName);
		if (TargetPlatform == nullptr)
		{
			Ar.Logf(TEXT("Target platform %s wasn't found."), *PlatformName);
			return true;
		}

		FCookByTheBookStartupOptions StartupOptions;

		StartupOptions.TargetPlatforms.Add(TargetPlatform);
		for (const FName& StandardPackageName : PackageNames)
		{
			FName PackageFileFName = PackageDatas->GetFileNameByPackageName(StandardPackageName);
			if (!PackageFileFName.IsNone())
			{
				StartupOptions.CookMaps.Add(StandardPackageName.ToString());
			}
		}
		StartupOptions.CookOptions = ECookByTheBookOptions::NoAlwaysCookMaps | ECookByTheBookOptions::NoDefaultMaps | ECookByTheBookOptions::NoGameAlwaysCookPackages | ECookByTheBookOptions::SkipSoftReferences | ECookByTheBookOptions::ForceDisableSaveGlobalShaders;
		
		StartCookByTheBook(StartupOptions);
	}
	else if (FParse::Command(&Cmd, TEXT("clearall")))
	{
		StopAndClearCookedData();
	}
	else if (FParse::Command(&Cmd, TEXT("stats")))
	{
		DumpStats();
	}

	return false;
}

UE::Cook::FInstigator UCookOnTheFlyServer::GetInstigator(FName PackageName) const
{
	return GetInstigator(PackageName, UE::Cook::EReachability::All);
}

UE::Cook::FInstigator UCookOnTheFlyServer::GetInstigator(FName PackageName, UE::Cook::EReachability Reachability) const
{
	using namespace UE::Cook;

	FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(PackageName);
	if (!PackageData)
	{
		return FInstigator(EInstigator::NotYetRequested);
	}
	return PackageData->GetInstigator(Reachability);
}

TArray<UE::Cook::FInstigator> UCookOnTheFlyServer::GetInstigatorChain(FName PackageName) const
{
	using namespace UE::Cook;
	TArray<FInstigator> Result;
	TSet<FName> NamesOnChain;
	NamesOnChain.Add(PackageName);

	for (;;)
	{
		FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(PackageName);
		if (!PackageData)
		{
			Result.Add(FInstigator(EInstigator::NotYetRequested));
			return Result;
		}
		const FInstigator& Last = Result.Add_GetRef(PackageData->GetInstigator(EReachability::All));
		bool bGetNext = false;
		switch (Last.Category)
		{
			case EInstigator::Dependency: bGetNext = true; break;
			case EInstigator::HardDependency: bGetNext = true; break;
			case EInstigator::HardEditorOnlyDependency: bGetNext = true; break;
			case EInstigator::SoftDependency: bGetNext = true; break;
			case EInstigator::Unsolicited: bGetNext = true; break;
			case EInstigator::EditorOnlyLoad: bGetNext = true; break;
			case EInstigator::SaveTimeHardDependency: bGetNext = true; break;
			case EInstigator::SaveTimeSoftDependency: bGetNext = true; break;
			case EInstigator::ForceExplorableSaveTimeSoftDependency: bGetNext = true; break;
			case EInstigator::GeneratedPackage: bGetNext = true; break;
			case EInstigator::BuildDependency: bGetNext = true; break;
			default: break;
		}
		if (!bGetNext)
		{
			return Result;
		}
		PackageName = Last.Referencer;
		if (PackageName.IsNone())
		{
			return Result;
		}
		bool bAlreadyExists = false;
		NamesOnChain.Add(PackageName, &bAlreadyExists);
		if (bAlreadyExists)
		{
			return Result;
		}
	}
}

UE::Cook::ECookType UCookOnTheFlyServer::GetCookType() const
{
	if (IsDirectorCookByTheBook())
	{
		return UE::Cook::ECookType::ByTheBook;
	}
	else
	{
		check(this->IsDirectorCookOnTheFly());
		return UE::Cook::ECookType::OnTheFly;
	}
}

UE::Cook::ECookingDLC UCookOnTheFlyServer::GetCookingDLC() const
{
	using namespace UE::Cook;
	if (!IsCookingDLC())
	{
		return ECookingDLC::No;
	}
	if (bCookingDLCBaseGamePlugin)
	{
		// Temporary solution to support ExperimentalFeature MultiProductBuild: separate cooks that will be welded
		// together into the basegame BuildLayer should not be treated as the legacy idea of DLC. We plan to change
		// this later into a new concept - BuildLayers - and replace the ECookingDLC API with functions that read
		// properties about the BuildLayer being cooked from project-specific code in AssetManager.
		return ECookingDLC::No;
	}
	return ECookingDLC::Yes;
}

FString UCookOnTheFlyServer::GetDLCName() const
{
	return CookByTheBookOptions->DlcName;
}

FString UCookOnTheFlyServer::GetBasedOnReleaseVersion() const
{
	if (!IsCookingDLC() && CookByTheBookOptions->CreateReleaseVersion.IsEmpty())
	{
		return FString();
	}
	return CookByTheBookOptions->BasedOnReleaseVersion;
}

FString UCookOnTheFlyServer::GetCreateReleaseVersion() const
{
	return CookByTheBookOptions->CreateReleaseVersion;
}

UE::Cook::EProcessType UCookOnTheFlyServer::GetProcessType() const
{
	using namespace UE::Cook;
	if (IsCookWorkerMode())
	{
		return EProcessType::Worker;
	}
	else if (CookDirector)
	{
		return EProcessType::Director;
	}
	else
	{
		return EProcessType::SingleProcess;
	}
}

UE::Cook::ECookValidationOptions UCookOnTheFlyServer::GetCookValidationOptions() const
{
	using namespace UE::Cook;
	ECookValidationOptions ValidationOptions = ECookValidationOptions::None;
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::RunAssetValidation))
	{
		ValidationOptions |= ECookValidationOptions::RunAssetValidation;
	}
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::RunMapValidation))
	{
		ValidationOptions |= ECookValidationOptions::RunMapValidation;
	}
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::ValidationErrorsAreFatal))
	{
		ValidationOptions |= ECookValidationOptions::ValidationErrorsAreFatal;
	}
	return ValidationOptions;
}

bool UCookOnTheFlyServer::IsIncremental() const
{
	// TODO: For simplicity, we provide a single bool for all platforms in multiprocess cooks
	// But it is not currently guaranteed that they all have the same value; add enforcement of
	// that in CookByTheBookStarted.
	if (!PlatformManager || PlatformManager->GetNumSessionPlatforms() == 0)
	{
		return false;
	}
	const ITargetPlatform* TargetPlatform = PlatformManager->GetSessionPlatforms()[0];
	UE::Cook::FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
	return !PlatformData->bFullBuild;
}

TArray<const ITargetPlatform*> UCookOnTheFlyServer::GetSessionPlatforms() const
{
	return PlatformManager ? PlatformManager->GetSessionPlatforms() : TArray<const ITargetPlatform*>();
}

FString UCookOnTheFlyServer::GetCookOutputFolder(const ITargetPlatform* TargetPlatform) const
{
	if (!SandboxFile || !PlatformManager || !TargetPlatform)
	{
		return FString();
	}
	if (!PlatformManager->HasSessionPlatform(TargetPlatform))
	{
		return FString();
	}
	FString Result = SandboxFile->GetSandboxDirectory(TargetPlatform->PlatformName());
	FPaths::MakeStandardFilename(Result);
	return Result;
}

FString UCookOnTheFlyServer::GetCookMetadataOutputFolderInternal(const ITargetPlatform* TargetPlatform) const
{
	if (!SandboxFile || !TargetPlatform)
	{
		return FString();
	}

	return ConvertToFullSandboxPath(GetMetadataDirectory(), true, TargetPlatform->PlatformName());
}

FString UCookOnTheFlyServer::GetCookMetadataOutputFolder(const ITargetPlatform* TargetPlatform) const
{
	if (!PlatformManager || !PlatformManager->HasSessionPlatform(TargetPlatform))
	{
		return FString();
	}

	return GetCookMetadataOutputFolderInternal(TargetPlatform);
}

void UCookOnTheFlyServer::RegisterCollector(UE::Cook::IMPCollector* Collector, UE::Cook::EProcessType ProcessType)
{
	using namespace UE::Cook;

	TRefCountPtr<IMPCollector> DeleteIfUnusedAndCallerHasNoReference(Collector);
	if (CookDirector)
	{
		if (ProcessType == EProcessType::Director || ProcessType == EProcessType::AllMPCook)
		{
			CookDirector->Register(Collector);
		}
	}
	else if (CookWorkerClient)
	{
		if (ProcessType == EProcessType::Worker || ProcessType == EProcessType::AllMPCook)
		{
			CookWorkerClient->Register(Collector);
		}
	}
}

void UCookOnTheFlyServer::UnregisterCollector(UE::Cook::IMPCollector* Collector)
{
	TRefCountPtr<UE::Cook::IMPCollector> DeleteIfCallerHasNoReference(Collector);
	if (CookDirector)
	{
		CookDirector->Unregister(Collector);
	}
	else if (CookWorkerClient)
	{
		CookWorkerClient->Unregister(Collector);
	}
}

void UCookOnTheFlyServer::RegisterArtifact(UE::Cook::ICookArtifact* Artifact)
{
	using namespace UE::Cook;
	if (!Artifact)
	{
		return;
	}

	TRefCountPtr<UE::Cook::ICookArtifact> DeleteIfCallerHasNoReference(Artifact);
	if (IsCookWorkerMode() && !Artifact->RegisterOnCookWorker())
	{
		// Artifacts need to specify if used on CookWorkers
		return;
	}

	FString ArtifactName = Artifact->GetArtifactName();
	if (ArtifactName.IsEmpty())
	{
		UE_LOGF(LogCook, Error, "RegisterArtifact called with empty GetArtifactName(), which is invalid. Ignoring the Artifact.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		return;
	}

	for (TRefCountPtr<ICookArtifact>& Existing : CookArtifacts)
	{
		if (Existing->GetArtifactName() == ArtifactName)
		{
			if (Existing != Artifact)
			{
				UE_LOGF(LogCook, Error,
					"RegisterArtifact Called with already existing ArtifactName \"%ls\". ArtifactNames must be unique. Ignoring the artifact.",
					*ArtifactName);
				FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
			}
			return;
		}
	}
	CookArtifacts.Add(Artifact);
}

void UCookOnTheFlyServer::UnregisterArtifact(UE::Cook::ICookArtifact* Artifact)
{
	if (!Artifact)
	{
		return;
	}
	TRefCountPtr<UE::Cook::ICookArtifact> RefCountPtr(Artifact);
	CookArtifacts.Remove(RefCountPtr);
}

void UCookOnTheFlyServer::GetCulturesToCook(TArray<FString>& OutCulturesToCook) const
{
	if (CookByTheBookOptions)
	{
		OutCulturesToCook.Append(CookByTheBookOptions->AllCulturesToCook);
	}
}

EPackageWriterResult UCookOnTheFlyServer::CookerBeginCacheForCookedPlatformData(
	UE::PackageWriter::Private::FBeginCacheForCookedPlatformDataInfo& Info)
{
	return SavePackageBeginCacheForCookedPlatformData(Info.PackageName,
		Info.TargetPlatform, Info.SaveableObjects, Info.SaveFlags);
}

void UCookOnTheFlyServer::RegisterDeterminismHelper(ICookedPackageWriter* PackageWriter,
	UObject* SourceObject, const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	if (!IsDeterminismDebug())
	{
		return;
	}
	UE::Cook::FCookSavePackageContext* PackageWriterContext = nullptr;
	for (UE::Cook::FCookSavePackageContext* Context : SavePackageContexts)
	{
		if (Context->PackageWriter == PackageWriter)
		{
			PackageWriterContext = Context;
			break;
		}
	}
	if (!PackageWriterContext || !PackageWriterContext->DeterminismManager)
	{
		return;
	}
	PackageWriterContext->DeterminismManager->RegisterDeterminismHelper(SourceObject, DeterminismHelper);
}

bool UCookOnTheFlyServer::IsDeterminismDebug() const
{
	return bDeterminismDebug || DiffModeHelper->IsDeterminismDebug();
}

void UCookOnTheFlyServer::WriteFileOnCookDirector(const UE::PackageWriter::Private::FWriteFileData& FileData,
	FMD5& AccumulatedHash, const TRefCountPtr<FPackageHashes>& PackageHashes,
	IPackageWriter::EWriteOptions WriteOptions)
{
	// This function can be called from Async threads, so we can only call cooker functions that are threadsafe;
	// many cooker functions are read/write only from scheduler thread.
	if (CookWorkerClient || CookDirector)
	{
		FString CookRoot;
		FStringView RelPath;
		const ITargetPlatform* TargetPlatform = nullptr;
		FString TargetFileNameAbsPath = FPaths::ConvertRelativePathToFull(FileData.Filename);

		{
			UE::Cook::FPlatformManager::FReadScopeLock PlatformScopeLock(PlatformManager->ReadLockPlatforms());
			PlatformManager->EnumerateSessionPlatforms(
				[&CookRoot, &RelPath, &TargetPlatform, &TargetFileNameAbsPath, this]
				(const ITargetPlatform* IterTargetPlatform)
				{
					if (!TargetPlatform)
					{
						CookRoot = FPaths::ConvertRelativePathToFull(GetCookOutputFolder(IterTargetPlatform));
						if (FPathViews::TryMakeChildPathRelativeTo(TargetFileNameAbsPath, CookRoot, RelPath))
						{
							TargetPlatform = IterTargetPlatform;
						}
					}
				});
		}

		if (!TargetPlatform)
		{
			UE_LOGF(LogCook, Error,
				"Output file requested during cooking from CookAdditionalFilesOverride was invalid. " "Output files must be under the platform's cooked root output, such as %ls. " "File %ls is not under the cooked root for any platform being cooked. The file will be missing from the cooked output.",
				!CookRoot.IsEmpty() ? *CookRoot : TEXT("<UnknownBecauseNoSessionPlatforms>"),
				*FileData.Filename);
			return;
		}
		FString FileTransferRoot = FPaths::Combine(CookRoot, UE::Cook::MPCookTransferRootRelPath);
		FString TempFileName = FPaths::CreateTempFilename(*FileTransferRoot);

		UE::PackageWriter::Private::FWriteFileData TempFileData = FileData;
		TempFileData.Filename = TempFileName;
		UE::PackageWriter::Private::HashAndWrite(TempFileData, AccumulatedHash, PackageHashes, WriteOptions);
		if (CookWorkerClient)
		{
			CookWorkerClient->ReportFileTransfer(TempFileName, FileData.Filename);
		}
		else
		{
			CookDirector->ReportFileTransferFromLocalWorker(TempFileName, FileData.Filename);
		}
	}
	else
	{
		UE::PackageWriter::Private::HashAndWrite(FileData, AccumulatedHash, PackageHashes, WriteOptions);
	}
}

UE::Cook::ICookSandbox* UCookOnTheFlyServer::GetCookSandbox()
{
	return SandboxFile.Get();
}

void UCookOnTheFlyServer::DumpStats()
{
	UE_LOGF(LogCook, Display, "IntStats:");
	UE_LOGF(LogCook, Display, "  %ls=%d", TEXT("LoadPackage"), this->StatLoadedPackageCount);
	UE_LOGF(LogCook, Display, "  %ls=%d", TEXT("SavedPackage"), this->StatSavedPackageCount);

	OutputHierarchyTimers();
#if PROFILE_NETWORK
	UE_LOGF(LogCook, Display, "Network Stats \n"
		"TimeTillRequestStarted %f\n"
		"TimeTillRequestForfilled %f\n"
		"TimeTillRequestForfilledError %f\n"
		"WaitForAsyncFilesWrites %f\n",
		TimeTillRequestStarted,
		TimeTillRequestForfilled,
		TimeTillRequestForfilledError,

		WaitForAsyncFilesWrites);
#endif
}

uint32 UCookOnTheFlyServer::NumConnections() const
{
	int Result= 0;
	for ( int i = 0; i < NetworkFileServers.Num(); ++i )
	{
		INetworkFileServer *NetworkFileServer = NetworkFileServers[i];
		if ( NetworkFileServer )
		{
			Result += NetworkFileServer->NumConnections();
		}
	}
	return Result;
}

FString UCookOnTheFlyServer::GetOutputDirectoryOverride(FBeginCookContext& BeginContext) const
{
	FString OutputDirectory = OutputDirectoryOverride;
	// Output directory override.	
	if (OutputDirectory.Len() <= 0)
	{
		if ( IsCookingDLC() )
		{
			check( !IsDirectorCookOnTheFly() );
			OutputDirectory = FPaths::Combine(*GetBaseDirectoryForDLC(), TEXT("Saved"), TEXT("Cooked"), TEXT("[Platform]"));
		}
		else if ( IsCookingInEditor() )
		{
			// Full path so that the sandbox wrapper doesn't try to re-base it under Sandboxes
			OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("EditorCooked"), TEXT("[Platform]"));
		}
		else
		{
			// Full path so that the sandbox wrapper doesn't try to re-base it under Sandboxes
			OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), TEXT("[Platform]"));
		}
		
		OutputDirectory = FPaths::ConvertRelativePathToFull(OutputDirectory);
	}
	else if (!OutputDirectory.Contains(TEXT("[Platform]"), ESearchCase::IgnoreCase, ESearchDir::FromEnd) )
	{
		// Output directory needs to contain [Platform] token to be able to cook for multiple targets.
		if ( !IsDirectorCookOnTheFly() )
		{
			checkf(BeginContext.TargetPlatforms.Num() == 1,
				TEXT("If OutputDirectoryOverride is provided when cooking multiple platforms, it must include [Platform] in the text, to be replaced with the name of each of the requested Platforms.") );
		}
		else
		{
			// In cook on the fly mode we always add a [Platform] subdirectory rather than requiring the command-line user to include it in their path it because we assume they 
			// don't know which platforms they are cooking for up front
			OutputDirectory = FPaths::Combine(*OutputDirectory, TEXT("[Platform]"));
		}
	}
	FPaths::NormalizeDirectoryName(OutputDirectory);

	return OutputDirectory;
}

template<class T>
void GetVersionFormatNumbersForIniVersionStrings( TArray<FString>& IniVersionStrings, const FString& FormatName, const TArray<const T> &FormatArray )
{
	for ( const T& Format : FormatArray )
	{
		TArray<FName> SupportedFormats;
		Format->GetSupportedFormats(SupportedFormats);
		for ( const FName& SupportedFormat : SupportedFormats )
		{
			int32 VersionNumber = Format->GetVersion(SupportedFormat);
			FString IniVersionString = FString::Printf( TEXT("%s:%s:VersionNumber%d"), *FormatName, *SupportedFormat.ToString(), VersionNumber);
			IniVersionStrings.Emplace( IniVersionString );
		}
	}
}




template<class T>
void GetVersionFormatNumbersForIniVersionStrings(TMap<FString, FString>& IniVersionMap, const FString& FormatName, const TArray<T> &FormatArray)
{
	for (const T& Format : FormatArray)
	{
		TArray<FName> SupportedFormats;
		Format->GetSupportedFormats(SupportedFormats);
		for (const FName& SupportedFormat : SupportedFormats)
		{
			int32 VersionNumber = Format->GetVersion(SupportedFormat);
			FString IniVersionString = FString::Printf(TEXT("%s:%s:VersionNumber"), *FormatName, *SupportedFormat.ToString());
			IniVersionMap.Add(IniVersionString, FString::Printf(TEXT("%d"), VersionNumber));
		}
	}
}


void GetAdditionalCurrentIniVersionStrings( const UCookOnTheFlyServer* CookOnTheFlyServer, const ITargetPlatform* TargetPlatform, TMap<FString, FString>& IniVersionMap )
{
	FConfigFile EngineSettings;
	FConfigCacheIni::LoadLocalIniFile(EngineSettings, TEXT("Engine"), true, *TargetPlatform->IniPlatformName());

	TArray<FString> VersionedRValues;
	EngineSettings.GetArray(TEXT("/Script/UnrealEd.CookerSettings"), TEXT("VersionedIntRValues"), VersionedRValues);

	for (const FString& RValue : VersionedRValues)
	{
		const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(*RValue);
		if (CVar)
		{
			IniVersionMap.Add(*RValue, FString::Printf(TEXT("%d"), CVar->GetValueOnGameThread()));
		}
	}

	// save off the ddc version numbers also
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	check(TPM);

	{
		TArray<FName> AllWaveFormatNames;
		TargetPlatform->GetAllWaveFormats(AllWaveFormatNames);
		TArray<const IAudioFormat*> SupportedWaveFormats;
		for ( const auto& WaveName : AllWaveFormatNames )
		{
			const IAudioFormat* AudioFormat = TPM->FindAudioFormat(WaveName);
			if (AudioFormat)
			{
				SupportedWaveFormats.Add(AudioFormat);
			}
			else
			{
				UE_LOGF(LogCook, Warning, "Unable to find audio format \"%ls\" which is required by \"%ls\"", *WaveName.ToString(), *TargetPlatform->PlatformName());
			}
			
		}
		GetVersionFormatNumbersForIniVersionStrings(IniVersionMap, TEXT("AudioFormat"), SupportedWaveFormats);
	}

	{
		// GetModule , not Load.  Should be loaded by TargetPlatform or Program startup on main thread.
		//  cannot load modules from threads so must be done before any thread needs this module
		ITextureFormatManagerModule* TFMM = FModuleManager::GetModulePtr<ITextureFormatManagerModule>("TextureFormat");
		if ( TFMM == nullptr )
		{
			UE_LOGF(LogInit, Error, "Texture format manager Ptr was requested but not available.");
		}

		//note: this gets All ITextureFormat modules in the Engine, not just ones relevant to this TargetPlatform
		const TArray<const ITextureFormat*> & SupportedTextureFormats = TFMM->GetAllTextureFormats();
		
		GetVersionFormatNumbersForIniVersionStrings(IniVersionMap, TEXT("TextureFormat"), SupportedTextureFormats);
	}

	if (AllowShaderCompiling())
	{
		TArray<FName> AllFormatNames;
		TargetPlatform->GetAllTargetedShaderFormats(AllFormatNames);
		TArray<const IShaderFormat*> SupportedFormats;
		for (const auto& FormatName : AllFormatNames)
		{
			const IShaderFormat* Format = TPM->FindShaderFormat(FormatName);
			if ( Format )
			{
				SupportedFormats.Add(Format);
			}
			else
			{
				UE_LOGF(LogCook, Warning, "Unable to find shader \"%ls\" which is required by format \"%ls\"", *FormatName.ToString(), *TargetPlatform->PlatformName());
			}
		}
		GetVersionFormatNumbersForIniVersionStrings(IniVersionMap, TEXT("ShaderFormat"), SupportedFormats);
	}


	// TODO: Add support for physx version tracking, currently this happens so infrequently that invalidating a cook based on it is not essential
	//GetVersionFormatNumbersForIniVersionStrings(IniVersionMap, TEXT("PhysXCooking"), TPM->GetPhysXCooking());


	if ( FParse::Param( FCommandLine::Get(), TEXT("fastcook") ) )
	{
		IniVersionMap.Add(TEXT("fastcook"));
	}

	FCustomVersionContainer AllCurrentVersions = FCurrentCustomVersions::GetAll();
	for (const FCustomVersion& CustomVersion : AllCurrentVersions.GetAllVersions())
	{
		FString CustomVersionString = FString::Printf(TEXT("%s:%s"), *CustomVersion.GetFriendlyName().ToString(), *CustomVersion.Key.ToString());
		FString CustomVersionValue = FString::Printf(TEXT("%d"), CustomVersion.Version);
		IniVersionMap.Add(CustomVersionString, CustomVersionValue);
	}

	IniVersionMap.Add(TEXT("PackageFileVersionUE4"), FString::Printf(TEXT("%d"), GPackageFileUEVersion.FileVersionUE4));
	IniVersionMap.Add(TEXT("PackageFileVersionUE5"), FString::Printf(TEXT("%d"), GPackageFileUEVersion.FileVersionUE5));
	IniVersionMap.Add(TEXT("PackageLicenseeVersion"), FString::Printf(TEXT("%d"), GPackageFileLicenseeUEVersion));

	/*FString UE4EngineVersionCompatibleName = TEXT("EngineVersionCompatibleWith");
	FString UE4EngineVersionCompatible = FEngineVersion::CompatibleWith().ToString();
	
	if ( UE4EngineVersionCompatible.Len() )
	{
		IniVersionMap.Add(UE4EngineVersionCompatibleName, UE4EngineVersionCompatible);
	}*/

	IniVersionMap.Add(TEXT("MaterialShaderMapDDCVersion"), *GetMaterialShaderMapDDCGuid().ToString());
	IniVersionMap.Add(TEXT("GlobalDDCVersion"), *GetGlobalShaderMapDDCGuid().ToString());

	UProjectPackagingSettings* PackagingSettings = Cast<UProjectPackagingSettings>(UProjectPackagingSettings::StaticClass()->GetDefaultObject());
	IniVersionMap.Add(TEXT("IsUsingShaderCodeLibrary"), FString::Printf(TEXT("%d"), CookOnTheFlyServer->IsUsingShaderCodeLibrary()));
}

bool UCookOnTheFlyServer::GetCurrentIniVersionStrings( const ITargetPlatform* TargetPlatform, UE::Cook::FIniSettingContainer& IniVersionStrings ) const
{
#if !UE_WITH_CONFIG_TRACKING
	IniVersionStrings.Reset();
	return true;
#else
	using namespace UE::ConfigAccessTracking;

	// This function should be called after the cook is finished
	TArray<UE::ConfigAccessTracking::FConfigAccessData> AccessedRecordsArray =
		UE::ConfigAccessTracking::FCookConfigAccessTracker::Get().GetCookRecords(TargetPlatform);
	TConstArrayView<UE::ConfigAccessTracking::FConfigAccessData> AccessedRecords(AccessedRecordsArray);

	TArray<const FConfigValue*> Values;
	int32 ConfigFileEndIndex = 0;
	int32 EndIndex = AccessedRecords.Num();
	TStringBuilder<128> FullConfigFileNameStr;
	while (ConfigFileEndIndex < EndIndex)
	{
		int32 ConfigFileStartIndex = ConfigFileEndIndex;
		const FConfigAccessData& FileStartRecord = AccessedRecords[ConfigFileStartIndex];
		++ConfigFileEndIndex;
		while (ConfigFileEndIndex < EndIndex && AccessedRecords[ConfigFileEndIndex].IsSameConfigFile(FileStartRecord))
		{
			++ConfigFileEndIndex;
		}

		FConfigFile Temp;
		FName ConfigFileName(FileStartRecord.GetFileName());
		FConfigAccessData FileRecord = FileStartRecord.GetFileOnlyData();
		TStringBuilder<64> ConfigFileNameStr(InPlace, ConfigFileName);
		using UE::String::FindFirst;

		// Hardcoded additions to ConfigSettingsDenyList for ini files used by the cook. These are early-exited earlier
		// to prevent bugs from arising if we tried to track their data and discard later.
		if (FindFirst(ConfigFileNameStr, TEXT("CookedIniVersion.txt"), ESearchCase::IgnoreCase) != INDEX_NONE ||
			FindFirst(ConfigFileNameStr, TEXT("CookedSettings.txt"), ESearchCase::IgnoreCase) != INDEX_NONE)
		{
			continue;
		}

		FullConfigFileNameStr.Reset();
		FileRecord.AppendFullPath(FullConfigFileNameStr);
		FName FullConfigFileName(FullConfigFileNameStr);
		const FConfigFile* ConfigFile = UE::ConfigAccessTracking::FindOrLoadConfigFile(FileRecord, Temp);
		if (!ConfigFile)
		{
			// This is logged as Warning; it is unexpected that we were able to load a file from disk that
			// existed previously when we received the OnConfigValueRead call.
			UE_LOGF(LogCook, Display,
				"Could not load config file '%ls'. Changes to settings in this file will not be detected in legacyiterative cooks.",
				*FullConfigFileNameStr);
			continue;
		}

		TMap<FName, TMap<FName, TArray<FString>>>& FileVersionStrings = IniVersionStrings.FindOrAdd(FullConfigFileName);
		int32 ConfigSectionEndIndex = ConfigFileStartIndex;
		while (ConfigSectionEndIndex < ConfigFileEndIndex)
		{
			int32 ConfigSectionStartIndex = ConfigSectionEndIndex;
			++ConfigSectionEndIndex;
			const UE::ConfigAccessTracking::FConfigAccessData& SectionStartRecord = AccessedRecords[ConfigSectionStartIndex];
			while (ConfigSectionEndIndex < ConfigFileEndIndex && AccessedRecords[ConfigSectionEndIndex].SectionName == SectionStartRecord.SectionName)
			{
				++ConfigSectionEndIndex;
			}

			FName SectionName(SectionStartRecord.GetSectionName());
			const FConfigSection* ConfigSection = ConfigFile->FindSection(SectionName.ToString());
 			if (!ConfigSection)
			{
				// This is logged as Verbose rather than Warning because the section could have been added by code
				// after loading and never existed on disk.
				UE_LOGF(LogCook, Verbose,
					"Could not find config section %ls:[%ls]. Changes to settings in this section will not be detected in legacyiterative cooks.",
					*FullConfigFileNameStr, *WriteToString<32>(SectionName));
				continue;
			}
			TMap<FName, TArray<FString>>& SectionVersionStrings = FileVersionStrings.FindOrAdd(SectionName);

			for (const UE::ConfigAccessTracking::FConfigAccessData& Record :
				AccessedRecords.Slice(ConfigSectionStartIndex, ConfigSectionEndIndex - ConfigSectionStartIndex))
			{
				FName ValueName(Record.GetValueName());
				Values.Reset();
				ConfigSection->MultiFindPointer(ValueName, Values, true /* bMaintainOrder */);
				if (Values.IsEmpty())
				{
					// This is logged as Verbose rather than Warning because the value could have been added by code
					// after loading and never existed on disk.
					UE_LOGF(LogCook, Verbose,
						"Could not find config value %ls:[%ls]:%ls. Changes to this value will not be detected in legacyiterative cooks.",
						*FullConfigFileNameStr, *WriteToString<32>(SectionName), *WriteToString<32>(ValueName));
					continue;
				}
				TArray<FString>& ValueVersionStrings = SectionVersionStrings.FindOrAdd(ValueName);
				for (const FConfigValue* Value : Values)
				{
					FString ValueStr = Value->GetSavedValue();
					ValueStr.ReplaceInline(TEXT(":"), TEXT(""));
					ValueVersionStrings.Add(MoveTemp(ValueStr));
				}
			}
		}
	}

	// remove any ConfigFiles,Sections,Values which are marked as ignored by ConfigSettingDenyList
	struct FParsedDenyEntry
	{
		FStringView ConfigFileName;
		FStringView SectionName;
		FStringView ValueName;
	};
	TArray<FParsedDenyEntry> ParsedConfigSettings;
	TArray<FStringView> Tokens;
	for (const FString& Filter : ConfigSettingDenyList)
	{
		Tokens.Reset();
		UE::String::ParseTokens(Filter, ':', Tokens, UE::String::EParseTokensOptions::Trim | UE::String::EParseTokensOptions::SkipEmpty);
		if (Tokens.Num() >= 1)
		{
			FParsedDenyEntry& DenyEntry = ParsedConfigSettings.Emplace_GetRef();
			DenyEntry.ConfigFileName = Tokens[0];
			if (Tokens.Num() >= 2) DenyEntry.SectionName = Tokens[1];
			if (Tokens.Num() >= 3) DenyEntry.ValueName = Tokens[2];
		}
	}

	TStringBuilder<128> FullConfigFileName;
	for (UE::Cook::FIniSettingContainer::TIterator ConfigFile(IniVersionStrings.CreateIterator()); ConfigFile; ++ConfigFile)
	{
		ConfigFile.Key().ToString(FullConfigFileName);
		FStringView FullConfigFileNameView(FullConfigFileName);
		int32 DotIndex = FullConfigFileNameView.Find(TEXTVIEW("."));
		FString Platform = FString(FullConfigFileNameView.LeftChop(DotIndex));
		FString PlatformAndFileName = FString(FullConfigFileNameView.RightChop(DotIndex + 1));
		FString ConfigFileName = PlatformAndFileName.RightChop(PlatformAndFileName.Find(TEXTVIEW(".")) + 1);
		FString BaseFileName = FPaths::GetBaseFilename(ConfigFileName);
		FString PlatformAndBaseFileName = FString::Printf(TEXT("%s.%s"), *Platform, *BaseFileName);

		for (const FParsedDenyEntry& DenyEntry : ParsedConfigSettings)
		{
			// FullConfigFileName is of the form "LoadType.Platform.ConfigFile".
			// Wildcards are written in the form "*.ConfigFile" or "ConfigFile".
			// We allow a match of the wildcard against either Platform.ConfigFile or just ConfigFile.
			// We also allow a match of the wildcard against Platform.BaseName or BaseName.
			if (PlatformAndFileName.MatchesWildcard(DenyEntry.ConfigFileName) ||
				ConfigFileName.MatchesWildcard(DenyEntry.ConfigFileName) ||
				PlatformAndBaseFileName.MatchesWildcard(DenyEntry.ConfigFileName) ||
				BaseFileName.MatchesWildcard(DenyEntry.ConfigFileName))
			{
				if (!DenyEntry.SectionName.IsEmpty())
				{
					for (TMap<FName,TMap<FName,TArray<FString>>>::TIterator Section(ConfigFile.Value().CreateIterator());
						Section; ++Section)
					{
						if (Section.Key().ToString().MatchesWildcard(DenyEntry.SectionName))
						{
							if (!DenyEntry.ValueName.IsEmpty())
							{
								for (TMap<FName, TArray<FString>>::TIterator Value(Section.Value().CreateIterator()); Value; ++Value)
								{
									if (Value.Key().ToString().MatchesWildcard(DenyEntry.ValueName))
									{
										Value.RemoveCurrent();
									}
								}
							}
							else
							{
								Section.RemoveCurrent();
							}
						}
					}
				}
				else
				{
					ConfigFile.RemoveCurrent();
					break;
				}
			}
		}
	}
	return true;
#endif // !UE_WITH_CONFIG_TRACKING
}

bool UCookOnTheFlyServer::GetCookedIniVersionStrings(const ITargetPlatform* TargetPlatform, UE::Cook::FIniSettingContainer& OutIniSettings, TMap<FString,FString>& OutAdditionalSettings) const
{
	const FString PlatformSandboxEditorIni = GetCookMetadataOutputFolderInternal(TargetPlatform) / TEXT("CookedIniVersion.txt"); 

	FConfigFile ConfigFile;
	ConfigFile.Read(*PlatformSandboxEditorIni);

	const static FString NAME_UsedSettings(TEXT("UsedSettings"));
	const FConfigSection* UsedSettings = ConfigFile.FindSection(NAME_UsedSettings);
	if (UsedSettings == nullptr)
	{
		return false;
	}

	const static FString NAME_AdditionalSettings(TEXT("AdditionalSettings"));
	const FConfigSection* AdditionalSettings = ConfigFile.FindSection(NAME_AdditionalSettings);
	if (AdditionalSettings == nullptr)
	{
		return false;
	}

	TStringBuilder<256> KeyStr;
	TStringBuilder<128> Filename;
	TStringBuilder<64> SectionName;
	TStringBuilder<64> ValueName;
	TStringBuilder<64> ValueIndexStr;
	FStringBuilderBase* TokenBuffer[] = { &Filename, &SectionName, &ValueName, &ValueIndexStr };
	TArrayView<FStringBuilderBase*> Tokens = TokenBuffer;

	using namespace UE::ConfigAccessTracking;
	for (const auto& UsedSetting : *UsedSettings )
	{
		KeyStr.Reset();
		KeyStr << UsedSetting.Key;
		if (!TryTokenizeConfigTrackingString(KeyStr, Tokens))
		{
			UE_LOGF(LogCook, Warning, "Found unparsable ini setting %ls for platform %ls, invalidating cook.",
				*KeyStr, *TargetPlatform->PlatformName());
			return false;
		}

		auto& OutFile = OutIniSettings.FindOrAdd(FName(Filename));
		auto& OutSection = OutFile.FindOrAdd(FName(SectionName));
		auto& ValueArray = OutSection.FindOrAdd(FName(ValueName));
		const int32 ValueIndex = FCString::Atoi(*ValueIndexStr);
		if ( ValueArray.Num() < (ValueIndex+1) )
		{
			ValueArray.AddZeroed( ValueIndex - ValueArray.Num() +1 );
		}
		ValueArray[ValueIndex] = UsedSetting.Value.GetSavedValue();
	}

	for (const auto& AdditionalSetting : *AdditionalSettings)
	{
		const FName& Key = AdditionalSetting.Key;
		const FString& Value = AdditionalSetting.Value.GetSavedValue();
		OutAdditionalSettings.Add(Key.ToString(), Value);
	}

	return true;
}

FString UCookOnTheFlyServer::GetCookSettingsFileName(const ITargetPlatform* TargetPlatform,
	const FString& ArtifactName) const
{
	check(!ArtifactName.IsEmpty());
	FString LeafName = ArtifactName != TEXT("global")
		? FString::Printf(TEXT("CookedSettings_%s.txt"), *ArtifactName)
		: TEXT("CookedSettings.txt");
	return GetCookMetadataOutputFolderInternal(TargetPlatform) / LeafName;
}

FConfigFile UCookOnTheFlyServer::LoadCookSettings(const ITargetPlatform* TargetPlatform, const FString& ArtifactName)
{
	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;
	FConfigFile Result;
	FString CookSettingsFileName = GetCookSettingsFileName(TargetPlatform, ArtifactName);
	TUniquePtr<FArchive> Reader;
	Reader.Reset(FindOrCreateCookArtifactReader(TargetPlatform).CreateFileReader(*CookSettingsFileName));
	if (Reader)
	{
		FString CookSettingsFileContents;
		if (FFileHelper::LoadFileToString(CookSettingsFileContents, *Reader.Get()))
		{
			Result.ProcessInputFileContents(CookSettingsFileContents, CookSettingsFileName);
		}
	}
	return Result;
}

void UCookOnTheFlyServer::SaveCookSettings(const FConfigFile& ConfigFile, const ITargetPlatform* TargetPlatform,
	const FString& ArtifactName) const
{
	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;
	FString SettingsFileName = GetCookSettingsFileName(TargetPlatform, ArtifactName);

	if (ConfigFile.IsEmpty())
	{
		IFileManager::Get().Delete(*SettingsFileName, false /* bRequireExists */);
	}
	else
	{
		// Use ConfigFile.WriteToString instead of Write because Write applies suppressions specific to GConfig and Dirty.
		FString Text;
		// TODO: Change WriteToString to be a const function.
		const_cast<FConfigFile&>(ConfigFile).WriteToString(Text);
		FFileHelper::SaveStringToFile(Text, *SettingsFileName);
	}
}

bool UCookOnTheFlyServer::IniSettingsOutOfDate(const ITargetPlatform* TargetPlatform) const
{
#if !UE_WITH_CONFIG_TRACKING
	return false;
#else
	using namespace UE::ConfigAccessTracking;

	FIgnoreScope IgnoreScope;

	UE::Cook::FIniSettingContainer OldIniSettings;
	TMap<FString, FString> OldAdditionalSettings;
	if ( GetCookedIniVersionStrings(TargetPlatform, OldIniSettings, OldAdditionalSettings) == false)
	{
		UE_LOGF(LogCook, Display, "Invalidating inisettings: Unable to read previous cook inisettings for platform %ls.", *TargetPlatform->PlatformName());
		return true;
	}

	// compare against current settings
	TMap<FString, FString> CurrentAdditionalSettings;
	GetAdditionalCurrentIniVersionStrings(this, TargetPlatform, CurrentAdditionalSettings);

	for ( const auto& OldIniSetting : OldAdditionalSettings)
	{
		const FString* CurrentValue = CurrentAdditionalSettings.Find(OldIniSetting.Key);
		if ( !CurrentValue )
		{
			UE_LOGF(LogCook, Display,
				"Invalidating inisettings: Unable to find additional ini setting used by platform %ls: %ls was not found.",
				*TargetPlatform->PlatformName(), *OldIniSetting.Key);
			return true;
		}

		if ( *CurrentValue != OldIniSetting.Value )
		{
			UE_LOGF(LogCook, Display,
				"Invalidating inisettings: Additional ini setting used by platform %ls is different for %ls, value '%ls' != '%ls'.",
				*TargetPlatform->PlatformName(), *OldIniSetting.Key, **CurrentValue, *OldIniSetting.Value );
			return true;
		}
	}

	TStringBuilder<256> ConfigNameKeyStr;
	for (const auto& OldIniFile : OldIniSettings)
	{
		OldIniFile.Key.ToString(ConfigNameKeyStr);

		FConfigAccessData FullFilePathData = FConfigAccessData::Parse(ConfigNameKeyStr);
		if (!IsLoadableLoadType(FullFilePathData.LoadType))
		{
			UE_LOGF(LogCook, Warning,
				"Invalidating inisettings: Invalid filename key in old ini settings file used by platform %ls: key '%ls' is invalid.",
				*TargetPlatform->PlatformName(), *ConfigNameKeyStr);
			return true;
		}

		FConfigFile Temp;
		const FConfigFile* ConfigFile = UE::ConfigAccessTracking::FindOrLoadConfigFile(FullFilePathData, Temp);
		if (!ConfigFile)
		{
			UE_LOGF(LogCook, Display,
				"Invalidating inisettings: Unable to find config file in old ini settings file used by platform %ls: '%ls' was not found.",
				*TargetPlatform->PlatformName(), *ConfigNameKeyStr);
			return true;
		}

		for ( const auto& OldIniSection : OldIniFile.Value )
		{
			FName SectionName = OldIniSection.Key;
			const FConfigSection* IniSection = ConfigFile->FindSection( SectionName.ToString() );
			auto GetDenyListMessageStart = [&ConfigNameKeyStr, SectionName]()
				{
					return FString::Printf(
						TEXT("To avoid invalidating due to this setting, add a deny list setting")
						TEXT("\n\tDefaultEditor.ini:[CookSettings]:+CookOnTheFlyConfigSettingDenyList=%s:%s"),
						*ConfigNameKeyStr, *SectionName.ToString());
				};

			if ( IniSection == nullptr )
			{
				UE_LOGF(LogCook, Display,
					"Invalidating inisettings: Inisetting used by platform %ls is different for %ls:[%ls]. The section doesn't exist in current config.",
					*TargetPlatform->PlatformName(), *ConfigNameKeyStr, *SectionName.ToString());
				UE_LOGF(LogCook, Display, "%ls", *GetDenyListMessageStart());
				return true;
			}

			for ( const auto& OldIniValue : OldIniSection.Value )
			{
				const FName& ValueName = OldIniValue.Key;

				TArray<FConfigValue> CurrentValues;
				IniSection->MultiFind( ValueName, CurrentValues, true );

				if ( CurrentValues.Num() != OldIniValue.Value.Num() )
				{
					UE_LOGF(LogCook, Display,
						"Invalidating inisettings: Inisetting used by platform %ls is different for %ls:[%ls]:%ls. Mismatched num array elements %d != %d.",
						*TargetPlatform->PlatformName(), *ConfigNameKeyStr, *SectionName.ToString(),
						*ValueName.ToString(), CurrentValues.Num(), OldIniValue.Value.Num());
					UE_LOGF(LogCook, Display, "%ls:%ls", *GetDenyListMessageStart(), *ValueName.ToString());
					return true;
				}
				for ( int Index = 0; Index < CurrentValues.Num(); ++Index )
				{
					const FString FilteredCurrentValue = CurrentValues[Index].GetSavedValue().Replace(TEXT(":"), TEXT(""));
					if ( FilteredCurrentValue != OldIniValue.Value[Index] )
					{
						UE_LOGF(LogCook, Display,
							"Invalidating inisettings: Inisetting used by platform %ls is different for %ls:[%ls]:%ls%ls. Value '%ls' != '%ls'.",
							*TargetPlatform->PlatformName(), *ConfigNameKeyStr, *SectionName.ToString(),
							*ValueName.ToString(),
							(CurrentValues.Num() == 1 ? TEXT("") : *FString::Printf(TEXT(" %d"), Index)),
							*CurrentValues[Index].GetSavedValue(),
							*OldIniValue.Value[Index] );
						UE_LOGF(LogCook, Display, "%ls:%ls", *GetDenyListMessageStart(), *ValueName.ToString());
						return true;
					}
				}
			}
		}
	}

	return false;
#endif // UE_WITH_CONFIG_TRACKING
}

bool UCookOnTheFlyServer::SaveCurrentIniSettings(const ITargetPlatform* TargetPlatform) const
{
	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;

	TMap<FString, FString> AdditionalIniSettings;
	GetAdditionalCurrentIniVersionStrings(this, TargetPlatform, AdditionalIniSettings);
	AdditionalIniSettings.KeySort(TLess<FString>());

	UE::Cook::FIniSettingContainer CurrentIniSettings;
	GetCurrentIniVersionStrings(TargetPlatform, CurrentIniSettings);

	const FString PlatformSandboxEditorIni = GetCookMetadataOutputFolderInternal(TargetPlatform) / TEXT("CookedIniVersion.txt");

	FConfigFile ConfigFile;
	// ConfigFile.Read(*PlatformSandboxEditorIni);

	ConfigFile.Dirty = true;
	const static TCHAR* NAME_UsedSettings =TEXT("UsedSettings");
	ConfigFile.Remove(NAME_UsedSettings);

	using namespace UE::ConfigAccessTracking;
	{
		TStringBuilder<256> NewKey;
		TStringBuilder<128> FilenameStr;
		TStringBuilder<64> SectionStr;
		TStringBuilder<64> ValueNameStr;
		UE_SCOPED_HIERARCHICAL_COOKTIMER(ProcessingAccessedStrings)
		for (const auto& CurrentIniFilename : CurrentIniSettings)
		{
			FName Filename = CurrentIniFilename.Key;
			EscapeConfigTrackingTokenToString(Filename, FilenameStr);
			for (const auto& CurrentSection : CurrentIniFilename.Value)
			{
				FName Section = CurrentSection.Key;
				EscapeConfigTrackingTokenToString(Section, SectionStr);
				for (const auto& CurrentValue : CurrentSection.Value)
				{
					FName ValueName = CurrentValue.Key;
					EscapeConfigTrackingTokenToString(ValueName, ValueNameStr);

					const TArray<FString>& Values = CurrentValue.Value;
					for (int Index = 0; Index < Values.Num(); ++Index)
					{
						NewKey.Reset();
						NewKey.Appendf(TEXT("%s:%s:%s:%d"), *FilenameStr, *SectionStr, *ValueNameStr, Index);
						ConfigFile.AddToSection(NAME_UsedSettings, FName(NewKey), Values[Index]);
					}
				}
			}
		}
	}


	const static FString NAME_AdditionalSettings(TEXT("AdditionalSettings"));
	ConfigFile.Remove(NAME_AdditionalSettings);
	for (const auto& AdditionalIniSetting : AdditionalIniSettings)
	{
		ConfigFile.AddToSection(*NAME_AdditionalSettings, FName(*AdditionalIniSetting.Key), AdditionalIniSetting.Value);
	}

	ConfigFile.Write(PlatformSandboxEditorIni);


	return true;

}

void UCookOnTheFlyServer::OnRequestClusterCompleted(const UE::Cook::FRequestCluster& RequestCluster)
{
}

FAsyncIODelete& UCookOnTheFlyServer::GetAsyncIODelete()
{
	if (AsyncIODelete)
	{
		return *AsyncIODelete;
	}

	FString SharedDeleteRoot = GetSandboxDirectory(TEXT("_Del"));
	FPaths::NormalizeDirectoryName(SharedDeleteRoot);
	AsyncIODelete = MakeUnique<FAsyncIODelete>(SharedDeleteRoot);
	return *AsyncIODelete;
}

void UCookOnTheFlyServer::PopulateCookedPackages(TArrayView<const ITargetPlatform* const> TargetPlatforms)
{
	using namespace UE::Cook;
	using EDifference = FAssetRegistryGenerator::EDifference;
	TRACE_CPUPROFILER_EVENT_SCOPE(UCookOnTheFlyServer::PopulateCookedPackages);
	checkf(!IsCookWorkerMode(), TEXT("Calling PopulateCookedPackages should be impossible in a CookWorker."));

	// TODO: NumPackagesIncrementallySkipped is only counted for the first platform; to count all platforms we would
	// have to check whether each one is already cooked.
	bool bFirstPlatform = true;
	COOK_STAT(DetailedCookStats::NumPackagesIncrementallySkipped = 0);
	COOK_STAT(DetailedCookStats::NumPackagesIncrementallyModifiedByClass.Empty());
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
		FAssetRegistryGenerator& PlatformAssetRegistry = *(PlatformData->RegistryGenerator);
		FCookSavePackageContext& CookSavePackageContext = FindOrCreateSaveContext(TargetPlatform);
		ICookedPackageWriter& PackageWriter = *CookSavePackageContext.PackageWriter;
		UE_LOGF(LogCook, Display, "Populating cooked package(s) from %ls package store on platform '%ls'",
			*CookSavePackageContext.WriterDebugName, *TargetPlatform->PlatformName());

		TUniquePtr<FAssetRegistryState> PreviousAssetRegistry(GlobalArtifact->DetachAssetRegistry(TargetPlatform));
		if (!PreviousAssetRegistry)
		{
			UE_LOGF(LogCook, Error, "Could not load PreviousAssetRegistry, which is unexpected when cooking incrementally. Abandoning incremental cook and executing a full cook.");
			continue;
		}
		int32 NumPreviousPackages;
		PackageWriter.PopulateOplog(*PreviousAssetRegistry, NumPreviousPackages);
		if (NumPreviousPackages == 0)
		{
			UE_LOGF(LogCook, Display, "Found %d cooked package(s) in package store.", NumPreviousPackages);
			continue;
		}

		TMap<FName, FAssetRegistryGenerator::FGeneratorPackageInfo> PreviousGeneratorPackages;
		if (!bLegacyBuildDependencies)
		{
			// Incremental cook does the equivalent operation of bRecurseModifications=true and bRecurseScriptModifications=true,
			// but checks for out-of-datedness are done by FRequestCluster using the TargetDomainKey (which is built
			// from dependencies), so we do not need to check for out-of-datedness here
			int32 NumNeverCookPlaceHolderPackages;
			PlatformAssetRegistry.ReportGeneratorPackages(*PreviousAssetRegistry, PreviousGeneratorPackages,
				NumNeverCookPlaceHolderPackages);

			// Do not show NeverCookPlaceholder packages in the package counts
			NumPreviousPackages -= NumNeverCookPlaceHolderPackages;
			NumPreviousPackages = FMath::Max(0, NumPreviousPackages);
			UE_LOGF(LogCook, Display, "Found %d cooked package(s) in package store.", NumPreviousPackages);
		}
		else
		{
			// Without incremental cook, we use the AssetRegistry graph of dependencies to find out of date packages
			// We also implement other -legacyiterative behaviors:
			//  *) Remove modified packages from the PackageWriter in addition to the no-longer-exist packages
			//  *) Skip packages that failed to cook on the previous cook
			//  *) Cook all modified packages even if the requested cook packages don't reference them
			FAssetRegistryGenerator::FComputeDifferenceOptions Options;
			Options.bRecurseModifications = true;
			Options.bRecurseScriptModifications = !IsCookFlagSet(ECookInitializationFlags::IgnoreScriptPackagesOutOfDate);
			Options.bLegacyIterativeUseClassFilters = true;
			GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterativeUseClassFilters"), Options.bLegacyIterativeUseClassFilters, GEditorIni);
			GConfig->GetBool(TEXT("CookSettings"), TEXT("IterativeUseClassFilters"), Options.bLegacyIterativeUseClassFilters, GEditorIni);
			FAssetRegistryGenerator::FAssetRegistryDifference Difference;
			PlatformAssetRegistry.ComputePackageDifferences(Options, *PreviousAssetRegistry, Difference);
			PreviousGeneratorPackages = MoveTemp(Difference.GeneratorPackages);

			TArray<FName> IdenticalCooked;
			TArray<FName> PackagesToRemove;
			int32 DeferredEvaluateGeneratedNum = 0;
			int32 IdenticalCookedNum = 0;
			int32 ModifiedCookedNum = 0;
			int32 RemovedCookedNum = 0;

			auto AddPlaceholderPackage =
				[this, TargetPlatform](const FName PackageName, ECookResult CookResult, bool bLegacyIterativelyUnmodified)
				{
					FPackageData* PackageData = PackageDatas->TryAddPackageDataByPackageName(PackageName, true /* bRequireExists */);
					if (PackageData)
					{
						if (bLegacyIterativelyUnmodified)
						{
							PackageData->FindOrAddPlatformData(TargetPlatform).SetIncrementallyUnmodified(true);
						}
						PackageData->SetPlatformCooked(TargetPlatform, CookResult);
					}
				};
			bool bCookByTheBook = IsCookByTheBookMode();
			auto UpdateCookedPackage = [this, TargetPlatform, bCookByTheBook, &Difference, &IdenticalCookedNum,
				&ModifiedCookedNum, &PackageWriter, &bFirstPlatform, &PackagesToRemove]
			(const FName PackageName, bool bRequireExists, bool bLegacyIterativelyUnmodified)
			{
				++(bLegacyIterativelyUnmodified ? IdenticalCookedNum : ModifiedCookedNum);
				FPackageData* PackageData = PackageDatas->TryAddPackageDataByPackageName(PackageName, bRequireExists);
				if (PackageData)
				{
					if (bLegacyIterativelyUnmodified)
					{
						PackageData->FindOrAddPlatformData(TargetPlatform).SetIncrementallyUnmodified(true);
					}
					bool bShouldLegacyIterativeSkip = bLegacyIterativelyUnmodified;
					ICookedPackageWriter::FUpdatePackageModifiedStatusContext Context;
					Context.PackageName = PackageName;
					Context.bIncrementallyUnmodified = bLegacyIterativelyUnmodified;
					Context.bPreviouslyCooked = true;
					Context.bInOutShouldIncrementallySkip = bShouldLegacyIterativeSkip;
					PackageWriter.UpdatePackageModifiedStatus(Context);
					bShouldLegacyIterativeSkip = Context.bInOutShouldIncrementallySkip;
					if (bShouldLegacyIterativeSkip && !bLegacyIterativelyUnmodified)
					{
						// Override the PackageWriter's request to skip the modified generator package, because we
						// need to cook the generator packages to evaluate whether their generated packages should be skipped.
						EDifference* GeneratorDifference = Difference.Packages.Find(PackageName);
						if (GeneratorDifference)
						{
							bShouldLegacyIterativeSkip = false;
						}
					}
					if (bShouldLegacyIterativeSkip)
					{
						PackageData->SetPlatformCooked(TargetPlatform, ECookResult::Succeeded);
						if (bFirstPlatform)
						{
							COOK_STAT(++DetailedCookStats::NumPackagesIncrementallySkipped);
						}
						// Declare the package to the EDLCookInfo verification so we don't warn about missing exports from it
						FEDLCookCheckerThreadState::Get().AddPackageWithUnknownExports(PackageName);
					}
					else
					{
						if (bCookByTheBook)
						{
							// cook on the fly will queue packages when it needs them, but for cook by the book we force cook the modified files
							// so that the output set of packages is up to date (even if the user is currently cooking only a subset)
							WorkerRequests->AddStartCookByTheBookRequest(FFilePlatformRequest(PackageData->GetFileName(),
								EInstigator::LegacyIterativeCook, TConstArrayView<const ITargetPlatform*>{ TargetPlatform }));
						}
						PackagesToRemove.Add(PackageName);
					}
				}
			};

			// Add CookedPackages for any identical packages, delete from disk any modified packages
			for (const TPair<FName, EDifference>& Pair : Difference.Packages)
			{
				FName PackageName = Pair.Key;
				switch (Pair.Value)
				{
				case EDifference::IdenticalCooked:
					UpdateCookedPackage(PackageName, true /* bRequireExists */, true /* bLegacyIterativelyUnmodified */);
					break;
				case EDifference::ModifiedCooked:
					UpdateCookedPackage(PackageName, true /* bRequireExists */, false /* bLegacyIterativelyUnmodified */);
					break;
				case EDifference::RemovedCooked:
					PackagesToRemove.Add(PackageName);
					++RemovedCookedNum;
					break;
				case EDifference::IdenticalUncooked:
					AddPlaceholderPackage(PackageName, ECookResult::Failed, true /* bLegacyIterativelyUnmodified */);
					break;
				case EDifference::ModifiedUncooked:
					PackagesToRemove.Add(PackageName);
					break;
				case EDifference::RemovedUncooked:
					PackagesToRemove.Add(PackageName);
					break;
				case EDifference::IdenticalNeverCookPlaceholder:
					AddPlaceholderPackage(PackageName, ECookResult::NeverCookPlaceholder, true /* bLegacyIterativelyUnmodified */);
					PackagesToRemove.Add(PackageName);
					break;
				case EDifference::ModifiedNeverCookPlaceholder:
					PackagesToRemove.Add(PackageName);
					break;
				case EDifference::RemovedNeverCookPlaceholder:
					PackagesToRemove.Add(PackageName);
					break;
				default:
					break;
				}
			}

			// Add as identical any generated packages from any identical generator, because we will skip cooking
			// the generator and therefore will skip cooking the generated. Count the number of generated packages from
			// modified generators and report that we will evaluate them later.
			for (TMap<FName, FAssetRegistryGenerator::FGeneratorPackageInfo>::TIterator Iter(PreviousGeneratorPackages);
				Iter; ++Iter)
			{
				FName Generator = Iter->Key;
				FPackageData* PackageData = PackageDatas->TryAddPackageDataByPackageName(Generator, false /* bRequireExists */);
				if (PackageData && PackageData->FindOrAddPlatformData(TargetPlatform).IsCookAttempted())
				{
					for (const TPair<FName, FAssetPackageData>& GeneratedPair : Iter->Value.Generated)
					{
						UpdateCookedPackage(GeneratedPair.Key, false /* bRequireExists */, true /* bLegacyIterativelyUnmodified */);
					}
					Iter.RemoveCurrent();
				}
				else
				{
					DeferredEvaluateGeneratedNum += Iter->Value.Generated.Num();
				}
			}

			int32 CookedNum = IdenticalCooked.Num() + ModifiedCookedNum + RemovedCookedNum + DeferredEvaluateGeneratedNum;
			UE_LOGF(LogCook, Display, "Found %d cooked package(s) in package store.", CookedNum);
			UE_LOGF(LogCook, Display, "Keeping %d. Recooking %d. Removing %d. %d generated packages to be evaluated for legacyiterative skipping later.",
				IdenticalCookedNum, ModifiedCookedNum, RemovedCookedNum, DeferredEvaluateGeneratedNum);
			bFirstPlatform = false;

			PackageWriter.RemoveCookedPackages(PackagesToRemove);
		}

		for (TPair<FName, FAssetRegistryGenerator::FGeneratorPackageInfo> Pair : PreviousGeneratorPackages)
		{
			FPackageData* Generator = PackageDatas->TryAddPackageDataByPackageName(Pair.Key);
			if (!Generator)
			{
				UE_LOGF(LogCook, Display,
					"Previous cook results returned a record for generator package %ls, but that package can no longer be found; its generated packages will not be removed from cook results. Run a full cook to remove them.",
					*Pair.Key.ToString());
				continue;
			}
			TRefCountPtr<FGenerationHelper> GenerationHelper = Generator->CreateUninitializedGenerationHelper();
			GenerationHelper->SetPreviousGeneratedPackages(TargetPlatform, MoveTemp(Pair.Value.Generated));
		}

		PlatformAssetRegistry.SetPreviousAssetRegistry(MoveTemp(PreviousAssetRegistry));
	}
}

const FString ExtractPackageNameFromObjectPath( const FString ObjectPath )
{
	// get the path 
	int32 Beginning = ObjectPath.Find(TEXT("'"), ESearchCase::CaseSensitive);
	if ( Beginning == INDEX_NONE )
	{
		return ObjectPath;
	}
	int32 End = ObjectPath.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart, Beginning + 1);
	if (End == INDEX_NONE )
	{
		End = ObjectPath.Find(TEXT("'"), ESearchCase::CaseSensitive, ESearchDir::FromStart, Beginning + 1);
	}
	if ( End == INDEX_NONE )
	{
		// one more use case is that the path is "Class'Path" example "OrionBoostItemDefinition'/Game/Misc/Boosts/XP_1Win" dunno why but this is actually dumb
		if ( ObjectPath[Beginning+1] == '/' )
		{
			return ObjectPath.Mid(Beginning+1);
		}
		return ObjectPath;
	}
	return ObjectPath.Mid(Beginning + 1, End - Beginning - 1);
}

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
void DumpAssetRegistryForCooker(IAssetRegistry* AssetRegistry)
{
	FString DumpDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() + TEXT("Reports/AssetRegistryStatePages"));
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FAsyncIODelete DeleteReportDir(DumpDir + TEXT("_Del"));
	DeleteReportDir.DeleteDirectory(DumpDir);
	PlatformFile.CreateDirectoryTree(*DumpDir);
	TArray<FString> Pages;
	TArray<FString> Arguments({ TEXT("ObjectPath"),TEXT("PackageName"),TEXT("Path"),TEXT("Class"),
		TEXT("DependencyDetails"), TEXT("PackageData"), TEXT("LegacyDependencies"), TEXT("AssetTags") });
	AssetRegistry->DumpState(Arguments, Pages, 10000 /* LinesPerPage */);
	int PageIndex = 0;
	TStringBuilder<256> FileName;
	for (FString& PageText : Pages)
	{
		FileName.Reset();
		FileName.Appendf(TEXT("%s_%05d.txt"), *(DumpDir / TEXT("Page")), PageIndex++);
		PageText.ToLowerInline();
		FFileHelper::SaveStringToFile(PageText, *FileName);
	}
}
#endif


void UCookOnTheFlyServer::BlockOnAssetRegistry(TConstArrayView<FString> CommandlinePackages)
{
	if (!bFirstCookInThisProcess)
	{
		return;
	}
	TRACE_CPUPROFILER_EVENT_SCOPE(UCookOnTheFlyServer::BlockOnAssetRegistry);
	COOK_STAT(FScopedDurationTimer TickTimer(DetailedCookStats::BlockOnAssetRegistryTimeSec));

	bool bAssetGatherCompleted = true;
	UE_LOGF(LogCook, Display, "Waiting for Asset Registry");
	// Blocking on the AssetRegistry has to be done on the game thread since some AssetManager functions require it
	check(IsInGameThread());
	if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::SkipHardReferences) &&
		!CommandlinePackages.IsEmpty() &&
		bCookFastStartup)
	{
		TArray<FString> PackageNames;
		for (const FString& FileNameOrPackageName : CommandlinePackages)
		{
			FString PackageName;
			if (FPackageName::TryConvertFilenameToLongPackageName(FileNameOrPackageName, PackageName))
			{
				PackageNames.Add(MoveTemp(PackageName));
			}
		}
		AssetRegistry->ScanFilesSynchronous(PackageNames);
		bAssetGatherCompleted = false;
	}
	else if (ShouldPopulateFullAssetRegistry())
	{
		// Trigger or wait for completion the primary AssetRegistry scan.
		// Additionally scan any cook-specific paths from ini
		TArray<FString> ScanPaths;
		GConfig->GetArray(TEXT("AssetRegistry"), TEXT("PathsToScanForCook"), ScanPaths, GEngineIni);
		AssetRegistry->ScanPathsSynchronous(ScanPaths);
		if (AssetRegistry->IsSearchAsync() && AssetRegistry->IsSearchAllAssets())
		{
			AssetRegistry->WaitForCompletion();
		}
		else
		{
			AssetRegistry->SearchAllAssets(true /* bSynchronousSearch */);
		}
	}
	else if (IsCookingDLC())
	{
		TArray<FString> ScanPaths;
		ScanPaths.Add(FString::Printf(TEXT("/%s/"), *CookByTheBookOptions->DlcName));
		AssetRegistry->ScanPathsSynchronous(ScanPaths);
		AssetRegistry->WaitForCompletion();
	}
	UE::Cook::FPackageDatas::OnAssetRegistryGenerated(*AssetRegistry);

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	if (FParse::Param(FCommandLine::Get(), TEXT("DumpAssetRegistry")))
	{
		DumpAssetRegistryForCooker(AssetRegistry);
	}
#endif

	if (!IsCookWorkerMode())
	{
		FAssetRegistryGenerator::UpdateAssetManagerDatabase();
	}
	if (bAssetGatherCompleted)
	{
		AssetRegistry->ClearGathererCache();
	}
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif

	// CookCommandlet CookWorker and CookByTheBook do not initialize startup packages until BlockOnAssetRegistry,
	// because systems that subscribe to the AssetRegistry's OnFilesLoaded can load further packages at that time.
	if (!IsCookingInEditor() && (IsCookByTheBookMode() || IsCookWorkerMode()))
	{
		TSet<FName> StartupPackages;
		PackageTracker->InitializeTracking(StartupPackages);
		NumPackagesHistory.Initialize(PackageTracker->NumLoadedPackages());
		CookByTheBookOptions->StartupPackages = MoveTemp(StartupPackages);
	}
}

void UCookOnTheFlyServer::RefreshPlatformAssetRegistries(const TArrayView<const ITargetPlatform* const>& TargetPlatforms)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UCookOnTheFlyServer::RefreshPlatformAssetRegistries);

	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		FName PlatformName = FName(*TargetPlatform->PlatformName());

		UE::Cook::FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
		UE::Cook::IAssetRegistryReporter* RegistryReporter = PlatformData->RegistryReporter.Get();
		if (!RegistryReporter)
		{
			if (!IsCookWorkerMode())
			{
				PlatformData->RegistryGenerator = MakeUnique<FAssetRegistryGenerator>(TargetPlatform);
				PlatformData->RegistryReporter = MakeUnique<UE::Cook::FAssetRegistryReporterLocal>(*PlatformData->RegistryGenerator);
				PlatformData->RegistryGenerator->Initialize();
			}
			else
			{
				PlatformData->RegistryReporter = MakeUnique<UE::Cook::FAssetRegistryReporterRemote>(*CookWorkerClient, TargetPlatform);
			}
			RegistryReporter = PlatformData->RegistryReporter.Get();
		}
	}
}

void UCookOnTheFlyServer::GenerateLongPackageNames(TArray<FName>& FilesInPath, TMap<FName, UE::Cook::FInstigator>& Instigators)
{
	TSet<FName> FilesInPathSet;
	TArray<FName> FilesInPathReverse;
	TMap<FName, UE::Cook::FInstigator> NewInstigators;
	FilesInPathSet.Reserve(FilesInPath.Num());
	FilesInPathReverse.Reserve(FilesInPath.Num());
	NewInstigators.Reserve(Instigators.Num());

	for (int32 FileIndex = 0; FileIndex < FilesInPath.Num(); FileIndex++)
	{
		const FName& FileInPathFName = FilesInPath[FilesInPath.Num() - FileIndex - 1];
		const FString& FileInPath = FileInPathFName.ToString();
		UE::Cook::FInstigator& Instigator = Instigators.FindChecked(FileInPathFName);
		if (FPackageName::IsValidLongPackageName(FileInPath))
		{
			bool bIsAlreadyAdded;
			FilesInPathSet.Add(FileInPathFName, &bIsAlreadyAdded);
			if (!bIsAlreadyAdded)
			{
				FilesInPathReverse.Add(FileInPathFName);
				NewInstigators.Add(FileInPathFName, MoveTemp(Instigator));
			}
		}
		else
		{
			FString LongPackageName;
			FPackageName::EErrorCode FailureReason;
			bool bFound = FPackageName::TryConvertToMountedPath(FileInPath, nullptr /* LocalPath */, &LongPackageName,
				nullptr /* ObjectName */, nullptr /* SubObjectName */, nullptr /* Extension */,
				nullptr /* FlexNameType */, &FailureReason);
			if (!bFound && FPackageName::IsShortPackageName(FileInPath))
			{
				TArray<FName> LongPackageNames;
				AssetRegistry->GetPackagesByName(FileInPath, LongPackageNames);
				if (LongPackageNames.Num() == 1)
				{
					bFound = true;
					LongPackageName = LongPackageNames[0].ToString();
				}
			}

			if (bFound)
			{
				const FName LongPackageFName(*LongPackageName);
				bool bIsAlreadyAdded;
				FilesInPathSet.Add(LongPackageFName, &bIsAlreadyAdded);
				if (!bIsAlreadyAdded)
				{
					FilesInPathReverse.Add(LongPackageFName);
					NewInstigators.Add(LongPackageFName, MoveTemp(Instigator));
				}
			}
			else
			{
				LogCookerMessage(FString::Printf(TEXT("Unable to generate long package name, %s. %s"), *FileInPath,
					*FPackageName::FormatErrorAsString(FileInPath, FailureReason)), EMessageSeverity::Warning);
			}
		}
	}
	FilesInPath.Empty(FilesInPathReverse.Num());
	FilesInPath.Append(FilesInPathReverse);
	Swap(Instigators, NewInstigators);
}

void UCookOnTheFlyServer::AddFlexPathToCook(TArray<FName>& InOutFilesToCook,
	TMap<FName, UE::Cook::FInstigator>& InOutInstigators,
	const FString& InFlexPath, const UE::Cook::FInstigator& Instigator) const
{
	using namespace UE::Cook;

	FString FlexPath(InFlexPath);
	// Convert \ to / so that IsShortPackageName works.
	// We can still interpret the path as a filepath even with \ converted to /
	FlexPath.ReplaceCharInline('\\', '/');
	if (FPackageName::IsShortPackageName(FlexPath))
	{
		TArray<FName> LongPackageNames;
		AssetRegistry->GetPackagesByName(FlexPath, LongPackageNames);
		if (LongPackageNames.IsEmpty())
		{
			LogCookerMessage(FString::Printf(TEXT("Unable to find package for path `%s`."), *InFlexPath),
				EMessageSeverity::Warning);
		}
		else if (LongPackageNames.Num() > 1)
		{
			constexpr int32 MaxMessageLen = 256;
			TStringBuilder<256> Message;
			Message.Appendf(
				TEXT("Multiple packages found for path `%s`; it will not be added. Specify the full LongPackageName. Packages found:"),
				*InFlexPath);
			for (FName LongPackageName : LongPackageNames)
			{
				Message << TEXT("\n\t");
				if (Message.Len() >= MaxMessageLen)
				{
					Message << TEXT("...");
					break;
				}
				else
				{
					Message << LongPackageName;
				}
			}
			LogCookerMessage(FString(*Message), EMessageSeverity::Warning);
		}
		else
		{
			AddFileToCook(InOutFilesToCook, InOutInstigators, LongPackageNames[0].ToString(), Instigator);
		}
	}
	else
	{
		FString PackageName;
		if (!FPackageName::TryConvertFilenameToLongPackageName(FlexPath, PackageName))
		{
			LogCookerMessage(FString::Printf(TEXT("Unable to find package for path `%s`."), *InFlexPath),
				EMessageSeverity::Warning);
			return;
		}
		AddFileToCook(InOutFilesToCook, InOutInstigators, PackageName, Instigator);
	}
}

void UCookOnTheFlyServer::AddFileToCook( TArray<FName>& InOutFilesToCook,
	TMap<FName, UE::Cook::FInstigator>& InOutInstigators,
	const FString &InFilename, const UE::Cook::FInstigator& Instigator) const
{ 
	using namespace UE::Cook;

	if (!FPackageName::IsScriptPackage(InFilename) && !FPackageName::IsMemoryPackage(InFilename))
	{
		FName InFilenameName = FName(*InFilename);
		if (InFilenameName.IsNone())
		{
			return;
		}

		FInstigator& ExistingInstigator = InOutInstigators.FindOrAdd(InFilenameName);
		if (ExistingInstigator.Category == EInstigator::InvalidCategory)
		{
			InOutFilesToCook.Add(InFilenameName);
			ExistingInstigator = Instigator;
		}
	}
}

void UCookOnTheFlyServer::RegisterValidationCookArtifact()
{
	if (ValidationArtifact)
	{
		UnregisterArtifact(ValidationArtifact);
		ValidationArtifact = nullptr;
	}

	check(!ValidationArtifact);
	ValidationArtifact = new UE::Cook::FValidationCookArtifact(*this);
	TRefCountPtr<UE::Cook::ICookArtifact> RefCounter(ValidationArtifact);
	RegisterArtifact(ValidationArtifact);
	// We keep a raw pointer and rely on RegisterArtifact holding the reference.
	check(RefCounter.GetRefCount() > 1);
}

const TCHAR* GCookRequestUsageMessage = TEXT(
	"By default, the cooker does not cook any packages. Packages must be requested by one of the following methods.\n"
	"All transitive dependencies of a requested package are also cooked. Packages can be specified by LocalFilename/Filepath\n"
	"or by LongPackagename/LongPackagePath.\n"
	"	RecommendedMethod:\n"
	"		Use the AssetManager's default behavior of PrimaryAssetTypesToScan rules\n"
	"			Engine.ini:[/Script/Engine.AssetManagerSettings]:+PrimaryAssetTypesToScan\n"
	"	Commandline:\n"
	"		-package=<PackageName>\n"
	"			Request the given package.\n"
	"		-cookdir=<PackagePath>\n"
	"			Request all packages in the given directory.\n"
	"		-mapinisection=<SectionNameInEditorIni>	\n"
	"			Specify an ini section of packages to cook, in the style of AlwaysCookMaps.\n"
	"		-EDLCookFilters=<EDLObjectPath1>+<EDLObjectPath2>+... \n"
	"			Specify EDL object paths to cook exclusively, skipping all other EDLs used by the base map and the base map itself.\n"
	"	Ini:\n"
	"		Editor.ini\n"
	"			[AlwaysCookMaps]\n"
	"				+Map=<PackageName>\n"
	"					; Request the package on every cook. Repeatable.\n"
	"			[AllMaps]\n"
	"				+Map=<PackageName>\n"
	"					; Request the package on default cooks. Not used if commandline, AlwaysCookMaps, or MapsToCook are present.\n"
	"		Game.ini\n"
	"			[/Script/UnrealEd.ProjectPackagingSettings]\n"
	"				+MapsToCook=(FilePath=\"<PackageName>\")\n"
	"					; Request the package in default cooks. Repeatable.\n"
	"					; Not used if commandline packages or AlwaysCookMaps are present.\n"
	"				DirectoriesToAlwaysCook=(Path=\"<PackagePath>\")\n"
	"					; Request the array of packages in every cook. Repeatable.\n"
	"				bCookAll=true\n"
	"					; \n"
	"		Engine.ini\n"
	"			[/Script/EngineSettings.GameMapsSettings]\n"
	"				GameDefaultMap=<PackageName>\n"
	"				; And other default types; see GameMapsSettings.\n"
	"			[/Script/Engine.WorldPartitionSettings]\n"
	"				; Specify EDL object paths to cook exclusively, skipping all other EDLs used by the base map and the base map itself.\n"
	"				+EDLCookFilters=<EDLObjectPath>\n"
	"	C++API\n"
	"		FAssetManager::ModifyCook\n"
	"			// Subclass FAssetManager (Engine.ini:[/Script/Engine.Engine]:AssetManagerClassName) and override this hook.\n"
	"           // The default AssetManager behavior cooks all packages specified by PrimaryAssetTypesToScan rules from ini.\n"
	"		FGameDelegates::Get().GetModifyCookDelegate()\n"
	"			// Subscribe to this delegate during your module startup. (Will be deprecated in the future, use UE::Cook::FDelegates::ModifyCook instead).\n"
	"       UE::Cook::FDelegates::ModifyCook\n"
	"           // Subscribe to this delegate during your module startup.\n"
	"		ITargetPlatform::GetExtraPackagesToCook\n"
	"			// Override this hook on a given TargetPlatform.\n"
);
void UCookOnTheFlyServer::CollectFilesToCook(TArray<FName>& FilesInPath, TMap<FName, UE::Cook::FInstigator>& Instigators,
	const TArray<FString>& CookMaps, const TArray<FString>& InCookDirectories,
	const TArray<FString> &IniMapSections, ECookByTheBookOptions FilesToCookFlags, const TArrayView<const ITargetPlatform* const>& TargetPlatforms,
	const TMap<FName, TArray<FName>>& GameDefaultObjects)
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER(CollectFilesToCook);
	using namespace UE::Cook;

	if (FParse::Param(FCommandLine::Get(), TEXT("helpcookusage")))
	{
		UE::String::ParseLines(GCookRequestUsageMessage, [](FStringView Line)
			{
				UE_LOGF(LogCook, Warning, "%.*ls", Line.Len(), Line.GetData());
			});
	}
	UProjectPackagingSettings* PackagingSettings = Cast<UProjectPackagingSettings>(UProjectPackagingSettings::StaticClass()->GetDefaultObject());

	bool bCookAll = (!!(FilesToCookFlags & ECookByTheBookOptions::CookAll)) || PackagingSettings->bCookAll;
	bool bMapsOnly = (!!(FilesToCookFlags & ECookByTheBookOptions::MapsOnly)) || PackagingSettings->bCookMapsOnly;
	bool bNoDev = !!(FilesToCookFlags & ECookByTheBookOptions::NoDevContent);

	int32 PreviousFilesInPathNum = FilesInPath.Num();
	int32 NumFilesAddedByCommandLineOrGameCallback = 0;

	struct FNameWithInstigator
	{
		FInstigator Instigator;
		FName Name;
	};
	TArray<FNameWithInstigator> CookDirectories;
	for (const FString& InCookDirectory : InCookDirectories)
	{
		FName InCookDirectoryName(*InCookDirectory);
		CookDirectories.Add(FNameWithInstigator{
			FInstigator(EInstigator::CommandLineDirectory, InCookDirectoryName), InCookDirectoryName });
	}
	
	if (!IsCookingDLC() && 
		!(FilesToCookFlags & ECookByTheBookOptions::NoAlwaysCookMaps))
	{

		{
			TArray<FString> MapList;
			// Add the default map section
			GEditor->LoadMapListFromIni(TEXT("AlwaysCookMaps"), MapList);

			for (int32 MapIdx = 0; MapIdx < MapList.Num(); MapIdx++)
			{
				UE_LOGF(LogCook, Verbose, "Maplist contains %ls ", *MapList[MapIdx]);
				AddFileToCook(FilesInPath, Instigators, MapList[MapIdx], EInstigator::AlwaysCookMap);
			}
		}


		bool bFoundMapsToCook = CookMaps.Num() > 0;

		{
			TArray<FString> MapList;
			for (const FString& IniMapSection : IniMapSections)
			{
				UE_LOGF(LogCook, Verbose, "Loading map ini section %ls", *IniMapSection);
				MapList.Reset();
				GEditor->LoadMapListFromIni(*IniMapSection, MapList);
				FName MapSectionName(*IniMapSection);
				for (const FString& MapName : MapList)
				{
					UE_LOGF(LogCook, Verbose, "Maplist contains %ls", *MapName);
					AddFileToCook(FilesInPath, Instigators, MapName,
						FInstigator(EInstigator::IniMapSection, MapSectionName));
					bFoundMapsToCook = true;
				}
			}
		}

		// If we didn't find any maps look in the project settings for maps
		if (bFoundMapsToCook == false)
		{
			for (const FFilePath& MapToCook : PackagingSettings->MapsToCook)
			{
				UE_LOGF(LogCook, Verbose, "Maps to cook list contains %ls", *MapToCook.FilePath);
				AddFileToCook(FilesInPath, Instigators, MapToCook.FilePath, EInstigator::PackagingSettingsMapToCook);
				bFoundMapsToCook = true;
			}
		}

		// If we didn't find any maps, cook the AllMaps section
		if (bFoundMapsToCook == false)
		{
			UE_LOGF(LogCook, Verbose, "Loading default map ini section AllMaps");
			TArray<FString> AllMapsSection;
			GEditor->LoadMapListFromIni(TEXT("AllMaps"), AllMapsSection);
			for (const FString& MapName : AllMapsSection)
			{
				UE_LOGF(LogCook, Verbose, "Maplist contains %ls", *MapName);
				AddFileToCook(FilesInPath, Instigators, MapName, EInstigator::IniAllMaps);
			}
		}

		// Also append any cookdirs from the project ini files; these dirs are relative to the game content directory or start with a / root
		{
			for (const FDirectoryPath& DirToCook : PackagingSettings->DirectoriesToAlwaysCook)
			{
				FString LocalPath;
				if (FPackageName::TryConvertGameRelativePackagePathToLocalPath(DirToCook.Path, LocalPath))
				{
					UE_LOGF(LogCook, Verbose, "Loading directory to always cook %ls", *DirToCook.Path);
					FName LocalPathFName(*LocalPath);
					CookDirectories.Add(FNameWithInstigator{ FInstigator(EInstigator::DirectoryToAlwaysCook, LocalPathFName), LocalPathFName });
				}
				else
				{
					UE_LOGF(LogCook, Warning, "'ProjectSettings -> Directories to never cook -> Directories to always cook' has invalid element '%ls'", *DirToCook.Path);
				}
			}
		}
	}

	TSet<FName> ScratchNewFiles;
	TArray<FName> ScratchRemoveFiles;
	auto UpdateInstigators = [&FilesInPath, &Instigators, &ScratchNewFiles, &ScratchRemoveFiles]
	(const FInstigator& InInstigator, int32* NumFilesCounter)
	{
		ScratchNewFiles.Reset();
		ScratchNewFiles.Reserve(FilesInPath.Num());
		for (FName FileInPath : FilesInPath)
		{
			ScratchNewFiles.Add(FileInPath);
			FInstigator& Existing = Instigators.FindOrAdd(FileInPath);
			if (Existing.Category == EInstigator::InvalidCategory)
			{
				Existing = InInstigator;
				if (NumFilesCounter)
				{
					++(*NumFilesCounter);
				}
			}
		}
		ScratchRemoveFiles.Reset();
		for (const TPair<FName, FInstigator>& Pair : Instigators)
		{
			if (!ScratchNewFiles.Contains(Pair.Key))
			{
				ScratchRemoveFiles.Add(Pair.Key);
			}
		}
		for (FName RemoveFile : ScratchRemoveFiles)
		{
			Instigators.Remove(RemoveFile);
		}
	};

	for (const FString& CurrEntry : CookMaps)
	{
		AddFlexPathToCook(FilesInPath, Instigators, CurrEntry, EInstigator::CommandLinePackage);
	}

	if (!(FilesToCookFlags & ECookByTheBookOptions::SkipSoftReferences)
		&& !(FilesToCookFlags & ECookByTheBookOptions::NoGameAlwaysCookPackages))
	{
		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			TargetPlatform->GetExtraPackagesToCook(FilesInPath);
		}
		UpdateInstigators(EInstigator::TargetPlatformExtraPackagesToCook, nullptr);
	}

	if (!(FilesToCookFlags & ECookByTheBookOptions::SkipSoftReferences))
	{
		const FString ExternalMountPointName(TEXT("/Game/"));
		for (const FNameWithInstigator& CurrEntry : CookDirectories)
		{
			TArray<FString> Files;
			FString DirectoryName = CurrEntry.Name.ToString();
			IFileManager::Get().FindFilesRecursive(Files, *DirectoryName, *(FString(TEXT("*")) + FPackageName::GetAssetPackageExtension()), true, false);
			for (int32 Index = 0; Index < Files.Num(); Index++)
			{
				FString StdFile = Files[Index];
				FPaths::MakeStandardFilename(StdFile);
				AddFileToCook(FilesInPath, Instigators, StdFile, CurrEntry.Instigator);

				// this asset may not be in our currently mounted content directories, so try to mount a new one now
				FString LongPackageName;
				if (!FPackageName::IsValidLongPackageName(StdFile) && !FPackageName::TryConvertFilenameToLongPackageName(StdFile, LongPackageName))
				{
					FPackageName::RegisterMountPoint(ExternalMountPointName, DirectoryName);
				}
			}
		}
	}

	NumFilesAddedByCommandLineOrGameCallback += FilesInPath.Num() - PreviousFilesInPathNum;
	PreviousFilesInPathNum = FilesInPath.Num();

	if (!(FilesToCookFlags & ECookByTheBookOptions::NoDefaultMaps))
	{
		for (const auto& GameDefaultSet : GameDefaultObjects)
		{
			if (GameDefaultSet.Key == FName(TEXT("ServerDefaultMap")) && !IsCookFlagSet(ECookInitializationFlags::IncludeServerMaps))
			{
				continue;
			}

			for (FName PackagePath : GameDefaultSet.Value)
			{
				TArray<FAssetData> Assets;
				if (!AssetRegistry->GetAssetsByPackageName(PackagePath, Assets))
				{
					const FText ErrorMessage = FText::Format(LOCTEXT("GameMapSettingsMissing",
						"{0} contains a path to a missing asset '{1}'. "
						"The intended asset will fail to load in a packaged build. "
						"Select the intended asset again in Project Settings to fix this issue."),
						FText::FromName(GameDefaultSet.Key), FText::FromName(PackagePath));
					LogCookerMessage(ErrorMessage.ToString(), EMessageSeverity::Error);
				}
				else
				{
					TArray<const FAssetData*, TInlineAllocator<1>> AssetPtrs;
					AssetPtrs.Reserve(Assets.Num());
					for (const FAssetData& AssetData : Assets)
					{
						AssetPtrs.Add(&AssetData);
					}
					const FAssetData* PrimaryAssetData = UE::AssetRegistry::GetMostImportantAsset(AssetPtrs);
					if (PrimaryAssetData && PrimaryAssetData->IsRedirector())
					{
						const FText ErrorMessage = FText::Format(LOCTEXT("GameMapSettingsRedirectorDetected",
							"{0} contains a redirected reference '{1}'. "
							"The intended asset will fail to load in a packaged build. "
							"Select the intended asset again in Project Settings to fix this issue."),
							FText::FromName(GameDefaultSet.Key), FText::FromName(PackagePath));
						LogCookerMessage(ErrorMessage.ToString(), EMessageSeverity::Error);
					}
				}

				AddFileToCook(FilesInPath, Instigators, PackagePath.ToString(),
					FInstigator(EInstigator::GameDefaultObject, GameDefaultSet.Key));
			}
		}
	}

	if (!(FilesToCookFlags & ECookByTheBookOptions::NoInputPackages))
	{
		// make sure we cook any extra assets for the default touch interface
		// @todo need a better approach to cooking assets which are dynamically loaded by engine code based on settings
		FConfigFile InputIni;
		FString InterfaceFile;
		FConfigCacheIni::LoadLocalIniFile(InputIni, TEXT("Input"), true);
		if (InputIni.GetString(TEXT("/Script/Engine.InputSettings"), TEXT("DefaultTouchInterface"), InterfaceFile))
		{
			if (InterfaceFile != TEXT("None") && InterfaceFile != TEXT(""))
			{
				AddFileToCook(FilesInPath, Instigators, InterfaceFile, EInstigator::InputSettingsIni);
			}
		}
	}

	// Preserve legacy behavior: the legacy definition of "packages added by commandline or game callback" - to decide
	// whether we need to apply bCookAll below - does not include GameDefaultObjects or InputPackages.
	// So update PreviousFilesInPathNum without adding the delta to NumFilesAddedByCommandLineOrGameCallback.
	PreviousFilesInPathNum = FilesInPath.Num();

	// Call the ModifyCook delegates (almost) last, after adding all other sources of packages, so that they can see
	// and modify the complete list of what the ProjectPackagingSettings and CommandLine specified should be cooked.
	// The one exception is the legacy bCookAll behaivor, which needs to take into account whether the delegates
	// added anything to be cooked.
	if (!(FilesToCookFlags & ECookByTheBookOptions::NoGameAlwaysCookPackages))
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(CookModificationDelegate, DetailedCookStats::GameCookModificationDelegateTimeSec);

		FModifyCookDelegate& ModifyCookDelegate = FGameDelegates::Get().GetModifyCookDelegate();
		TArray<FName> PackagesToNeverCook;

		// allow the AssetManager to fill out the asset registry, as well as get a list of objects to always cook
		UAssetManager::Get().ModifyCook(TargetPlatforms, FilesInPath, PackagesToNeverCook);
		UpdateInstigators(EInstigator::AssetManagerModifyCook, &NumFilesAddedByCommandLineOrGameCallback);

		if (ModifyCookDelegate.IsBound())
		{
			// allow game or plugins to fill out the asset registry, as well as get a list of objects to always cook
			ModifyCookDelegate.Broadcast(TargetPlatforms, FilesInPath, PackagesToNeverCook);
			UpdateInstigators(EInstigator::ModifyCookDelegate, &NumFilesAddedByCommandLineOrGameCallback);
		}

		FCookInfoModifyCookDelegate& CookInfoModifyCook = UE::Cook::FDelegates::ModifyCook;
		TArray<FPackageCookRule> CookRules;
		if (CookInfoModifyCook.IsBound())
		{
			CookInfoModifyCook.Broadcast(*this, CookRules);
			for (FPackageCookRule& CookRule : CookRules)
			{
				if (CookRule.PackageName.IsNone())
				{
					continue;
				}
				switch (CookRule.CookRule)
				{
				case EPackageCookRule::None:
					break;
				case EPackageCookRule::AddToCook:
				{
					FInstigator& Existing = Instigators.FindOrAdd(CookRule.PackageName);
					if (Existing.Category == EInstigator::InvalidCategory)
					{
						Existing = FInstigator(EInstigator::ModifyCookDelegate, CookRule.InstigatorName);
					}
					FilesInPath.Add(CookRule.PackageName);
					++NumFilesAddedByCommandLineOrGameCallback;
					break;
				}
				case EPackageCookRule::NeverCook:
					PackagesToNeverCook.Add(CookRule.PackageName);
					break;
				// case EPackageCookRule::IgnoreStartupPackage:
				// To implement IgnoreStartupPackage, we will need to pass in StartupPackages, and
				// delay the calculation of ProcessSoftObjectPathPackageList.
				default:
					checkNoEntry();
					break;
				}
			}
			UpdateInstigators(EInstigator::ModifyCookDelegate, nullptr);
		}

		for (FName NeverCookPackage : PackagesToNeverCook)
		{
			FName PackageName;
			if (PackageDatas->TryGetNamesByFlexName(NeverCookPackage, &PackageName, nullptr, true /* bRequireExists */))
			{
				PackageTracker->NeverCookPackageList.Add(PackageName);
			}
		}
	}

	if (IsCookingDLC())
	{
		TArray<FName> PackagesToNeverCook;
		UAssetManager::Get().ModifyDLCCook(CookByTheBookOptions->DlcName, TargetPlatforms, FilesInPath, PackagesToNeverCook);
		UpdateInstigators(EInstigator::AssetManagerModifyDLCCook, &NumFilesAddedByCommandLineOrGameCallback);

		for (FName NeverCookPackage : PackagesToNeverCook)
		{
			FName PackageName;
			if (PackageDatas->TryGetNamesByFlexName(NeverCookPackage, &PackageName, nullptr, true /* bRequireExists */))
			{
				PackageTracker->NeverCookPackageList.Add(PackageName);
			}
		}
	}

	// NumFilesAddedByCommandLineOrGameCallback for the ModifyCook delegates was incremented by UpdateInstigators,
	// so don't update it here, but do update the other accumulator PreviousFilesInPathNum.
	PreviousFilesInPathNum = FilesInPath.Num();

	// If no packages were explicitly added by command line or game callback, add all maps
	if (!(FilesToCookFlags & ECookByTheBookOptions::SkipSoftReferences)
		&& !(FilesToCookFlags & ECookByTheBookOptions::NoGameAlwaysCookPackages))
	{
		if (bCookAll || (UE::Cook::bCookAllByDefault && NumFilesAddedByCommandLineOrGameCallback == 0))
		{
			TArray<FString> Tokens;
			Tokens.Empty(2);
			Tokens.Add(FString("*") + FPackageName::GetAssetPackageExtension());
			Tokens.Add(FString("*") + FPackageName::GetMapPackageExtension());

			uint8 PackageFilter = NORMALIZE_DefaultFlags | NORMALIZE_ExcludeEnginePackages | NORMALIZE_ExcludeLocalizedPackages;
			if (bMapsOnly)
			{
				PackageFilter |= NORMALIZE_ExcludeContentPackages;
			}

			if (bNoDev)
			{
				PackageFilter |= NORMALIZE_ExcludeDeveloperPackages;
			}

			// assume the first token is the map wildcard/pathname
			TArray<FString> Unused;
			for (int32 TokenIndex = 0; TokenIndex < Tokens.Num(); TokenIndex++)
			{
				TArray<FString> TokenFiles;
				if (!NormalizePackageNames(Unused, TokenFiles, Tokens[TokenIndex], PackageFilter))
				{
					UE_LOGF(LogCook, Display, "No packages found for parameter %i: '%ls'", TokenIndex, *Tokens[TokenIndex]);
					continue;
				}

				for (int32 TokenFileIndex = 0; TokenFileIndex < TokenFiles.Num(); ++TokenFileIndex)
				{
					AddFileToCook(FilesInPath, Instigators, TokenFiles[TokenFileIndex], EInstigator::FullDepotSearch);
				}
			}
		}
		else if (NumFilesAddedByCommandLineOrGameCallback == 0)
		{
			LogCookerMessage(TEXT("No package requests specified on -run=Cook commandline or ini. ")
				TEXT("Set the flag 'Edit->Project Settings->Project/Packaging->Packaging/Advanced->Cook Everything in the Project Content Directory'. ")
				TEXT("Or launch 'UnrealEditor -run=cook -helpcookusage' to see all package request options."), EMessageSeverity::Warning);
		}
	}
}

void UCookOnTheFlyServer::GetGameDefaultObjects(const TArray<ITargetPlatform*>& TargetPlatforms, TMap<FName, TArray<FName>>& OutGameDefaultObjects)
{
	// Collect all default objects from all cooked platforms engine configurations.
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		// load the platform specific ini to get its DefaultMap
		FConfigFile PlatformEngineIni;
		FConfigCacheIni::LoadLocalIniFile(PlatformEngineIni, TEXT("Engine"), true, *TargetPlatform->IniPlatformName());

		const FConfigSection* MapSettingsSection = PlatformEngineIni.FindSection(TEXT("/Script/EngineSettings.GameMapsSettings"));

		if (MapSettingsSection == nullptr)
		{
			continue;
		}

		auto AddDefaultObject = [&OutGameDefaultObjects, &PlatformEngineIni, MapSettingsSection](FName PropertyName)
		{
			const FConfigValue* PairString = MapSettingsSection->Find(PropertyName);
			if (PairString == nullptr)
			{
				return;
			}
			FString ObjectPath = PairString->GetValue();
			if (ObjectPath.IsEmpty())
			{
				return;
			}

			FSoftObjectPath Path(ObjectPath);
			FName PackageName = Path.GetLongPackageFName();
			if (PackageName.IsNone())
			{
				return;
			}
			OutGameDefaultObjects.FindOrAdd(PropertyName).AddUnique(PackageName);
		};

		// get the server and game default maps/modes and cook them
		AddDefaultObject(FName(TEXT("GameDefaultMap")));
		AddDefaultObject(FName(TEXT("ServerDefaultMap")));
		AddDefaultObject(FName(TEXT("GlobalDefaultGameMode")));
		AddDefaultObject(FName(TEXT("GlobalDefaultServerGameMode")));
		AddDefaultObject(FName(TEXT("GameInstanceClass")));
	}
}

bool UCookOnTheFlyServer::IsCookByTheBookRunning() const
{
	return IsCookByTheBookMode() && IsInSession();
}

FString UCookOnTheFlyServer::GetSandboxDirectory( const FString& PlatformName ) const
{
	return SandboxFile->GetSandboxDirectory(PlatformName);
}

FString UCookOnTheFlyServer::ConvertToFullSandboxPath( const FString &FileName, bool bForWrite ) const
{
	check(SandboxFile);
	return SandboxFile->ConvertToFullSandboxPath(FileName, bForWrite);
}

FString UCookOnTheFlyServer::ConvertToFullSandboxPath( const FString &FileName, bool bForWrite, const FString& PlatformName ) const
{
	check(SandboxFile);
	return SandboxFile->ConvertToFullPlatformSandboxPath(FileName, bForWrite, PlatformName);
}

FString UCookOnTheFlyServer::GetSandboxAssetRegistryFilename()
{
	if (IsCookingDLC())
	{
		check(IsDirectorCookByTheBook());
		const FString Filename = FPaths::Combine(*GetBaseDirectoryForDLC(), GetAssetRegistryFilename());
		return ConvertToFullSandboxPath(*Filename, true);
	}

	static const FString RegistryFilename = FPaths::ProjectDir() / GetAssetRegistryFilename();
	return ConvertToFullSandboxPath(*RegistryFilename, true);
}

FString UCookOnTheFlyServer::GetCookedAssetRegistryFilename(const FString& PlatformName)
{
	const FString CookedAssetRegistryFilename = GetSandboxAssetRegistryFilename().Replace(TEXT("[Platform]"), *PlatformName);
	return CookedAssetRegistryFilename;
}

FString UCookOnTheFlyServer::GetCookedCookMetadataFilename(const ITargetPlatform* TargetPlatform)
{
	return GetCookMetadataOutputFolderInternal(TargetPlatform) / UE::Cook::GetCookMetadataFilename();
}

FString UCookOnTheFlyServer::GetSandboxCachedEditorThumbnailsFilename()
{
	if (IsCookingDLC())
	{
		check(IsDirectorCookByTheBook());
		const FString Filename = FPaths::Combine(*GetBaseDirectoryForDLC(), FThumbnailExternalCache::GetCachedEditorThumbnailsFilename());
		return ConvertToFullSandboxPath(*Filename, true);
	}

	static const FString CachedEditorThumbnailsFilename = FPaths::ProjectDir() / FThumbnailExternalCache::GetCachedEditorThumbnailsFilename();
	return ConvertToFullSandboxPath(*CachedEditorThumbnailsFilename, true);
}

void UCookOnTheFlyServer::WriteCookMetadata(const ITargetPlatform* InTargetPlatform, uint64 InDevelopmentAssetRegistryHash)
{
	FString PlatformNameString = InTargetPlatform->PlatformName();

	//
	// Write the plugin hierarchy for the plugins enabled. Technically for a DLC cook we aren't cooking all plugins,
	// however there's not a direct way to narrow the list (e.g. enabled by default + dlc plugins is too narrow), 
	// so we just always write the entire set.
	//
	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();

	// NOTE: We can't use IsEnabledForPlugin because it has an issue with the AllowTargets list where preventing a plugin on
	// a target at the uproject level doesn't get overridden by a dependent plugins' reference. This manifests as packages
	// on disk during stage existing but the plugin isn't in the cook manifest. I wasn't able to find a way to fix this
	// with the current plugin system, so we include all enabled plugins.

	constexpr int32 AdditionalPseudoPlugins = 2; // /Engine and /Game.
	if (IntFitsIn<uint16>(EnabledPlugins.Num() + AdditionalPseudoPlugins) == false)
	{
		UE_LOGF(LogCook, Warning, "Number of plugins exceeds 64k, unable to write cook metadata file (count = %d)", EnabledPlugins.Num() + AdditionalPseudoPlugins);
	}
	else
	{
		TArray<UE::Cook::FCookMetadataPluginEntry> PluginsToAdd;
		TMap<FString, uint16> IndexForPlugin;
		TArray<uint16> PluginChildArray;

		PluginsToAdd.AddDefaulted(EnabledPlugins.Num());
		uint16 AddIndex = 0;
		for (TSharedRef<IPlugin>& EnabledPlugin : EnabledPlugins)
		{
			UE::Cook::FCookMetadataPluginEntry& NewEntry = PluginsToAdd[AddIndex];
			NewEntry.Name = EnabledPlugin->GetName();
			IndexForPlugin.Add(NewEntry.Name, AddIndex);
			AddIndex++;
		}

		TArray<FString> BoolCustomFieldsList;
		TArray<FString> StringCustomFieldsList;
		TArray<FString> PerPlatformBoolCustomFieldsList;
		TArray<FString> PerPlatformStringCustomFieldsList;
		GConfig->GetArray(TEXT("CookMetadataCustomPluginFields"), TEXT("BoolFields"), BoolCustomFieldsList, GEditorIni);		
		GConfig->GetArray(TEXT("CookMetadataCustomPluginFields"), TEXT("StringFields"), StringCustomFieldsList, GEditorIni);
		GConfig->GetArray(TEXT("CookMetadataCustomPluginFields"), TEXT("PerPlatformBoolFields"), PerPlatformBoolCustomFieldsList, GEditorIni);
		GConfig->GetArray(TEXT("CookMetadataCustomPluginFields"), TEXT("PerPlatformStringFields"), PerPlatformStringCustomFieldsList, GEditorIni);

		// Get the names as a unique list for indexing the names for serialization
		TArray<FString> CustomFieldNames;
		TArray<UE::Cook::ECookMetadataCustomFieldType> CustomFieldTypes;
		TMap<FString, uint8> CustomFieldNameIndex;
		{
			auto GetNames = [&CustomFieldNameIndex, &CustomFieldNames, &CustomFieldTypes](const TArray<FString>& FieldList, UE::Cook::ECookMetadataCustomFieldType FieldType)
			{
				for (const FString& FieldName : FieldList)
				{
					uint8& FoundAtIndex = CustomFieldNameIndex.FindOrAdd(FieldName, MAX_uint8);
					if (FoundAtIndex == MAX_uint8)
					{
						CustomFieldNames.Add(FieldName);
						CustomFieldTypes.Add(FieldType);
						FoundAtIndex = (uint8)(CustomFieldNames.Num() - 1);
					}
				}
			};

			GetNames(BoolCustomFieldsList, UE::Cook::ECookMetadataCustomFieldType::Bool);
			GetNames(StringCustomFieldsList, UE::Cook::ECookMetadataCustomFieldType::String);
			GetNames(PerPlatformBoolCustomFieldsList, UE::Cook::ECookMetadataCustomFieldType::Bool);
			GetNames(PerPlatformStringCustomFieldsList, UE::Cook::ECookMetadataCustomFieldType::String);
		}

		if (CustomFieldNames.Num() > 255)
		{
			// Sanity check integer limits - should never hit this, but if we do all bets are off.
			UE_LOGF(LogCook, Warning, "Number of custom plugin fields exceeds 255 (count = %d), custom fields will be incorrect!", CustomFieldNames.Num());
		}

		// Add the /Engine and /Game pseudo plugins. These are placeholders for holding size information when unrealpak runs.
		const uint16 EnginePluginIndex = (uint16)PluginsToAdd.Num();
		{
			UE::Cook::FCookMetadataPluginEntry& EnginePseudoPlugin = PluginsToAdd.AddDefaulted_GetRef();
			EnginePseudoPlugin.Name = TEXT("Engine");
			EnginePseudoPlugin.Type = UE::Cook::ECookMetadataPluginType::EnginePseudo;
		}
		const uint16 GamePluginIndex = (uint16)PluginsToAdd.Num();
		{
			UE::Cook::FCookMetadataPluginEntry& GamePseudoPlugin = PluginsToAdd.AddDefaulted_GetRef();
			GamePseudoPlugin.Name = TEXT("Game");
			GamePseudoPlugin.Type = UE::Cook::ECookMetadataPluginType::GamePseudo;
		}

		// Construct the dependency list.
		TArray<uint16> RootPlugins;
		for (TSharedRef<IPlugin>& EnabledPlugin : EnabledPlugins)
		{
			uint16 SelfIndex = IndexForPlugin[EnabledPlugin->GetName()];
			UE::Cook::FCookMetadataPluginEntry& Entry = PluginsToAdd[SelfIndex];
			Entry.Type = UE::Cook::ECookMetadataPluginType::Normal;

			// We detect if this would overflow below and cancel the write - so while this could store
			// bogus data, it won't get saved.
			Entry.DependencyIndexStart = (uint32)PluginChildArray.Num();

			if (EnabledPlugin->IsRootPlugin())
			{
				Entry.Type = UE::Cook::ECookMetadataPluginType::Root;
				RootPlugins.Add(SelfIndex);
			}

			const FPluginDescriptor& Descriptor = EnabledPlugin->GetDescriptor();
			// Pull in any custom fields the project wants to pass on.
			auto CheckForCustomFields = [&Descriptor, &PlatformNameString, &EnabledPlugin, &Entry, &CustomFieldNameIndex](bool bIsBool, bool bIsPerPlatform, const TArray<FString>& FieldNames)
			{
				for (const FString& FieldName : FieldNames)
				{
					UE::Cook::FCookMetadataPluginEntry::CustomFieldVariantType FieldValue;
			
					bool bHasField = false;

					if (bIsBool)
					{
						bool bPlatformAgnosticValue = false;
						bHasField = Descriptor.CachedJson->TryGetBoolField(FieldName, bPlatformAgnosticValue);
						FieldValue.Set<bool>(bPlatformAgnosticValue);
					}
					else
					{
						FString PlatformAgnosticString;
						bHasField = Descriptor.CachedJson->TryGetStringField(FieldName, PlatformAgnosticString);
						FieldValue.Set<FString>(MoveTemp(PlatformAgnosticString));
					}

					if (bIsPerPlatform)
					{
						// If the field is marked as per-platform, then it has a field with the same name except
						// prepended with PerPlatform. Inside that is an array of objects with Platform and Value
						// to specify the override for specific platforms.
						bool bHasPerPlatform = false;

						const TArray<TSharedPtr<FJsonValue>>* Array;
						if (Descriptor.CachedJson->TryGetArrayField(TEXT("PerPlatform") + FieldName, Array))
						{
							bHasPerPlatform = true;

							for (const TSharedPtr<FJsonValue>& Value : *Array)
							{
								const TSharedPtr<FJsonObject>& ValueObject = Value->AsObject();
								if (ValueObject.IsValid())
								{
									FString OverridePlatformName;
									if (!ValueObject->TryGetStringField(TEXT("Platform"), OverridePlatformName))
									{
										UE_LOGF(LogCook, Error, "Unable to get Platform field from PerPlatform%ls array in plugin %ls json.", *FieldName, *EnabledPlugin->GetName());
										continue;
									}

									if (OverridePlatformName == PlatformNameString)
									{
										bool bGotOverride = false;

										if (bIsBool)
										{
											bool bPlatformValue = false;
											bGotOverride = ValueObject->TryGetBoolField(TEXT("Value"), bPlatformValue);
											FieldValue.Set<bool>(bPlatformValue);
										}
										else
										{
											FString PlatformString;
											bGotOverride = ValueObject->TryGetStringField(TEXT("Value"), PlatformString);
											FieldValue.Set<FString>(MoveTemp(PlatformString));
										}

										if (!bGotOverride)
										{
											UE_LOGF(LogCook, Error, "Unable to get Value field from PerPlatform%ls array in plugin %ls json for platform %ls", *FieldName, *EnabledPlugin->GetName(), *PlatformNameString);
											continue;
										}
										bHasField = true;
									}
								}
							}
						} // end if the plugin has overrides

						// If the field has a per platform value, but no value for this platform in either the
						// agnostic or the specific area, then we fill it with default values so it's still
						// present in the output, even if it's just default values.
						if (bHasPerPlatform && !bHasField)
						{
							bHasField = true;
							if (bIsBool)
							{
								FieldValue.Set<bool>(false);
							}
							else
							{
								FieldValue.Set<FString>(FString());
							}
						}
					} // end if field is per platform

					if (bHasField)
					{
						Entry.CustomFields.Add(CustomFieldNameIndex[FieldName], FieldValue);
					}
				} // end each field name

			}; // end local lambda

			CheckForCustomFields(true, false, BoolCustomFieldsList);
			CheckForCustomFields(true, true, PerPlatformBoolCustomFieldsList);
			CheckForCustomFields(false, false, StringCustomFieldsList);
			CheckForCustomFields(false, true, PerPlatformStringCustomFieldsList);

			for (FPluginReferenceDescriptor ChildPlugin : Descriptor.Plugins)
			{
				if (uint16* ChildIndex = IndexForPlugin.Find(ChildPlugin.Name); ChildIndex != nullptr)
				{
					PluginChildArray.Add(*ChildIndex);
				}
				else
				{
					UE_LOGF(LogCook, Display, "Dependent plugin \"%ls\" referenced by \"%ls\" wasn't found in enabled plugins list when creating cook metadata file... skipping", *ChildPlugin.Name, *Descriptor.FriendlyName);
				}
			}

			//
			// We've created two pseudo plugins Engine and Game, however no one depends on them explicitly.
			// In order to facilitate the size computations, we inject artificial dependencies based on where
			// the plugin was loaded from.
			PluginChildArray.Add(EnginePluginIndex);
			if (EnabledPlugin->GetLoadedFrom() == EPluginLoadedFrom::Project)
			{
				PluginChildArray.Add(GamePluginIndex);
			}

			Entry.DependencyIndexEnd = (uint32)PluginChildArray.Num();
		}

		// Also ensure Game depends on Engine.
		PluginsToAdd[GamePluginIndex].DependencyIndexStart = (uint32)PluginChildArray.Num();
		PluginChildArray.Add(EnginePluginIndex);
		PluginsToAdd[GamePluginIndex].DependencyIndexEnd = (uint32)PluginChildArray.Num();

		UE::Cook::FCookMetadataState MetadataState;
		UE::Cook::FCookMetadataPluginHierarchy PluginHierarchy;


		PluginHierarchy.PluginsEnabledAtCook = MoveTemp(PluginsToAdd);
		PluginHierarchy.PluginDependencies = MoveTemp(PluginChildArray);
		PluginHierarchy.RootPlugins = MoveTemp(RootPlugins);

		for (int32 FieldIndex = 0; FieldIndex < CustomFieldNames.Num(); FieldIndex++)
		{
			UE::Cook::FCookMetadataPluginHierarchy::FCustomFieldEntry& NewEntry = PluginHierarchy.CustomFieldEntries.AddDefaulted_GetRef();
			NewEntry.Name = MoveTemp(CustomFieldNames[FieldIndex]);
			NewEntry.Type = CustomFieldTypes[FieldIndex];
		}

		// Sanity check we assigned plugin types
		for (UE::Cook::FCookMetadataPluginEntry& Entry : PluginHierarchy.PluginsEnabledAtCook)
		{
			if (Entry.Type == UE::Cook::ECookMetadataPluginType::Unassigned)
			{
				UE_LOGF(LogCook, Warning, "Found unassigned plugin type in cook metadata generation: %ls", *Entry.Name);
			}
		}

		MetadataState.SetPluginHierarchyInfo(MoveTemp(PluginHierarchy));
		MetadataState.SetAssociatedDevelopmentAssetRegistryHash(InDevelopmentAssetRegistryHash);

		MetadataState.SetPlatformAndBuildVersion(PlatformNameString, FApp::GetBuildVersion());
		MetadataState.SetHordeJobId(FPlatformMisc::GetEnvironmentVariable(TEXT("UE_HORDE_JOBID")));

		MetadataState.SaveToFile(GetCookedCookMetadataFilename(InTargetPlatform));
	}
}

void UCookOnTheFlyServer::WriteReferencedSet(const ITargetPlatform* InTargetPlatform, TArray<FName>&& CookedPackageNames)
{
	const FString MetadataFilename = GetCookMetadataOutputFolderInternal(InTargetPlatform) / UE::Cook::GetReferencedSetFilename();

	CookedPackageNames.Sort(FNameLexicalLess());

	TStringBuilder<256> OplogKeyStr;
	auto NormalizeOplogKey = [&OplogKeyStr](FName PackageName)
		{
			OplogKeyStr.Reset();
			OplogKeyStr << PackageName;
			for (TCHAR& C : MakeArrayView(OplogKeyStr))
			{
				C = FChar::ToLower(C);
			}
			return OplogKeyStr.ToView();
		};
	constexpr FStringView VersionStr(TEXTVIEW("# Version 1"));
	int32 CombinedLength = VersionStr.Len() + CookedPackageNames.Num() * UE_ARRAY_COUNT(LINE_TERMINATOR);
	for (FName PackageName : CookedPackageNames)
	{
		CombinedLength += PackageName.GetStringLength();
	}
	FString CombinedString;
	CombinedString.Reserve(CombinedLength);
	CombinedString += VersionStr;
	for (FName PackageName : CookedPackageNames)
	{
		CombinedString += LINE_TERMINATOR;
		CombinedString += NormalizeOplogKey(PackageName);
	}
	FFileHelper::SaveStringToFile(CombinedString, *MetadataFilename,
		FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
}

void UCookOnTheFlyServer::CookByTheBookFinished()
{
	UE_LOGF(LogCook, Display, "---- Finalisation: Start ----");
	{
		// Add a timer around most of CookByTheBookFinished, but the timer can not exist during or after
		// ShutdownCookSession because it deletes memory for the timers
		UE_SCOPED_HIERARCHICAL_COOKTIMER(CookByTheBookFinished);
		CookByTheBookFinishedInternal();
	}

	for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
	{
		ClearCookInProgressFlagFromGlobalCookSettings(TargetPlatform);
	}
	ShutdownCookSession();
	UE_LOGF(LogCook, Display, "---- Finalisation: End ----");
	UE_LOGF(LogCook, Display, "Done!");
}

void UCookOnTheFlyServer::CookByTheBookFinishedInternal()
{
	UE_LOGF(LogCookStatus, Display, "FinishedInternal: Start");

	using namespace UE::Cook;

	check(IsInGameThread());
	check(IsCookByTheBookMode());
	check(IsInSession());
	check(PackageDatas->GetRequestQueue().IsEmpty());
	check(PackageDatas->GetRequestQueue().GetDiscoveryQueue().IsEmpty());
	check(PackageDatas->GetRequestQueue().GetBuildDependencyDiscoveryQueue().IsEmpty());
	check(PackageDatas->GetAssignedToWorkerSet().IsEmpty());
	check(PackageDatas->GetLoadQueue().IsEmpty());
	check(PackageDatas->GetSaveQueue().IsEmpty());
	check(PackageDatas->GetSaveStalledSet().IsEmpty());

	TArray<FPackageData*> DanglingGenerationHelpers;
	PackageDatas->LockAndEnumeratePackageDatas([&DanglingGenerationHelpers](FPackageData* PackageData)
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
			if (GenerationHelper)
			{
				// One reason it might still be around is the keepforincremental flag, if it was in the oplog but never
				// cooked. Clear that flag now and then retest whether it is still referenced.
				GenerationHelper->ClearKeepForIncrementalAllPlatforms();
				GenerationHelper.SafeRelease();
				if (PackageData->GetGenerationHelper())
				{
					DanglingGenerationHelpers.Add(PackageData);
				}
			}
		});
	for (FPackageData* PackageData : DanglingGenerationHelpers)
	{
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
		if (GenerationHelper)
		{
			GenerationHelper->DiagnoseWhyNotShutdown();
			if (GenerationHelper->IsInitialized())
			{
				GenerationHelper->ForceUninitialize();
			}
		}
	};

	{
		UE_SCOPED_COOKTIMER(TickCookableObjects);
		const double CurrentTime = FPlatformTime::Seconds();
		FTickableCookObject::TickObjects(static_cast<float>(CurrentTime - LastCookableObjectTickTime), true /* bTickComplete */);
		LastCookableObjectTickTime = CurrentTime;
	}

	UE_LOGF(LogCookStatus, Display, "Waiting for AsyncFileWrites");
	UPackage::WaitForAsyncFileWrites();

	UE_LOGF(LogCookStatus, Display, "Waiting for BuildDefinitions");
	BuildDefinitions->Wait();
	
	UE_LOGF(LogCookStatus, Display, "Waiting for DDC");
	GetDerivedDataCacheRef().WaitForQuiescence(true);
	
	bool bSaveAssetRegistry = !FParse::Param(FCommandLine::Get(), TEXT("SkipSaveAssetRegistry"));
	// if we are cooking DLC, the DevelopmentAR isn't needed - it's used when making DLC against shipping, so there's no need to make it
	// again, as we don't make DLC against DLC (but allow an override just in case)
	bool bSaveDevelopmentAssetRegistry = !FParse::Param(FCommandLine::Get(), TEXT("NoSaveDevAR"));
	bool bForceNoFilterAssetsFromAssetRegistry = IsForceNoFilterAssetsFromAssetRegistry();
	bool bSaveManifests = true;
	bool bSaveIniSettings = !FParse::Param(FCommandLine::Get(), TEXT("SkipSaveCookSettings"));
	bool bSaveCookerOpenOrder = true;
	bool bCacheShaderLibraries = IsUsingShaderCodeLibrary();
	// SkipSaveAssetRegistry skips some other optional artifacts, because it is used as a
	// "cook for testing purposes quickly" flag. They also have dependencies on each other in the current code.
	if (!bSaveAssetRegistry)
	{
		bCacheShaderLibraries = false;
		bSaveDevelopmentAssetRegistry = false;
		bSaveManifests = false;
		bSaveCookerOpenOrder = false;
	}

	UCookerSettings const* CookerSettings = GetDefault<UCookerSettings>();

	FString LibraryName = GetProjectShaderLibraryName();
	check(!LibraryName.IsEmpty());

	// Save modified asset registry with all streaming chunk info generated during cook
	const FString SandboxRegistryFilename = GetSandboxAssetRegistryFilename();

	// Saving the current ini settings. This is only required for legacyiterative cooking and may take seconds.
	if (bSaveIniSettings)
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(SavingCurrentIniSettings)
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms() )
		{
			if (FindOrCreateSaveContext(TargetPlatform).PackageWriterCapabilities.bReadOnly)
			{
				continue;
			}
			SaveCurrentIniSettings(TargetPlatform);
		}
	}

	if (bSaveAssetRegistry)
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(ChunkGeneration);

		RegisterLocalizationChunkDataGenerator();
		if (bCacheShaderLibraries)
		{
			RegisterShaderChunkDataGenerator();
		}

		for (auto ChunkGeneratorFactory : IChunkDataGenerator::GetChunkDataGeneratorFactories())
		{
			for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
			{
				FAssetRegistryGenerator& RegistryGenerator = *(PlatformManager->GetPlatformData(TargetPlatform)->RegistryGenerator);
				RegistryGenerator.RegisterChunkDataGenerator(ChunkGeneratorFactory(*this));
			}
		}
	}

	UE_LOGF(LogCookStatus, Display, "Collecting cook information for all platforms");

	struct FEndCookPlatformData
	{
		TSet<FName> CookedPackageNames;
		TArray<FPackageData*> CookedPackageDatas;
		TArray<FPackageData*> IgnorePackageDatas;
	};
	TArray<FEndCookPlatformData> EndCookPlatformDatas;
	EndCookPlatformDatas.SetNum(PlatformManager->GetSessionPlatforms().Num());

	// Call FinishPopulateShaderLibrary to load the previous version of the shader libraries for all platforms, and
	// call UpdateOplogPackages to report which assets are still referenced, before pruning any no-longer-referenced
	// shaders from any platforms. We do it in this order because platforms can share shaderformats, so we need to
	// know if any platform is referencing shaderformat X before we can prune it.
	if (bCacheShaderLibraries)
	{
		const bool bInitializeReferenceTracking = !bLegacyBuildDependencies;
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(FinishPopulateShaderLibrary);
			FinishPopulateShaderLibrary(TargetPlatform, LibraryName, bInitializeReferenceTracking);
		}
	}

	int32 PlatformIndex = 0;
	for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
	{
		FEndCookPlatformData& EndCookPlatformData = EndCookPlatformDatas[PlatformIndex++];
		FCookSavePackageContext& SaveContext = FindOrCreateSaveContext(TargetPlatform);
		if (SaveContext.PackageWriterCapabilities.bReadOnly)
		{
			continue;
		}

		FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
		FAssetRegistryGenerator& Generator = *PlatformData->RegistryGenerator;

		TArray<FPackageData*>& CookedPackageDatas = EndCookPlatformData.CookedPackageDatas;
		TArray<FPackageData*>& IgnorePackageDatas = EndCookPlatformData.IgnorePackageDatas;

		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(CalculateReferencedSet);

			PackageDatas->GetCommittedPackagesForPlatform(TargetPlatform, CookedPackageDatas, IgnorePackageDatas);

			if (IsCookingDLC())
			{
				// We need to add the packages cooked (or incrementally skipped) in this session into the new
				// AssetRegistry, but need to not add the cooked-elsewhere (e.g. basegame cooked) packages into it. So
				// we remove them from CookedPackageDatas and add them to IgnorePackageDatas.
				for (TArray<FPackageData*>::TIterator Iter(CookedPackageDatas); Iter; ++Iter)
				{
					FPackageData* PackageData = *Iter;
					FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
					if (PackagePlatformData.GetWhereCooked() != EWhereCooked::ThisSession)
					{
						Iter.RemoveCurrentSwap();
						IgnorePackageDatas.Add(PackageData);
					}
				}
			}
		}

		// Call UpdateOplogPackages on all Cook Artifacts. We have to do it at this point to handle shader libraries
		// correctly. FShaderLibraryCookArtifact needs UpdateOplogPackages called after FinishPopulateShaderLibrary,
		// so that it has all of its data loaded and ready to be pruned. But it has to do this pruning before any calls
		// to FinalizeChunkIds or SaveShaderLibrary; those functions need to write out the pruned version of the data.
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(CookArtifacts.UpdateOplogPackages);
			using namespace UE::Cook::Artifact;
			FUpdateOplogPackagesContext OplogContext(*this);
			OplogContext.TargetPlatform = TargetPlatform;
			for (FPackageData* PackageData : CookedPackageDatas)
			{
				FOplogPackageData& Data = OplogContext.OplogPackages.FindOrAdd(PackageData->GetPackageName());
				Data.ReferenceType = UE::Cook::EPackageReferenceType::Stage;
				const FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
				Data.bIncrementallyCooked = !PackagePlatformData.IsIncrementallySkipped();
			}
			for (FPackageData* PackageData : IgnorePackageDatas)
			{
				FOplogPackageData& Data = OplogContext.OplogPackages.FindOrAdd(PackageData->GetPackageName());
				Data.ReferenceType = UE::Cook::EPackageReferenceType::Cook;
				const FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
				Data.bIncrementallyCooked = !PackagePlatformData.IsIncrementallySkipped();
			}

			// Call UpdateLastReferenceDateAndPruneStaleOps to remove any stale ops from the oplog, and then
			// add all of the ops in the oplog to OplogContext.OplogPackages, if they do not already exist.
			SaveContext.PackageWriter->UpdateLastReferenceDateAndPruneStaleOps(OplogContext);
			for (FName PackageName : SaveContext.PackageWriter->GetOplogPackageNames())
			{
				// We rely on ReferenceType being initialized to EPackageReferenceType::Unreferenced for newly added
				// elements. In that case, calling FindOrAdd is sufficient to add the entry we need because we only
				// need to add entries for any non-referenced-set packages; all of the referenced-set packages were
				// added in the for (PackageData : CookedPackageDatas) loop above.
				OplogContext.OplogPackages.FindOrAdd(PackageName);
			}

			for (const TRefCountPtr<UE::Cook::ICookArtifact>& Artifact : CookArtifacts)
			{
				if (!bLegacyBuildDependencies || Artifact == GlobalArtifact)
				{
					Artifact->UpdateOplogPackages(OplogContext);
				}
			}
		}
	}

	// Trigger the ShaderLibrary to prune the ShaderFormats used by each platform, now that we have reported
	// which assets are still in use by each platform.
	if (!bLegacyBuildDependencies && bCacheShaderLibraries)
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(PruneShaderLibrary);

		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			FCookSavePackageContext& SaveContext = FindOrCreateSaveContext(TargetPlatform);
			if (SaveContext.PackageWriterCapabilities.bReadOnly)
			{
				continue;
			}

			PruneShaderLibrary(TargetPlatform, LibraryName);
		}
	}

	PlatformIndex = 0;
	for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
	{
		FEndCookPlatformData& EndCookPlatformData = EndCookPlatformDatas[PlatformIndex++];
		FCookSavePackageContext& SaveContext = FindOrCreateSaveContext(TargetPlatform);
		if (SaveContext.PackageWriterCapabilities.bReadOnly)
		{
			continue;
		}

		FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
		FAssetRegistryGenerator& Generator = *PlatformData->RegistryGenerator;

		FString PlatformNameString = TargetPlatform->PlatformName();

		TArray<FPackageData*>& CookedPackageDatas = EndCookPlatformData.CookedPackageDatas;
		TArray<FPackageData*>& IgnorePackageDatas = EndCookPlatformData.IgnorePackageDatas;
		TSet<FName>& CookedPackageNames = EndCookPlatformData.CookedPackageNames;
		TSet<FName> IgnorePackageNames;
		CookedPackageNames.Reserve(CookedPackageDatas.Num());
		for (FPackageData* PackageData : CookedPackageDatas)
		{
			CookedPackageNames.Add(PackageData->GetPackageName());
		}
		IgnorePackageNames.Reserve(IgnorePackageDatas.Num());
		for (FPackageData* PackageData : IgnorePackageDatas)
		{
			IgnorePackageNames.Add(PackageData->GetPackageName());
		}
	
		if (bSaveManifests || bSaveAssetRegistry)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(GeneratorPreSave);

			// Make changes to all AssetPackageDatas to record data from the cook
			// PackageHashes are guaranteed finished by UPackage::WaitForAsyncFileWrites(), which is called above.
			TMap<FName, TRefCountPtr<FPackageHashes>>& AllPackageHashes = SaveContext.PackageWriter->GetPackageHashes();
			for (TPair<FName, TRefCountPtr<FPackageHashes>>& HashSet : AllPackageHashes)
			{
				FAssetPackageData* AssetPackageData = Generator.GetAssetPackageData(HashSet.Key);

				// Add the package hashes to the relevant AssetPackageDatas.
				TRefCountPtr<FPackageHashes>& PackageHashes = HashSet.Value;
				AssetPackageData->CookedHash = PackageHashes->PackageHash;
				Move(AssetPackageData->ChunkHashes, PackageHashes->ChunkHashes);

				// Mark that the Assets in the cooked files are from the IoDispatcher.
				// This assumes that all cooked files are loaded by the IoDispatcher, which is the case for the primary 
				// supported workflow, but it is possible that a developer could choose to stage to loose files, in which case
				// IoDispatcher would be incorrect.
				// TODO: Modify the package location in the cooked asset registry when staging to loose files.
				AssetPackageData->SetPackageLocation(FPackageName::EPackageLocationFilter::IoDispatcher);
			}

			Generator.PreSave(CookedPackageNames);
		}

		if (bSaveManifests)
		{
			{
				UE_SCOPED_HIERARCHICAL_COOKTIMER(BuildChunkManifest);
				Generator.FinalizeChunkIDs(CookedPackageNames, IgnorePackageNames, *SandboxFile,
					CookByTheBookOptions->bGenerateStreamingInstallManifests, CookByTheBookOptions->StartupPackages,
					PlatformData->AssetRegistryDevelopmentSaveOptions);
			}
			{
				UE_SCOPED_HIERARCHICAL_COOKTIMER(SaveManifests);
				if (!Generator.SaveManifests(*SandboxFile, *this))
				{
					UE_LOGF(LogCook, Warning, "Failed to save chunk manifest");
				}

				int64 ExtraFlavorChunkSize;
				if (FParse::Value(FCommandLine::Get(), TEXT("ExtraFlavorChunkSize="), ExtraFlavorChunkSize) && ExtraFlavorChunkSize > 0)
				{
					// ExtraFlavor is a legacy term for this override; etymology unknown. Override the chunksize specified by the platform,
					// and write the manifest files created with that chunksize into a separate subdirectory.
					const TCHAR* ManifestSubDir = TEXT("ExtraFlavor");
					if (!Generator.SaveManifests(*SandboxFile, *this, ExtraFlavorChunkSize, ManifestSubDir))
					{
						UE_LOGF(LogCook, Warning, "Failed to save chunk manifest");
					}
				}
			}
		}

		uint64 DevelopmentAssetRegistryHash = 0; // The hashes of the entire files for the platform
		if (bSaveAssetRegistry)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(SaveAssetRegistry);
			Generator.SaveAssetRegistry(SandboxRegistryFilename, bSaveDevelopmentAssetRegistry,
				bForceNoFilterAssetsFromAssetRegistry, DevelopmentAssetRegistryHash, CookedPackageNames,
				PlatformData->AssetRegistryDevelopmentSaveOptions);
		}

		if (bSaveManifests || bSaveAssetRegistry)
		{
			Generator.PostSave();
		}

		if (bSaveCookerOpenOrder)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(WriteCookerOpenOrder);
			if (!IsCookFlagSet(ECookInitializationFlags::LegacyIterative))
			{
				Generator.WriteCookerOpenOrder(*SandboxFile);
			}
		}

		if (bCacheShaderLibraries)
		{
			// now that we have the asset registry and cooking open order, we have enough information to split the shader library
			// into parts for each chunk and (possibly) lay out the code in accordance with the file order
			// Assert that the other saves are enabled because we depend on those files being written.
			check(bSaveCookerOpenOrder && bSaveAssetRegistry);
			// Save shader code map
			SaveShaderLibrary(TargetPlatform, LibraryName);
			CreatePipelineCache(TargetPlatform, LibraryName);
			DumpShaderTypeStats(TargetPlatform, PlatformNameString);
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("fastcook")))
		{
			FFileHelper::SaveStringToFile(FString(), *(GetSandboxDirectory(PlatformNameString) / TEXT("fastcook.txt")));
		}

		if (bSaveAssetRegistry && IsCreatingReleaseVersion())
		{
			const FString VersionedRegistryPath = GetCreateReleaseVersionAssetRegistryPath(CookByTheBookOptions->CreateReleaseVersion, PlatformNameString);
			IFileManager::Get().MakeDirectory(*VersionedRegistryPath, true);
			const FString VersionedRegistryFilename = VersionedRegistryPath / GetAssetRegistryFilename();
			const FString CookedAssetRegistryFilename = SandboxRegistryFilename.Replace(TEXT("[Platform]"), *PlatformNameString);
			IFileManager::Get().Copy(*VersionedRegistryFilename, *CookedAssetRegistryFilename, true, true);

			// Also copy development registry if it exists
			FString DevelopmentAssetRegistryRelativePath = FString::Printf(TEXT("Metadata/%s"), GetDevelopmentAssetRegistryFilename());
			const FString DevVersionedRegistryFilename = VersionedRegistryFilename.Replace(TEXT("AssetRegistry.bin"), *DevelopmentAssetRegistryRelativePath);
			const FString DevCookedAssetRegistryFilename = CookedAssetRegistryFilename.Replace(TEXT("AssetRegistry.bin"), *DevelopmentAssetRegistryRelativePath);
			IFileManager::Get().Copy(*DevVersionedRegistryFilename, *DevCookedAssetRegistryFilename, true, true);
		}

		// Write cook metadata file for each platform
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(WriteCookMetadata);
			WriteCookMetadata(TargetPlatform, DevelopmentAssetRegistryHash);
		}

		// Write ReferencedSet for use by staging and zen commands on incremental cook oplogs: they use only the ops
		// referenced by the most recent cook.
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(WriteReferencedSet);
			WriteReferencedSet(TargetPlatform, CookedPackageNames.Array());
		}
	}

	if (!PlatformManager->GetSessionPlatforms().IsEmpty() && IncrementallyModifiedDiagnostics)
	{
		const ITargetPlatform* TargetPlatform = PlatformManager->GetSessionPlatforms()[0];
		FString DiagnosticsDir;
		if (GIsBuildMachine)
		{
			DiagnosticsDir = FPaths::Combine(*FPaths::EngineDir(), TEXT("Programs"), TEXT("AutomationTool"), TEXT("Saved"), TEXT("Logs"), TargetPlatform->PlatformName());
		}
		else
		{
			DiagnosticsDir = GetCookMetadataOutputFolderInternal(TargetPlatform);
		}
		IncrementallyModifiedDiagnostics->WriteFinal(DiagnosticsDir);
	}

	UE_LOGF(LogCookStatus, Display, "Shutting down shader library cooker & shader compilers");
	ShutdownShaderLibraryCookerAndCompilers(LibraryName);

	if (CookByTheBookOptions->bGenerateDependenciesForMaps)
	{
		UE_LOGF(LogCookStatus, Display, "Generating dependencies for maps");

		UE_SCOPED_HIERARCHICAL_COOKTIMER(GenerateMapDependencies);
		for (const ITargetPlatform* Platform : PlatformManager->GetSessionPlatforms())
		{
			if (FindOrCreateSaveContext(Platform).PackageWriterCapabilities.bReadOnly)
			{
				continue;
			}

			TMap<FName, TSet<FName>> MapDependencyGraph = BuildMapDependencyGraph(Platform);
			WriteMapDependencyGraph(Platform, MapDependencyGraph);
		}
	}


	if (FParse::Param(FCommandLine::Get(), TEXT("CachedEditorThumbnails")))
	{
		UE_LOGF(LogCookStatus, Display, "Generating editor thumbnails");
		GenerateCachedEditorThumbnails();
	}

	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(FinalizePackageStore);
		UE_LOGF(LogCook, Display, "Finalize package store(s)...");
		PlatformIndex = 0;
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			FEndCookPlatformData& EndCookPlatformData = EndCookPlatformDatas[PlatformIndex++];
			FinalizePackageStore(TargetPlatform, EndCookPlatformData.CookedPackageNames);
		}
	}

	UE_LOGF(LogCookStatus, Display, "FinishedInternal: Done");
}

bool UCookOnTheFlyServer::IsForceNoFilterAssetsFromAssetRegistry()
{
	if (IsDirectorCookByTheBook())
	{
		return IsCookingDLC() ? !FParse::Param(FCommandLine::Get(), TEXT("ApplyFiltersToDLCAssetRegistry")) : false;
	}
	else
	{
		return false;
	}
}

void UCookOnTheFlyServer::ShutdownCookSession()
{
	ODSCClientData.Reset();

	if (CookDirector)
	{
		CookDirector->ShutdownCookSession();
	}

	if (IsCookByTheBookMode())
	{
		// CookWorkers report false for IsCookByTheBookMode; they are CookWorker mode. They need to shutdown in a
		// custom manner, which we do in the else if below.
		check(!CookWorkerClient);
		UnregisterCookByTheBookDelegates();

		PrintFinishStats();
		OutputHierarchyTimers();
		if (PlatformManager->GetSessionPlatforms().Num() > 0 &&
			PlatformManager->GetPlatformData(PlatformManager->GetSessionPlatforms()[0])->bAllowIncrementalResults)
		{
			OutputIncrementalCookStats();
		}
		PrintDetailedCookStats();

		// BroadcastCookFinished needs to be called before clearing the session data, so that subscribers
		// can access information about the session such as DLCName.
		BroadcastCookFinished();
	}
	else if (IsCookOnTheFlyMode())
	{
		BroadcastCookFinished();
	}
	else if (CookWorkerClient)
	{
		CookAsCookWorkerFinished();
		// CookAsCookWorkerFinished is responsible for calling BroadcastCookFinished.
	}

	CookByTheBookOptions->ClearSessionData();
	PlatformManager->ClearSessionPlatforms(*this);
	ClearHierarchyTimers();
}

void UCookOnTheFlyServer::PrintFinishStats()
{
	using namespace UE::Cook;

	const float TotalCookTime = (float)(FPlatformTime::Seconds() - CookByTheBookOptions->CookStartTime);
	if (IsCookByTheBookMode())
	{
		UE_LOGF(LogCook, Display, "Cook by the book total time in tick %fs total time %f", CookByTheBookOptions->CookTime, TotalCookTime);
	}
	else if (IsCookWorkerMode())
	{
		UE_LOGF(LogCook, Display, "CookWorker total time %f", TotalCookTime);
	}

	// Suppress NumPackagesIncrementallySkipped display if the PackageWriter is modifying what gets skipped
	bool bReportIncrementalSkips = true;
	int32 ReportedNumPackagesIncrementallySkipped = DetailedCookStats::NumPackagesIncrementallySkipped;
	const ITargetPlatform* FirstTargetPlatform = PlatformManager->GetSessionPlatforms().IsEmpty() ? nullptr :
		PlatformManager->GetSessionPlatforms()[0];
	if (FirstTargetPlatform)
	{
		if (FindOrCreateSaveContext(FirstTargetPlatform).PackageWriterCapabilities.bOverridesPackageModificationStatus)
		{
			bReportIncrementalSkips = false;
			ReportedNumPackagesIncrementallySkipped = 0;
		}
	}
	int32 ReportedNumCooked = PackageDatas->GetNumCooked(ECookResult::Succeeded)
		- ReportedNumPackagesIncrementallySkipped - PackageDataFromBaseGameNum;
	int32 ReportedTotalPackages = PackageDatas->GetNumCooked()
		- PackageDatas->GetNumCooked(ECookResult::NeverCookPlaceholder) - PackageDataFromBaseGameNum;

	const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();
	UE_LOGF(LogCook, Display, "Peak Used virtual %llu MiB Peak Used physical %llu MiB", MemStats.PeakUsedVirtual / 1024 / 1024, MemStats.PeakUsedPhysical / 1024 / 1024);

	COOK_STAT(UE_LOGF(LogCook, Display,
		"Packages Cooked: %d,%ls Packages Skipped by Platform: %d, Total Packages: %d",
		ReportedNumCooked,
		(bReportIncrementalSkips
			? *FString::Printf(TEXT(" Packages Incrementally Skipped: %d,"), ReportedNumPackagesIncrementallySkipped)
			: TEXT("")),
		PackageDatas->GetNumCooked(ECookResult::Failed),
		ReportedTotalPackages
	));
}

void UCookOnTheFlyServer::PrintDetailedCookStats()
{
	// Stats are aggregated on the director, so writing the stats CSVs is only needed on the director 
	// (or single cook process in the sp cook case)
	if (!IsCookWorkerMode())
	{
		FShaderStatsFunctions::WriteShaderStats();
	}

	COOK_STAT(
		{
			double Now = FPlatformTime::Seconds();
			if (DetailedCookStats::CookStartTime <= 0.)
			{
				DetailedCookStats::CookStartTime = GStartTime;
			}
			DetailedCookStats::CookWallTimeSec = Now - GStartTime;
			DetailedCookStats::StartupWallTimeSec = DetailedCookStats::CookStartTime - GStartTime;
			DetailedCookStats::SendLogCookStats(CurrentCookMode);
		});
}

TMap<FName, TSet<FName>> UCookOnTheFlyServer::BuildMapDependencyGraph(const ITargetPlatform* TargetPlatform)
{
	TMap<FName, TSet<FName>> MapDependencyGraph;

	TArray<UE::Cook::FPackageData*> PlatformCookedPackages;
	TArray<UE::Cook::FPackageData*> FailedPackages;
	PackageDatas->GetCommittedPackagesForPlatform(TargetPlatform, PlatformCookedPackages, FailedPackages);

	// assign chunks for all the map packages
	for (const UE::Cook::FPackageData* const CookedPackage : PlatformCookedPackages)
	{
		FName Name = CookedPackage->GetPackageName();

		if (!ContainsMap(Name))
		{
			continue;
		}

		TSet<FName> DependentPackages;
		TSet<FName> Roots; 

		Roots.Add(Name);

		GetDependentPackages(Roots, DependentPackages);

		MapDependencyGraph.Add(Name, DependentPackages);
	}
	return MapDependencyGraph;
}

void UCookOnTheFlyServer::WriteMapDependencyGraph(const ITargetPlatform* TargetPlatform, TMap<FName, TSet<FName>>& MapDependencyGraph)
{
	FString MapDependencyGraphFile = FPaths::ProjectDir() / TEXT("MapDependencyGraph.json");
	// dump dependency graph. 
	FString DependencyString;
	DependencyString += "{";
	for (auto& Ele : MapDependencyGraph)
	{
		TSet<FName>& Deps = Ele.Value;
		FName MapName = Ele.Key;
		DependencyString += TEXT("\t\"") + MapName.ToString() + TEXT("\" : \n\t[\n ");
		for (FName& Val : Deps)
		{
			DependencyString += TEXT("\t\t\"") + Val.ToString() + TEXT("\",\n");
		}
		DependencyString.RemoveFromEnd(TEXT(",\n"));
		DependencyString += TEXT("\n\t],\n");
	}
	DependencyString.RemoveFromEnd(TEXT(",\n"));
	DependencyString += "\n}";

	FString CookedMapDependencyGraphFilePlatform = ConvertToFullSandboxPath(MapDependencyGraphFile, true).Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
	FFileHelper::SaveStringToFile(DependencyString, *CookedMapDependencyGraphFilePlatform, FFileHelper::EEncodingOptions::ForceUnicode);
}

void UCookOnTheFlyServer::GenerateCachedEditorThumbnails()
{
	UE_SCOPED_HIERARCHICAL_COOKTIMER(GenerateCachedEditorThumbnails);
	for (const ITargetPlatform* Platform : PlatformManager->GetSessionPlatforms())
	{
		// CachedEditorThumbnails only make sense for data used in the editor
		if (Platform->PlatformName() != TEXT("WindowsClient"))
		{
			continue;
		}

		const FString CachedEditorThumbnailsFilename = GetSandboxCachedEditorThumbnailsFilename();
		UE_LOGF(LogCook, Display, "Generating %ls", *CachedEditorThumbnailsFilename);

		// Gather public assets
		// We don't need thumbnails for private assets because they aren't visible in the editor
		TArray<FAssetData> PublicAssets;
		{
			TArray<UE::Cook::FPackageData*> CookedPackages;
			{
				TArray<UE::Cook::FPackageData*> FailedPackages;
				PackageDatas->GetCommittedPackagesForPlatform(Platform, CookedPackages, FailedPackages);
			}

			for (const UE::Cook::FPackageData* const CookedPackage : CookedPackages)
			{
				if (CookedPackage->GetWasCookedThisSession())
				{
					TArray<FAssetData> Assets;
					ensure(AssetRegistry->GetAssetsByPackageName(CookedPackage->GetPackageName(), Assets, /*bIncludeOnlyDiskAssets=*/true));

					for (FAssetData& Asset : Assets)
					{
						if (Asset.GetAssetAccessSpecifier() != EAssetAccessSpecifier::Private)
						{
							UE_LOGF(LogCook, Verbose, "Adding thumbnail for '%ls'", *Asset.GetObjectPathString());
							PublicAssets.Add(MoveTemp(Asset));
						}
					}
				}
			}
		}

		// Write CachedEditorThumbnails.bin file
		if (!PublicAssets.IsEmpty())
		{
			FThumbnailExternalCacheSettings Settings;
			// Convert lossless thumbnails to lossy to save space
			Settings.bRecompressLossless = true;
			Settings.MaxImageSize = ThumbnailTools::DefaultThumbnailSize;

			// Sort to be deterministic
			FThumbnailExternalCache::SortAssetDatas(PublicAssets);
			FThumbnailExternalCache::Get().SaveExternalCache(CachedEditorThumbnailsFilename, PublicAssets, Settings);
		}
	}
}

void UCookOnTheFlyServer::QueueCancelCookByTheBook()
{
	if (IsCookByTheBookMode() && IsInSession())
	{
		QueuedCancelPollable->Trigger(*this);
	}
}

void UCookOnTheFlyServer::PollQueuedCancel(UE::Cook::FTickStackData& StackData)
{
	StackData.bCookCancelled = true;
	StackData.ResultFlags |= COSR_YieldTick;
}

void UCookOnTheFlyServer::CancelCookByTheBook()
{
	check(IsCookByTheBookMode());
	check(IsInGameThread());
	if (IsInSession())
	{
		CancelAllQueues();
		ShutdownCookSession();
		UE::Cook::FTickStackData StackData(MAX_flt, ECookTickFlags::None);
		SetIdleStatus(StackData, EIdleStatus::Done);
	} 
}

void UCookOnTheFlyServer::StopAndClearCookedData()
{
	if ( IsCookByTheBookMode() )
	{
		CancelCookByTheBook();
	}
	else
	{
		CancelAllQueues();
	}

	PackageTracker->RecompileRequests.Empty();
	PackageTracker->UnsolicitedCookedPackages.Empty();
	PackageDatas->ClearCookedPlatforms();
}

void UCookOnTheFlyServer::ClearAllCookedData()
{
	checkf(!IsInSession(), TEXT("We do not handle removing SessionPlatforms, so ClearAllCookedData must not be called while in a cook session"));

	// if we are going to clear the cooked packages it is conceivable that we will recook the packages which we just cooked 
	// that means it's also conceivable that we will recook the same package which currently has an outstanding async write request
	UPackage::WaitForAsyncFileWrites();

	PackageTracker->UnsolicitedCookedPackages.Empty();
	PackageDatas->ClearCookedPlatforms();
	ClearPackageStoreContexts();
}

void UCookOnTheFlyServer::CancelAllQueues()
{
	// Discard the external build requests, but execute any pending SchedulerCallbacks since these might have important teardowns
	TArray<UE::Cook::FSchedulerCallback> SchedulerCallbacks;
	TArray<UE::Cook::FFilePlatformRequest> UnusedRequests;
	WorkerRequests->DequeueAllExternal(SchedulerCallbacks, UnusedRequests);
	for (UE::Cook::FSchedulerCallback& SchedulerCallback : SchedulerCallbacks)
	{
		SchedulerCallback();
	}

	using namespace UE::Cook;
	// Remove all elements from all Queues and send them to Idle
	FPackageDataQueue& SaveQueue = PackageDatas->GetSaveQueue();
	while (!SaveQueue.IsEmpty())
	{
		DemoteToIdle(*SaveQueue.PopFrontValue(), ESendFlags::QueueAdd, ESuppressCookReason::CookCanceled);
	}
	FLoadQueue& LoadQueue = PackageDatas->GetLoadQueue();
	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	TArray<FPackageData*> DatasInStatesWithTSet;
	DatasInStatesWithTSet.Reset(LoadQueue.Num()
		+ PackageDatas->GetAssignedToWorkerSet().Num()
		+ PackageDatas->GetSaveStalledSet().Num()
		+ RequestQueue.GetRestartedRequests().Num());
	for (FPackageData* PackageData : LoadQueue)
	{
		DatasInStatesWithTSet.Add(PackageData);
	}
	for (FPackageData* PackageData : PackageDatas->GetAssignedToWorkerSet())
	{
		DatasInStatesWithTSet.Add(PackageData);
	}
	for (FPackageData* PackageData : PackageDatas->GetSaveStalledSet())
	{
		DatasInStatesWithTSet.Add(PackageData);
	}
	for (TPair<FPackageData*, ESuppressCookReason>& Pair : RequestQueue.GetRestartedRequests())
	{
		DatasInStatesWithTSet.Add(Pair.Key);
	}
	for (FPackageData* PackageData : DatasInStatesWithTSet)
	{
		DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::CookCanceled);
	}
	RequestQueue.GetDiscoveryQueue().Empty();
	RequestQueue.GetBuildDependencyDiscoveryQueue().Empty();
	TRingBuffer<TUniquePtr<FRequestCluster>>& RequestClusters = RequestQueue.GetRequestClusters();
	for (TUniquePtr<FRequestCluster>& RequestCluster : RequestClusters)
	{
		TArray<FPackageData*> RequestsToLoad;
		TArray<TPair<FPackageData*, ESuppressCookReason>> RequestsToDemote;
		TMap<FPackageData*, TArray<FPackageData*>> UnusedRequestGraph;
		RequestCluster->ClearAndDetachOwnedPackageDatas(RequestsToLoad, RequestsToDemote, UnusedRequestGraph);
		for (FPackageData* PackageData : RequestsToLoad)
		{
			DemoteToIdle(*PackageData, ESendFlags::QueueAdd, ESuppressCookReason::CookCanceled);
		}
		for (TPair<FPackageData*, ESuppressCookReason>& Pair : RequestsToDemote)
		{
			DemoteToIdle(*Pair.Key, ESendFlags::QueueAdd, ESuppressCookReason::CookCanceled);
		}
	}
	RequestClusters.Empty();

	while (!RequestQueue.IsReadyRequestsEmpty())
	{
		DemoteToIdle(*RequestQueue.PopReadyRequest(), ESendFlags::QueueAdd, ESuppressCookReason::CookCanceled);
	}

	SetLoadBusy(false);
	SetSaveBusy(false, 0);
}


void UCookOnTheFlyServer::ClearPlatformCookedData(const ITargetPlatform* TargetPlatform)
{
	if (!TargetPlatform)
	{
		return;
	}
	if (!SandboxFile)
	{
		// We cannot get the PackageWriter without it, and we do not have anything to clear if it has not been created
		return;
	}
	ResetCook({ TPair<const ITargetPlatform*,bool>{TargetPlatform, true /* bResetResults */}});

	FindOrCreatePackageWriter(TargetPlatform).RemoveCookedPackages();
}

void UCookOnTheFlyServer::ResetCook(TConstArrayView<TPair<const ITargetPlatform*, bool>> TargetPlatforms)
{
	using namespace UE::Cook;

	PackageDatas->LockAndEnumeratePackageDatas([TargetPlatforms](UE::Cook::FPackageData* PackageData)
	{
		PackageData->FindOrAddPlatformData(CookerLoadingPlatformKey).ResetReachable(EReachability::All);

		for (const TPair<const ITargetPlatform*, bool>& Pair : TargetPlatforms)
		{
			const ITargetPlatform* TargetPlatform = Pair.Key;
			FPackagePlatformData* PlatformData = PackageData->FindPlatformData(TargetPlatform);
			if (PlatformData)
			{
				bool bResetResults = Pair.Value;
				PlatformData->ResetReachable(EReachability::All);
				if (bResetResults)
				{
					PackageData->ClearCookResults(TargetPlatform);
				}
			}
		}

		PackageData->SetSuppressCookReason(ESuppressCookReason::NotSuppressed);
		PackageData->SetLeafToRootRank(MAX_uint32);
	});

	PackageDatas->ResetLeafToRootRank();

	TArray<FName> PackageNames;
	for (const TPair<const ITargetPlatform*, bool>& Pair : TargetPlatforms)
	{
		const ITargetPlatform* TargetPlatform = Pair.Key;
		bool bResetResults = Pair.Value;
		if (bResetResults)
		{
			PackageNames.Reset();
			PackageTracker->UnsolicitedCookedPackages.GetPackagesForPlatformAndRemove(TargetPlatform, PackageNames);
		}
	}
}

void UCookOnTheFlyServer::ClearCachedCookedPlatformDataForPlatform(const ITargetPlatform* TargetPlatform)
{
	if (TargetPlatform)
	{
		for (TObjectIterator<UObject> It; It; ++It)
		{
			It->ClearCachedCookedPlatformData(TargetPlatform);
		}
	}
}

void UCookOnTheFlyServer::OnTargetPlatformChangedSupportedFormats(const ITargetPlatform* TargetPlatform)
{
	for (TObjectIterator<UObject> It; It; ++It)
	{
		It->ClearCachedCookedPlatformData(TargetPlatform);
	}
}

void UCookOnTheFlyServer::CreateSandboxFile(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(CreateSandboxFile);
	// Output directory override. This directory depends on whether we are cooking dlc, so we cannot
	// create the sandbox until after StartCookByTheBook or StartCookOnTheFly
	FString OutputDirectory = GetOutputDirectoryOverride(BeginContext);
	check(!OutputDirectory.IsEmpty());
	check((SandboxFile == nullptr) == SandboxFileOutputDirectory.IsEmpty());

	if (SandboxFile)
	{
		if (SandboxFileOutputDirectory == OutputDirectory)
		{
			return;
		}
		ClearAllCookedData(); // Does not delete files on disk, only deletes in-memory data
		SandboxFile.Reset();
	}

	// Filename lookups in the cooker must Use this SandboxFile to do path conversion to properly handle sandbox paths
	// (outside of standard paths in particular).
	SandboxFile.Reset(new UE::Cook::FCookSandbox(OutputDirectory, PluginsToRemap));
	SandboxFileOutputDirectory = OutputDirectory;

}

void UCookOnTheFlyServer::LoadBeginCookConfigSettings(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(LoadBeginCookConfigSettings);
	UE::Cook::FBeginCookConfigSettings Settings;
	WorkerRequests->GetBeginCookConfigSettings(*this, BeginContext, Settings);
	SetBeginCookConfigSettings(BeginContext, MoveTemp(Settings));
}

namespace UE::Cook
{

struct FDependencyTrackingConfigParser
{
	void Parse();

	// Input parameters
	bool bInIssueLogs = false;
	TOptional<bool> bInUsingZenStore;

	// Output environment settings
	bool bOutLegacyBuildDependencies = false;
	bool bOutLegacyBuildDependenciesUsedDueToNoZenStore = false;
	bool bOutDependencyTrackingEnabled = true;
};

void FDependencyTrackingConfigParser::Parse()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	GConfig->GetBool(TEXT("CookSettings"), TEXT("LegacyIterative"), bOutLegacyBuildDependencies, GEditorIni);

	if (!bOutLegacyBuildDependencies && FParse::Param(CommandLine, TEXT("ODSC")))
	{
		// INCREMENTALCOOK_TODO: implement incremental cook for ODSC
		UE_CLOGF(bInIssueLogs, LogCook, Display,
			"INCREMENTAL COOK DEPENDENCIES: Disabled. The cooker is running as ODSC which forces use of legacyiterative.");
		bOutLegacyBuildDependencies = true;
	}
	// TODO: Parse bIsUsingZenStore in this function from config and commandline, so that we can have a more accurate result
	// for bDependencyTrackingEnabled for callers that don't know the value of bInUsingZenStore.
	else if (!bOutLegacyBuildDependencies && bInUsingZenStore.IsSet() && !*bInUsingZenStore)
	{
		// Incremental cook uses TargetDomain storage of dependencies which is only implemented in ZenStore
		UE_CLOGF(bInIssueLogs, LogCook, Display,
			"INCREMENTAL COOK DEPENDENCIES: Disabled. Incremental Cook Dependencies are disabled because the cooker is not using ZenStore, " "and we do not yet support storage of Incremental Cook dependency data with loose cooked packages. " "Falling back to legacy iterative cook mode.");
		bOutLegacyBuildDependencies = true;
		bOutLegacyBuildDependenciesUsedDueToNoZenStore = true;
	}
	else if (!bOutLegacyBuildDependencies)
	{
		UE_CLOGF(bInIssueLogs, LogCook, Display,
			"INCREMENTAL COOK DEPENDENCIES: Enabled. Incremental Cook Dependencies are enabled in Editor.ini:[CookSettings]:LegacyIterative=false.");
	}
	else
	{
		UE_CLOGF(bInIssueLogs, LogCook, Display,
			"INCREMENTAL COOK DEPENDENCIES: Disabled. Incremental Cook Dependencies are disabled in Editor.ini:[CookSettings]:LegacyIterative=true.");
	}

	if (bOutLegacyBuildDependencies)
	{
		bOutDependencyTrackingEnabled = false;
	}
	else
	{
		bOutDependencyTrackingEnabled = true;
		FParse::Bool(CommandLine, TEXT("-Cook.DependencyTracking="), bOutDependencyTrackingEnabled);
	}
}
bool IsDependencyTrackingEnabled()
{
	FDependencyTrackingConfigParser Parser;
	Parser.Parse();
	return Parser.bOutDependencyTrackingEnabled;
}

void FBeginCookConfigSettings::LoadLocal(FBeginCookContext& BeginContext)
{
	FDependencyTrackingConfigParser Parser;
	Parser.bInIssueLogs= true;
	Parser.bInUsingZenStore.Emplace(BeginContext.COTFS.IsUsingZenStore());
	Parser.Parse();
	bLegacyBuildDependencies = Parser.bOutLegacyBuildDependencies;
	bLegacyBuildDependenciesDueToNoZenStore = Parser.bOutLegacyBuildDependenciesUsedDueToNoZenStore;
	bDependencyTrackingEnabled = Parser.bOutDependencyTrackingEnabled;

	const TCHAR* CommandLine = FCommandLine::Get();
	bCookIncrementalAllowAllClasses = FParse::Param(CommandLine, TEXT("CookIncrementalAllowAllClasses"));

	FParse::Value(CommandLine, TEXT("-CookShowInstigator="), CookShowInstigator);
	LoadNeverCookLocal(BeginContext);

	for (const ITargetPlatform* TargetPlatform : BeginContext.TargetPlatforms)
	{
		FConfigFile PlatformEngineIni;
		FConfigCacheIni::LoadLocalIniFile(PlatformEngineIni, TEXT("Engine"), true, *TargetPlatform->IniPlatformName());

		bool bLegacyBulkDataOffsets = false;
		PlatformEngineIni.GetBool(TEXT("Core.System"), TEXT("LegacyBulkDataOffsets"), bLegacyBulkDataOffsets);
		if (bLegacyBulkDataOffsets)
		{
			UE_LOGF(LogCook, Warning, "Engine.ini:[Core.System]:LegacyBulkDataOffsets is no longer supported in UE5. The intended use was to reduce patch diffs, but UE5 changed cooked bytes in every package for other reasons, so removing support for this flag does not cause additional patch diffs.");
		}
	}
}

}

void UCookOnTheFlyServer::SetBeginCookConfigSettings(FBeginCookContext& BeginContext, UE::Cook::FBeginCookConfigSettings&& Settings)
{
	bLegacyBuildDependencies = Settings.bLegacyBuildDependencies;
	bLegacyBuildDependenciesDueToNoZenStore = Settings.bLegacyBuildDependenciesDueToNoZenStore;
	bDependencyTrackingEnabled = Settings.bDependencyTrackingEnabled;
	bCookIncrementalAllowAllClasses = Settings.bCookIncrementalAllowAllClasses;
	PackageDatas->SetBeginCookConfigSettings(Settings.CookShowInstigator);
	UE::Cook::FGenerationHelper::SetBeginCookConfigSettings();
	SetNeverCookPackageConfigSettings(BeginContext, Settings);
	UE::Cook::CVarControl::UpdateCVars(BeginContext, OverrideDeviceProfileName, OverrideCookCVarControl);
}

namespace UE::Cook
{

void FBeginCookConfigSettings::LoadNeverCookLocal(FBeginCookContext& BeginContext)
{
	NeverCookPackageList.Reset();
	PlatformSpecificNeverCookPackages.Reset();

	TArrayView<const FString> ExtraNeverCookDirectories;
	if (BeginContext.StartupOptions)
	{
		ExtraNeverCookDirectories = BeginContext.StartupOptions->NeverCookDirectories;
	}
	for (FName NeverCookPackage : BeginContext.COTFS.GetNeverCookPackageNames(ExtraNeverCookDirectories))
	{
		NeverCookPackageList.Add(NeverCookPackage);
	}

	// use temp list of UBT platform strings to discover PlatformSpecificNeverCookPackages
	if (BeginContext.TargetPlatforms.Num())
	{
		TArray<FString> UBTPlatformStrings;
		UBTPlatformStrings.Reserve(BeginContext.TargetPlatforms.Num());
		for (const ITargetPlatform* Platform : BeginContext.TargetPlatforms)
		{
			FString UBTPlatformName;
			Platform->GetPlatformInfo().UBTPlatformName.ToString(UBTPlatformName);
			UBTPlatformStrings.Emplace(MoveTemp(UBTPlatformName));
		}

		BeginContext.COTFS.DiscoverPlatformSpecificNeverCookPackages(BeginContext.TargetPlatforms, UBTPlatformStrings, *this);
	}
}

}

void UCookOnTheFlyServer::SetNeverCookPackageConfigSettings(FBeginCookContext& BeginContext, UE::Cook::FBeginCookConfigSettings& Settings)
{
	UE::Cook::FThreadSafeSet<FName>& NeverCookPackageList = PackageTracker->NeverCookPackageList;
	NeverCookPackageList.Empty();
	for (FName PackageName : Settings.NeverCookPackageList)
	{
		NeverCookPackageList.Add(PackageName);
	}
	PackageTracker->PlatformSpecificNeverCookPackages = MoveTemp(Settings.PlatformSpecificNeverCookPackages);
}

void UCookOnTheFlyServer::LoadBeginCookIncrementalFlags(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(LoadBeginCookIncrementalFlags);
	WorkerRequests->GetBeginCookIncrementalFlags(*this, BeginContext);
}

void UCookOnTheFlyServer::LoadBeginCookIncrementalFlagsLocal(FBeginCookContext& BeginContext)
{
	const TCHAR* CommandLine = FCommandLine::Get();
	const bool bIsDiffOnly = FParse::Param(CommandLine, TEXT("DIFFONLY"));
	// When running with incrementalvalidate without allowwrite, we have a contract with our user that
	// we will not wipe their state, even if we find that an incremental build is not normally allowed.
	// Abort the cook in that case instead.
	const bool bFullBuildAllowed = DiffModeHelper->IsFullBuildAllowed();

	bool bDefaultIncremental = true;
	if (!bLegacyBuildDependencies)
	{
		GConfig->GetBool(TEXT("CookSettings"), TEXT("CookIncrementalDefaultIncremental"), bDefaultIncremental, GEditorIni);
	}
	else
	{
		bDefaultIncremental = IsCookFlagSet(ECookInitializationFlags::LegacyIterative);
	}

	bool bForceRecook = !bDefaultIncremental;
	bool bForceRecookCommandLineArgPresent = false;
	if (FParse::Param(CommandLine, TEXT("CookIncremental")))
	{
		bForceRecook = false;
		bForceRecookCommandLineArgPresent = true;
	}
	if (IsCookFlagSet(ECookInitializationFlags::LegacyIterative))
	{
		bForceRecook = false;
		bForceRecookCommandLineArgPresent = true;
	}
	FString Text;
	if (FParse::Value(CommandLine, TEXT("-CookIncremental="), Text))
	{
		bool bValue = true;
		LexFromString(bValue, Text);
		bForceRecook = !bValue;
		bForceRecookCommandLineArgPresent = true;
	}
	if (FParse::Param(CommandLine, TEXT("fullcook")) || FParse::Param(CommandLine, TEXT("forcerecook")))
	{
		bForceRecook = true;
		bForceRecookCommandLineArgPresent = true;
	}
	if (FParse::Value(CommandLine, TEXT("-forcerecook="), Text))
	{
		LexFromString(bForceRecook, Text);
		bForceRecookCommandLineArgPresent = true;
	}

	const bool bIncrementalOrLegacyIterative = !bForceRecook &&
		(!bLegacyBuildDependencies || IsCookFlagSet(ECookInitializationFlags::LegacyIterative));
	const bool bIsSharedLegacyIterativeCook = bLegacyBuildDependencies &&
		IsCookFlagSet(ECookInitializationFlags::LegacyIterativeSharedBuild);

	if (bIncrementalOrLegacyIterative && bLegacyBuildDependenciesDueToNoZenStore
		&& EnumHasAnyFlags(CookFlags, ECookInitializationFlags::Unversioned))
	{
		UE_LOGF(LogCook, Warning,
			"CookIncremental and UnversionedCookedContent were requested, but LegacyIterative is required due to ZenStore not being used. "
			"LegacyIterative incremental cooks do not robustly support unversioned cooked content due to failure to track dependencies on c++ serialization changes. "
			"The cooker will cook with these settings, but may fail to recook packages that require it those packages will cause runtime crashes when loaded. "
			"For robust incremental cooks, set bUseZenStore=true and bUseIoStore=true in ProjectPackagingSettings, or turn off UnversionedCookedContent."
		);
	}

	for (FBeginCookContextPlatform& PlatformContext : BeginContext.PlatformContexts)
	{
		const ITargetPlatform* TargetPlatform = PlatformContext.TargetPlatform;
		UE::Cook::FPlatformData* PlatformData = PlatformContext.PlatformData;
		const ICookedPackageWriter* PackageWriterPtr = FindPackageWriter(TargetPlatform);
		check(PackageWriterPtr); // PackageContexts should have been created by SelectSessionPlatforms or by FindOrCreateSaveContexts in AddCookOnTheFlyPlatformFromGameThread
		const ICookedPackageWriter& PackageWriter(*PackageWriterPtr);
		bool bLegacyIterativeSharedBuild = false;
		if (bIncrementalOrLegacyIterative && bIsSharedLegacyIterativeCook && !PlatformData->bIsSandboxInitialized)
		{
			checkf(!IsCookingDLC(), TEXT("SharedLegacyIterativeCook is not implemented for DLC")); // The paths below look in the ProjectSavedDir, we don't have a save dir per dlc

			// see if the shared build is newer then the current cooked content in the local directory
			FString SharedCookedAssetRegistry = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("SharedIterativeBuild"),
				*TargetPlatform->PlatformName(), TEXT("Metadata"), GetDevelopmentAssetRegistryFilename());

			FDateTime PreviousLocalCookedBuild = PackageWriter.GetPreviousCookTime();
			FDateTime PreviousSharedCookedBuild = IFileManager::Get().GetTimeStamp(*SharedCookedAssetRegistry);
			if (PreviousSharedCookedBuild != FDateTime::MinValue() &&
				PreviousSharedCookedBuild >= PreviousLocalCookedBuild)
			{
				// copy the ini settings from the shared cooked build. 
				const FString SharedCookedIniFile = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("SharedIterativeBuild"),
					*TargetPlatform->PlatformName(), TEXT("Metadata"), TEXT("CookedIniVersion.txt"));
				FString SandboxCookedIniFile = FPaths::ProjectDir() / TEXT("Metadata") / TEXT("CookedIniVersion.txt");
				SandboxCookedIniFile = ConvertToFullSandboxPath(*SandboxCookedIniFile, true, *TargetPlatform->PlatformName());
				IFileManager::Get().Copy(*SandboxCookedIniFile, *SharedCookedIniFile);
				bLegacyIterativeSharedBuild = true;
				UE_LOGF(LogCook, Display, "Shared legacyiterative build is newer then local cooked build, legacyiteratively cooking from shared build.");
			}
			else
			{
				UE_LOGF(LogCook, Display, "Local cook is newer then shared cooked build, legacyiteratively cooking from local build.");
			}
		}
		for (const TRefCountPtr<UE::Cook::ICookArtifact>& Artifact : CookArtifacts)
		{
			UE::Cook::FCookArtifactsCurrentSettings& CurrentSettings
				= PlatformContext.CookArtifactsCurrentSettings.FindOrAdd(Artifact.GetReference());
			CurrentSettings.Settings = Artifact->CalculateCurrentSettings(*this, TargetPlatform);
		}
		PlatformContext.bHasMemoryResults = PlatformData->bIsSandboxInitialized;

		if (bIsDiffOnly)
		{
			UE_LOGF(LogCook, Display,
				"INCREMENTAL COOK: cooking incrementally due to -DiffOnly flag. Keeping previously cooked packages for platform %ls and cooking into memory buffers.",
				*TargetPlatform->PlatformName());
			// When looking for deterministic cooking differences in cooked packages, don't delete the packages on disk
			PlatformContext.bFullBuild = false;
			PlatformContext.bAllowIncrementalResults = false;
			PlatformContext.bClearMemoryResults = true;
			PlatformContext.bPopulateMemoryResultsFromDiskResults = false;
			PlatformContext.bLegacyIterativeSharedBuild = false;
		}
		else
		{
			bool bIncrementalOrLegacyIterativeAllowed = true;
			if (!bIncrementalOrLegacyIterative && !PlatformData->bIsSandboxInitialized)
			{
				if (!bLegacyBuildDependencies)
				{
					if (!bDefaultIncremental && !bForceRecookCommandLineArgPresent)
					{
						UE_LOGF(LogCook, Display,
							"FULL COOK: Incremental Cooks are disabled by default in Editor.ini:[CookSettings]:CookIncrementalDefaultIncremental=false, and commandline did not specify -forcerecook=false. " "Deleting previously cooked packages for platform %ls and recooking all packages discovered in the current cook.",
							*TargetPlatform->PlatformName());
					}
					else
					{
						UE_LOGF(LogCook, Display,
							"FULL COOK: -forcerecook=true (or an equivalent) was specified. Deleting previously cooked packages for platform %ls and recooking all packages discovered in the current cook.",
							*TargetPlatform->PlatformName());
					}
				}
				else
				{
					UE_LOGF(LogCook, Display,
						"FULL COOK: Incremental Cook Dependencies are disabled, and -legacyiterative was not specified. Deleting previously cooked packages for platform %ls and recooking all packages discovered in the current cook.",
						*TargetPlatform->PlatformName());
				}
				bIncrementalOrLegacyIterativeAllowed = false;
				if (!bForceRecookCommandLineArgPresent && bRunningAsShaderServer)
				{
					UE_LOGF(LogCook, Display,
						"'-odsc' was passed on commandline, but '-legacyiterative' was not, so the cooker as a side effect is clearing cook results. The build will need to be recooked before it can be staged. Add the commandline argument '-legacyiterative' to avoid this unnecessary clear."
					);
				}
			}

			if (bIncrementalOrLegacyIterativeAllowed && !bLegacyBuildDependencies && !bDependencyTrackingEnabled)
			{
				UE_LOGF(LogCook, Display,
					"FULL COOK: -cookincremental was specified, but -Cook.DependencyTracking=0 was present on commandline and so cookincremental is not supported. Deleting previously cooked packages for platform %ls and recooking all packages discovered in the current cook.",
					*TargetPlatform->PlatformName());
				bIncrementalOrLegacyIterativeAllowed = false;
			}

			if (bIncrementalOrLegacyIterativeAllowed)
			{
				UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;
				for (const TRefCountPtr<UE::Cook::ICookArtifact>& Artifact: CookArtifacts)
				{
					if (bLegacyBuildDependencies && Artifact != GlobalArtifact)
					{
						continue;
					}

					FString ArtifactName = Artifact->GetArtifactName();
					FConfigFile Previous = LoadCookSettings(TargetPlatform, ArtifactName);
					UE::Cook::FCookArtifactsCurrentSettings& Current
						= PlatformContext.CookArtifactsCurrentSettings.FindChecked(Artifact.GetReference());
					FString PreviousFileName = GetCookSettingsFileName(TargetPlatform, ArtifactName);

					UE::Cook::Artifact::FCompareSettingsContext Context(*this,
						FindOrCreateCookArtifactReader(TargetPlatform),
						Previous, Current.Settings, PreviousFileName);
					Context.TargetPlatform = TargetPlatform;
					Artifact->CompareSettings(Context);

					Current.bRequestInvalidate = Context.IsRequestInvalidate();
					if (!Context.IsRequestFullRecook())
					{
						continue;
					}

					UE_LOGF(LogCook, Display,
						"FULL COOK: %ls was specified, but %ls settings have changed and all previously cooked packages are invalidated. Deleting previously cooked packages for platform %ls and recooking all packages discovered in the current cook.",
						bLegacyBuildDependencies ? TEXT("-legacyiterative") : TEXT("-cookincremental"), *ArtifactName, *TargetPlatform->PlatformName());
					if (bRunningAsShaderServer)
					{
						UE_LOGF(LogCook, Display,
							"'-odsc -legacyiterative' was passed on commandline, but due to unrelated changes in global settings the cooker has to clear cook results. The build will need to be recooked before it can be staged."
						);
					}
					bIncrementalOrLegacyIterativeAllowed = false;
					break;
				}
			}

			if (bIncrementalOrLegacyIterativeAllowed)
			{
				UE_LOGF(LogCook, Display,
					"INCREMENTAL COOK: %ls settings are still valid. Keeping previously cooked packages for platform %ls and cooking only packages that have been modified.",
					bLegacyBuildDependencies ? TEXT("-legacyiterative was specified and global") : TEXT("Global"),
					*TargetPlatform->PlatformName());
				PlatformContext.bFullBuild = false;
				PlatformContext.bAllowIncrementalResults = true;
				PlatformContext.bClearMemoryResults = false;
				PlatformContext.bPopulateMemoryResultsFromDiskResults = !PlatformContext.bHasMemoryResults;
				PlatformContext.bLegacyIterativeSharedBuild = bLegacyIterativeSharedBuild;
			}
			else
			{
				PlatformContext.bFullBuild = true;
				PlatformContext.bAllowIncrementalResults = false;
				PlatformContext.bClearMemoryResults = true;
				PlatformContext.bPopulateMemoryResultsFromDiskResults = false;
				PlatformContext.bLegacyIterativeSharedBuild = false;
			}
		}
		PlatformData->bFullBuild = PlatformContext.bFullBuild;
		PlatformData->bAllowIncrementalResults = PlatformContext.bAllowIncrementalResults;
		PlatformData->bLegacyIterativeSharedBuild = PlatformContext.bLegacyIterativeSharedBuild;
		PlatformData->bWorkerOnSharedSandbox = PlatformContext.bWorkerOnSharedSandbox;

		COOK_STAT(DetailedCookStats::IsIncrementalCook = IsIncremental());
		
		if (PlatformData->bFullBuild && !bFullBuildAllowed)
		{
			UE_LOGF(LogCook, Fatal,
				"INCREMENTAL COOK: A diffonly cook mode was specified (-diffonly or -incrementalvalidate), " "and we guarantee state will not be wiped. But an incremental cook was found to not be possible (see logs above). Aborting the cook.");
		}
	}
}

void UCookOnTheFlyServer::BeginCookSandbox(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookSandbox);
	using namespace UE::Cook;

#if OUTPUT_COOKTIMING
	double CleanSandboxTime = 0.0;
#endif
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(CleanSandbox, CleanSandboxTime);
		TArray<TPair<const ITargetPlatform*, bool>, TInlineAllocator<ExpectedMaxNumPlatforms>> ResetPlatforms;
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> PopulatePlatforms;
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> AlreadyCookedPlatforms;
		for (FBeginCookContextPlatform& PlatformContext : BeginContext.PlatformContexts)
		{
			const ITargetPlatform* TargetPlatform = PlatformContext.TargetPlatform;
			FPlatformData* PlatformData = PlatformContext.PlatformData;
			FCookSavePackageContext& SavePackageContext = FindOrCreateSaveContext(TargetPlatform);
			ICookedPackageWriter& PackageWriter = *SavePackageContext.PackageWriter;
			ICookedPackageWriter::FCookInfo CookInfo;
			CookInfo.CookMode = IsDirectorCookOnTheFly() ? ICookedPackageWriter::FCookInfo::CookOnTheFlyMode : ICookedPackageWriter::FCookInfo::CookByTheBookMode;
			CookInfo.bFullBuild = PlatformContext.bFullBuild;
			CookInfo.bLegacyIterativeSharedBuild = PlatformContext.bLegacyIterativeSharedBuild;
			CookInfo.bWorkerOnSharedSandbox = PlatformContext.bWorkerOnSharedSandbox;
			PackageWriter.Initialize(CookInfo);
			// Refresh PackageWriterCapabilities because they can change during Initialize
			SavePackageContext.PackageWriterCapabilities = PackageWriter.GetCookCapabilities();

			ICookArtifactReader& Reader = *SavePackageContext.ArtifactReader;
			bool bCleanBuild = CookInfo.bFullBuild && !CookInfo.bWorkerOnSharedSandbox;
			Reader.Initialize(bCleanBuild);

			if (!PlatformContext.bWorkerOnSharedSandbox)
			{
				check(!IsCookWorkerMode());
				checkf(SandboxFile, TEXT("Cannot begin cooking to a sandbox until after CreateSandboxFile has been called from a StartCook function."));
				// Clean the Manifest directory even on incremental cooks; it is written from scratch each time
				// But only do this if we own the output directory
				PlatformData->RegistryGenerator->CleanManifestDirectories(*SandboxFile, *this);
			}

			if (PlatformContext.bPopulateMemoryResultsFromDiskResults)
			{
				PopulatePlatforms.Add(TargetPlatform);
			}
			else if (PlatformContext.bHasMemoryResults && !PlatformContext.bClearMemoryResults)
			{
				AlreadyCookedPlatforms.Add(TargetPlatform);
			}
			bool bResetResults = PlatformContext.bHasMemoryResults && PlatformContext.bClearMemoryResults;
			ResetPlatforms.Emplace(TargetPlatform, bResetResults);
			if (!PlatformContext.bWorkerOnSharedSandbox)
			{
				check(!IsCookWorkerMode());
				for (const TRefCountPtr<ICookArtifact>& Artifact: CookArtifacts)
				{
					FCookArtifactsCurrentSettings& Current
						= PlatformContext.CookArtifactsCurrentSettings.FindChecked(Artifact.GetReference());
					FString ArtifactName = Artifact->GetArtifactName();

					if (!bLegacyBuildDependencies || PlatformContext.bFullBuild)
					{
						if (PlatformContext.bFullBuild)
						{
							Artifact->OnFullRecook(TargetPlatform);
						}
						else if (Current.bRequestInvalidate)
						{
							UE_LOGF(LogCook, Display,
								"COOKINCREMENTAL: CLEARING ARTIFACT: -cookincremental was specified, but %ls settings have changed and the artifact is invalidated. Deleting the artifact; it will be reconstructed from the packags that cook during this session.",
								*ArtifactName);
							Artifact->OnInvalidate(TargetPlatform);
						}
					}
					if (!bLegacyBuildDependencies || Artifact == GlobalArtifact)
					{
						SaveCookSettings(Current.Settings, TargetPlatform, ArtifactName);
					}
				}
			}

			PlatformData->bIsSandboxInitialized = true;
		}

		ResetCook(ResetPlatforms);
		if (PopulatePlatforms.Num())
		{
			PopulateCookedPackages(PopulatePlatforms);
		}
		else if (AlreadyCookedPlatforms.Num())
		{
			// Set the NumPackagesIncrementallySkipped field to include all of the already CookedPackages
			COOK_STAT(DetailedCookStats::NumPackagesIncrementallySkipped = 0);
			// TODO: Initializing NumPackagesIncrementallyModifiedByClass is not yet implemented in this path (cooking
			// multiple times in editor process) because we plan to deprecate this path soon.
			const ITargetPlatform* TargetPlatform = AlreadyCookedPlatforms[0];
			PackageDatas->LockAndEnumeratePackageDatas([TargetPlatform](FPackageData* PackageData)
			{
				if (PackageData->HasCookedPlatform(TargetPlatform, true /* bIncludeFailed */))
				{
					COOK_STAT(++DetailedCookStats::NumPackagesIncrementallySkipped);
				}
			});
		}
	}

#if OUTPUT_COOKTIMING
	FString PlatformNames;
	for (const ITargetPlatform* Target : BeginContext.TargetPlatforms)
	{
		PlatformNames += Target->PlatformName() + TEXT(" ");
	}
	PlatformNames.TrimEndInline();
	UE_LOGF(LogCook, Display, "Sandbox cleanup took %5.3f seconds for platforms %ls", CleanSandboxTime, *PlatformNames);
#endif
}

UE::Cook::FCookSavePackageContext* UCookOnTheFlyServer::CreateSaveContext(const ITargetPlatform* TargetPlatform)
{
	using namespace UE::Cook;
	checkf(SandboxFile, TEXT("SaveContexts cannot be created until after CreateSandboxFile has been called from a StartCook function."));

	const FString RootPathSandbox = ConvertToFullSandboxPath(FPaths::RootDir(), true);
	const FString PlatformString = TargetPlatform->PlatformName();
	const FString ResolvedRootPath = RootPathSandbox.Replace(TEXT("[Platform]"), *PlatformString);
	const FString ResolvedMetadataPath = GetCookMetadataOutputFolderInternal(TargetPlatform);

	TUniquePtr<FDeterminismManager> DeterminismManager;
	TSharedPtr<ICookArtifactReader> CookArtifactReader = nullptr;
	ICookedPackageWriter* PackageWriter = nullptr;
	FString WriterDebugName;
	DiffModeHelper->Initialize();
	if (bDeterminismDebug || DiffModeHelper->IsDeterminismDebug())
	{
		DeterminismManager = MakeUnique<FDeterminismManager>();
	}
	if ((bIncrementallyModifiedDiagnostics || DiffModeHelper->IsIncrementallyModifiedDiagnostics())
		&& !IncrementallyModifiedDiagnostics
		&& !IsCookWorkerMode())
	{
		IncrementallyModifiedDiagnostics = MakeUnique<UE::Cook::FIncrementallyModifiedDiagnostics>();
	}

	if (IsUsingZenStore())
	{
		TSharedRef<FLayeredCookArtifactReader> LayeredReader = MakeShared<FLayeredCookArtifactReader>();
		TSharedRef<FZenCookArtifactReader> ZenReader = MakeShared<FZenCookArtifactReader>(ResolvedRootPath, ResolvedMetadataPath, TargetPlatform);
		LayeredReader->AddLayer(SharedLooseFilesCookArtifactReader.ToSharedRef());
		LayeredReader->AddLayer(ZenReader);
		AllContextArtifactReader->AddLayer(ZenReader);
		CookArtifactReader = LayeredReader;
		FZenStoreWriter* ZenWriter = new FZenStoreWriter(ResolvedRootPath, ResolvedMetadataPath, TargetPlatform, LayeredReader);
		ZenWriter->SetCooker(this);
		// Reader InitializeConnection must come after writer InitializeConnection, it uses data written by that function.
		ZenWriter->InitializeConnection();
		ZenReader->InitializeConnection();
		PackageWriter = ZenWriter;
		WriterDebugName = TEXT("ZenStore");
	}
	else
	{
		CookArtifactReader = SharedLooseFilesCookArtifactReader;
		PackageWriter = new FLooseCookedPackageWriter(ResolvedRootPath, ResolvedMetadataPath, TargetPlatform,
			GetAsyncIODelete(), *SandboxFile, SharedLooseFilesCookArtifactReader.ToSharedRef());
		PackageWriter->SetCooker(this);
		WriterDebugName = TEXT("LooseCookedPackageWriter");
	}

	DiffModeHelper->InitializePackageWriter(*this, PackageWriter, ResolvedMetadataPath, DeterminismManager.Get());

	// Setup save package settings (i.e. validation)
	TSet<IPlugin*> EnabledPlugins;
	TArray<TSharedRef<IPlugin>> EnabledPluginRefPtrs;

	FSavePackageSettings SavePackageSettings = FSavePackageSettings::GetDefaultSettings();
	{
		// Setup Import Validation for suppressed module native classes
		TSet<FName> DisabledNativeScriptPackages;
		for (TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
		{
			if (TargetPlatform->IsEnabledForPlugin(Plugin.Get()))
			{
				bool bExists;
				EnabledPlugins.Add(&Plugin.Get(), &bExists);
				if (!bExists)
				{
					EnabledPluginRefPtrs.Add(Plugin);
				}
			}
			else
			{
				for (const FModuleDescriptor& Module : Plugin->GetDescriptor().Modules)
				{
					DisabledNativeScriptPackages.Add(FPackageName::GetModuleScriptPackageName(Module.Name));
				}
			}
		}

		SavePackageSettings.AddExternalImportValidation([SuppressedNativeScriptPackages = MoveTemp(DisabledNativeScriptPackages), TargetPlatform, this](const FImportsValidationContext& ValidationContext)
			{
				for (const TObjectPtr<UObject>& Object : ValidationContext.Imports)
				{
					if (Object->IsNative() && SuppressedNativeScriptPackages.Contains(Object->GetPackage()->GetFName()))
					{
						FInstigator Instigator = GetInstigator(ValidationContext.Package->GetFName(), EReachability::Runtime);
						bool bIsError = true;
						if (Instigator.Category == EInstigator::StartupPackage || Instigator.Category == EInstigator::ModifyCookDelegate)
						{
							// StartupPackages might be around just because of the editor;
							// if they're not available on client, ignore them without error
							bIsError = false;
						}

						if (Object->IsA<UClass>())
						{
							if (bIsError)
							{
								// If you receive this message in a package that you do want to cook, you can remove the object of the
								// unavailable class by overriding UObject::NeedsLoadForTargetPlatform on that class to return false.
								UE_ASSET_LOG(LogCook, Error, ValidationContext.Package, TEXT("Failed to cook %s for platform %s. It imports class %s, which is in a module that is not available on the platform."),
									*ValidationContext.Package->GetName(), *TargetPlatform->PlatformName(), *Object->GetPathName());
								return ESavePackageResult::ValidatorError;
							}
							else
							{
								UE_ASSET_LOG(LogCook, Display, ValidationContext.Package, TEXT("Skipping package %s for platform %s. It imports class %s, which is in a module that is not available on the platform."),
									*ValidationContext.Package->GetName(), *TargetPlatform->PlatformName(), *Object->GetPathName());
								return ESavePackageResult::ValidatorSuppress;
							}
						}
						else
						{
							if (bIsError)
							{
								// If you receive this message in a package that you do want to cook, you can remove the reference to the missing object
								// marking the property holding reference as editor-only, or otherwise failing to serialize it, or nulling it out during PreSave
								// and restoring it during PostSave.
								UE_ASSET_LOG(LogCook, Error, ValidationContext.Package, TEXT("Failed to cook %s for platform %s. It imports type '%s', which is in a module that is not available on the platform."),
									*ValidationContext.Package->GetName(), *TargetPlatform->PlatformName(), *Object->GetFullName());
								return ESavePackageResult::ValidatorError;
							}
							else
							{
								UE_ASSET_LOG(LogCook, Display, ValidationContext.Package, TEXT("Skipping package %s for platform %s. It imports type '%s', which is in a module that is not available on the platform."),
									*ValidationContext.Package->GetName(), *TargetPlatform->PlatformName(), *Object->GetFullName());
								return ESavePackageResult::ValidatorSuppress;
							}
						}
					}
				}
				return ESavePackageResult::Success;
			});
	}

	FCookSavePackageContext* Context = new FCookSavePackageContext(TargetPlatform, CookArtifactReader, PackageWriter, WriterDebugName, MoveTemp(SavePackageSettings),
		MoveTemp(DeterminismManager));
	Context->EnabledPlugins = MoveTemp(EnabledPlugins);
	Context->EnabledPluginRefPtrs = MoveTemp(EnabledPluginRefPtrs);
	return Context;

}

const TSet<IPlugin*>* UCookOnTheFlyServer::GetEnabledPlugins(const ITargetPlatform* TargetPlatform) const
{
	using namespace UE::Cook;

	const FCookSavePackageContext* Context = FindSaveContext(TargetPlatform);
	if (Context)
	{
		return &Context->EnabledPlugins;
	}
	return nullptr;
}

void UCookOnTheFlyServer::DeleteOutputForPackage(FName PackageName, const ITargetPlatform* TargetPlatform)
{
	FindOrCreatePackageWriter(TargetPlatform).RemoveCookedPackages(TArrayView<FName>(&PackageName, 1));
}

void UCookOnTheFlyServer::FinalizePackageStore(const ITargetPlatform* TargetPlatform, const TSet<FName>& CookedPackageNames)
{
	using namespace UE::Cook;

	FPlatformData* PlatformData = PlatformManager->GetPlatformData(TargetPlatform);
	ICookedPackageWriter::FCookInfo CookInfo;
	CookInfo.CookMode = IsDirectorCookOnTheFly() ? ICookedPackageWriter::FCookInfo::CookOnTheFlyMode : ICookedPackageWriter::FCookInfo::CookByTheBookMode;
	CookInfo.bFullBuild = PlatformData->bFullBuild;
	CookInfo.bLegacyIterativeSharedBuild = PlatformData->bLegacyIterativeSharedBuild;
	CookInfo.bWorkerOnSharedSandbox = PlatformData->bWorkerOnSharedSandbox;

	if (GetCookingDLC() == ECookingDLC::Yes && !CookInfo.bWorkerOnSharedSandbox)
	{
		CalculateReferencedPlugins(TargetPlatform, CookedPackageNames, CookInfo.ReferencedPlugins.Emplace());
	}

	ICookedPackageWriter& CookWriter = FindOrCreatePackageWriter(TargetPlatform);
	// Collect end-of-cook metadata from artifacts and the asset manager.
	if (!CookInfo.bWorkerOnSharedSandbox)
	{
		Artifact::FStoreDataInOplogContext StoreDataContext;
		for (const TRefCountPtr<ICookArtifact>& Artifact : CookArtifacts)
		{
			Artifact->StoreDataInOplog(StoreDataContext);
		}
		for (Artifact::FStoreDataInOplogContext::FEntry& AdditionalOp : StoreDataContext.Entries)
		{
			CookWriter.AppendOp(AdditionalOp.Key, AdditionalOp.Value);
		}
	}
	CookWriter.EndCook(CookInfo);
}

void UCookOnTheFlyServer::CalculateReferencedPlugins(const ITargetPlatform* TargetPlatform,
	const TSet<FName>& CookedPackageNames, ICookedPackageWriter::FReferencedPluginsInfo& OutReferencedPluginsInfo)
{
	using namespace UE::Cook;

	OutReferencedPluginsInfo.bReferencesEngine = false;
	OutReferencedPluginsInfo.bReferencesGame = false;
	OutReferencedPluginsInfo.ReferencedPlugins.Reset();

	IPluginManager& PluginManager = IPluginManager::Get();

	FString PluginNameStr;
	PluginNameStr.Reserve(256);
	for (FName PackageName : CookedPackageNames)
	{
		TStringBuilder<FName::StringBufferSize> PackageNameStr(InPlace, PackageName);
		if (PackageNameStr.Len() == 0 || (*PackageNameStr)[0] != '/')
		{
			continue;
		}
		FStringView MountName(PackageNameStr);
		MountName.RightChopInline(1);
		int32 SecondSlashIndex;
		if (!MountName.FindChar('/', SecondSlashIndex))
		{
			continue;
		}
		MountName.LeftInline(SecondSlashIndex);

		if (MountName == TEXT("Engine"))
		{
			OutReferencedPluginsInfo.bReferencesEngine = true;
			continue;
		}
		if (MountName == TEXT("Game"))
		{
			OutReferencedPluginsInfo.bReferencesGame = true;
			continue;
		}
		TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(MountName);
		if (Plugin)
		{
			PluginNameStr.Reset();
			PluginNameStr.Append(MountName);
			OutReferencedPluginsInfo.ReferencedPlugins.Add(PluginNameStr);
		}
	}

	if (GetCookingDLC() == ECookingDLC::No)
	{
		// For basegame cooks, mark that they reference Engine and Game even if they
		// contain no packages, so that they stage the non-package Engine and Game files.
		OutReferencedPluginsInfo.bReferencesEngine = true;
		OutReferencedPluginsInfo.bReferencesGame = true;
	}
}

void UCookOnTheFlyServer::ClearPackageStoreContexts()
{
	if (AllContextArtifactReader)
	{
		AllContextArtifactReader->EmptyLayers();
		AllContextArtifactReader->AddLayer(SharedLooseFilesCookArtifactReader.ToSharedRef());
	}
	for (UE::Cook::FCookSavePackageContext* Context : SavePackageContexts)
	{
		delete Context;
	}

	SavePackageContexts.Empty();
}

void UCookOnTheFlyServer::DiscoverPlatformSpecificNeverCookPackages(
	const TArrayView<const ITargetPlatform* const>& TargetPlatforms, const TArray<FString>& UBTPlatformStrings,
	UE::Cook::FBeginCookConfigSettings& Settings) const
{
	TArray<const ITargetPlatform*> PluginUnsupportedTargetPlatforms;
	TArray<FAssetData> PluginAssets;
	FARFilter PluginARFilter;
	FString PluginPackagePath;

	TArray<TSharedRef<IPlugin>> AllContentPlugins = IPluginManager::Get().GetEnabledPluginsWithContent();
	for (TSharedRef<IPlugin> Plugin : AllContentPlugins)
	{
		const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();

		// we are only interested in plugins that does not support all platforms
		if (Descriptor.SupportedTargetPlatforms.Num() == 0 && !Descriptor.bHasExplicitPlatforms)
		{
			continue;
		}

		// find any unsupported target platforms for this plugin
		PluginUnsupportedTargetPlatforms.Reset();
		for (int32 I = 0, Count = TargetPlatforms.Num(); I < Count; ++I)
		{
			if (!Descriptor.SupportedTargetPlatforms.Contains(UBTPlatformStrings[I]))
			{
				PluginUnsupportedTargetPlatforms.Add(TargetPlatforms[I]);
			}
		}

		// if there are unsupported target platforms,
		// then add all packages for this plugin for these platforms to the PlatformSpecificNeverCookPackages map
		if (PluginUnsupportedTargetPlatforms.Num() > 0)
		{
			PluginPackagePath.Reset(127);
			PluginPackagePath.AppendChar(TEXT('/'));
			PluginPackagePath.Append(Plugin->GetName());

			PluginARFilter.bRecursivePaths = true;
			PluginARFilter.bIncludeOnlyOnDiskAssets = true;
			PluginARFilter.PackagePaths.Reset(1);
			PluginARFilter.PackagePaths.Emplace(*PluginPackagePath);

			PluginAssets.Reset();
			AssetRegistry->GetAssets(PluginARFilter, PluginAssets);

			for (const ITargetPlatform* TargetPlatform: PluginUnsupportedTargetPlatforms)
			{
				TSet<FName>& NeverCookPackages = Settings.PlatformSpecificNeverCookPackages.FindOrAdd(TargetPlatform);
				for (const FAssetData& Asset : PluginAssets)
				{
					NeverCookPackages.Add(Asset.PackageName);
				}
			}
		}
	}
}

void UCookOnTheFlyServer::StartCookByTheBook( const FCookByTheBookStartupOptions& CookByTheBookStartupOptions )
{
	TOptional<FCookByTheBookStartupOptions> ModifiedStartupOptions;
	bool bAbort = false;

	const FCookByTheBookStartupOptions& EffectiveStartupOptions = BlockOnPrebootCookGate(bAbort,
		CookByTheBookStartupOptions,
		ModifiedStartupOptions);

	if (bAbort)
	{
		return;
	}

	UE_LOGF(LogCook, Display, "---- Post Content Initialisation: Start ----");
	UE_SCOPED_COOKTIMER(StartCookByTheBook);
	LLM_SCOPE_BYTAG(Cooker);
	check(IsInGameThread());
	check(IsCookByTheBookMode());

	// Initialize systems and settings that the rest of StartCookByTheBook depends on
	// Functions in this section are ordered and can depend on the functions before them
	InitializeSession();
	FBeginCookContext BeginContext = CreateBeginCookByTheBookContext(EffectiveStartupOptions);
	BlockOnAssetRegistry(BeginContext.StartupOptions->CookMaps);
	CreateSandboxFile(BeginContext);
	LoadBeginCookConfigSettings(BeginContext);
	SelectSessionPlatforms(BeginContext);
	UE::Cook::FDelegates::ConfigureCookSession.Broadcast(*this);
	CalculateGlobalDependenciesHashes();
	RegisterValidationCookArtifact();
	RegisterShaderLibraryIncrementalCookArtifact(BeginContext);
	LoadBeginCookIncrementalFlags(BeginContext);

	// Initialize systems referenced by later stages or that need to start early for async performance
	// Functions in this section must not need to read/write the SandboxDirectory or MemoryCookedPackages
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance
	BeginCookStartShaderCodeLibrary(BeginContext); // start shader code library cooking asynchronously; we block on it later
	RefreshPlatformAssetRegistries(BeginContext.TargetPlatforms); // Required by BeginCookSandbox stage
	InitializeAllCulturesToCook(BeginContext.StartupOptions->CookCultures);

	// Clear the sandbox directory, or preserve it and populate incremental cooks
	// Clear in-memory CookedPackages, or preserve them and cook incrementally in-process
	BeginCookSandbox(BeginContext);

	// Initialize systems that need to write files to the sandbox directory, for consumption later in StartCookByTheBook
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance
	BeginCookFinishShaderCodeLibrary(BeginContext);

	// Functions in this section can depend on functions before this section but not on each other,
	// and can be ordered arbitrarily or for async performance.
	BeginCookEditorSystems();
	BeginCookEDLCookInfo(BeginContext);
	BeginCookPackageWriters(BeginContext);
	GenerateInitialRequests(BeginContext);
	CompileDLCLocalization(BeginContext);
	GenerateLocalizationReferences();
	InitializePollables();
	RecordDLCPackagesFromBaseGame(BeginContext);
	RegisterCookByTheBookDelegates();

	// Functions in this section can depend on functions before this section but not on each other,
	// and can be ordered arbitrarily or for async performance.
	BeginCookDirector(BeginContext);

	// BroadcastCookStarted by contract is sent after all internal CookByTheBookStarted is complete.
	UE_LOGF(LogCook, Display, "---- Post Content Initialisation: End ----");
	BroadcastCookStarted();
}

const UCookOnTheFlyServer::FCookByTheBookStartupOptions& UCookOnTheFlyServer::BlockOnPrebootCookGate(bool& bOutAbortCook,
	const FCookByTheBookStartupOptions& CookByTheBookStartupOptions,
	TOptional<FCookByTheBookStartupOptions>& ModifiedStartupOptions)
{
	const FCookByTheBookStartupOptions* EffectiveStartupOptions = &CookByTheBookStartupOptions;
	bOutAbortCook = false;
	
	if (IsCookByTheBookMode() && !IsCookingInEditor())
	{
		UE_LOGF(LogCook, Display, "---- Pre Content Initialisation: End ----");
		ConditionalWaitOnCommandFile(TEXTVIEW("Cook"),
			[this, &CookByTheBookStartupOptions, &ModifiedStartupOptions, &EffectiveStartupOptions, &bOutAbortCook] (FStringView CommandContents)
		{
			UE::String::ParseTokensMultiple(CommandContents, { ' ', '\t', '\r', '\n' },
				[this, &CookByTheBookStartupOptions, &ModifiedStartupOptions, &EffectiveStartupOptions, &bOutAbortCook] (FStringView Token)
				{
					const FStringView PackageToken(TEXT("-Package="));

					auto MarkModified = [&ModifiedStartupOptions, &EffectiveStartupOptions, &CookByTheBookStartupOptions]()
						{
							if (!ModifiedStartupOptions.IsSet())
							{
								ModifiedStartupOptions.Emplace(CookByTheBookStartupOptions);
								EffectiveStartupOptions = &ModifiedStartupOptions.GetValue();
							}
						};

					if (Token == TEXT("-CookAbort"))
					{
						UE_LOGF(LogCook, Display, "Received CookAbort token in cook command file, exiting cook.");
						bOutAbortCook = true;
					}
					else if (Token == TEXT("-FastExit"))
					{
						FCommandLine::Append(TEXT(" -FastExit"));
					}
					else if (Token.StartsWith(PackageToken))
					{
						MarkModified();

						UE::String::ParseTokens(Token.RightChop(PackageToken.Len()), TEXT('+'),
							[&ModifiedStartupOptions](FStringView PackageToken)
							{
								ModifiedStartupOptions->CookMaps.Add(FString(PackageToken));

								UE_LOGF(LogCook, Display, "Received Package token in cook command file, adding package '%.*ls' to cook workload.", PackageToken.Len(), PackageToken.GetData());
							}, UE::String::EParseTokensOptions::SkipEmpty);
					}
					else if (Token.StartsWith(TEXTVIEW("-DLCName=")))
					{
						MarkModified();

						int32 TokenIndex = 0;
						Token.FindChar(TEXT('='), TokenIndex);
						ModifiedStartupOptions->DLCName = Token.RightChop(TokenIndex+1);
						UE_LOGF(LogCook, Display, "Updated DLCName=%ls", *ModifiedStartupOptions->DLCName);
					}
					else if (Token.StartsWith(TEXTVIEW("-RunAssetValidation")))
					{
						MarkModified();

						ModifiedStartupOptions->CookOptions |= ECookByTheBookOptions::RunAssetValidation;
						UE_LOGF(LogCook, Display, "RunAssetValidation enabled");
					}
					else if (Token.StartsWith(TEXTVIEW("-ValidationErrorsAreFatal")))
					{
						MarkModified();

						ModifiedStartupOptions->CookOptions |= ECookByTheBookOptions::ValidationErrorsAreFatal;
						UE_LOGF(LogCook, Display, "ValidationErrorsAreFatal enabled");
					}
					else if (Token.StartsWith(TEXTVIEW("-dpcvar")) || Token.StartsWith(TEXT("-ForceDPCVars=")))
					{
						// TODO: multiprocess cook will not get cvar changes
						int32 TokenIndex = 0;
						Token.FindChar(TEXT('='), TokenIndex);
						const FString CVarParam{ Token.RightChop(TokenIndex + 1) };
						TArray<FString> CVars;
						CVarParam.ParseIntoArray(CVars, TEXT(","), true);
						for (const FString& CVarValueKey : CVars)
						{
							FString CVarKey, CVarValue;
							if (CVarValueKey.Split(TEXT("="), &CVarKey, &CVarValue))
							{
								IConsoleVariable* ConsoleVariable = IConsoleManager::Get().FindConsoleVariable(*CVarKey);
								if (ConsoleVariable)
								{
									ConsoleVariable->Set(*CVarValue, ECVF_SetByCode);
									UE_LOGF(LogCook, Display, "Set CVar %ls", *CVarValueKey);
								}
								else
								{
									UE_LOGF(LogCook, Display, "Failed to find cvar '%ls'", *CVarKey);
								}
							}
							else
							{
								UE_LOGF(LogCook, Warning, "Failed to parse cvar from:%ls", *CVarValueKey);
							}
						}
					}
					else if (Token.StartsWith(TEXTVIEW("-ini:")))
					{
						const FString TokenValue{ Token };
						TArray<FString> Tokens;
						TokenValue.ParseIntoArray(Tokens, TEXT(":"), true); //-ini:IniName:[Section1]:Key=Value
						if (Tokens.Num() == 4)
						{
							FConfigBranch* Branch = GConfig->FindBranch(*Tokens[1], TEXT(""));
							if (Branch)
							{
								static const FName CookCmdOverrideName = TEXT("CookCommand");
								UE::DynamicConfig::PerformDynamicConfig(CookCmdOverrideName, [Tokens, Branch](FConfigModificationTracker* ChangeTracker)
									{
										const FString ConfigString = Tokens[2] + TEXT("\n") + Tokens[3];
										// to force reloading cvars through -ini
										ChangeTracker->CVars.Add(TEXT("ConsoleVariables")).CVarPriority = (int)ECVF_SetByHotfix;
										Branch->AddDynamicLayerStringToHierarchy(Tokens[1], ConfigString, CookCmdOverrideName, DynamicLayerPriority::Hotfix, ChangeTracker);
									});
							}
							else
							{
								UE_LOGF(LogCook, Warning, "Failed to find config file:%ls", *Tokens[1]);
							}
						}
						else
						{
							UE_LOGF(LogCook, Warning, "Failed to parse ini from:%ls", *TokenValue);
						}
					}
					else if (UAssetManager::Get().HandleCookCommand(Token))
					{
						// Do nothing - handled by the asset manager
					}
					else
					{
						UE_LOGF(LogCook, Warning, "Ignoring unknown/unsupported token in cook command file: %.*ls", Token.Len(), Token.GetData());
					}
				}, UE::String::EParseTokensOptions::SkipEmpty);
		});
	}

	return *EffectiveStartupOptions;
}

FBeginCookContext UCookOnTheFlyServer::CreateBeginCookByTheBookContext(const FCookByTheBookStartupOptions& StartupOptions)
{
	UE_SCOPED_COOKTIMER(CreateBeginCookByTheBookContext);
	FBeginCookContext BeginContext(*this);

	BeginContext.StartupOptions = &StartupOptions;
	const ECookByTheBookOptions& CookOptions = StartupOptions.CookOptions;
	bZenStore = !!(CookOptions & ECookByTheBookOptions::ZenStore);
	if (bZenStore)
	{
		bool bFallbackToLooseFileWriter = false;
		GConfig->GetBool(TEXT("CookSettings"), TEXT("FallbackToLooseFileWriter"), bFallbackToLooseFileWriter, GEditorIni);
		if (bFallbackToLooseFileWriter && !FZenStoreWriter::CanUse())
		{
			UE_LOGF(LogCook, Warning, "Zen storage not available, falling back to loose files");
			bZenStore = false;
		}
	}
	CookByTheBookOptions->StartupOptions = CookOptions;
	CookByTheBookOptions->CookTime = 0.0f;
	CookByTheBookOptions->CookStartTime = FPlatformTime::Seconds();
	CookByTheBookOptions->bGenerateStreamingInstallManifests = StartupOptions.bGenerateStreamingInstallManifests;
	CookByTheBookOptions->bGenerateDependenciesForMaps = StartupOptions.bGenerateDependenciesForMaps;
	CookByTheBookOptions->BasedOnReleaseVersion = StartupOptions.BasedOnReleaseVersion;
	CookByTheBookOptions->CreateReleaseVersion = StartupOptions.CreateReleaseVersion;
	CookByTheBookOptions->bSkipHardReferences = !!(CookOptions & ECookByTheBookOptions::SkipHardReferences);
	CookByTheBookOptions->bSkipSoftReferences = !!(CookOptions & ECookByTheBookOptions::SkipSoftReferences);
	CookByTheBookOptions->bCookSoftPackageReferences = FParse::Param(FCommandLine::Get(), TEXT("CookSoftPackageReferences"));
	CookByTheBookOptions->bCookAgainstFixedBase = !!(CookOptions & ECookByTheBookOptions::CookAgainstFixedBase);
	CookByTheBookOptions->bDlcLoadMainAssetRegistry = !!(CookOptions & ECookByTheBookOptions::DlcLoadMainAssetRegistry);
	CookByTheBookOptions->bErrorOnEngineContentUse = StartupOptions.bErrorOnEngineContentUse;
	CookByTheBookOptions->bAllowUncookedAssetReferences = FParse::Param(FCommandLine::Get(), TEXT("AllowUncookedAssetReferences"));
	CookByTheBookOptions->bCookList = StartupOptions.bCookList;
	CookByTheBookOptions->DlcName = StartupOptions.DLCName;
	CookByTheBookOptions->bDlcPluginOnly = FParse::Param(FCommandLine::Get(), TEXT("DlcPluginOnly"));
	if (CookByTheBookOptions->bSkipHardReferences && !CookByTheBookOptions->bSkipSoftReferences)
	{
		UE_LOGF(LogCook, Display, "Setting bSkipSoftReferences to true since bSkipHardReferences is true and skipping hard references requires skipping soft references.");
		CookByTheBookOptions->bSkipSoftReferences = true;
	}

	BeginContext.TargetPlatforms = StartupOptions.TargetPlatforms;
	Algo::Sort(BeginContext.TargetPlatforms);
	BeginContext.TargetPlatforms.SetNum(Algo::Unique(BeginContext.TargetPlatforms));

	BeginContext.PlatformContexts.SetNum(BeginContext.TargetPlatforms.Num());
	for (int32 Index = 0; Index < BeginContext.TargetPlatforms.Num(); ++Index)
	{
		BeginContext.PlatformContexts[Index].TargetPlatform = BeginContext.TargetPlatforms[Index];
		// PlatformContext.PlatformData is currently null and is set in SelectSessionPlatforms
	}

	if (!IsCookingInEditor())
	{
		TArray<FWeakObjectPtr>& SessionStartupObjects = CookByTheBookOptions->SessionStartupObjects;
		SessionStartupObjects.Reset();
		for (FThreadSafeObjectIterator Iter; Iter; ++Iter)
		{
			SessionStartupObjects.Emplace(*Iter);
		}
		SessionStartupObjects.Shrink();
	}

	return BeginContext;
}

FBeginCookContext UCookOnTheFlyServer::CreateBeginCookOnTheFlyContext(const FCookOnTheFlyStartupOptions& Options)
{
	bZenStore = Options.bZenStore;
	CookOnTheFlyOptions->Port = Options.Port;
	CookOnTheFlyOptions->bPlatformProtocol = Options.bPlatformProtocol;
	return FBeginCookContext(*this);
}

FBeginCookContext UCookOnTheFlyServer::CreateAddPlatformContext(ITargetPlatform* TargetPlatform)
{
	FBeginCookContext BeginContext(*this);

	BeginContext.TargetPlatforms.Add(TargetPlatform);

	FBeginCookContextPlatform& PlatformContext = BeginContext.PlatformContexts.Emplace_GetRef();
	PlatformContext.TargetPlatform = TargetPlatform;
	PlatformContext.PlatformData = &PlatformManager->CreatePlatformData(*this, TargetPlatform);

	return BeginContext;
}

void UCookOnTheFlyServer::StartCookAsCookWorker()
{
	UE_SCOPED_COOKTIMER(StartCookWorker);
	LLM_SCOPE_BYTAG(Cooker);
	check(IsInGameThread());
	check(IsCookWorkerMode());

	// Initialize systems and settings that the rest of StartCookAsCookWorker depends on
	// Functions in this section are ordered and can depend on the functions before them
	InitializeSession();
	FBeginCookContext BeginContext = CreateCookWorkerContext();
	// MPCOOKTODO: Load serialized AssetRegistry from Director
	BlockOnAssetRegistry(TConstArrayView<FString>());
	CreateSandboxFile(BeginContext);
	LoadBeginCookConfigSettings(BeginContext);
	SelectSessionPlatforms(BeginContext);
	UE::Cook::FDelegates::ConfigureCookSession.Broadcast(*this);
	CalculateGlobalDependenciesHashes();
	LoadBeginCookIncrementalFlags(BeginContext);
	if (IsDirectorCookOnTheFly())
	{
		GShaderCompilingManager->SkipShaderCompilation(bDisableShaderCompilationDuringCookOnTheFly);
		GShaderCompilingManager->SetAllowForIncompleteShaderMaps(bAllowIncompleteShaderMapsDuringCookOnTheFly);
	}
	CookWorkerClient->DoneWithInitialSettings();

	// Initialize systems referenced by later stages or that need to start early for async performance
	if (IsDirectorCookByTheBook())
	{
		BeginCookStartShaderCodeLibrary(BeginContext); // start shader code library cooking asynchronously; we block on it later
	}
	RefreshPlatformAssetRegistries(BeginContext.TargetPlatforms); // Required by BeginCookSandbox stage

	// Clear in-memory CookedPackages, or preserve them and cook incrementally in-process. We do not modify the
	// CookedPackages on disk, because that was already done as necessary by the Director
	BeginCookSandbox(BeginContext);

	// Initialize systems that nothing in StartCookAsCookWorker references
	// Functions in this section are not dependent upon each other and can be ordered arbitrarily or for async performance	
	BeginCookEDLCookInfo(BeginContext);
	BeginCookPackageWriters(BeginContext);
	InitializePollables();
	if (IsDirectorCookByTheBook())
	{
		RegisterCookByTheBookDelegates();
		BeginCookFinishShaderCodeLibrary(BeginContext);
	}
	BroadcastCookStarted();
}

void UCookOnTheFlyServer::LogCookWorkerStats()
{
	if (IsDirectorCookByTheBook())
	{
		PrintFinishStats();
		OutputHierarchyTimers();
		PrintDetailedCookStats();
	}
}

void UCookOnTheFlyServer::CookAsCookWorkerFinished()
{
	if (CookWorkerClient->HasRunFinished())
	{
		return;
	}
	CookWorkerClient->SetHasRunFinished(true);

	{
		UE_SCOPED_COOKTIMER(TickCookableObjects);
		const double CurrentTime = FPlatformTime::Seconds();
		FTickableCookObject::TickObjects(static_cast<float>(CurrentTime - LastCookableObjectTickTime), true /* bCookComplete */);
		LastCookableObjectTickTime = CurrentTime;
	}

	ShutdownShaderLibraryCookerAndCompilers(GetProjectShaderLibraryName());
	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(FinalizePackageStore);
		UE_LOGF(LogCook, Display, "Finalize package store(s)...");
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			FinalizePackageStore(TargetPlatform, TSet<FName>());
		}
	}

	if (IsDirectorCookOnTheFly())
	{
		if (bDisableShaderCompilationDuringCookOnTheFly)
		{
			GShaderCompilingManager->SkipShaderCompilation(false);
		}
		if (bAllowIncompleteShaderMapsDuringCookOnTheFly)
		{
			GShaderCompilingManager->SetAllowForIncompleteShaderMaps(false);
		}
	}

	LogCookWorkerStats();
	BroadcastCookFinished();
	CookWorkerClient->FlushLogs();
}

void UCookOnTheFlyServer::GetPackagesToRetract(int32 NumToRetract, TArray<FName>& OutRetractionPackages)
{
	using namespace UE::Cook;
	OutRetractionPackages.Reset(NumToRetract);
	if (OutRetractionPackages.Num() >= NumToRetract)
	{
		return;
	}

	auto AddPackageIfPossibleAndReportDone = [&OutRetractionPackages, NumToRetract, this](FPackageData* PackageData)
	{
		if (OutRetractionPackages.Num() >= NumToRetract)
		{
			return true;
		}

		if (PackageData->GetWorkerAssignmentConstraint().IsValid())
		{
			// Don't send back Packages that are constrained to this worker. Doing so will just
			// cause the CookDirector to send it back to us, and this can cause the cooker to crash
			// on WorldPartition packages, if we abort them and then try to restart them later.
			return false;
		}
		if (PackageData->IsGenerated())
		{
			if (PackageData->DoesGeneratedRequireGenerator() >= ICookPackageSplitter::EGeneratedRequiresGenerator::Save
				|| MPCookGeneratorSplit == UE::Cook::EMPCookGeneratorSplit::AllOnSameWorker)
			{
				// With EGeneratedRequiresGenerator::Save or the AllOnSameWorker setting, GeneratedPackages are
				// constrained to this worker.
				return false;
			}
		}
		if (FGenerationHelper* GenerationHelper = PackageData->GetGenerationHelper())
		{
			if (PackageData->GetSaveSubState() >= ESaveSubState::Generation_QueueGeneratedPackages)
			{
				if (GenerationHelper->DoesGeneratedRequireGenerator()
					>= ICookPackageSplitter::EGeneratedRequiresGenerator::Save
					||
					MPCookGeneratorSplit != UE::Cook::EMPCookGeneratorSplit::AnyWorker)
				{
					// With EGeneratedRequiresGenerator::Save or with any MPCookGeneratorSplit setting other than
					// AnyWorker, we make assignment decisions based on the worker that saved and queued the generator
					// package. We do not track queuing separately; we assume it happened on the worker that saved the
					// package. Therefore, do not allow retraction of a generator package if it has already entered
					// the QueueGeneratedPackages state.
					return false;
				}
			}
		}

		OutRetractionPackages.Add(PackageData->GetPackageName());
		return OutRetractionPackages.Num() >= NumToRetract;
	};

	FRequestQueue& RequestQueue = PackageDatas->GetRequestQueue();
	TArray<FPackageData*> PoppedPackages;
	if (!RequestQueue.IsReadyRequestsEmpty())
	{
		while (!RequestQueue.IsReadyRequestsEmpty())
		{
			PoppedPackages.Add(RequestQueue.PopReadyRequest());
		}
		for (FPackageData* PackageData : PoppedPackages)
		{
			RequestQueue.AddReadyRequest(PackageData);
			AddPackageIfPossibleAndReportDone(PackageData);
		}
	}
	if (OutRetractionPackages.Num() >= NumToRetract)
	{
		return;
	}

	// Send back loadstate packages that have not started loading before sending back any that have.
	FLoadQueue& LoadQueue = PackageDatas->GetLoadQueue();
	for (FPackageData* PackageData : LoadQueue)
	{
		TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
		if (!(Preloader && (Preloader->IsPackageLoaded() || Preloader->GetState() >= EPreloaderState::ActivePreload)))
		{
			if (AddPackageIfPossibleAndReportDone(PackageData))
			{
				return;
			}
		}
	}
	for (FPackageData* PackageData : LoadQueue)
	{
		TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
		if (Preloader && (Preloader->IsPackageLoaded() || Preloader->GetState() >= EPreloaderState::ActivePreload))
		{
			if (AddPackageIfPossibleAndReportDone(PackageData))
			{
				return;
			}
		}
	}
	// Send back savestate packages that have not started saving before sending back any that have.
	for (FPackageData* PackageData : PackageDatas->GetSaveQueue())
	{
		if (PackageData->GetSaveSubState() <= ESaveSubState::StartSave)
		{
			if (AddPackageIfPossibleAndReportDone(PackageData))
			{
				return;
			}
		}
	}
	for (FPackageData* PackageData : PackageDatas->GetSaveQueue())
	{
		if (PackageData->GetSaveSubState() > ESaveSubState::StartSave)
		{
			if (AddPackageIfPossibleAndReportDone(PackageData))
			{
				return;
			}
		}
	}
}

void UCookOnTheFlyServer::ShutdownCookAsCookWorker()
{
	if (IsDirectorCookByTheBook())
	{
		UnregisterCookByTheBookDelegates();
	}
	if (IsInSession())
	{
		ShutdownCookSession();
	}
}

FBeginCookContext UCookOnTheFlyServer::CreateCookWorkerContext()
{
	FBeginCookContext BeginContext(*this);
	*CookByTheBookOptions = CookWorkerClient->ConsumeCookByTheBookOptions();
	bZenStore = CookWorkerClient->GetInitializationIsZenStore();
	CookByTheBookOptions->CookTime = 0.0f;
	CookByTheBookOptions->CookStartTime = FPlatformTime::Seconds();
	*CookOnTheFlyOptions = CookWorkerClient->ConsumeCookOnTheFlyOptions();
	BeginContext.TargetPlatforms = CookWorkerClient->GetTargetPlatforms();

	TArray<ITargetPlatform*> UniqueTargetPlatforms = BeginContext.TargetPlatforms;
	Algo::Sort(UniqueTargetPlatforms);
	UniqueTargetPlatforms.SetNum(Algo::Unique(UniqueTargetPlatforms));
	checkf(UniqueTargetPlatforms.Num() == BeginContext.TargetPlatforms.Num(), TEXT("List of TargetPlatforms received from Director was not unique."));

	BeginContext.PlatformContexts.SetNum(BeginContext.TargetPlatforms.Num());
	for (int32 Index = 0; Index < BeginContext.TargetPlatforms.Num(); ++Index)
	{
		BeginContext.PlatformContexts[Index].TargetPlatform = BeginContext.TargetPlatforms[Index];
		// PlatformContext.PlatformData is currently null and is set in SelectSessionPlatforms
	}

	TArray<FWeakObjectPtr>& SessionStartupObjects = CookByTheBookOptions->SessionStartupObjects;
	SessionStartupObjects.Reset();
	for (FThreadSafeObjectIterator Iter; Iter; ++Iter)
	{
		SessionStartupObjects.Emplace(*Iter);
	}
	SessionStartupObjects.Shrink();

	return BeginContext;
}

void UCookOnTheFlyServer::GenerateInitialRequests(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(GenerateInitialRequests);
	TArray<ITargetPlatform*>& TargetPlatforms = BeginContext.TargetPlatforms;
	TMap<FName, FName> StartupSoftObjectPackageReferencers;
	if (!CookByTheBookOptions->bSkipSoftReferences)
	{
		TSet<FName> PackagesFromRedirectCollector;
		// Get the list of soft references, for both empty package and all startup packages
		GRedirectCollector.ProcessSoftObjectPathPackageList(NAME_None, false, PackagesFromRedirectCollector);
		for (FName PackageName : PackagesFromRedirectCollector)
		{
			StartupSoftObjectPackageReferencers.Add(PackageName, NAME_None);
		}

		for (FName StartupPackage : CookByTheBookOptions->StartupPackages)
		{
			PackagesFromRedirectCollector.Empty();
			GRedirectCollector.ProcessSoftObjectPathPackageList(StartupPackage, false, PackagesFromRedirectCollector);
			for (FName PackageName : PackagesFromRedirectCollector)
			{
				StartupSoftObjectPackageReferencers.Add(PackageName, StartupPackage);
			}
		}
	}
	GRedirectCollector.OnStartupPackageLoadComplete();

	TMap<FName, TArray<FName>> GameDefaultObjects;
	GetGameDefaultObjects(TargetPlatforms, GameDefaultObjects);

	// Strip out the default maps from SoftObjectPaths collected from startup packages. They will be added to the cook if necessary by CollectFilesToCook.
	for (const TTuple<FName, TArray<FName>>& GameDefaultSet : GameDefaultObjects)
	{
		for (FName AssetName : GameDefaultSet.Value)
		{
			StartupSoftObjectPackageReferencers.Remove(AssetName);
		}
	}
	// Strip out missing packages from SoftObjectPaths collected from startup packages.
	for (TMap<FName,FName>::TIterator It = StartupSoftObjectPackageReferencers.CreateIterator(); It; ++It)
	{
		const FName PackageName = It.Key();
		if (PackageName.IsNone() || PackageDatas->GetFileNameByPackageName(PackageName).IsNone())
		{
			It.RemoveCurrent();
		}
	}

	TArray<FString> CookMaps = BeginContext.StartupOptions->CookMaps;
	TArray<FString> CookFirstPackages;
	TArray<FString> CookLastPackages;
	FString Text;
	TConstArrayView<const TCHAR*> CommandLineDelimiters = UE::Cook::GetCommandLineDelimiterStrs();
	if (FParse::Param(FCommandLine::Get(), TEXT("CookFirst")))
	{
		CookFirstPackages.Append(CookMaps);
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-CookFirst="), Text))
	{
		TArray<FString> Array;
		Text.ParseIntoArray(Array, CommandLineDelimiters.GetData(), CommandLineDelimiters.Num(), true);
		CookFirstPackages.Append(Array);
		CookMaps.Append(Array);
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("CookLast")))
	{
		CookLastPackages.Append(CookMaps);
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-CookLast="), Text))
	{
		TArray<FString> Array;
		Text.ParseIntoArray(Array, CommandLineDelimiters.GetData(), CommandLineDelimiters.Num(), true);
		CookLastPackages.Append(Array);
		CookMaps.Append(Array);
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-CookReferencersOf="), Text))
	{
		TArray<FString> Array;
		Text.ParseIntoArray(Array, CommandLineDelimiters.GetData(), CommandLineDelimiters.Num(), true);
		for (FString& PackageName : Array)
		{
			CookMaps.Add(PackageName);
			TArray<FName> Referencers;
			AssetRegistry->GetReferencers(FName(FStringView(PackageName)), Referencers);
			for (FName Referencer : Referencers)
			{
				CookMaps.Add(Referencer.ToString());
			}
		}
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-CookPackageDir="), Text))
	{
		TArray<FString> Array;
		Text.ParseIntoArray(Array, CommandLineDelimiters.GetData(), CommandLineDelimiters.Num(), true);
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		// search in all paths specified
		for (FString& PackageDir : Array)
		{
			Filter.PackagePaths.Add(*PackageDir);
		}

		// add each discovered package to the list of packages to cook
		TArray<FAssetData> FoundAssets;
		AssetRegistry->GetAssets(Filter, FoundAssets);
		for (FAssetData& AssetData : FoundAssets)
		{
			// add path to package, not any asset within
			CookMaps.AddUnique(AssetData.GetSoftObjectPath().GetLongPackageName());
		}
	}


	TArray<FName> FilesInPath;
	TMap<FName, UE::Cook::FInstigator> FilesInPathInstigators;
	const TArray<FString>& CookDirectories = BeginContext.StartupOptions->CookDirectories;
	const TArray<FString>& IniMapSections = BeginContext.StartupOptions->IniMapSections;
	ECookByTheBookOptions CookOptions = CookByTheBookOptions->StartupOptions;
	CollectFilesToCook(FilesInPath, FilesInPathInstigators, CookMaps, CookDirectories, IniMapSections, CookOptions, TargetPlatforms, GameDefaultObjects);

	// Add soft/hard startup references after collecting requested files and handling empty requests
	FlushAsyncLoading();
	// TODO: When we add a way for projects to change to NoStartupPackages by default, we will need to allow packages
	// that have ECookLoadType::UsedInGame even if the project has opted out of StartupPackages. So we will need to
	// call ProcessUnsolicitedPackages here even if the project has opted out of StartupPackages, and inside it
	// skip the StartupPackages but keep the StartupPackageCookLoadScope packages.
	// We will also need a separate container for StartupCookLoadScopeSoftObjectPackages, so we
	// can add the SoftObjectPaths required by those packages.
	if (!CookByTheBookOptions->bSkipHardReferences && !EnumHasAnyFlags(CookOptions, ECookByTheBookOptions::NoStartupPackages))
	{
		ProcessUnsolicitedPackages(&FilesInPath, &FilesInPathInstigators);
	}
	else
	{
		// Clear the list of startup packages currently held by the packagetracker so that we don't see them when we
		// ProcessUnsolicitedPackages to find hard references used by the first requested package we load.
		(void)PackageTracker->GetPackageStream();
	}
	if (!CookByTheBookOptions->bSkipSoftReferences && !EnumHasAnyFlags(CookOptions, ECookByTheBookOptions::NoStartupPackages))
	{
		for (TPair<FName,FName> SoftObjectReferencerPair : StartupSoftObjectPackageReferencers)
		{
			TMap<FSoftObjectPath, FSoftObjectPath> RedirectedPaths;

			// If this is a redirector, extract destination from asset registry
			if (ContainsRedirector(SoftObjectReferencerPair.Key, RedirectedPaths))
			{
				for (TPair<FSoftObjectPath, FSoftObjectPath>& RedirectedPath : RedirectedPaths)
				{
					GRedirectCollector.AddAssetPathRedirection(RedirectedPath.Key, RedirectedPath.Value);
				}
			}
			AddFileToCook(FilesInPath, FilesInPathInstigators, SoftObjectReferencerPair.Key.ToString(),
				UE::Cook::FInstigator(UE::Cook::EInstigator::StartupSoftObjectPath, SoftObjectReferencerPair.Value));
		}
	}

	if (FilesInPath.Num() == 0)
	{
		LogCookerMessage(FString::Printf(TEXT("No files found to cook.")), EMessageSeverity::Warning);
	}

	{
		UE_SCOPED_HIERARCHICAL_COOKTIMER(GenerateLongPackageName);
		GenerateLongPackageNames(FilesInPath, FilesInPathInstigators);
	}
	TSet<FName> CookFirstOrLastPackages;
	TMap<FString, TOptional<bool>> CookFirstOrLastPackagesInputs;
	for (const FString& PackageName : CookFirstPackages)
	{
		CookFirstOrLastPackagesInputs.Add(PackageName, TOptional<bool>(true));
	}
	for (const FString& PackageName : CookLastPackages)
	{
		TOptional<bool>& IsCookFirst = CookFirstOrLastPackagesInputs.FindOrAdd(PackageName);
		if (IsCookFirst.IsSet() && IsCookFirst.GetValue())
		{
			UE_LOGF(LogCook, Error, "-CookFirst and -CookLast are mutually exclusive. Ignoring -CookLast for %ls.",
				*PackageName);
		}
		else
		{
			IsCookFirst.Emplace(false);
		}
	}
	if (!CookFirstOrLastPackagesInputs.IsEmpty())
	{
		for (const TPair<FString, TOptional<bool>>& Pair : CookFirstOrLastPackagesInputs)
		{
			bool bCookLast = !Pair.Value.GetValue();
			FString LongPackageName;
			if (FPackageName::TryConvertFilenameToLongPackageName(Pair.Key, LongPackageName))
			{
				FName LongPackageFName(*LongPackageName);
				CookFirstOrLastPackages.Add(LongPackageFName);
				if (bCookLast)
				{
					UE::Cook::FPackageData* PackageDataToDelay = PackageDatas->TryAddPackageDataByPackageName(LongPackageFName);
					if (PackageDataToDelay)
					{
						PackageDataToDelay->SetIsCookLast(true);
					}
				}
			}
		}
	}
	bool bCookFirstOrLast = !CookFirstOrLastPackages.IsEmpty();

	// add all the files to the cook list for the requested platforms
	for (FName PackageName : FilesInPath)
	{
		if (PackageName.IsNone())
		{
			continue;
		}

		const FName PackageFileFName = PackageDatas->GetFileNameByPackageName(PackageName);

		UE::Cook::FInstigator& Instigator = FilesInPathInstigators.FindChecked(PackageName);
		if (!PackageFileFName.IsNone())
		{
			UE::Cook::FFilePlatformRequest Request(PackageFileFName, MoveTemp(Instigator), TargetPlatforms);
			Request.SetUrgent(bCookFirstOrLast && CookFirstOrLastPackages.Contains(PackageName));
			WorkerRequests->AddStartCookByTheBookRequest(MoveTemp(Request));
		}
		else if (!FLinkerLoad::IsKnownMissingPackage(PackageName))
		{
			LogCookerMessage(FString::Printf(TEXT("Unable to find package for cooking %s. Instigator: { %s }."),
				*PackageName.ToString(), *Instigator.ToString()),
				EMessageSeverity::Warning);
		}
	}
	InitialRequestCount = WorkerRequests->GetNumExternalRequests();

	const FString& BasedOnReleaseVersion = CookByTheBookOptions->BasedOnReleaseVersion;
	const FString& CreateReleaseVersion = CookByTheBookOptions->CreateReleaseVersion;
	if (!IsCookingDLC() && !BasedOnReleaseVersion.IsEmpty())
	{
		// if we are based on a release and we are not cooking dlc then we should always be creating a new one (note that we could be creating the same one we are based on).
		// note that we might erroneously enter here if we are generating a patch instead and we accidentally passed in BasedOnReleaseVersion to the cooker instead of to unrealpak
		UE_CLOGF(CreateReleaseVersion.IsEmpty(), LogCook, Fatal, "-BasedOnReleaseVersion must be used together with either -dlcname or -CreateReleaseVersion.");

		// if we are creating a new Release then we need cook all the packages which are in the previous release (as well as the new ones)
		for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			// if we are based of a cook and we are creating a new one we need to make sure that at least all the old packages are cooked as well as the new ones
			FString OriginalAssetRegistryPath = GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, TargetPlatform->PlatformName()) / GetAssetRegistryFilename();

			TArray<UE::Cook::FConstructPackageData> BasedOnReleaseDatas;
			bool bFoundAssetRegistry = GetAllPackageFilenamesFromAssetRegistry(OriginalAssetRegistryPath, true, false, BasedOnReleaseDatas);
			ensureMsgf(bFoundAssetRegistry, TEXT("Unable to find AssetRegistry results from cook of previous version. Expected to find file %s.\n")
				TEXT("This prevents us from running validation that all files cooked in the previous release are also added to the current release."),
				*OriginalAssetRegistryPath);

			TArray<const ITargetPlatform*, TInlineAllocator<1>> RequestPlatforms;
			RequestPlatforms.Add(TargetPlatform);
			for (const UE::Cook::FConstructPackageData& PackageData : BasedOnReleaseDatas)
			{
				WorkerRequests->AddStartCookByTheBookRequest(
					UE::Cook::FFilePlatformRequest(PackageData.NormalizedFileName,
						UE::Cook::EInstigator::PreviousAssetRegistry, RequestPlatforms));
			}
		}
	}

	if (CookByTheBookOptions->bCookList)
	{
		WorkerRequests->LogAllRequestedFiles();
	}
}

void UCookOnTheFlyServer::RecordDLCPackagesFromBaseGame(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(RecordDLCPackagesFromBaseGame);
	using namespace UE::Cook;

	if (!IsCookingDLC())
	{
		return;
	}

	// If we're only cooking the DLC plugin we don't need Development Asset Registry.
	// All references outside the plugin are never explored or cooked, just mark any
	// existing PackageData's as already cooked.
	if (IsCookingDLCPluginOnly())
	{
		FString DLCPath = GetMountedAssetPathForDLC();
		UE_LOGF(LogCook, Log, "Performing plugin only cook for: %ls", *DLCPath);

		PackageDatas->LockAndEnumeratePackageDatas([&BeginContext,&DLCPath, this](FPackageData* PackageData)
		{
			FString PackageName = PackageData->GetPackageName().ToString();

			if (!FStringView(PackageName).StartsWith(DLCPath))
			{
				for (const ITargetPlatform* TargetPlatform : BeginContext.TargetPlatforms)
				{
					PackageData->SetPlatformCooked(TargetPlatform, ECookResult::Succeeded, /*bWasCookedThisSession=*/false);
					FPackagePlatformData& PlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
					PlatformData.SetWhereCooked(EWhereCooked::BaseGame);
				}

				// Incremented only for the first platform, similar to 
				// NumPackagesIncrementallySkipped and other relevant stats. 
				++PackageDataFromBaseGameNum;
			}
		});

		return;
	}

	const ECookByTheBookOptions& CookOptions = CookByTheBookOptions->StartupOptions;
	const FString& BasedOnReleaseVersion = CookByTheBookOptions->BasedOnReleaseVersion;

	// If we're cooking against a fixed base, we don't need to verify the packages exist on disk, we simply want to use the Release Data 
	const bool bVerifyPackagesExist = !IsCookingAgainstFixedBase();
	const bool bReevaluateUncookedPackages = !!(CookOptions & ECookByTheBookOptions::DlcReevaluateUncookedAssets);

	// if we are cooking dlc we must be based on a release version cook
	check(!BasedOnReleaseVersion.IsEmpty());

	auto ReadDevelopmentAssetRegistry = [this, &BasedOnReleaseVersion, bVerifyPackagesExist, bReevaluateUncookedPackages]
	(TArray<FConstructPackageData>& OutPackageList, const FString& InPlatformName)
	{
		TArray<FString> AttemptedNames;
		FString OriginalSandboxRegistryFilename = GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, InPlatformName) / TEXT("Metadata") / GetDevelopmentAssetRegistryFilename();
		AttemptedNames.Add(OriginalSandboxRegistryFilename);

		// if this check fails probably because the asset registry can't be found or read
		bool bSucceeded = GetAllPackageFilenamesFromAssetRegistry(OriginalSandboxRegistryFilename, bVerifyPackagesExist, bReevaluateUncookedPackages, OutPackageList);
		if (!bSucceeded)
		{
			OriginalSandboxRegistryFilename = GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, InPlatformName) / GetAssetRegistryFilename();
			AttemptedNames.Add(OriginalSandboxRegistryFilename);
			bSucceeded = GetAllPackageFilenamesFromAssetRegistry(OriginalSandboxRegistryFilename, bVerifyPackagesExist, bReevaluateUncookedPackages, OutPackageList);
		}

		if (!bSucceeded)
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(*InPlatformName);
			if (PlatformInfo)
			{
				for (const PlatformInfo::FTargetPlatformInfo* PlatformFlavor : PlatformInfo->Flavors)
				{
					OriginalSandboxRegistryFilename = GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, PlatformFlavor->Name.ToString()) / GetAssetRegistryFilename();
					AttemptedNames.Add(OriginalSandboxRegistryFilename);
					bSucceeded = GetAllPackageFilenamesFromAssetRegistry(OriginalSandboxRegistryFilename, bVerifyPackagesExist, bReevaluateUncookedPackages, OutPackageList);
					if (bSucceeded)
					{
						break;
					}
				}
			}
		}

		if (bSucceeded)
		{
			UE_LOGF(LogCook, Log, "Loaded assetregistry: %ls", *OriginalSandboxRegistryFilename);
		}
		else
		{
			UE_LOGF(LogCook, Log, "Failed to load DevelopmentAssetRegistry for platform %ls. Attempted the following names:\n%ls", *InPlatformName, *FString::Join(AttemptedNames, TEXT("\n")));
		}
		return bSucceeded;
	};

	TArray<FConstructPackageData> OverridePackageList;
	FString DevelopmentAssetRegistryPlatformOverride;
	const bool bUsingDevRegistryOverride = FParse::Value(FCommandLine::Get(), TEXT("DevelopmentAssetRegistryPlatformOverride="), DevelopmentAssetRegistryPlatformOverride);
	if (bUsingDevRegistryOverride)
	{
		// Read the contents of the asset registry for the overriden platform. We'll use this for all requested platforms so we can just keep one copy of it here
		bool bReadSucceeded = ReadDevelopmentAssetRegistry(OverridePackageList, *DevelopmentAssetRegistryPlatformOverride);
		if (!bReadSucceeded || OverridePackageList.Num() == 0)
		{
			UE_LOGF(LogCook, Fatal,
				"%ls based-on AssetRegistry file %ls for DevelopmentAssetRegistryPlatformOverride %ls. "
				"When cooking DLC, if DevelopmentAssetRegistryPlatformOverride is specified %ls is expected to exist under Release/<override> and contain some valid data. Terminating the cook.",
				!bReadSucceeded ? TEXT("Could not find") : TEXT("Empty"),
				*(GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, DevelopmentAssetRegistryPlatformOverride) / TEXT("Metadata") / GetAssetRegistryFilename()),
				*DevelopmentAssetRegistryPlatformOverride, *GetAssetRegistryFilename());
		}
	}

	bool bFirstAddExistingPackageDatas = true;
	for (const ITargetPlatform* TargetPlatform : BeginContext.TargetPlatforms)
	{
		TArray<FConstructPackageData> PackageList;
		FString PlatformNameString = TargetPlatform->PlatformName();
		FName PlatformName(*PlatformNameString);

		if (!bUsingDevRegistryOverride)
		{
			bool bReadSucceeded = ReadDevelopmentAssetRegistry(PackageList, PlatformNameString);
			if (!bReadSucceeded && !CookByTheBookOptions->bAllowUncookedAssetReferences)
			{
				UE_LOGF(LogCook, Fatal,
					"Could not find based-on AssetRegistry file %ls for platform %ls. "
					"When cooking DLC, %ls is expected to exist Release/<platform> for each platform being cooked. (Or use DevelopmentAssetRegistryPlatformOverride=<PlatformName> to specify an override platform that all platforms should use to find the %ls file). Terminating the cook.",
					*(GetBasedOnReleaseVersionAssetRegistryPath(BasedOnReleaseVersion, PlatformNameString) / TEXT("Metadata") / GetAssetRegistryFilename()),
					*PlatformNameString, *GetAssetRegistryFilename(), *GetAssetRegistryFilename());
			}
		}

		int32 UnusedBaseGameNumForOtherPlatforms = 0;
		int32& NumForStatusReporting = bFirstAddExistingPackageDatas ? this->PackageDataFromBaseGameNum : UnusedBaseGameNumForOtherPlatforms;
		TArray<FConstructPackageData>& ActivePackageList = OverridePackageList.Num() > 0 ? OverridePackageList : PackageList;
		if (ActivePackageList.Num() > 0)
		{
			PackageDatas->AddExistingPackageDatasForPlatform(ActivePackageList, TargetPlatform,
				bFirstAddExistingPackageDatas, NumForStatusReporting);
		}

		{
			// allow game or plugins to modify if certain packages from the base game should be recooked.
			TArray<FName> PlatformBasedPackages;
			PlatformBasedPackages.Reset(ActivePackageList.Num());
			for (UE::Cook::FConstructPackageData& PackageData : ActivePackageList)
			{
				PlatformBasedPackages.Add(PackageData.NormalizedFileName);
			}

			TSet<FName> PackagesToClearCookResults;
			UAssetManager::Get().ModifyDLCBasePackages(TargetPlatform, PlatformBasedPackages, PackagesToClearCookResults);
			if (PackagesToClearCookResults.Num())
			{
				PackageDatas->ClearCookResultsForPackages(PackagesToClearCookResults, TargetPlatform, NumForStatusReporting);
			}
		}
		bFirstAddExistingPackageDatas = false;
	}

	FString ExtraReleaseVersionAssetsFile;
	const bool bUsingExtraReleaseVersionAssets = FParse::Value(FCommandLine::Get(), TEXT("ExtraReleaseVersionAssets="), ExtraReleaseVersionAssetsFile);
	if (bUsingExtraReleaseVersionAssets)
	{
		// read AssetPaths out of the file and add them as already-cooked PackageDatas
		TArray<FString> OutAssetPaths;
		FPaths::MakePlatformFilename(ExtraReleaseVersionAssetsFile);
		FString FullPathFromBaseDir = FPaths::ConvertRelativePathToFull(FPlatformProcess::BaseDir(), ExtraReleaseVersionAssetsFile);
		FString FullPathFromRootDir = FPaths::ConvertRelativePathToFull(FPaths::RootDir(), ExtraReleaseVersionAssetsFile);
		if (!FFileHelper::LoadFileToStringArray(OutAssetPaths, *FullPathFromBaseDir))
		{
			ensureMsgf(FFileHelper::LoadFileToStringArray(OutAssetPaths, *FullPathFromRootDir), TEXT("Failed to load from %s or %s"), *FullPathFromBaseDir, *FullPathFromRootDir);
		}
		
		for (const FString& AssetPath : OutAssetPaths)
		{
			if (FPackageData* PackageData = PackageDatas->TryAddPackageDataByFileName(FName(*AssetPath)))
			{
				for (const ITargetPlatform* TargetPlatform : BeginContext.TargetPlatforms)
				{
					{
						FPackagePlatformData& PlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
						if (!PlatformData.IsCookAttempted() && TargetPlatform == BeginContext.TargetPlatforms[0])
						{
							++PackageDataFromBaseGameNum;
						}
						PlatformData.SetWhereCooked(EWhereCooked::ExtraReleaseVersionAssets);
					}
					PackageData->SetPlatformCooked(TargetPlatform, ECookResult::Succeeded, /*bWasCookedThisSession=*/false);
				}
			}
			else
			{
				UE_LOGF(LogCook, Error, "Failed to resolve package data for ExtraReleaseVersionAsset [%ls]", *AssetPath);
			}
		}
	}
}

void UCookOnTheFlyServer::BeginCookPackageWriters(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookPackageWriters);
	for (FBeginCookContextPlatform& Context : BeginContext.PlatformContexts)
	{
		ICookedPackageWriter::FCookInfo CookInfo;
		CookInfo.CookMode = IsDirectorCookOnTheFly() ? ICookedPackageWriter::FCookInfo::CookOnTheFlyMode : ICookedPackageWriter::FCookInfo::CookByTheBookMode;
		CookInfo.bFullBuild = Context.bFullBuild;
		CookInfo.bLegacyIterativeSharedBuild = Context.bLegacyIterativeSharedBuild;
		CookInfo.bWorkerOnSharedSandbox = Context.bWorkerOnSharedSandbox;

		FindOrCreatePackageWriter(Context.TargetPlatform).BeginCook(CookInfo);
	}
}

void UCookOnTheFlyServer::SelectSessionPlatforms(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(SelectSessionPlatforms);
	PlatformManager->SelectSessionPlatforms(*this, BeginContext.TargetPlatforms);

	FindOrCreateSaveContexts(BeginContext.TargetPlatforms);
	for (FBeginCookContextPlatform& PlatformContext : BeginContext.PlatformContexts)
	{
		PlatformContext.PlatformData = PlatformManager->GetPlatformData(PlatformContext.TargetPlatform);
		PlatformContext.TargetPlatform->InitializeForCook();
	}
}

void UCookOnTheFlyServer::CalculateGlobalDependenciesHashes()
{
	UE_SCOPED_COOKTIMER(CalculateGlobalDependenciesHashes);
	const TArray<const ITargetPlatform*>& Platforms = PlatformManager->GetSessionPlatforms();
	for (const ITargetPlatform* Platform : Platforms)
	{
		UE::Cook::CalculateGlobalDependenciesHash(Platform, *this);
	}
}

void UCookOnTheFlyServer::BeginCookEditorSystems()
{
	UE_SCOPED_COOKTIMER(BeginCookEditorSystems);
	if (!IsCookingInEditor())
	{
		return;
	}

	if (IsCookByTheBookMode())
	{
		//force precache objects to refresh themselves before cooking anything
		LastUpdateTick = INT_MAX;

		COOK_STAT(UE::SavePackageUtilities::ResetCookStats());
	}

	// Notify AssetRegistry to update itself for any saved packages
	if (!bFirstCookInThisProcess)
	{
		// Force a rescan of modified package files
		TArray<FString> ModifiedPackageFileList;
		for (FName ModifiedPackage : ModifiedAssetFilenames)
		{
			ModifiedPackageFileList.Add(ModifiedPackage.ToString());
		}
		AssetRegistry->ScanModifiedAssetFiles(ModifiedPackageFileList);
	}
	ModifiedAssetFilenames.Empty();
}

void UCookOnTheFlyServer::BeginCookDirector(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookDirector);
	if (CookDirector)
	{
		CookDirector->StartCook(BeginContext);
	}
}

namespace UE::Cook
{

/** CookMultiprocess collector for FEDLCookChecker data. */
class FEDLMPCollector : public IMPCollector
{
public:
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("FEDLMPCollector"); }

	virtual void ClientTickPackage(FMPCollectorClientTickPackageContext& Context) override;
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

	static FGuid MessageType;
};
FGuid FEDLMPCollector::MessageType(TEXT("0164FD08F6884F6A82D2D00F8F70B182"));

void FEDLMPCollector::ClientTickPackage(FMPCollectorClientTickPackageContext& Context)
{
	FCbWriter Writer;
	bool bHasData;

	// For simplicity, instead of sending only information related to the given Package, we send all data.
	FEDLCookChecker::MoveToCompactBinaryAndClear(Writer, bHasData);
	if (bHasData)
	{
		Context.AddMessage(Writer.Save().AsObject());
	}
}

void FEDLMPCollector::ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	FEDLCookChecker::AppendFromCompactBinary(Message.AsFieldView());
}

}

bool UCookOnTheFlyServer::ShouldVerifyEDLCookInfo() const
{
	return CookByTheBookOptions->DlcName.IsEmpty() && !bCookFilter && !DiffModeHelper->IsDiffModeActive();
}

void UCookOnTheFlyServer::BeginCookEDLCookInfo(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookEDLCookInfo);
	if (IsCookingInEditor())
	{
		return;
	}
	FEDLCookChecker::StartSavingEDLCookInfoForVerification();
	if (CookDirector)
	{
		CookDirector->Register(new UE::Cook::FEDLMPCollector());
	}
	else if (CookWorkerClient)
	{
		CookWorkerClient->Register(new UE::Cook::FEDLMPCollector());
	}
}

void UCookOnTheFlyServer::RegisterCookByTheBookDelegates()
{
	UE_SCOPED_COOKTIMER(RecordCookByTheBookDelegates);
	if (!IsCookingInEditor())
	{
		FCoreUObjectDelegates::PackageCreatedForLoad.AddUObject(this, &UCookOnTheFlyServer::MaybeMarkPackageAsAlreadyLoaded);
	}
#if UE_WITH_OBJECT_HANDLE_TRACKING
	if (bHiddenDependenciesDebug)
	{
		ObjectHandleReadHandle = UE::CoreUObject::AddObjectHandleReadCallback(
			[this](const TArrayView<const UObject*const>& ReadObjects) { UCookOnTheFlyServer::OnObjectHandleReadDebug(ReadObjects); });
	}
#endif // UE_WITH_OBJECT_HANDLE_TRACKING
}

void UCookOnTheFlyServer::UnregisterCookByTheBookDelegates()
{
	if (!IsCookingInEditor())
	{
		FCoreUObjectDelegates::PackageCreatedForLoad.RemoveAll(this);
	}
#if UE_WITH_OBJECT_HANDLE_TRACKING
	if (ObjectHandleReadHandle.IsValid())
	{
		UE::CoreUObject::RemoveObjectHandleReadCallback(ObjectHandleReadHandle);
		ObjectHandleReadHandle = UE::CoreUObject::FObjectHandleTrackingCallbackId();
	}
#endif // UE_WITH_OBJECT_HANDLE_TRACKING
}

TArray<FName> UCookOnTheFlyServer::GetNeverCookPackageNames(TArrayView<const FString> ExtraNeverCookDirectories) const
{
	TArray<FString> NeverCookDirectories(ExtraNeverCookDirectories);

	if (bRunningAsShaderServer)
	{
		return TArray<FName>();
	}

	auto AddDirectoryPathArray = [&NeverCookDirectories](const TArray<FDirectoryPath>& DirectoriesToNeverCook, const TCHAR* SettingName)
	{
		for (const FDirectoryPath& DirToNotCook : DirectoriesToNeverCook)
		{
			FString LocalPath;
			if (FPackageName::TryConvertGameRelativePackagePathToLocalPath(DirToNotCook.Path, LocalPath))
			{
				NeverCookDirectories.Add(LocalPath);
			}
			else
			{
				// An unmounted directory that we try to add to nevercook settings is not an error case; since the
				// directory is unmounted nothing in it can be cooked. And no plugins should be loading after the first
				// call to this function (which is after CookCommandlet::Main or after editor startup), so we shouldn't
				// have the problem of a plugin possibly loading later. So downgrade this warning message to verbose.
				UE_LOGF(LogCook, Verbose, "'%ls' has invalid element '%ls'", SettingName, *DirToNotCook.Path);
			}
		}

	};
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();

	if (IsDirectorCookByTheBook())
	{
		// Respect the packaging settings nevercook directories for CookByTheBook
		AddDirectoryPathArray(PackagingSettings->DirectoriesToNeverCook, TEXT("ProjectSettings -> Project -> Packaging -> Directories to never cook"));
		AddDirectoryPathArray(PackagingSettings->TestDirectoriesToNotSearch, TEXT("ProjectSettings -> Project -> Packaging -> Test directories to not search"));
	}

	// For all modes, never cook External Actors; they are handled by the parent map
	FString ExternalActorsFolderName = ULevel::GetExternalActorsFolderName();
	FString ExternalObjectsFolderName = FPackagePath::GetExternalObjectsFolderName();
	FString FullExternalPath;
	for (const TCHAR* ProjectFolder : { TEXT("/Game/"), TEXT("/Engine/") })
	{
		for (const FString* ExternalFolderName : { &ExternalActorsFolderName, &ExternalObjectsFolderName })
		{
			FullExternalPath = FPaths::Combine(FString(ProjectFolder), *ExternalFolderName);
			NeverCookDirectories.Add(MoveTemp(FullExternalPath));
		}
	}
	for (TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPluginsWithContent())
	{
		for (const FString* ExternalFolderName : { &ExternalActorsFolderName, &ExternalObjectsFolderName })
		{
			FullExternalPath = FPaths::Combine(Plugin->GetMountedAssetPath(), *ExternalFolderName);
			NeverCookDirectories.Add(MoveTemp(FullExternalPath));
		}
	}

	TArray<FName> NeverCookPackages;
	if (AssetRegistry->IsSearchAllAssets() && !AssetRegistry->IsLoadingAssets())
	{
		TDirectoryTree<int32> NeverCookDirectoryTree;
		for (const FString& LocalDirectory : NeverCookDirectories)
		{
			FString PackagePath;
			if (FPackageName::TryConvertFilenameToLongPackageName(LocalDirectory, PackagePath))
			{
				NeverCookDirectoryTree.FindOrAdd(PackagePath);
			}
		}

		FRWLock Lock;
		AssetRegistry->EnumerateAllPackages(
			[&NeverCookPackages, &NeverCookDirectoryTree, &Lock](FName PackageName, const FAssetPackageData& PackageData)
			{
				FNameBuilder PackageNameBuilder(PackageName);
				if (NeverCookDirectoryTree.ContainsPathOrParent(PackageNameBuilder))
				{
					FWriteScopeLock ScopeLock(Lock);
					NeverCookPackages.Add(PackageName);
				}
			}, UE::AssetRegistry::EEnumeratePackagesFlags::Parallel);
	}
	else
	{
		// CookOnTheFly in editor calls this function at editorstartup, before the AssetRegistry has loaded.
		// Rather than blocking on the AssetRegistry now, fallback to scanning the directories on disk
		// TODO: Change CookOnTheFlyStartup in the editor to delay most of its startup until the AssetRegistry has
		// finished loading so we can block on the AssetRegistry before calling this function.
		bool bUseDirectoryScanFallback = true;

		if (EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::SkipHardReferences) &&
			bCookFastStartup)
		{
			// When using -cooksinglepackagenorefs, skip the calculation of NeverCook packages since it requires
			// waiting for the full AssetRegistry scan
			bUseDirectoryScanFallback = false;
		}

		if (bUseDirectoryScanFallback)
		{
			TArray<FString> ResultFilePathsToNeverCook;
			FPackageName::FindPackagesInDirectories(ResultFilePathsToNeverCook, NeverCookDirectories);
			NeverCookPackages.Reserve(ResultFilePathsToNeverCook.Num());
			FString PackageName;
			for (FString& FilePath : ResultFilePathsToNeverCook)
			{
				if (FPackageName::TryConvertFilenameToLongPackageName(FilePath, PackageName))
				{
					NeverCookPackages.Add(FName(PackageName));
				}
			}
		}
	}

	return NeverCookPackages;
}

/* UCookOnTheFlyServer callbacks
 *****************************************************************************/

void UCookOnTheFlyServer::MaybeMarkPackageAsAlreadyLoaded(UPackage *Package)
{
	// can't use this optimization while cooking in editor
	check(IsCookingInEditor()==false);
	check(IsDirectorCookByTheBook());

	// if the package is already fully loaded then we are not going to mark it up anyway
	if ( Package->IsFullyLoaded() )
	{
		return;
	}

	bool bShouldMarkAsAlreadyProcessed = false;

	TArray<const ITargetPlatform*> CookedPlatforms;
	UE::Cook::FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(Package->GetFName());
	if (!PackageData)
	{
		return;
	}
	FName StandardName = PackageData->GetFileName();
	// MPCOOKTODO: Mark it as fastload if its saving on another worker
	if (PackageData->HasAnyCookedPlatform())
	{
		bShouldMarkAsAlreadyProcessed = PackageData->HasAllCookedPlatforms(PlatformManager->GetSessionPlatforms(), true /* bIncludeFailed */);

		if (IsCookFlagSet(ECookInitializationFlags::LogDebugInfo))
		{
			FString Platforms;
			for (const TPair<const ITargetPlatform*, UE::Cook::FPackagePlatformData>& Pair : PackageData->GetPlatformDatas())
			{
				if (Pair.Key != CookerLoadingPlatformKey && Pair.Value.IsCookAttempted())
				{
					Platforms += TEXT(" ");
					Platforms += Pair.Key->PlatformName();
				}
			}
			if (!bShouldMarkAsAlreadyProcessed)
			{
				UE_LOGF(LogCook, Display, "Reloading package %ls slowly because it wasn't cooked for all platforms %ls.", *StandardName.ToString(), *Platforms);
			}
			else
			{
				UE_LOGF(LogCook, Display, "Marking %ls as reloading for cooker because it's been cooked for platforms %ls.", *StandardName.ToString(), *Platforms);
			}
		}
	}

	check(IsInGameThread());
	if (bShouldMarkAsAlreadyProcessed)
	{
		if (Package->IsFullyLoaded() == false)
		{
			Package->SetPackageFlags(PKG_ReloadingForCooker);
		}
	}
}

static void AppendExistingPackageSidecarFiles(const FString& PackageSandboxFilename, const FString& PackageStandardFilename, TArray<FString>& OutPackageSidecarFiles)
{
	const TCHAR* const PackageSidecarExtensions[] =
	{
		TEXT(".uexp"),
		// TODO: re-enable this once the client-side of the NetworkPlatformFile isn't prone to becoming overwhelmed by slow writing of unsolicited files
		//TEXT(".ubulk"),
		//TEXT(".uptnl"),
		//TEXT(".m.ubulk")
	};

	for (const TCHAR* PackageSidecarExtension : PackageSidecarExtensions)
	{
		const FString SidecarSandboxFilename = FPathViews::ChangeExtension(PackageSandboxFilename, PackageSidecarExtension);
		if (IFileManager::Get().FileExists(*SidecarSandboxFilename))
		{
			OutPackageSidecarFiles.Add(FPathViews::ChangeExtension(PackageStandardFilename, PackageSidecarExtension));
		}
	}
}

void UCookOnTheFlyServer::GetCookOnTheFlyUnsolicitedFiles(const ITargetPlatform* TargetPlatform, const FString& PlatformName, TArray<FString>& UnsolicitedFiles, const FString& Filename, bool bIsCookable)
{
	UPackage::WaitForAsyncFileWrites();

	if (bIsCookable)
		AppendExistingPackageSidecarFiles(ConvertToFullSandboxPath(*Filename, true, PlatformName), Filename, UnsolicitedFiles);

	TArray<FName> UnsolicitedFilenames;
	PackageTracker->UnsolicitedCookedPackages.GetPackagesForPlatformAndRemove(TargetPlatform, UnsolicitedFilenames);

	for (const FName& UnsolicitedFile : UnsolicitedFilenames)
	{
		FString StandardFilename = UnsolicitedFile.ToString();
		FPaths::MakeStandardFilename(StandardFilename);

		// check that the sandboxed file exists... if it doesn't then don't send it back
		// this can happen if the package was saved but the async writer thread hasn't finished writing it to disk yet

		FString SandboxFilename = ConvertToFullSandboxPath(*StandardFilename, true, PlatformName);
		if (IFileManager::Get().FileExists(*SandboxFilename))
		{
			UnsolicitedFiles.Add(StandardFilename);
			if (FPackageName::IsPackageExtension(*FPaths::GetExtension(StandardFilename, true)))
				AppendExistingPackageSidecarFiles(SandboxFilename, StandardFilename, UnsolicitedFiles);
		}
		else
		{
			UE_LOGF(LogCook, Warning, "Unsolicited file doesn't exist in sandbox, ignoring %ls", *StandardFilename);
		}
	}
}

bool UCookOnTheFlyServer::GetAllPackageFilenamesFromAssetRegistry(const FString& AssetRegistryPath, bool bVerifyPackagesExist,
	bool bReevaluateUncookedPackages, TArray<UE::Cook::FConstructPackageData>& OutPackageDatas) const
{
	using namespace UE::Cook;

	UE_SCOPED_COOKTIMER(GetAllPackageFilenamesFromAssetRegistry);
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*AssetRegistryPath));
	if (Reader)
	{
		// is there a matching preloaded AR?
		GPreloadARInfoEvent->Wait();

		bool bHadPreloadedAR = false;
		FAssetRegistryState* SerializedState = nullptr;
		TOptional<FAssetRegistryState> NonPreloadedState;
		if (AssetRegistryPath == GPreloadedARPath)
		{
			// make sure the Serialize call is done
			double Start = FPlatformTime::Seconds();
			GPreloadAREvent->Wait();
			double TimeWaiting = FPlatformTime::Seconds() - Start;
			UE_LOGF(LogCook, Display, "Blocked %.4f ms waiting for AR to finish loading", TimeWaiting * 1000.0);
			
			// if something went wrong, the num assets may be zero, in which case we do the normal load 
			bHadPreloadedAR = GPreloadedARState.GetNumAssets() > 0;
			SerializedState = &GPreloadedARState;
		}
		else
		{
			NonPreloadedState.Emplace();
			SerializedState = &NonPreloadedState.GetValue();
		}

		// if we didn't preload an AR, then we need to do a blocking load now
		if (!bHadPreloadedAR)
		{
			SerializedState->Serialize(*Reader.Get(), FAssetRegistrySerializationOptions());
		}

		check(OutPackageDatas.Num() == 0);

		// Apply lock striping to reduce contention.
		constexpr int32 UNIQUEPACKAGENAMES_BUCKETS = 31; /* prime number for best distribution using modulo */

		struct FUniquePackageNames
		{
			FRWLock Lock;
			TSet<FName> Names;
		} UniquePackageNames[UNIQUEPACKAGENAMES_BUCKETS];

		const int32 NumAssets = SerializedState->GetNumAssets();
		TArray<const FAssetData*> StateAssets;
		StateAssets.Reserve(NumAssets);
		SerializedState->EnumerateAllAssets([&StateAssets](const FAssetData& AssetData)
			{
				StateAssets.Add(&AssetData);
			});

		// We set the output packages size to the number of assets, even though the number of packages will be less than
		// the number of assets. We check for duplicates in a critical section inside the parallel for and skip the duplicate
		// work. We remove the entries for the skipped duplicates after the parallel for.
		// We are iterating over assets instead of packages because it is faster in the parallelfor to do the flat iteration over assets.
		OutPackageDatas.SetNum(NumAssets);
		// populate PackageNames in the output array
		ParallelFor(NumAssets,
			[&](int32 Index)
			{
				const FAssetData& RegistryData = *StateAssets[Index];

				// If we want to reevaluate (try cooking again) the uncooked packages (packages that were found to be empty when we cooked them before),
				// then remove the uncooked packages from the set of known packages. Uncooked packages are identified by PackageFlags == 0.
				if (bReevaluateUncookedPackages && (RegistryData.PackageFlags == 0))
				{
					return;
				}

				const FName PackageName = RegistryData.PackageName;
				const uint32 NameHash = GetTypeHash(PackageName);
				FUniquePackageNames& Bucket = UniquePackageNames[NameHash % UNIQUEPACKAGENAMES_BUCKETS];
				bool bPackageAlreadyAdded;
				{
					FWriteScopeLock ScopeLock(Bucket.Lock);
					Bucket.Names.FindOrAddByHash(NameHash, RegistryData.PackageName, &bPackageAlreadyAdded);
				}
				
				if (bPackageAlreadyAdded)
				{
					return;
				}

				if (FPackageName::GetPackageMountPoint(PackageName.ToString()).IsNone())
				{
					// Skip any packages that are not currently mounted; if we tried to find their FileNames below
					// we would get log spam
					return;
				}

				FConstructPackageData& PackageData = OutPackageDatas[Index];
				PackageData.PackageName = PackageName;

				// For any PackageNames that already have PackageDatas, mark them ahead of the loop to
				// skip the effort of checking whether they exist on disk inside the loop
				FPackageData* ExistingPackageData = PackageDatas->FindPackageDataByPackageName(PackageName);
				if (ExistingPackageData)
				{
					PackageData.NormalizedFileName = ExistingPackageData->GetFileName();
					return;
				}

				FName PackageFileName = FPackageDatas::LookupFileNameOnDisk(PackageName, true /* bRequireExists */);
				if (!PackageFileName.IsNone())
				{
					PackageData.NormalizedFileName = PackageFileName;
					return;
				}

				bool bGeneratedPackage = !!(RegistryData.PackageFlags & PKG_CookGenerated);
				if (bVerifyPackagesExist && !bGeneratedPackage)
				{
					UE_LOGF(LogCook, Warning, "Could not resolve package %ls from %ls",
						*PackageName.ToString(), *AssetRegistryPath);
				}
				else
				{
					const bool bContainsMap = !!(RegistryData.PackageFlags & PKG_ContainsMap);
					PackageFileName = FPackageDatas::LookupFileNameOnDisk(PackageName,
						false /* bRequireExists */, bContainsMap);
					if (!PackageFileName.IsNone())
					{
						PackageData.NormalizedFileName = PackageFileName;
					}
				}
			}
		);

		OutPackageDatas.RemoveAllSwap([](FConstructPackageData& PackageData)
			{
				return PackageData.NormalizedFileName.IsNone();
			}, EAllowShrinking::No);
		return true;
	}

	return false;
}

UE::Cook::ICookArtifactReader& UCookOnTheFlyServer::FindOrCreateCookArtifactReader(
	const ITargetPlatform* TargetPlatform)
{
	return *FindOrCreateSaveContext(TargetPlatform).ArtifactReader;
}

const UE::Cook::ICookArtifactReader* UCookOnTheFlyServer::FindCookArtifactReader(
	const ITargetPlatform* TargetPlatform) const
{
	const UE::Cook::FCookSavePackageContext* Context = FindSaveContext(TargetPlatform);
	return Context ? Context->ArtifactReader.Get() : nullptr;
}

ICookedPackageWriter& UCookOnTheFlyServer::FindOrCreatePackageWriter(const ITargetPlatform* TargetPlatform)
{
	return *FindOrCreateSaveContext(TargetPlatform).PackageWriter;
}

const ICookedPackageWriter* UCookOnTheFlyServer::FindPackageWriter(const ITargetPlatform* TargetPlatform) const
{
	const UE::Cook::FCookSavePackageContext* Context = FindSaveContext(TargetPlatform);
	return Context ? Context->PackageWriter : nullptr;
}

void UCookOnTheFlyServer::FindOrCreateSaveContexts(TConstArrayView<const ITargetPlatform*> TargetPlatforms)
{
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		FindOrCreateSaveContext(TargetPlatform);
	}
}

UE::Cook::FCookSavePackageContext& UCookOnTheFlyServer::FindOrCreateSaveContext(const ITargetPlatform* TargetPlatform)
{
	for (UE::Cook::FCookSavePackageContext* Context : SavePackageContexts)
	{
		if (Context->SaveContext.TargetPlatform == TargetPlatform)
		{
			return *Context;
		}
	}
	return *SavePackageContexts.Add_GetRef(CreateSaveContext(TargetPlatform));
}

const UE::Cook::FCookSavePackageContext* UCookOnTheFlyServer::FindSaveContext(const ITargetPlatform* TargetPlatform) const
{
	for (const UE::Cook::FCookSavePackageContext* Context : SavePackageContexts)
	{
		if (Context->SaveContext.TargetPlatform == TargetPlatform)
		{
			return Context;
		}
	}
	return nullptr;
}

void UCookOnTheFlyServer::InitializeAllCulturesToCook(TConstArrayView<FString> CookCultures)
{
	UE_SCOPED_COOKTIMER(InitializeAllCulturesToCook);
	CookByTheBookOptions->AllCulturesToCook.Reset();

	TArray<FString> AllCulturesToCook(CookCultures);
	for (const FString& CultureName : CookCultures)
	{
		const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(CultureName);
		for (const FString& PrioritizedCultureName : PrioritizedCultureNames)
		{
			AllCulturesToCook.AddUnique(PrioritizedCultureName);
		}
	}
	AllCulturesToCook.Sort();

	CookByTheBookOptions->AllCulturesToCook = MoveTemp(AllCulturesToCook);
}

void UCookOnTheFlyServer::CompileDLCLocalization(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(CompileDLCLocalization);
	if (!IsCookingDLC())
	{
		return;
	}
	if (!GetDefault<UUserGeneratedContentLocalizationSettings>()->bCompileDLCLocalizationDuringCook)
	{
		UE_LOGF(LogCook, Display, "DLC plugin '%ls' skipped compiling UGC localization as bCompileDLCLocalizationDuringCook is false.", *CookByTheBookOptions->DlcName);
		return;
	}

	if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(CookByTheBookOptions->DlcName))
	{
		if (Plugin->GetDescriptor().LocalizationTargets.Num() > 0)
		{
			// Used to validate that we're not loading/compiling invalid cultures during the compile step below
			FUserGeneratedContentLocalizationDescriptor DefaultUGCLocDescriptor;
			DefaultUGCLocDescriptor.InitializeFromProject();

			// Also filter the validation list by the cultures we're cooking against
			DefaultUGCLocDescriptor.CulturesToGenerate.RemoveAll([this](const FString& Culture)
			{
				return !CookByTheBookOptions->AllCulturesToCook.Contains(Culture);
			});

			// Compile UGC localization (if available) for this DLC plugin
			for (const FLocalizationTargetDescriptor& LocalizationTarget : Plugin->GetDescriptor().LocalizationTargets)
			{
				const FString InputContentDirectory = GetContentDirectoryForDLC();
				const FString InputLocalizationTargetDirectory = UserGeneratedContentLocalization::GetLocalizationTargetDirectory(LocalizationTarget.Name, InputContentDirectory);
				for (const FBeginCookContextPlatform& PlatformContext : BeginContext.PlatformContexts)
				{
					const FString PlatformName = PlatformContext.TargetPlatform->PlatformName();
					const FString OutputContentDirectory = ConvertToFullSandboxPath(InputContentDirectory, /*bForWrite*/true, PlatformName);
					const FString OutputLocalizationTargetDirectory = ConvertToFullSandboxPath(InputLocalizationTargetDirectory, /*bForWrite*/true, PlatformName);
					if (UserGeneratedContentLocalization::CompileLocalization(LocalizationTarget.Name, InputLocalizationTargetDirectory, OutputLocalizationTargetDirectory, GetDefault<UUserGeneratedContentLocalizationSettings>()->bValidateDLCLocalizationDuringCook ? &DefaultUGCLocDescriptor : nullptr))
					{
						IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
						PlatformFile.IterateDirectoryRecursively(*OutputLocalizationTargetDirectory,
							[this, &PlatformName, &OutputContentDirectory](const TCHAR* Path, bool bIsDirectory) -> bool
							{
								if (!bIsDirectory)
								{
									FString RelativePath = Path;
									FPaths::MakePathRelativeTo(RelativePath, *OutputContentDirectory);
									UE_LOGF(LogCook, Display, "DLC plugin '%ls' UGC localization compile generated '%ls' (%ls).", *CookByTheBookOptions->DlcName, *RelativePath, *PlatformName);
								}
								return true;
							});
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogCook, Display, "DLC plugin '%ls' skipped compiling UGC localization as it has no localization targets.", *CookByTheBookOptions->DlcName);
		}
	}
	else
	{
		UE_LOGF(LogCook, Display, "DLC plugin '%ls' was not known to IPluginManager. UGC localization has not been compiled and will be missing at runtime.", *CookByTheBookOptions->DlcName);
	}
}

void UCookOnTheFlyServer::GenerateLocalizationReferences()
{
	UE_SCOPED_COOKTIMER(GenerateLocalizationReferences);
	CookByTheBookOptions->SourceToLocalizedPackageVariants.Reset();

	// Find all the localized packages and map them back to their source package
	UE_LOGF(LogCook, Display, "Discovering localized assets for cultures: %ls", *FString::Join(CookByTheBookOptions->AllCulturesToCook, TEXT(", ")));

	TArray<FString> RootPaths;
	FPackageName::QueryRootContentPaths(RootPaths);

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = false;
	Filter.PackagePaths.Reserve(CookByTheBookOptions->AllCulturesToCook.Num() * RootPaths.Num());
	for (const FString& RootPath : RootPaths)
	{
		for (const FString& CultureName : CookByTheBookOptions->AllCulturesToCook)
		{
			// Cook both UE style (eg, "en-US") and Verse style (eg, "en_US") localized assets
			const FString VerseIdentifier = FCulture::CultureNameToVerseIdentifier(CultureName);
			if (CultureName != VerseIdentifier)
			{
				Filter.PackagePaths.Add(*(RootPath / TEXT("L10N") / VerseIdentifier));
			}
			Filter.PackagePaths.Add(*(RootPath / TEXT("L10N") / CultureName));
		}
	}

	TArray<FAssetData> AssetDataForCultures;
	AssetRegistry->GetAssets(Filter, AssetDataForCultures);

	UE_LOGF(LogCook, Display, "Found %d localized assets", AssetDataForCultures.Num());

	TArray<TPair<FName, FName>> FlatData;
	FlatData.SetNum(AssetDataForCultures.Num());

	ParallelFor(AssetDataForCultures.Num(), [&AssetDataForCultures, &FlatData](int32 Index)
	{
		const FAssetData& AssetData = AssetDataForCultures[Index];
		const FName LocalizedPackageName = AssetData.PackageName;
		const FName SourcePackageName = *FPackageName::GetSourcePackagePath(LocalizedPackageName.ToString());
		FlatData[Index] = {SourcePackageName, LocalizedPackageName};
	});
	
	for (const TPair<FName, FName>& Pair: FlatData)
	{
		TArray<FName>& LocalizedPackageNames = CookByTheBookOptions->SourceToLocalizedPackageVariants.FindOrAdd(Pair.Get<0>());
		LocalizedPackageNames.AddUnique(Pair.Get<1>());
	}
}

void UCookOnTheFlyServer::RegisterLocalizationChunkDataGenerator()
{
	check(!IsCookWorkerMode());

	// Localization chunking is disabled when cooking DLC as it produces output that can override the base localization data
	// Localization chunking is disabled when we're not cooking for any languages, as there would be no output generated
	if ((GetCookingDLC() == UE::Cook::ECookingDLC::Yes) || CookByTheBookOptions->AllCulturesToCook.IsEmpty())
	{
		return;
	}

	// Get the list of localization targets to chunk, and remove any targets that we've been asked not to stage
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	TArray<FString> LocalizationTargetsToChunk = PackagingSettings->LocalizationTargetsToChunk;
	{
		TArray<FString> BlocklistLocalizationTargets;
		GConfig->GetArray(TEXT("Staging"), TEXT("DisallowedLocalizationTargets"), BlocklistLocalizationTargets, GGameIni);
		if (BlocklistLocalizationTargets.Num() > 0)
		{
			LocalizationTargetsToChunk.RemoveAll([&BlocklistLocalizationTargets](const FString& InLocalizationTarget)
				{
					return BlocklistLocalizationTargets.Contains(InLocalizationTarget);
				});
		}
	}

	// Localization chunking is disabled when there are no localization targets to chunk
	if (LocalizationTargetsToChunk.IsEmpty())
	{
		return;
	}

	for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
	{
		FAssetRegistryGenerator& RegistryGenerator = *(PlatformManager->GetPlatformData(TargetPlatform)->RegistryGenerator);
		TSharedRef<FLocalizationChunkDataGenerator> LocalizationGenerator =
			MakeShared<FLocalizationChunkDataGenerator>(RegistryGenerator.GetPakchunkIndex(PackagingSettings->LocalizationTargetCatchAllChunkId),
				LocalizationTargetsToChunk, CookByTheBookOptions->AllCulturesToCook);
		RegistryGenerator.RegisterChunkDataGenerator(MoveTemp(LocalizationGenerator));
	}

	FLocalizationCookArtifact* LocalizationArtifact = new FLocalizationCookArtifact(CookByTheBookOptions->AllCulturesToCook);
	TRefCountPtr<UE::Cook::ICookArtifact> LocalizationRefCounter(LocalizationArtifact);
	RegisterArtifact(LocalizationArtifact);
}

void UCookOnTheFlyServer::RouteBeginCacheForCookedPlatformData(UE::Cook::FPackageData& PackageData, UObject* Obj,
	const ITargetPlatform* TargetPlatform, UE::Cook::ECachedCookedPlatformDataEvent* ExistingEvent)
{
	using namespace UE::Cook;

	LLM_SCOPE_BYTAG(Cooker_CachedPlatformData);
	UE_SCOPED_TEXT_COOKTIMER(*WriteToString<128>(GetClassTraceScope(Obj), TEXT("_BeginCacheForCookedPlatformData")));
	FName PackageName = PackageData.GetPackageName();
	UE_SCOPED_COOK_STAT(PackageName, EPackageEventStatType::BeginCacheForCookedPlatformData);
	
	if (!ExistingEvent)
	{
		FCachedCookedPlatformDataState& CCPDState = PackageDatas->GetCachedCookedPlatformDataObjects().FindOrAdd(Obj);
		CCPDState.AddRefFrom(&PackageData);
		ExistingEvent = &CCPDState.PlatformStates.FindOrAdd(TargetPlatform, ECachedCookedPlatformDataEvent::None);
	}
	if (*ExistingEvent != ECachedCookedPlatformDataEvent::None)
	{
		// BeginCacheForCookedPlatformData was already called; do not call it again
		return;
	}

	// Set scopes for e.g. TObjectPtr reads around the call to BeginCacheForCookedPlatformData.
	// When called from inside an existing tracking scope (e.g. SavePackage), preserve the outer
	// scope's tracking op rather than overriding it with NAME_CookerBuildObject.
	UE::Cook::FScopedCookActivePackage ScopedActivePackage(PackageName,
#if UE_WITH_OBJECT_HANDLE_TRACKING
		PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadScope()
			? NAME_None : PackageAccessTrackingOps::NAME_CookerBuildObject
#else
		FName()
#endif
	);
	Obj->BeginCacheForCookedPlatformData(TargetPlatform);
	*ExistingEvent = ECachedCookedPlatformDataEvent::BeginCacheForCookedPlatformDataCalled;
}

bool UCookOnTheFlyServer::RouteIsCachedCookedPlatformDataLoaded(UE::Cook::FPackageData& PackageData, UObject* Obj,
	const ITargetPlatform* TargetPlatform, UE::Cook::ECachedCookedPlatformDataEvent* ExistingEvent)
{
	using namespace UE::Cook;

	LLM_SCOPE_BYTAG(Cooker_CachedPlatformData);
	UE_SCOPED_TEXT_COOKTIMER(*WriteToString<128>(GetClassTraceScope(Obj), TEXT("_IsCachedCookedPlatformDataLoaded")));
	FName PackageName = PackageData.GetPackageName();
	UE_SCOPED_COOK_STAT(Obj->GetPackage()->GetFName(), EPackageEventStatType::IsCachedCookedPlatformDataLoaded);

	if (!ExistingEvent)
	{
		FCachedCookedPlatformDataState& CCPDState = PackageDatas->GetCachedCookedPlatformDataObjects().FindOrAdd(Obj);
		CCPDState.AddRefFrom(&PackageData);
		ExistingEvent = &CCPDState.PlatformStates.FindOrAdd(TargetPlatform, ECachedCookedPlatformDataEvent::None);
	}

	// Set scopes for e.g. TObjectPtr reads around the call to IsCachedCookedPlatformDataLoaded.
	// When called from inside an existing tracking scope (e.g. SavePackage), preserve the outer
	// scope's tracking op rather than overriding it with NAME_CookerBuildObject.
	UE::Cook::FScopedCookActivePackage ScopedActivePackage(PackageName,
#if UE_WITH_OBJECT_HANDLE_TRACKING
		PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadScope()
			? NAME_None : PackageAccessTrackingOps::NAME_CookerBuildObject
#else
		FName()
#endif
	);

	if (*ExistingEvent != ECachedCookedPlatformDataEvent::BeginCacheForCookedPlatformDataCalled
		&& *ExistingEvent != ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedCalled)
	{
		// We are trying to call IsCachedCookedPlatformData on an object without first calling
		// BeginCacheForCookedPlatformData, which is contractually invalid, and it might cause
		// the object to misbehave immediately, or it might just never return true.
		// This can occur when system-specific code reallocates the object out from under us by
		// calling NewObject on an existing object.
		// MaterialInstanceConstants in particual will fail to return true forever if we have not
		// called BeginCacheForCookedPlatformData again after they were reallocated.

		// To handle this reallocation case, call BeginCacheForCookedPlatformData first.
		UE_LOGF(LogCook, Display,
			"%ls was reallocated after BeginCacheForCookedPlatformData and before IsCacheCookedPlatformData returned true. Calling BeginCacheForCookedPlatformData on it again.",
			*Obj->GetFullName());
		Obj->BeginCacheForCookedPlatformData(TargetPlatform);
		*ExistingEvent = ECachedCookedPlatformDataEvent::BeginCacheForCookedPlatformDataCalled;
	}

	bool bResult = Obj->IsCachedCookedPlatformDataLoaded(TargetPlatform);
	if (bResult)
	{
		*ExistingEvent = ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedReturnedTrue;
	}
	return bResult;
}

EPackageWriterResult UCookOnTheFlyServer::SavePackageBeginCacheForCookedPlatformData(FName PackageName,
	const ITargetPlatform* TargetPlatform, TConstArrayView<UObject*> SaveableObjects, uint32 SaveFlags)
{
	using namespace UE::Cook;

	FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(PackageName);
	check(PackageData); // This callback is called from a packagesave we initiated, so it should exist

	TArray<FCachedObjectInOuter>& CachedObjectsInOuter = PackageData->GetCachedObjectsInOuter();
	int32& NextIndex = PackageData->GetCookedPlatformDataNextIndex();
	TArray<UObject*> PendingObjects;
	for (UObject* Object : SaveableObjects)
	{
		FCachedCookedPlatformDataState& CCPDState = PackageDatas->GetCachedCookedPlatformDataObjects().FindOrAdd(Object);
		if (!CCPDState.PackageDatas.Contains(PackageData))
		{
			CCPDState.AddRefFrom(PackageData);

			// NextIndex is usually at the end of CachedObjectsInOuter, but in case it is not, insert the new Object 
			// at NextIndex so that we still record that we have not called BeginCache on objects after it. Then increment
			// NextIndex to indicate we have already called (down below) BeginCache on the added object, so that
			// ReleaseCookedPlatformData knows that it needs to call Clear on it.
			check(NextIndex >= 0);
			CachedObjectsInOuter.Insert(FCachedObjectInOuter(Object), NextIndex);
			++NextIndex;
		}

		ECachedCookedPlatformDataEvent& ExistingEvent =
			CCPDState.PlatformStates.FindOrAdd(TargetPlatform, ECachedCookedPlatformDataEvent::None);
		if (ExistingEvent != ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedReturnedTrue)
		{
			if (ExistingEvent == ECachedCookedPlatformDataEvent::None)
			{
				RouteBeginCacheForCookedPlatformData(*PackageData, Object, TargetPlatform, &ExistingEvent);
			}
			if (bCallIsCachedOnSaveCreatedObjects)
			{
				// TODO: Enable bCallIsCachedOnSaveCreatedObjects so that we call IsCachedCookedPlatformDataLoaded on all the objects until it
				// returns true. This is required for the BeginCacheForCookedPlatformData contract.
				// Doing so will cause us to return Timeout and retry the save later after the pending objects have completed.
				// We tried enabling this once, but it created knockon bugs: Textures created by landscape were not handling it correctly
				// (which we have subsequently fixed) and MaterialInstanceConstants created by landscape were not handling it correctly (which 
				// we have not yet diagnosed).
				if (!RouteIsCachedCookedPlatformDataLoaded(*PackageData, Object, TargetPlatform, &ExistingEvent))
				{
					PackageDatas->AddPendingCookedPlatformData(FPendingCookedPlatformData(Object, TargetPlatform,
						*PackageData, false /* bNeedsResourceRelease */, *this));
					PendingObjects.Add(Object);
				}
			}
		}
	}

	if (!PendingObjects.IsEmpty())
	{
		constexpr float MaxWaitSeconds = 30.f;
		constexpr float SleepTimeSeconds = 0.010f;
		const double EndTimeSeconds = FPlatformTime::Seconds() + MaxWaitSeconds;
		for (;;)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(PollPendingCookedPlatformDatas);
			// Note that we don't report the NumAsyncWorkRetired value here to the PumpSaves output variable.
			// That reporting is currently only necessary for softlock detection, and since we're in the middle
			// of a save we will already be reporting some progress and don't need to report the additional
			// progress of NumAsyncWorkRetired occurrences.
			int32 NumAsyncWorkRetired;
			PackageDatas->PollPendingCookedPlatformDatas(true /* bForce */, LastCookableObjectTickTime,
				NumAsyncWorkRetired);
			if (PackageData->GetNumPendingCookedPlatformData() == 0)
			{
				break;
			}
			if (SaveFlags & SAVE_AllowTimeout)
			{
				return EPackageWriterResult::Timeout;
			}
			if (FPlatformTime::Seconds() > EndTimeSeconds)
			{
				UObject* Culprit = nullptr;
				for (UObject* Object : PendingObjects)
				{
					FCachedCookedPlatformDataState& CCPDState = PackageDatas->GetCachedCookedPlatformDataObjects().FindOrAdd(Object);
					ECachedCookedPlatformDataEvent& ExistingEvent = CCPDState.PlatformStates.FindOrAdd(TargetPlatform,
						ECachedCookedPlatformDataEvent::None);
					if (ExistingEvent != ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedReturnedTrue)
					{
						Culprit = Object;
						break;
					}
				}
				if (!Culprit)
				{
					UE_LOGF(LogCook, Warning,
						"SavePackageBeginCacheForCookedPlatformData Error for package %ls: GetNumPendingCookedPlatformData() != 0 but no Culprit found. Ignoring it and continuing.",
						*PackageName.ToString());
					break;
				}
				UE_LOGF(LogSavePackage, Error, "Save of %ls failed: timed out waiting for IsCachedCookedPlatformDataLoaded on %ls.",
					*PackageName.ToString(), *Culprit->GetFullName());
				return EPackageWriterResult::Error;
			}

			FPlatformProcess::Sleep(SleepTimeSeconds);
		}
	}
	return EPackageWriterResult::Success;
}

void UCookOnTheFlyServer::OnDiscoveredPackageDebug(FName PackageName, const UE::Cook::FInstigator& Instigator)
{
	using namespace UE::Cook;

	if (!bHiddenDependenciesDebug)
	{
		return;
	}
	switch (Instigator.Category)
	{
	case EInstigator::StartupPackage: [[fallthrough]];
	case EInstigator::StartupPackageCookLoadScope: [[fallthrough]];
	case EInstigator::GeneratedPackage: [[fallthrough]];
	case EInstigator::ForceExplorableSaveTimeSoftDependency:
	case EInstigator::BuildDependency:
		// Not a Hidden dependency
		return;
	default:
		break;
	}

	bool bShouldReport = true;
	PackageDatas->UpdateThreadsafePackageData(PackageName,
		[&bShouldReport](FThreadsafePackageData& Value, bool bNew)
		{
			if (!bNew)
			{
				switch (Value.Instigator.Category)
				{
				case EInstigator::NotYetRequested:
				case EInstigator::InvalidCategory:
					break;
				default:
					// Discovered earlier; nothing to report now
					bShouldReport = false;
					return;
				}

				if (Value.bHasLoggedDiscoveryWarning)
				{
					// Discovered and warned earlier; has not yet completed the request phase so Instigator is not yet set
					// Do not log it again
					bShouldReport = false;
					return;
				}
			}

			bShouldReport = true;
			Value.bHasLoggedDiscoveryWarning = true;
		});

	if (!bShouldReport)
	{
		return;
	}
	ReportHiddenDependency(Instigator.Referencer, PackageName);
}

FName EngineTransientName(TEXT("/Engine/Transient"));

void UCookOnTheFlyServer::OnObjectHandleReadDebug(const TArrayView<const UObject*const>& ReadObjects)
{
	using namespace UE::Cook;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	if (ReadObjects.IsEmpty() || (ReadObjects.Num() == 1 && (!ReadObjects[0] || !ReadObjects[0]->HasAnyFlags(RF_Public))))
	{
		return;
	}

	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (!AccumulatedScopeData || AccumulatedScopeData->BuildOpName.IsNone())
	{
		return;
	}

	FName ReferencerPackageName = AccumulatedScopeData->PackageName;
	if (ReferencerPackageName.IsNone() || ReferencerPackageName == EngineTransientName)
	{
		return;
	}
	TStringBuilder<256> ReferencerPackageNameStr;
	ReferencerPackageName.ToString(ReferencerPackageNameStr);
	if (FPackageName::IsTempPackage(ReferencerPackageNameStr) ||
		FPackageName::IsVersePackage(ReferencerPackageNameStr))
	{
		return;
	}

	// Accelerate analysis and make hitting breakpoints more unique by ignoring dependencies that we have already
	// logged for the most-recently-used referencer
	static FName LastReferencer;
	static TSet<FName> HandledDependencies;

	TArray<FName, TInlineAllocator<16>> DependencyPackageNames;
	for (const UObject* ReadObject : ReadObjects)
	{
		if (!ReadObject || !ReadObject->HasAnyFlags(RF_Public))
		{
			continue;
		}
		UPackage* DependencyPackage = ReadObject->GetOutermost();
		if (DependencyPackage->HasAnyFlags(RF_Transient))
		{
			continue;
		}
		FName DependencyPackageName = DependencyPackage->GetFName();
		if (ReferencerPackageName == DependencyPackageName)
		{
			continue;
		}
		if (ReferencerPackageName != LastReferencer)
		{
			LastReferencer = ReferencerPackageName;
			HandledDependencies.Reset();
		}
		bool bAlreadyExists;
		HandledDependencies.Add(DependencyPackageName, &bAlreadyExists);
		if (bAlreadyExists)
		{
			continue;
		}

		TStringBuilder<256> DependencyPackageNameStr;
		DependencyPackageName.ToString(DependencyPackageNameStr);
		if (FPackageName::IsScriptPackage(DependencyPackageNameStr) ||
			FPackageName::IsTempPackage(DependencyPackageNameStr))
		{
			continue;
		}
		DependencyPackageNames.Add(DependencyPackageName);
	}
	DependencyPackageNames.RemoveAllSwap([this, ReferencerPackageName](FName DependencyPackageName)
		{
			return AssetRegistry->ContainsDependency(ReferencerPackageName, DependencyPackageName,
				UE::AssetRegistry::EDependencyCategory::Package);
		});
	if (DependencyPackageNames.IsEmpty())
	{
		return;
	}

	// Only report the first hidden dependency from a referencerpackage, to reduce spam
	bool bShouldReport = true;
	PackageDatas->UpdateThreadsafePackageData(ReferencerPackageName,
		[&bShouldReport](FThreadsafePackageData& Value, bool bNew)
		{
			if (Value.bHasLoggedDependencyWarning)
			{
				bShouldReport = false;
				return;
			}
			Value.bHasLoggedDependencyWarning = true;
		});
	if (!bShouldReport)
	{
		return;
	}
	ReportHiddenDependency(ReferencerPackageName, DependencyPackageNames[0]);
#endif
}

void UCookOnTheFlyServer::ReportHiddenDependency(FName Referencer, FName Dependency)
{
	using namespace UE::Cook;

	FScopeLock HiddenDependenciesScopeLock(&HiddenDependenciesLock);

	if (!HiddenDependenciesClassPathFilterList.IsEmpty())
	{
		TOptional<FAssetPackageData> AssetPackageData;

		bool bImportedClassInFilterList = false;
		if (!Referencer.IsNone())
		{
			AssetPackageData = AssetRegistry->GetAssetPackageDataCopy(Referencer);
			if (AssetPackageData.IsSet())
			{
				for (FName ImportedClass : AssetPackageData->ImportedClasses)
				{
					if (HiddenDependenciesClassPathFilterList.Contains(ImportedClass))
					{
						bImportedClassInFilterList = true;
						break;
					}
				}
			}
			if (!bImportedClassInFilterList)
			{
				TOptional<FThreadsafePackageData> Data = PackageDatas->FindThreadsafePackageData(Referencer);
				FName Generator = Data ? Data->Generator : NAME_None;
				if (!Generator.IsNone())
				{
					AssetPackageData = AssetRegistry->GetAssetPackageDataCopy(Generator);
					if (AssetPackageData.IsSet())
					{
						for (FName ImportedClass : AssetPackageData->ImportedClasses)
						{
							if (HiddenDependenciesClassPathFilterList.Contains(ImportedClass))
							{
								bImportedClassInFilterList = true;
								break;
							}
						}
					}
				}
			}
		}

		bool bShouldReport = (bHiddenDependenciesClassPathFilterListIsAllowList ?
			bImportedClassInFilterList : !bImportedClassInFilterList);
		if (!bShouldReport)
		{
			return;
		}
	}

	FPackageData* ReferencerPackageData = PackageDatas->TryAddPackageDataByFileName(Referencer);
	FPackageData* DependencyPackageData = PackageDatas->TryAddPackageDataByFileName(Dependency);
	if (!ReferencerPackageData || !DependencyPackageData)
	{
		return;
	}
	ReferencerPackageData->AddDiscoveredDependency(EDiscoveredPlatformSet::CopyFromInstigator, DependencyPackageData,
		EInstigator::Unsolicited);
}

static
void ConditionalWaitOnCommandFile(FStringView GateName, TFunctionRef<void (FStringView)> CommandHandler)
{
	TStringBuilder<128> ArgPrefix(InPlace, TEXTVIEW("-"), GateName, TEXTVIEW("WaitOnCommandFile="));

	FString WaitOnCommandFile;
	if (!FParse::Value(FCommandLine::Get(), *ArgPrefix, WaitOnCommandFile))
	{
		return;
	}

	UE_LOGF(LogCook, Display, "---- Waiting For Content: Start ----");
	uint64 WaitStartTime = FPlatformTime::Cycles64();
	uint64 LastMessageTime = WaitStartTime;
	FString CommandContents;
	if (!FLockFile::TryReadAndClear(*WaitOnCommandFile, CommandContents))
	{
		UE_LOGF(LogCook, Display, "Waiting for %.*ls command file at %ls...", GateName.Len(), GateName.GetData(), *WaitOnCommandFile);

		while (!FLockFile::TryReadAndClear(*WaitOnCommandFile, CommandContents))
		{
			uint64 LoopTime = FPlatformTime::Cycles64();
			if (FPlatformTime::ToSeconds64(LoopTime - LastMessageTime) > 60.f)
			{
				double TimeSinceWaitStartTime = FPlatformTime::ToSeconds64(LoopTime - WaitStartTime);
				UE_LOGF(LogCook, Display, "Waited %.1fs for %.*ls command file at %ls...", TimeSinceWaitStartTime, GateName.Len(), GateName.GetData(), *WaitOnCommandFile);
				LastMessageTime = LoopTime;
			}
			FPlatformProcess::Sleep(20.f/1000.f);
		}
	}

	CommandHandler(CommandContents);
	UE_LOGF(LogCook, Display, "---- Waiting For Content: End ----");
}

void UCookOnTheFlyServer::BroadcastCookStarted()
{
	UE_SCOPED_COOKTIMER(BroadcastCookStarted);
	UE::Cook::FDelegates::CookStarted.Broadcast(*this);
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().UpdateStatsPerFrame();
#endif

	// Register collectors used internally by CookOnTheFlyServer.
	// External systems would do this during CookStarted.Broadcast
	if (GetProcessType() != UE::Cook::EProcessType::SingleProcess)
	{
#if UE_WITH_CONFIG_TRACKING
		ConfigCollector = new UE::ConfigAccessTracking::FConfigAccessTrackingCollector();
		RegisterCollector(ConfigCollector);
#endif
	}
}

void UCookOnTheFlyServer::BroadcastCookFinished()
{
	// Unregister collectors used internally by CookOnTheFlyServer.
	if (GetProcessType() != UE::Cook::EProcessType::SingleProcess)
	{
#if UE_WITH_CONFIG_TRACKING
		UnregisterCollector(ConfigCollector);
		ConfigCollector.SafeRelease();
#endif
	}

	UE::Cook::FDelegates::CookFinished.Broadcast(*this);
}

bool UCookOnTheFlyServer::IsStalled()
{
	return StallDetector->IsStalled(PackageDatas->GetNumCooked(), PackageDatas->GetMonitor().GetNumInProgress());
}

#undef LOCTEXT_NAMESPACE
