// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphConvert.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Algo/Compare.h"
#endif

namespace Algo::Graph
{

void ConstructTransposeGraph(TConstArrayView<TConstArrayView<FVertex>> Graph, TArray64<FVertex>& OutTransposeGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutTransposeGraph, EConvertToGraphOptions Options)
{
	int32 NumVertices = Graph.Num();

	TArray<TArray<FVertex>> TransposeGraph;
	TransposeGraph.SetNum(NumVertices);

	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		for (FVertex Edge : Graph[Vertex])
		{
			// Normalize Step 1: Remove edges to self
			if (Edge == Vertex)
			{
				continue;
			}

			TransposeGraph[Edge].Add(Vertex);
		}
	}

	// Finish normalizing; we have to do this before converting to the buffer format
	// because we need the duplicates removed to get an accurate count
	int32 NumTotalEdges = 0;
	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		TArray<FVertex>& Edges = TransposeGraph[Vertex];
		// Normalize Step 2: Sort edges
		Algo::Sort(Edges);
		// Normalize Step 3: Remove duplicates
		Edges.SetNum(Algo::Unique(Edges), EAllowShrinking::No);
		NumTotalEdges += Edges.Num();
	}

	// Convert the Array of Arrays into the output VertexBuffer and Array of Arrayviews
	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutTransposeGraphBuffer.Empty(NumTotalEdges);
		OutTransposeGraph.Empty(NumVertices);
	}
	else
	{
		OutTransposeGraphBuffer.Reset(NumTotalEdges);
		OutTransposeGraph.Reset(NumVertices);
	}

	FVertex* BufferData = OutTransposeGraphBuffer.GetData();
	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		TArray<FVertex>& Edges = TransposeGraph[Vertex];
		OutTransposeGraph.Emplace(BufferData + OutTransposeGraphBuffer.Num(), Edges.Num());
		OutTransposeGraphBuffer.Append(Edges);
	}
	check(OutTransposeGraphBuffer.Num() == NumTotalEdges);
	check(BufferData == OutTransposeGraphBuffer.GetData()); // We have TArrayViews into it
}

bool TryConstructCondensationGraph(TConstArrayView<TConstArrayView<FVertex>> Graph, TArray64<FVertex>& OutGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph, TArray64<FVertex>* OutOutVertexToInVerticesBuffer,
	TArray<TConstArrayView<FVertex>>* OutOutVertexToInVertices, TArray<FVertex>* OutInVertexToOutVertex,
	EConvertToGraphOptions Options)
{
	// Kosaraju-Sharir's algorithm - order the vertices from first out in a DFS search to last out. Any vertex that is
	// referenced by a referencer vertex that exited the dfs earlier is in a strongly connected component with the
	// referencer vertex.

	// Provide local buffers for any optional outputs that were not provided
	checkf((OutOutVertexToInVerticesBuffer != nullptr) == (OutOutVertexToInVertices != nullptr),
		TEXT("OutOutVertexToInVerticesBuffer and OutOutVertexToInVertices must either both be null or both be non-null."));
	TArray64<FVertex> LocalBufferOutVertexToInVerticesBuffer;
	TArray<TConstArrayView<FVertex>> LocalBufferOutVertexToInVertices;
	TArray<FVertex> LocalBufferInVertexToOutVertex;

	TArray64<FVertex>& OutVertsBuffer(OutOutVertexToInVerticesBuffer ?
		*OutOutVertexToInVerticesBuffer : LocalBufferOutVertexToInVerticesBuffer);
	TArray<TConstArrayView<FVertex>>& OutVertexToInVertices(OutOutVertexToInVertices ?
		*OutOutVertexToInVertices : LocalBufferOutVertexToInVertices);
	TArray<FVertex>& InVertexToOutVertex(OutInVertexToOutVertex ?
		*OutInVertexToOutVertex : LocalBufferInVertexToOutVertex);


	int32 NumVertices = Graph.Num();
	enum class EVisitStatus : uint8
	{
		NotVisited,
		InProgress,
		Visited,
	};
	struct FVisitData
	{
		FVertex Vertex;
		int32 NextEdge;
	};

	TArray<FVertex> DFSOutFirstToLast;
	bool bHasCycle = false;
	TArray<FVisitData> Stack;
	Stack.Reserve(NumVertices);
	int32 NumEdges = 0;

	// Create the DFSOutFirstToLast order. DepthFirstSearch the input graph and record each vertex after its edges.
	// Record whether we find any cycles during the search.
	{
		TArray<EVisitStatus> VisitStatus;
		Stack.Reset();
		VisitStatus.SetNumUninitialized(NumVertices);
		DFSOutFirstToLast.Reserve(NumVertices);

		for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
		{
			VisitStatus[Vertex] = EVisitStatus::NotVisited;
		}
		for (FVertex Root = 0; Root < NumVertices; ++Root)
		{
			if (VisitStatus[Root] != EVisitStatus::NotVisited)
			{
				continue;
			}
			VisitStatus[Root] = EVisitStatus::InProgress;
			Stack.Add({ Root, 0 });
			while (!Stack.IsEmpty())
			{
				FVisitData& VisitData = Stack.Last();
				FVertex Vertex = VisitData.Vertex;
				TConstArrayView<FVertex> Edges = Graph[Vertex];
				bool bPushed = false;
				while (VisitData.NextEdge < Edges.Num())
				{
					FVertex EdgeVertex = Edges[VisitData.NextEdge++];
					EVisitStatus& EdgeVisitStatus = VisitStatus[EdgeVertex];
					if (EdgeVisitStatus == EVisitStatus::Visited)
					{
					}
					else if (EdgeVisitStatus == EVisitStatus::NotVisited)
					{
						EdgeVisitStatus = EVisitStatus::InProgress;
						Stack.Add({ EdgeVertex, 0 });
						bPushed = true;
						break;
					}
					else // EVisitStatus::InProgress
					{
						// Don't count edges to self as a cycle, they are removed in normalization of the input graph
						if (EdgeVertex != Vertex)
						{
							bHasCycle = true;
						}
					}
				}
				if (!bPushed)
				{
					NumEdges += Edges.Num();
					DFSOutFirstToLast.Add(Vertex);
					VisitStatus[Vertex] = EVisitStatus::Visited;
					Stack.Pop();
				}
			}
		}
	}

	// Allocate the OutVertexToInVertices output values, using NumVertices as a conservative estimate of NumOutVertices
	bool bShrink = EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink);
	if (bHasCycle || OutOutVertexToInVertices || OutInVertexToOutVertex)
	{
		if (bShrink)
		{
			OutVertexToInVertices.Empty(NumVertices);
			OutVertsBuffer.Empty(NumVertices);
		}
		else
		{
			OutVertexToInVertices.Reset(NumVertices);
			OutVertsBuffer.Reset(NumVertices);
		}
	}

	// If we did not find any cycles, then the CondensationGraph is equal to the input Graph, so return false.
	if (!bHasCycle)
	{
		OutGraphBuffer.Reset();
		OutGraph.Reset();
		if (OutOutVertexToInVertices || OutInVertexToOutVertex)
		{
			if (bShrink)
			{
				InVertexToOutVertex.Empty(NumVertices);
			}
			else
			{
				InVertexToOutVertex.Reset(NumVertices);
			}

			// Copy reversed DFSOutFirstToLast into OutVertexToInVertices, and inverted into InVertexToOutVertex
			OutVertexToInVertices.SetNum(NumVertices, EAllowShrinking::No);
			OutVertsBuffer.SetNum(NumVertices, EAllowShrinking::No);
			InVertexToOutVertex.SetNum(NumVertices, EAllowShrinking::No);
			for (FVertex OutVertex = 0; OutVertex < NumVertices; ++OutVertex)
			{
				FVertex InVertex = DFSOutFirstToLast[NumVertices - 1 - OutVertex];
				OutVertsBuffer[OutVertex] = InVertex;
				OutVertexToInVertices[OutVertex] = TConstArrayView<FVertex>(OutVertsBuffer.GetData() + OutVertex, 1);
				InVertexToOutVertex[InVertex] = OutVertex;
			}
		}
		return false;
	}

	// Get the TransposeGraph
	TArray64<FVertex> ReferenceGraphBuffer;
	TArray<TConstArrayView<FVertex>> ReferenceGraph;
	ConstructTransposeGraph(Graph, ReferenceGraphBuffer, ReferenceGraph);

	TBitArray<> Assigned;

	// Iterate vertices from last DFSOut to first DFSOut. Each one that is not already assigned creates its own
	// strongly connected component. Any that are not involved in a cycle will have a strongly connected component
	// that consists of a single vertex. When we find a vertex that is not already assigned, visit all unassigned
	// vertices in its reference graph and assign them all to its component.
	{
		Stack.Reset();
		Assigned.Init(false, NumVertices);
		FVertex* OutVertsData = OutVertsBuffer.GetData();

		for (int32 IndexOfRoot = NumVertices - 1; IndexOfRoot >= 0; --IndexOfRoot)
		{
			FVertex Root = DFSOutFirstToLast[IndexOfRoot];
			if (Assigned[Root])
			{
				continue;
			}
			Assigned[Root] = true;
			int64 InitialOffset = OutVertsBuffer.Num();
			OutVertsBuffer.Add(Root);
			Stack.Add({ Root, 0 });
			while (!Stack.IsEmpty())
			{
				FVisitData& VisitData = Stack.Last();
				TConstArrayView<FVertex> Edges = ReferenceGraph[VisitData.Vertex];
				bool bPushed = false;
				while (VisitData.NextEdge < Edges.Num())
				{
					FVertex EdgeVertex = Edges[VisitData.NextEdge++];
					if (Assigned[EdgeVertex])
					{
					}
					else
					{
						Assigned[EdgeVertex] = true;
						OutVertsBuffer.Add(EdgeVertex);
						Stack.Add({ EdgeVertex, 0 });
						bPushed = true;
						break;
					}
				}
				if (!bPushed)
				{
					Stack.Pop();
				}
			}
			OutVertexToInVertices.Emplace(OutVertsData + InitialOffset, OutVertsBuffer.Num() - InitialOffset);
		}
		// We have TArrayViews into OutVertsBuffer; assert its data buffer did not change
		check(OutVertsBuffer.GetData() == OutVertsData);
	}
	int32 NumOutVertices = OutVertexToInVertices.Num();

	// Allocate the output variables that depend on NumOutVertices
	if (bShrink)
	{
		// Shrink OutVertexToInVertices and OutVertexToInVerticesBuffer if requested
		if (OutOutVertexToInVertices)
		{
			TArray64<FVertex> CopyBuffer = OutVertsBuffer;
			FVertex* OutVertsData = OutVertsBuffer.GetData();
			FVertex* CopyData = CopyBuffer.GetData();
			OutVertexToInVertices.Shrink();
			for (TConstArrayView<FVertex>& InVertices : OutVertexToInVertices)
			{
				InVertices = TConstArrayView<FVertex>(CopyData + (InVertices.GetData() - OutVertsData), InVertices.Num());
			}
			OutVertsBuffer = MoveTemp(CopyBuffer);
		}
		InVertexToOutVertex.Empty(NumVertices);
		OutGraph.Empty(NumOutVertices);
	}
	else
	{
		InVertexToOutVertex.Reset(NumVertices);
		OutGraph.Reset(NumOutVertices);
	}
	// We don't know NumEdges in the CondensationGraph yet; NumEdges from the original graph is a conservative estimate
	OutGraphBuffer.Reset(NumEdges);

	// Invert OutVertexToInVertices to create InVertexToOutVertex
	InVertexToOutVertex.SetNumUninitialized(NumVertices);
	for (FVertex OutVertex = 0; OutVertex < NumOutVertices; ++OutVertex)
	{
		for (FVertex InVertex : OutVertexToInVertices[OutVertex])
		{
			InVertexToOutVertex[InVertex] = OutVertex;
		}
	}

	// Create the edges of the condensationgraph by converting each source vertex edge vertex into the condensation
	// edge vertex and unioning all transformed edge vertices of all source vertices for the condensation root vertex
	{
		Assigned.SetRange(0, NumVertices, false);
		FVertex* OutGraphData = OutGraphBuffer.GetData();
		for (FVertex OutVertex = 0; OutVertex < NumOutVertices; ++OutVertex)
		{
			// Uniquely add all of the unioned edge vertices for this OutVertex onto a range in OutGraphBuffer
			int64 InitialOffset = OutGraphBuffer.Num();
			for (FVertex InVertex : OutVertexToInVertices[OutVertex])
			{
				for (FVertex SourceEdgeVertex : Graph[InVertex])
				{
					FVertex OutEdgeVertex = InVertexToOutVertex[SourceEdgeVertex];
					if (!Assigned[OutEdgeVertex]) // Note this condition also removes edges to self
					{
						Assigned[OutEdgeVertex] = true;
						OutGraphBuffer.Add(OutEdgeVertex);
					}
				}
			}
			// Record that range of edges in our OutGraph
			TConstArrayView<FVertex>& OutEdges = OutGraph.Emplace_GetRef(OutGraphData + InitialOffset,
				OutGraphBuffer.Num() - InitialOffset);

			// Clear the values in Assigned that we set; this is faster than clearing the whole array
			for (FVertex OutEdgeVertex : OutEdges)
			{
				Assigned[OutEdgeVertex] = false;
			}
		}
		// We have TArrayViews into OutVertexToInVertexBuffer; assert its data buffer did not change
		check(OutGraphBuffer.GetData() == OutGraphData);

		// Shrink OutGraphBuffer if requested
		if (bShrink)
		{
			TArray64<FVertex> CopyBuffer = OutGraphBuffer;
			FVertex* CopyData = CopyBuffer.GetData();
			for (TConstArrayView<FVertex>& Edges : OutGraph)
			{
				Edges = TConstArrayView<FVertex>(CopyData + (Edges.GetData() - OutGraphData), Edges.Num());
			}
			OutGraphBuffer = MoveTemp(CopyBuffer);
			check(OutGraphBuffer.GetData() == CopyData);
		}
	}
	return true;
}

void ConstructPartialTransposeGraph(TConstArrayView<TConstArrayView<FVertex>> Graph,
	TArrayView<FVertex> InVertices, int64 MaxOutGraphEdges,
	TArray64<FVertex>& OutTransposeGraphBuffer, TArray<TConstArrayView<FVertex>>& OutTransposeGraph,
	TArray<FVertex>& OutInVerticesInOutGraph)
{
	OutInVerticesInOutGraph.Reset(InVertices.Num());
	Algo::Sort(InVertices, [Graph](FVertex A, FVertex B)\
		{
			int32 NumA = Graph[A].Num();
			int32 NumB = Graph[B].Num();
			return NumA > NumB || (NumA == NumB && A < B);
		});

	int32 NumVertices = Graph.Num();
	TArray<TArray<FVertex>> TransposeGraph;
	TransposeGraph.SetNum(NumVertices);
	int32 NumEdges = 0;
	for (FVertex InVertex : InVertices)
	{
		NumEdges += Graph[InVertex].Num();
		if (NumEdges > MaxOutGraphEdges)
		{
			NumEdges -= Graph[InVertex].Num();
			break;
		}
		OutInVerticesInOutGraph.Add(InVertex);
		for (FVertex TargetVertex : Graph[InVertex])
		{
			TransposeGraph[TargetVertex].Add(InVertex);
		}
	}
	ConvertToSingleBufferGraph(TransposeGraph, OutTransposeGraphBuffer, OutTransposeGraph);
}

}

#if WITH_DEV_AUTOMATION_TESTS
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCondensationGraphTest, "System.Core.Algo.CondensationGraph", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FCondensationGraphTest::RunTest(const FString& Parameters)
{
	using namespace Algo::Graph;

	TArray<TArray<FVertex>> Buffer;
	TArray<TConstArrayView<FVertex>> Graph;
	TArray64<FVertex> CondensationBuffer;
	TArray<TConstArrayView<FVertex>> CondensationGraph;
	TArray64<FVertex> OutVertexToInVerticesBuffer;
	TArray<TConstArrayView<FVertex>> OutVertexToInVertices;
	TArray<FVertex> InVertexToOutVertex;
	TArray64<TArray<FVertex>> ExpectedOutVertexToInVertices;
	TArray<FVertex> ExpectedInVertexToOutVertex;
	TArray<FVertex> ActualInVertexToOutVertex;
	TArray<FVertex> ExpectedComponentVerticesScratch;
	TArray<FVertex> ActualComponentVerticesScratch;
	TArray<FVertex> ExpectedOutEdges;
	TArray<FVertex> ActualOutEdges;
	TArray<FVertex> Reachable;
	TBitArray<> Assigned;

	auto AddVertex = [&Buffer, &Graph](FVertex Vertex, TConstArrayView<FVertex> Edges)
	{
		if (Graph.Num() <= Vertex)
		{
			Graph.SetNum(Vertex + 1);
			Buffer.SetNum(Vertex + 1);
		}
		Buffer[Vertex] = Edges;
		Graph[Vertex] = Buffer[Vertex];
	};
	auto AddExpectedComponent = [&ExpectedOutVertexToInVertices](TConstArrayView<FVertex> InputVertices)
	{
		ExpectedOutVertexToInVertices.Emplace_GetRef().Append(InputVertices);
	};
	auto Clear = [&Buffer, &Graph, &ExpectedOutVertexToInVertices]()
	{
		Buffer.Reset();
		Graph.Reset();
		ExpectedOutVertexToInVertices.Reset();
	};
	auto WriteArrayToString = [](TConstArrayView<FVertex> A)
	{
		TStringBuilder<256> Writer;
		Writer << TEXT("[");
		Writer.Join(A, TEXT(','));
		Writer << TEXT("]");
		return FString(Writer);
	};
	auto ConfirmResults =
		[this, &Graph, &CondensationBuffer, &CondensationGraph, &OutVertexToInVerticesBuffer, &OutVertexToInVertices,
		&InVertexToOutVertex , &ExpectedOutVertexToInVertices, &ExpectedInVertexToOutVertex, &Assigned,
		&ActualInVertexToOutVertex, &ExpectedComponentVerticesScratch, &ActualComponentVerticesScratch,
		&WriteArrayToString, &ExpectedOutEdges, &ActualOutEdges, &Reachable]
		(const TCHAR* TestCaseName)
	{
		bool bResult = TryConstructCondensationGraph(Graph, CondensationBuffer, CondensationGraph,
			&OutVertexToInVerticesBuffer, &OutVertexToInVertices, &InVertexToOutVertex, EConvertToGraphOptions::Shrink);
		bool bExpectedResult = !ExpectedOutVertexToInVertices.IsEmpty();
		if (bResult != bExpectedResult)
		{
			AddError(FString::Printf(TEXT("Failed for case \"%s\": expected ConstructCondensationGraph to return %s but it returned %s."),
				TestCaseName, bExpectedResult ? TEXT("true") : TEXT("false"), bResult ? TEXT("true") : TEXT("false")));
			return;
		}

		int32 NumVertices = Graph.Num();
		int32 NumOutVertices = CondensationGraph.Num();
		Assigned.SetNumUninitialized(NumVertices);

		// Verify InVertexToOutVertex and OutVertexToInVertices match, cover all vertices, and do not put the same
		// invertex into two outvertices
		ActualInVertexToOutVertex.SetNumUninitialized(NumVertices, EAllowShrinking::No);
		Assigned.SetRange(0, NumVertices, false);
		int32 ComponentIndex = 0;
		for (TConstArrayView<FVertex> ActualComponentVertices : OutVertexToInVertices)
		{
			for (FVertex InVertex : ActualComponentVertices)
			{
				if (Assigned[InVertex])
				{
					AddError(FString::Printf(TEXT("Failed for case \"%s\": OutVertexToInVertices has the same invertex %d in both outvertex %d and outvertex %d."),
						TestCaseName, (int32)InVertex, (int32)ActualInVertexToOutVertex[InVertex], (int32)ComponentIndex));
					return;
				}
				Assigned[InVertex] = true;
				ActualInVertexToOutVertex[InVertex] = ComponentIndex;
			}
			++ComponentIndex;
		}
		for (FVertex InVertex = 0; InVertex < NumVertices; ++InVertex)
		{
			if (!Assigned[InVertex])
			{
				AddError(FString::Printf(TEXT("Failed for case \"%s\": OutVertexToInVertices is missing invertex %d; it did not assign that vertex to any outvertex."),
					TestCaseName, (int32) InVertex));
				return;
			}
			if (ActualInVertexToOutVertex[InVertex] != InVertexToOutVertex[InVertex])
			{
				AddError(FString::Printf(TEXT("Failed for case \"%s\": OutVertexToInVertices has outvertex %d <- invertex %d, but InVertexToOutVertex has invertex %d -> outvertex %d."),
					TestCaseName, (int32)ActualInVertexToOutVertex[InVertex], (int32)InVertex, (int32)InVertex,
					(int32)InVertexToOutVertex[InVertex]));
			}
		}

		// Finish the construction of ExpectedOutVertexToInVertices; it has the contract that any unmentioned vertex is in a
		// component by itself. Also construct its inverse - ExpectedInVertexToOutVertex.
		ExpectedInVertexToOutVertex.SetNumUninitialized(NumVertices, EAllowShrinking::No);
		Assigned.SetRange(0, NumVertices, false);
		ComponentIndex = 0;
		for (TArray<FVertex>& ExpectedComponentVertices : ExpectedOutVertexToInVertices)
		{
			for (FVertex InVertex : ExpectedComponentVertices)
			{
				Assigned[InVertex] = true;
				ExpectedInVertexToOutVertex[InVertex] = ComponentIndex;
			}
			++ComponentIndex;
		}
		for (FVertex InVertex = 0; InVertex < NumVertices; ++InVertex)
		{
			if (!Assigned[InVertex])
			{
				ExpectedInVertexToOutVertex[InVertex] = ComponentIndex++;
				ExpectedOutVertexToInVertices.Add({ InVertex });
			}
		}

		// Verify that the actual components place every InVertex in a component with the same list of Inputvertices
		// as the expected components
		Assigned.SetRange(0, NumVertices, false);
		for (FVertex InVertex = 0; InVertex < NumVertices; ++InVertex)
		{
			if (Assigned[InVertex])
			{
				continue;
			}
			FVertex ExpectedOutVertex = ExpectedInVertexToOutVertex[InVertex];
			FVertex ActualOutVertex = ActualInVertexToOutVertex[InVertex];
			ExpectedComponentVerticesScratch.Reset();
			ExpectedComponentVerticesScratch.Append(ExpectedOutVertexToInVertices[ExpectedOutVertex]);
			ActualComponentVerticesScratch.Reset();
			ActualComponentVerticesScratch.Append(OutVertexToInVertices[ActualOutVertex]);
			Algo::Sort(ExpectedComponentVerticesScratch);
			Algo::Sort(ActualComponentVerticesScratch);
			if (!Algo::Compare(ExpectedComponentVerticesScratch, ActualComponentVerticesScratch))
			{
				AddError(FString::Printf(TEXT("Failed for case \"%s\": Input vertex %d was placed into a component with invertices %s, but it was expected in a component with invertices %s."),
					TestCaseName, (int32)InVertex, *WriteArrayToString(ActualComponentVerticesScratch),
					*WriteArrayToString(ExpectedComponentVerticesScratch)));
				return;
			}
		}

		// Verify that every edge in the input graph is a component edge in the output graph, or is inside the component
		if (bResult)
		{
			for (FVertex InSourceVertex = 0; InSourceVertex < NumVertices; ++InSourceVertex)
			{
				FVertex OutSourceVertex = ActualInVertexToOutVertex[InSourceVertex];
				for (FVertex InEdgeVertex : Graph[InSourceVertex])
				{
					FVertex ExpectedOutEdgeVertex = ActualInVertexToOutVertex[InEdgeVertex];
					if (ExpectedOutEdgeVertex == OutSourceVertex)
					{
						continue;
					}
					if (!CondensationGraph[OutSourceVertex].Contains(ExpectedOutEdgeVertex))
					{
						AddError(FString::Printf(TEXT("Failed for case \"%s\": Missing edge in the condensation graph: ")
							TEXT("expected edge from vertex %d in component %d to vertex %d in component %d, but this edge is missing."),
							TestCaseName, (int32)InSourceVertex, (int32)OutSourceVertex, (int32)InEdgeVertex,
							(int32)ExpectedOutEdgeVertex));
						return;
					}
				}
			}

			// Verify that no component edge exists in the output graph unless there is a witness edge for it in the input graph
			// Create the expected edges of the condensationgraph by converting each source vertex edge vertex into the condensation
			// edge vertex and unioning all transformed edge vertices of all source vertices for the condensation root vertex
			for (FVertex OutVertex = 0; OutVertex < NumOutVertices; ++OutVertex)
			{
				Assigned.SetRange(0, NumOutVertices, false);
				ExpectedOutEdges.Reset();

				for (FVertex InVertex : OutVertexToInVertices[OutVertex])
				{
					for (FVertex SourceEdgeVertex : Graph[InVertex])
					{
						FVertex OutEdgeVertex = ActualInVertexToOutVertex[SourceEdgeVertex];
						if (!Assigned[OutEdgeVertex])
						{
							Assigned[OutEdgeVertex] = true;
							ExpectedOutEdges.Add(OutEdgeVertex);
						}
					}
				}

				ActualOutEdges.Reset();
				ActualOutEdges.Append(CondensationGraph[OutVertex]);
				Algo::Sort(ExpectedOutEdges);
				Algo::Sort(ActualOutEdges);
				if (!Algo::Compare(ActualOutEdges, ExpectedOutEdges))
				{
					AddError(FString::Printf(TEXT("Failed for case \"%s\": Expected component %d to have edges %s, but it had %s."),
						TestCaseName, (int32)OutVertex, *WriteArrayToString(ActualOutEdges),
						*WriteArrayToString(ExpectedOutEdges)));
					return;
				}
			}

			// Verify the output is sorted from root to leaf
			for (FVertex Root = 0; Root < NumOutVertices; ++Root)
			{
				Assigned.SetRange(0, NumOutVertices, false);
				Reachable.Reset(NumOutVertices);
				Assigned[Root] = true;
				Reachable.Add(Root);
				int32 NextIndex = 0;
				while (NextIndex < Reachable.Num())
				{
					FVertex Vertex = Reachable.Pop(EAllowShrinking::No);
					for (FVertex EdgeVertex : CondensationGraph[Vertex])
					{
						if (!Assigned[EdgeVertex])
						{
							Assigned[EdgeVertex] = true;
							Reachable.Add(EdgeVertex);
						}
					}
				}
				for (FVertex ReachableVertex : Reachable) //-V1078
				{
					if (ReachableVertex < Root)
					{
						AddError(FString::Printf(TEXT("Failed for case \"%s\": Expected outgraph to be topologically sorted from root to leaf, but vertex %d is reachable from vertex %d."),
							TestCaseName, (int32)ReachableVertex, (int32)Root));
						return;
					}
				}
			}
		}
		else // !if (bResult)
		{
			// Should have cleared the graph arguments
			if (!CondensationGraph.IsEmpty() || !CondensationBuffer.IsEmpty())
			{
				AddError(FString::Printf(TEXT("Failed for case \"%s\": Expected outgraph to be cleared since result was false, but it is non-empty."),
					TestCaseName));
				return;
			}
		}
	};

	{
		Clear();
		AddVertex(0, { });
		AddVertex(1, { 0 });
		AddVertex(2, { 1 });
		ConfirmResults(TEXT("Each node depends on the previous one"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 2 });
		AddVertex(2, { });
		ConfirmResults(TEXT("Each node depends on the next one"));
	}
	{
		Clear();
		AddVertex(0, { 0 });
		AddVertex(1, { 0,1 });
		AddVertex(2, { 1,2 });
		ConfirmResults(TEXT("SelfReferences"));
	}
	{
		//              6
		//             / \
		//            5   7
		//           / \   \
		//          0   1   8
		//           \ / \   \
		//            3   4   |
		//             \   \ /
		//              \   9
		//               \ /
		//                2       
		Clear();
		AddVertex(6, { 5,7 });
		AddVertex(5, { 0,1 });
		AddVertex(7, { 8 });
		AddVertex(0, { 3 });
		AddVertex(1, { 3,4 });
		AddVertex(8, { 9 });
		AddVertex(3, { 2 });
		AddVertex(4, { 9 });
		AddVertex(9, { 2 });
		AddVertex(2, { });
		ConfirmResults(TEXT("SketchedOutExample1"));
	}
	{
		Clear();
		AddVertex(0, { 0,1 });
		AddVertex(1, { 0,1 });
		AddExpectedComponent({ 0, 1 });
		ConfirmResults(TEXT("Simple cycle"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 0,1,2 });
		AddVertex(2, { 2,0 });
		AddExpectedComponent({ 0, 1, 2 });
		ConfirmResults(TEXT("Short cycle in a long cycle"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 0,2 });
		AddVertex(2, { 3 });
		AddVertex(3, { });
		AddExpectedComponent({ 0, 1 });
		ConfirmResults(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts"));
	}
	{
		Clear();
		AddVertex(0, { });
		AddVertex(1, { 0 });
		AddVertex(2, { 1,3 });
		AddVertex(3, { 2 });
		AddExpectedComponent({ 2, 3 });
		ConfirmResults(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 2 });
		AddVertex(2, { 3 });
		AddVertex(3, { 2 });
		AddExpectedComponent({ 2,3 });
		ConfirmResults(TEXT("Cycle at a leaf and a chain from the root depending on that cycle"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 0 });
		AddVertex(2, { 1 });
		AddVertex(3, { 2 });
		AddExpectedComponent({ 0, 1 });
		ConfirmResults(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts"));
	}
	{
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 2,3 });
		AddVertex(2, { 1,3 });
		AddVertex(3, { 1,2 });
		AddExpectedComponent({ 1,2,3 });
		ConfirmResults(TEXT("Vertex dependent upon a cycle"));
	}
	{
		// 0 -> 1 -> 2 -> 3 -> 1
		//           |
		//           v
		//      4 -> 5 -> 6 -> 4
		Clear();
		AddVertex(0, { 1 });
		AddVertex(1, { 2 });
		AddVertex(2, { 3,5 });
		AddVertex(3, { 1 });
		AddVertex(4, { 5 });
		AddVertex(5, { 6 });
		AddVertex(6, { 4 });
		AddExpectedComponent({ 1,2,3 });
		AddExpectedComponent({ 4,5,6 });
		ConfirmResults(TEXT("One cycle dependent upon another"));
	}
	{
		// 5 -> 0 -> (1 -> 2 -> 1)
		//      |          |
		//      |          v
		//      |          5
		//      v
		//     (3 -> 4 -> 3)
		Clear();
		AddVertex(0, { 1,3 });
		AddVertex(1, { 2 });
		AddVertex(2, { 1,5 });
		AddVertex(3, { 4 });
		AddVertex(4, { 3 });
		AddVertex(5, { 0 });
		AddExpectedComponent({ 0,1,2,5 });
		AddExpectedComponent({ 3,4 });
		ConfirmResults(TEXT("MutuallyReachableSet Problem1"));
	}
	{
		Clear();
		TArray<FVertex> AllButItself;
		TArray<FVertex> All;
		for (FVertex Vertex = 0; Vertex < 6; ++Vertex)
		{
			AllButItself.Add(Vertex);
			All.Add(Vertex);
		}
		for (FVertex Vertex = 0; Vertex < 6; ++Vertex)
		{
			AllButItself.Remove(Vertex);
			AddVertex(Vertex, AllButItself);
			AllButItself.Add(Vertex);
		}
		AddExpectedComponent(All);
		ConfirmResults(TEXT("FullyConnectedGraph"));
	}
	{
		Clear();
		// 0 - 1 - 2 - 0
		// |
		// 3 - 4 - 5 - 3
		//         |
		// 6 - 7 - 8 - 6
		//     |
		//     1
		AddVertex(0, { 1,3 });
		AddVertex(1, { 2 });
		AddVertex(2, { 0 });
		AddVertex(3, { 4 });
		AddVertex(4, { 5 });
		AddVertex(5, { 3,8 });
		AddVertex(6, { 7 });
		AddVertex(7, { 1,8 });
		AddVertex(8, { 6 });
		AddExpectedComponent({ 0,1,2,3,4,5,6,7,8 });
		ConfirmResults(TEXT("Cycle of cycles"));
	}
	{
		Clear();
		// 0 ----- 1 ----- 2 ----- 0
		// |       |       |
		// |       0       |
		// |               |
		// 5 - 4 - 3 - 5   6 - 7 - 6
		//     |\              |
		//     | 5             |
		//     |               |
		//     8 - 9 - 10 - 9  11
		//     |   |   |
		//     10  8   8
		AddVertex(0, { 1,5 });
		AddVertex(1, { 0,2 });
		AddVertex(2, { 0,6 });
		AddVertex(3, { 5 });
		AddVertex(4, { 3,5,8 });
		AddVertex(5, { 4 });
		AddVertex(6, { 7 });
		AddVertex(7, { 6,11 });
		AddVertex(8, { 9,10 });
		AddVertex(9, { 8,10 });
		AddVertex(10, { 8,9 });
		AddVertex(11, { });
		AddExpectedComponent({ 0,1,2 });
		AddExpectedComponent({ 3,4,5 });
		AddExpectedComponent({ 6,7 });
		AddExpectedComponent({ 8,9,10 });
		ConfirmResults(TEXT("Tree of cycles"));
	}

	return true;
}

#endif
