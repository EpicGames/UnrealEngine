// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Invoke.h"
#include "Traits/ElementType.h"

#include <utility>

namespace Algo::Graph
{

typedef int32 FVertex;
inline constexpr FVertex InvalidVertex = static_cast<FVertex>(INDEX_NONE);

enum class EConvertToGraphOptions
{
	None = 0,
	/** The output buffer will be shrunk to reduce memory to minimum required. */
	Shrink = 1 << 0,
};
ENUM_CLASS_FLAGS(EConvertToGraphOptions);

/**
 * Convert an array of Keys and a function that returns the directed edges from each Key into the Normalized Graph
 * Form. Normalized Graph Form is an implicit set of vertices identified as integers from 0 to N - 1, and an array of
 * arrayviews of edges from each vertex. The Normalized Graph form has two structures: The GraphBuffer which contains
 * the memory storage for the Graph and the Graph itself which is the array of arrayviews of edges. The length of the
 * array in the graph defines the number of vertices.
 * 
 * @param UniqueKeys A range with element type KeyType. KeyType must support GetTypeHash and copy+move constructors.
 *        KeyType being pointertype is recommended. Keys in the range must be unique. The ith element of the range
 *        will correspond to as vertex i in the OutGraph. UniqueKeys must have length < MAX_int32.
 * @param GetKeyEdges A callable with prototype that is one of
 *            RangeType<KeyType> GetKeyEdges(const KeyType& Key);
 *            const RangeType<KeyType>& GetKeyEdges(const KeyType& Key);
 *        It must return the TargetKeys that are pointed to from the directed edges from Key. TargetKeys that are 
 *        not elements of UniqueKeys will be discarded. RangeType must support ranged-for (begin() and end()).
 * @param OutGraphBuffer Output value that holds the memory used by the graph. The array must remain allocated and
 *        unmodified until OutGraph is no longer referenced.
 * @param OutGraph Output value that holds the graph. OutGraph[i] is a TConstArrayView of edges from Vertex i, which
 *        corresponds to UniqueKeys[i]. Each element of OutGraph[i] is a an FVertex which when cast to an integer is
 *        the index into UniqueKeys that corresponds to the vertex. Elements of OutGraph[i] are sorted by FVertex
 *        value, aka the index into UniqueKeys.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
template <typename RangeType, typename GetKeyEdgesType>
inline void ConvertToGraph(const RangeType& UniqueKeys, GetKeyEdgesType GetKeyEdges,
	TArray64<FVertex>& OutGraphBuffer, TArray<TConstArrayView<FVertex>>& OutGraph,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	using KeyType = TElementType_T<RangeType>;

	// Map Elements to vertices in VertexOfKey
	int32 NumVertices = GetNum(UniqueKeys);
	TMap<KeyType, FVertex> VertexOfKey;
	VertexOfKey.Reserve(NumVertices);
	FVertex Vertex = 0;
	for (const KeyType& Key : UniqueKeys)
	{
		FVertex& ExistingHandle = VertexOfKey.FindOrAdd(Key, InvalidVertex);
		check(ExistingHandle == InvalidVertex);
		if (ExistingHandle == InvalidVertex)
		{
			ExistingHandle = Vertex++;
		}
	}

	TArray<TPair<int64, int32>> EdgeOffsets;
	EdgeOffsets.Reserve(NumVertices);
	OutGraphBuffer.Reset();

	// Call GetKeyEdges on each Element, map its returnvalue to Vertices, normalize them, and store them in EdgeOffsets
	Vertex = 0;
	for (const KeyType& Element : UniqueKeys)
	{
		int64 InitialOffset = OutGraphBuffer.Num();
		for (const KeyType& Dependency : Invoke(GetKeyEdges, Element))
		{
			FVertex* TargetVertex = VertexOfKey.Find(Dependency);
			if (!TargetVertex)
			{
				continue;
			}

			// Normalize Step 1: remove edges to self
			if (*TargetVertex == Vertex)
			{
				continue;
			}
			OutGraphBuffer.Add(*TargetVertex);
		}

		// Normalize Step 2: Sort
		TArrayView<FVertex> VertexEdges = TArrayView<FVertex>(OutGraphBuffer).Mid(InitialOffset);
		Algo::Sort(VertexEdges);

		// Normalize Step 3: Remove duplicates
		VertexEdges = VertexEdges.Left(Algo::Unique(VertexEdges));
		OutGraphBuffer.SetNum(InitialOffset + VertexEdges.Num(), EAllowShrinking::No);

		// Store the vertex's offset into GraphBuffer, for later conversion to TArrayView
		EdgeOffsets.Emplace(InitialOffset, VertexEdges.Num());
		++Vertex;
	}
	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutGraphBuffer.Shrink();
		OutGraph.Empty(NumVertices);
	}
	else
	{
		OutGraph.Reset(NumVertices);
	}

	// Convert offsets to TArrayView
	for (const TPair<int64, int32>& EdgeOffset : EdgeOffsets)
	{
		OutGraph.Add(TConstArrayView<FVertex>(OutGraphBuffer.GetData() + EdgeOffset.Key, EdgeOffset.Value));
	}
}

/**
 * Convert an array of separately allocated edge ranges into a single buffer shared by all vertices and an edge graph
 * of arrayviews into that buffer. No overlapping is done; every input edge range is reproduced in the output buffer.
 * 
 * @param Graph ArrayView of N vertices; each element is interpreted by the ProjectionType to return a range of edges
 *        for that vertex.
 * @param OutGraphBuffer Output value that holds the memory used by the graph. This array must remain allocated and
 *        unmodified until OutGraph is no longer referenced.
 * @param OutGraph Output value that holds the graph. @see ConvertToGraph for a description of the format.
 * @param Proj Returns the edges for Graph[n]. The prototype is
 *            // Input is either const& or by value. Output can be const&, &&, or by value.
 *            EdgeRangeType Proj(MultiBufferRangeType VertexEdges)
 *        EdgeRangeType must support GetData and GetNum.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
template <typename RangeType, typename ProjectionType>
inline void ConvertToSingleBufferGraph(RangeType&& Graph, TArray64<FVertex>& OutGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph, ProjectionType Proj,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	typedef TElementType_T<RangeType> InEdgeRangeType;
	typedef decltype(Proj(std::declval<const InEdgeRangeType&>())) ProjectedEdgeRangeType;

	int32 NumVertices = Graph.Num();
	int64 NumEdges = 0;
	for (const InEdgeRangeType& VertexEdges : Graph)
	{
		NumEdges += GetNum(Proj(VertexEdges));
	}

	if (EnumHasAnyFlags(Options, EConvertToGraphOptions::Shrink))
	{
		OutGraphBuffer.Empty(NumEdges);
		OutGraph.Empty(NumVertices);
	}
	else
	{
		OutGraphBuffer.Reset(NumEdges);
		OutGraph.Reset(NumVertices);
	}

	FVertex* OutBufferData = OutGraphBuffer.GetData();
	for (const InEdgeRangeType& InVertexEdges : Graph)
	{
		ProjectedEdgeRangeType VertexEdges = Proj(InVertexEdges);
		int32 NumVertexEdges = GetNum(VertexEdges);
		OutGraph.Emplace(OutBufferData + OutGraphBuffer.Num(), NumVertexEdges);
		OutGraphBuffer.Append(GetData(VertexEdges), NumVertexEdges);
	}
	check(OutGraphBuffer.GetData() == OutBufferData); // We have arrayviews into OutBufferData
}

/**
 * Convert an array of separately allocated edge ranges into a single buffer shared by all vertices and an edge graph
 * of arrayviews into that buffer. No overlapping is done; every input edge range is reproduced in the output buffer.
 * This version requires MultiBufferRange type to be a range type; see other prototype to provide a Projection.
 * 
 * @param Graph ArrayView of N vertices; each element is interpreted by the ProjectionType to return a range of edges
 *        for that vertex.
 * @param OutBuffer Output value that holds the memory used by the graph. This array must remain allocated and
 *        unmodified until OutGraph is no longer referenced.
 * @param OutGraph Output value that holds the graph. @see ConvertToGraph for a description of the format.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
template <typename RangeType>
inline void ConvertToSingleBufferGraph(RangeType Graph, TArray64<FVertex>& OutBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph, EConvertToGraphOptions Options = EConvertToGraphOptions::None)
{
	ConvertToSingleBufferGraph(Graph, OutBuffer, OutGraph, FIdentityFunctor(), Options);
}

/**
 * Return a new buffer and graph with the same vertices but with each edge reversed. If and only if (i,j) is an edge
 * in the input Graph, (j,i) is an edge in the output TransposeGraph.
 * 
 * @param Graph A standard GraphConvert-format Graph. @see ConvertToGraph for a description of the format.
 * @param OutTransposeGraphBuffer Output value that holds the memory used by the graph. This array must remain
 *        allocated and unmodified until OutTransposeGraph is no longer referenced.
 * @param OutTransposeGraph Output value that holds the graph. @see ConvertToGraph for a description of the format.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 */
void ConstructTransposeGraph(TConstArrayView<TConstArrayView<FVertex>> Graph,
	TArray64<FVertex>& OutTransposeGraphBuffer, TArray<TConstArrayView<FVertex>>& OutTransposeGraph,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None);

/**
 * Return a new graph with cycles in the input graph replaced by a single vertex. The new graph is topologically sorted
 * from root to leaf. If the input graph has no cycles, the function returns false and the OutGraph is reset to empty.
 *
 * @param Graph The input graph, may contain cycles. @see ConvertToGraph for a description of the format.
 * @param OutGraphBuffer Output value that holds the memory used by OutGraph. The memory held by the array must remain
 *        allocated and unmodified until OutGraph is no longer referenced.
 * @param OutGraph The edges of the condensed graph, in a standard GraphConvert-format Graph. @see ConvertToGraph for a
 *        description of the format. It has no cycles and the vertices are topologically sorted from root to leaf.
 * @param OutOutVertexToInVerticesBuffer Output value, optional, can be null. If null, OutOutVertexToInVertices must
 *        also be null. If non-null, it is populated with the memory used by *OutOutVertexToInVertices. It must remain
 *        remain allocated and unmodified until *OutOutVertexToSourceVertex is no longer referenced. This out value
 *        is set even if the function returns false.
 * @param OutOutVertexToInVertices Output value, optional, can be null. If null, OutOutVertexToInVerticesBuffer must
 *        also be null. If non-null, it is populated with an array of arrayviews. The length of the array is equal to
 *        the length of OutGraph, which is the number of vertices in the condensed graph. THe nth element of the array
 *        contains the list of (not necessarily sorted) vertices from the input graph that are contained within vertex
 *        n of the condensed graph. Vertices in the outgraph that correspond to an input vertex that was not in a cycle
 *        will have an array of only a single element. This out value is set even if the function returns false.
 * @param OutInVertexToOutVertex Output value, optional, can be null. If non-null, it is populated with an array. The
 *        length of the array is the number of vertices in the input graph. The nth element of the array is the vertex of
 *        the condensed graph that contains the nth vertex of the input graph. This out value is set even if the function
 *        returns false.
 * @param Options for the conversion, @see EConvertToGraphOptions.
 *
 * @return True if the input graph had and cycles, false otherwise. If false is returned, OutGraphBuffer and OutGraph
 *         are reset to empty, but the other output variables are still populated if present.
 */
bool TryConstructCondensationGraph(TConstArrayView<TConstArrayView<FVertex>> Graph,
	TArray64<FVertex>& OutGraphBuffer, TArray<TConstArrayView<FVertex>>& OutGraph,
	TArray64<FVertex>* OutOutVertexToInVerticesBuffer, TArray<TConstArrayView<FVertex>>* OutOutVertexToInVertices,
	TArray<FVertex>* OutInVertexToOutVertex,
	EConvertToGraphOptions Options = EConvertToGraphOptions::None);

/**
 * Construct the TransposeGraph (see @ConstructTransposeGraph), but only include edges in the original graph
 * from the given vertices (in the transpose graph, edges will be present to the given vertices).
 * Additionally, limit the number of edges in the returned graph, and report which InVertices were added.
 * This is used to create a partial ReachedBy graph from a large Reachability graph.
 * 
 * @param InVertices Vertices in the input graph, edges from which will be included in the output.
 *        This list will be modified: vertices will be sorted from largest number of edges to smallest.
 */
void ConstructPartialTransposeGraph(TConstArrayView<TConstArrayView<FVertex>> Graph,
	TArrayView<FVertex> InVertices, int64 MaxOutGraphEdges,
	TArray64<FVertex>& OutTransposeGraphBuffer, TArray<TConstArrayView<FVertex>>& OutTransposeGraph,
	TArray<FVertex>& OutInVerticesInOutGraph);

}