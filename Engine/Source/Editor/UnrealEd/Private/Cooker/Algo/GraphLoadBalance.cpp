// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphLoadBalance.h"

#include "Algo/MinElement.h"
#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "GraphReachability.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopeExit.h"
#include "Packing.h"
#include "SortSpecialCases.h"

namespace Algo::Graph
{
/** Implements ConstructLoadBalance. */
class FLoadBalanceBuilder
{
public:
	FLoadBalanceBuilder(TConstArrayView<TConstArrayView<FVertex>> InGraph, int32 InNumBuckets,
		TConstArrayView<TConstArrayView<FVertex>> InTransposeGraph,
		TConstArrayView<TConstArrayView<FVertex>> InReachabilityGraph,
		TArray<TArray<FVertex>>& OutAssignments, TArray<TArray<FVertex>>& OutRootAssignments);

	void LoadBalance();

private:
	/**
	 * Roots and vertices in a cluster of vertices, which is a transitively-closed set of vertices that should be
	 * assigned to a bucket as a group.
	 * 
	 * Initially every cluster is one of the disjoint subgraphs of the ingraph, but we might need to split some of
	 * those clusters to balance the buckets.
	 * 
	 * Also holds intermediate data used by PackClusters.
	 */
	struct FCluster
	{
		TArray<Algo::Graph::FVertex> Roots;
		TArray<Algo::Graph::FVertex> Vertices;
		int32 SplitGroup = INDEX_NONE;
	};

	/** An output bucket of vertices, represented as a collection of clusters. */
	struct FBucket
	{
		TArray<int32> ClusterIndices;
		int32 Size = 0;

		void CalculateSize(TConstArrayView<FCluster> Clusters);
	};

	/** Helper enum for SplitCluster. Types of estimations for the results of merging a root into a cluster. */
	enum class EEstimationType
	{
		Exact,
		TightUpperBound,
		LooseUpperBoundUniformIncrease,
	};
	/**
	 * Helper struct for SplitCluster. Records metrics about the results of a merge between a root's reachable vertices
	 * and a cluster's accumulated vertices. Calculating the merge is too expensive, and even updating estimates for
	 * the merge results for every root after every decision is too expensive, so this struct supports multiple levels
	 * of estimation.
	 */
	struct FRootAndClusterMergeData
	{
		/** The root being merged. Note the cluster being merged is implicit - the owner of this mergedata. */
		FVertex Root = Algo::Graph::InvalidVertex;
		/**
		 * UpperBound for how many vertices will reduce when merging this cluster. 
		 * The definition of a vertex reducing is that it exists in both the cluster and the root and will therefore
		 * not cause an increase in cluster size when the two are merged.
		 * This field is an exact value rather than an estimate if bExact is true. It can be modified upwards based
		 * on changes to the size of the cluster.
		 */
		int32 ReductionUpperBound = 0;
		/** Number of vertices in the root being merged. Copied here for spatial locality of the data when sorting. */
		int32 RootSize = 0;
		/** The size the cluster had the last time we estimated or modified ReductionUpperBound. */
		int32 ClusterSizeWhenEstimated = 0;
		/**
		 * The value of ReductionUpperBound the last time we calculated reduction exactly. Used in combination with
		 * the cluster's RootReductions list to recalculate the exact reduction without needing to compare vertices.
		 */
		int32 PreviousExactEstimate = 0;
		/** Whether ReductionUpperBound is known to be exact or is an upperbound estimate. */
		bool bExact = false;

		/**
		 * Return whether A is a worse merge (e.g. lower reduction value) than B. We sort merges from worst to best
		 * so we can pop the best merge off the back of a TArray without any shifts.
		 */
		static bool IsWorse(FRootAndClusterMergeData& A, FRootAndClusterMergeData& B);
	};

	/** Helper struct for SplitCluster. Data about one of the clusters we are building up by merging root vertices. */
	struct FMergeCluster
	{
		/** Roots that have been merged into the cluster. */
		TArray<Algo::Graph::FVertex> Roots;
		/** NumVertices-length bitarray specifying which vertices are in the cluster. */
		TBitArray<> VertexInCluster;
		/**
		 * NumVertices-length integer array specifying for each root vertex how many vertices reachable from that root
		 * have been added onto VertexInCluster since the last time RootReductions were consumed. This allows us to
		 * recalculate the number of reductions that will occur when merging the root into the cluster.
		 */
		TArray<int32> RootReductions;
		/**
		 * An array of the mergedatas for merging every remaining root into the cluster. 
		 * The array is sorted from worst to best estimated merge results. After each new root is committed we pop the
		 * root's mergedata off the end, and do some work to update the estimates for the remaining roots and restore
		 * the sortedness of the array.
		 */
		TArray<FRootAndClusterMergeData> MergeDatas;

		/** Number of true bits in VertexInCluster, aka the number of vertices in the cluster. */
		int32 VertexInClusterCount = 0;
	};

	/**
	 * Helper struct for SplitCluster. Metrics about the best merge for each cluster. We sort a list of these structs
	 * to find the best merge across all clusters.
	 */
	struct FClusterMergeSelectionData
	{
		/** The root being merged */
		FVertex Root;
		/* The cluster being merged */
		FMergeCluster* Cluster;
		/** The exact reduction value - @see FRootAndClusterMergeData for definition of reduction. */
		int32 Reduction;
		/** The post-merge size of the cluster. */
		int32 NewSize;
		/** The spread between the new max-sized cluster and the new min-sized cluster if this merge is selected. */
		int32 Spread;
		/** True iff the new spread is over the maximum allowed and the importance of the overage is not suppressed. */
		bool bOverSpread;

		/** Return whether A is a better merge to select than B */
		static bool IsBetter(const FClusterMergeSelectionData& A, const FClusterMergeSelectionData& B);
	};

	void FindDisjointSubgraphs(TArray<FCluster>& OutDisjointSubgraphs);
	void FindRoots(TArrayView<FCluster> InOutDisjointSubgraphs);
	void PackClusters(TArray<FCluster>&& Clusters, TArray<FCluster>& OutBuckets);
	/** Merge the clusters listed in each bucket into a single merged cluster for the bucket. */
	void CreateBucketClusters(TConstArrayView<FBucket> Buckets, TConstArrayView<FCluster> Clusters,
		TArray<FCluster>& OutBuckets);
	TArray<FLoadBalanceBuilder::FCluster> SplitCluster(FCluster&& InCluster, int32 SplitSize, int32 DesiredSpread);
	/** After growing the cluster, update estimates in the cluster's sorted list of MergeDatas and resort the list. */
	void UpdateClusterMergeDatas(FMergeCluster& Cluster, TBitArray<>& RootAssigned);
	/**
	 * Merges the vertices of the given cluster and root, optionally stores the merged results in cluster, and returns the
	 * number of vertices from root that reduced (were already present in cluster).
	 */
	int32 CalculateMergeResults(FMergeCluster& Cluster, FVertex Root, bool bWriteResultsToCluster);
	/**
	 * Update the estimate for the given mergedata in the given cluster, using the given estimatetype, after growing
	 * the cluster. Report whether the update changed the ReductionUpperBound.
	 */
	void UpdateMergeDataEstimate(FMergeCluster& Cluster, FRootAndClusterMergeData& MergeData,
		EEstimationType EstimationType, bool& bOutEstimateWasModified);

	TConstArrayView<TConstArrayView<FVertex>> Graph;
	TConstArrayView<TConstArrayView<FVertex>> TransposeGraph;
	TConstArrayView<TConstArrayView<FVertex>> ReachabilityGraph;
	/**
	 * Only valid during SplitCluster. List for each vertex of the cluster's roots that have the vertex in their
	 * reachability graph.
	 */
	TArray<TConstArrayView<FVertex>> ReachableByRootGraph;
	TArray64<FVertex> ReachableByRootGraphBuffer;
	/**
	 * Only valid during SplitCluster. Due to memory constraints not all roots can be reported in ReachableByRootGraph.
	 * This NumVertices-length array records whether the given vertex is a root and is in that graph.
	 */
	TBitArray<> RootInReachableByRootSet;
	TBitArray<> VisitedScratch;
	TArray<FVertex> StackScratch;
	TArray<FRootAndClusterMergeData> UpdateMergeScratch;
	TArray<TArray<FVertex>>& Assignments;
	TArray<TArray<FVertex>>& RootAssignments;
	int32 NumBuckets;
	int32 NumVertices;
};

void ConstructLoadBalance(TConstArrayView<TConstArrayView<FVertex>> Graph, int32 NumBuckets,
	TArray<TArray<FVertex>>& OutAssignments)
{
	FLoadBalanceContext Context;
	Context.Graph = Graph;
	Context.NumBuckets = NumBuckets;
	Context.OutAssignments = &OutAssignments;
	ConstructLoadBalance(Context);
}

void ConstructLoadBalance(FLoadBalanceContext& Context)
{
	check(Context.OutAssignments != nullptr);
	check(Context.NumBuckets > 0);

	TArray64<FVertex> ReachabilityGraphEdgesBuffer;
	TArray<TConstArrayView<FVertex>> ReachabilityGraphBuffer;
	TConstArrayView<TConstArrayView<FVertex>> ReachabilityGraph = Context.ReachabilityGraph;
	TArray64<FVertex> TransposeGraphEdgesBuffer;
	TArray<TConstArrayView<FVertex>> TransposeGraphBuffer;
	TConstArrayView<TConstArrayView<FVertex>> TransposeGraph = Context.TransposeGraph;
	TArray<TArray<FVertex>> RootAssignmentsBuffer;
	TArray<TArray<FVertex>>* RootAssignments = Context.OutRootAssignments;
	int32 NumVertices = Context.Graph.Num();

	if (ReachabilityGraph.Num() != NumVertices)
	{
		ConstructReachabilityGraph(Context.Graph, ReachabilityGraphEdgesBuffer, ReachabilityGraphBuffer);
		ReachabilityGraph = ReachabilityGraphBuffer;
	}
	if (TransposeGraph.Num() != NumVertices)
	{
		ConstructTransposeGraph(Context.Graph, TransposeGraphEdgesBuffer, TransposeGraphBuffer);
		TransposeGraph = TransposeGraphBuffer;
	}
	if (RootAssignments == nullptr)
	{
		RootAssignments = &RootAssignmentsBuffer;
	}

	FLoadBalanceBuilder Builder(Context.Graph, Context.NumBuckets, TransposeGraph, ReachabilityGraph,
		*Context.OutAssignments, *RootAssignments);
	Builder.LoadBalance();
}
	
FLoadBalanceBuilder::FLoadBalanceBuilder(TConstArrayView<TConstArrayView<FVertex>> InGraph, int32 InNumBuckets,
	TConstArrayView<TConstArrayView<FVertex>> InTransposeGraph,
	TConstArrayView<TConstArrayView<FVertex>> InReachabilityGraph,
	TArray<TArray<FVertex>>& OutAssignments, TArray<TArray<FVertex>>& OutRootAssignments)
	: Graph(InGraph)
	, TransposeGraph(InTransposeGraph)
	, ReachabilityGraph(InReachabilityGraph)
	, Assignments(OutAssignments)
	, RootAssignments(OutRootAssignments)
	, NumBuckets(InNumBuckets)
	, NumVertices(InGraph.Num())
{
}

void FLoadBalanceBuilder::LoadBalance()
{
	TArray<FCluster> DisjointSubgraphs;
	FindDisjointSubgraphs(DisjointSubgraphs);

	FindRoots(DisjointSubgraphs);

	TArray<FCluster> Buckets;
	PackClusters(MoveTemp(DisjointSubgraphs), Buckets);

	Assignments.SetNum(NumBuckets, EAllowShrinking::No);
	RootAssignments.SetNum(NumBuckets, EAllowShrinking::No);
	for (int32 BucketIndex = 0; BucketIndex < NumBuckets; ++BucketIndex)
	{
		Assignments[BucketIndex] = MoveTemp(Buckets[BucketIndex].Vertices);
		RootAssignments[BucketIndex] = MoveTemp(Buckets[BucketIndex].Roots);
	}
}

void FLoadBalanceBuilder::FindDisjointSubgraphs(TArray<FCluster>& OutDisjointSubgraphs)
{
	TArray<FCluster>& Subgraphs = OutDisjointSubgraphs;
	Subgraphs.Reset();
	TBitArray<>& Visited = VisitedScratch;
	TArray<FVertex>& Stack = StackScratch;

	Visited.Init(false, NumVertices);
	Stack.Reset(NumVertices);

	// While there are unvisted vertices, graphsearch from an arbitrary vertex on the union graph of Edges+References.
	// All vertices found in that search are a maximal subgraph and one of our disjoint subgraphs.
	for (FVertex RootVertex = 0; RootVertex < NumVertices; ++RootVertex)
	{
		if (Visited[RootVertex])
		{
			continue;
		}

		check(Stack.IsEmpty());
		FCluster& Subgraph = Subgraphs.Emplace_GetRef();
		Visited[RootVertex] = true;
		Subgraph.Vertices.Add(RootVertex);
		Stack.Add(RootVertex);

		while (!Stack.IsEmpty())
		{
			FVertex Vertex = Stack.Pop(EAllowShrinking::No);
			for (TConstArrayView<FVertex> Edges : { Graph[Vertex], TransposeGraph[Vertex] })
			{
				for (FVertex Edge : Edges)
				{
					if (!Visited[Edge])
					{
						Visited[Edge] = true;
						Subgraph.Vertices.Add(Edge);
						Stack.Add(Edge);
					}
				}
			}
		}
	}

	// Sort the vertices in each Subgraph back into Root to Leaf order
	for (FCluster& Subgraph : Subgraphs)
	{
		Algo::Sort(Subgraph.Vertices);
	}
}

void FLoadBalanceBuilder::FindRoots(TArrayView<FCluster> InOutDisjointSubgraphs)
{
	TBitArray<>& Visited = VisitedScratch;

	// Since the subgraphs are disjoint, we can use the same Visited set for all of them without clearing it
	// between subgraphs, and we can read reachability from the reachabilitygraph for the entire graph.
	Visited.Init(false, NumVertices);
	int32 NumMarkedVertices = 0;

	for (FCluster& Subgraph : InOutDisjointSubgraphs)
	{
		// Roots include all the vertices of the subgraph that do not have any referencers, but they can also include
		// vertices in a cycle, so to find them we we have to iteratively subtract reachable vertices from remaining.
		int32 NumSubgraphVertices = Subgraph.Vertices.Num();
		Subgraph.Roots.Reset(NumSubgraphVertices);

		// The vertices are sorted in RootToLeaf order. Iterating from 0 to N-1 and ignoring any reachables from
		// previous roots will mean that each new vertex is a root.
		for (FVertex Root : Subgraph.Vertices)
		{
			if (Visited[Root])
			{
				continue;
			}
			Subgraph.Roots.Add(Root);
			for (FVertex Reachable : ReachabilityGraph[Root])
			{
				if (!Visited[Reachable])
				{
					Visited[Reachable] = true;
					++NumMarkedVertices;
				}
			}
		}
	}
	check(NumMarkedVertices == NumVertices); // The subgraphs should span the graph
}

void FLoadBalanceBuilder::FBucket::CalculateSize(TConstArrayView<FCluster> Clusters)
{
	Size = 0;
	for (int32 ClusterIndex : ClusterIndices)
	{
		Size += Clusters[ClusterIndex].Vertices.Num();
	}
};

void FLoadBalanceBuilder::PackClusters(TArray<FCluster>&& InDisjointSubgraphs, TArray<FCluster>& OutBuckets)
{
	// Our clusters initially are the disjoint subgraphs, but we might split some of the subgraphs into
	// strongly-related but not disjoint Clusters. So for the rest of the algorithm we assume they are generalized
	// clusters and might overlap.
	TArray<FCluster>& Clusters = InDisjointSubgraphs;
	TArray<FBucket> Buckets; // Buckets that we populate by calling ScheduleValues
	Buckets.SetNum(NumBuckets);

	if (NumBuckets < 2)
	{
		Buckets[0].ClusterIndices = Algo::RangeArray<TArray<int32>>(0, Clusters.Num());
		Buckets[0].CalculateSize(Clusters);
		CreateBucketClusters(Buckets, Clusters, OutBuckets);
		return;
	}

	// MaxSpreadToNumVerticesRatio is a tuning variable used to specify how hard we want to look for an optimal
	// solution. When the spread is less than this fraction of NumVertices, we stop looking.
	constexpr double MaxSpreadToNumVerticesRatio = .1;
	int32 MaxSpread = MaxSpreadToNumVerticesRatio * NumVertices;
	TArray<TArray<int32>> BucketsClusterIndices; // Output from Algo::ScheduleValues
	TArray<TArray<int32>> PackExclusionGroups; // Input to Algo::ScheduleValues
	TArray<int32> ClusterCosts; // Input to Algo::ScheduleValues
	Algo::Transform(Clusters, ClusterCosts, [](const FCluster& Cluster) { return Cluster.Vertices.Num(); });
	int32 SplitAttempts = 0;

	for (;;)
	{
		Algo::ScheduleValues(ClusterCosts, NumBuckets, PackExclusionGroups, BucketsClusterIndices);
		for (int32 BucketIndex = 0; BucketIndex < NumBuckets; ++BucketIndex)
		{
			FBucket& Bucket = Buckets[BucketIndex];
			Bucket.ClusterIndices = MoveTemp(BucketsClusterIndices[BucketIndex]);
			Bucket.CalculateSize(Clusters);
		}
		Algo::Sort(Buckets, [](const FBucket& A, const FBucket& B) { return A.Size > B.Size; });
		FBucket& BiggestBucket = Buckets[0];
		int32 Spread = BiggestBucket.Size - Buckets.Last().Size;
		if (Spread <= MaxSpread)
		{
			break;
		}
		if (SplitAttempts >= NumBuckets)
		{
			// Tried too many times; we should only need to split at most one cluster per bucket
			break;
		};

		int32 SplitSize = NumBuckets;
		// We can only split clusters up into roots. Splitting within a root does not help minimize the results because
		// adding the vertex that is not the root to a bucket makes that bucket overlap the bucket containing the root
		// of the vertex without adding any new vertices to the vertices spanned by the pair of buckets.
		// From the clusters assigned to the biggest bucket, pick the smallest one that has >= SplitSize roots.
		// If none have >= SplitSize roots, pick the smallest one with the largest number of roots and clamp the
		// SplitSize to number of roots.
		int32 IndexToSplit = *Algo::MinElement(BiggestBucket.ClusterIndices,
			[&Clusters, SplitSize](int32 A, int32 B)
			{
				FCluster& ClusterA = Clusters[A];
				FCluster& ClusterB = Clusters[B];
				if ((ClusterA.SplitGroup == INDEX_NONE) != (ClusterB.SplitGroup == INDEX_NONE))
				{
					// Clusters resulting from a split are not allowed to be split a second time, because their fragments
					// would reduce when merged into a single cluster in a bucket and the bucket size would not match
					// the size we expected when packing. Push all clusters with a SplitGroup to the back of the sort.
					return ClusterA.SplitGroup == INDEX_NONE;
				}
				bool bAHasEnoughRoots = ClusterA.Roots.Num() >= SplitSize;
				if (bAHasEnoughRoots != (ClusterB.Roots.Num() >= SplitSize))
				{
					return bAHasEnoughRoots;
				}
				if (!bAHasEnoughRoots && ClusterA.Roots.Num() != ClusterB.Roots.Num())
				{
					return ClusterA.Roots.Num() < ClusterB.Roots.Num();
				}
				return ClusterA.Vertices.Num() < ClusterB.Vertices.Num();
			});
		FCluster& ClusterToSplit = Clusters[IndexToSplit];
		if (ClusterToSplit.SplitGroup != INDEX_NONE)
		{
			// All elements were invalid; there is nothing we can split in the biggest bucket, which means there is
			// nothing further we can split to reduce the spread
			break;
		}
		SplitSize = FMath::Min(SplitSize, ClusterToSplit.Roots.Num());
		if (SplitSize < 2)
		{
			// All elements in the biggest bucket had only a single root and are unsplittable; there is nothing further
			// we can split to reduce the spread
			break;
		}

		int32 NewBiggestBucketSize = BiggestBucket.Size - ClusterToSplit.Vertices.Num();
		int32 NewMaximum = FMath::Max(NewBiggestBucketSize, Buckets[1].Size);
		int32 NewMinimum = FMath::Min(NewBiggestBucketSize, Buckets.Last().Size);
		int32 DesiredSpread = NewMaximum - NewMinimum + MaxSpread;

		TArray<FCluster> NewClusters = SplitCluster(MoveTemp(ClusterToSplit), SplitSize, DesiredSpread);

		// Remove the SplitCluster from wherever it is in the middle of Clusters, and shift down by one all of our
		// indices in data that persists between loops that were pointing to clusters at a higher index.
		Clusters.RemoveAt(IndexToSplit);
		ClusterCosts.RemoveAt(IndexToSplit);
		for (TArray<int32>& ExclusionGroup : PackExclusionGroups)
		{
			for (int32& ClusterIndex : ExclusionGroup)
			{
				// We should not have tried to split any cluster in an exclusiongroup
				check(ClusterIndex != IndexToSplit);
				if (ClusterIndex > IndexToSplit)
				{
					--ClusterIndex;
				}
			}
		}

		// SplitClusters from the same original Cluster are not allowed to be assigned to the same
		// bucket, because they would merge and reduce the total size of that bucket and make our 
		// packing not balanced, so we create an exclusiongroup for each split.
		TArray<int32>& ExclusionGroup = PackExclusionGroups.Emplace_GetRef();
		check(NewClusters.Num() == SplitSize);
		for (FCluster& Cluster : NewClusters)
		{
			ClusterCosts.Add(Cluster.Vertices.Num());
			int32 ClusterIndex = Clusters.Num();
			Cluster.SplitGroup = SplitAttempts;
			ExclusionGroup.Add(ClusterIndex);
			Clusters.Add(MoveTemp(Cluster));
		}
		++SplitAttempts;
	}

	CreateBucketClusters(Buckets, Clusters, OutBuckets);
}

void FLoadBalanceBuilder::CreateBucketClusters(TConstArrayView<FBucket> Buckets, TConstArrayView<FCluster> Clusters, TArray<FCluster>& OutBuckets)
{
	OutBuckets.SetNum(NumBuckets, EAllowShrinking::No);
	for (int32 BucketIndex = 0; BucketIndex < NumBuckets; ++BucketIndex)
	{
		const FBucket& Bucket = Buckets[BucketIndex];
		FCluster& OutBucket = OutBuckets[BucketIndex];
		int32 NumRoots = 0;
		for (int32 ClusterIndex : Bucket.ClusterIndices)
		{
			NumRoots += Clusters[ClusterIndex].Roots.Num();
		}
		OutBucket.Vertices.Reset(Bucket.Size);
		OutBucket.Roots.Reset(NumRoots);
		for (int32 ClusterIndex : Bucket.ClusterIndices)
		{
			const FCluster& Cluster = Clusters[ClusterIndex];
			OutBucket.Vertices.Append(Cluster.Vertices);
			OutBucket.Roots.Append(Cluster.Roots);
		}
		Algo::Sort(OutBucket.Vertices);
		OutBucket.Vertices.SetNum(Algo::Unique(OutBucket.Vertices), EAllowShrinking::No );
		Algo::Sort(OutBucket.Roots);
		OutBucket.Roots.SetNum(Algo::Unique(OutBucket.Roots), EAllowShrinking::No);
	}
}

template <typename RangeType, typename MinimumsRangeType, typename ProjectedElementType, typename ProjectionType>
void GetMaxAndMins(RangeType&& Range, ProjectedElementType& OutMaximum,	MinimumsRangeType&& InOutMinimums,
	ProjectionType Proj)
{
	int32 NumMinimums = GetNum(InOutMinimums);
	int32 NumRange = GetNum(Range);
	check(NumRange >= NumMinimums && NumMinimums > 0);
	OutMaximum = Proj(Range[0]);
	InOutMinimums[0] = OutMaximum;
	int32 Index;
	for (Index = 1; Index < NumMinimums; ++Index)
	{
		ProjectedElementType Element = Proj(Range[Index]);
		InOutMinimums[Index] = Element;
		OutMaximum = FMath::Max(OutMaximum, Element);
	}
	Algo::Sort(InOutMinimums);
	for (; Index < NumRange; ++Index)
	{
		ProjectedElementType Element = Proj(Range[Index]);
		if (Element < InOutMinimums.Last())
		{
			InOutMinimums.Last() = Element;
			Algo::RestoreSort(InOutMinimums, NumMinimums - 1, TLess<ProjectedElementType>());
		}
		OutMaximum = FMath::Max(OutMaximum, Element);
	}
}

TArray<FLoadBalanceBuilder::FCluster> FLoadBalanceBuilder::SplitCluster(FCluster&& InCluster, int32 SplitSize, int32 DesiredSpread)
{
	check(SplitSize >= 2);

	// Create the ReachedByRoot graph edges for all vertices, so we know which other roots to update when the
	// vertices of a root are merged into one of the output clusters. There might be a large number of roots
	// and so the ReachedByRoot graph might be too large to fit in memory; restrict it in size to 1000*NumVertices.
	constexpr int32 ReachableByRootEdgesPerVertexLimit = 1000;
	ReachableByRootGraph.Reset();
	ReachableByRootGraphBuffer.Reset();
	TArray<FVertex> RootsInGraph;
	ConstructPartialTransposeGraph(ReachabilityGraph, InCluster.Roots,
		ReachableByRootEdgesPerVertexLimit * NumVertices, ReachableByRootGraphBuffer,
		ReachableByRootGraph, RootsInGraph);
	RootInReachableByRootSet.Init(false, NumVertices);
	for (FVertex Root : RootsInGraph)
	{
		RootInReachableByRootSet[Root] = true;
	}
	ON_SCOPE_EXIT
	{
		ReachableByRootGraph.Empty();
		ReachableByRootGraphBuffer.Empty();
		RootInReachableByRootSet.Empty();
		UpdateMergeScratch.Empty();
	};

	// Create RootAssigned arraymap to specify whether each root is assigned
	TBitArray<> RootAssigned;
	RootAssigned.SetNumUninitialized(NumVertices);
	RootAssigned.SetRange(0, NumVertices, false);
	int32 NumRemainingRoots = InCluster.Roots.Num();

	// Create one MergeCluster per splitsize
	int32 NumClusters = SplitSize;
	TArray<FMergeCluster> Clusters;
	Clusters.Reserve(NumClusters);
	for (int32 ClusterIndex = 0; ClusterIndex < NumClusters; ++ClusterIndex)
	{
		FMergeCluster& Cluster = Clusters.Emplace_GetRef();
		Cluster.VertexInCluster.Init(false, NumVertices);
		Cluster.RootReductions.SetNumZeroed(NumVertices);
	}

	// Assign the biggest root to output cluster 0. Its better for the algorithm to know where it's headed so it can
	// prefer to merge in roots that reduce well with that biggest root. We cannot seed the other buckets however,
	// because we don't know which other roots will end up NOT being assigned to the bucket with the biggest root.
	{
		FVertex BestRoot = *Algo::MinElement(InCluster.Roots, [this](FVertex A, FVertex B)
			{ return ReachabilityGraph[A].Num() > ReachabilityGraph[B].Num(); });

		check(!RootAssigned[BestRoot]);
		RootAssigned[BestRoot] = true;
		--NumRemainingRoots;
		Clusters[0].Roots.Add(BestRoot);

		CalculateMergeResults(Clusters[0], BestRoot, true /* bWriteResultsToCluster */);
	}

	// Initialize the MergeDatas in each cluster; for better performance do this after assigning the intial seed.
	for (FMergeCluster& Cluster : Clusters)
	{
		TArray<FRootAndClusterMergeData>& MergeDatas = Cluster.MergeDatas;
		MergeDatas.Reserve(NumRemainingRoots);
		for (FVertex Root : InCluster.Roots)
		{
			if (!RootAssigned[Root])
			{
				FRootAndClusterMergeData& MergeData = MergeDatas.Emplace_GetRef();
				MergeData.Root = Root;
				MergeData.RootSize = ReachabilityGraph[Root].Num();
				bool bModified;
				UpdateMergeDataEstimate(Cluster, MergeData, EEstimationType::TightUpperBound, bModified);
			}
		}
		Algo::Sort(MergeDatas, FRootAndClusterMergeData::IsWorse);
	}

	// The main loop: on each iteration of the loop find the best root to move into the best cluster,
	// where best is defined by maximizing reduction and minimizing spread. Assign that best root to the best cluster.
	int32 BalancedAmountPerCluster = InCluster.Vertices.Num() / NumClusters;
	TArray<FClusterMergeSelectionData> SelectionDatas;
	while (NumRemainingRoots > 0)
	{
		for (FMergeCluster& Cluster : Clusters)
		{
			// We need to update every Cluster's MergeDatas on every loop, rather than just the one that won the last
			// loop, in case the best root in a non-winning cluster was the same as the winner's.
			UpdateClusterMergeDatas(Cluster, RootAssigned);
		}
		int32 MaxClusterSize;
		int32 MinClusterSizes[2];
		GetMaxAndMins(Clusters, MaxClusterSize, TArrayView<int32>(MinClusterSizes),
			[&Clusters](const FMergeCluster& Cluster) { return Cluster.VertexInClusterCount; });
		SelectionDatas.Reset();

		for (FMergeCluster& Cluster : Clusters)
		{
			FRootAndClusterMergeData& MergeData = Cluster.MergeDatas.Last();
			FClusterMergeSelectionData& SelectionData = SelectionDatas.Emplace_GetRef();
			SelectionData.Root = MergeData.Root;
			SelectionData.Cluster = &Cluster;

			int32 OldSize = Cluster.VertexInClusterCount;
			check(MergeData.bExact&& MergeData.ClusterSizeWhenEstimated == OldSize);
			SelectionData.Reduction = MergeData.ReductionUpperBound;
			SelectionData.NewSize = OldSize + MergeData.RootSize - SelectionData.Reduction;

			int32 NewMaximum = FMath::Max(MaxClusterSize, SelectionData.NewSize);
			int32 NewMinimum;
			if (OldSize == MinClusterSizes[0])
			{
				NewMinimum = FMath::Min(SelectionData.NewSize, MinClusterSizes[1]);
			}
			else
			{
				NewMinimum = MinClusterSizes[0];
			}
			SelectionData.Spread = NewMaximum - NewMinimum;
			bool bCausedSpread = NewMaximum > MaxClusterSize;
			bool bSpreadIsAProblem = NewMaximum > BalancedAmountPerCluster;
			bool bSpreadIsOver = SelectionData.Spread > DesiredSpread;
			SelectionData.bOverSpread = bCausedSpread && bSpreadIsAProblem && bSpreadIsOver;
		}

		FClusterMergeSelectionData* BestData = Algo::MinElement(SelectionDatas, FClusterMergeSelectionData::IsBetter);
		FMergeCluster& BestCluster = *BestData->Cluster;
		int32 Reduction = CalculateMergeResults(BestCluster, BestData->Root, true /* bWriteResultsToCluster */);
		check(Reduction == BestData->Reduction);
		BestCluster.Roots.Add(BestData->Root);
		check(!RootAssigned[BestData->Root]);
		RootAssigned[BestData->Root] = true;
		--NumRemainingRoots;
	}

	TArray<FCluster> OutClusters;
	OutClusters.Reserve(Clusters.Num());
	for (FMergeCluster& Cluster : Clusters)
	{
		FCluster& OutCluster = OutClusters.Emplace_GetRef();
		OutCluster.Vertices.Empty(Cluster.VertexInClusterCount);
		for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
		{
			if (Cluster.VertexInCluster[Vertex])
			{
				OutCluster.Vertices.Add(Vertex);
			}
		}
		OutCluster.Roots = MoveTemp(Cluster.Roots);
	}
	return OutClusters;
}

void FLoadBalanceBuilder::UpdateClusterMergeDatas(FMergeCluster& Cluster, TBitArray<>& RootAssigned)
{
	TArray<FRootAndClusterMergeData>& MergeDatas = Cluster.MergeDatas;
	check(!MergeDatas.IsEmpty());

	int32 OriginalMergeDataNum = MergeDatas.Num();
	int32 RemovedMergedDataNum = 0;

	// Pop mergedatas for completed roots off the back of the list as we encounter them
	// Set BestMergeData to the first mergedata found for a remaining root
	FRootAndClusterMergeData BestMergeData = MergeDatas.Pop(EAllowShrinking::No);
	while (RootAssigned[BestMergeData.Root])
	{
		++RemovedMergedDataNum;
		check(!MergeDatas.IsEmpty()); // We only update when there are remaining roots, so it should never be empty
		BestMergeData = MergeDatas.Pop(EAllowShrinking::No);
	}

	// Update the mergedata to get the exact value; usually this will be less than the upperbound and we need to
	// compare exact values when choosing the best. The mergedata at the back of the list is the estimated best and
	// is likely to be the actual best, so its a good first mergedata to make exact.
	bool bModified;
	UpdateMergeDataEstimate(Cluster, BestMergeData, EEstimationType::Exact, bModified);

	// Collect all the mergedatas from the back of the list until we reach a point in the list where the
	// (uniform-upperbound) estimates for all remaining mergedatas are worse than the (tighter-bound) estimate for
	// the worst merge we've found so far.
	UpdateMergeScratch.Reset(MergeDatas.Num());
	TOptional<FRootAndClusterMergeData> WorstMergeData;
	for (;;)
	{
		// Continue to pop off mergedatas for completed roots as we find them.
		while (!MergeDatas.IsEmpty() && RootAssigned[MergeDatas.Last().Root])
		{
			++RemovedMergedDataNum;
			MergeDatas.Pop(EAllowShrinking::No);
		}
		if (MergeDatas.IsEmpty())
		{
			// This happens e.g. when only one root remains
			break;
		}

		// Compare the new end of the list. Update its estimate using LooseUpperBoundUniformIncrease.
		// (1) This is the most conservative estimate, so if it's worse than any of the estimates or exact values we've
		// found so far, we know its tighter estimate will also be worse.
		// (2) This is a uniform estimate: it raises all up-to-date estimate types of all roots by the same amount.
		// So since the roots from this point and earlier in the list were previously sorted by their estimate, 
		// the sort order will be unchanged even if we bring them all up-to-date with LooseUpperBoundUniformIncrease
		// estimation, and further, their non-up-to-date value will be <= their up-to-date value.
		//
		// These two conditions are sufficient to guarantee that we don't have inspect mergedatas earlier in the list
		// to correctly find the best merge or to resort the list once the end of the list has a worse estimate than
		// our current worst.
		{
			FRootAndClusterMergeData& CompareMergeData = MergeDatas.Last();
			FRootAndClusterMergeData* WorstMergeDataPtr = WorstMergeData.IsSet() ?
				&WorstMergeData.GetValue() : &BestMergeData;
			UpdateMergeDataEstimate(Cluster, CompareMergeData, EEstimationType::LooseUpperBoundUniformIncrease,
				bModified);
			if (!FRootAndClusterMergeData::IsWorse(*WorstMergeDataPtr, CompareMergeData))
			{
				break;
			}
		}

		// NewMergeData's most-conservative estimate is as good or better than the worst estimate we've found
		// so far. Sort it into the (partially sorted) list of mergedatas we're collecting: either it's the best,
		// or the worse, or it's in the otherwise-unsorted middle.
		// Sorting the middle at the end is faster than sorting it as we go because it avoids shifts.
		// NewMergeData doesn't get to count itself as the best unless its estimate has been made exact, but don't
		// spend time making the estimate exact unless we have to.
		FRootAndClusterMergeData NewMergeData = MergeDatas.Pop(EAllowShrinking::No);

		bool bIsNewBest = false;
		bool bIsNewWorst = false;
		// If WorstMergeData is not set then we already compared NewMergeData to BestMergeData above and do not
		// need to redo the comparison
		if (!WorstMergeData.IsSet() || FRootAndClusterMergeData::IsWorse(BestMergeData, NewMergeData))
		{
			// Change to the tighter estimate and recompare
			UpdateMergeDataEstimate(Cluster, NewMergeData, EEstimationType::TightUpperBound, bModified);
			// If the NewMergeData's value was not modified when we changed to a tighter estimate, then we
			// know it is still better than the BestMergeData and we do not need to compare again
			if (!bModified || FRootAndClusterMergeData::IsWorse(BestMergeData, NewMergeData))
			{
				// Change to the tightest estimate (the exact value) and recompare
				UpdateMergeDataEstimate(Cluster, NewMergeData, EEstimationType::Exact, bModified);
				bIsNewBest = !bModified || FRootAndClusterMergeData::IsWorse(BestMergeData, NewMergeData);
			}
			// NewMergeData was previously better than the worst mergedata, but we changed it to a TightUpperBound
			// or Exact value so it might be worse now. It can't be worse than the worst if it's better than the best.
			bIsNewWorst = !bIsNewBest && (!WorstMergeData.IsSet() || FRootAndClusterMergeData::IsWorse(NewMergeData, *WorstMergeData));
		}
		if (bIsNewBest)
		{
			if (!WorstMergeData.IsSet())
			{
				WorstMergeData.Emplace(MoveTemp(BestMergeData));
			}
			else
			{
				UpdateMergeScratch.Add(MoveTemp(BestMergeData));
			}
			BestMergeData = MoveTemp(NewMergeData);
		}
		else if (bIsNewWorst)
		{
			if (WorstMergeData.IsSet())
			{
				UpdateMergeScratch.Add(MoveTemp(*WorstMergeData));
			}
			WorstMergeData.Emplace(MoveTemp(NewMergeData));
		}
		else
		{
			UpdateMergeScratch.Add(MoveTemp(NewMergeData));
		}
	}

	// Push all the mergedatas we pulled off the list back onto the list, sorted from worst to best
	if (WorstMergeData.IsSet())
	{
		MergeDatas.Add(*WorstMergeData);
	}
	if (!UpdateMergeScratch.IsEmpty())
	{
		Algo::Sort(UpdateMergeScratch, FRootAndClusterMergeData::IsWorse);
		for (FRootAndClusterMergeData& Updated : UpdateMergeScratch)
		{
			MergeDatas.Add(MoveTemp(Updated));
		}
		UpdateMergeScratch.Reset();
	}
	MergeDatas.Add(MoveTemp(BestMergeData));
	check(MergeDatas.Num() == OriginalMergeDataNum - RemovedMergedDataNum);
}

int32 FLoadBalanceBuilder::CalculateMergeResults(FMergeCluster& Cluster, FVertex Root, bool bWriteResultsToCluster)
{
	int32 Reduction = 0;
	if (bWriteResultsToCluster)
	{
		TArray<int32>& RootReductions = Cluster.RootReductions;
		for (FVertex Reachable : ReachabilityGraph[Root])
		{
			if (!Cluster.VertexInCluster[Reachable])
			{
				Cluster.VertexInCluster[Reachable] = true;
				// When we add a vertex to the cluster, we need to inform every remainingroot in the cluster that has
				// that vertex in its reachability set that the number of reductions it will have when merged into the
				// cluster has increased by one.
				for (FVertex ReachedByRoot : ReachableByRootGraph[Reachable])
				{
					++RootReductions[ReachedByRoot];
				}
			}
			else
			{
				++Reduction;
			}
		}
		Cluster.VertexInClusterCount += ReachabilityGraph[Root].Num() - Reduction;
	}
	else
	{
		for (FVertex Reachable : ReachabilityGraph[Root])
		{
			Reduction += Cluster.VertexInCluster[Reachable] ? 1 : 0;
		}
	}
	return Reduction;
}

void FLoadBalanceBuilder::UpdateMergeDataEstimate(FMergeCluster& Cluster, FRootAndClusterMergeData& MergeData,
	EEstimationType EstimationType, bool& bOutEstimateWasModified)
{
	int32 ClusterSize = Cluster.VertexInClusterCount;
	MergeData.bExact = MergeData.bExact & (ClusterSize == MergeData.ClusterSizeWhenEstimated);
	if (MergeData.bExact)
	{
		bOutEstimateWasModified = false;
		return;
	}

	int32 PreviousReductionUpperBound = MergeData.ReductionUpperBound;
	if (EstimationType == EEstimationType::Exact ||
		(EstimationType != EEstimationType::LooseUpperBoundUniformIncrease &&
			RootInReachableByRootSet[MergeData.Root]))
	{
		if (RootInReachableByRootSet[MergeData.Root])
		{
			// We have data for every vertex about whether it is reachable by the root, and we use that data
			// every time we merge a vertex into a cluster, to increment the reduction count of every root
			// that includes the vertex. 
			// This allows this function to do a cheap update of the exact value of the root's reduction:
			// we just consume the recorded delta reduction value of this root and add it to the reduction it
			// had the last time we updated it.
			TArray<int32>& RootReductions = Cluster.RootReductions;
			MergeData.ReductionUpperBound = MergeData.PreviousExactEstimate + RootReductions[MergeData.Root];
			RootReductions[MergeData.Root] = 0;
		}
		else
		{
			// We don't have the reachedby data for this root, so we have to do the slow calculation of the
			// merge results
			MergeData.ReductionUpperBound = CalculateMergeResults(Cluster, MergeData.Root,
				false /* bWriteResultsToCluster */);
		}
		MergeData.PreviousExactEstimate = MergeData.ReductionUpperBound;
		MergeData.bExact = true;
	}
	else
	{
		int32 LooseEstimate = MergeData.ReductionUpperBound + ClusterSize - MergeData.ClusterSizeWhenEstimated;
		if (EstimationType == EEstimationType::TightUpperBound)
		{
			// TightUpperBound assumes all new vertices in the cluster will reduce with this one,
			// but notes that the number of reductions is <= size of this root
			MergeData.ReductionUpperBound = FMath::Min(LooseEstimate, MergeData.RootSize);
		}
		else
		{
			check(EstimationType == EEstimationType::LooseUpperBoundUniformIncrease);
			// LooseUpperBoundUniformIncrease is the same as TightUpperBound, but without the clamping
			// It has two properties useful for our sorted array:
			// 1) UpperBoundUniformIncrease(CurrentCluster) >= UpperBoundUniformIncrease(PreviousSmallerCluster)
			//    for all MergeDatas and all growth from PreviousSmallerCluster to CurrentCluster
			// 2) UpperBoundUniformIncrease(CurrentCluster, MergeDataA) - UpperBoundUniformIncrease(PreviousSmallerCluster, MergeDataA) ==
			//    UpperBoundUniformIncrease(CurrentCluster, MergeDataB) - UpperBoundUniformIncrease(PreviousSmallerCluster, MergeDataB)
			//    for all MergeDataA, MergeDataB
			// These two properties allow us to know we do not need to investigate MergeDatas with an UpperBoundUniformIncrease estimate
			//    worse than our current worst estimate.
			MergeData.ReductionUpperBound = LooseEstimate;
		}
	}
	MergeData.ClusterSizeWhenEstimated = ClusterSize;
	bOutEstimateWasModified = MergeData.ReductionUpperBound != PreviousReductionUpperBound;
};

bool FLoadBalanceBuilder::FRootAndClusterMergeData::IsWorse(FRootAndClusterMergeData& A, FRootAndClusterMergeData& B)
{
	if (A.ReductionUpperBound != B.ReductionUpperBound)
	{
		// First maximize reduction, aka maximize sharing
		return A.ReductionUpperBound < B.ReductionUpperBound;
	}
	if (A.RootSize != B.RootSize)
	{
		// If the reduction is the same, prefer to merge the smaller root; if we can reduce
		// a smaller root by the same amount as a larger root, we should take the smaller root because it
		// adds fewer unreduced vertices
		return A.RootSize > B.RootSize;
	}
	// Prefer earlier root values if all else is equal.
	// TODO: This improves the splitting, I'm not sure why.
	return A.Root > B.Root;
};

bool FLoadBalanceBuilder::FClusterMergeSelectionData::IsBetter(const FClusterMergeSelectionData& A,
	const FClusterMergeSelectionData& B)
{
	if (A.bOverSpread != B.bOverSpread)
	{
		// If a merge goes over the spread, don't use it unless all merges go over the spread
		return !A.bOverSpread;
	}
	if (A.bOverSpread && A.Spread != B.Spread)
	{
		// If all merges go over the spread, pick the one that minimizes the resultant spread
		return A.Spread < B.Spread;
	}
	if (A.Reduction != B.Reduction)
	{
		// When spread is not a factor, we want to maximize reduction, aka maximize sharing
		return A.Reduction > B.Reduction;
	}
	// If the reduction is the same, prefer the merge with the smaller final size; if we can reduce
	// a smaller root by the same amount as a larger root, we should take the smaller root because it
	// adds fewer unreduced vertices
	return A.NewSize < B.NewSize;
}

}

