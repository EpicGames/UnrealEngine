// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphConvert.h"

namespace Algo::Graph
{

/**
 * Attempt to create an assignment of transitively-closed sets of reachable vertices to buckets such that all vertices
 * are assigned to at least one bucket and the number of vertices assigned to each bucket is
 * longpoleandspread-minimized.
 *
 * A transitively-closed set of vertices means that if a vertex is present in a bucket then all vertices reachable
 * from it are also present.
 * 
 * The definition of longpoleandspread-minimized is:
 * 1) The size of the largest bucket is minimized.
 * 2) If multiple assignments achieve that minimum, the difference in size between the largest bucket and smallest
 *    bucket is minimized.
 * 
 * This problem is NP-Complete and we do not have an exact solution. The result may fail to achieve the minimum.
 * 
 * @param Graph Input graph that will be loadbalanced. For format, @see ConvertToGraph.
 *        The implicit vertices of this graph (0 to Num-1) must be sorted in RootToLeaf order.
 * @param NumBuckets Number of buckets to divide vertices into. Must be >= 1.
 * @param OutAssignments Output array of buckets, of length NumBuckets. Each element is an array of vertices assigned
 *        to the corresponding bucket.
 */
void ConstructLoadBalance(TConstArrayView<TConstArrayView<FVertex>> Graph, int32 NumBuckets, 
	TArray<TArray<FVertex>>& OutAssignments);

/**
 * Expanded arguments for ConstructLoadBalance for callers that want to provide as input or receive as output some
 * of the intermediate data.
 */
struct FLoadBalanceContext
{
	/**
	 * Required. Input graph that will be LoadBalanced. For format, @see ConvertToGraph.
     * The implicit vertices of this graph (0 to Num-1) must be sorted in RootToLeaf order.
	 */
	TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> Graph;
	/** Optional. Transpose of the input graph, as constructed by Algo::Graph::ConstructTransposeGraph. */
	TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> TransposeGraph;
	/** Optional. Reachability graph of the input graph, as constructed by Algo::Graph::ConstructReachabilityGraph. */
	TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> ReachabilityGraph;
	/**
	 * Required, must not be null. Output array of buckets, of length NumBuckets. Each element is an array of vertices
	 * assigned to the corresponding bucket.
	 */
	TArray<TArray<FVertex>>* OutAssignments = nullptr;
	/**
	 * Optional. If non-null, will be assigned NumBuckets elements. The nth element corresponds to the nth element of
	 * OutAssignments, and records the roots of the graph that are present in that bucket. In the absence of cycles, 
	 * roots are all vertices with no referencers, also known as no incoming edges. The list of vertices reachable
	 * from the roots spans the graph. In the presence of cycles at the root level, an arbitrary vertex from each
	 * rooted cycle will be chosen as the root for that cycle.
	 *
	 * Each root will appear in exactly one bucket. All vertices in the bucket will be reachable from one of the roots
	 * in the bucket.
	 */
	TArray<TArray<FVertex>>* OutRootAssignments = nullptr;
	/** Required. Number of buckets to divide vertices into. Must be >= 1. */
	int32 NumBuckets = 1;
};

/** ConstructLoadBalance that takes extra arguments. @see FLoadBalanceContext for argument descriptions. */
void ConstructLoadBalance(FLoadBalanceContext& Context);


}
