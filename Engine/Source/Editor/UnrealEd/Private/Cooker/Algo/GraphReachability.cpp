// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphReachability.h"

#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/BinaryHeap.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/EnumClassFlags.h"
#include "SortSpecialCases.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"
#endif

template <typename ArrayType, typename NumToAddSizeType>
void ReserveGrowth(ArrayType& Array, NumToAddSizeType NumToAdd)
{
	typename ArrayType::SizeType OriginalCount = Array.Num();
	Array.AddUninitialized(NumToAdd); // this will grow the array geometrically using CalculateSlackGrow
	Array.SetNum(OriginalCount, EAllowShrinking::No);
}

namespace Algo::Graph
{

/** Implements ConstructReachabilityGraph. */
class FReachabilityBuilder
{
public:
	FReachabilityBuilder(TConstArrayView<TConstArrayView<FVertex>> InGraph, TArray64<FVertex>& OutReachabilityGraphBuffer,
		TArray<TConstArrayView<FVertex>>& OutGraph);

	/** Calculate ReachabilityGraph for the current Graph, which might be cyclic or acyclic. */
	void Build();

private:
	/** Status for vertices in the graphsearch. InProgress should never be encountered since our graph is acyclic. */
	enum class EVertexStatus : uint8
	{
		NotStarted,
		InProgress,
		Done,
	};

	/**
	 * A view into an array. Similar to an arrayview, but it is based on a pointer to the array and an index rather
	 * than a pointer to the allocation. This prevents us from having to update it when the array reallocates.
	 */
	struct FSliceOfArray
	{
		TArray64<FVertex>* Buffer = nullptr;
		int64 Offset = INDEX_NONE;
		int32 Num = 0;
		TConstArrayView<FVertex> GetView() const
		{
			return TConstArrayView<FVertex>(Buffer->GetData() + Offset, Num);
		}
		void Set(TArray64<FVertex>* InBuffer, int64 InOffset, int32 InNum)
		{
			Buffer = InBuffer;
			Offset = InOffset;
			Num = InNum;
		}
	};
	/**
	 * Data about each vertex that lasts beyond the current graph search stack. ReachablesSlice will be volatile during the 
	 * search that first finds the vertex but will end up as a slice of the ReachabilityBuffer.
	 */
	struct FVertexData
	{
		FSliceOfArray ReachablesSlice;
		EVertexStatus Status = EVertexStatus::NotStarted;
	};
	/**
	 * A buffer that holds the ReachablesSlice for one or more vertices encountered during a graph search stack. It is
	 * formed from a vertex that cannot be embedded into its referencer's reachability slice. When the stack is
	 * emptied, the Buffer is copied onto the reachabilitybuffer after the slices embedded into the reachabilitybuffer
	 * during the search, and the EmbeddedSlices are updated to point to the copy on the reachabilitybuffer.
	 */
	struct FSeparateBuffer
	{
		TArray64<FVertex> Buffer;
		TArray<FSliceOfArray*> EmbeddedSlices;
	};
	/**
	 * Information about the reachables of direct edges of a vertex on the stack. The reachables of these edges are
	 * embedded into the vertex's reachables during FinishVertex, if possible.
	 */
	struct FStackEdgeData
	{
		FVertex EdgeVertex = InvalidVertex;
		FSeparateBuffer EdgeBuffer;
		bool bEmbeddable = false;
	};
	/**
	 * Information about the reachables of a vertex on the stack, including its direct edge's reachables that need
	 * to be embedded into it. Buffer and EmbeddedSlices are provided by the referencer of the vertex, for it to
	 * store its reachables into so that it can be embedded into its referencer when its turn to be embedded comes.
	 */
	struct FStackData
	{
		TArray64<FVertex>* Buffer;
		TArray<FSliceOfArray*>* EmbeddedSlices;
		TArray<FStackEdgeData> StackEdgeDatas;
		FVertex Vertex = InvalidVertex;
		int32 NextEdge = 0;
	};

private:
	/** Calculate ReachabilityGraph for the current Graph. Assumes Graph is acyclic. */
	void BuildAcyclic(bool bShrink);
	/**
	 * Push a vertex onto the graph search stack and initialize the data used to calculate its reachables.
	 * Its reachables (and the list of ReachabilitySlices for all vertices using the buffer) will be stored in
	 * Buffer and EmbeddedSlices.
	 */
	void StartVertex(FVertex Vertex, TArray64<FVertex>* Buffer, TArray<FSliceOfArray*>* EmbeddedSlices);
	/**
	 * Called after all directedges have recursively calculated their reachables (or were previously calculated.
	 * Creates the list of reachables for the vertex, embeds the reachables of all possible direct edges into it,
	 * stores the list in the Buffer that was passed into StartVertex, and records its VertexData's ReachabilitySlice
	 * as (currently) pointing to that Buffer. The ReachabilitySlice will be updated as the Buffer moves around.
	 * Also pops the vertex from the graph search stack.
	 */
	void FinishVertex();

	/** Called when the input graph is already acyclic. Calculate RootToLeafOrder required by BuildAcyclic. */
	void ReadRootToLeafOrderFromInputVertexToGraphVertex();
	/**
	 * Called when the input graph has cycles and so BuildAcyclic was called on the CondensationGraph.
	 * Transform the results from the CondensationGraph back into the results for the input graph.
	 */
	void TransformCondensationReachabilityToInputReachability();

	/**
	 * Append the given SourceBuffer onto the TargetBuffer, and update the EmbeddedSlices that were pointing to the
	 * SourceBuffer to point to their new copy in the TargetBuffer.
	 */
	static void RelocateViews(TArray64<FVertex>& InOutTargetBuffer, TArray64<FVertex>& SourceBuffer,
		TArrayView<FSliceOfArray*> EmbeddedSlices);

private:

	/**
	 * The graph for which we calculate reachability. Possibly used during the graph search, but we will instead
	 * operate on its CondensationGraph if it has cycles
	 */
	TConstArrayView<TConstArrayView<FVertex>> InputGraph;
	/** Output buffer for the edges of the reachability graph. We write to it during the graph search. */
	TArray64<FVertex>& ReachabilityBuffer;
	/**
	 * Output list of arrayviews for the edges of the reachabilitygraph. To save memory, these arrayviews will
	 * overlap each other. We calculate it at the end from intermediate data
	 */
	TArray<TConstArrayView<FVertex>>& ReachabilityGraph;


	/** The graph of edges used in the graph search. Either the InputGraph or its CondensationGraph. */
	TConstArrayView<TConstArrayView<FVertex>> Graph;

	/** Status and reachables Data about each vertex. Reachables are recorded as a slice into a changeable buffer. */
	TArray<FVertexData> VertexDatas;
	/** Information to keep place and calculate reachables for vertices encountered during a search from a root. */
	TArray<FStackData> Stack;
	/**
	 * List of reachables buffers for vertices that were not embeddable during the graph search and need to be added
	 * to reachabilitybuffer afterwards.
	 */
	TArray<FSeparateBuffer> SeparateBuffers;

	/** RootToLeafOrder of the input vertices. Calculated at the same time as we check for cycles and create the
	 * CondensationGraph. Our algorithm for embedding reachables relies on acyclic graph and RootToLleaf order.
	 */
	TArray<FVertex> RootToLeafOrder;
	/** Buffer for edges for the CondensationGraph. Empty and unused if InputGraph is acyclic. */
	TArray64<FVertex> CondensationGraphBuffer;
	/** ArrayViews of edges of the CondensationGraph. Empty and unused if InputGraph is acyclic. */
	TArray<TConstArrayView<FVertex>> GraphEdges;
	/** Buffer for input graph vertices for each condensationgraph vertex. unused if InputGraph is acyclic. */
	TArray64<FVertex> CondensationVertexToInputVerticesBuffer;
	/** List of input graph vertices for each condensationgraph vertex unused if InputGraph is acyclic. */
	TArray<TConstArrayView<FVertex>> GraphVertexToInputVertices;
	/**
	 * Map from input vertex to condensationgraph vertex; multiple input vertices can map to the same condensation
	 * vertex. Unused (after creating RootToLeafOrder) if InputGraph is acyclic
	 */
	TArray<FVertex> InputVertexToGraphVertex;

	/** Scratch-space buffer for unioning sets of vertices during FinishVertex. NumVertices array of bools. */
	TBitArray<> IsReachable;
};

void ConstructReachabilityGraph(TConstArrayView<TConstArrayView<FVertex>> InGraph, TArray64<FVertex>& OutReachabilityGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph)
{
	FReachabilityBuilder Builder(InGraph, OutReachabilityGraphBuffer, OutGraph);
	Builder.Build();
}

FReachabilityBuilder::FReachabilityBuilder(TConstArrayView<TConstArrayView<FVertex>> InGraph, TArray64<FVertex>& OutReachabilityGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph)
	: InputGraph(InGraph)
	, ReachabilityBuffer(OutReachabilityGraphBuffer)
	, ReachabilityGraph(OutGraph)
{
}
void FReachabilityBuilder::Build()
{
	// Our compaction algorithm requires that the graph is acyclic, so create the condensation graph,
	// run the reachability graph on the condensation graph, and then transform the reachability of the
	// condensation graph into reachability for the input graph
	if (TryConstructCondensationGraph(InputGraph, CondensationGraphBuffer, GraphEdges, &CondensationVertexToInputVerticesBuffer,
		&GraphVertexToInputVertices, &InputVertexToGraphVertex))
	{
		Graph = GraphEdges;
		RootToLeafOrder = Algo::RangeArray<TArray<FVertex>>(0, Graph.Num());
		BuildAcyclic(false /* bShrink */);
		TransformCondensationReachabilityToInputReachability();
	}
	else
	{
		// If the input graph is acyclic, no transforming is needed, but we do still need a root to leaf
		// order of the input graph. TryConstructCondensationGraph left it for us in InputVertexToGraphVertex
		Graph = InputGraph;
		ReadRootToLeafOrderFromInputVertexToGraphVertex();
		BuildAcyclic(true /* bShrink */);
	}
}

void FReachabilityBuilder::BuildAcyclic(bool bShrink)
{
	// About compaction:
	// The reachability graph for N vertices could be as large as NxN, and in practice will often
	// have size N*N/2. When N is 1 million this is too big to fit into memory in our current machines
	// which have on the order of 100GB. (Half of million*million = 1TB*BytesPerElement/2).
	// But reachability graphs have a lot of redundancy and can be compacted; multiple vertices will overlap
	// each other in the buffer of reachable vertices.
	// As we search through the graph, we embed the reachables of direct edges into the reachables of the current
	// vertex when possible.
	int32 NumVertices = Graph.Num();
	Stack.Reset(NumVertices);
	SeparateBuffers.Reset(NumVertices);
	VertexDatas.Reset(NumVertices);
	VertexDatas.SetNum(NumVertices, EAllowShrinking::No);
	ReachabilityBuffer.Reset();
	IsReachable.Init(false, NumVertices);

	// Start searching at root vertices so we can maximize compaction
	for (FVertex RootVertex : RootToLeafOrder)
	{
		EVertexStatus RootStatus = VertexDatas[RootVertex].Status;
		check(RootStatus == EVertexStatus::NotStarted || RootStatus == EVertexStatus::Done);
		if (RootStatus == EVertexStatus::Done)
		{
			continue;
		}

		// For each new root vertex, graph search from it to calculate its direct edges' reachables, union those
		// reachables into its own complete list of reachables, and embed the direct edges' reachables into its
		// bigger list.
		check(Stack.IsEmpty());
		StartVertex(RootVertex, &ReachabilityBuffer, nullptr);
		while (!Stack.IsEmpty())
		{
			FStackData& StackData = Stack.Last();
			FVertex Vertex = StackData.Vertex;
			TConstArrayView<FVertex> Edges = Graph[Vertex];
			bool bPushed = false;
			while (!bPushed && StackData.NextEdge < Edges.Num())
			{
				int32 NextEdge = StackData.NextEdge++;
				FVertex EdgeVertex = Edges[NextEdge];
				if (EdgeVertex == Vertex)
				{
					continue; // Ignore edges to self
				}
				FVertexData& EdgeData = VertexDatas[EdgeVertex];
				if (EdgeData.Status == EVertexStatus::NotStarted)
				{
					FStackEdgeData& StackEdgeData = StackData.StackEdgeDatas[NextEdge];
					StartVertex(EdgeVertex, &StackEdgeData.EdgeBuffer.Buffer, &StackEdgeData.EdgeBuffer.EmbeddedSlices);
					bPushed = true;
					break;
				}
				else
				{
					// Our graph is acyclic so we should never encounter an in-progress vertex
					check(EdgeData.Status == EVertexStatus::Done);
				}
			}
			if (!bPushed)
			{
				FinishVertex();
			}
		}

		// Whenever we are unable to embed a vertex into its parent, we add it as a SeparateBuffer.
		// Now that the graph search from the root is complete, add all these SeparateBuffers onto
		// the reachabilitybuffer after the root's reachables.
		for (FSeparateBuffer& Buffer : SeparateBuffers)
		{
			RelocateViews(ReachabilityBuffer, Buffer.Buffer, Buffer.EmbeddedSlices);
		}
		SeparateBuffers.Reset();
	}
	check(SeparateBuffers.IsEmpty());

	// We're done creating ReachabilityBuffer. All of the vertices have FSliceOfArrays into it to record
	// their reachables. Shrink it now to save memory, unless we're going to reallocate it anyway later.
	if (bShrink)
	{
		ReachabilityBuffer.Shrink();
	}

	// Convert the FSliceOfArrays into TArrayViews for the output ReachabilityGraph
	FVertex* ReachabilityData = ReachabilityBuffer.GetData();
	ReachabilityGraph.Reset(NumVertices);
	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		FVertexData& VertexData = VertexDatas[Vertex];
		check(VertexData.Status == EVertexStatus::Done);
		FSliceOfArray& ReachablesSlice = VertexData.ReachablesSlice;
		check(ReachablesSlice.Buffer == &ReachabilityBuffer);
		ReachabilityGraph.Emplace(ReachablesSlice.GetView());
	}

	// Clear memory for the structures we used during the search
	Stack.Empty();
	VertexDatas.Empty();
	SeparateBuffers.Empty();
	IsReachable.Empty();
}

void FReachabilityBuilder::StartVertex(FVertex Vertex, TArray64<FVertex>* Buffer, TArray<FSliceOfArray*>* EmbeddedSlices)
{
	FStackData* StackDataPointer = Stack.GetData();
	FStackData& StackData = Stack.Emplace_GetRef();
	check(Stack.GetData() == StackDataPointer); // StackDatas have pointers to TArrays higher on the stack so we cannot let it reallocate
	StackData.Buffer = Buffer;
	StackData.EmbeddedSlices = EmbeddedSlices;
	StackData.Vertex = Vertex;
	StackData.NextEdge = 0;
	TConstArrayView<FVertex> Edges = Graph[Vertex];
	int32 NumEdges = Edges.Num();
	StackData.StackEdgeDatas.Reset(NumEdges);
	for (int32 EdgeIndex = 0; EdgeIndex < NumEdges; ++EdgeIndex)
	{
		FStackEdgeData& StackEdgeData = StackData.StackEdgeDatas.Emplace_GetRef();
		StackEdgeData.EdgeVertex = Edges[EdgeIndex];
	}
	FVertexData& VertexData = VertexDatas[Vertex];
	VertexData.Status = EVertexStatus::InProgress;
};

void FReachabilityBuilder::FinishVertex()
{
	FStackData& StackData = Stack.Last();
	FVertex StackVertex = StackData.Vertex;
	int32 NumVertices = Graph.Num();
	int32 NumEdges = StackData.StackEdgeDatas.Num();

	// Sort the edges from largest to smallest reachables count; push already allocated to the end
	// DO NOT move the FStackEdgeDatas around. Some vertices have FSliceOfArrays pointing at TArray
	// fields on the FStackEdgeData, and these pointers will be left pointing to an array with the 
	// wrong allocation if we move-assign the arrays' allocations.
	TArray<FStackEdgeData*> StackEdgeDatas;
	StackEdgeDatas.Reset(NumEdges);
	for (FStackEdgeData& StackEdgeData : StackData.StackEdgeDatas)
	{
		StackEdgeDatas.Add(&StackEdgeData);
	}
	Algo::Sort(StackEdgeDatas, [this, StackVertex](const FStackEdgeData* A, const FStackEdgeData* B)
		{
			bool bAIsEmpty = A->EdgeBuffer.Buffer.IsEmpty();
			if (bAIsEmpty != B->EdgeBuffer.Buffer.IsEmpty())
			{
				return !bAIsEmpty;
			}
			int32 NumA = VertexDatas[A->EdgeVertex].ReachablesSlice.Num;
			int32 NumB = VertexDatas[B->EdgeVertex].ReachablesSlice.Num;
			if (NumA != NumB)
			{
				return NumA > NumB;
			}
			return A->EdgeVertex < B->EdgeVertex;
		});

	// Ignoring edges that were already complete before we entered this vertex, try to embed all of the direct
	// edges' reachables into this vertex's reachables. We can only embed a set of reachables if it does not
	// overlap with reachables we have already collected. (So in a diamond graph A->B->D, A->C->D, we will only
	// be able to embed one of B or C into A). Iterate through direct edges from largest to smallest and greedily
	// embed the largest direct edge possible until no more are possible.
	
	// This is a heavy set of set operations for large vertices; we use the IsReachable array of bools to execute
	// those operations. An array of bools for all vertices has the down-side of being expensive to clear. To mitigate
	// that we initialze IsReachable to false for all vertices at the start of BuildAcyclic, and we incrementally clear
	// it back to false when we're done with it here.

	// List of leftover vertices to add on to this vertex's reachables from direct edges that we could not embed.
	TArray<FVertex> NonEmbeddedReachables;

	int32 NumReachables = 1; // Start at 1 to count this vertex as part of its own reachables
	for (FStackEdgeData* StackEdgeData : StackEdgeDatas)
	{
		FVertex EdgeVertex = StackEdgeData->EdgeVertex;
		FVertexData& EdgeData = VertexDatas[EdgeVertex];
		if (StackEdgeData->EdgeBuffer.Buffer.IsEmpty())
		{
			// The edgevertex was already complete before we entered this vertex, so we can't embed it
			StackEdgeData->bEmbeddable = false;
		}
		else
		{
			StackEdgeData->bEmbeddable = true;
			for (FVertex Vertex : EdgeData.ReachablesSlice.GetView())
			{
				if (IsReachable[Vertex])
				{
					// The edgevertex's reachables overlap with a reachable we already added; we can't embed it
					StackEdgeData->bEmbeddable = false;
					break;
				}
			}
			if (StackEdgeData->bEmbeddable)
			{
				// The edgevertex is embeddable, and its the biggest, so commit to the embedding now
				for (FVertex Vertex : EdgeData.ReachablesSlice.GetView())
				{
					IsReachable[Vertex] = true;
				}
				NumReachables += EdgeData.ReachablesSlice.Num;
			}
		}
	}

	// Gather all the unique vertices we don't already have from the non-embeddable edgevertices
	TArray<FVertex> OverlappedReachables;
	for (FStackEdgeData* StackEdgeData : StackEdgeDatas)
	{
		if (!StackEdgeData->bEmbeddable)
		{
			if (StackEdgeData->EdgeVertex == StackVertex)
			{
				continue; // Ignore edges to self
			}
			FVertex EdgeVertex = StackEdgeData->EdgeVertex;
			FVertexData& EdgeData = VertexDatas[EdgeVertex];
			for (FVertex Vertex : EdgeData.ReachablesSlice.GetView())
			{
				if (!IsReachable[Vertex])
				{
					++NumReachables;
					IsReachable[Vertex] = true;
					NonEmbeddedReachables.Add(Vertex);
				}
			}
		}
	}

	// Create our ReachablesSlice to point to the buffer we're about to populate
	int64 InitialBufferNum = StackData.Buffer->Num();
	FVertexData& VertexData = VertexDatas[StackVertex];
	VertexData.ReachablesSlice.Set(StackData.Buffer, InitialBufferNum, NumReachables);
	if (StackData.EmbeddedSlices)
	{
		StackData.EmbeddedSlices->Add(&VertexData.ReachablesSlice);
	}

	// Add all of our reachables into the buffer, copying the reachables for each embedded
	// edgevertex into a contiguous range so its ReachablesSlice can overlap our own
	ReserveGrowth(*StackData.Buffer, NumReachables);
	StackData.Buffer->Add(StackVertex);
	for (FStackEdgeData* StackEdgeData : StackEdgeDatas)
	{
		FVertex EdgeVertex = StackEdgeData->EdgeVertex;
		FVertexData& EdgeData = VertexDatas[EdgeVertex];
		if (StackEdgeData->bEmbeddable)
		{
			// Copies from the edgevertex's buffer into ours
			RelocateViews(*StackData.Buffer, StackEdgeData->EdgeBuffer.Buffer, 
				StackEdgeData->EdgeBuffer.EmbeddedSlices);
			if (StackData.EmbeddedSlices)
			{
				StackData.EmbeddedSlices->Append(StackEdgeData->EdgeBuffer.EmbeddedSlices);
			}
		}
		else if (!StackEdgeData->EdgeBuffer.Buffer.IsEmpty())
		{
			// For all of the edgevertices we recursively calculated but cannot embed, add them as separatebuffers to
			// copy onto reachabilitybuffer after the graph search stack empties.
			FSeparateBuffer* SeparateBuffersData = SeparateBuffers.GetData();
			SeparateBuffers.Add(MoveTemp(StackEdgeData->EdgeBuffer));
			check(SeparateBuffers.GetData() == SeparateBuffersData);
			FSeparateBuffer& SeparateBuffer = SeparateBuffers.Last();
			for (FSliceOfArray* EmbeddedSlice : SeparateBuffer.EmbeddedSlices)
			{
				EmbeddedSlice->Buffer = &SeparateBuffer.Buffer;
			}
		}
	}
	StackData.Buffer->Append(NonEmbeddedReachables);
	check(StackData.Buffer->Num() == InitialBufferNum + NumReachables);

	// Fast-clear IsReachable, by individually clearing all of the values we set to true
	constexpr int32 FastClearFraction = 10;
	if (((int64)NumReachables) * FastClearFraction > NumVertices)
	{
		// In this case we touched so much of IsReachable that it's faster to bulk-clear all of it
		IsReachable.Init(false, NumVertices);
	}
	else
	{
		for (FVertex Vertex : VertexData.ReachablesSlice.GetView())
		{
			IsReachable[Vertex] = false;
		}
	}

	// Mark our status complete and pop the stack
	VertexData.Status = EVertexStatus::Done;
	FStackData* StackDataPointer = Stack.GetData();
	Stack.Pop(EAllowShrinking::No);
	// StackDatas have pointers to TArrays higher on the stack so we cannot let it reallocate
	check(Stack.GetData() == StackDataPointer);
}

void FReachabilityBuilder::TransformCondensationReachabilityToInputReachability()
{
	// We walk through all CondensationGraph vertices in the ReachabilityBuffer, and replace them with their single or
	// multiple InputGraph vertices from GraphVertexToInputVertices.
	// Whenever we insert vertices in the ReachabilityGraph instead of merely replacing them, we have to update all of
	// the affected TArrayViews: any TArrayViews that started after that spot have to be shifted higher up the buffer,
	// and any TArrayViews that overlapped the insertion spot have to have their Num increased.
	// We handle the shifting by sorting the arrayviews by start point and keeping track of the cumulative increase
	// as we reach the startpoint of each arrayview. We handle the num growing by keeping a heap of arrayviews that
	// have started but not yet finished at our current position (heap-sorted by their ending position so we can pop
	// them off as we move along the buffer).

	// Sort the ArrayViews into ViewsSortedByStart
	int32 NumVertices = ReachabilityGraph.Num();
	TArray<TConstArrayView<FVertex>*> ViewsSortedByStart;
	ViewsSortedByStart.Reserve(NumVertices);
	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		ViewsSortedByStart.Add(&ReachabilityGraph[Vertex]);
	}
	Algo::Sort(ViewsSortedByStart, [](TConstArrayView<FVertex>* A, TConstArrayView<FVertex>* B)
		{
			if (A->GetData() != B->GetData())
			{
				return A->GetData() < B->GetData();
			}
			return A->Num() > B->Num();
		});
	int64 AddedSize = 0;
	for (FVertex Vertex : ReachabilityBuffer)
	{
		AddedSize += GraphVertexToInputVertices[Vertex].Num() - 1;
	}

	// The new version of ReachabilityBuffer with all of the CondensationGraph verts converted to InputGraph verts
	TArray64<FVertex> CopyBuffer;
	CopyBuffer.Reserve(AddedSize + ReachabilityBuffer.Num());

	// Start and end of the source and copy buffers' allocations
	FVertex* SourceData = ReachabilityBuffer.GetData();
	FVertex* SourceDataEnd = SourceData + ReachabilityBuffer.Num();
	FVertex* CopyData = CopyBuffer.GetData();

	// The next ArrayView from ViewsSortedByStart that we need to add when reaching its start position
	int32 NextIndexInSortedViews = 0;

	// The heap of ArrayViews that overlap the current position. The arrayview that ends next is at top of heap
	FBinaryHeap<const FVertex*, uint32> ActiveViews;

	// An array NumVertices in size that stores the CumulativeGrowth that was present when we reached the start of the
	// vertex's ArrayView. Element n corresponds to the vertex in ViewsSortedByStat[n]. We use the delta between
	// CumulativeGrowth from start to end of the ArrayView into the CondensationGraph's buffer to to detect how much
	// the ArrayView's Num needs to grow for the InputGraph's version of the ArrayView.
	TArray<int64> CumulativeGrowthWhenAdded;
	CumulativeGrowthWhenAdded.SetNumUninitialized(NumVertices);
	int64 CumulativeGrowth = 0;

	for (FVertex* NextVertex = SourceData; ; ++NextVertex)
	{
		// Remove any active ArrayViews that reached the end of their view in the source data, and update their Num
		while (!ActiveViews.IsEmpty())
		{
			uint32 IndexInSortedViewsUnsigned = ActiveViews.Top();
			const FVertex* SourceViewEnd = ActiveViews.GetKey(IndexInSortedViewsUnsigned);
			if (SourceViewEnd != NextVertex)
			{
				check(SourceViewEnd > NextVertex);
				break;
			}
			int32 IndexInSortedViews = static_cast<int32>(IndexInSortedViewsUnsigned);
			TConstArrayView<FVertex>& View = *ViewsSortedByStart[IndexInSortedViews];
			View = TConstArrayView<FVertex>(View.GetData(),
				View.Num() + CumulativeGrowth - CumulativeGrowthWhenAdded[IndexInSortedViews]);
			ActiveViews.Pop();
		}
		// write the for loop termination condition here so we can break after finalizing arrayviews at end of buffer
		if (NextVertex == SourceDataEnd)
		{
			break;
		}

		// Add any new ArrayViews from ViewsSortedByStart that start at the current position in SourceData
		while (NextIndexInSortedViews < ViewsSortedByStart.Num() &&
			ViewsSortedByStart[NextIndexInSortedViews]->GetData() == NextVertex)
		{
			TConstArrayView<FVertex>& View = *ViewsSortedByStart[NextIndexInSortedViews];
			CumulativeGrowthWhenAdded[NextIndexInSortedViews] = CumulativeGrowth;
			const FVertex* ViewSourceData = View.GetData();
			int32 ViewNum = View.Num();
			const FVertex* ViewEndData = ViewSourceData + ViewNum;
			View = TConstArrayView<FVertex>(CopyData + (ViewSourceData - SourceData) + CumulativeGrowth, ViewNum);
			ActiveViews.Add(ViewEndData, static_cast<uint32>(NextIndexInSortedViews));
			++NextIndexInSortedViews;
		}
		check(NextIndexInSortedViews == ViewsSortedByStart.Num() ||
			ViewsSortedByStart[NextIndexInSortedViews]->GetData() > SourceData);

		// Convert the CondensationGraph vertex to 1 or more Input vertices and copy them onto the CopyBuffer
		TConstArrayView<FVertex>& OutVertices = GraphVertexToInputVertices[*NextVertex];
		CopyBuffer.Append(OutVertices);
		CumulativeGrowth += OutVertices.Num() - 1;
	}
	// Swap the new buffer into our output variable
	ReachabilityBuffer = MoveTemp(CopyBuffer);
	check(ReachabilityBuffer.GetData() == CopyData);

	// Phase two: we've updated the data in the buffer, but now we need to update which vertices point to which
	// ranges of it. This phase is much easier since we just need to copy ArrayViews without modifying them.
	TArray<TConstArrayView<FVertex>> InputReachabilityGraph;
	int32 NumInputVertices = InputGraph.Num();
	InputReachabilityGraph.Reserve(NumInputVertices);
	for (FVertex InputVertex = 0; InputVertex < NumInputVertices; ++InputVertex)
	{
		FVertex GraphVertex = InputVertexToGraphVertex[InputVertex];
		InputReachabilityGraph.Add(ReachabilityGraph[GraphVertex]);
	}

	// Swap the new graph into our output variable
	ReachabilityGraph = MoveTemp(InputReachabilityGraph);
}

void FReachabilityBuilder::ReadRootToLeafOrderFromInputVertexToGraphVertex()
{
	int32 NumVertices = Graph.Num();
	RootToLeafOrder.SetNumUninitialized(NumVertices);
	for (FVertex& Vertex : RootToLeafOrder)
	{
		Vertex = InvalidVertex;
	}
	check(InputVertexToGraphVertex.Num() == NumVertices);
	for (FVertex UnsortedVertex = 0; UnsortedVertex < NumVertices; ++UnsortedVertex)
	{
		int32 SortedIndex = (int32)InputVertexToGraphVertex[UnsortedVertex];
		check(SortedIndex < NumVertices && RootToLeafOrder[SortedIndex] == InvalidVertex);
		RootToLeafOrder[SortedIndex] = UnsortedVertex;
	}
}

void FReachabilityBuilder::RelocateViews(TArray64<FVertex>& InOutTargetBuffer, TArray64<FVertex>& SourceBuffer,
	TArrayView<FSliceOfArray*> EmbeddedSlices)
{
	int64 InitialOffset = InOutTargetBuffer.Num();
	InOutTargetBuffer.Append(SourceBuffer);
	for (FSliceOfArray* EmbeddedSlize : EmbeddedSlices)
	{
		check(EmbeddedSlize->Buffer == &SourceBuffer);
		EmbeddedSlize->Buffer = &InOutTargetBuffer;
		EmbeddedSlize->Offset += InitialOffset;
	}
}

} // namespace Algo::Graph



#if WITH_DEV_AUTOMATION_TESTS
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReachabilityTest, "System.Core.Algo.ReachabilityGraph", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FReachabilityTest::RunTest(const FString& Parameters)
{
	using namespace Algo::Graph;

	TArray<TArray<FVertex>> Buffer;
	TArray<TConstArrayView<FVertex>> Graph;
	TArray64<FVertex> ReachabilityBuffer;
	TArray<TConstArrayView<FVertex>> Reachability;
	TArray<TArray<FVertex>> ExpectedReachability;
	auto AddVertex = [&Buffer, &Graph, &ExpectedReachability](FVertex Vertex, TConstArrayView<FVertex> Edges, TConstArrayView<FVertex> ExpectedVertexReachability)
	{
		if (Graph.Num() <= Vertex)
		{
			Graph.SetNum(Vertex + 1);
			Buffer.SetNum(Vertex + 1);
			ExpectedReachability.SetNum(Vertex + 1);
		}
		Buffer[Vertex] = Edges;
		Graph[Vertex] = Buffer[Vertex];
		ExpectedReachability[Vertex] = ExpectedVertexReachability;
	};
	auto Clear = [&Buffer, &Graph, &ExpectedReachability]()
	{
		Buffer.Reset();
		Graph.Reset();
		ExpectedReachability.Reset();
	};
	auto WriteArrayToString = [](TConstArrayView<FVertex> A)
	{
		TStringBuilder<256> Writer;
		Writer << TEXT("[");
		Writer.Join(A, TEXT(','));
		Writer << TEXT("]");
		return FString(Writer);
	};
	auto ConfirmResults = [&Reachability, &ExpectedReachability, &WriteArrayToString](FString& OutError)
	{
		int32 NumVertices = Reachability.Num();
		if (NumVertices != ExpectedReachability.Num())
		{
			OutError = FString::Printf(TEXT("reachability graph has incorrect number of vertices, Expected=%d, Actual=%d"),
				ExpectedReachability.Num(), NumVertices);
			return false;
		}
		for (int32 Vertex = 0; Vertex < NumVertices; ++Vertex)
		{
			TArray<FVertex> Expected = ExpectedReachability[Vertex];
			TArray<FVertex> Actual(Reachability[Vertex]);
			Algo::Sort(Expected);
			Algo::Sort(Actual);
			if (Expected != Actual)
			{
				OutError = FString::Printf(TEXT("reachability for vertex %d does not match. Expected=%s, Actual=%s"),
					Vertex, *WriteArrayToString(Expected), *WriteArrayToString(Actual));
				return false;
			}
		}
		return true;
	};
	auto RunTrialsForCase = [&ConfirmResults, &Graph, &ReachabilityBuffer, &Reachability, this](const TCHAR* TestCaseName)
	{
		FString Error;
		{
			FReachabilityBuilder Builder(Graph, ReachabilityBuffer, Reachability);
			Builder.Build();
		}
		if (!ConfirmResults(Error))
		{
			AddError(FString::Printf(TEXT("Search failed for case \"%s\": %s."), TestCaseName, *Error));
		}
	};

	{
		Clear();
		AddVertex(0, { }, { 0 });
		AddVertex(1, { 0 }, { 0,1 });
		AddVertex(2, { 1 }, { 0,1,2 });
		RunTrialsForCase(TEXT("Each node depends on the previous one"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, {0,1,2});
		AddVertex(1, { 2 }, { 1,2 });
		AddVertex(2, { }, { 2 });
		RunTrialsForCase(TEXT("Each node depends on the next one"));
	}
	{
		Clear();
		AddVertex(0, { 0 }, { 0 });
		AddVertex(1, { 0,1 }, { 0,1 });
		AddVertex(2, { 1,2 }, { 0,1,2 });
		RunTrialsForCase(TEXT("SelfReferences"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, { 0,1 });
		AddVertex(1, { 0 }, { 0,1 });
		RunTrialsForCase(TEXT("Simple cycle"));
	}
	{
		Clear();
		for (FVertex Vertex = 0; Vertex < 10; ++Vertex)
		{
			if (Vertex != 5)
			{
				AddVertex(Vertex, { 5 }, { Vertex, 5 });
			}
		}
		AddVertex(5, { }, { 5 });
		RunTrialsForCase(TEXT("Every node has a dependency on a single node"));
	}
	{
		//              6
		//             / \
		//            5   7
		//           / \   \
		//          0   1   8
		//           \ / \   \
		//            3   4   |
		//             \   \ /
		//              \   9
		//               \ /
		//                2       
		Clear();
		AddVertex(6, { 5,7 }, { 0,1,2,3,4,5,6,7,8,9 });
		AddVertex(5, { 0,1 }, { 5,0,1,3,4,9,2 });
		AddVertex(7, { 8 }, { 7,8,9,2 });
		AddVertex(0, { 3 }, { 0,3,2 });
		AddVertex(1, { 3,4 }, { 1,3,4,9,2 });
		AddVertex(8, { 9 }, { 8,9,2 });
		AddVertex(3, { 2 }, { 3,2 });
		AddVertex(4, { 9 }, { 4,9,2 });
		AddVertex(9, { 2 }, { 2,9 });
		AddVertex(2, { }, { 2 });
		RunTrialsForCase(TEXT("SketchedOutExample1"));
	}
	{
		//            0
		//      1          2
		//   3     4     5     6
		//  7  8  9 10 11 12 13 14
		Clear();
		AddVertex(0, { 1,2 }, { 0,1,2,3,4,5,6,7,8,9,10,11,12,13,14 });
		AddVertex(1, { 3,4 }, { 1,3,4,7,8,9,10 });
		AddVertex(2, { 5,6 }, { 2,5,6,11,12,13,14 });
		AddVertex(3, { 7,8 }, { 3,7,8 });
		AddVertex(4, { 9,10 }, { 4,9,10 });
		AddVertex(5, { 11,12 }, { 5,11,12 });
		AddVertex(6, { 13,14 }, { 6,13,14 });
		for (FVertex Vertex = 7; Vertex <= 14; ++Vertex)
		{
			AddVertex(Vertex, { }, { Vertex });
		}
		RunTrialsForCase(TEXT("BinaryTree"));
	}
	{
		//            0
		//      1          2
		//   3     4     5     6    // 3 through 6 have edges to every vertex in 7 through 14
		//  7  8  9 10 11 12 13 14
		Clear();
		AddVertex(0, { 1,2 }, { 0,1,2,3,4,5,6,7,8,9,10,11,12,13,14 });
		AddVertex(1, { 3,4 }, { 1,3,4,7,8,9,10,11,12,13,14 });
		AddVertex(2, { 5,6 }, { 2,5,6,7,8,9,10,11,12,13,14 });
		AddVertex(3, { 7,8,9,10,11,12,13,14 }, { 3,7,8,9,10,11,12,13,14 });
		AddVertex(4, { 7,8,9,10,11,12,13,14 }, { 4,7,8,9,10,11,12,13,14 });
		AddVertex(5, { 7,8,9,10,11,12,13,14 }, { 5,7,8,9,10,11,12,13,14 });
		AddVertex(6, { 7,8,9,10,11,12,13,14 }, { 6,7,8,9,10,11,12,13,14 });
		for (FVertex Vertex = 7; Vertex <= 14; ++Vertex)
		{
			AddVertex(Vertex, { }, { Vertex });
		}
		RunTrialsForCase(TEXT("BinaryTreeWithEverythingFromRow2ToRow3"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, { 0,1,2,3 });
		AddVertex(1, { 0,2 }, { 0,1,2,3 });
		AddVertex(2, { 3 }, { 2,3 });
		AddVertex(3, { }, { 3 });
		RunTrialsForCase(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts"));
	}
	{
		Clear();
		AddVertex(0, { }, { 0 });
		AddVertex(1, { 0 }, { 0,1 });
		AddVertex(2, { 1,3 }, { 0,1,2,3 });
		AddVertex(3, { 2 }, { 0,1,2,3 });
		RunTrialsForCase(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts, submitted in reverse"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, { 0,1,2,3 });
		AddVertex(1, { 2 }, { 1,2,3 });
		AddVertex(2, { 3 }, { 2,3 });
		AddVertex(3, { 2 }, { 2,3 });
		RunTrialsForCase(TEXT("Cycle at a leaf and a chain from the root depending on that cycle"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, { 0,1 });
		AddVertex(1, { 0 }, { 0,1 });
		AddVertex(2, { 1 }, { 2,1,0 });
		AddVertex(3, { 2 }, { 3,2,1,0 });
		RunTrialsForCase(TEXT("Cycle in the root and with the root cycle depending on a chain of non-cycle verts"));
	}
	{
		Clear();
		AddVertex(0, { 1 }, { 0,1,2,3 });
		AddVertex(1, { 2,3 }, { 1,2,3 });
		AddVertex(2, { 1,3 }, { 1,2,3 });
		AddVertex(3, { 1,2 }, { 1,2,3 });
		RunTrialsForCase(TEXT("Vertex dependent upon a cycle"));
	}
	{
		// 0 -> 1 -> 2 -> 3 -> 1
		//           |
		//           v
		//      4 -> 5 -> 6 -> 4
		Clear();
		AddVertex(0, { 1 }, { 0,1,2,3,4,5,6 });
		AddVertex(1, { 2 }, { 1,2,3,4,5,6 });
		AddVertex(2, { 3,5 }, { 1,2,3,4,5,6 });
		AddVertex(3, { 1 }, { 1,2,3,4,5,6 });
		AddVertex(4, { 5 }, { 4,5,6 });
		AddVertex(5, { 6 }, { 4,5,6 });
		AddVertex(6, { 4 }, { 4,5,6 });
		RunTrialsForCase(TEXT("One cycle dependent upon another"));
	}
	{
		// 5 -> 0 -> (1 -> 2 -> 1)
		//      |          |
		//      |          v
		//      |          5
		//      v
		//     (3 -> 4 -> 3)
		Clear();
		AddVertex(0, { 1,3 }, { 0,1,2,3,4,5 });
		AddVertex(1, { 2 }, { 0,1,2,3,4,5 });
		AddVertex(2, { 1,5 }, { 0,1,2,3,4,5 });
		AddVertex(3, { 4 }, { 3,4 });
		AddVertex(4, { 3 }, { 3,4 });
		AddVertex(5, { 0 }, { 0,1,2,3,4,5 });
		RunTrialsForCase(TEXT("MutuallyReachableSet Problem1"));
	}
	{
		Clear();
		TArray<FVertex> All;
		TArray<FVertex> AllButItself;
		for (FVertex Vertex = 0; Vertex < 6; ++Vertex)
		{
			All.Add(Vertex);
			AllButItself.Add(Vertex);
		}
		for (FVertex Vertex = 0; Vertex < 6; ++Vertex)
		{
			AllButItself.Remove(Vertex);
			AddVertex(Vertex, AllButItself, All);
			AllButItself.Add(Vertex);
		}
		RunTrialsForCase(TEXT("FullyConnectedGraph"));
	}

	return true;
}
#endif