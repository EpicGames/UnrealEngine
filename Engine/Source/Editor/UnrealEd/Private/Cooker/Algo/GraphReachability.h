// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphConvert.h"

namespace Algo::Graph
{

/**
 * Calculates the reachability graph for the given input graph: an edge exists in the reachability graph from vertex A
 * to vertex B if and only if B is reachable from A by following edges in the input graph.
 * 
 * @param InGraph The input graph
 * @param OutReachabilityGraphBuffer Buffer that holds memory for the output graph. It must not be deallocated or
 *        modified until OutReachabilityGraph is no longer referenced.
 * @param OutReachabilityGraph The reachability graph of the input graph. Note that it is in Graph Form but is
 *        unnormalized: edges for each vertex are arbitrarily ordered according to internal calculations to 
 *        reduce memory use.
 */
void ConstructReachabilityGraph(TConstArrayView<TConstArrayView<FVertex>> InGraph, TArray64<FVertex>& OutReachabilityGraphBuffer,
	TArray<TConstArrayView<FVertex>>& OutGraph);

}