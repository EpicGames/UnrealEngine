// Copyright Epic Games, Inc. All Rights Reserved.

#include "Packing.h"

#include "Algo/Sort.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "SortSpecialCases.h"

namespace Algo
{

void ScheduleValues(TConstArrayView<int32> Values, int32 NumBuckets, TConstArrayView<TArray<int32>> ExclusionGroups,
	TArray<TArray<int32>>& OutBucketsIndices)
{
	// Set the size of the output array
	int32 NumValues = Values.Num();
	TArray<TArray<int32>>& Buckets = OutBucketsIndices;
	check(NumBuckets > 0);
	Buckets.SetNum(NumBuckets, EAllowShrinking::No);
	for (TArray<int32>& BucketIndices : Buckets)
	{
		BucketIndices.Reset();
	}

	// Verify exclusion groups are valid; invalid exclusiongroups will cause us to be unable to find any packing
	for (const TArray<int32>& ExclusionGroup : ExclusionGroups)
	{
		check(ExclusionGroup.Num() <= NumBuckets);
	}

	// Handle the trivial case
	if (NumBuckets == 1)
	{
		Buckets[0].Append(RangeArray<TArray<int32>>(0, NumValues));
		return;
	}

	// Reverse the information in exclusiongroups so we can quickly find which exclusiongroup a valueindex is in
	TMap<int32, int32> ExclusionGroupOfValueIndex;
	int32 NumExclusionGroups = ExclusionGroups.Num();
	for (int32 ExclusionGroupIndex = 0; ExclusionGroupIndex < NumExclusionGroups; ++ExclusionGroupIndex)
	{
		for (int32 ValueIndex : ExclusionGroups[ExclusionGroupIndex])
		{
			ExclusionGroupOfValueIndex.FindOrAdd(ValueIndex) = ExclusionGroupIndex;
		}
	}

	// Initialize remaining values to the entire list of values, and sort them into the order in which we should
	// try to place them.
	TArray<int32> RemainingValueIndices = RangeArray<TArray<int32>>(0, NumValues);

	auto ValueIndexSelectionOrder = [&Values, &ExclusionGroupOfValueIndex](int32 AValueIndex, int32 BValueIndex)
	{
		int32* AExclusionGroup = ExclusionGroupOfValueIndex.Find(AValueIndex);
		int32* BExclusionGroup = ExclusionGroupOfValueIndex.Find(BValueIndex);
		if ((AExclusionGroup != nullptr) != (BExclusionGroup != nullptr))
		{
			// Place values in an exclusiongroup into a bucket before placing unconstrained values
			return AExclusionGroup != nullptr;
		}
		if (AExclusionGroup != BExclusionGroup)
		{
			// Place all members of one exclusiongroup before starting the next exclusiongroup
			return AExclusionGroup < BExclusionGroup;
		}
		// Place bigger pieces before smaller pieces
		return Values[AValueIndex] > Values[BValueIndex];
	};
	Algo::Sort(RemainingValueIndices, ValueIndexSelectionOrder);

	// After every piece we place we need to resort the modified smallest bucket back into the list and find the new
	// smallest. FBucketData is our sortable information about a bucket
	struct FBucketData
	{
		TArray<int32>* Bucket = nullptr;
		int32 Size = 0;
		TBitArray<> HasExclusionGroup;
	};
	TArray<FBucketData> SortedBuckets;
	SortedBuckets.Reserve(NumBuckets);
	for (int32 BucketIndex = 0; BucketIndex < NumBuckets; ++BucketIndex)
	{
		FBucketData& BucketData = SortedBuckets.Emplace_GetRef();
		BucketData.Bucket = &Buckets[BucketIndex];
		BucketData.Size = 0;
		BucketData.HasExclusionGroup.Init(false, NumExclusionGroups);
	}
	auto SortBucketDataDescending = [](const FBucketData& A, const FBucketData& B)
	{
		if (A.Size != B.Size)
		{
			return A.Size > B.Size;
		}
		// When buckets are equal, prefer to add to the earlier buckets, by pushing them to the back of the list
		return A.Bucket > B.Bucket;
	};
	Algo::Sort(SortedBuckets, SortBucketDataDescending);

	// Main loop: iterate over each piece and put it into the current smallest bucket (or the smallest valid bucket
	// if the piece is in an exclusion group)
	for (int32 ValueIndex : RemainingValueIndices)
	{
		int32* ExclusionGroupIndex = ExclusionGroupOfValueIndex.Find(ValueIndex);
		int32 BestBucketIndex = INDEX_NONE;
		if (!ExclusionGroupIndex)
		{
			BestBucketIndex = NumBuckets - 1;
		}
		else
		{
			for (int32 BucketIndex = NumBuckets - 1; BucketIndex >= 0; --BucketIndex)
			{
				FBucketData& Bucket = SortedBuckets[BucketIndex];
				if (!Bucket.HasExclusionGroup[*ExclusionGroupIndex])
				{
					Bucket.HasExclusionGroup[*ExclusionGroupIndex] = true;
					BestBucketIndex = BucketIndex;
					break;
				}
			}
			// exclusiongroups are <= NumBuckets in size so we always have a free bucket for an exclusiongroup member
			check(BestBucketIndex != INDEX_NONE);
		}

		FBucketData& SmallestBucket = SortedBuckets[BestBucketIndex];
		SmallestBucket.Bucket->Add(ValueIndex);
		SmallestBucket.Size += Values[ValueIndex];

		RestoreSort(SortedBuckets, NumBuckets-1, SortBucketDataDescending);
	}
}

}

