// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"

namespace Algo
{

/**
 * Assign labeled values to N buckets such that the sum of the values in the maximum bucket is minimized.
 * 
 * @param Values Range of values to schedule.
 * @param NumBuckets How many buckets to divide the values into.
 * @param ExclusionGroups Each element is a range of indexes into Values. Each of the Values in an ExclusionGroup will
 *        be sorted into a different bucket. ExclusionGroups must not overlap, and the size of each ExclusionGroup
 *        must be <= the number of buckets.
 * @param OutBucketsIndices Filled with the output buckets; each element is a range of unsorted indices into Values.
 */
void ScheduleValues(TConstArrayView<int32> Values, int32 NumBuckets, TConstArrayView<TArray<int32>> ExclusionGroups,
	TArray<TArray<int32>>& OutBucketsIndices);

}

