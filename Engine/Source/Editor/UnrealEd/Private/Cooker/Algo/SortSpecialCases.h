// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h"
#include "Traits/ElementType.h"

namespace Algo
{

/** Given two ranges each of which is sorted and unique, calculate the sorted and unique Union */
template <typename RangeTypeA, typename RangeTypeB, typename RangeTypeMerged, typename PredicateType>
inline void MergeUniqueSortedRanges(RangeTypeA&& A, RangeTypeB&& B, RangeTypeMerged& OutMerged,
	PredicateType Pred)
{
	typedef TElementType_T<RangeTypeA> ElementType;
	typedef decltype(GetNum(A)) IndexType;
	
	IndexType ANum = GetNum(A);
	IndexType BNum = GetNum(B);
	OutMerged.Reset(ANum >= BNum ? BNum : ANum);
	const ElementType* AData = GetData(A);
	const ElementType* BData = GetData(B);
	const ElementType* AEnd = AData + ANum;
	const ElementType* BEnd = BData + BNum;
	while (AData < AEnd && BData < BEnd)
	{
		if (Pred(*AData, *BData))
		{
			OutMerged.Add(*(AData++));
		}
		else if (Pred(*BData, *AData))
		{
			OutMerged.Add(*(BData++));
		}
		else
		{
			OutMerged.Add(*(AData++));
			++BData;
		}
	}
	while (AData < AEnd)
	{
		OutMerged.Add(*(AData++));
	}
	while (BData < BEnd)
	{
		OutMerged.Add(*(BData++));
	}
};

/** Given two ranges each of which is sorted and unique, calculate the sorted and unique Union */
template <typename RangeTypeA, typename RangeTypeB, typename RangeTypeMerged>
inline void MergeUniqueSortedRanges(RangeTypeA&& A, RangeTypeB&& B, RangeTypeMerged& OutMerged)
{
	MergeUniqueSortedRanges(Forward<RangeTypeA>(A), Forward<RangeTypeB>(B), OutMerged,
		TLess<TElementType_T<RangeTypeA>>());
}

/** Given two ranges each of which is sorted and unique, calculate the sorted and unique Union */
template <typename RangeTypeMerged, typename RangeTypeA, typename RangeTypeB, typename PredicateType>
inline RangeTypeMerged MergeUniqueSortedRanges(RangeTypeA&& A, RangeTypeB&& B, PredicateType Pred)
{
	RangeTypeMerged OutMerged;
	MergeUniqueSortedRanges(Forward<RangeTypeA>(A), Forward<RangeTypeB>(B), OutMerged, MoveTemp(Pred));
	return OutMerged;
}

/** Given two ranges each of which is sorted and unique, calculate the sorted and unique Union */
template <typename RangeTypeMerged, typename RangeTypeA, typename RangeTypeB>
inline RangeTypeMerged MergeUniqueSortedRanges(RangeTypeA&& A, RangeTypeB&& B)
{
	RangeTypeMerged OutMerged;
	MergeUniqueSortedRanges(Forward<RangeTypeA>(A), Forward<RangeTypeB>(B), OutMerged,
		TLess<TElementType_T<RangeTypeA>>());
	return OutMerged;
}

/**
 * Given an array that is sorted except for the element at the ModifiedIndex, shift the
 * modified element into its correct sorted location.
 */
template <typename RangeType, typename ModifiedIndexType, typename PredicateType>
inline void RestoreSort(RangeType&& Range, ModifiedIndexType ModifiedIndex, PredicateType Pred)
{
	typedef TElementType_T<RangeType> ElementType;
	typedef decltype(GetNum(Range)) IndexType;
	typedef TArrayView<ElementType, IndexType> ArrayViewType;

	ElementType* RangeData = GetData(Range);
	IndexType RangeNum = GetNum(Range);
	ArrayViewType RangeArray(RangeData, RangeNum);
	ElementType& Modified = RangeData[ModifiedIndex];
	if (ModifiedIndex > 0 && Pred(Modified, RangeData[ModifiedIndex - 1]))
	{
		ArrayViewType LowerRange = RangeArray.Left(ModifiedIndex - 1);
		IndexType DesiredIndex = Algo::UpperBound(LowerRange, Modified, Pred);
		ElementType Swap = MoveTemp(Modified);
		for (IndexType SwapFromIndex = ModifiedIndex - 1; SwapFromIndex >= DesiredIndex; --SwapFromIndex)
		{
			RangeData[SwapFromIndex + 1] = MoveTemp(RangeData[SwapFromIndex]);
		}
		RangeData[DesiredIndex] = MoveTemp(Swap);
	}
	else if (ModifiedIndex < RangeNum - 1 && Pred(RangeData[ModifiedIndex + 1], Modified))
	{
		ArrayViewType UpperRange = RangeArray.RightChop(ModifiedIndex + 1);
		IndexType DesiredIndex = ModifiedIndex + 1 + Algo::LowerBound(UpperRange, Modified, Pred);
		ElementType Swap = MoveTemp(Modified);
		for (IndexType SwapFromIndex = ModifiedIndex + 1; SwapFromIndex <= DesiredIndex; ++SwapFromIndex)
		{
			RangeData[SwapFromIndex - 1] = MoveTemp(RangeData[SwapFromIndex]);
		}
		RangeData[DesiredIndex] = MoveTemp(Swap);
	}
}

/** Construct and return a range containing consecutive integers in the range [Min, MaxPlusOne) */
template <typename RangeType, typename IntType>
RangeType RangeArray(IntType Min, IntType MaxPlusOne)
{
	check(MaxPlusOne >= Min);
	RangeType Result;
	Result.Reserve(MaxPlusOne - Min);
	for (IntType Value = Min; Value < MaxPlusOne; ++Value)
	{
		Result.Add(Value);
	}
	return Result;
}

}
