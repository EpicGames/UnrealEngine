// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncIODelete.h"

#include "Async/Async.h"
#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookOnTheFlyServer.h" // needed for DECLARE_LOG_CATEGORY_EXTERN(LogCook,...)
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "Math/NumericLimits.h"
#include "Misc/StringBuilder.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Templates/UnrealTemplate.h"

#if WITH_ASYNCIODELETE_DEBUG
TArray<FString> FAsyncIODelete::AllTempRoots;
#endif

FAsyncIODelete::FAsyncIODelete(const FStringView& InOwnedTempRoot)
{
	SetTempRoot(InOwnedTempRoot);
}

FAsyncIODelete::~FAsyncIODelete()
{
	SetTempRoot(FStringView());
}

void FAsyncIODelete::SetTempRoot(FStringView InSharedTempRoot)
{
	Teardown();

#if WITH_ASYNCIODELETE_DEBUG
	if (!SharedTempRoot.IsEmpty())
	{
		RemoveTempRoot(SharedTempRoot);
	}
#endif

	SharedTempRoot = InSharedTempRoot;

#if WITH_ASYNCIODELETE_DEBUG
	if (!SharedTempRoot.IsEmpty())
	{
		AddTempRoot(*SharedTempRoot);
	}
#endif
}

void FAsyncIODelete::SetDeletesPaused(bool bInPaused)
{
	bPaused = bInPaused;
	if (AsyncEnabled())
	{
		if (!bPaused)
		{
			IFileManager& FileManager = IFileManager::Get();
			for (const FString& DeletePath : PausedDeletes)
			{
				const bool IsDirectory = FileManager.DirectoryExists(*DeletePath);
				const bool IsFile = !IsDirectory && FileManager.FileExists(*DeletePath);
				if (!IsDirectory && !IsFile)
				{
					continue;
				}
				CreateDeleteTask(DeletePath, IsDirectory ? EPathType::Directory : EPathType::File);
			}
			PausedDeletes.Empty();
		}
	}
}

void FAsyncIODelete::Setup()
{
	if (bInitialized)
	{
		return;
	}

	bInitialized = true;
	if (SharedTempRoot.IsEmpty())
	{
		checkf(false, TEXT("DeleteDirectory called without having first set a TempRoot"));
		return;
	}

	if (AsyncEnabled())
	{
		// Create the new root and at the same time clear the results from any previous process using the same
		// TempRoot that did not shut down cleanly
		TArray<FDeleteRequest> OrphanedRootsToDelete;
		if (!TryPurgeOldAndCreateRoot(true, OrphanedRootsToDelete))
		{
			// TryPurgeOldAndCreateRoot logged the warning
			return;
		}

		// Allocate the task event
		check(TasksComplete == nullptr);
		TasksComplete = FPlatformProcess::GetSynchEventFromPool(true /* IsManualReset */);
		check(ActiveTaskCount == 0);
		TasksComplete->Trigger(); // We have 0 tasks so the event should be in the Triggered state

		// Assert that all other teardown-transient variables were cleared by the constructor or by the previous teardown
		// TempRoot and bPaused are preserved across setup/teardown and may have any value
		check(PausedDeletes.Num() == 0);
		check(DeleteCounter == OrphanedRootsToDelete.Num()); // We should have started with DeleteCounter==0, and incremented it for each OrphanedRootToDelete request
		bAsyncInitialized = true;

		// Queue the async deletion of any discovered orphaned roots
		for (const FDeleteRequest& DeleteRequest : OrphanedRootsToDelete)
		{
			if (!DeleteRequest.Path.IsEmpty())
			{
				CreateDeleteTask(DeleteRequest.Path, DeleteRequest.PathType);
			}
		}
	}
}

void FAsyncIODelete::Teardown()
{
	if (!bInitialized)
	{
		return;
	}

	if (bAsyncInitialized)
	{
		// Clear task variables
		WaitForAllTasks();
		check(ActiveTaskCount == 0 && TasksComplete != nullptr && TasksComplete->Wait(0));
		FPlatformProcess::ReturnSynchEventToPool(TasksComplete);
		TasksComplete = nullptr;

		// Remove the temp directory from disk
		TArray<FDeleteRequest> OrphanedRootsToDelete;
		TryPurgeOldAndCreateRoot(false, OrphanedRootsToDelete);
		check(OrphanedRootsToDelete.IsEmpty()); // Should not be populated when bCreateRoot is false

		// Clear delete variables; we don't need to run the tasks for the remaining pauseddeletes because synchronously deleting the temp directory above did the work they were going to do
		PausedDeletes.Empty();
		DeleteCounter = 0;
		bAsyncInitialized = false;
	}
	TempRoot.Reset();

	// We are now torn down and ready for a new setup
	bInitialized = false;
}

bool FAsyncIODelete::WaitForAllTasks(float TimeLimitSeconds)
{
	if (!bAsyncInitialized)
	{
		return true;
	}

	if (TimeLimitSeconds <= 0.f)
	{
		TasksComplete->Wait();
	}
	else
	{
		if (!TasksComplete->Wait(FTimespan::FromSeconds(TimeLimitSeconds)))
		{
			return false;
		}
	}
	check(ActiveTaskCount == 0);
	return true;
}

bool FAsyncIODelete::AsyncEnabled()
{
#if PLATFORM_LINUX
	// Temporarily disable Async on Linux until we have fixed TryPurgeOldAndCreateRoot to request exclusive-write behavior explicitly via calls to flock
	return false;
#else
	return FPlatformMisc::SupportsMultithreadedFileHandles();
#endif
}


bool FAsyncIODelete::Delete(const FStringView& PathToDelete, EPathType ExpectedType)
{
	IFileManager& FileManager = IFileManager::Get();
	FString PathToDeleteStr(PathToDelete);

	const bool IsDirectory = FileManager.DirectoryExists(*PathToDeleteStr);
	const bool IsFile = !IsDirectory && FileManager.FileExists(*PathToDeleteStr);
	if (!IsDirectory && !IsFile)
	{
		return true;
	}
	if (ExpectedType == EPathType::Directory && !IsDirectory)
	{
		checkf(false, TEXT("DeleteDirectory called on \"%.*s\" which is not a directory."), PathToDelete.Len(), PathToDelete.GetData());
		return false;
	}
	if (ExpectedType == EPathType::File && !IsFile)
	{
		checkf(false, TEXT("DeleteFile called on \"%.*s\" which is not a file."), PathToDelete.Len(), PathToDelete.GetData());
		return false;
	}

	if (bAsyncInitialized)
	{
		if (DeleteCounter == UINT32_MAX)
		{
			Teardown();
		}
	}
	Setup();
	// Prevent the user from trying to delete our temproot or anything inside it
	if (!SharedTempRoot.IsEmpty() &&
		(FPaths::IsUnderDirectory(PathToDeleteStr, SharedTempRoot) || FPaths::IsUnderDirectory(SharedTempRoot, PathToDeleteStr)))
	{
		return false;
	}
	if (bAsyncInitialized)
	{
		const FString TempPath = FPaths::Combine(TempRoot, FString::Printf(TEXT("%u"), DeleteCounter));
		DeleteCounter++;

		const bool bReplace = true;
		const bool bEvenIfReadOnly = true;
		const bool bMoveAttributes = false;
		const bool bDoNotRetryOnError = true;
		if (!IFileManager::Get().Move(*TempPath, *PathToDeleteStr, bReplace, bEvenIfReadOnly, bMoveAttributes, bDoNotRetryOnError)) // IFileManager::Move works on either files or directories
		{
			// The move failed; try a synchronous delete as backup
			UE_LOGF(LogCook, Warning, "Failed to move path '%.*ls' for async delete (LastError == %i); falling back to synchronous delete.", PathToDelete.Len(), PathToDelete.GetData(), FPlatformMisc::GetLastError());
			return SynchronousDelete(*PathToDeleteStr, ExpectedType);
		}

		if (bPaused)
		{
			PausedDeletes.Add(TempPath);
		}
		else
		{
			CreateDeleteTask(TempPath, ExpectedType);
		}
		return true;
	}
	else
	{
		return SynchronousDelete(*PathToDeleteStr, ExpectedType);
	}
}

void FAsyncIODelete::CreateDeleteTask(const FStringView& InDeletePath, EPathType PathType)
{
	{
		FScopeLock Lock(&CriticalSection);
		TasksComplete->Reset();
		ActiveTaskCount++;
	}

	AsyncThread(
		[this, DeletePath = FString(InDeletePath), PathType]() { SynchronousDelete(*DeletePath, PathType); },
		0, TPri_Normal,
		[this]() { OnTaskComplete(); });
}

void FAsyncIODelete::OnTaskComplete()
{
	FScopeLock Lock(&CriticalSection);
	check(ActiveTaskCount > 0);
	ActiveTaskCount--;
	if (ActiveTaskCount == 0)
	{
		TasksComplete->Trigger();
	}
}

bool FAsyncIODelete::SynchronousDelete(const TCHAR* InDeletePath, EPathType PathType)
{
	bool Result;
	const bool bRequireExists = false;
	if (PathType == EPathType::Directory)
	{
		const bool bTree = true;
		Result = IFileManager::Get().DeleteDirectory(InDeletePath, bRequireExists, bTree);
	}
	else
	{
		const bool bEvenIfReadOnly = true;
		Result = IFileManager::Get().Delete(InDeletePath, bRequireExists, bEvenIfReadOnly);
	}

	if (!Result)
	{
		UE_LOGF(LogCook, Warning, "Failed to asyncdelete %ls '%ls'. LastError == %i.", PathType == EPathType::Directory ? TEXT("directory") : TEXT("file"), InDeletePath, FPlatformMisc::GetLastError());
	}
	return Result;
}

FStringView FAsyncIODelete::GetLockSuffix()
{
	return TEXTVIEW(".lock");
}

constexpr float MaxWaitSecondsForLockDefault = 5.0f;
constexpr float SleepSecondsForLock = 0.01f;
namespace UE::AsyncIODelete::Private
{
	float MaxWaitSecondsForLock = MaxWaitSecondsForLockDefault;
}

void FAsyncIODelete::SetMaxWaitSecondsForLock(float MaxWaitTimeSeconds)
{
	if (MaxWaitTimeSeconds < 0)
	{
		MaxWaitTimeSeconds = MaxWaitSecondsForLockDefault;
	}
	UE::AsyncIODelete::Private::MaxWaitSecondsForLock = MaxWaitTimeSeconds;
}

bool FAsyncIODelete::TryPurgeOldAndCreateRoot(bool bCreateRoot, TArray<FDeleteRequest>& OutOrphanedRootsToDelete)
{
	check(!SharedTempRoot.IsEmpty());
	check(bCreateRoot == TempRoot.IsEmpty()); // We should only call TryPurgeOldAndCreateRoot(true) during setup and (false) during Teardown
	IFileManager& FileManager = IFileManager::Get();

	FString ParentDir = FPaths::GetPath(SharedTempRoot);
	if (!FileManager.DirectoryExists(*ParentDir))
	{
		if (bCreateRoot)
		{
			if (!FileManager.MakeDirectory(*ParentDir, true /* Tree */) && !FileManager.DirectoryExists(*ParentDir))
			{
				UE_LOGF(LogCook, Error, "Could not create AsyncIoDelete parent directory %ls. LastError: %d. Falling back to synchronous delete.",
					*ParentDir, FPlatformMisc::GetLastError());
				return false;
			}
		}
		else
		{
			// If parent directory doesn't exist when we are shutting down, do not create it
			return true;
		}
	}

	bool bDirectoryEmpty = true;
	if (!TempRoot.IsEmpty())
	{
		// Our directory is empty because we waited on all of the async delete tasks, so delete it synchronously
		if (!FileManager.DeleteDirectory(*TempRoot, false /* bRequireExists */, true /* Tree */) &&
			FileManager.DirectoryExists(*TempRoot))
		{
			UE_LOGF(LogCook, Display, "AsyncIoDelete could not clean up its root %ls. LastError: %d.",
				*TempRoot, FPlatformMisc::GetLastError());
			bDirectoryEmpty = false;
		}
	}

	// Temporarily lock the SharedTempRoot while we are querying the directory and need to be inside a machine-wide critical section
	FStringView LockSuffix = GetLockSuffix();
	FString SharedTempRootLockFileName = SharedTempRoot + LockSuffix;
	TUniquePtr<FArchive> SharedTempRootLockFile = nullptr;
	double StartTimeSeconds = FPlatformTime::Seconds();
	do
	{
		SharedTempRootLockFile.Reset(FileManager.CreateFileWriter(*SharedTempRootLockFileName));
		if (!SharedTempRootLockFile)
		{
			if (FPlatformTime::Seconds() - StartTimeSeconds > UE::AsyncIODelete::Private::MaxWaitSecondsForLock)
			{
				if (bCreateRoot)
				{
					UE_LOGF(LogCook, Error, "AsyncIoDelete could not create LockFile %ls. Falling back to synchronous delete.",
						*SharedTempRootLockFileName);
				}
				else
				{
					UE_LOGF(LogCook, Display, "AsyncIoDelete could not clean up its root %ls, because another process has LockFile %ls locked. LastError: %d.",
						*TempRoot, *SharedTempRootLockFileName, FPlatformMisc::GetLastError());
					TempRootLockFile.Reset(); // Drop our lock that prevents the lockfile from being deleted, so the next AsyncIODelete can clean up after us.
				}
				return false;
			}
			FPlatformProcess::Sleep(SleepSecondsForLock);
		}
	} while (!SharedTempRootLockFile);
	ON_SCOPE_EXIT
	{
		SharedTempRootLockFile.Reset();
		FileManager.Delete(*SharedTempRootLockFileName, false /* bRequireExists*/, true /* bEvenIfReadOnly */, true /* Quiet */);
	};

	// Delete our TempRoot inside the machine-wide critical section.
	// Doing it before would remove our right to expect that the SharedTempRoot would not be deleted out from under us.
	FString TempRootLockFileName;
	if (!TempRoot.IsEmpty())
	{
		TempRootLockFile.Reset(); // Drop our lock that prevents the lockfile from being deleted
		TempRootLockFileName = TempRoot + LockSuffix;
		if (!FileManager.Delete(*TempRootLockFileName, false /* bRequireExists */, true /* bEvenIfReadOnly */, true /* Quiet */))
		{
			UE_LOGF(LogCook, Display, "AsyncIoDelete could not clean up its lock file %ls. LastError: %d.",
				*TempRootLockFileName, FPlatformMisc::GetLastError());
			bDirectoryEmpty = false;
		}
	}

	auto GetCountFromFilename = [](const FString& BaseFileName)
	{
		int32 IntValue;
		LexFromString(IntValue, *BaseFileName);
		return IntValue; // Will be 0 if invalid
	};

	int32 FirstUnusedCount = 1;
	bool bHasUnexpectedFiles = false;
	TSet<FString> ExistingLockFileLeafs;
	TSet<FString> ExistingDirLeafs;
	FileManager.IterateDirectory(*SharedTempRoot,
		[&ExistingLockFileLeafs, &ExistingDirLeafs, &bHasUnexpectedFiles, LockSuffix,
		&FirstUnusedCount, &GetCountFromFilename](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			// Convert FilenameOrDirectory to a relative path because absolute vs relative paths and junctions
			// may change the name of the parent directory
			FStringView ExistingPath(FilenameOrDirectory);
			FString ExistingLeaf = FString(FPathViews::GetPathLeaf(ExistingPath));

			if (!bIsDirectory)
			{
				if (FStringView(ExistingLeaf).EndsWith(LockSuffix))
				{
					FirstUnusedCount = FMath::Max(FirstUnusedCount, 1+GetCountFromFilename(ExistingLeaf.LeftChop(LockSuffix.Len())));
					ExistingLockFileLeafs.Add(ExistingLeaf);
				}
				else
				{
					UE_LOGF(LogCook, Warning, "AsyncIoDelete found unexpected file '%ls' in its SharedTempRoot. This file will block cleanup of the SharedTempRoot, please delete it manually.",
						FilenameOrDirectory);
					bHasUnexpectedFiles = true;
				}
			}
			else
			{
				ExistingDirLeafs.Add(ExistingLeaf);
				FirstUnusedCount = FMath::Max(FirstUnusedCount, 1+GetCountFromFilename(ExistingLeaf));
			}
			return true;
		});
	for (const FString& ExistingDirLeaf : ExistingDirLeafs)
	{
		if (!ExistingLockFileLeafs.Contains(ExistingDirLeaf + LockSuffix))
		{
			UE_LOGF(LogCook, Warning, "AsyncIoDelete found unexpected directory '%ls' in its SharedTempRoot. This directory will block cleanup of the SharedTempRoot, please delete it manually.",
				*FPaths::Combine(SharedTempRoot, ExistingDirLeaf));
			bHasUnexpectedFiles = true;
		}
	}

	if (bCreateRoot)
	{
		FString TempRootDirName = FPaths::Combine(SharedTempRoot, LexToString(FirstUnusedCount));
		TempRootLockFileName = TempRootDirName + LockSuffix;
		if (!FileManager.MakeDirectory(*TempRootDirName, true /* Tree */))
		{
			UE_LOGF(LogCook, Error, "AsyncIoDelete could not create its root %ls. LastError: %d. Falling back to synchronous delete.",
				*TempRootDirName, FPlatformMisc::GetLastError());
			return false;
		}
		TempRootLockFile.Reset(FileManager.CreateFileWriter(*TempRootLockFileName));
		if (!TempRootLockFile)
		{
			UE_LOGF(LogCook, Error, "AsyncIoDelete could not create LockFile %ls. LastError: %d. Falling back to synchronous delete.",
				*TempRootLockFileName, FPlatformMisc::GetLastError());
			FileManager.DeleteDirectory(*TempRootDirName, false /* bRequireExists */, true /* Tree */);
			return false;
		}
		TempRoot = TempRootDirName;

		// We do cleanup of orphaned temp roots only on startup, so that we can move their orphaned directories
		// into the new directory we are creating, for async delete
		for (const FString& ExistingDirLeaf : ExistingDirLeafs)
		{
			FString SourceDir = FPaths::Combine(SharedTempRoot, ExistingDirLeaf);
			FString SourceLockFile = SourceDir + LockSuffix;
			FString DestLockFile = FPaths::Combine(TempRoot, FString::Printf(TEXT("%u"), DeleteCounter));

			// If we can move-delete the lock file, the other process must have dropped its lock, so we can move-delete the directory.
			if (FileManager.Move(*DestLockFile, *SourceLockFile, true /* bReplace */, true /* EvenIfReadOnly */,
				false /* Attributes */, true /* bDoNotRetryOnError */))
			{
				++DeleteCounter;
				FString DestDir = FPaths::Combine(TempRoot, FString::Printf(TEXT("%u"), DeleteCounter));
				if (FileManager.Move(*DestDir, *SourceDir, true /* bReplace */, true /* EvenIfReadOnly */,
					false /* Attributes */, true /* bDoNotRetryOnError */))
				{
					++DeleteCounter;
					OutOrphanedRootsToDelete.Add({ DestLockFile, EPathType::File });
					OutOrphanedRootsToDelete.Add({ DestDir, EPathType::Directory });
				}
				else
				{
					// If we move-deleted the lock file but failed to move-delete the directory, put the lock file back so a future
					// AsyncIODelete can try to delete it
					FileManager.Move(*SourceLockFile, *DestLockFile, true /* bReplace */, true /* EvenIfReadOnly */,
						false /* Attributes */, true /* bDoNotRetryOnError */);

					// Add a dummy entry in OutOrphanedRootsToDelete so that the DeleteCounter == OutOrphanedRootsToDelete.Num().
					OutOrphanedRootsToDelete.Add({ FString(), EPathType::File});
				}
			}
		}
	}
	else
	{
		// When deleting our TempRoot, also delete the SharedTempRoot if we're the last thing in the directory
		// Remove our TempRoot from the ExistingDirs if it showed up in the iteration even though we deleted it due to FileManager delete lag
		ExistingDirLeafs.Remove(FPaths::GetBaseFilename(TempRoot));
		ExistingLockFileLeafs.Remove(FPaths::GetBaseFilename(TempRootLockFileName));
		if (!ExistingDirLeafs.IsEmpty() || !ExistingLockFileLeafs.IsEmpty() || bHasUnexpectedFiles)
		{
			bDirectoryEmpty = false;
		}
		if (bDirectoryEmpty)
		{
			FileManager.DeleteDirectory(*SharedTempRoot, false /* bRequireExists */, true /* Tree */);
		}
	}
	return true;
}

#if WITH_ASYNCIODELETE_DEBUG
void FAsyncIODelete::AddTempRoot(const FStringView& InTempRoot)
{
	FString TempRoot(InTempRoot);
	for (FString& Existing : AllTempRoots)
	{
		if (FPaths::IsSamePath(Existing, TempRoot))
		{
			continue;
		}
		checkf(!FPaths::IsUnderDirectory(Existing, TempRoot), TEXT("New FAsyncIODelete has TempRoot \"%s\" that is a subdirectory of existing TempRoot \"%s\"."), *TempRoot, *Existing);
		checkf(!FPaths::IsUnderDirectory(TempRoot, Existing), TEXT("New FAsyncIODelete has TempRoot \"%s\" that is a parent directory of existing TempRoot \"%s\"."), *TempRoot, *Existing);
	}
	AllTempRoots.Add(MoveTemp(TempRoot));
}

void FAsyncIODelete::RemoveTempRoot(const FStringView& InTempRoot)
{
	AllTempRoots.Remove(FString(InTempRoot));
}
#endif