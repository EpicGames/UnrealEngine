// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

#ifndef WITH_ASYNCIODELETE_DEBUG
	#if DO_CHECK
		#define WITH_ASYNCIODELETE_DEBUG 1 // When enabled the class uses extra memory and does some tracking
	#else
		#define WITH_ASYNCIODELETE_DEBUG 0
	#endif
#endif

class FEvent;

/**
 * Helper class to asynchronously delete paths (files or directories).
 * The deleted paths are moved to a unique temporary location immediately, and then asynchronously deleted later.
 * Note that two FAsyncIODeletes (in the same process or in multiple processes) at the same with the same InOwnedTempRoot 
 * is forbidden and will cause behavior errors.  Caller must ensure the machine-global uniqueness of InOwnedTempRoot.
 *
 * FAsyncIODelete's public interface is not threadsafe; all calls to the public interface must be run from the same thread.
 */
class FAsyncIODelete
{
public:
	/**
	 * Quick constructor to create flags; does not take any IO action until Setup or a Delete function is used
	 *
	 * @param InOwnedTempRoot - The FAsyncIODelete system takes ownership of this directory, and deletes it when done.
	 * See SetTempRoot. The FAsyncIODelete will not be able to delete directories until a valid TempRoot has been set,
	 * either in the constructor or in SetTempRoot
	 */
	explicit FAsyncIODelete(const FStringView& InOwnedTempRoot=FStringView());

	/* Synchronously waits for all requested deletions to complete */
	~FAsyncIODelete();

	/**
	 * Set a TempRoot directory to hold deleted directories from elsewhere, which is potentially shared with other
	 * processes running other FAsyncIODelete. The AsyncIoDelete processes take ownership of this directory and will
	 * delete it; do not pass in a directory shared with other systems, create a subdirectory for it.
	 * Must not be a parent or child of any other FAsyncDelete's TempRoot, but is allowed to be the same.
	 * Each new FAsyncIODelete process will negotiate with other proccesses using lock files, to delete the directory
	 * when all processes are done.
	 * If the FAsyncIODelete has already been used to delete files, this Set will block until all deletes are finished.
	 */
	void SetTempRoot(FStringView InSharedRoot);
	FStringView GetTempRoot() const { return SharedTempRoot; }
	/**
	 * Returns the path this AsyncIODelete is using for its uniquely owned deletion root (a subdirectory of TempRoot).
	 * Will return empty before Setup (or on-demand Setup) or after Teardown.
	 */
	FStringView GetDeletionRoot() const { return TempRoot; };

	/**
	 * Set whether new background deletes are paused.  If paused, paths will be moved immediately but will not be deleted from the temporary location until unpaused (or at the FAsyncIODelete's destruction).
	 * Setting deletes paused does not affect deletes already in progress.
	 */
	void SetDeletesPaused(bool bInPaused);
	bool GetDeletesPaused() const { return bPaused; }

	/**
	 * Prepare the directory on disk.  Called OnDemand from the other interface functions.  Can also be called if desired by client code.
	 * May take a long time to run if cleanup from a crashed previous process is required.
	 */
	void Setup();

	/** Synchronously wait for all tasks to complete, and remove the temproot. */
	void Teardown();

	/**
	 * Asynchronously delete the given directory (and all subdirs).  The directory is moved and no longer exists before returning, but the move location will be deleted from disk later.
	 * The delete will fail if the given path is not a directory.
	 * The delete will fail and return false if the given path is a parent directory, child directory, or is equal to TempRoot.
	 * The delete can also fail due to IO Errors - insufficient permissions, device failure.
	 * If the given path is on a different volume than temp root, or the move fails for other reasons, the delete will proceed synchronously but will succeed.
	 * If the given path does not exist, DeleteDirectory will return true.
	 *
	 * @param PathToDelete - If DeleteDirectory returns true, this path has been removed from disk.  
	 *
	 * @return Whether the requested path is no longer present on disk.
	 */
	bool DeleteDirectory(const FStringView& PathToDelete)
	{
		return Delete(PathToDelete, EPathType::Directory);
	}

	/** Asynchronously delete a file.  Behavior is otherwise the same as DeleteDirectory. */
	bool DeleteFile(const FStringView& PathToDelete)
	{
		return Delete(PathToDelete, EPathType::File);
	}

	/** Synchronously wait for all deletes to complete.  */
	bool WaitForAllTasks(float TimeLimitSeconds = 0.0f);

	static bool AsyncEnabled();

	/** Internal accessors for unittests */
	static FStringView GetLockSuffix();
	static void SetMaxWaitSecondsForLock(float WaitSeconds = -1.f);

private:
	/** Update the ActiveTaskCount and events for a task completing */
	void OnTaskComplete();

	enum class EPathType
	{
		File,
		Directory
	};
	struct FDeleteRequest
	{
		FString Path;
		EPathType PathType;
	};
	/** Asynchronously delete a file or directory.  We handle both in the same function, but we want the interface to be explicit. */
	bool Delete(const FStringView& PathToDelete, EPathType ExpectedType);

	/** Create and store the task to delete the given Directory or File */
	void CreateDeleteTask(const FStringView& InDeletePath, EPathType PathType);

	/** Delete the given path synchronously; called from a task or in error fallback cases from the public thread */
	bool SynchronousDelete(const TCHAR* InDeletePath, EPathType PathType);

	bool TryPurgeOldAndCreateRoot(bool bCreateRoot, TArray<FDeleteRequest>& OutHangingRootsToDelete);

	FString SharedTempRoot;
	FString TempRoot;
	TUniquePtr<FArchive> TempRootLockFile;
	TArray<FString> PausedDeletes;
	FCriticalSection CriticalSection; // We use a CriticalSection instead of a TAtomic ActiveTaskCount so that we can atomically { trigger TasksComplete if ActiveTaskCount == 0 }
	FEvent* TasksComplete = nullptr;
	uint32 ActiveTaskCount = 0;
	uint32 DeleteCounter = 0;
	bool bInitialized = false;
	bool bAsyncInitialized = false;
	bool bPaused = false;

#if WITH_ASYNCIODELETE_DEBUG
	static TArray<FString> AllTempRoots;
	static void AddTempRoot(const FStringView& InTempRoot);
	static void RemoveTempRoot(const FStringView& InTempRoot);
#endif
};