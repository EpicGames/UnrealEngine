// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncIODelete.h"

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"
#include "Trace/Detail/Channel.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAsyncIODeleteTest, "System.Core.Misc.AsyncIODelete", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAsyncIODeleteTest::RunTest(const FString& Parameters)
{
	bool bAsyncEnabled = FAsyncIODelete::AsyncEnabled();
	UE_LOGF(LogCore, Display, "FAsyncIODeleteTest::RunTest, ASYNCIODELETE is %ls", bAsyncEnabled ? TEXT("ENABLED") : TEXT("DISABLED"));

	FString TestRoot = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("AsyncIODelete"), TEXT(""));
	IFileManager& FileManager = IFileManager::Get();
	const bool bApplyToTreeTrue = true;
	ON_SCOPE_EXIT
	{
		const bool bRequireExists = false;
		FileManager.DeleteDirectory(*TestRoot, bRequireExists, bApplyToTreeTrue);
	};

	FString SharedTempRoot = FPaths::Combine(TestRoot, TEXT("TempRoot"));
	FString SharedTempRoot2 = FPaths::Combine(TestRoot, TEXT("TempRoot2"));
	FString SharedTempRoot3 = FPaths::Combine(TestRoot, TEXT("TempRoot3"));
	FString SharedTempRoot4 = FPaths::Combine(TestRoot, TEXT("TempRoot4"));
	FString TestFile1 = FPaths::Combine(TestRoot, TEXT("TestFile1"));
	FString TestDir1 = FPaths::Combine(TestRoot, TEXT("TestDir1"));
	int32 NumFiles = 0;
	int32 NumDirs = 0;
	const TCHAR* TestText = TEXT("Test");
	auto CreateTestPathsToDelete = [=, &TestFile1, &TestDir1, &FileManager]()
	{
		FFileHelper::SaveStringToFile(TestText, *TestFile1);
		FileManager.MakeDirectory(*TestDir1, bApplyToTreeTrue);
	};
	auto CountFilesAndDirs = [&NumFiles, &NumDirs](const TCHAR* VisitFilename, bool VisitIsDir)
	{
		NumFiles += VisitIsDir ? 1 : 0;
		NumDirs += VisitIsDir ? 0 : 1;
		return true;
	};
	auto TestTempRootCountsEqual = [&](const FString& RootDir, int32 ExpectedFileCount, int32 ExpectedDirCount, const TCHAR* TestDescription)
	{
		NumFiles = 0;
		NumDirs = 0;
		FileManager.IterateDirectory(*RootDir, CountFilesAndDirs);
		TestTrue(TestDescription, NumFiles == ExpectedFileCount && NumDirs == ExpectedDirCount);
	};
	auto TestRequestedPathsDeleted = [&](const TCHAR* TestDescription)
	{
		TestTrue(TestDescription, !FileManager.FileExists(*TestFile1) && !FileManager.DirectoryExists(*TestDir1));
	};

	FileManager.MakeDirectory(*TestRoot);

	const bool bVerbose = true;
	auto StartSection = [bVerbose](const TCHAR* Section)
	{
		if (bVerbose)
		{
			UE_LOGF(LogCore, Display, "%ls", Section);
		}
	};
	const float MaxWaitTime = 5.0f;
	auto WaitForAllTasksAndVerify = [this, MaxWaitTime](FAsyncIODelete& InAsyncIODelete)
	{
		bool WaitResult = InAsyncIODelete.WaitForAllTasks(MaxWaitTime);
		TestTrue(TEXT("WaitForAllTasks timed out"), WaitResult);
	};
	{
		StartSection(TEXT("Constructing first FAsyncIODelete"));
		FAsyncIODelete AsyncIODelete(SharedTempRoot);

		StartSection(TEXT("Waiting for tasks to complete when none have been launched should succeed"));
		WaitForAllTasksAndVerify(AsyncIODelete);

		StartSection(TEXT("Moving file and directory from the source location should be finished by the time DeleteFile/DeleteDirectory returns"));
		CreateTestPathsToDelete();
		AsyncIODelete.DeleteFile(TestFile1);
		AsyncIODelete.DeleteDirectory(TestDir1);
		TestRequestedPathsDeleted(TEXT("AsyncIODelete::Delete should have moved the deleted paths before returning."));

		FString DeletionRoot = FString(AsyncIODelete.GetDeletionRoot());
		StartSection(TEXT("Deleting the temporary files/directories should be finished before WaitForAllTasks returns"));
		WaitForAllTasksAndVerify(AsyncIODelete);
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("DeletionRoot is valid after deleting some files"), !DeletionRoot.IsEmpty());
			if (!DeletionRoot.IsEmpty())
			{
				TestTempRootCountsEqual(DeletionRoot, 0, 0, TEXT("AsyncIODelete should have deleted the moved paths before WaitForAllTasks returned."));
			}
		}

		StartSection(TEXT("Two FAsyncIODelete constructed at once should be legal, as long as they have different TempRoots"));
		FAsyncIODelete AsyncIODelete2(SharedTempRoot2);

		StartSection(TEXT("Use the pause feature to verify that the paths are indeed moved into the TempRoot"));
		AsyncIODelete2.SetDeletesPaused(true);
		CreateTestPathsToDelete();
		AsyncIODelete2.DeleteFile(TestFile1);
		AsyncIODelete2.DeleteDirectory(TestDir1);
		TestRequestedPathsDeleted(TEXT("AsyncIODelete::Delete should have moved the deleted paths before returning even when paused."));
		WaitForAllTasksAndVerify(AsyncIODelete2);
		FString DeletionRoot2 = FString(AsyncIODelete2.GetDeletionRoot());
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("DeletionRoot is valid after deleting some files"), !DeletionRoot2.IsEmpty());
			if (!DeletionRoot2.IsEmpty())
			{
				TestTempRootCountsEqual(DeletionRoot2, 1, 1, TEXT("AsyncIODelete should not have deleted the moved paths because it is paused."));
			}
		}
		AsyncIODelete2.SetDeletesPaused(false);
		WaitForAllTasksAndVerify(AsyncIODelete2);
		if (bAsyncEnabled)
		{
			if (!DeletionRoot2.IsEmpty())
			{
				TestTempRootCountsEqual(DeletionRoot2, 0, 0, TEXT("AsyncIODelete should have deleted the moved paths after unpausing."));
			}
		}

		StartSection(TEXT("Verify Teardown() deletes the TempRoot and Setup() creates it"));
		AsyncIODelete2.Teardown();
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("AsyncIODelete::Teardown should have deleted its TempRoot."), !FileManager.DirectoryExists(*SharedTempRoot2));
		}
		AsyncIODelete2.Setup();
		DeletionRoot2 = AsyncIODelete2.GetDeletionRoot();
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("AsyncIODelete::Setup should have created its TempRoot."),
				!DeletionRoot2.IsEmpty() && FileManager.DirectoryExists(*SharedTempRoot2) && FileManager.DirectoryExists(*DeletionRoot2));
		}

		StartSection(TEXT("Manual setup works as long as you call SetTempRoot before Setup"));
		FAsyncIODelete AsyncIODelete3;
		AsyncIODelete3.SetTempRoot(SharedTempRoot3);
		AsyncIODelete3.Setup();
		FString DeletionRoot3 = FString(AsyncIODelete3.GetDeletionRoot());
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("Setup should have created the TempRoot."),
				!DeletionRoot3.IsEmpty() && FileManager.DirectoryExists(*SharedTempRoot3) && FileManager.DirectoryExists(*DeletionRoot3));
		}

		StartSection(TEXT("Check that even after Setup, waiting for tasks to complete when none have been launched should succeed"));
		WaitForAllTasksAndVerify(AsyncIODelete);

		StartSection(TEXT("Changing TempRoot and then deleting a file works"));
		CreateTestPathsToDelete();
		AsyncIODelete3.DeleteFile(TestFile1);
		AsyncIODelete3.DeleteDirectory(TestDir1);
		AsyncIODelete3.SetTempRoot(SharedTempRoot4);
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("SetTempRoot should have deleted the old TempRoot."), !FileManager.DirectoryExists(*SharedTempRoot3));
		}

		CreateTestPathsToDelete();
		AsyncIODelete3.DeleteFile(TestFile1);
		AsyncIODelete3.DeleteDirectory(TestDir1);
		TestRequestedPathsDeleted(TEXT("AsyncIODelete::Delete should have worked after changing the TempRoot."));
		FString DeletionRoot4 = FString(AsyncIODelete3.GetDeletionRoot());
		if (bAsyncEnabled)
		{
			TestTrue(TEXT("Delete should have created the new TempRoot after SetTempRoot."),
				!DeletionRoot4.IsEmpty() && FileManager.DirectoryExists(*SharedTempRoot4) && FileManager.DirectoryExists(*DeletionRoot4));
		}
		WaitForAllTasksAndVerify(AsyncIODelete3);
		if (bAsyncEnabled)
		{
			TestTempRootCountsEqual(DeletionRoot4, 0, 0, TEXT("AsyncIODelete::Delete should have created the tasks to delete moved files after changing the TempRoot."));
		}

		StartSection(TEXT("Attempting to delete a parent directory of the temproot, the temproot itself, or a child inside of it fails"));
		if (bAsyncEnabled && !DeletionRoot4.IsEmpty())
		{
			FString SubDirInTempRoot4 = FPaths::Combine(DeletionRoot4, TEXT("SubDir"));
			FileManager.MakeDirectory(*SubDirInTempRoot4, bApplyToTreeTrue); // Note it's illegal to add files into TempRoot, but we're not currently checking for it and we're not colliding with the DeleteN paths AsyncIODelete uses, so this breaking of the rule will not cause problems
			TestFalse(TEXT("AsyncIODelete should refuse to delete a parent of its TempRoot."), AsyncIODelete3.DeleteDirectory(TestRoot));
			TestFalse(TEXT("AsyncIODelete should refuse to delete its TempRoot."), AsyncIODelete3.DeleteDirectory(DeletionRoot4));
			TestFalse(TEXT("AsyncIODelete should refuse to delete its SharedTempRoot."), AsyncIODelete3.DeleteDirectory(SharedTempRoot4));
			TestFalse(TEXT("AsyncIODelete should refuse to delete a child of its TempRoot."), AsyncIODelete3.DeleteDirectory(SubDirInTempRoot4));
		}
	}
	if (bAsyncEnabled)
	{
		TestTrue(TEXT("AsyncIODelete destructor should have deleted its TempRoot."),
			!FileManager.DirectoryExists(*SharedTempRoot) && !FileManager.DirectoryExists(*SharedTempRoot2)
			&& !FileManager.DirectoryExists(*SharedTempRoot3) && !FileManager.DirectoryExists(*SharedTempRoot4));
	}


	FString SharedTempRoot5 = FPaths::Combine(TestRoot, TEXT("TempRoot5"));
	if (bAsyncEnabled)
	{
		StartSection(TEXT("Two AsyncIODeletes in the same SharedTempRoot do not delete each other's TempRoots."));
		FAsyncIODelete Async1(SharedTempRoot5);
		Async1.Setup();
		FString TempRoot1 = FString(Async1.GetDeletionRoot());
		TestTrue(TEXT("Async1 creates its root."), !TempRoot1.IsEmpty() && FileManager.DirectoryExists(*TempRoot1));
		FString DeletionRoot2;
		{
			FAsyncIODelete Async2(SharedTempRoot5);
			Async2.Setup();
			DeletionRoot2 = FString(Async2.GetDeletionRoot());
			TestTrue(TEXT("Async2 creates its root."), !DeletionRoot2.IsEmpty() && FileManager.DirectoryExists(*DeletionRoot2));
			TestTrue(TEXT("Async1's root still exists after Async2 Setup."), !TempRoot1.IsEmpty() && FileManager.DirectoryExists(*TempRoot1));
		}
		TestTrue(TEXT("Async2 deletes its root."), DeletionRoot2.IsEmpty() || !FileManager.DirectoryExists(*DeletionRoot2));
		TestTrue(TEXT("Async1's root still exists after Async2 Teardown."), !TempRoot1.IsEmpty() && FileManager.DirectoryExists(*TempRoot1));

		CreateTestPathsToDelete();
		Async1.DeleteFile(TestFile1);
		FString DeletionRoot3;
		{
			FAsyncIODelete Async3(SharedTempRoot5);
			Async3.DeleteDirectory(TestDir1);
			DeletionRoot3 = FString(Async3.GetDeletionRoot());
			TestTrue(TEXT("Async3 creates its root."), !DeletionRoot3.IsEmpty() && FileManager.DirectoryExists(*DeletionRoot3));
			TestTrue(TEXT("Async1's root still exists after Async3 Setup."), !TempRoot1.IsEmpty() && FileManager.DirectoryExists(*TempRoot1));
		}
		TestTrue(TEXT("Async3 deletes its root."), DeletionRoot3.IsEmpty() || !FileManager.DirectoryExists(*DeletionRoot3));
		TestTrue(TEXT("Async1's root still exists after Async3 Teardown."), !TempRoot1.IsEmpty() && FileManager.DirectoryExists(*TempRoot1));
	}
	if (bAsyncEnabled)
	{
		TestTrue(TEXT("AsyncIODelete destructor should have deleted its TempRoot."),
			!FileManager.DirectoryExists(*SharedTempRoot5));
	}

	FString SharedTempRoot6 = FPaths::Combine(TestRoot, TEXT("TempRoot6"));
	if (bAsyncEnabled)
	{
		StartSection(TEXT("AsyncIODelete cleans up a hanging directory from a crashed process."));
		FString HangingTempRoot = FPaths::Combine(SharedTempRoot6, TEXT("5"));
		FString HangingTempRootLockFile = HangingTempRoot + FAsyncIODelete::GetLockSuffix();
		FileManager.MakeDirectory(*HangingTempRoot, true /* Tree */);
		TUniquePtr<FArchive> LockFile(FileManager.CreateFileWriter(*HangingTempRootLockFile));
		LockFile.Reset();
		TestTrue(TEXT("Hanging directory was successfully created."), FileManager.DirectoryExists(*HangingTempRoot) && FileManager.FileExists(*HangingTempRootLockFile));

		{
			FAsyncIODelete Async(SharedTempRoot6);
			Async.Setup();
			FString DeletionRoot = FString(Async.GetDeletionRoot());
			TestTrue(TEXT("Async creates its root."), !DeletionRoot.IsEmpty() && FileManager.DirectoryExists(*DeletionRoot));
		}
		TestTrue(TEXT("Async deletes the hanging directory."), !FileManager.DirectoryExists(*SharedTempRoot6));
	}
	if (bAsyncEnabled)
	{
		TestTrue(TEXT("AsyncIODelete destructor should have deleted its TempRoot."),
			!FileManager.DirectoryExists(*SharedTempRoot6));
	}

	FString SharedTempRoot7 = FPaths::Combine(TestRoot, TEXT("TempRoot7"));
	if (bAsyncEnabled)
	{
		StartSection(TEXT("A lock on the shared root prevents AsyncIODeletes from working with it."));
		FAsyncIODelete::SetMaxWaitSecondsForLock(0.02f);
		FileManager.MakeDirectory(*SharedTempRoot7, true /* Tree */);
		FString SharedTempRoot7LockFileName = SharedTempRoot7 + FAsyncIODelete::GetLockSuffix();
		TUniquePtr<FArchive> LockFile(FileManager.CreateFileWriter(*SharedTempRoot7LockFileName));
		{
			FAsyncIODelete AsyncFail(SharedTempRoot7);
			AddExpectedError(SharedTempRoot7, EAutomationExpectedErrorFlags::Contains);
			AsyncFail.Setup();
			TestTrue(TEXT("AsyncIODelete on a locked root is unable to create its deletion root."), AsyncFail.GetDeletionRoot().IsEmpty());
			CreateTestPathsToDelete();
			AsyncFail.DeleteFile(TestFile1);
			AsyncFail.DeleteDirectory(TestDir1);
			TestRequestedPathsDeleted(TEXT("AsyncIODelete that failed due to a locked root should delete its requested paths synchronously."));
			TestTrue(TEXT("AsyncIODelete on a locked root is unable to create its deletion root."), AsyncFail.GetDeletionRoot().IsEmpty());
			TestTempRootCountsEqual(SharedTempRoot7, 0, 0, TEXT("AsyncIODelete that failed due to a locked root should not have created a TempRoot in the locked root."));
		}
		FAsyncIODelete::SetMaxWaitSecondsForLock(-1.f);
		LockFile.Reset();
		FileManager.DeleteDirectory(*SharedTempRoot7, false /* RequireEixsts */, true /* Tree */);
		FileManager.Delete(*SharedTempRoot7LockFileName, false /* RequireEixsts */, true/* EvenReadOnly */, true /* Quiet */);
	}

	return true;
}