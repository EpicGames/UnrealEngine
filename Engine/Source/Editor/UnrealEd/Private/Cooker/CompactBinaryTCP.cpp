// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompactBinaryTCP.h"

#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

namespace UE::CompactBinaryTCP
{

const TCHAR* DescribeStatus(EConnectionStatus Status)
{
	switch (Status)
	{
	case EConnectionStatus::Okay: return TEXT("Connection is okay");
	case EConnectionStatus::Terminated: return TEXT("Connection terminated");
	case EConnectionStatus::FormatError: return TEXT("Connection received invalidly formatted bytes");
	case EConnectionStatus::Failed: return TEXT("Operation failed");
	case EConnectionStatus::Incomplete: return TEXT("Operation is incomplete");
	default: checkNoEntry(); return TEXT("InvalidConnectionStatus");
	}
}

struct FPacketHeader
{
	uint64 Magic;
	uint64 Size;
};
constexpr uint64 MessageHeaderExpectedMagic = 0x854EBA92854EBA92;

void FReceiveBuffer::Reset()
{
	Payload.Reset();
	bParsedHeader = false;
	BytesRead = 0;
}

void FSendBuffer::Reset()
{
	PendingMessages.Empty();
	Payload.Reset();
	BytesWritten = 0;
	bCreatedPayload = false;
	bSentHeader = false;
}

class FCompactBinaryTCPImpl
{
public:
	static EConnectionStatus TryReadPacket(FSocket* Socket, FReceiveBuffer& Buffer,
		TArray<FMarshalledMessage>& Messages, uint64 MaxPacketSize);
	static bool TryCreatePayload(TArray<FMarshalledMessage>& PendingMessages,
		uint64 MaxPacketSize, FUniqueBuffer& OutBuffer);
	static EConnectionStatus PollSendBytes(FSocket* Socket, const void* Data,
		uint64 DataSize, uint64& InOutBytesWritten);
	/** Note that the elements in AppendMessages are mutated */
	static EConnectionStatus TryWritePacket(FSocket* Socket, FSendBuffer& Buffer,
		TArrayView<FMarshalledMessage>&& AppendMessages, uint64 MaxPacketSize);
	static void QueueMessage(FSendBuffer& Buffer, FMarshalledMessage&& Message);
};

EConnectionStatus FCompactBinaryTCPImpl::TryReadPacket(FSocket* Socket, FReceiveBuffer& Buffer,
	TArray<FMarshalledMessage>& Messages, uint64 MaxPacketSize)
{
	int32 BytesRead;
	if (!Buffer.bParsedHeader)
	{
		// When reading the relatively small PacketHeader, we wait for its total size to be ready
		// using Peek to reduce complexity in this function.
		FPacketHeader MessageHeader;
		bool bStillAlive = Socket->Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader),
			BytesRead, ESocketReceiveFlags::Peek);
		check(BytesRead <= sizeof(FPacketHeader));
		if (BytesRead < sizeof(FPacketHeader))
		{
			if (!bStillAlive)
			{
				return EConnectionStatus::Terminated;
			}
			else
			{
				return EConnectionStatus::Okay;
			}
		}

		Socket->Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader), BytesRead,
			ESocketReceiveFlags::None);
		check(BytesRead == sizeof(FPacketHeader));
		if (MessageHeader.Magic != MessageHeaderExpectedMagic ||
			(MaxPacketSize > 0 && MessageHeader.Size > MaxPacketSize))
		{
			return EConnectionStatus::FormatError;
		}

		Buffer.Payload = FUniqueBuffer::Alloc(MessageHeader.Size);
		Buffer.bParsedHeader = true;
		Buffer.BytesRead = 0;
	}


	while (Buffer.Payload.GetSize() > Buffer.BytesRead)
	{
		// When reading the possibly large payload size that will block the remote socket
		// from continuing until we have read some of it, we read as much of the payload
		// as is available and store it in our dynamic and larger buffer in Buffer.Payload.
		uint64 RemainingSize = Buffer.Payload.GetSize() - Buffer.BytesRead;
		// Avoid possible OS restrictions on maximum read size by imposing our own moderate
		// size per call to Recv.
		constexpr uint64 MaxReadSize = 1000 * 1000 * 64;
		int32 ReadSize = static_cast<int32>(FMath::Min(RemainingSize, MaxReadSize));
		uint8* ReadData = static_cast<uint8*>(Buffer.Payload.GetData()) + Buffer.BytesRead;
		bool bConnectionAlive = Socket->Recv(ReadData, ReadSize, BytesRead);
		if (BytesRead <= 0)
		{
			if (!bConnectionAlive)
			{
				return EConnectionStatus::Terminated;
			}
			else
			{
				return EConnectionStatus::Okay;
			}
		}

		check(BytesRead <= ReadSize);
		Buffer.BytesRead += BytesRead;
	}

	// The FCbObjects we return have a pointer to this buffer and will keep it allocated
	// until they are destructed.
	FSharedBuffer SharedBuffer = Buffer.Payload.MoveToShared();
	Buffer.Reset();
	if (ValidateCompactBinary(SharedBuffer, ECbValidateMode::Default) != ECbValidateError::None)
	{
		return EConnectionStatus::FormatError;
	}

	FCbFieldIterator It = FCbFieldIterator::MakeRange(SharedBuffer);
	while (It)
	{
		FMarshalledMessage& Message = Messages.Emplace_GetRef();
		FCbFieldView MessageTypeView = *It;
		Message.MessageType = MessageTypeView.AsUuid();
		if (MessageTypeView.HasError())
		{
			return EConnectionStatus::FormatError;
		}
		++It;
		FCbField ObjectField = *It;
		Message.Object = ObjectField.AsObject();
		if (ObjectField.HasError())
		{
			return EConnectionStatus::FormatError;
		}
		++It;
	}
	return EConnectionStatus::Okay;
}

bool FCompactBinaryTCPImpl::TryCreatePayload(TArray<FMarshalledMessage>& PendingMessages,
	uint64 MaxPacketSize, FUniqueBuffer& OutBuffer)
{
	int32 NumMessages = PendingMessages.Num();
	checkSlow(NumMessages > 0); // Caller guarantees

	int32 NumMessagesTaken = 0;
	FCbWriter Writer;
	int32 MaxNumMessagesTaken = NumMessages;
	int32 RetryCount = 0;
	for (;;)
	{
		for (NumMessagesTaken = 0; NumMessagesTaken < MaxNumMessagesTaken; ++NumMessagesTaken)
		{
			FMarshalledMessage& Message = PendingMessages[NumMessagesTaken];
			if (MaxPacketSize > 0 && RetryCount == 0)
			{
				// We need to know the size the writer will have after serializing the object
				// so we can avoid exceeding our max packet size. If we exceed we have to retry
				// and write everything over again. But there's no cheap way to compute
				// that, and the max message size is usually large and won't be exceeded
				// go over, so we use a cheap hardcoded value rather than an accurate estimate.
				constexpr uint64 EstimatedSize = 1000;
				if (NumMessagesTaken > 0 && Writer.GetSaveSize() + EstimatedSize > MaxPacketSize)
				{
					break;
				}
			}
			Writer << Message.MessageType;
			Writer << Message.Object;
		}
		uint64 SaveSize = Writer.GetSaveSize();
		if (MaxPacketSize > 0 && Writer.GetSaveSize() > MaxPacketSize)
		{
			if (NumMessagesTaken == 1)
			{
				UE_LOG(LogSockets, Error, TEXT("Could not WritePacket to Socket. A single message with Guid %s is larger than MaxPacketSize %" UINT64_FMT "."),
					*PendingMessages[0].MessageType.ToString(), MaxPacketSize);
				return false;
			}
			check(RetryCount < 1); // If this is the Retry, then MaxNumMessagesTaken should have blocked going over size
			MaxNumMessagesTaken = NumMessagesTaken - 1;
			Writer.Reset();
			++RetryCount;
			continue;
		}

		// Successful
		OutBuffer = FUniqueBuffer::Alloc(Writer.GetSaveSize());
		Writer.Save(OutBuffer.GetView());
		PendingMessages.RemoveAt(0, NumMessagesTaken);
		return true;
	}
}

EConnectionStatus FCompactBinaryTCPImpl::PollSendBytes(FSocket* Socket, const void* Data,
	uint64 DataSize, uint64& InOutBytesWritten)
{
	check(DataSize <= MAX_int32);
	check(InOutBytesWritten < DataSize);

	int32 WriteSize = static_cast<int32>(DataSize - InOutBytesWritten);
	const uint8* WriteData = static_cast<const uint8*>(Data) + InOutBytesWritten;
	int32 BytesSent;
	bool bResult = Socket->Send(WriteData, WriteSize, BytesSent);
	if (BytesSent < WriteSize)
	{
		ESocketErrors Error = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode();
		if (BytesSent >= 0)
		{
			InOutBytesWritten += BytesSent;
		}

		if (!bResult)
		{
			if (Socket->GetConnectionState() != SCS_ConnectionError &&
				(Error == SE_EWOULDBLOCK || Error == SE_TRY_AGAIN))
			{
				// Not an error, this can occur if the process on the other end of the socket is not
				// reading fast enough (breakpoint or a busy tick) and the socket's buffers have filled up.
				return EConnectionStatus::Incomplete;
			}

			UE_LOGF(LogSockets, Log, "FCompactBinaryTCPImpl::PollSendBytes: Socket->Send(WriteSize==%d) failed with error %d and BytesSent == %d.",
				WriteSize, (int32)Error, BytesSent);

			// Return values of false indicate the socket send failed with an error, but the error
			// might be temporary, such as EWOULDBLOCK. Even when return value is false, return incomplete
			// unless socket's ConnectionState has been set to error
			if (Socket->GetConnectionState() == SCS_ConnectionError)
			{
				UE_LOGF(LogSockets, Error, "Could not WritePacket to Socket. Socket is in the error state.");
				return EConnectionStatus::Failed;
			}

			// TODO: Decide which errors are fatal even if the socket is still connected. We have seen one case
			// where the socket send fails forever, but we were not able to see what error it was returning.
		}

		return EConnectionStatus::Incomplete;
	}
	InOutBytesWritten = 0;
	return EConnectionStatus::Okay;
};

EConnectionStatus FCompactBinaryTCPImpl::TryWritePacket(FSocket* Socket, FSendBuffer& Buffer,
	TArrayView<FMarshalledMessage>&& AppendMessages, uint64 MaxPacketSize)
{
	// Copy AppendMessages into the SendBuffer before any early exit; we are responsible for holding
	// a reference to them now.
	if (!AppendMessages.IsEmpty())
	{
		Buffer.PendingMessages.Reserve(Buffer.PendingMessages.Num() + AppendMessages.Num());
		for (FMarshalledMessage& NewMessage : AppendMessages)
		{
			Buffer.PendingMessages.Add(MoveTemp(NewMessage));
		}
	}

	if (!Socket)
	{
		return EConnectionStatus::Terminated;
	}

	MaxPacketSize = MaxPacketSize == 0 ? MaxOSPacketSize : FMath::Min(MaxPacketSize, MaxOSPacketSize);

	for (;;)
	{
		if (!Buffer.bCreatedPayload)
		{
			if (Buffer.PendingMessages.IsEmpty())
			{
				return EConnectionStatus::Okay;
			}
			if (!TryCreatePayload(Buffer.PendingMessages, MaxPacketSize, Buffer.Payload))
			{
				return EConnectionStatus::Failed;
			}
			Buffer.bCreatedPayload = true;
			Buffer.BytesWritten = 0;
		}

		if (!Buffer.bSentHeader)
		{
			FPacketHeader MessageHeader;
			MessageHeader.Magic = MessageHeaderExpectedMagic;
			MessageHeader.Size = Buffer.Payload.GetSize();
			EConnectionStatus Status = PollSendBytes(Socket, &MessageHeader, 
				sizeof(FPacketHeader), Buffer.BytesWritten);
			if (Status != EConnectionStatus::Okay)
			{
				return Status;
			}
			Buffer.bSentHeader = true;
		}

		{
			EConnectionStatus Status = PollSendBytes(Socket, Buffer.Payload.GetData(),
				Buffer.Payload.GetSize(), Buffer.BytesWritten);
			if (Status != EConnectionStatus::Okay)
			{
				return Status;
			}
			Buffer.bCreatedPayload = false;
			Buffer.bSentHeader = false;
			Buffer.Payload.Reset();
		}
	}
}

void FCompactBinaryTCPImpl::QueueMessage(FSendBuffer& Buffer, FMarshalledMessage&& Message)
{
	Buffer.PendingMessages.Add(MoveTemp(Message));
}

EConnectionStatus TryReadPacket(FSocket* Socket, FReceiveBuffer& Buffer,
	TArray<FMarshalledMessage>& Messages, uint64 MaxPacketSize)
{
	return FCompactBinaryTCPImpl::TryReadPacket(Socket, Buffer, Messages, MaxPacketSize);
}

EConnectionStatus TryWritePacket(FSocket* Socket, FSendBuffer& Buffer, 
	TArray<FMarshalledMessage>&& AppendMessages, uint64 MaxPacketSize)
{
	return FCompactBinaryTCPImpl::TryWritePacket(Socket, Buffer, MoveTemp(AppendMessages), MaxPacketSize);
}

EConnectionStatus TryWritePacket(FSocket* Socket, FSendBuffer& Buffer,
	FMarshalledMessage&& AppendMessage, uint64 MaxPacketSize)
{
	return FCompactBinaryTCPImpl::TryWritePacket(Socket, Buffer,
		TArrayView<FMarshalledMessage>(&AppendMessage, 1), MaxPacketSize);
}

void QueueMessage(FSendBuffer& Buffer, FMarshalledMessage&& Message)
{
	FCompactBinaryTCPImpl::QueueMessage(Buffer, MoveTemp(Message));
}

EConnectionStatus TryFlushBuffer(FSocket* Socket, FSendBuffer& Buffer, uint64 MaxPacketSize)
{
	return FCompactBinaryTCPImpl::TryWritePacket(Socket, Buffer, TArrayView<FMarshalledMessage>(), MaxPacketSize);
}

}

FCbWriter& operator<<(FCbWriter& Writer, const FSoftObjectPathSerializationWrapper& Path)
{
	Writer << Path.Inner.ToString();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FSoftObjectPathSerializationWrapper& Path)
{
	FString PathString;
	if (!LoadFromCompactBinary(Field, PathString))
	{
		Path.Inner.Reset();
		return false;
	}
	Path.Inner.SetPath(PathString);
	return true;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::CompactBinaryTCP::FMarshalledMessage& Value)
{
	bool bOk = !Field.HasError();
	bOk &= LoadFromCompactBinary(Field["T"], Value.MessageType);

	if (bOk)
	{
		FCbObjectView ValueView = Field["V"].AsObjectView();
		bOk &= !Field.HasError();
		if (bOk)
		{
			Value.Object = FCbObject::Clone(ValueView);
		}
	}
	if (!bOk)
	{
		Value.MessageType.Invalidate();
		Value.Object.Reset();
	}
	return bOk;
}
