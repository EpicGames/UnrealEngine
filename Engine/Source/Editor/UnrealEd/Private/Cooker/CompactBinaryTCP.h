// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Guid.h"
#include "Serialization/CompactBinaryContainerSerialization.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/SoftObjectPath.h"

class FSocket;

namespace UE::CompactBinaryTCP
{

class FCompactBinaryTCPImpl;
struct FReceiveBuffer;
struct FSendBuffer;

// Our Socket API restricts us to int32 Count field (Max == 1 << 31 - 1), but we further
// restrict that by a factor of two to prevent impact from any OS limitations less than that
// and to allow room for the 16 byte PacketHeader.
constexpr uint64 MaxOSPacketSize = (1 << 30) - 1;

enum class EConnectionStatus
{
	/** Connection is still okay or operation succeeded. */
	Okay,
	/** Connection failed and no further use of the Socket is possible. */
	Terminated,
	/** The Socket is still active but the data received was invalid and recovery is not possible. */
	FormatError,
	/** The operation failed and the Socket is no longer usable. */
	Failed,
	/** The Socket is busy and the operation needs to be retried later */
	Incomplete,
};
const TCHAR* DescribeStatus(EConnectionStatus Status);

/** The atom of data that can be sent through CompactBinaryTCP: guid identifier and compact binary payload. */
struct FMarshalledMessage
{
	FGuid MessageType;
	FCbObject Object;
};

/**
 * Attempt to read messages from the socket. Returns all messages currently available and returns. Partially
 * transferred messages are stored in Buffer and can be completed by further calls to TryReadPacket.
 * 
 * @param MaxPacketSize If non-zero, all messages are assumed to be <= and a FormatError is given if they are larger.
 * 
 * @return EConnectionStatus::Okay on success and EConnectionStatus::Terminated or EConnection::FormatError on error.
 * If EConnectionStatus::Okay is returned, Messages may be empty (no message yet received) or populated.
 */
EConnectionStatus TryReadPacket(FSocket* Socket, FReceiveBuffer& Buffer,
	TArray<FMarshalledMessage>& Messages, uint64 MaxPacketSize=0);

/**
 * Attempt to write messages to the socket. If the socket is busy the new messages are stored in Buffer and will be
 * sent on a future call to TryWritePacket.
 * 
 * @param MaxPacketSize When combining multiple messages into a single packet, messages will stopped being added and
 * will be pushed into the next packet if the packet size is > this amount. If 0, will use the maximum size allowed
 * by the socket, which is >= 2^30. If a single message has larger size than MaxPacketSize, EConnectionStatus::Failed
 * is return and the socket will no longer be usable.
 * 
 * @return EConnectionStatus::Okay on success and all messages have been sent. EConnectionStatus::Incomplete if some
 * messages could not yet be sent. EConnectionStatus::Failed if connection failed and socket is no longer usable.
 */
EConnectionStatus TryWritePacket(FSocket* Socket, FSendBuffer& Buffer,
	TArray<FMarshalledMessage>&& AppendMessages, uint64 MaxPacketSize = 0);
EConnectionStatus TryWritePacket(FSocket* Socket, FSendBuffer& Buffer,
	FMarshalledMessage&& AppendMessage, uint64 MaxPacketSize = 0);
void QueueMessage(FSendBuffer& Buffer, FMarshalledMessage&& Message);


/** Attempt to finish sending packets that were previously queued by TryWritePacket */
EConnectionStatus TryFlushBuffer(FSocket* Socket, FSendBuffer& Buffer, uint64 MaxPacketSize = 0);

/**
 * Holds the state of the CompactBinaryTCP communication read from a Socket. Expects the socket to be a
 * series of packets of messages and keeps track of the next packet when a packet is as yet only partially received.
 */
struct FReceiveBuffer
{
public:
	void Reset();

private:
	FUniqueBuffer Payload;
	uint64 BytesRead = 0;
	bool bParsedHeader = false;

	friend class UE::CompactBinaryTCP::FCompactBinaryTCPImpl;
};

/**
 * Holds the state of the CompactBinaryTCP communication written to a Socket. Allows the socket to be a
 * series of packets of messages and keeps track of a partially sent packet and of pending messages.
 */
struct FSendBuffer
{
public:
	void Reset();

private:
	TArray<FMarshalledMessage> PendingMessages;
	FUniqueBuffer Payload;
	uint64 BytesWritten = 0;
	bool bCreatedPayload = false;
	bool bSentHeader = false;

	friend class UE::CompactBinaryTCP::FCompactBinaryTCPImpl;
};

}

template <typename KeyType, typename KeyFuncs, typename Allocator>
inline bool LoadFromCompactBinary(FCbFieldView Field, TSet<KeyType, KeyFuncs, Allocator>& OutValue)
{
	OutValue.Reset();
	OutValue.Reserve(Field.AsArrayView().Num());
	bool bOk = !Field.HasError();
	for (const FCbFieldView& ElementField : Field)
	{
		KeyType Key;
		if (LoadFromCompactBinary(ElementField, Key))
		{
			OutValue.Add(MoveTemp(Key));
		}
		else
		{
			bOk = false;
		}
	}
	return bOk;
}

template <typename KeyType, typename KeyFuncs, typename Allocator,
	std::void_t<decltype(std::declval<FCbWriter&>() << std::declval<const KeyType&>())>* = nullptr>
inline FCbWriter& operator<<(FCbWriter& Writer, const TSet<KeyType, KeyFuncs, Allocator>& Value)
{
	Writer.BeginArray();
	for (const KeyType& Key : Value)
	{
		Writer << Key;
	}
	Writer.EndArray();
	return Writer;
}

template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
inline bool LoadFromCompactBinary(FCbFieldView Field, TMap<KeyType, ValueType, SetAllocator, KeyFuncs>& OutValue)
{
	OutValue.Reset();
	OutValue.Reserve(Field.AsArrayView().Num());
	bool bOk = !Field.HasError();
	for (const FCbFieldView& PairField : Field)
	{
		bOk &= PairField.IsObject();
		KeyType Key;
		if (LoadFromCompactBinary(PairField["K"], Key))
		{
			ValueType& Value = OutValue.FindOrAdd(MoveTemp(Key));
			bOk = LoadFromCompactBinary(PairField["V"], Value) & bOk;
		}
		else
		{
			bOk = false;
		}
	}
	return bOk;
}

template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs,
	std::void_t<decltype(std::declval<FCbWriter&>() << std::declval<const KeyType&>())>* = nullptr,
	std::void_t<decltype(std::declval<FCbWriter&>() << std::declval<const ValueType&>())>* = nullptr>
inline FCbWriter& operator<<(FCbWriter& Writer, const TMap<KeyType, ValueType, SetAllocator, KeyFuncs>& Value)
{
	Writer.BeginArray();
	for (const auto& Pair : Value)
	{
		Writer.BeginObject();
		Writer << "K" << Pair.Key;
		Writer << "V" << Pair.Value;
		Writer.EndObject();
	}
	Writer.EndArray();
	return Writer;
}

inline FCbWriter& operator<<(FCbWriter& Writer, const UE::CompactBinaryTCP::FMarshalledMessage& Value)
{
	Writer.BeginObject();
	Writer << "T" << Value.MessageType;
	Writer << "V" << Value.Object;
	Writer.EndObject();
	return Writer;
}

UNREALED_API bool LoadFromCompactBinary(FCbFieldView Field, UE::CompactBinaryTCP::FMarshalledMessage& Value);

// FSoftObjectPath has an implicit constructor from FString for backwards compatibility; if we
// try to create an operator<< for it, it will cause operator<< to be ambiguous.
// This prevents us from using it directly in containers. To hack around this,
// containers of FSoftObjectPath need to be reinterpret_casted to containers of FSoftObjectPathSerializationWrapper
struct FSoftObjectPathSerializationWrapper
{
	FSoftObjectPath Inner;
	bool operator==(const FSoftObjectPathSerializationWrapper& Other) const
	{
		return Inner == Other.Inner;
	}
	friend uint32 GetTypeHash(const FSoftObjectPathSerializationWrapper& Wrapper)
	{
		return GetTypeHash(Wrapper.Inner);
	}
};

FCbWriter& operator<<(FCbWriter& Writer, const FSoftObjectPathSerializationWrapper& Path);
bool LoadFromCompactBinary(FCbFieldView Field, FSoftObjectPathSerializationWrapper& Path);

