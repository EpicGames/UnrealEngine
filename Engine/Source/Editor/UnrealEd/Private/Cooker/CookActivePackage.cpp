// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookActivePackage.h"

#include "Containers/StripedMap.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/PlatformTLS.h"

// Thread-safe map: ThreadId -> active cook package name.
// Replaces the former thread_local approach which failed because the crash
// delegate was invoked from the crash-handler thread, reading the wrong TLS.
// Updated by FScopedCookActivePackage and by the InheritedContext extension
// (so child tasks are reflected too).
static TStripedMap<1, uint32, FName>& GetCookActivePackageMap()
{
	static TStripedMap<1, uint32, FName> Map;

#if WITH_ADDITIONAL_CRASH_CONTEXTS
	// Register crash context callback once. Uses GetCrashedThreadId() to look up
	// the crashing thread's entry even when the callback runs on a different thread.
	static bool bRegistered = []()
	{
		FGenericCrashContext::OnAdditionalCrashContextDelegateEx().AddStatic(
			[](FCrashContextExtendedWriter& Writer, const FAdditionalCrashContextParams& Params)
			{
				FName PackageName = Map.FindRef(Params.CrashedThreadId);
				if (!PackageName.IsNone())
				{
					Writer.AddString(TEXT("ActivePackage"), *WriteToString<256>(PackageName));
				}
			});
		return true;
	}();
#endif

	return Map;
}

// Getter/setter for the InheritedContext extension.
// When a child task is launched, the InheritedContext system calls
// GetCookActivePackage() on the parent thread to capture the value,
// then SetCookActivePackage() on the child thread to apply it.
// Both go through the map, so the child thread's entry is visible
// to crash reporting.
static FName GetCookActivePackage()
{
	return GetCookActivePackageMap().FindRef(FPlatformTLS::GetCurrentThreadId());
}

static void SetCookActivePackage(FName PackageName)
{
	const uint32 ThreadId = FPlatformTLS::GetCurrentThreadId();
	if (PackageName.IsNone())
	{
		GetCookActivePackageMap().Remove(ThreadId);
	}
	else
	{
		GetCookActivePackageMap().Add(ThreadId, PackageName);
	}
}

static UE::FInheritedContextExtension& GetCookActivePackageExtension()
{
	static UE::FInheritedContextExtension Extension =
		UE::MakeInheritedContextExtension<&GetCookActivePackage, &SetCookActivePackage>();
	return Extension;
}

UE::Cook::FScopedCookActivePackage::FScopedCookActivePackage(FName PackageName, FName PackageTrackingOpsName)
	: PreviousPackageName(GetCookActivePackage())
	, ExtensionScope(GetCookActivePackageExtension())
{
	SetCookActivePackage(PackageName);
	if (!PackageTrackingOpsName.IsNone())
	{
		UE_TRACK_REFERENCING_PACKAGE_ACTIVATE_SCOPE_VARIABLE(TrackingScope, PackageName, PackageTrackingOpsName);
	}
}

UE::Cook::FScopedCookActivePackage::~FScopedCookActivePackage()
{
	SetCookActivePackage(PreviousPackageName);
}