// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/InheritedContext.h"
#include "Misc/PackageAccessTracking.h"
#include "UObject/NameTypes.h"

namespace UE::Cook
{

/**
 * RAII scope that sets the active cook package for crash-context diagnostics and package access tracking.
 * Thread-safe and task-friendly:
 * - Active package is stored in a global TStripedMap (ThreadId -> FName) so crash reporting works
 *   even when the crash handler runs on a different thread.
 * - Propagated to child tasks via FInheritedContextExtension (getter/setter go through the map,
 *   so child tasks are also visible in crash context).
 * - Package access tracking uses FPackageAccessRefScope which is already thread_local and InheritedContext-aware.
 */
struct FScopedCookActivePackage
{
	FScopedCookActivePackage(FName PackageName, FName PackageTrackingOpsName);
	~FScopedCookActivePackage();

	UE_NONCOPYABLE(FScopedCookActivePackage);

private:
	FName PreviousPackageName;
	UE::FInheritedContextExtensionScope ExtensionScope;
	UE_TRACK_REFERENCING_PACKAGE_DECLARE_SCOPE_VARIABLE(TrackingScope);
};

} // namespace UE::Cook
