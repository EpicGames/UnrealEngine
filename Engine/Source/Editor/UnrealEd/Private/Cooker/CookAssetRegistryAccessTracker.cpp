// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookAssetRegistryAccessTracker.h"

#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Cooker/CookAssetRegistryAccessTracker_AddPackageDep.h"
#include "Cooker/CookAssetRegistryAccessTracker_IgnoreScope.h"
#include "CookOnTheSide/CookLog.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/PackageAccessTracking.h"

namespace UE::CookAssetRegistryAccessTracker
{
	uint32 GetTypeHash(const FRecord& Data)
	{
		uint32 Hash = 0;
		Hash = HashCombineFast(Hash, GetTypeHash(Data.Platform));
		Hash = HashCombineFast(Hash, UE::AssetRegistry::GetTypeHash(Data.Filter));
		Hash = HashCombineFast(Hash, ::GetTypeHash(Data.bAddPackageDependencies));
		return Hash;
	}

	bool FRecord::operator==(const FRecord& Other) const
	{
		return Filter == Other.Filter && Platform == Other.Platform && bAddPackageDependencies == Other.bAddPackageDependencies;
	}

	bool FRecord::operator<(const FRecord& Other) const
	{
		if (Filter != Other.Filter)
		{
			return Filter < Other.Filter;
		}

		if (Platform != Other.Platform)
		{
			if (Platform == nullptr)
			{
				return true;
			}
			if (Other.Platform == nullptr)
			{
				return false;
			}

			return Platform->PlatformName() < Other.Platform->PlatformName();
		}

		if (bAddPackageDependencies != Other.bAddPackageDependencies)
		{
			if (!bAddPackageDependencies)
			{
				return true;
			}

			return false;
		}

		return false;
	}

	FCookAssetRegistryAccessTracker FCookAssetRegistryAccessTracker::Singleton;

	FCookAssetRegistryAccessTracker& FCookAssetRegistryAccessTracker::Get()
	{ 
		return Singleton; 
	}

	void FCookAssetRegistryAccessTracker::Init()
	{
		if (bEnabled)
		{
			return;
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.GetRegistry();
		Handle = AssetRegistry.OnEnumerateAssetsEvent().AddLambda([this](const FARCompiledFilter& Filter, UE::AssetRegistry::EEnumerateAssetsFlags Flag) 
			{ 
				OnEnumerateAssets(Filter, Flag);
			});

		bEnabled = true;
	}

	void FCookAssetRegistryAccessTracker::Shutdown()
	{
		if (!bEnabled)
		{
			return;
		}

		if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
		{
			IAssetRegistry& AssetRegistry = AssetRegistryModule->GetRegistry();
			AssetRegistry.OnEnumerateAssetsEvent().Remove(Handle);
		}

		bEnabled = false;
	}

	TArray<FRecord> FCookAssetRegistryAccessTracker::GetRecords(const FName& PackageName, const ITargetPlatform* Platform) const
	{
		TArray<FRecord> PlatformRecords;

		{
			UE::TUniqueLock RecordsLock(RecordsMutex);

			const TSet<FRecord>* FoundRecords = AccessRecords.Find(PackageName);
			if (!FoundRecords)
			{
				return TArray<FRecord>();
			}

			PlatformRecords = FoundRecords->Array();
		}

		for (TArray<FRecord>::TIterator Iter(PlatformRecords); Iter; ++Iter)
		{
			FRecord& Record = *Iter;
			if (Record.Platform != Platform && Record.Platform != nullptr)
			{
				Iter.RemoveCurrentSwap();
			}
		}

		Algo::Sort(PlatformRecords);
		PlatformRecords.SetNum(Algo::Unique(PlatformRecords));

		return PlatformRecords;
	}

	FCookAssetRegistryAccessTracker::FCookAssetRegistryAccessTracker()
		: AccessRecords()
		, Handle()
		, RecordsMutex()
		, bEnabled(false)
	{ }

	FCookAssetRegistryAccessTracker::~FCookAssetRegistryAccessTracker()
	{ }

	void FCookAssetRegistryAccessTracker::OnEnumerateAssets(const FARCompiledFilter& CompiledFilter, UE::AssetRegistry::EEnumerateAssetsFlags Flag)
	{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
		if (!bEnabled)
		{
			return;
		}

		if (FIgnoreScope::ShouldIgnoreAccessTracker())
		{
			return;
		}

		PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
		if (!AccumulatedScopeData || AccumulatedScopeData->BuildOpName.IsNone())
		{
			return;
		}

		FName Referencer = AccumulatedScopeData->PackageName;

		{
			UE::TUniqueLock RecordsLock(RecordsMutex);

			TSet<FRecord>& Records = AccessRecords.FindOrAdd(Referencer);
			FRecord NewRecord;
			NewRecord.Platform = AccumulatedScopeData->TargetPlatform;
			NewRecord.bAddPackageDependencies = FAddPackageDependenciesScope::GetAddPackageDependenciesEnabled();
			NewRecord.Filter = UE::AssetRegistry::ConvertToNonCompiledFilter(CompiledFilter);
			if (!NewRecord.Filter.bIncludeOnlyOnDiskAssets)
			{
				UE_LOGF(LogCook, Warning,
					"An AssetRegistry query is executing during a package load or save, and it uses an ARFilter with bIncludeOnlyOnDiskAssets=false. " "This will create cookindeterminism since the packages in can vary between cooks. Caller should set bIncludeOnlyOnDiskAssets=true. " "For the recorded dependency we are forcing bIncludeOnlyOnDiskAssets=true. Callstack of the AssetRegistryQuery construction: "
				);
				FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
				NewRecord.Filter.bIncludeOnlyOnDiskAssets = true;
			}
			NewRecord.Filter.SortForSaving();

			Records.Add(NewRecord);
		}
#endif // UE_WITH_PACKAGE_ACCESS_TRACKING
	}
}
