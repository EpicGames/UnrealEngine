// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "Async/Mutex.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/AssetRegistryInterface.h"
#include "UObject/NameTypes.h"

struct FAssetData;
class ITargetPlatform;

namespace UE::CookAssetRegistryAccessTracker
{
	struct FRecord
	{
		FARFilter Filter;
		const ITargetPlatform* Platform;

		/** 
		 * If true, when the dependency for the asset registry query is created, it will also create a package dependency for each package
		 * returned by the query.
		*/
		bool bAddPackageDependencies = false;

		bool operator==(const FRecord& Other) const;
		bool operator<(const FRecord& Other) const;
	};

	class FCookAssetRegistryAccessTracker : public FNoncopyable
	{
	public:
		static FCookAssetRegistryAccessTracker& Get();

		void Init();
		void Shutdown();

		TArray<FRecord> GetRecords(const FName& PackageName, const ITargetPlatform* Platform) const;

	private:
		FCookAssetRegistryAccessTracker();
		virtual ~FCookAssetRegistryAccessTracker();

		void OnEnumerateAssets(const FARCompiledFilter& Filter, UE::AssetRegistry::EEnumerateAssetsFlags Flag);

		static FCookAssetRegistryAccessTracker Singleton;

		TMap<FName, TSet<FRecord>> AccessRecords;

		FDelegateHandle Handle;

		// Use a mutex rather than a critical section for synchronization.  Calls into system libraries, such as windows critical section
		// functions, are 50 times more expensive on build farm VMs, radically affecting cook times, which this avoids. 
		mutable UE::FMutex RecordsMutex;
		bool bEnabled = false;
	};
}
