// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookConfigAccessTracker.h"

#if UE_WITH_CONFIG_TRACKING

#include "Algo/Unique.h"
#include "Algo/Sort.h"
#include "Async/UniqueLock.h"
#include "CookOnTheSide/CookLog.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"
#include "Misc/CString.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryWriter.h"

#endif

#if UE_WITH_CONFIG_TRACKING

DEFINE_LOG_CATEGORY_STATIC(LogConfigBuildDependencyTracker, Log, All);

namespace UE::ConfigAccessTracking
{

FCookConfigAccessTracker FCookConfigAccessTracker::Singleton;

TStringBuilder<256> ToString(FNameEntryId FileName, FNameEntryId SectionName, FMinimalName ValueName)
{
	return TStringBuilder<256>(InPlace, FileName, TEXT(":["), SectionName, TEXT("]:"), FName(ValueName));
}

void FCookConfigAccessTracker::Disable()
{
	if (bEnabled)
	{
		UE::ConfigAccessTracking::RemoveConfigValueReadCallback(OnConfigValueReadCallbackHandle);
		OnConfigValueReadCallbackHandle = UE::ConfigAccessTracking::FConfigValueReadCallbackId{};
		PackageRecords.Empty();
		bEnabled = false;
	}
}

bool FCookConfigAccessTracker::IsEnabled() const
{
	return bEnabled;
}

void FCookConfigAccessTracker::DumpStats() const
{
	using namespace UE::ConfigAccessTracking;

	if (!IsEnabled())
	{
		return;
	}

	UE::TUniqueLock RecordsScopeLock(RecordsLock);
	uint64 ReferencingPackageCount = 0;
	uint64 ReferenceCount = 0;
	uint64 GlobalReferenceCount = 0;
	for (const TPair<FName, TSet<FConfigAccessData>>& PackageAccessRecord : PackageRecords)
	{
		if (PackageAccessRecord.Key == NAME_None)
		{
			for (const FConfigAccessData& AccessedData : PackageAccessRecord.Value)
			{
				++GlobalReferenceCount;
			}
		}
		else
		{
			++ReferencingPackageCount;
			for (const FConfigAccessData& AccessedData : PackageAccessRecord.Value)
			{
				++ReferenceCount;
			}
		}
	}
	UE_LOGF(LogConfigBuildDependencyTracker, Display,
		"Config Accesses (%llu referencing packages with a total of %llu unique accesses). %llu unique accesses that were not associated with a package.",
		ReferencingPackageCount, ReferenceCount, GlobalReferenceCount);

	constexpr bool bDetailedDump = false;
	if (bDetailedDump)
	{
		UE_LOGF(LogConfigBuildDependencyTracker, Display, "=========================================================================");
		for (const TPair<FName, TSet<FConfigAccessData>>& PackageAccessRecord : PackageRecords)
		{
			UE_LOGF(LogConfigBuildDependencyTracker, Display, "%ls:", *PackageAccessRecord.Key.ToString());
			for (const FConfigAccessData& AccessedData : PackageAccessRecord.Value)
			{
				UE_LOGF(LogConfigBuildDependencyTracker, Display, "    %ls",
					*ToString(AccessedData.FileName, AccessedData.SectionName, AccessedData.ValueName));
			}
		}
	}
}

TArray<UE::ConfigAccessTracking::FConfigAccessData>
FCookConfigAccessTracker::GetPackageRecords(FName ReferencerPackage, const ITargetPlatform* TargetPlatform) const
{
	using namespace UE::ConfigAccessTracking;

	TArray<FConfigAccessData> Records;
	{
		UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
		const TSet<FConfigAccessData>* ReferencerSet = Singleton.PackageRecords.Find(ReferencerPackage);
		if (!ReferencerSet)
		{
			return TArray<FConfigAccessData>();
		}
		Records = ReferencerSet->Array();
	}
	SortRecordsAndFilterByPlatform(Records, TargetPlatform);
	return Records;
}

TArray<UE::ConfigAccessTracking::FConfigAccessData>
FCookConfigAccessTracker::GetCookRecords() const
{
	using namespace UE::ConfigAccessTracking;

	TSet<FConfigAccessData> CookRecords;
	{
		UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
		for (const TPair<FName, TSet<FConfigAccessData>>& Pair : Singleton.PackageRecords)
		{
			CookRecords.Append(Pair.Value);
		}
	}
	TArray<FConfigAccessData> ResultRecords = CookRecords.Array();
	Algo::Sort(ResultRecords);
	return ResultRecords;
}

TArray<UE::ConfigAccessTracking::FConfigAccessData>
FCookConfigAccessTracker::GetCookRecords(const ITargetPlatform* TargetPlatform) const
{
	using namespace UE::ConfigAccessTracking;

	TSet<FConfigAccessData> CookRecords;
	{
		UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
		for (const TPair<FName, TSet<FConfigAccessData>>& Pair : Singleton.PackageRecords)
		{
			CookRecords.Append(Pair.Value);
		}
	}
	TArray<FConfigAccessData> Records = CookRecords.Array();
	SortRecordsAndFilterByPlatform(Records, TargetPlatform);
	return Records;
}

void FCookConfigAccessTracker::SortRecordsAndFilterByPlatform(TArray<FConfigAccessData>& Records, const ITargetPlatform* TargetPlatform)
{
	// Remove records not relevant to the current platform, and set the TargetPlatform to null, so we can
	// remove records that are duplicated between platform keys.
	for (TArray<FConfigAccessData>::TIterator Iter(Records); Iter; ++Iter)
	{
		FConfigAccessData& Record = *Iter;
		if (Record.RequestingPlatform != TargetPlatform && Record.RequestingPlatform != nullptr)
		{
			Iter.RemoveCurrentSwap();
		}
		else
		{
			Record.RequestingPlatform = nullptr;
		}
	}
	Algo::Sort(Records);
	Records.SetNum(Algo::Unique(Records));
}

void FCookConfigAccessTracker::AddRecord(FName PackageName,
	const UE::ConfigAccessTracking::FConfigAccessData& AccessData)
{
	UE::TUniqueLock RecordsScopeLock(RecordsLock);
	PackageRecords.FindOrAdd(PackageName).Add(AccessData);
}

FCookConfigAccessTracker::FCookConfigAccessTracker()
{
	OnConfigValueReadCallbackHandle = UE::ConfigAccessTracking::AddConfigValueReadCallback(StaticOnConfigValueRead);
	bEnabled = true;
}

FCookConfigAccessTracker::~FCookConfigAccessTracker()
{
	Disable();
}

static FConfigFile* FindConfigCacheIniFile(FName ConfigPlatform, FName FileName)
{
	// ForPlatform will return GConfig if ConfigPlatform is NAME_None
	FConfigCacheIni* ConfigSystem = FConfigCacheIni::ForPlatform(ConfigPlatform);

	// The ini files may have been recorded by fullpath or by shortname; search first for a fullpath match using
	// FindConfigFile and if that fails search for the shortname match by iterating over all files in GConfig
	FConfigFile* ConfigFile = ConfigSystem->FindConfigFile(FileName.ToString());
	if (ConfigFile)
	{
		return ConfigFile;
	}

	for (const FString& ConfigFilename : ConfigSystem->GetFilenames())
	{
		ConfigFile = ConfigSystem->FindConfigFile(ConfigFilename);
		if (ConfigFile->Name == FileName)
		{
			return ConfigFile;
		}
	}
	return nullptr;
}

FString FCookConfigAccessTracker::GetValue(const FConfigAccessData& AccessData)
{
	if (AccessData.SectionName.IsNone() || AccessData.ValueName.IsNone())
	{
		return FString();
	}

	switch (AccessData.LoadType)
	{
	case ELoadType::ConfigSystem:
	{
		FIgnoreScope IgnoreScope;
		FConfigFile* ConfigFile = FindConfigCacheIniFile(AccessData.GetConfigPlatform(), AccessData.GetFileName());
		if (!ConfigFile)
		{
			return FString();
		}
		const FConfigSection* ConfigSection = ConfigFile->FindSection(AccessData.GetSectionName().ToString());
		if (!ConfigSection)
		{
			return FString();
		}
		return MultiValueToString(*ConfigSection, AccessData.GetValueName());
	}
	case ELoadType::LocalIniFile:
	case ELoadType::LocalSingleIniFile:
	case ELoadType::ExternalIniFile:
	case ELoadType::ExternalSingleIniFile:
	{
		FConfigAccessData PathOnlyData = AccessData.GetPathOnlyData();
		FConfigAccessData FileOnlyData = PathOnlyData.GetFileOnlyData();
		uint32 PathOnlyDataHash = GetTypeHash(PathOnlyData);
		uint32 FileOnlyDataHash = GetTypeHash(FileOnlyData);
		{
			UE::TUniqueLock ConfigCacheScopeLock(ConfigCacheLock);
			if (LoadedConfigFiles.ContainsByHash(FileOnlyDataHash, FileOnlyData))
			{
				FString* Result = LoadedValues.FindByHash(PathOnlyDataHash, PathOnlyData);
				return Result ? *Result : FString();
			}
		}

		FIgnoreScope IgnoreScope;
		FConfigFile Buffer;
		const FConfigFile* LoadedFile = FindOrLoadConfigFile(FileOnlyData, Buffer);
		if (!LoadedFile)
		{
			return FString();
		}
		RecordValuesFromFile(FileOnlyData, *LoadedFile);
		{
			UE::TUniqueLock ConfigCacheScopeLock(ConfigCacheLock);
			FString* Result = LoadedValues.FindByHash(PathOnlyDataHash, PathOnlyData);
			return Result ? *Result : FString();
		}
	}
	default:
		return FString();
	}
}

FString FCookConfigAccessTracker::GetValue(FStringView AccessDataFullPath)
{
	return GetValue(FConfigAccessData::Parse(AccessDataFullPath));
}

void FCookConfigAccessTracker::StaticOnConfigValueRead(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
	const FConfigValue& ConfigValue)
{
	if (!Section)
	{
		return;
	}
	UE::ConfigAccessTracking::FFile* FileAccess = Section->FileAccess.GetReference();
	if (!FileAccess)
	{
		return;
	}
	const FConfigFile* ConfigFile = FileAccess->ConfigFile;
	if (!ConfigFile)
	{
		return;
	}
	if (!UE::ConfigAccessTracking::IsLoadableLoadType(ConfigFile->LoadType))
	{
		return;
	}
	FNameEntryId FileName = FileAccess->GetFilenameToLoad().GetComparisonIndex();
	if (FileName.IsNone())
	{
		return;
	}
	FNameEntryId SectionName = Section->SectionName;
	if (SectionName.IsNone())
	{
		return;
	}
	FNameEntryId ConfigPlatform = FileAccess->GetPlatformName().GetComparisonIndex();

	FName Referencer = NAME_None;
	const ITargetPlatform* RequestedPlatform = nullptr;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (AccumulatedScopeData && !AccumulatedScopeData->BuildOpName.IsNone())
	{
		RequestedPlatform = AccumulatedScopeData->TargetPlatform;
		Referencer = AccumulatedScopeData->PackageName;

		if (AccumulatedScopeData->OpName == PackageAccessTrackingOps::NAME_NoAccessExpected)
		{
			UE_LOGF(LogConfigBuildDependencyTracker, Warning,
				"Object %ls is referencing config value %ls inside of a NAME_NoAccessExpected scope. Programmer should narrow the scope or debug the reference.",
				*Referencer.ToString(), *ToString(FileName, SectionName, ValueName));
		}
	}
#endif

	LLM_SCOPE_BYNAME("ConfigAccessTracking");
	FConfigAccessData AccessData(ConfigFile->LoadType, ConfigPlatform, FileName, SectionName, ValueName, RequestedPlatform);
	uint32 ReferencerHash = GetTypeHash(Referencer);
	uint32 AccessDataHash = GetTypeHash(AccessData);
	FConfigAccessData FileOnlyData = AccessData.GetFileOnlyData();
	uint32 FileOnlyHash = GetTypeHash(FileOnlyData);
	{
		UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
		Singleton.PackageRecords.FindOrAddByHash(ReferencerHash, Referencer).AddByHash(AccessDataHash, MoveTemp(AccessData));
	}
	bool bNeedRecordValuesFromFile = false;
	{
		UE::TUniqueLock ConfigCacheScopeLock(Singleton.ConfigCacheLock);
		bNeedRecordValuesFromFile = !Singleton.LoadedConfigFiles.ContainsByHash(FileOnlyHash, FileOnlyData);
	}
	if (bNeedRecordValuesFromFile)
	{
		Singleton.RecordValuesFromFile(FileOnlyData, *ConfigFile);
	}
}

void FCookConfigAccessTracker::RecordValuesFromFile(const FConfigAccessData& FileOnlyData,
	const FConfigFile& ConfigFile)
{
	FConfigAccessData FullPathData = FileOnlyData.GetFileOnlyData();

	uint32 FullPathHash = GetTypeHash(FullPathData);
	bool bAlreadyExists = false;
	UE::TUniqueLock ConfigCacheScopeLock(ConfigCacheLock);
	LoadedConfigFiles.FindOrAddByHash(FullPathHash, FullPathData, &bAlreadyExists);
	if (bAlreadyExists)
	{
		return;
	}
	FIgnoreScope IgnoreScope;

	TArray<FName> ValueNames;
	for (const TPair<FString, FConfigSection>& SectionPair : ConfigFile)
	{
		FullPathData.SectionName = FName(FStringView(SectionPair.Key), NAME_NO_NUMBER_INTERNAL).GetComparisonIndex();
		const FConfigSection& Section = SectionPair.Value;
		ValueNames.Reset();
		SectionPair.Value.GetKeys(ValueNames);
		for (FName ValueName : ValueNames)
		{
			FullPathData.ValueName = FMinimalName(ValueName);
			LoadedValues.FindOrAdd(FullPathData) = MultiValueToString(Section, ValueName);
		}
	}
}

FString FCookConfigAccessTracker::MultiValueToString(const FConfigSection& Section, FName ValueName)
{
	TArray<const FConfigValue*, TInlineAllocator<8>> Values;
	Section.MultiFindPointer(ValueName, Values, true /* bMaintainOrder */);
	if (Values.IsEmpty())
	{
		return FString();
	}
	if (Values.Num() == 1)
	{
		return Values[0]->GetValue();
	}
	TStringBuilder<256> ArrayValueStr;
	ArrayValueStr << Values[0]->GetValue();
	for (const FConfigValue* Value : TConstArrayView<const FConfigValue*>(Values).RightChop(1))
	{
		ArrayValueStr << TEXT("\n") << Value->GetValue();
	}
	return FString(*ArrayValueStr);
}

const FConfigFile* FindOrLoadConfigFile(const FConfigAccessData& AccessData, FConfigFile& Buffer)
{
	FName ConfigPlatform(AccessData.GetConfigPlatform());
	FName FileName(AccessData.GetFileName());

	switch (AccessData.LoadType)
	{
	case ELoadType::ConfigSystem:
	{
		return FindConfigCacheIniFile(ConfigPlatform, FileName);
	}
	case ELoadType::LocalIniFile:
		if (FConfigCacheIni::LoadLocalIniFile(Buffer, *WriteToString<128>(FileName), true /* bIsBaseIniName */,
			*WriteToString<64>(ConfigPlatform)))
		{
			return &Buffer;
		}
		return nullptr;
	case ELoadType::LocalSingleIniFile:
		if (FConfigCacheIni::LoadLocalIniFile(Buffer, *WriteToString<128>(FileName), false /* bIsBaseIniName */,
			*WriteToString<64>(ConfigPlatform)))
		{
			return &Buffer;
		}
		return nullptr;
	case ELoadType::ExternalIniFile:
		// TODO: LoadExternalIniFile is the same as LoadLocalIniFile, but with possibly redirected
		// EngineConfigDir and ProjectConfigDir. We can not load them without that extra information.
		// For now, assume it used the default EngineConfigDir and ProjectConfigDir.
		if (FConfigCacheIni::LoadLocalIniFile(Buffer, *WriteToString<128>(FileName), true /* bIsBaseIniName */,
			*WriteToString<64>(ConfigPlatform)))
		{
			return &Buffer;
		}
		return nullptr;
	case ELoadType::ExternalSingleIniFile:
		// TODO: Same comment as in ExternalIniFile
		if (FConfigCacheIni::LoadLocalIniFile(Buffer, *WriteToString<128>(FileName), false /* bIsBaseIniName */,
			*WriteToString<64>(ConfigPlatform)))
		{
			return &Buffer;
		}
		return nullptr;
	default:
		return nullptr;
	}
}

/** Return whether LoadType is a type that can be loaded by FindOrLoadConfigFile. */
bool IsLoadableLoadType(ELoadType LoadType)
{
	switch (LoadType)
	{
	case ELoadType::ConfigSystem:
	case ELoadType::LocalIniFile:
	case ELoadType::LocalSingleIniFile:
	case ELoadType::ExternalIniFile:
	case ELoadType::ExternalSingleIniFile:
		return true;
	default:
		return false;
	}
}

} // namespace UE::ConfigAccessTracking

namespace UE::ConfigAccessTracking
{

FGuid FConfigAccessTrackingCollector::MessageType(TEXT("B3F36AFEF6AE467E9E8F0DDA604856C3"));
static const FUtf8StringView ConfigDependencyCollectorRecordsName = UTF8TEXTVIEW("R");

void FConfigAccessTrackingCollector::ClientTick(UE::Cook::FMPCollectorClientTickContext& Context)
{
	using namespace UE::ConfigAccessTracking;

	if (!Context.IsFlush())
	{
		return;
	}

	TArray<FConfigAccessData> Records = FCookConfigAccessTracker::Get().GetCookRecords();

	FCbWriter Writer;
	Writer.BeginObject();
	Writer.SetName(ConfigDependencyCollectorRecordsName);
	Writer.BeginArray();
	for (FConfigAccessData& Record : Records)
	{
		Writer.BeginArray();
		uint8 LoadType = static_cast<uint8>(Record.LoadType);
		FName ConfigPlatform(Record.GetConfigPlatform());
		FName FileName(Record.GetFileName());
		FName SectionName(Record.GetSectionName());
		FName ValueName(Record.GetValueName());
		Writer << LoadType << ConfigPlatform << FileName << SectionName << ValueName
			<< Context.PlatformToInt(Record.RequestingPlatform);
		Writer.EndArray();
	}
	Writer.EndArray();
	Writer.EndObject();
	Context.AddMessage(Writer.Save().AsObject());
}

void FConfigAccessTrackingCollector::ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context,
	FCbObjectView Message)
{
	using namespace UE::ConfigAccessTracking;

	FCookConfigAccessTracker& Tracker = FCookConfigAccessTracker::Get();

	FCbFieldView RecordsField = Message[ConfigDependencyCollectorRecordsName];
	FCbArrayView RecordsView = RecordsField.AsArrayView();
	if (RecordsField.HasError())
	{
		UE_LOGF(LogCook, Error,
			"Corrupt message received from CookWorker when replicating ConfigDependencies. FalsePositiveIncrementalSkips may occur in next cook.");
		return;
	}
	for (FCbFieldView RecordField : RecordsView)
	{
		FCbArrayView RecordArray = RecordField.AsArrayView();
		FCbFieldViewIterator RecordIt = RecordArray.CreateViewIterator();
		FConfigAccessData Record;

		Record.LoadType = static_cast<ELoadType>(RecordIt.AsUInt8());
		++RecordIt;
		Record.ConfigPlatform= FName(RecordIt.AsString()).GetComparisonIndex();
		++RecordIt;
		Record.FileName = FName(RecordIt.AsString()).GetComparisonIndex();
		++RecordIt;
		Record.SectionName = FName(RecordIt.AsString(), NAME_NO_NUMBER_INTERNAL).GetComparisonIndex();
		++RecordIt;
		Record.ValueName = FMinimalName(FName(RecordIt.AsString()));
		++RecordIt;
		Record.RequestingPlatform = Context.IntToPlatform(RecordIt.AsUInt8());
		if (RecordIt.HasError())
		{
			UE_LOGF(LogCook, Error,
				"Corrupt message received from CookWorker when replicating ConfigDependencies. FalsePositiveIncrementalSkips may occur in next cook.");
			return;
		}
		Tracker.AddRecord(NAME_None, Record);
	}
}

} // namespace UE::ConfigAccessTracking

#endif // UE_WITH_CONFIG_TRACKING
