// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigAccessTracking.h"

#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"

#if UE_WITH_CONFIG_TRACKING
#include "Async/Mutex.h"
#include "Cooker/MPCollector.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/ConfigAccessData.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageAccessTracking.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#endif

#if UE_WITH_CONFIG_TRACKING

namespace UE::ConfigAccessTracking
{

/**
 * Tracker that subscribes to AddConfigValueReadCallback and for each access records the access associated with
 * the package that is currently in scope according to PackageAccessTracking_Private.
 */
class FCookConfigAccessTracker: public FNoncopyable
{
public:
	static FCookConfigAccessTracker& Get() { return Singleton; }

	void Disable();
	bool IsEnabled() const;
	void DumpStats() const;
	/**
	 * Get records requested for the given package and given platform, including RequestingPlatform=nullptr.
	 * Returned records are SORTED by FConfigAccessData::operator<.
	 */
	TArray<FConfigAccessData> GetPackageRecords(FName ReferencerPackage, const ITargetPlatform* TargetPlatform) const;
	/**
	 * Get records for all requesting packages, including records not associated with a package.
	 * Returned records are SORTED by FConfigAccessData::operator<.
	 */
	TArray<FConfigAccessData> GetCookRecords() const;
	/**
	 * Get records requested for all requesting packages, including records not associated with a package,
	 * but filtered by the given TargetPlatform. Includes records requested with no RequestingPlatform.
	 * TargetPlatform==nullptr returns only records requested with no RequestingPlatform.
	 * Returned records are SORTED by FConfigAccessData::operator<.
	 */
	TArray<FConfigAccessData> GetCookRecords(const ITargetPlatform* TargetPlatform) const;
	/** Add a record as if requested by the given package, or not associated with a package if PackageName.IsNone(). */
	void AddRecord(FName PackageName, const FConfigAccessData& Data);

	/** Lookup in GConfig, LoadConfigFile, or in already-cached values a value indicated by an FConfigAccessData. */
	FString GetValue(const FConfigAccessData& AccessData);
	/** Unmarshal a FConfigAccessData.FullPathToString string back to an FConfigAccessData and lookup its value. */
	FString GetValue(FStringView AccessDataFullPath);

private:
	FCookConfigAccessTracker();
	virtual ~FCookConfigAccessTracker();

	void RecordValuesFromFile(const FConfigAccessData& FileOnlyData, const FConfigFile& ConfigFile);

	/** Track object reference reads */
	static void StaticOnConfigValueRead(UE::ConfigAccessTracking::FSection* Section, FMinimalName ValueName,
		const FConfigValue& ConfigValue);

	/** Helper function for GetRecords functions. */
	static void SortRecordsAndFilterByPlatform(TArray<FConfigAccessData>& Records,
		const ITargetPlatform* TargetPlatform);

	static FString MultiValueToString(const FConfigSection& Section, FName ValueName);

private:
	// Use a mutex rather than a critical section for synchronization.  Calls into system libraries, such as windows critical section
	// functions, are 50 times more expensive on build farm VMs, radically affecting cook times, which this avoids. 
	mutable UE::FMutex RecordsLock;
	mutable UE::FMutex ConfigCacheLock;
	TMap<FName, TSet<UE::ConfigAccessTracking::FConfigAccessData>> PackageRecords;
	TSet<FConfigAccessData> LoadedConfigFiles;
	TMap<FConfigAccessData, FString> LoadedValues;
	UE::ConfigAccessTracking::FConfigValueReadCallbackId OnConfigValueReadCallbackHandle;
	bool bEnabled = false;

private:
	static FCookConfigAccessTracker Singleton;
};

/**
 * Find a ConfigFile by name and ConfigPlatform, either in GConfig or loaded from disk.
 * @param AccessData Specifies the LoadType, ConfigPlatform, Filename to load.
 * @param Buffer FConfigFile Buffer that will hold the result if LoadConfigFile was called
 * @param The discovered configfile, or nullptr.
 */
const FConfigFile* FindOrLoadConfigFile(const FConfigAccessData& AccessData, FConfigFile& Buffer);

/** Return whether LoadType is a type that can be loaded by FindOrLoadConfigFile. */
bool IsLoadableLoadType(ELoadType LoadType);

} // namespace UE::ConfigAccessTracking

namespace UE::ConfigAccessTracking
{

/** CookMultiprocess collector for ConfigAccess data. */
class FConfigAccessTrackingCollector : public UE::Cook::IMPCollector
{
public:
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("FConfigAccessTrackingCollector"); }

	virtual void ClientTick(UE::Cook::FMPCollectorClientTickContext& Context) override;
	virtual void ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context,
		FCbObjectView Message) override;

	static FGuid MessageType;
};

}

#endif // #if UE_WITH_CONFIG_TRACKING
