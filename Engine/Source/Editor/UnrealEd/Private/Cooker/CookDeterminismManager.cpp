// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookDeterminismManager.h"

#include "Cooker/CookArtifact.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

namespace UE::Cook
{

constexpr const ANSICHAR* DeterminismManagerName = "DeterminismManager";
constexpr int32 DeterminismManagerVersion = 1;

FDeterminismManager::FDeterminismManager()
	: PackageData(*this)
{
}

void FDeterminismManager::BeginPackage(UPackage* InPackage, const ITargetPlatform* TargetPlatform,
	ICookedPackageWriter* InOplogProvider)
{
	check(InPackage);
	check(InOplogProvider);
	Package = InPackage;
	OplogProvider = InOplogProvider;
	if (!OplogAvailable.IsSet())
	{
		OplogAvailable = OplogProvider->GetCookCapabilities().bOplogAttachments;
	}
	// We require an empty PackageData to populate; we expect it to be empty from constructor or cleared by EndPackage.
	check(PackageData.IsEmpty());
	PackageData.TargetPlatform = TargetPlatform;
}

void FDeterminismManager::RegisterDeterminismHelper(UObject* SourceObject,
	const TRefCountPtr<IDeterminismHelper>& DeterminismHelper)
{
	if (!DeterminismHelper)
	{
		return;
	}

	FExportDeterminismData& ExportData = PackageData.FindOrAddExportData(SourceObject);
	ExportData.DeterminismHelpers.Add(DeterminismHelper);

	FDeterminismConstructDiagnosticsContext Context(ExportData);
	DeterminismHelper->ConstructDiagnostics(Context);
}

void FDeterminismManager::RecordPackageModified(UObject* InPrimaryAsset)
{
	PackageData.bModified = true;
	if (InPrimaryAsset && InPrimaryAsset->GetPackage() != Package)
	{
		InPrimaryAsset = nullptr;
	}
	PackageData.PrimaryAsset = InPrimaryAsset;
	if (PackageData.PrimaryAsset)
	{
		FExportDeterminismData& ExportData = PackageData.FindOrAddExportData(PackageData.PrimaryAsset);
		ExportData.bPrimaryAsset = true;
	}

	FetchOldDiagnostics();
}

void FDeterminismManager::RecordExportModified(const FString& ExportPathName)
{
	UObject* Export = StaticFindObject(nullptr, nullptr, *ExportPathName);
	if (!Export || Export->GetPackage() != Package)
	{
		return;
	}

	FExportDeterminismData& ExportData = PackageData.FindOrAddExportData(Export);
	ExportData.bModified = true;
}

FString FDeterminismManager::GetCurrentPackageDiagnosticsAsText()
{
	if (!OplogAvailable.Get(false /* default value */))
	{
		// Comparison text is not implemented when we are unable to fetch attachments from the oplog.
		return FString();
	}

	PackageData.Sort();

	for (TPair<UObject*, TUniquePtr<FExportDeterminismData>>& ExportPair : PackageData.Exports)
	{
		FExportDeterminismData& ExportData = *ExportPair.Value;
		ExportData.Sort();
	}

	TStringBuilder<256> Logger;
	for (TPair<UObject*, TUniquePtr<FExportDeterminismData>>& ExportPair : PackageData.Exports)
	{
		FExportDeterminismData& ExportData = *ExportPair.Value;
		ExportData.Logger = &Logger;
		for (TRefCountPtr<IDeterminismHelper>& Helper : ExportData.DeterminismHelpers)
		{
			Helper->OnPackageModified(ExportData);
		}
		ExportData.Logger = nullptr;
	}
	if (Logger.ToView().EndsWith('\n'))
	{
		Logger.RemoveSuffix(Logger.ToView().EndsWith(TEXT("\r\n"))? 2 : 1);
	}
	return FString(*Logger);
}

void FDeterminismManager::AppendCommitAttachments(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context)
{
	PackageData.Sort();
	FCbWriter Writer;
	if (TrySave(Writer))
	{
		Context.SetAttachment(DeterminismManagerName, Writer.Save().AsObject());
	}
}

void FDeterminismManager::EndPackage()
{
	Package = nullptr;
	OplogProvider = nullptr;
	PackageData = FPackageDeterminismData(*this);
}

void FDeterminismManager::FetchOldDiagnostics()
{
	if (!OplogAvailable.Get(false /* default value */))
	{
		return;
	}
	FCbObject MarshalledData = OplogProvider->GetOplogAttachment(Package->GetFName(), DeterminismManagerName);
	TryLoad(MarshalledData.AsFieldView());
}

bool FDeterminismManager::TrySave(FCbWriter& Writer)
{
	bool bHasValues = false;

	PackageData.Sort();
	Writer.BeginObject();
	{
		Writer << "Version" << DeterminismManagerVersion;
		Writer.BeginArray("Exports");
		TStringBuilder<256> ExportPackagePath;
		for (TPair<UObject*, TUniquePtr<FExportDeterminismData>>& ExportPair : PackageData.Exports)
		{
			UObject* Export = ExportPair.Key;
			FExportDeterminismData& ExportData = *ExportPair.Value;
			if (Export == Package || ExportData.NewDiagnostics.IsEmpty())
			{
				continue;
			}
			ExportPackagePath.Reset();
			ExportPair.Key->GetPathName(Package, ExportPackagePath);
			if (ExportPackagePath.Len() == 0)
			{
				continue;
			}

			bHasValues = true;
			Writer.BeginArray();
			{
				Writer << ExportPackagePath;
				Writer.BeginArray();
				for (TPair<FUtf8String, FCbField>& DiagnosticPair : ExportData.NewDiagnostics)
				{
					Writer.BeginArray();
					Writer << DiagnosticPair.Key;
					Writer << DiagnosticPair.Value;
					Writer.EndArray();
				}
				Writer.EndArray();
			}
			Writer.EndArray();
		}
		Writer.EndArray();
	}
	Writer.EndObject();

	return bHasValues;
}

bool FDeterminismManager::TryLoad(FCbFieldView Field)
{
	int32 Version = Field["Version"].AsInt32();
	if (Version != DeterminismManagerVersion)
	{
		return false;
	}

	for (FCbFieldView ExportPairField : Field["Exports"])
	{
		FCbFieldViewIterator ExportPairIter = ExportPairField.CreateViewIterator();
		FString ExportPackagePath;
		if (!LoadFromCompactBinary(*ExportPairIter++, ExportPackagePath))
		{
			continue;
		}
		if (ExportPackagePath.IsEmpty())
		{
			// This could indicate the Package itself, but we don't allow recording diagnostics for the package itself
			// because LinkerLoad does not record it as a serialized export.
			continue;
		}

		UObject* Export = StaticFindObject(nullptr, Package, *ExportPackagePath);
		if (!Export)
		{
			continue;
		}
		FExportDeterminismData* AllocatedExportData = nullptr;
		auto AllocateExportData = [this, Export, &AllocatedExportData]()
			{
				if (!AllocatedExportData)
				{
					AllocatedExportData = &PackageData.FindOrAddExportData(Export);
				}
				return AllocatedExportData;
			};

		FCbFieldView DiagnosticArrayField = (*ExportPairIter++);
		for (FCbFieldView DiagnosticPairField : DiagnosticArrayField)
		{
			FCbFieldViewIterator DiagnosticPairIter = DiagnosticPairField.CreateViewIterator();
			FUtf8StringView DiagnosticName = (*DiagnosticPairIter).AsString();
			if ((*DiagnosticPairIter++).HasError())
			{
				continue;
			}
			FCbField DiagnosticValue = FCbField::Clone(*DiagnosticPairIter++);
			AllocateExportData()->AddOldDiagnostic(DiagnosticName, DiagnosticValue);
		}
	}

	return true;
}

FDeterminismConstructDiagnosticsContext::FDeterminismConstructDiagnosticsContext(FExportDeterminismData& InExportData)
	: ExportData(InExportData)
{
}

const ITargetPlatform* FDeterminismConstructDiagnosticsContext::GetTargetPlatform()
{
	return ExportData.PackageData.TargetPlatform;
}

void FDeterminismConstructDiagnosticsContext::AddDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value)
{
	ExportData.AddNewDiagnostic(DiagnosticName, Value);
}

bool FDeterminismConstructDiagnosticsContext::FullDDCKeysRequested() const
{
	return false;
}

FExportDeterminismData::FExportDeterminismData(FPackageDeterminismData& InPackageData, UObject* InExport)
	: PackageData(InPackageData)
	, Export(InExport)
{
}

bool FExportDeterminismData::IsModified()
{
	return bModified;
}

bool FExportDeterminismData::IsPrimaryAsset()
{
	return bPrimaryAsset;
}

const ITargetPlatform* FExportDeterminismData::GetTargetPlatform()
{
	return PackageData.TargetPlatform;
}

const TMap<FUtf8String, FCbField>& FExportDeterminismData::GetOldDiagnostics()
{
	Sort();
	return OldDiagnostics;
}

const TMap<FUtf8String, FCbField>& FExportDeterminismData::GetNewDiagnostics()
{
	Sort();
	return NewDiagnostics;
}

void FExportDeterminismData::AppendLog(FStringView LogText)
{
	if (Logger)
	{
		Logger->Append(LogText);
	}
}

void FExportDeterminismData::AppendDiagnostics()
{
	if (!bAppendedDiagnostics)
	{
		AppendLog(GetCompareText());
		bAppendedDiagnostics = true;
	}
}

IDeterminismModifiedPackageContext& FExportDeterminismData::GetPackageContext()
{
	return PackageData;
}

struct FCompareDiagnosticName
{
	bool operator()(const FUtf8String& A, const FUtf8String& B) const
	{
		int32 Compare = A.Compare(B, ESearchCase::IgnoreCase);
		if (Compare != 0)
		{
			return Compare < 0;
		}
		return A.Compare(B, ESearchCase::CaseSensitive) < 0;
	}
};

FString FExportDeterminismData::GetCompareText()
{
	if (OldDiagnostics.IsEmpty() && NewDiagnostics.IsEmpty())
	{
		return FString();
	}

	Sort();
	TStringBuilder<64> ExportRelPath;
	Export->GetPathName(PackageData.Owner->Package, ExportRelPath);
	TSet<FUtf8String> Keys;
	for (const TPair<FUtf8String, FCbField>& Pair : OldDiagnostics)
	{
		Keys.Add(Pair.Key);
	}
	for (const TPair<FUtf8String, FCbField>& Pair : NewDiagnostics)
	{
		Keys.Add(Pair.Key);
	}

	TStringBuilder<256> Text;
	for (const FUtf8String& Key : Keys)
	{
		Text << TEXT("'") << ExportRelPath << TEXT("':") << Key << TEXT(":Old Value\n");
		const FCbField* OldValue = OldDiagnostics.Find(Key);
		if (OldValue)
		{
			CompactBinaryToJson(*OldValue, Text);
			Text << TEXT("\n");
		}
		Text << TEXT("'") << ExportRelPath << TEXT("':") << Key << TEXT(":New Value\n");
		const FCbField* NewValue = NewDiagnostics.Find(Key);
		if (NewValue)
		{
			CompactBinaryToJson(*NewValue, Text);
			Text << TEXT("\n");
		}
	}
	return FString(Text);
}

void FExportDeterminismData::AddNewDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value)
{
	FCbField& StoredField = NewDiagnostics.FindOrAdd(FUtf8String(DiagnosticName));
	StoredField = Value;
	StoredField.MakeOwned();
	bSortDirty = true;
}

void FExportDeterminismData::AddOldDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value)
{
	FCbField& StoredField = OldDiagnostics.FindOrAdd(FUtf8String(DiagnosticName));
	StoredField = Value;
	StoredField.MakeOwned();
	bSortDirty = true;
}

void FExportDeterminismData::Sort()
{
	if (!bSortDirty)
	{
		return;
	}
	bSortDirty = false;
	OldDiagnostics.KeySort(FCompareDiagnosticName());
	NewDiagnostics.KeySort(FCompareDiagnosticName());
}

FPackageDeterminismData::FPackageDeterminismData(FDeterminismManager& InOwner)
	: Owner(&InOwner)
{
}

const ITargetPlatform* FPackageDeterminismData::GetTargetPlatform()
{
	return TargetPlatform;
}

const TSet<UObject*>& FPackageDeterminismData::GetModifiedExports()
{
	Sort();
	return ModifiedExports;
}

UObject* FPackageDeterminismData::GetPrimaryAsset()
{
	return PrimaryAsset;
}

const IDeterminismModifiedExportContext& FPackageDeterminismData::GetExportContext(UObject* Export)
{
	if (!Export || Export->GetPackage() != Owner->Package)
	{
		ensureMsgf(false, TEXT("GetExportContext called with object %s which is not in package %s."),
			Export ? *Export->GetPathName() : TEXT("<nullptr>"), *Owner->Package->GetName());
		Export = Owner->Package;
	}
	return FindOrAddExportData(Export);
}

FExportDeterminismData& FPackageDeterminismData::FindOrAddExportData(UObject* Object)
{
	check(Object);
	TUniquePtr<FExportDeterminismData>& Ptr = Exports.FindOrAdd(Object);
	if (!Ptr.IsValid())
	{
		Ptr.Reset(new FExportDeterminismData(*this, Object));
		bSortDirty = true;
	}
	return *Ptr;
}

bool FPackageDeterminismData::IsEmpty()
{
	return Exports.IsEmpty();
}

void FPackageDeterminismData::Sort()
{
	if (!bSortDirty)
	{
		return;
	}
	bSortDirty = false;
	UObject* Package = Owner->Package;

	Exports.KeySort([Package](UObject& A, UObject& B)
		{
			TStringBuilder<256> ARelPath;
			TStringBuilder<256> BRelPath;
			A.GetPathName(Package, ARelPath);
			B.GetPathName(Package, BRelPath);
			return ARelPath.ToView().Compare(BRelPath.ToView(), ESearchCase::IgnoreCase) < 0;
		});
	ModifiedExports.Reset();
	ModifiedExports.Reserve(Exports.Num());
	for (TPair<UObject*, TUniquePtr<FExportDeterminismData>>& Pair : Exports)
	{
		if (Pair.Value->bModified)
		{
			ModifiedExports.Add(Pair.Key);
		}
	}
}

} // namespace UE::Cook