// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookDeterminismHelper.h"
#include "Misc/Optional.h"
#include "Serialization/PackageWriter.h"
#include "UObject/NameTypes.h"

class UObject;
class FCbFieldView;
class FCbWriter;
namespace UE::Cook::Artifact { struct FAppendPackageMetaDataContext; }

namespace UE::Cook
{

class FDeterminismManager;
struct FExportDeterminismData;
struct FPackageDeterminismData;

/**
 * Helper for FDeterminismManager: the context object that is passed to IDeterminismHelper ConstructDiagnostics
 * functions.
 */
struct FDeterminismConstructDiagnosticsContext : public IDeterminismConstructDiagnosticsContext
{
public:
	FDeterminismConstructDiagnosticsContext(FExportDeterminismData& InExportData);

	virtual const ITargetPlatform* GetTargetPlatform() override;
	virtual void AddDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value) override;

	virtual bool FullDDCKeysRequested() const override;

public:
	FExportDeterminismData& ExportData;
};

/**
 * Helper for FDeterminismManager: the data about the a UObject export that is passed
 * to IDeterminismHelper OnPackageModified functions. Also serves as the container
 * for all diagnostic data gathered by the manager for an export.
 */
struct FExportDeterminismData : public IDeterminismModifiedExportContext
{
public:
	FExportDeterminismData(FPackageDeterminismData& InPackageData, UObject* InExport);

	virtual bool IsModified() override;
	virtual bool IsPrimaryAsset() override;
	virtual const ITargetPlatform* GetTargetPlatform() override;
	virtual const TMap<FUtf8String, FCbField>& GetOldDiagnostics() override;
	virtual const TMap<FUtf8String, FCbField>& GetNewDiagnostics() override;
	virtual IDeterminismModifiedPackageContext& GetPackageContext() override;
	virtual FString GetCompareText() override;

	virtual void AppendLog(FStringView LogText) override;
	virtual void AppendDiagnostics() override;

	void AddOldDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value);
	void AddNewDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value);
	void Sort();
public:
	TMap<FUtf8String, FCbField> OldDiagnostics;
	TMap<FUtf8String, FCbField> NewDiagnostics;
	FStringBuilderBase* Logger = nullptr;
	TArray<TRefCountPtr<IDeterminismHelper>> DeterminismHelpers;
	FPackageDeterminismData& PackageData;
	UObject* Export = nullptr;
	bool bPrimaryAsset = false;
	bool bModified = false;
	bool bSortDirty = true;
	bool bAppendedDiagnostics = false;
};

/**
 * Helper for FDeterminismManager: the data about the package that is passed
 * to IDeterminismHelper OnPackageModified functions. Also serves as the container
 * for all diagnostic data gathered by the manager for a package.
 */
struct FPackageDeterminismData : public IDeterminismModifiedPackageContext
{
public:
	FPackageDeterminismData(FDeterminismManager& InOwner);

	virtual const ITargetPlatform* GetTargetPlatform() override;
	virtual const TSet<UObject*>& GetModifiedExports() override;
	virtual UObject* GetPrimaryAsset() override;
	virtual const IDeterminismModifiedExportContext& GetExportContext(UObject* Export) override;

	bool IsEmpty();
	void Sort();
	FExportDeterminismData& FindOrAddExportData(UObject* Object);
public:
	TMap<UObject*, TUniquePtr<FExportDeterminismData>> Exports;
	TSet<UObject*> ModifiedExports;
	FDeterminismManager* Owner = nullptr; // non-null after constructor
	UObject* PrimaryAsset = nullptr;
	const ITargetPlatform* TargetPlatform = nullptr;
	bool bModified = false;
	bool bSortDirty = true;
};

/**
 * Manager that receives the diagnostics from IDeterminismHelpers declared in UObject PreSave
 * functions, and saves/loads them as attachments to the package in the oplog, and calls the
 * DeterminismHelpers callback functions when a package is found to be unexpectedly modified.
 */
class FDeterminismManager
{
public:
	FDeterminismManager();

	void BeginPackage(UPackage* InPackage, const ITargetPlatform* TargetPlatform,
		ICookedPackageWriter* InOplogProvider);
	void RegisterDeterminismHelper(UObject* SourceObject,
		const TRefCountPtr<IDeterminismHelper>& DeterminismHelper);
	void RecordPackageModified(UObject* PrimaryAsset);
	void RecordExportModified(const FString& ExportPathName);
	FString GetCurrentPackageDiagnosticsAsText();
	void AppendCommitAttachments(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context);
	void EndPackage();

private:
	void FetchOldDiagnostics();
	bool TrySave(FCbWriter& Writer);
	bool TryLoad(FCbFieldView Field);

	ICookedPackageWriter* OplogProvider = nullptr;
	UPackage* Package = nullptr;
	FPackageDeterminismData PackageData;
	TOptional<bool> OplogAvailable;

	friend UE::Cook::FDeterminismConstructDiagnosticsContext;
	friend UE::Cook::FExportDeterminismData;
	friend UE::Cook::FPackageDeterminismData;
};

} // namespace UE::Cook