// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookDiagnostics.h"

#include "Algo/Sort.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/RingBuffer.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookPackageArtifacts.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookRequestCluster.h"
#include "Cooker/CookTypes.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

class ITargetPlatform;

DEFINE_LOG_CATEGORY_STATIC(LogHiddenDependencies, Log, All);

namespace UE::Cook
{

struct FPackageReferenceAndPropertyChain
{
	FName PackageName;
	TArray<FProperty*, TInlineAllocator<1>> Properties;
};
/**
 * From a passed in export, collects referenced exports and imports, and stores the property chain
 * that references each import.
 */
class FWhyImportedCollector : public FArchiveUObject
{
public:
	explicit FWhyImportedCollector(UPackage* InRootPackage)
		: RootPackage(InRootPackage)
	{
		ArIsObjectReferenceCollector = true;
		SetIsSaving(true);
		SetIsPersistent(true);
	}

	void Reset()
	{
		Exports.Reset();
		Imports.Reset();
	}
	virtual FArchive& operator<<(UObject*& Obj) override
	{
		if (Obj)
		{
			UPackage* Package = Obj->GetPackage();
			if (Package == RootPackage)
			{
				Exports.Add(Obj);
			}
			else
			{
				Imports.Add(FPackageReferenceAndPropertyChain{ Package->GetFName(), ConvertSerializedPropertyChain() });
			}
		}
		return *this;
	}
	virtual FArchive& operator<<(FSoftObjectPath& Value) override
	{
		FName Package = Value.GetLongPackageFName();
		if (Package != RootPackage->GetFName() && !Package.IsNone())
		{
			Imports.Add(FPackageReferenceAndPropertyChain{ Package, ConvertSerializedPropertyChain() });
		}
		return *this;
	}

	TArray<FProperty*, TInlineAllocator<1>> ConvertSerializedPropertyChain()
	{
		TArray<FProperty*, TInlineAllocator<1>> Result;
		const FArchiveSerializedPropertyChain* Chain = GetSerializedPropertyChain();
		if (Chain && Chain->GetNumProperties())
		{
			Result.Reserve(Chain->GetNumProperties());
			for (int32 Index = 0; Index < Chain->GetNumProperties(); ++Index)
			{
				Result.Add(Chain->GetPropertyFromRoot(Index));
			}
		}
		return Result;
	}

public:
	UPackage* RootPackage;
	TArray<UObject*> Exports;
	TArray<FPackageReferenceAndPropertyChain> Imports;
};

void FDiagnostics::AnalyzeHiddenDependencies(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
	TMap<FPackageData*, EInstigator>* DiscoveredDependencies, TMap<FPackageData*, EInstigator>& SaveReferences,
	TConstArrayView<const ITargetPlatform*> ReachablePlatforms, bool bOnlyEditorOnlyDebug,
	bool bHiddenDependenciesDebug)
{
	TOptional<TArray<FName>> ExpectedDependencies;
	TOptional<TMap<FName, TMap<UObject*, TArray<FProperty*, TInlineAllocator<1>>>>> ExportsUsingPackageName;
	bool bHasResaved = false;
	FMultiPackageReaderResults Resave;
	UPackage* Package = PackageData.GetPackage();
	check(Package); // We are called from SavePackage so the PackageData still has a pointer to it

	// Unsolicited packages are added to the cook when not using OnlyEditorOnly.
	// When using OnlyEditorOnly, only packages referenced by the cooked SavePackage are added.
	// For the period where we are transitioning the engine to always run OnlyEditorOnly, there will be
	// some false negatives: OnlyEditorOnly will fail to add a package that is necessary.
	// This function investigates each unsolicited package that was not added and writes information about
	// why it was not added. Each of these needs to be investigated, and either fixed somehow so that it is
	// referenced from the cooked SavePackage, or marked as an expected correct difference using FCookLoadScope.
	if (!DiscoveredDependencies)
	{
		return;
	}
	for (const TPair<FPackageData*, EInstigator>& UnsolicitedPair : *DiscoveredDependencies)
	{
		const FPackageData* Unsolicited = UnsolicitedPair.Key;
		EInstigator Instigator = UnsolicitedPair.Value;
		if (!bHiddenDependenciesDebug && SaveReferences.Contains(Unsolicited))
		{
			// This package is included by OnlyEditorOnly as well. Remove it from our list to investigate.
			continue;
		}
		if (!bOnlyEditorOnlyDebug && Instigator != EInstigator::Unsolicited)
		{
			// Dependencies are only hidden if they're detected as Unsolicited; other instigator types
			// are reported only for comparing SkipOnlyEditorOnly to legacy WhatGetsCooked rules.
			continue;
		}
		FName UnsolicitedPackageName = Unsolicited->GetPackageName();

		// If the package is not cookable on any of the platforms we are cooking, then even though it was loaded
		// it will not be cooked by LegacyWhatShouldBeCooked. Remove it from our list to investigate.
		UPackage* UnsolicitedUPackage = FindPackage(nullptr, *WriteToString<256>(UnsolicitedPackageName));
		bool bSuppressedOnAllPlatforms = true;
		for (const ITargetPlatform* ReachablePlatform : ReachablePlatforms)
		{
			if (UnsolicitedUPackage && FCoreUObjectDelegates::ShouldCookPackageForPlatform.IsBound())
			{
				if (!FCoreUObjectDelegates::ShouldCookPackageForPlatform.Execute(UnsolicitedUPackage, ReachablePlatform))
				{
					continue;
				}
			}
			bool bCookable;
			bool bExplorable;
			ESuppressCookReason Reason;
			FRequestCluster::IsRequestCookable(ReachablePlatform, *Unsolicited, COTFS, Reason, bCookable, bExplorable);
			if (!bCookable)
			{
				continue;
			}
			bSuppressedOnAllPlatforms = false;
			break;
		}
		if (bSuppressedOnAllPlatforms)
		{
			continue;
		}

		if (PackageData.GetGenerationHelper())
		{
			// TODO: Collect SaveReferences for all generated packages and collect Unsolicited from all externalactor packages
			// and run the missing solicited test when the GeneratorPackage finishes saving all generated
			continue;
		}

		// If the referenced package was one of the declared dependencies of the source package, then it is not unsolicited.
		// We only need to log unsolicited dependencies. Declared dependencies are either added already, or they were editoronly
		// and were intentionally not added to the cook. They would still be cooked by LegacyWhatShouldBeCooked, and that is a difference,
		// but we will trust the system that declared them as editoronly and therefore not log them as needs investigation.
		if (!ExpectedDependencies)
		{
			ExpectedDependencies.Emplace();
			COTFS.AssetRegistry->GetDependencies(PackageData.GetPackageName(), *ExpectedDependencies,
				UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::NoRequirements);
		}
		if (ExpectedDependencies->Contains(Unsolicited->GetPackageName()))
		{
			continue;
		}

		// Maybe the unsolicited package was a used by a previous version the source package, but postload steps
		// removed it. Or maybe it was previously undeclared as a dependency but new code in savepackage declares it.
		// Try resaving the source package, and if the unsolicited package was an import but is no longer, or it is
		// now an editor-only import, then it would no longer be unsolicited at head version and it is correct that
		// OnlyEditorOnly removes it.
		FLinkerLoad* LinkerLoad = Package->GetLinker();
		FName ClassNameOfUPackage = UPackage::StaticClass()->GetFName();
		FName ClassPackageOfUPackage = UPackage::StaticClass()->GetOuter()->GetFName();
		bool bIsImportOfOriginalPackageLoad = false;
		if (LinkerLoad)
		{
			for (FObjectImport& Import : LinkerLoad->ImportMap)
			{
				if (Import.ClassName == ClassNameOfUPackage && Import.ClassPackage == ClassPackageOfUPackage &&
					Import.OuterIndex.IsNull() && Import.ObjectName == UnsolicitedPackageName)
				{
					bIsImportOfOriginalPackageLoad = true;
					break;
				}
			}
			for (FName SoftPackageReference : LinkerLoad->SoftPackageReferenceList)
			{
				if (SoftPackageReference == UnsolicitedPackageName)
				{
					bIsImportOfOriginalPackageLoad = true;
					break;
				}
			}
		}

		if (!bHasResaved)
		{
			bHasResaved = true;
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.SaveFlags = SAVE_NoError // Do not crash the SaveServer on an error
				| SAVE_BulkDataByReference	// EditorDomain saves reference bulkdata from the WorkspaceDomain rather than duplicating it
				| SAVE_Async				// SavePackage support for PackageWriter is only implemented with SAVE_Async
				;
			SaveArgs.bSlowTask = false;
			Resave = GetSaveExportsAndImports(Package, nullptr, SaveArgs);
		}

		if (Resave.Realms[0].bValid)
		{
			FPackageReaderResults& Realm = Resave.Realms[0];
			FSoftObjectPath PackagePath(FTopLevelAssetPath(UnsolicitedPackageName, NAME_None), FString());
			FSoftObjectPath PackageClassPath(UPackage::StaticClass());
			bool bIsImportOfResavedPackage = false;
			bool bIsEditorOnlyImportOfResavedPackage = false;
			for (TPair<FSoftObjectPath, FPackageReader::FObjectData>& Pair : Realm.Imports)
			{
				if (Pair.Value.ClassPath == PackageClassPath && Pair.Key == PackagePath)
				{
					bIsImportOfResavedPackage = true;
					bIsEditorOnlyImportOfResavedPackage = Pair.Value.bUsedInGame;
					break;
				}
			}
			for (TPair<FName, bool>& Pair : Realm.SoftPackageReferences)
			{
				if (Pair.Key == UnsolicitedPackageName)
				{
					bIsImportOfResavedPackage = true;
					bIsEditorOnlyImportOfResavedPackage = !Pair.Value;
					break;
				}
			}

			if (bIsImportOfOriginalPackageLoad && !bIsImportOfResavedPackage)
			{
				// An import that was removed by upgrade steps; ignore it
				continue;
			}

			if (bIsEditorOnlyImportOfResavedPackage)
			{
				// An import that the new save code identifies as editoronly; ignore it
				continue;
			}
		}

		// No more filtering, we're going to log this as an unexpected difference.
		// Try to find out how it was loaded so we can give information about what code needs to
		// be modified to support OnlyEditorOnly.

		if (bOnlyEditorOnlyDebug)
		{
			// Serialize all referenced exports in the source package and record all the packages each one
			// references. If any of those packages is the unsolicited package, print out the export and property
			// that refers to it.
			if (!ExportsUsingPackageName)
			{
				ExportsUsingPackageName.Emplace();
				TSet<UObject*> Exports;
				TRingBuffer<UObject*> ExportsQueue;
				ForEachObjectWithPackage(Package, [&Exports, &ExportsQueue](UObject* Object)
					{
						if (Object->HasAnyFlags(RF_Public))
						{
							bool bAlreadyExists;
							Exports.Add(Object, &bAlreadyExists);
							if (!bAlreadyExists)
							{
								ExportsQueue.Add(Object);
							}
						}
						return true;
					});
				FWhyImportedCollector Collector(Package);
				while (!ExportsQueue.IsEmpty())
				{
					UObject* Export = ExportsQueue.PopFrontValue();
					Export->Serialize(Collector);
					for (UObject* Dependency : Collector.Exports)
					{
						bool bAlreadyExists;
						Exports.Add(Dependency, &bAlreadyExists);
						if (!bAlreadyExists)
						{
							ExportsQueue.Add(Dependency);
						}
					}
					for (FPackageReferenceAndPropertyChain& ImportChain : Collector.Imports)
					{
						TArray<FProperty*, TInlineAllocator<1>>& ExistingChain = ExportsUsingPackageName->FindOrAdd(ImportChain.PackageName).FindOrAdd(Export);
						if (ExistingChain.IsEmpty())
						{
							ExistingChain = MoveTemp(ImportChain.Properties);
						}
					}
					Collector.Reset();
				}
			}
			TMap<UObject*, TArray<FProperty*, TInlineAllocator<1>>>* ExportsUsingUnsolicited = ExportsUsingPackageName->Find(UnsolicitedPackageName);
			TStringBuilder<256> WhyReferenced;

			int32 PackageNameLen = Package->GetName().Len();
			if (ExportsUsingUnsolicited)
			{
				int32 Count = 0;
				constexpr int32 MaxCount = 3;
				WhyReferenced << TEXT("\n\tReferencers:");
				for (TPair<UObject*, TArray<FProperty*, TInlineAllocator<1>>>&Pair : *ExportsUsingUnsolicited)
				{
					UObject* Export = Pair.Key;
					WhyReferenced << TEXT("\n\t\t");
					if (Count++ == MaxCount)
					{
						WhyReferenced << TEXT("...");
						break;
					}
					WhyReferenced << Export->GetClass()->GetPathName() << TEXT(" ");
					WhyReferenced << Export->GetPathName().RightChop(PackageNameLen + 1);
					WhyReferenced << TEXT("#");
					if (Pair.Value.IsEmpty())
					{
						WhyReferenced << TEXT("<UnknownProperty>");
					}
					else
					{
						int32 NumProperties = Pair.Value.Num();
						bool bAddedSeparator = false;
						for (int32 PropertyIndex = 0; PropertyIndex < NumProperties; ++PropertyIndex)
						{
							FProperty* Property = Pair.Value[PropertyIndex];
							if (PropertyIndex < NumProperties - 1 && Property->GetFName() == Pair.Value[PropertyIndex + 1]->GetFName() &&
								Property->IsA(FArrayProperty::StaticClass()))
							{
								// Omit adding a duplicate entry for inner property of an array which has the same name as its array property
								continue;
							}
							WhyReferenced << Property->GetName() << TEXT("#");
							bAddedSeparator = true;
						}
						if (bAddedSeparator)
						{
							WhyReferenced.RemoveSuffix(1);
						}
					}
				}
			}
			else
			{
				// If we didn't find any exports referring to the unsolicited package, then just print out the list of class types
				// in the package to show which code needs to be investigated.
				enum class EClassPriority
				{
					PrimaryUAsset,
					Asset,
					Public,
					Private,
					Count,
				};
				TMap<UClass*, EClassPriority> ObjectClasses;
				FName PackageLeafName = FName(*FPaths::GetBaseFilename(Package->GetName(), true /* bRemovePath */));
				ForEachObjectWithPackage(Package, [&ObjectClasses, PackageLeafName](UObject* Object)
					{
						UClass* Class = Object->GetClass();
						EClassPriority NewPriority = EClassPriority::Private;
						if (Object->IsAsset())
						{
							NewPriority = Object->GetFName() == PackageLeafName ? EClassPriority::PrimaryUAsset : EClassPriority::Asset;
						}
						else
						{
							NewPriority = Object->HasAnyFlags(RF_Public) ? EClassPriority::Public : EClassPriority::Private;
						}
						EClassPriority& OldPriority = ObjectClasses.FindOrAdd(Class, NewPriority);
						OldPriority = static_cast<EClassPriority>(FMath::Min(static_cast<int32>(OldPriority), static_cast<int32>(NewPriority)));
						return true;
					});
				WhyReferenced << TEXT("\n\tNo exports found referencing the dependency. Classes in the referencer package:");
				int32 ClassCount = 0;
				constexpr int32 MaxClassCount = 10;
				ObjectClasses.ValueSort([](EClassPriority A, EClassPriority B)
					{
						if (A != B)
						{
							return (int32)A < (int32)B;
						}
						return false;
					});
				for (TPair<UClass*, EClassPriority>& ClassPair : ObjectClasses)
				{
					if (ClassCount++ >= MaxClassCount)
					{
						break;
					}
					WhyReferenced << TEXT("\n\t\t") << ClassPair.Key->GetPathName();
				}
			}

			UE_LOGF(LogHiddenDependencies, Display, "Skipped adding %ls -> (%ls) %ls%ls",
				*WriteToString<256>(PackageData.GetPackageName()), LexToString(Instigator),
				*WriteToString<256>(Unsolicited->GetPackageName()), *WhyReferenced);
		}
		else
		{
			UE_LOGF(LogHiddenDependencies, Display, "HiddenDependency %ls -> (%ls) %ls",
				*WriteToString<256>(PackageData.GetPackageName()), LexToString(Instigator),
				*WriteToString<256>(Unsolicited->GetPackageName()));
		}
	}
}

const TCHAR* LexToString(UE::Cook::EIncrementallyModifiedReason Reason)
{
	static_assert(static_cast<int>(EIncrementallyModifiedReason::Count) == 7);
	switch (Reason)
	{
	case EIncrementallyModifiedReason::None: return TEXT("None");
	case EIncrementallyModifiedReason::NotPreviouslyCooked: return TEXT("NotPreviouslyCooked");
	case EIncrementallyModifiedReason::TargetDomainKey:  return TEXT("TargetDomainKey");
	case EIncrementallyModifiedReason::TransitiveBuildDependency:  return TEXT("TransitiveBuildDependency");
	case EIncrementallyModifiedReason::IncrementalCookDisabled:  return TEXT("IncrementalCookDisabled");
	case EIncrementallyModifiedReason::NoGenerator:  return TEXT("NoGenerator");
	case EIncrementallyModifiedReason::GeneratorOtherPlatform: return TEXT("GeneratorOtherPlatform");
	default: return TEXT("Invalid");
	}
}

FIncrementallyModifiedDiagnostics::FIncrementallyModifiedDiagnostics()
{
	const TCHAR* CommandLine = FCommandLine::Get();

	bool bCommandLineOptionSet = false;

	if (FParse::Param(CommandLine, TEXT("WriteCookIncrementallyModifiedDiagnosticsToLogs"))
		|| FParse::Param(CommandLine, TEXT("Cook.Diagnostic.Modified.FinalLog")))
	{
		bWriteFinalLog = true;
		bCommandLineOptionSet = true;
	}
	bCommandLineOptionSet |= FParse::Bool(CommandLine, TEXT("-Cook.Diagnostic.Modified.FinalLog="), bWriteFinalLog);
	if (FParse::Param(CommandLine, TEXT("Cook.Diagnostic.Modified.FinalFiles")))
	{
		bWriteFinalFiles = true;
		bCommandLineOptionSet = true;
	}
	bCommandLineOptionSet |= FParse::Bool(CommandLine, TEXT("-Cook.Diagnostic.Modified.FinalFiles="), bWriteFinalFiles);
	if (FParse::Param(CommandLine, TEXT("Cook.Diagnostic.Modified.Immediate")))
	{
		bWriteImmediateLog = true;
		bCommandLineOptionSet = true;
	}
	bCommandLineOptionSet |= FParse::Bool(CommandLine, TEXT("-Cook.Diagnostic.Modified.Immediate="), bWriteImmediateLog);

	if (!bCommandLineOptionSet)
	{
		bWriteFinalFiles = true;
	}
}

bool FIncrementallyModifiedDiagnostics::IsWriteFinalLog() const
{
	return bWriteFinalLog;
}

bool FIncrementallyModifiedDiagnostics::IsWriteFinalFiles() const
{
	return bWriteFinalFiles;
}

bool FIncrementallyModifiedDiagnostics::IsWriteImmediateLog() const
{
	return bWriteImmediateLog;
}

void FIncrementallyModifiedDiagnostics::AddPackage(FIncrementallyModifiedContext& Context)
{
	check(Context.PackageData);
	check(Context.Artifacts);

	FPackageData& PackageData = PackageDatas.Emplace_GetRef();
	PackageData.AssetClass = Context.AssetClass;
	PackageData.PackageName = Context.PackageData->GetPackageName();
	PackageData.bSelfModified = Context.Artifacts->GetIsPackageModified().Get(false /* DefaultValue */);
	PackageData.Reason = Context.Reason;

	switch (Context.Reason)
	{
	case EIncrementallyModifiedReason::TargetDomainKey:
	{
		TArray<FString>& Lines = PackageData.ReasonLines;
		TArray<FString> ModifiedDependencies;
		Context.Artifacts->TryGetModifiedDependencies(Context.TargetPlatform, Context.GenerationHelper,
			ModifiedDependencies);

		const int32 MaxDependencyCount = 3;
		for (int32 Index = 0; Index < ModifiedDependencies.Num(); ++Index)
		{
			if (Index == MaxDependencyCount)
			{
				Lines.Add(TEXT("..."));
				break;
			}
			Lines.Add(ModifiedDependencies[Index]);
		}
		if (Lines.Num() == 0)
		{
			Lines.Add(TEXT("<UnknownModifiedDependency>"));
		}
		break;
	}
	case EIncrementallyModifiedReason::TransitiveBuildDependency:
		PackageData.ReasonLines.Add(Context.TransitiveBuildDependencyName.ToString());
		break;
	default:
		break;
	}

	if (IsWriteImmediateLog())
	{
		if (PackageData.ReasonLines.Num() == 0)
		{
			UE_LOGFMT(LogCook, Display, "Modified: {Package}, {Reason}",
				PackageData.PackageName, LexToString(PackageData.Reason));
		}
		else if (PackageData.ReasonLines.Num() == 1)
		{
			UE_LOGFMT(LogCook, Display, "Modified: {Package}, {Reason}: {ReasonDetail}",
				PackageData.PackageName, LexToString(PackageData.Reason), *PackageData.ReasonLines[0]);
		}
		else
		{
			UE_LOGFMT(LogCook, Display, "Modified: {Package}, {Reason}:",
				PackageData.PackageName, LexToString(PackageData.Reason));
			for (const FString& ReasonLine : PackageData.ReasonLines)
			{
				UE_LOGFMT(LogCook, Display, "    {ReasonDetail}", ReasonLine);
			}
		}

		if (!IsWriteFinalFiles() && !IsWriteFinalLog())
		{
			PackageData.ReasonLines.Empty();
		}
	}
}

void FIncrementallyModifiedDiagnostics::WriteFinal(FStringView OutputDirectory) const
{
	if (IsWriteFinalLog())
	{
		WriteToLog();
	}
	if (IsWriteFinalFiles())
	{
		WriteToFiles(OutputDirectory);
	}
}

void FIncrementallyModifiedDiagnostics::WriteToFiles(FStringView OutputDirectory) const
{
	TMap<FTopLevelAssetPath, TArray<const FPackageData*>> PackagesPerClass;
	SortPackageData(PackagesPerClass);

	{
		TArray<FString> Lines;
		CreateModifiedCookedPackageContent(PackagesPerClass, Lines);

		FString FilePath = FPaths::Combine(OutputDirectory, TEXT("ModifiedCookedPackages.txt"));
		FFileHelper::SaveStringArrayToFile(Lines, *FilePath);
	}

	{
		TArray<FString> Lines;
		CreateModifiedEditorPackageContent(PackagesPerClass, Lines);

		FString FilePath = FPaths::Combine(OutputDirectory, TEXT("ModifiedEditorPackages.txt"));
		FFileHelper::SaveStringArrayToFile(Lines, *FilePath);
	}
}

void FIncrementallyModifiedDiagnostics::WriteToLog() const
{
	TMap<FTopLevelAssetPath, TArray<const FPackageData*>> PackagesPerClass;
	SortPackageData(PackagesPerClass);

	{
		TArray<FString> Lines;
		CreateModifiedCookedPackageContent(PackagesPerClass, Lines);

		UE_LOGF(LogCook, Display, "Start ModifiedCookedPackages.txt");
		for (const FString& Line : Lines)
		{
			UE_LOGF(LogCook, Display, "%ls", *Line);
		}
		UE_LOGF(LogCook, Display, "End ModifiedCookedPackages.txt");
	}

	{
		TArray<FString> Lines;
		CreateModifiedEditorPackageContent(PackagesPerClass, Lines);

		UE_LOGF(LogCook, Display, "Start ModifiedEditorPackages.txt");
		for (const FString& Line : Lines)
		{
			UE_LOGF(LogCook, Display, "%ls", *Line);
		}
		UE_LOGF(LogCook, Display, "End ModifiedEditorPackages.txt");
	}
}

void FIncrementallyModifiedDiagnostics::SortPackageData(TMap<FTopLevelAssetPath, TArray<const FPackageData*>>& OutPackagesPerClass) const
{
	for (const FPackageData& PackageData : PackageDatas)
	{
		OutPackagesPerClass.FindOrAdd(PackageData.AssetClass).Add(&PackageData);
	}

	OutPackagesPerClass.KeySort([](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B)
		{
			return A.Compare(B) < 0;
		});
	for (TPair<FTopLevelAssetPath, TArray<const FPackageData*>>& Pair : OutPackagesPerClass)
	{
		Algo::Sort(Pair.Value, [](const FPackageData* A, const FPackageData* B)
			{
				return A->PackageName.LexicalLess(B->PackageName);
			});
	}
}

void FIncrementallyModifiedDiagnostics::CreateModifiedCookedPackageContent(const TMap<FTopLevelAssetPath, TArray<const FPackageData*>>& PackagesPerClass, TArray<FString>& OutLines) const
{
	OutLines.Reserve(PackageDatas.Num() + PackagesPerClass.Num());
	const TCHAR* Indent = TEXT("    ");

	// Reserve leading line for the total
	OutLines.Emplace();
	int32 NumTotalModified = 0;

	for (const TPair<FTopLevelAssetPath, TArray<const FPackageData*>>& Pair : PackagesPerClass)
	{
		OutLines.Add(FString::Printf(TEXT("%s, %d"), *Pair.Key.ToString(), Pair.Value.Num()));
		NumTotalModified += Pair.Value.Num();
		for (const FPackageData* PackageData : Pair.Value)
		{
			if (PackageData->ReasonLines.Num() == 0)
			{
				OutLines.Add(FString::Printf(TEXT("%s%s, %s"), Indent, *PackageData->PackageName.ToString(),
					LexToString(PackageData->Reason)));
			}
			else if (PackageData->ReasonLines.Num() == 1)
			{
				OutLines.Add(FString::Printf(TEXT("%s%s, %s: %s"), Indent, *PackageData->PackageName.ToString(),
					LexToString(PackageData->Reason), *PackageData->ReasonLines[0]));
			}
			else
			{
				OutLines.Add(FString::Printf(TEXT("%s%s, %s:"), Indent, *PackageData->PackageName.ToString(),
					LexToString(PackageData->Reason)));
				for (const FString& ReasonLine : PackageData->ReasonLines)
				{
					OutLines.Add(FString::Printf(TEXT("%s%s%s"), Indent, Indent, *ReasonLine));
				}
			}
		}
	}

	OutLines[0] = FString::Printf(TEXT("Total, %d"), NumTotalModified);
}

void FIncrementallyModifiedDiagnostics::CreateModifiedEditorPackageContent(const TMap<FTopLevelAssetPath, TArray<const FPackageData*>>& PackagesPerClass, TArray<FString>& OutLines) const
{
	int32 NumTotalModified = 0;

	OutLines.Reserve(PackageDatas.Num() + PackagesPerClass.Num() + 1);
	const TCHAR* Indent = TEXT("    ");

	// Reserve leading line for the total
	OutLines.Emplace();

	for (const TPair<FTopLevelAssetPath, TArray<const FPackageData*>>& Pair : PackagesPerClass)
	{
		int NumClassModified = 0;
		int32 ClassModifiedIndex = -1;
		for (const FPackageData* PackageData : Pair.Value)
		{
			if (PackageData->bSelfModified)
			{
				if (ClassModifiedIndex == -1)
				{
					ClassModifiedIndex = OutLines.Num();
					OutLines.Emplace();
				}
				OutLines.Add(FString::Printf(TEXT("%s%s"), Indent, *PackageData->PackageName.ToString()));
				++NumClassModified;
			}
		}
		if (ClassModifiedIndex != -1)
		{
			OutLines[ClassModifiedIndex] = FString::Printf(TEXT("%s, %d"), *Pair.Key.ToString(), NumClassModified);
			NumTotalModified += NumClassModified;
		}
	}

	OutLines[0] = FString::Printf(TEXT("Total, %d"), NumTotalModified);
}

} // namespace UE::Cook