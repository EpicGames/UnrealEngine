// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "UObject/TopLevelAssetPath.h"

class ITargetPlatform;
class UCookOnTheFlyServer;
namespace UE::Cook { class FPackageArtifacts; }
namespace UE::Cook { enum class EInstigator : uint8; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FGenerationHelper; }

namespace UE::Cook
{

class FDiagnostics
{
public:
	/**
	 * Compares packages that would be added to the cook when using legacy WhatGetsCookedRules to the packages that
	 * are added to the cook using OnlyEditorOnly rules. Silently ignores expected differences, but logs a message
	 * with diagnostics for unexpected differences.
	 */
	static void AnalyzeHiddenDependencies(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
		TMap<FPackageData*, EInstigator>* DiscoveredDependencies, TMap<FPackageData*, EInstigator>& SaveReferences,
		TConstArrayView<const ITargetPlatform*> ReachablePlatforms, bool bOnlyEditorOnlyDebug,
		bool bHiddenDependenciesDebug);

};

/** Reasons that CookRequestCluster can find to mark a package as IncrementallyModified. */
enum class EIncrementallyModifiedReason : uint8
{
	None,
	NotPreviouslyCooked,
	TargetDomainKey,
	TransitiveBuildDependency,
	IncrementalCookDisabled,
	NoGenerator,
	GeneratorOtherPlatform,
	Count,
};
const TCHAR* LexToString(UE::Cook::EIncrementallyModifiedReason Reason);

/** Diagnostic data passed in from CookRequestCluster for why a package was found IncrementallyModified. */
struct FIncrementallyModifiedContext
{
	FPackageData* PackageData = nullptr;
	const ITargetPlatform* TargetPlatform = nullptr;
	FGenerationHelper* GenerationHelper = nullptr;
	const FPackageArtifacts* Artifacts = nullptr;
	FTopLevelAssetPath AssetClass;
	FName TransitiveBuildDependencyName = NAME_None;
	EIncrementallyModifiedReason Reason = EIncrementallyModifiedReason::None;
};

/**
 * Record diagnostics from CookRequestCluster for which packages were found modified and why, and
 * write them out to files in MetaData.
 */
class FIncrementallyModifiedDiagnostics
{
public:
	FIncrementallyModifiedDiagnostics();

	void AddPackage(FIncrementallyModifiedContext& Context);
	void WriteFinal(FStringView OutputDirectory) const;
	bool IsWriteFinalLog() const;
	bool IsWriteFinalFiles() const;
	bool IsWriteImmediateLog() const;

private:
	/** The saved diagnostic data about each reported modified Package. */
	struct FPackageData
	{
		FName PackageName;
		FTopLevelAssetPath AssetClass;
		TArray<FString> ReasonLines;
		bool bSelfModified = false;
		EIncrementallyModifiedReason Reason = EIncrementallyModifiedReason::None;
	};

	void WriteToFiles(FStringView OutputDirectory) const;
	void WriteToLog() const;
	void SortPackageData(TMap<FTopLevelAssetPath, TArray<const FPackageData*>>& OutPackagesPerClass) const;
	void CreateModifiedCookedPackageContent(const TMap<FTopLevelAssetPath,
		TArray<const FPackageData*>>& PackagesPerClass, TArray<FString>& OutLines) const;
	void CreateModifiedEditorPackageContent(const TMap<FTopLevelAssetPath,
		TArray<const FPackageData*>>& PackagesPerClass, TArray<FString>& OutLines) const;

	TArray<FPackageData> PackageDatas;
	bool bWriteFinalFiles = false;
	bool bWriteFinalLog = false;
	bool bWriteImmediateLog = false;
};

} // namespace UE::Cook