// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookDirector.h"

#include "Async/Fundamental/Scheduler.h"
#include "CompactBinaryTCP.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookWorkerServer.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericPlatformOutputDevices.h"
#include "LoadBalanceCookBurden.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"
#include "HAL/RunnableThread.h"
#include "Math/NumericLimits.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PathViews.h"
#include "PackageTracker.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ShaderCompiler.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "String/ParseTokens.h"
#include "UnrealEdMisc.h"

extern CORE_API int32 GNumForegroundWorkers; // TaskGraph.cpp

LLM_DEFINE_TAG(Cooker_MPCook);

namespace UE::Cook
{

constexpr int32 RetractionMinimumNumAssignments = 100;

/** Profile data for each CookWorker that needs to be collected on the Director. */
struct FCookWorkerProfileData
{
	float IdleTimeSeconds = 0.f;
	bool bIsIdle = true;
	void UpdateIdle(bool bInIsIdle, float DeltaTime)
	{
		if (bInIsIdle)
		{
			if (bIsIdle)
			{
				IdleTimeSeconds += DeltaTime;
			}
		}
		bIsIdle = bInIsIdle;
	}
};

/**
 * A class that has an instance active while we need to handle retraction of assigned results from a CookWorker.
 * Keeps track of the expected message coming back from the remote worker, prevents repeatedly sending messages, gives
 * a warning if the remote worker does not respond.
 */
class FCookDirector::FRetractionHandler
{
public:
	FRetractionHandler(FCookDirector& InDirector);
	/** Initialize to search idle and busy workers to send a RetractionRequestMessage. */
	void Initialize();
	/** Initialize to handle an unexpected RetractionResultsMessage. */
	void InitializeForResultsMessage(const FWorkerId& FromWorker);
	void TickFromSchedulerThread(bool bAllWorkersConnected, bool bAnyIdle, int32 BusiestNumAssignments);
	/** Hook called by the director when a retraction message comes in. */
	void HandleRetractionMessage(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
		FRetractionResultsMessage&& Message);

private:
	enum class ERetractionState : uint8
	{
		Idle,
		WantToRetract,
		WaitingForResponse,
		Count,
	};
	enum class ERetractionResult : uint8
	{
		NoneAvailable,
		Retracted
	};
private:
	/** Try to select a worker for retraction */
	ERetractionState TickWantToRetract(bool& bOutAnyIdle, int32& OutBusiestNumAssignments);
	/** Tick the asynchronous wait for the message to come in, and synchronously handle it when it does. */
	ERetractionState TickWaitingForResponse();

	/**
	 * Pick workers to give the retracted packages to, and assign those packages to the worker
	 * in the local and remote state.
	 */
	ERetractionResult ReassignPackages(const FWorkerId& WorkerId, TConstArrayView<FPackageData*> Packages);
	/** Pick workers to give the retracted packages to. */
	TArray<FWorkerId> CalculateWorkersToSplitOver(int32 NumPackages, const FWorkerId& FromWorker,
		TConstArrayView<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers);

	void SetRetractionState(ERetractionState NewState, bool& bOutHadStateChange);
	bool IsAvailableForRetraction(const FWorkerId& WorkerId);

private:
	FCookDirector& Director;
	FWorkerId ExpectedWorker;
	TMap<FWorkerId, TArray<FName>> PackagesToRetract;
	TMap<FWorkerId, int32> WorkersUnavailableForRetract;
	FWorkerId WorkerWithResults;
	double MessageSentTimeSeconds = 0.;
	double LastWarnTimeSeconds = 0.;
	ERetractionState RetractionState = ERetractionState::Idle;
};

FCookDirector::FCookDirector(UCookOnTheFlyServer& InCOTFS, int32 CookProcessCount,
	bool bInCookProcessCountSetByCommandLine)
	: RunnableShunt(*this) 
	, COTFS(InCOTFS)
	, bCookProcessCountSetByCommandLine(bInCookProcessCountSetByCommandLine)
{
	check(CookProcessCount > 1);
	WorkersStalledStartTimeSeconds = MAX_flt;
	WorkersStalledWarnTimeSeconds = MAX_flt;
	ShutdownEvent->Reset();
	LocalWorkerProfileData = MakeUnique<FCookWorkerProfileData>();
	RetractionHandler = MakeUnique<FRetractionHandler>(*this);

	bool bConfigValid;
	ParseConfig(CookProcessCount, bConfigValid);
	if (!bConfigValid)
	{
		UE_LOGF(LogCook, Error,
			"CookDirector initialization failure: config settings are invalid for multiprocess. CookMultiprocess is disabled and the cooker is running as a single process.");
		bMultiprocessAvailable = false;
		return;
	}
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (!SocketSubsystem)
	{
		UE_LOGF(LogCook, Error,
			"CookDirector initialization failure: platform does not support network sockets. CookMultiprocess is disabled and the cooker is running as a single process.");
		bMultiprocessAvailable = false;
		return;
	}
	bMultiprocessAvailable = true;

	Register(new FLogMessagesMessageHandler(*COTFS.LogHandler));
	Register(new TMPCollectorServerMessageCallback<FRetractionResultsMessage>([this]
	(FMPCollectorServerMessageContext& Context, bool bReadSuccessful, FRetractionResultsMessage&& Message)
		{
			// Called from inside CommunicationLock
			RetractionHandler->HandleRetractionMessage(Context, bReadSuccessful, MoveTemp(Message));
		}));
	Register(new TMPCollectorServerMessageCallback<FHeartbeatMessage>([this]
	(FMPCollectorServerMessageContext& Context, bool bReadSuccessful, FHeartbeatMessage&& Message)
		{
			HandleHeartbeatMessage(Context, bReadSuccessful, MoveTemp(Message));
		}));
	Register(new FAssetRegistryMPCollector(COTFS));
	Register(new FPackageWriterMPCollector(COTFS));

	LastTickTimeSeconds = FPlatformTime::Seconds();

#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.AddRaw(this, &FCookDirector::LogCookStats);
#endif
}

bool FCookDirector::IsMultiprocessAvailable() const
{
	return bMultiprocessAvailable;
}

void FCookDirector::ParseConfig(int32 CookProcessCount, bool& bOutValid)
{
	bOutValid = true;
	const TCHAR* CommandLine = FCommandLine::Get();
	FString Text;

	// CookWorkerCount
	RequestedCookWorkerCount = CookProcessCount - 1;
	check(RequestedCookWorkerCount > 0);

	// CookDirectorListenPort
	WorkerConnectPort = Sockets::COOKDIRECTOR_DEFAULT_REQUEST_CONNECTION_PORT;
	FParse::Value(CommandLine, TEXT("-CookDirectorListenPort="), WorkerConnectPort);

	// ShowCookWorker
	if (!FParse::Value(CommandLine, TEXT("-ShowCookWorker="), Text))
	{
		if (FParse::Param(CommandLine, TEXT("ShowCookWorker")))
		{
			Text = TEXT("SeparateWindows");
		}
	}
	if (Text == TEXT("CombinedLogs")) { ShowWorkerOption = EShowWorker::CombinedLogs; }
	else if (Text == TEXT("SeparateLogs")) { ShowWorkerOption = EShowWorker::SeparateLogs; }
	else if (Text == TEXT("SeparateWindows")) { ShowWorkerOption = EShowWorker::SeparateWindows; }
	else
	{
		if (!Text.IsEmpty())
		{
			UE_LOGF(LogCook, Warning, "Invalid selection \"%ls\" for -ShowCookWorker.", *Text);
		}
		ShowWorkerOption = EShowWorker::CombinedLogs;
	}

	// LoadBalanceAlgorithm
	LoadBalanceAlgorithm = ELoadBalanceAlgorithm::CookBurden;
	if (FParse::Value(CommandLine, TEXT("-CookLoadBalance="), Text))
	{
		if (Text == TEXT("Striped")) { LoadBalanceAlgorithm = ELoadBalanceAlgorithm::Striped; }
		else if (Text == TEXT("CookBurden")) { LoadBalanceAlgorithm = ELoadBalanceAlgorithm::CookBurden; }
		else
		{
			UE_LOGF(LogCook, Warning, "Invalid selection \"%ls\" for -CookLoadBalance.", *Text);
		}
	}

	bAllowLocalCooks = !FParse::Param(CommandLine, TEXT("CookForceRemote"));

	
	int32 MultiprocessId = UE::GetMultiprocessId();
	if (MultiprocessId != 0)
	{
		bOutValid = false;
		UE_LOGF(LogCook, Error,
			"CookMultiprocess is incompatible with -MultiprocessId on the CookDirector's commandline. The CookDirector needs to be able to specify all MultiprocessIds.");
	}
}

FCookDirector::~FCookDirector()
{
	StopCommunicationThread();
#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.RemoveAll(this);
#endif

	TSet<FPackageData*> AbortedAssignments;
	for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
	{
		Pair.Value->AbortWorker(AbortedAssignments, ECookDirectorThread::SchedulerThread, HeartbeatNumber);
	}
	for (FPackageData* PackageData : AbortedAssignments)
	{
		// Packages that were assigned to workers should be in an AssignedToWorker state and
		// therefore should be InProgress.
		check(PackageData->IsInProgress());
		PackageData->SetWorkerAssignment(FWorkerId::Invalid(), ESendFlags::QueueNone);
		EPackageState NewState = PackageData->IsInStateProperty(EPackageStateProperty::Saving)
			? EPackageState::SaveActive
			: EPackageState::Request;
		PackageData->SendToState(NewState, ESendFlags::QueueAddAndRemove, EStateChangeReason::CookerShutdown);
	}
	RemoteWorkers.Empty();
	RemoteWorkerProfileDatas.Empty();
	PendingConnections.Empty();
	Sockets::CloseSocket(WorkerConnectSocket);
}

void FCookDirector::LaunchCommunicationThread()
{
	if (!CommunicationThread && FPlatformProcess::SupportsMultithreading())
	{
		CommunicationThread = FRunnableThread::Create(&RunnableShunt, TEXT("FCookDirector"), 0, TPri_Normal);
	}
}

void FCookDirector::StopCommunicationThread()
{
	ShutdownEvent->Trigger();
	if (CommunicationThread)
	{
		CommunicationThread->WaitForCompletion();
		delete CommunicationThread;
		CommunicationThread = nullptr;
	}
	ShutdownEvent->Reset();
}

uint32 FCookDirector::RunCommunicationThread()
{
	constexpr float TickPeriod = 1.f;
	constexpr float MinSleepTime = 0.001f;
	for (;;)
	{
		double StartTime = FPlatformTime::Seconds();
		TickCommunication(ECookDirectorThread::CommunicateThread);

		double CurrentTime = FPlatformTime::Seconds();
		float RemainingDuration = StartTime + TickPeriod - CurrentTime;
		if (RemainingDuration > .001f)
		{
			uint32 WaitTimeMilliseconds = static_cast<uint32>(RemainingDuration * 1000);
			if (ShutdownEvent->Wait(WaitTimeMilliseconds))
			{
				break;
			}
		}
	}
	return 0;
}

uint32 FCookDirector::FRunnableShunt::Run()
{
	return Director.RunCommunicationThread();
}

void FCookDirector::FRunnableShunt::Stop()
{
	Director.ShutdownEvent->Trigger();
}

void FCookDirector::StartCook(const FBeginCookContext& InBeginContext)
{
	BeginCookContext.Set(InBeginContext);

	// We launch the CookWorkers during StartCook, so that their startup time overlaps with
	// work the Director has to do during the first tick, for PumpRequests and AssignRequests.
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	InitializeWorkers();
}

void FCookDirector::AssignRequests(TArrayView<FPackageData*> Requests, TArray<FWorkerId>& OutAssignments,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph)
{
	TArray<FWorkerId> WorkerIds;
	TArray<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers;
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		InitializeWorkers();
	}
	LocalRemoteWorkers = CopyRemoteWorkers();;

	WorkerIds.Reserve(LocalRemoteWorkers.Num() + 1);
	if (bAllowLocalCooks)
	{
		WorkerIds.Add(FWorkerId::Local());
	}
	for (const TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		WorkerIds.Add(RemoteWorker->GetWorkerId());
	}

	AssignRequests(MoveTemp(WorkerIds), LocalRemoteWorkers, Requests, OutAssignments, MoveTemp(RequestGraph),
		true /* bInitialAssignment */);

	// Check for a race condition with the communication thread; if a Server was aborted after we CopyRemoteWorkers
	// above but before we assigned packages to it, we need to abort those assigments now. 
	TSet<FPackageData*> PackagesToReassignSet;
	for (TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		if (RemoteWorker->IsShuttingDown())
		{
			RemoteWorker->AbortAllAssignments(PackagesToReassignSet, ECookDirectorThread::SchedulerThread, HeartbeatNumber);
		}
	}
	if (!PackagesToReassignSet.IsEmpty())
	{
		// Defer the reassign because our caller has not yet put the packages into the assigned state.
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		DeferredPackagesToReassign.Append(PackagesToReassignSet.Array());
	}
}

TMap<FPackageData*, FAssignPackageExtraData> FCookDirector::GetAssignPackageExtraDatas(
	TConstArrayView<FPackageData*> Requests) const
{
	TConstArrayView<const ITargetPlatform*> SessionPlatforms = COTFS.PlatformManager->GetSessionPlatforms();
	FMPCollectorServerTickPackageContext Context;
	Context.Platforms = SessionPlatforms;

	TMap<FPackageData*, FAssignPackageExtraData> Results;
	for (FPackageData* Request : Requests)
	{
		FAssignPackageExtraData* ExtraData = nullptr;

		auto GetOrAllocateExtraData = [&Results, &ExtraData](FPackageData* InRequest)
		{
			if (!ExtraData)
			{
				ExtraData = &Results.FindOrAdd(InRequest);
			}
			return ExtraData;
		};

		TRefCountPtr<FGenerationHelper> GenerationHelper = Request->GetGenerationHelper();
		if (GenerationHelper)
		{
			ExtraData = GetOrAllocateExtraData(Request);
			ExtraData->bSkipSaveExistingGenerator = GenerationHelper->IsSkipSaveExistingGenerator();
			ExtraData->bQueuedGeneratedPackagesFencePassed = GenerationHelper->IsQueuedGeneratedPackagesFencePassed();
			ExtraData->GeneratorPerPlatformPreviousGeneratedPackages.Reserve(SessionPlatforms.Num());
			for (const ITargetPlatform* TargetPlatform : SessionPlatforms)
			{
				TMap<FName, FAssetPackageData>& PreviousGeneratedPackages
					= ExtraData->GeneratorPerPlatformPreviousGeneratedPackages.FindOrAdd(TargetPlatform);
				PreviousGeneratedPackages = GenerationHelper->GetPreviousGeneratedPackages(TargetPlatform);
			}
		}

		Context.PackageName = Request->GetPackageName();
		for (const TPair<FGuid, TRefCountPtr<IMPCollector>>& CollectorPair : Collectors)
		{
			IMPCollector* Collector = CollectorPair.Value.GetReference();
			Collector->ServerTickPackage(Context);
			if (!Context.Messages.IsEmpty())
			{
				ExtraData = GetOrAllocateExtraData(Request);
				FGuid MessageType = Collector->GetMessageType();
				for (FCbObject& Object : Context.Messages)
				{
					ExtraData->PerPackageCollectorMessages.Add({ MessageType, MoveTemp(Object) });
				}
				Context.Messages.Reset();
			}
		}
	}
	return Results;
}

TArray<FPackageData*> FCookDirector::GetInfoPackagesForRequests(TConstArrayView<FPackageData*> Requests) const
{
	TSet<FPackageData*> InfoPackages;
	for (FPackageData* Request : Requests)
	{
		if (!Request->GetParentGenerator().IsNone())
		{
			FPackageData* Generator = COTFS.PackageDatas->FindPackageDataByPackageName(Request->GetParentGenerator());
			if (Generator)
			{
				InfoPackages.Add(Generator);
			}
		}
	}
	return InfoPackages.Array();
}

void FCookDirector::AssignRequests(TArray<FWorkerId>&& InWorkers,
	TArray<TRefCountPtr<FCookWorkerServer>>& InRemoteWorkers,
	TArrayView<FPackageData*> Requests, TArray<FWorkerId>& OutAssignments,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, bool bInitialAssignment)
{
	check(InWorkers.Num() > 0);
	if (InWorkers.Num() <= 1)
	{
		FWorkerId WorkerId = InWorkers[0];
		OutAssignments.SetNum(Requests.Num());
		TArray<UE::Cook::FPackageData*> RemovedRequests;
		for (int32 RequestIndex = 0; RequestIndex < Requests.Num(); ++RequestIndex)
		{
			FPackageData* Request = Requests[RequestIndex];
			FWorkerId& Assignment = OutAssignments[RequestIndex];
			FWorkerId WorkerIdConstraint = Request->GetWorkerAssignmentConstraint();
			if (WorkerIdConstraint.IsValid() && WorkerIdConstraint != WorkerId)
			{
				UE_LOGF(LogCook, Warning,
					"Package %ls can only be cooked by a now-disconnected CookWorker. The package can not be cooked.",
					*Request->GetPackageName().ToString());
				Assignment = FWorkerId::Invalid();
				RemovedRequests.Add(Request);
			}
			else
			{
				Assignment = WorkerId;
			}
		}
		if (!WorkerId.IsLocal())
		{
			TRefCountPtr<FCookWorkerServer>* RemoteWorker = InRemoteWorkers.FindByPredicate(
				[&WorkerId](const TRefCountPtr<FCookWorkerServer>& X) { return X->GetWorkerId() == WorkerId; });
			check(RemoteWorker);
			TArrayView<FPackageData*> RequestsToSend = Requests;
			TArray<FPackageData*> RequestBuffer;
			if (!RemovedRequests.IsEmpty())
			{
				TSet<FPackageData*> RequestSet;
				for (FPackageData* Request : Requests)
				{
					RequestSet.Add(Request);
				}
				for (FPackageData* Remove : RemovedRequests)
				{
					RequestSet.Remove(Remove);
				}
				RequestBuffer = RequestSet.Array();
				RequestsToSend = RequestBuffer;
			}
			(*RemoteWorker)->AppendAssignments(RequestsToSend, GetAssignPackageExtraDatas(RequestsToSend),
				GetInfoPackagesForRequests(RequestsToSend), ECookDirectorThread::SchedulerThread);
			bHasSentActions = true;
		}
		return;
	}

	InWorkers.Sort();

	// Call the LoadBalancing algorithm to split the requests among the LocalWorker and RemoteWorkers
	LoadBalance(InWorkers, Requests, MoveTemp(RequestGraph), OutAssignments);

	int32 MaxRemoteIndex = InWorkers.Last().IsLocal() ? -1 : InWorkers.Last().GetRemoteIndex();

	// Split the output array of WorkerId assignments into a batch for each of the RemoteWorkers 
	TArray<TArray<FPackageData*>> RemoteBatches; // Indexed by WorkerId.GetRemoteIndex()
	TArray<bool> RemoteIndexIsValid; // Indexed by WorkerId.GetRemoteIndex()
	RemoteBatches.SetNum(MaxRemoteIndex+1);
	RemoteIndexIsValid.Init(false, MaxRemoteIndex+1);
	for (FWorkerId WorkerId : InWorkers)
	{
		if (!WorkerId.IsLocal())
		{
			RemoteIndexIsValid[WorkerId.GetRemoteIndex()] = true;
		}
	}

	// Debug function for generated packages. We want to be able to force their assignment to all,some,none
	// on the same worker that saved their generator.
	int32 MPCookGeneratorNextWorker = -1;
	auto GetNextWorkerForGeneratedPackage = [&MPCookGeneratorNextWorker, &InWorkers](FWorkerId WorkerNotToUse)
		{
			MPCookGeneratorNextWorker = (MPCookGeneratorNextWorker + 1) % InWorkers.Num();
			if (InWorkers[MPCookGeneratorNextWorker] == WorkerNotToUse)
			{
				MPCookGeneratorNextWorker = (MPCookGeneratorNextWorker + 1) % InWorkers.Num();
			}
			return InWorkers[MPCookGeneratorNextWorker];
		};
	for (int32 RequestIndex = 0; RequestIndex < Requests.Num(); ++RequestIndex)
	{
		FWorkerId& WorkerId = OutAssignments[RequestIndex];
		// Override the loadbalancer's assignment if the Package has a WorkerAssignmentConstraint
		// This allows us to guarantee that generated packages will be cooked on the worker that cooked
		// their generator package
		FPackageData* RequestPackage = Requests[RequestIndex];
		FWorkerId WorkerIdConstraint = RequestPackage->GetWorkerAssignmentConstraint();
		if (WorkerIdConstraint.IsValid())
		{
			WorkerId = WorkerIdConstraint;
		}
		// Override the loadbalancer's assignment to force it local if the Package is blocking urgency
		else if (RequestPackage->GetUrgency() == EUrgency::Blocking)
		{
			WorkerId = FWorkerId::Local();
		}
		else if (RequestPackage->IsGenerated() && bInitialAssignment)
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper = RequestPackage->GetParentGenerationHelper();
			if (GenerationHelper)
			{
				FWorkerId GeneratorWorker = GenerationHelper->GetWorkerIdThatSavedGenerator();
				if (GeneratorWorker.IsValid())
				{
					switch (COTFS.MPCookGeneratorSplit)
					{
					default: // fallthrough
					case EMPCookGeneratorSplit::AnyWorker:
						// If we're allowed to put them on any worker, put them on the GeneratorWorker anyway
						// for the initial assignment since it already has the GenerationHelper created
						WorkerId = GeneratorWorker;
						break;
					case EMPCookGeneratorSplit::AllOnSameWorker:
						WorkerId = GeneratorWorker;
						break;
					case EMPCookGeneratorSplit::SomeOnSameWorker:
						if ((GenerationHelper->GetMPCookNextAssignmentIndex()++) % 2 == 0)
						{
							WorkerId = GeneratorWorker;
						}
						else
						{
							WorkerId = GetNextWorkerForGeneratedPackage(GeneratorWorker);
						}
						break;
					case EMPCookGeneratorSplit::NoneOnSameWorker:
						WorkerId = GetNextWorkerForGeneratedPackage(GeneratorWorker);
						break;
					}
				}
			}
		}

		if (!WorkerId.IsLocal())
		{
			uint8 RemoteIndex = WorkerId.GetRemoteIndex();
			if (RemoteIndex >= RemoteBatches.Num() || !RemoteIndexIsValid[RemoteIndex])
			{
				UE_LOGF(LogCook, Error,
					"Package %ls can only be cooked by a now-disconnected CookWorker. The package can not be cooked.",
					*Requests[RequestIndex]->GetPackageName().ToString());
				WorkerId = FWorkerId::Invalid();
				continue;
			}
			TArray<FPackageData*>& RemoteBatch = RemoteBatches[RemoteIndex];
			if (RemoteBatch.Num() == 0)
			{
				RemoteBatch.Reserve(2 * Requests.Num() / (InWorkers.Num()));
			}
			RemoteBatch.Add(Requests[RequestIndex]);
		}
	}

	// Assign each batch to the FCookWorkerServer in RemoteWorkers;
	// the CookWorkerServer's tick will handle sending the message to the remote process
	for (FWorkerId WorkerId : InWorkers)
	{
		if (!WorkerId.IsLocal())
		{
			TRefCountPtr<FCookWorkerServer>* RemoteWorker = InRemoteWorkers.FindByPredicate(
				[&WorkerId](const TRefCountPtr<FCookWorkerServer>& X) { return X->GetWorkerId() == WorkerId; });
			check(RemoteWorker);
			TArray<FPackageData*>& RemoteBatch = RemoteBatches[WorkerId.GetRemoteIndex()];
			(*RemoteWorker)->AppendAssignments(RemoteBatch,
				GetAssignPackageExtraDatas(RemoteBatch), GetInfoPackagesForRequests(RemoteBatch), ECookDirectorThread::SchedulerThread);
			bHasSentActions = true;
		}
	}

	bIsFirstAssignment = false;
}

TArray<TRefCountPtr<FCookWorkerServer>> FCookDirector::CopyRemoteWorkers() const
{
	TArray<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers;
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	LocalRemoteWorkers.Reset(RemoteWorkers.Num());
	for (const TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
	{
		LocalRemoteWorkers.Add(Pair.Value);
	}
	return LocalRemoteWorkers;
}

void FCookDirector::RemoveFromWorker(FPackageData& PackageData)
{
	FWorkerId WorkerId = PackageData.GetWorkerAssignment();
	if (!WorkerId.IsRemote())
	{
		return;
	}

	TRefCountPtr<FCookWorkerServer> OwningWorker;
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		const TRefCountPtr<FCookWorkerServer>* RemoteWorkerPtr = FindRemoteWorkerInLock(WorkerId);
		if (!RemoteWorkerPtr)
		{
			return;
		}
		OwningWorker = *RemoteWorkerPtr;
	}

	OwningWorker->AbortAssignment(PackageData, ECookDirectorThread::SchedulerThread, HeartbeatNumber);
	bForceNextHeartbeat = true;
}

void FCookDirector::BroadcastMessage(const IMPCollectorMessage& Message, ECookBroadcastTiming Timing)
{
	BroadcastMessage(MarshalToCompactBinaryTCP(Message), Timing);
}

void FCookDirector::BroadcastMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message, ECookBroadcastTiming Timing)
{
	if (bReceivingMessages)
	{
		QueuedBroadcasts.Emplace(MoveTemp(Message), Timing);
		return;
	}

	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		InitializeWorkers();
	}
	TArray<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers = CopyRemoteWorkers();
	for (TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		UE::CompactBinaryTCP::FMarshalledMessage CopyMessage(Message);
		switch (Timing)
		{
		case ECookBroadcastTiming::Immediate:
		{
			if (RemoteWorker->IsConnected())
			{
				RemoteWorker->SendMessage(MoveTemp(CopyMessage), ECookDirectorThread::SchedulerThread);
			}
			else
			{
				RemoteWorker->AppendMessage(MoveTemp(CopyMessage), ECookDirectorThread::SchedulerThread);
			}
			break;
		}
		case ECookBroadcastTiming::AfterAssignPackages:
		{
			RemoteWorker->AppendMessage(MoveTemp(CopyMessage), ECookDirectorThread::SchedulerThread);
			break;
		}
		default:
			checkNoEntry();
			break;
		}
		bHasSentActions = true;
	}
}

int32 FCookDirector::InsertBroadcastFence()
{
	int32 LocalHeartbeatNumber = HeartbeatNumber;

	// We need to Tick, and have the Tick function send and update the heartbeat number, so that any preceding
	// messages get sent before the heartbeat.
	bForceNextHeartbeat = true;
	TickFromSchedulerThread();

	return LocalHeartbeatNumber;
}

bool FCookDirector::IsBroadcastFencePassed(int32 Fence, TArray<FWorkerId>* OutPendingWorkers)
{
	bool bHasPendingWorker = false;
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
	{
		FCookWorkerServer* RemoteWorker = Pair.Value.GetReference();
		if (RemoteWorker->GetLastReceivedHeartbeatNumber() < Fence)
		{
			bHasPendingWorker = true;
			if (OutPendingWorkers)
			{
				OutPendingWorkers->Add(RemoteWorker->GetWorkerId());
			}
		}
	}
	return !bHasPendingWorker;
}

void FCookDirector::TickFromSchedulerThread()
{
	double CurrentTime = FPlatformTime::Seconds();
	if (!CommunicationThread)
	{
		TickCommunication(ECookDirectorThread::SchedulerThread);
	}

	int32 BusiestNumAssignments = 0;
	bool bLocalWorkerIdle = true;
	bool bAnyIdle = false;
	float DeltaTime = static_cast<float>(CurrentTime - LastTickTimeSeconds);
	LastTickTimeSeconds = CurrentTime;
	if (bAllowLocalCooks)
	{
		BusiestNumAssignments = COTFS.NumMultiprocessLocalWorkerAssignments();
		bLocalWorkerIdle = BusiestNumAssignments == 0;
		bAnyIdle = bLocalWorkerIdle;
		LocalWorkerProfileData->UpdateIdle(bLocalWorkerIdle, DeltaTime);
	}
	
	bool bSendHeartbeat;
	int32 LocalHeartbeatNumber;
	TickHeartbeat(bForceNextHeartbeat, CurrentTime, bSendHeartbeat, LocalHeartbeatNumber);
	bForceNextHeartbeat = false;

	TArray<TRefCountPtr<FCookWorkerServer>, TInlineAllocator<16>> WorkersWithMessage;
	bool bAllWorkersConnected = false;
	if (bWorkersInitialized)
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		bAllWorkersConnected = true;
		for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
		{
			FCookWorkerServer* RemoteWorker = Pair.Value.GetReference();
			FCookWorkerProfileData& ProfileData = RemoteWorkerProfileDatas[RemoteWorker->GetProfileId()];
			bAllWorkersConnected &= RemoteWorker->IsConnected();
			int32 NumAssignments = RemoteWorker->NumAssignments();
			BusiestNumAssignments = FMath::Max(NumAssignments, BusiestNumAssignments);
			bool bWorkerIdle = NumAssignments == 0;
			bAnyIdle |= bWorkerIdle;
			ProfileData.UpdateIdle(bWorkerIdle, DeltaTime);
			if (RemoteWorker->HasMessages())
			{
				WorkersWithMessage.Add(RemoteWorker);
			}
			if (bSendHeartbeat)
			{
				RemoteWorker->SignalHeartbeat(ECookDirectorThread::SchedulerThread, LocalHeartbeatNumber);
			}
		}
		for (TPair<FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>>& Pair : ShuttingDownWorkers)
		{
			if (Pair.Value && Pair.Value->HasMessages())
			{
				WorkersWithMessage.Add(Pair.Value);
			}
		}
		ReassignAbortedPackages(DeferredPackagesToReassign);
		RetractionHandler->TickFromSchedulerThread(bAllWorkersConnected, bAnyIdle, BusiestNumAssignments);
	}

	bool bIsStalled = bLocalWorkerIdle && WorkersWithMessage.IsEmpty();
	if (bIsStalled)
	{
		// We are only stalled if we have no local work to do and there is a remote worker with assigned work. If no
		// worker has assigned work then we are done with the cook rather than stalled.
		bool bRemoteWorkerHasWork = !COTFS.PackageDatas->GetAssignedToWorkerSet().IsEmpty();
		if (!bRemoteWorkerHasWork)
		{
			// Do the slow check to see whether we have a package in the SaveStalledAssignedToWorker state only if
			// we have to
			for (FPackageData* PackageData : COTFS.PackageDatas->GetSaveStalledSet())
			{
				if (PackageData->IsInStateProperty(EPackageStateProperty::AssignedToWorkerProperty))
				{
					bRemoteWorkerHasWork = true;
					break;
				}
			}
		}
		bIsStalled = bIsStalled && bRemoteWorkerHasWork;
	}

	{
		bReceivingMessages = true;
		for (TRefCountPtr<FCookWorkerServer>& Worker : WorkersWithMessage)
		{
			Worker->HandleReceiveMessages(ECookDirectorThread::SchedulerThread);
		}
		bReceivingMessages = false;
	}
	for (TPair<UE::CompactBinaryTCP::FMarshalledMessage, ECookBroadcastTiming>& Pair : QueuedBroadcasts)
	{
		BroadcastMessage(MoveTemp(Pair.Key), Pair.Value);
	}
	QueuedBroadcasts.Empty();

	// Process any queued messages
	WorkersWithMessage.Empty();

	SetWorkersStalled(bIsStalled);
	LastTickTimeSeconds = CurrentTime;
}

void FCookDirector::UpdateDisplayDiagnostics() const
{
	DisplayRemainingPackages();
}

void FCookDirector::DisplayRemainingPackages() const
{
	constexpr int32 DisplayWidth = 16;
	UE_LOGF(LogCook, Display, "\t%ls: %d packages remain.", *GetDisplayName(FWorkerId::Local(), DisplayWidth),
		COTFS.NumMultiprocessLocalWorkerAssignments());
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	for (const TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
	{
		FCookWorkerServer* RemoteWorker = Pair.Value;
		UE_LOGF(LogCook, Display, "\t%ls: %d packages remain.",
			*GetDisplayName(*RemoteWorker, DisplayWidth), RemoteWorker->NumAssignments());
	}
}

FString FCookDirector::GetDisplayName(const FWorkerId& WorkerId, int32 PreferredWidth) const
{
	FString Result;
	if (WorkerId.IsInvalid())
	{
		Result = TEXTVIEW("None");
	}
	else if (WorkerId.IsLocal())
	{
		Result = TEXTVIEW("Local");
	}
	else
	{
		const TRefCountPtr<FCookWorkerServer>* RemoteWorker = nullptr;
		{
			FScopeLock CommunicationScopeLock(&CommunicationLock);
			RemoteWorker = FindRemoteWorkerInLock(WorkerId);
			if (!RemoteWorker)
			{
				for (const TPair<FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>>& Pair : ShuttingDownWorkers)
				{
					if (Pair.Value && Pair.Value->GetWorkerId() == WorkerId)
					{
						RemoteWorker = &Pair.Value;
						break;
					}
				}
			}
		}
		if (RemoteWorker)
		{
			Result = FString::Printf(TEXT("%d"), (*RemoteWorker)->GetProfileId());
		}
		else
		{
			Result = FString::Printf(TEXT("Unknown (WorkerId %d)"), WorkerId.GetRemoteIndex());
		}
	}
	constexpr FStringView Prefix(TEXTVIEW("CookWorker "));
	Result = FString(Prefix) + Result.LeftPad(PreferredWidth-Prefix.Len());
	return Result;
}

FString FCookDirector::GetDisplayName(const FCookWorkerServer& RemoteWorker, int32 PreferredWidth) const
{
	FString Result = FString::Printf(TEXT("%d"), RemoteWorker.GetProfileId());
	constexpr FStringView Prefix(TEXTVIEW("CookWorker "));
	Result = FString(Prefix) + Result.LeftPad(PreferredWidth - Prefix.Len());
	return Result;
}

const TRefCountPtr<FCookWorkerServer>* FCookDirector::FindRemoteWorkerInLock(const FWorkerId& WorkerId) const
{
	if (!WorkerId.IsRemote())
	{
		return nullptr;
	}
	return RemoteWorkers.Find(WorkerId.GetRemoteIndex());
}

void FCookDirector::TickCommunication(ECookDirectorThread TickThread)
{
	bool bHasShutdownWorkers = false;
	TickWorkerConnects(TickThread);
	TArray<TRefCountPtr<FCookWorkerServer>, TInlineAllocator<16>> LocalRemoteWorkers;
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		for (const TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair: RemoteWorkers)
		{
			LocalRemoteWorkers.Add(Pair.Value);
		}
		if (!RemoteWorkers.IsEmpty())
		{
			bWorkersActive = true;
		}
		else
		{
			bWorkersActive = false;
			for (const TPair <FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>>& Pair : ShuttingDownWorkers)
			{
				FCookWorkerServer* RemoteWorker = Pair.Key;
				check(RemoteWorker->IsShuttingDown());
				bWorkersActive = bWorkersActive || RemoteWorker->IsFlushingBeforeShutdown();
			}
		}
		bHasShutdownWorkers = !ShuttingDownWorkers.IsEmpty();
	}

	for (TRefCountPtr<FCookWorkerServer>& RemoteWorker: LocalRemoteWorkers)
	{
		RemoteWorker->TickCommunication(TickThread);
		if (RemoteWorker->IsShuttingDown())
		{
			FScopeLock CommunicationScopeLock(&CommunicationLock);
			TRefCountPtr<FCookWorkerServer>& Existing = ShuttingDownWorkers.FindOrAdd(RemoteWorker.GetReference());
			check(!Existing); // We should not be able to send the same pointer into ShuttingDown twice
			bHasShutdownWorkers = true;
		}
	}

	if (bHasShutdownWorkers)
	{
		TickWorkerShutdowns(TickThread);
	}
}

void FCookDirector::PumpCookComplete(bool& bCompleted)
{
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		if (!bCookCompleteSent)
		{
			bool bAllIdle = true;
			for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
			{
				FCookWorkerServer& RemoteWorker = *Pair.Value;
				if (RemoteWorker.NumAssignments() > 0)
				{
					bAllIdle = false;
					break;
				}
			}

			if (bAllIdle && FinalIdleHeartbeatFence == -1)
			{
				bool bSendHeartbeat;
				double CurrentTime = FPlatformTime::Seconds();
				TickHeartbeat(true /* bForceHeartbeat */, CurrentTime, bSendHeartbeat, FinalIdleHeartbeatFence);

				for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
				{
					FCookWorkerServer& RemoteWorker = *Pair.Value;
					RemoteWorker.SignalHeartbeat(ECookDirectorThread::SchedulerThread, FinalIdleHeartbeatFence);
				}
				bAllIdle = false;
			}
			else
			{
				for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
				{
					FCookWorkerServer& RemoteWorker = *Pair.Value;
					if (RemoteWorker.GetLastReceivedHeartbeatNumber() < FinalIdleHeartbeatFence)
					{
						bAllIdle = false;
						SetWorkersStalled(true);
						break;
					}
				}
			}

			if (bAllIdle)
			{
				for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
				{
					FCookWorkerServer& RemoteWorker = *Pair.Value;
					RemoteWorker.SignalCookComplete(ECookDirectorThread::SchedulerThread);
					check(RemoteWorker.IsShuttingDown());
				}
				bCookCompleteSent = true;
			}
		}
		bCompleted = !bWorkersActive;

		if (bCompleted)
		{
			DeactivateMachineResourceReduction();
		}
	}
	TickFromSchedulerThread();
}

void FCookDirector::ShutdownCookSession()
{
	StopCommunicationThread();

	// Cancel any inprogress workers and move them to the Shutdown list
	for (;;)
	{
		TRefCountPtr<FCookWorkerServer> RemoteWorker;
		{
			FScopeLock CommunicationScopeLock(&CommunicationLock);
			if (RemoteWorkers.IsEmpty())
			{
				break;
			}
			RemoteWorker = TMap<int32, TRefCountPtr<FCookWorkerServer>>::TIterator(RemoteWorkers).Value();
		}
		AbortWorker(RemoteWorker->GetWorkerId(), ECookDirectorThread::SchedulerThread);
	}

	// Immediately shutdown any gracefully shutting down workers
	TArray<TRefCountPtr<FCookWorkerServer>, TInlineAllocator<16>> WorkersNeedingAbort;
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		for (TPair<FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>>& Pair : ShuttingDownWorkers)
		{
			// The Value was set by AbortWorker for any new entries, and all old entries guarantee the value is set
			check(Pair.Value);
			if (Pair.Key->IsFlushingBeforeShutdown())
			{
				WorkersNeedingAbort.Add(Pair.Value);
			}
		}
	}
	for (TRefCountPtr<FCookWorkerServer>& RemoteWorker : WorkersNeedingAbort)
	{
		TSet<FPackageData*> UnusedPendingPackages;
		RemoteWorker->AbortWorker(UnusedPendingPackages, ECookDirectorThread::SchedulerThread, HeartbeatNumber);
	}

	// Wait for all the shutdowns to complete
	for (;;)
	{
		TickWorkerShutdowns(ECookDirectorThread::SchedulerThread);
		{
			FScopeLock CommunicationScopeLock(&CommunicationLock);
			if (ShuttingDownWorkers.IsEmpty())
			{
				break;
			}
		}
		constexpr float SleepSeconds = 0.010f;
		FPlatformProcess::Sleep(SleepSeconds);
	}

	// Kill any connections that had just been made and not yet assigned to a Server
	PendingConnections.Reset();

	// Delete temp directories
	{
		FScopeLock FileTransferScopeLock(&FileTransferLock);
		for (const ITargetPlatform* TargetPlatform : COTFS.PlatformManager->GetSessionPlatforms())
		{
			FString FileTransferRoot = FPaths::Combine(COTFS.GetCookOutputFolder(TargetPlatform),
				UE::Cook::MPCookTransferRootRelPath);
			IFileManager::Get().DeleteDirectory(*FileTransferRoot, false /* bRequireExists */, true /* bTree */);
		}
	}

	// Restore the FCookDirector to its original state so that it is ready for a new session
	bWorkersInitialized = false;
	bIsFirstAssignment = true;
	bCookCompleteSent = false;
	bWorkersActive = false;
	HeartbeatNumber = 0;
	NextHeartbeatTimeSeconds = 0.;
	FinalIdleHeartbeatFence = -1;
}

void FCookDirector::Register(IMPCollector* Collector)
{
	TRefCountPtr<IMPCollector>& Existing = Collectors.FindOrAdd(Collector->GetMessageType());
	if (Existing)
	{
		UE_LOGF(LogCook, Error,
			"Duplicate IMPCollectors registered. Guid: %ls, Existing: %ls, Registering: %ls. Keeping the Existing.",
			*Collector->GetMessageType().ToString(), Existing->GetDebugName(), Collector->GetDebugName());
		return;
	}
	Existing = Collector;
}

void FCookDirector::Unregister(IMPCollector* Collector)
{
	TRefCountPtr<IMPCollector> Existing;
	Collectors.RemoveAndCopyValue(Collector->GetMessageType(), Existing);
	if (Existing && Existing.GetReference() != Collector)
	{
		UE_LOGF(LogCook, Error,
			"Duplicate IMPCollector during Unregister. Guid: %ls, Existing: %ls, Unregistering: %ls. Ignoring the Unregister.",
			*Collector->GetMessageType().ToString(), Existing->GetDebugName(), Collector->GetDebugName());
		Collectors.Add(Collector->GetMessageType(), MoveTemp(Existing));
	}
}

void FCookDirector::SetWorkersStalled(bool bInWorkersStalled)
{
	if (bInWorkersStalled != bWorkersStalled)
	{
		bWorkersStalled = bInWorkersStalled;
		if (bWorkersStalled)
		{
			const double CurrentTime = FPlatformTime::Seconds();
			WorkersStalledStartTimeSeconds = CurrentTime;
			WorkersStalledWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
		}
		else
		{
			WorkersStalledStartTimeSeconds = MAX_flt;
			WorkersStalledWarnTimeSeconds = MAX_flt;
		}
	}
	else if (bWorkersStalled)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime >= WorkersStalledWarnTimeSeconds)
		{
			UE_LOGF(LogCook, Display,
				"Cooker has been blocked with no results from remote CookWorkers for %.0f seconds.",
				(float)(CurrentTime - WorkersStalledStartTimeSeconds));
			WorkersStalledWarnTimeSeconds = CurrentTime + GCookProgressWarnBusyTime;
		}
	}
}

void FCookDirector::TickHeartbeat(bool bForceHeartbeat, double CurrentTimeSeconds, bool& bOutSendHeartbeat,
	int32& OutHeartbeatNumber)
{
	constexpr float HeartbeatPeriodSeconds = 30.f;

	bOutSendHeartbeat = false;
	OutHeartbeatNumber = HeartbeatNumber;
	if (bForceHeartbeat)
	{
		bOutSendHeartbeat = true;
	}
	else if (NextHeartbeatTimeSeconds == 0.)
	{
		NextHeartbeatTimeSeconds = CurrentTimeSeconds + HeartbeatPeriodSeconds;
	}
	else if (CurrentTimeSeconds >= NextHeartbeatTimeSeconds)
	{
		bOutSendHeartbeat = true;
	}

	if (bOutSendHeartbeat)
	{
		checkf(HeartbeatNumber < MAX_int32, TEXT("Overflow"));
		HeartbeatNumber++;
		NextHeartbeatTimeSeconds = CurrentTimeSeconds + HeartbeatPeriodSeconds;
	}
}

void FCookDirector::ResetFinalIdleHeartbeatFence()
{
	FinalIdleHeartbeatFence = -1;
}

void FCookDirector::HandleHeartbeatMessage(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
	FHeartbeatMessage&& Message)
{
	if (!bReadSuccessful)
	{
		UE_LOGF(LogCook, Error, "Corrupt HeartbeatMessage received from CookWorker %d. It will be ignored.",
			Context.GetProfileId());
		return;
	}

	Context.GetCookWorkerServer()->SetLastReceivedHeartbeatNumberInLock(Message.HeartbeatNumber);
}

void FCookDirector::HandleFileTransferMessage(FCookWorkerServer& FromWorker, FFileTransferMessage&& Message)
{
	MoveFileSynchronized(*Message.TempFileName, *Message.TargetFileName, FromWorker.GetWorkerId(),
		FromWorker.GetProfileId());

}

void FCookDirector::ReportFileTransferFromLocalWorker(FStringView TempFileName, FStringView TargetFileName)
{
	FScopeLock FileTransferScopeLock(&FileTransferLock);
	TStringBuilder<256> TempFileNameStr(InPlace, TempFileName);
	TStringBuilder<256> TargetFileNameStr(InPlace, TargetFileName);
	MoveFileSynchronized(*TempFileNameStr, *TargetFileNameStr, FWorkerId::Local(), (uint32)-1);
}

void FCookDirector::MoveFileSynchronized(const TCHAR* TempFileName, const TCHAR* TargetFileName,
	FWorkerId WorkerId, uint32 WorkerProfileId)
{
	// This function can called within the lock of a CookWorkerServer, and we're blocking on a critical section.
	// Currently the work done is a little bit of IO to rename files within the Saved\Cooked directory.
	// If it becomes more contended or does more work, we could create a new thread or use AsyncTasks to do the work;
	// we would need to make sure that Director finalization includes waiting on the thread or Tasks,
	// and if using multiple tasks, they would need to synchronize to avoid trying to write to the same TargetFile at
	// once.
	FScopeLock FileTransferScopeLock(&FileTransferLock);
	bool bMoveSucceeded = IFileManager::Get().Move(TargetFileName, TempFileName, true /* bReplace */,
		false /* EvenIfReadOnly */);
	if (!bMoveSucceeded)
	{
		UE_LOGF(LogCook, Error,
			"Could not move UObject::CookAdditionalFilesOverride file transferred from CookWorker %ls from %ls -> %ls. " "The file will be missing from cook output.",
			WorkerId.IsLocal() ? TEXT("Local") : *FString::Printf(TEXT("%u"), WorkerProfileId),
			TempFileName, TargetFileName);
	}
}

bool FCookDirector::HasSentActions() const
{
	return bHasSentActions;
}

void FCookDirector::ClearHasSentActions()
{
	bHasSentActions = false;
}

FCookDirector::FPendingConnection::FPendingConnection(FPendingConnection&& Other)
{
	Swap(Socket, Other.Socket);
	Buffer = MoveTemp(Other.Buffer);
}

FCookDirector::FPendingConnection::~FPendingConnection()
{
	Sockets::CloseSocket(Socket);
}

FSocket* FCookDirector::FPendingConnection::DetachSocket()
{
	FSocket* Result = Socket;
	Socket = nullptr;
	return Result;
}

void FWorkerConnectMessage::Write(FCbWriter& Writer) const
{
	Writer << "RemoteIndex" << RemoteIndex;
}

bool FWorkerConnectMessage::TryRead(FCbObjectView Object)
{
	RemoteIndex = Object["RemoteIndex"].AsInt32(-1);
	return RemoteIndex >= 0;
}

FGuid FWorkerConnectMessage::MessageType(TEXT("302096E887DA48F7B079FAFAD0EE5695"));

bool FCookDirector::TryCreateWorkerConnectSocket()
{
	if (WorkerConnectSocket)
	{
		return true;
	}
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (!SocketSubsystem)
	{
		// Error was already logged in the constructor
		return false;
	}

	FString ErrorReason;
	TSharedPtr<FInternetAddr> ListenAddr;
	WorkerConnectSocket = Sockets::CreateListenSocket(WorkerConnectPort, ListenAddr, WorkerConnectAuthority,
		TEXT("FCookDirector-WorkerConnect"), ErrorReason);
	if (!WorkerConnectSocket)
	{
		UE_LOGF(LogCook, Error,
			"CookDirector could not create listen socket, CookWorkers will be disabled. Reason: %ls.",
			*ErrorReason);
		return false;
	}
	return true;
}


void FCookDirector::InitializeWorkers()
{
	if (bWorkersInitialized)
	{
		return;
	}
	int32 InitialRequestCount = COTFS.InitialRequestCount;
	bWorkersInitialized = true;

	check(!CommunicationThread);
	check(RemoteWorkers.IsEmpty());
	bool bSucceeded = false;
	ON_SCOPE_EXIT
	{
		if (!bSucceeded)
		{
			bWorkersActive = false;
		}
	};

	constexpr int32 MinRequestsForMPCookWithCookSinglePackage = 100;
	UE::Cook::FCookByTheBookOptions& Options = *COTFS.CookByTheBookOptions;
	if (!bCookProcessCountSetByCommandLine
		&& (Options.bSkipHardReferences
			|| (Options.bSkipSoftReferences && InitialRequestCount < MinRequestsForMPCookWithCookSinglePackage)))
	{
		UE_LOGF(LogCook, Display,
			"CookDirector initialization skipped: -CookSinglePackage was requested. CookMultiprocess is disabled and the cooker is running as a single process.");
		return;
	}
	if (Options.bCookList)
	{
		UE_LOGF(LogCook, Display,
			"CookDirector initialization skipped: -CookList was requested. CookMultiprocess is disabled and the cooker is running as a single process.");
		return;
	}

	UE_LOGF(LogCook, Display, "CookProcessCount=%d. CookMultiprocess is enabled with 1 CookDirector and %d %ls.",
		RequestedCookWorkerCount + 1, RequestedCookWorkerCount,
		RequestedCookWorkerCount > 1 ? TEXT("CookWorkers") : TEXT("CookWorker"));

	ActivateMachineResourceReduction();

	if (!TryCreateWorkerConnectSocket())
	{
		return;
	}

	RemoteWorkers.Reserve(RequestedCookWorkerCount);
	for (int32 RemoteIndex = 0; RemoteIndex < RequestedCookWorkerCount; ++RemoteIndex)
	{
		int32 ProfileId = RemoteWorkerProfileDatas.Num();
		RemoteWorkerProfileDatas.Emplace();
		RemoteWorkers.Add(RemoteIndex, new FCookWorkerServer(*this, ProfileId,
			FWorkerId::FromRemoteIndex(RemoteIndex)));
	}
	bWorkersActive = true;

	ConstructReadonlyThreadVariables();
	ShutdownEvent->Reset();
	LaunchCommunicationThread();
	bSucceeded = true;
}

void FCookDirector::RecreateWorkers()
{
	// TODO: Finish implementing the recreation of workers that have crashed

	// Find any unused RemoteIndex less than the maximum used RemoteIndex
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	if (RemoteWorkers.Num() >= RequestedCookWorkerCount || !WorkerConnectSocket)
	{
		return;
	}

	TArray<uint8> UnusedRemoteIndexes;
	RemoteWorkers.KeySort(TLess<>());
	uint8 NextPossiblyOpenIndex = 0;
	for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
	{
		check(NextPossiblyOpenIndex <= Pair.Key);
		while (NextPossiblyOpenIndex != Pair.Key)
		{
			UnusedRemoteIndexes.Add(NextPossiblyOpenIndex++);
		}
	}

	// Add RemoteWorkers, pulling the RemoteIndex id from the UnusedRemoteIndexes if any exist
	// otherwise use the next integer because all indexes up to RemoteWorkers.Num() are in use.
	while (RemoteWorkers.Num() < RequestedCookWorkerCount)
	{
		uint8 RemoteIndex;
		if (UnusedRemoteIndexes.Num())
		{
			RemoteIndex = UnusedRemoteIndexes[0];
			UnusedRemoteIndexes.RemoveAtSwap(0);
		}
		else
		{
			RemoteIndex = RemoteWorkers.Num();
		}
		int32 ProfileId = RemoteWorkerProfileDatas.Num();
		RemoteWorkerProfileDatas.Emplace();
		RemoteWorkers.Add(RemoteIndex, new FCookWorkerServer(*this, ProfileId,
			FWorkerId::FromRemoteIndex(RemoteIndex)));
		bWorkersActive = true;
	}

}
void FCookDirector::ActivateMachineResourceReduction()
{
	if (bHasReducedMachineResources)
	{
		return;
	}
	bHasReducedMachineResources = true;

	// When running a multiprocess cook, we remove the Memory triggers and trigger GC based solely on PressureLevel.
	// But keep the Soft GC settings
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SavedResources.MemoryMaxUsedPhysical = COTFS.MemoryMaxUsedPhysical;
	SavedResources.MemoryMaxUsedVirtual = COTFS.MemoryMaxUsedVirtual;
	COTFS.MemoryMaxUsedPhysical = 0;
	COTFS.MemoryMaxUsedVirtual = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	SavedResources.MemoryMinFreeVirtual = COTFS.MemoryMinFreeVirtual;
	SavedResources.MemoryMinFreePhysical = COTFS.MemoryMinFreePhysical;
	COTFS.MemoryMinFreeVirtual = 0;
	COTFS.MemoryMinFreePhysical = 0;
	SavedResources.MemoryTriggerGCAtPressureLevel = COTFS.MemoryTriggerGCAtPressureLevel;
	COTFS.MemoryTriggerGCAtPressureLevel = FGenericPlatformMemoryStats::EMemoryPressureStatus::Critical;

	UE_LOGF(LogCook, Display, "CookMultiprocess changed CookSettings for Memory:%ls",
		*COTFS.GetCookSettingsForMemoryLogText());

	// Set CoreLimit for updating workerthreads in this process and passing to the commandline for workers
	int32 NumProcesses = RequestedCookWorkerCount + 1;
	int32 NumberOfCores = FPlatformMisc::NumberOfCores();
	int32 HyperThreadCount = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	int32 NumberOfHyperThreadsPerCore = HyperThreadCount / NumberOfCores;
	CoreLimit = FMath::Max(NumberOfCores / NumProcesses, 1);
	SavedResources.NumberOfCores = NumberOfCores;
	SavedResources.HyperThreadCount = HyperThreadCount;

	const TCHAR* CommandLine = FCommandLine::Get();
	float CoreOversubscription = 1.0f;
	if (FParse::Value(CommandLine, TEXT("-MPCookCoreSubscription="), CoreOversubscription))
	{
		CoreLimit = FMath::Clamp(static_cast<int32>(CoreLimit*CoreOversubscription), 1, NumberOfCores);
	}

	int32 CoreIncludingHyperthreadsLimit = CoreLimit * NumberOfHyperThreadsPerCore;
	int32 NumberOfWorkers = FMath::Max(CoreLimit - 1, 1) * NumberOfHyperThreadsPerCore;

	// Update the number of Cores and WorkerThreads for this process
	check(IsInGameThread());
	int32 NumForegroundWorkers = FMath::Min<uint32>(GNumForegroundWorkers, NumberOfWorkers);
	int32 NumBackgroundWorkers = FMath::Max(1, NumberOfWorkers - NumForegroundWorkers);
	NumForegroundWorkers = FMath::Max(1, NumberOfWorkers - NumBackgroundWorkers);
	LowLevelTasks::FScheduler::Get().RestartWorkers(NumForegroundWorkers, NumBackgroundWorkers);

	// Update the number of ShaderCompilerWorkers that can be launched
	GShaderCompilingManager->OnMachineResourcesChanged(CoreLimit, CoreIncludingHyperthreadsLimit);

	UE_LOGF(LogCook, Display, "CookMultiprocess changed number of cores from %d to %d.",
		NumberOfCores, CoreLimit);
	UE_LOGF(LogCook, Display, "CookMultiprocess changed number of hyperthreads from %d to %d.",
		HyperThreadCount, CoreIncludingHyperthreadsLimit);
}

void FCookDirector::DeactivateMachineResourceReduction()
{
	if (!bHasReducedMachineResources)
	{
		return;
	}
	bHasReducedMachineResources = false;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	COTFS.MemoryMaxUsedPhysical = SavedResources.MemoryMaxUsedPhysical;
	COTFS.MemoryMaxUsedVirtual = SavedResources.MemoryMaxUsedVirtual;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	COTFS.MemoryMinFreeVirtual = SavedResources.MemoryMinFreeVirtual;
	COTFS.MemoryMinFreePhysical = SavedResources.MemoryMinFreePhysical;
	COTFS.MemoryTriggerGCAtPressureLevel = SavedResources.MemoryTriggerGCAtPressureLevel;

	UE_LOGF(LogCook, Display, "CookMultiprocess restored CookSettings for Memory:%ls",
		*COTFS.GetCookSettingsForMemoryLogText());

	check(IsInGameThread());
	// Calling RestartWorkers with no arguments calculates NumForegroundWorkers and NumBackgroundWorkers based on the
	// number of cores reported by FPlatformMisc; this is what we want for the restored-to-single-process worker state.
	LowLevelTasks::FScheduler::Get().RestartWorkers();

	GShaderCompilingManager->OnMachineResourcesChanged(SavedResources.NumberOfCores, SavedResources.HyperThreadCount);

	UE_LOGF(LogCook, Display, "CookMultiprocess restored number of cores to %d.", SavedResources.NumberOfCores);
	UE_LOGF(LogCook, Display, "CookMultiprocess restored number of hyperthreads to %d.",
		SavedResources.HyperThreadCount);
}

void FCookDirector::TickWorkerConnects(ECookDirectorThread TickThread)
{
	using namespace UE::CompactBinaryTCP;

	if (!WorkerConnectSocket)
	{
		return;
	}

	bool bReadReady;
	while (WorkerConnectSocket->HasPendingConnection(bReadReady) && bReadReady)
	{
		FSocket* WorkerSocket = WorkerConnectSocket->Accept(TEXT("Client Connection"));
		if (!WorkerSocket)
		{
			UE_LOGF(LogCook, Warning, "Pending connection failed to create a ClientSocket.");
		}
		else
		{
			WorkerSocket->SetNonBlocking(true);
			PendingConnections.Add(FPendingConnection(WorkerSocket));
		}
	}

	for (TArray<FPendingConnection>::TIterator Iter(PendingConnections); Iter; ++Iter)
	{
		FPendingConnection& Conn = *Iter;
		TArray<FMarshalledMessage> Messages;
		EConnectionStatus Status;
		Status = TryReadPacket(Conn.Socket, Conn.Buffer, Messages);
		if (Status != EConnectionStatus::Okay)
		{
			UE_LOGF(LogCook, Warning,
				"Pending connection failed before sending a WorkerPacket: %ls", DescribeStatus(Status));
			Iter.RemoveCurrent();
		}
		if (Messages.Num() == 0)
		{
			continue;
		}
		FPendingConnection LocalConn(MoveTemp(Conn));
		Iter.RemoveCurrent();

		if (Messages[0].MessageType != FWorkerConnectMessage::MessageType)
		{
			UE_LOGF(LogCook, Warning,
				"Pending connection sent a different message before sending a connection message. MessageType: %ls. Connection will be ignored.",
				*Messages[0].MessageType.ToString());
			continue;
		}
		FWorkerConnectMessage Message;
		if (!Message.TryRead(MoveTemp(Messages[0].Object)))
		{
			UE_LOGF(LogCook, Warning,
				"Pending connection sent an invalid Connection Message. Connection will be ignored.");
			continue;
		}
		TRefCountPtr<FCookWorkerServer> RemoteWorker;
		{
			FScopeLock CommunicationScopeLock(&CommunicationLock);
			TRefCountPtr<FCookWorkerServer>* RemoteWorkerPtr;
			RemoteWorkerPtr = RemoteWorkers.Find(Message.RemoteIndex);
			if (!RemoteWorkerPtr)
			{
				TStringBuilder<256> ValidIndexes;
				if (RemoteWorkers.Num())
				{
					RemoteWorkers.KeySort(TLess<>());
					for (TPair<int32, TRefCountPtr<FCookWorkerServer>>& Pair : RemoteWorkers)
					{
						ValidIndexes.Appendf(TEXT("%d,"), Pair.Key);
					}
					ValidIndexes.RemoveSuffix(1); // Remove the terminating comma
				}
				UE_LOGF(LogCook, Warning,
					"Pending connection sent a Connection Message with invalid RemoteIndex %d. ValidIndexes = {%ls}. Connection will be ignored.",
					Message.RemoteIndex, *ValidIndexes);
				continue;
			}
			RemoteWorker = *RemoteWorkerPtr;
		}

		FSocket* LocalSocket = LocalConn.DetachSocket();
		Messages.RemoveAt(0);
		if (!RemoteWorker->TryHandleConnectMessage(Message, LocalSocket, MoveTemp(Messages), TickThread))
		{
			UE_LOGF(LogCook, Warning,
				"Pending connection sent a Connection Message with an already in-use RemoteIndex. Connection will be ignored.");
			Sockets::CloseSocket(LocalSocket);
			continue;
		}
	}
}

void FCookDirector::TickWorkerShutdowns(ECookDirectorThread TickThread)
{
	// Move any newly shutting down workers from RemoteWorkers
	TArray<TRefCountPtr<FCookWorkerServer>> NewShutdowns;
	TArray<TRefCountPtr<FCookWorkerServer>, TInlineAllocator<16>> LocalRemoteWorkers;
	{
		FScopeLock RemoteWorkersScopeLock(&CommunicationLock);
		for (TPair<FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>>& Pair : ShuttingDownWorkers)
		{
			if (!Pair.Value)
			{
				NewShutdowns.Emplace(Pair.Key);
			}
			else
			{
				LocalRemoteWorkers.Add(Pair.Value);
			}
		}
	}
	if (!NewShutdowns.IsEmpty())
	{
		for (FCookWorkerServer* NewShutdown : NewShutdowns)
		{
			AbortWorker(NewShutdown->GetWorkerId(), TickThread);
			{
				FScopeLock CommunicationScopeLock(&CommunicationLock);
				verify(ShuttingDownWorkers.FindOrAdd(NewShutdown).IsValid()); // Abort worker should have set the value
			}
			LocalRemoteWorkers.Emplace(NewShutdown);
		}
	}

	TArray<TRefCountPtr<FCookWorkerServer>, TInlineAllocator<16>> CompletedWorkers;
	for (TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		RemoteWorker->TickCommunication(TickThread);
		if (RemoteWorker->IsShutdownComplete())
		{
			CompletedWorkers.Add(RemoteWorker);
		}
	}
	LocalRemoteWorkers.Empty();

	if (!CompletedWorkers.IsEmpty())
	{
		FScopeLock CommunicationScopeLock(&CommunicationLock);
		for (TRefCountPtr<FCookWorkerServer>& CompletedWorker : CompletedWorkers)
		{
			ShuttingDownWorkers.Remove(CompletedWorker.GetReference());
		}
	}
	CompletedWorkers.Empty();
}

FString FCookDirector::GetWorkerLogFileName(int32 ProfileId)
{
	FString DirectorLogFileName = FGenericPlatformOutputDevices::GetAbsoluteLogFilename();
	FStringView BaseFileName = FPathViews::GetBaseFilenameWithPath(DirectorLogFileName);
	FStringView Extension = FPathViews::GetExtension(DirectorLogFileName, true /* bIncludeDot */);
	return FString::Printf(TEXT("%.*s_Worker%d%*s"), BaseFileName.Len(), BaseFileName.GetData(), ProfileId,
		Extension.Len(), Extension.GetData());
}

FString FCookDirector::GetWorkerCommandLine(FWorkerId WorkerId, int32 ProfileId)
{
	const TCHAR* CommandLine = FCommandLine::Get();

	const TCHAR* ProjectName = FApp::GetProjectName();
	checkf(ProjectName && ProjectName[0], TEXT("Expected UnrealEditor to be running with a non-empty project name"));

	// Note that we need to handle quoted strings for e.g. a projectfile with spaces in it; FParse::Token
	// does handle them
	FString Token;
	TArray<FString> Tokens;
	bool bAssetRegistryCacheWritingDisabled = false;
	while (FParse::Token(CommandLine, Token, false /* bUseEscape */))
	{
		if (Token.IsEmpty())
		{
			continue;
		}
		if (Token.StartsWith(TEXT("-run=")) ||
			Token == TEXT("-CookOnTheFly") ||
			Token == TEXT("-CookWorker") ||
			Token.StartsWith(TEXT("-CookCultures")) ||
			Token.StartsWith(TEXT("-CookDirectorHost=")) ||
			Token.StartsWith(TEXT("-MultiprocessId=")) ||
			Token.StartsWith(TEXT("-CookProfileId=")) ||
			Token.StartsWith(TEXT("-ShowCookWorker")) ||
			Token.StartsWith(TEXT("-CoreLimit")) ||
			Token.StartsWith(TEXT("-PhysicalCoreLimit")) ||
			Token.StartsWith(TEXT("-MPCookCoreSubscription")) ||
			Token.StartsWith(TEXT("-CookProcessCount=")) ||
			Token.StartsWith(TEXT("-abslog=")) ||
			Token.StartsWith(TEXT("-unattended"))
			)
		{
			continue;
		}
		else if (Token.StartsWith(TEXT("-tracefile=")))
		{
			FString TraceFile;
			FString TokenString(Token);
			if (FParse::Value(*TokenString, TEXT("-tracefile="), TraceFile) && !TraceFile.IsEmpty())
			{
				FStringView BaseFilenameWithPath = FPathViews::GetBaseFilenameWithPath(TraceFile);
				FStringView Extension = FPathViews::GetExtension(TraceFile, true /* bIncludeDot */);
				Tokens.Add(FString::Printf(TEXT("-tracefile=\"%.*s_Worker%d%.*s\""),
					BaseFilenameWithPath.Len(), BaseFilenameWithPath.GetData(),
					ProfileId,
					Extension.Len(), Extension.GetData()));
				continue;
			}
		}
		else if (Token == TEXT("-NoAssetRegistryCacheWrite"))
		{
			// If cache writing has been disabled, we must also disable reading of the discovery cache on workers.
			// Otherwise workers could be looking at old discovery results that no longer match what's on disk.
			Tokens.Add(TEXT("-NoAssetRegistryDiscoveryCache"));
			bAssetRegistryCacheWritingDisabled = true;
		}
		Tokens.Add(MoveTemp(Token));
	}

	if (Tokens[0] != ProjectName && !Tokens[0].EndsWith(TEXT(".uproject"), ESearchCase::IgnoreCase))
	{
		FString ProjectFilePath = FPaths::GetProjectFilePath();
		if (!FPaths::IsSamePath(Tokens[0], ProjectFilePath))
		{
			Tokens.Insert(ProjectFilePath, 0);
		}
	}
	Tokens.Insert(TEXT("-run=cook"), 1);
	Tokens.Insert(TEXT("-cookworker"), 2);
	Tokens.Insert(FString::Printf(TEXT("-CookProfileId=%d"), ProfileId), 3);
	Tokens.Insert(FString::Printf(TEXT("-MultiprocessId=%d"), WorkerId.GetMultiprocessId()), 4);
	// This should have been constructed in TryCreateWorkerConnectSocket before any CookWorkerServers could exist to
	// call GetWorkerCommandLine
	check(!WorkerConnectAuthority.IsEmpty());
	Tokens.Add(FString::Printf(TEXT("-CookDirectorHost=%s"), *WorkerConnectAuthority));
	Tokens.Add(TEXT("-unattended"));
	Tokens.Add(FString::Printf(TEXT("-abslog=%s"), *GetWorkerLogFileName(ProfileId)));

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	if (AssetRegistry.HasSerializedDiscoveryCache())
	{
		UE_CLOGF(bAssetRegistryCacheWritingDisabled, LogCook, Warning, "Invalid configuration: The Cook Director has written an AssetRegistry discovery cache when explicitly told not to via -NoAssetRegistryCacheWrite");
		if (!bAssetRegistryCacheWritingDisabled)
		{
			// If the director was successful in writing a discovery cache, let workers re-use the director's cache
			Tokens.Add(TEXT("-AssetRegistryCacheSkipInvalidate"));
		}
	}

	if (CoreLimit > 0)
	{
		Tokens.Add(FString::Printf(TEXT("-PhysicalCoreLimit=%d"), CoreLimit));
	}

	// We are joining the tokens back into a commandline string; wrap tokens with whitespace in quotes
	for (FString& IterToken : Tokens)
	{
		int32 IndexOfWhitespace = UE::String::FindFirstOfAnyChar(IterToken, { ' ', '\r', '\n' });
		if (IndexOfWhitespace != INDEX_NONE)
		{
			int32 IndexOfQuote;
			if (!IterToken.FindChar('\"', IndexOfQuote))
			{
				IterToken = FString::Printf(TEXT("\"%s\""), *IterToken);
			}
		}
	}
	return FString::Join(Tokens, TEXT(" "));
}

bool FDirectorConnectionInfo::TryParseCommandLine()
{
	if (!FParse::Value(FCommandLine::Get(), TEXT("-CookDirectorHost="), HostURI))
	{
		UE_LOGF(LogCook, Error, "CookWorker startup failed: no CookDirector specified on commandline.");
		return false;
	}
	uint32 MultiprocessId = UE::GetMultiprocessId();
	if (MultiprocessId == 0 && !FParse::Value(FCommandLine::Get(), TEXT("-MultiprocessId="), MultiprocessId))
	{
		UE_LOGF(LogCook, Error, "CookWorker startup failed: no MultiprocessId specified on commandline.");
		return false;
	}
	if (MultiprocessId < 1 || 257 <= MultiprocessId)
	{
		UE_LOGF(LogCook, Error,
			"CookWorker startup failed: commandline had invalid -MultiprocessId=%d; MultiprocessId must be in the range [1, 256].",
			MultiprocessId);
		return false;
	}
	RemoteIndex = static_cast<int32>(MultiprocessId - 1);
	return true;
}

void FCookDirector::LoadBalance(TConstArrayView<FWorkerId> SortedWorkers, TArrayView<FPackageData*> Requests,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments)
{
	OutAssignments.Reset(Requests.Num());
	bool bLogResults = bIsFirstAssignment;

	switch (LoadBalanceAlgorithm)
	{
	case ELoadBalanceAlgorithm::Striped:
		return LoadBalanceStriped(SortedWorkers, Requests, MoveTemp(RequestGraph), OutAssignments, bLogResults);
	case ELoadBalanceAlgorithm::CookBurden:
		return LoadBalanceCookBurden(SortedWorkers, Requests, MoveTemp(RequestGraph), OutAssignments, bLogResults);
	}
	checkNoEntry();
	return LoadBalanceCookBurden(SortedWorkers, Requests, MoveTemp(RequestGraph), OutAssignments, bLogResults);
}

void FCookDirector::AbortWorker(FWorkerId WorkerId, ECookDirectorThread TickThread)
{
	check(!WorkerId.IsLocal());
	int32 Index = WorkerId.GetRemoteIndex();
	TRefCountPtr<FCookWorkerServer> RemoteWorker;
	{
		FScopeLock RemoteWorkersScopeLock(&CommunicationLock);
		RemoteWorkers.RemoveAndCopyValue(Index, RemoteWorker);
		if (!RemoteWorker)
		{
			return;
		}
	}
	TSet<FPackageData*> PackagesToReassignSet;
	RemoteWorker->AbortAllAssignments(PackagesToReassignSet, TickThread, HeartbeatNumber);
	if (!RemoteWorker->IsShuttingDown())
	{
		RemoteWorker->AbortWorker(PackagesToReassignSet, TickThread, HeartbeatNumber);
	}
	TArray<FPackageData*> PackagesToReassign = PackagesToReassignSet.Array();

	if (TickThread == ECookDirectorThread::SchedulerThread)
	{
		ReassignAbortedPackages(PackagesToReassign);
	}
	{
		FScopeLock RemoteWorkersScopeLock(&CommunicationLock);
		TRefCountPtr<FCookWorkerServer>& Existing = ShuttingDownWorkers.FindOrAdd(RemoteWorker.GetReference());
		// We should not be able to abort a worker twice because we removed it from RemoteWorkers above
		check(!Existing);
		Existing = MoveTemp(RemoteWorker);

		if (TickThread != ECookDirectorThread::SchedulerThread)
		{
			DeferredPackagesToReassign.Append(PackagesToReassign);
		}
	}
}

void FCookDirector::ReassignAbortedPackages(TArray<FPackageData*>& PackagesToReassign)
{
	for (FPackageData* PackageData : PackagesToReassign)
	{
		// Packages that were assigned to a worker should be in the AssignedToWorker state and therefore in progress.
		check(PackageData->IsInProgress());
		PackageData->SetWorkerAssignment(FWorkerId::Invalid());
		EPackageState NewState = PackageData->IsInStateProperty(EPackageStateProperty::Saving)
			? EPackageState::SaveActive
			: EPackageState::Request;
		PackageData->SendToState(NewState, ESendFlags::QueueAddAndRemove, EStateChangeReason::ReassignAbortedPackages);
	}
	PackagesToReassign.Empty();
}

void FCookDirector::ConstructReadonlyThreadVariables()
{
	IsCookIgnoreTimeouts(); // The global variables are read-only; call them now to initialize them
	CommandletExecutablePath = FUnrealEdMisc::Get().GetProjectEditorBinaryPath();

	InitialConfigMessage = MakeUnique<FInitialConfigMessage>();
	FMPCollectorServerTickContext StartupContext(FMPCollectorServerTickContext::EServerEventType::WorkerStartup);
	StartupContext.Platforms = COTFS.PlatformManager->GetSessionPlatforms();
	InitialConfigMessage->ReadFromLocal(COTFS, StartupContext.Platforms,
		*COTFS.CookByTheBookOptions, *COTFS.CookOnTheFlyOptions, BeginCookContext);


	for (const TPair<FGuid, TRefCountPtr<IMPCollector>>& CollectorPair : Collectors)
	{
		IMPCollector* Collector = CollectorPair.Value.GetReference();
		Collector->ServerTick(StartupContext);
		if (!StartupContext.Messages.IsEmpty())
		{
			FGuid MessageType = Collector->GetMessageType();
			for (FCbObject& Object : StartupContext.Messages)
			{
				InitialConfigMessage->AddMessage({ MessageType, MoveTemp(Object) });
			}
			StartupContext.Messages.Reset();
		}
	}
}

const FInitialConfigMessage& FCookDirector::GetInitialConfigMessage()
{
	return *InitialConfigMessage;
}

FCookDirector::FLaunchInfo FCookDirector::GetLaunchInfo(FWorkerId WorkerId, int32 ProfileId)
{
	FLaunchInfo Info;
	Info.ShowWorkerOption = GetShowWorkerOption();
	Info.CommandletExecutable = CommandletExecutablePath;
	Info.WorkerCommandLine = GetWorkerCommandLine(WorkerId, ProfileId);
	return Info;
}

#if ENABLE_COOK_STATS
void FCookDirector::LogCookStats(FCookStatsManager::AddStatFuncRef AddStat)
{
	auto IdleTimeToString = [](float IdleTime)
	{
		return FString::Printf(TEXT("%.1fs"), IdleTime);
	};
	TArray<FCookStatsManager::StringKeyValue> Stats;
	Stats.Emplace(TEXT("LocalWorker IdleTime"), IdleTimeToString(LocalWorkerProfileData->IdleTimeSeconds));
	for (int32 ProfileId = 0; ProfileId < RemoteWorkerProfileDatas.Num(); ++ProfileId)
	{
		FCookWorkerProfileData& ProfileData = RemoteWorkerProfileDatas[ProfileId];
		Stats.Emplace(FString::Printf(TEXT("CookWorker %d IdleTime"), ProfileId),
			IdleTimeToString(ProfileData.IdleTimeSeconds));
	}
	AddStat(TEXT("CookDirector"), Stats);
}
#endif

FCookDirector::FRetractionHandler::FRetractionHandler(FCookDirector& InDirector)
	: Director(InDirector)
{
}

FCookDirector::FRetractionHandler::ERetractionState FCookDirector::FRetractionHandler::TickWantToRetract(
	bool& bOutAnyIdle, int32& OutBusiestNumAssignments)
{
	FWorkerId BusiestWorker;
	TArray<FWorkerId> IdleWorkers;
	int32 BusiestNumAssignments = 0;

	if (Director.bAllowLocalCooks)
	{
		int32 NumAssignments = Director.COTFS.NumMultiprocessLocalWorkerAssignments();
		if (NumAssignments > BusiestNumAssignments)
		{
			if (IsAvailableForRetraction(FWorkerId::Local()))
			{
				BusiestWorker = FWorkerId::Local();
				BusiestNumAssignments = NumAssignments;
			}
		}
		if (NumAssignments == 0)
		{
			IdleWorkers.Add(FWorkerId::Local());
		}
	}

	TArray<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers = Director.CopyRemoteWorkers();
	for (const TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		int32 NumAssignments = RemoteWorker->NumAssignments();
		if (NumAssignments > BusiestNumAssignments)
		{
			if (IsAvailableForRetraction(RemoteWorker->GetWorkerId()))
			{
				BusiestWorker = RemoteWorker->GetWorkerId();
				BusiestNumAssignments = NumAssignments;
			}
		}
		if (NumAssignments == 0)
		{
			IdleWorkers.Add(RemoteWorker->GetWorkerId());
		}
	}
	bOutAnyIdle = !IdleWorkers.IsEmpty();
	OutBusiestNumAssignments = BusiestNumAssignments;

	if (IdleWorkers.IsEmpty() || BusiestNumAssignments < RetractionMinimumNumAssignments)
	{
		// Worker loads changed after the point where we decided to initialize the RetractionHandler,
		// or all workers with packages assigned are unavailable for retraction, so retraction is not
		// currently possible. Try again later.
		return ERetractionState::WantToRetract;
	}
	check(!BusiestWorker.IsInvalid());

	// Plan to divide the assignments evenly between all idle workers and the one busiest worker. This means
	// retracting all but 1/(N+1) packages from the busiest worker.
	int32 NumAssignmentsToRetract = (BusiestNumAssignments * IdleWorkers.Num()) / (IdleWorkers.Num() + 1);
	TStringBuilder<256> IdleWorkerListText;
	for (FWorkerId& WorkerId : IdleWorkers)
	{
		IdleWorkerListText << Director.GetDisplayName(WorkerId) << TEXT(", ");
	}
	IdleWorkerListText.RemoveSuffix(2);
	UE_LOGF(LogCook, Display,
		"Idle CookWorkers: { %ls }. Retracting %d packages from %ls to distribute to the idle CookWorkers.",
		*IdleWorkerListText, NumAssignmentsToRetract, *Director.GetDisplayName(BusiestWorker));
	Director.DisplayRemainingPackages();

	if (BusiestWorker.IsLocal())
	{
		ExpectedWorker = FWorkerId::Local();
		WorkerWithResults = ExpectedWorker;
		TArray<FName> LocalPackagesToRetract;
		Director.COTFS.GetPackagesToRetract(NumAssignmentsToRetract, LocalPackagesToRetract);
		PackagesToRetract.FindOrAdd(ExpectedWorker).Append(MoveTemp(LocalPackagesToRetract));
	}
	else
	{
		TRefCountPtr<FCookWorkerServer>* RemoteWorker = LocalRemoteWorkers.FindByPredicate(
			[&BusiestWorker](const TRefCountPtr<FCookWorkerServer>& X) { return X->GetWorkerId() == BusiestWorker; });
		check(RemoteWorker);
		FRetractionRequestMessage Message;
		Message.RequestedCount = NumAssignmentsToRetract;
		(*RemoteWorker)->SendMessage(Message, ECookDirectorThread::SchedulerThread);
		ExpectedWorker = BusiestWorker;
		MessageSentTimeSeconds = FPlatformTime::Seconds();
		LastWarnTimeSeconds = MessageSentTimeSeconds;
	}

	return ERetractionState::WaitingForResponse;
}

void FCookDirector::FRetractionHandler::InitializeForResultsMessage(const FWorkerId& FromWorker)
{
	ExpectedWorker = FromWorker;
}

void FCookDirector::FRetractionHandler::TickFromSchedulerThread(bool bAllWorkersConnected, bool bAnyIdle,
	int32 BusiestNumAssignments)
{
	bool bHadStateChange;

	// Enforce a constraint that the loop terminates. Some states require followup states to be executed in
	// the same frame, so we can only enforce this constraint during certain transitions. We enforce it based
	// on returning to an earlier state; we currently know that those are never required to execute in the same
	// frame.
	int32 NumReturnsToEarlierState = 0;
	constexpr int32 MaxNumReturnsToEarlierState = 2;
	for (;;)
	{
		bHadStateChange = false;
		ERetractionState OldState = RetractionState;
		switch (RetractionState)
		{
		case ERetractionState::Idle:
		{
			if (!bAnyIdle || !bAllWorkersConnected || BusiestNumAssignments <= RetractionMinimumNumAssignments)
			{
				break;
			}
			SetRetractionState(ERetractionState::WantToRetract, bHadStateChange);
			break;
		}
		case ERetractionState::WantToRetract:
		{
			if (!bAnyIdle || !bAllWorkersConnected || BusiestNumAssignments <= RetractionMinimumNumAssignments)
			{
				SetRetractionState(ERetractionState::Idle, bHadStateChange);
				break;
			}
			ERetractionState NewState = TickWantToRetract(bAnyIdle, BusiestNumAssignments);
			SetRetractionState(NewState, bHadStateChange);
			break;
		}
		case ERetractionState::WaitingForResponse:
		{
			ERetractionState NewState = TickWaitingForResponse();
			SetRetractionState(NewState, bHadStateChange);
			break;
		}
		default:
			checkNoEntry();
			break;
		}

		if (!bHadStateChange || RetractionState == ERetractionState::Idle)
		{
			break;
		}
		if (RetractionState < OldState)
		{
			++NumReturnsToEarlierState;
			if (NumReturnsToEarlierState > MaxNumReturnsToEarlierState)
			{
				break;
			}
		}
	}
}

void FCookDirector::FRetractionHandler::SetRetractionState(ERetractionState NewState, bool &bOutHadStateChange)
{
	if (RetractionState == NewState)
	{
		bOutHadStateChange = false;
		return;
	}

	bOutHadStateChange = true;
	RetractionState = NewState;
	if (NewState == ERetractionState::Idle)
	{
		WorkersUnavailableForRetract.Empty();
	}
}

bool FCookDirector::FRetractionHandler::IsAvailableForRetraction(const FWorkerId& WorkerId)
{
	// Called from inside CommunicationLock
	int32* AssignedPackagesFence = WorkersUnavailableForRetract.Find(WorkerId);
	if (!AssignedPackagesFence)
	{
		return true;
	}

	int32 CurrentFenceMarker;
	if (WorkerId.IsLocal())
	{
		CurrentFenceMarker = Director.COTFS.PackageDatas->GetMonitor().GetMPCookAssignedFenceMarker();
	}
	else
	{
		const TRefCountPtr<FCookWorkerServer>* RemoteWorker = Director.FindRemoteWorkerInLock(WorkerId);
		if (!RemoteWorker)
		{
			WorkersUnavailableForRetract.Remove(WorkerId);
			return true;
		}
		CurrentFenceMarker = (*RemoteWorker)->GetPackagesAssignedFenceMarker();
	}

	if (*AssignedPackagesFence == CurrentFenceMarker)
	{
		// FenceMarker has not changed since we recorded the worker as unavailable for retraction at that fence marker
		// The worker is still unavailable for retraction
		return false;
	}

	WorkersUnavailableForRetract.Remove(WorkerId);
	return true;
}

FCookDirector::FRetractionHandler::ERetractionState FCookDirector::FRetractionHandler::TickWaitingForResponse()
{
	// Called from inside CommunicationLock
	if (ExpectedWorker.IsInvalid())
	{
		// We decided to cancel
		checkf(PackagesToRetract.IsEmpty(), TEXT("We should not have any packages when we cancelled."));
		return ERetractionState::Idle;
	}
	if (WorkerWithResults.IsInvalid())
	{
		double CurrentTime = FPlatformTime::Seconds();
		constexpr float WarnDuration = 60.f;

		if (static_cast<float>(CurrentTime - LastWarnTimeSeconds) < WarnDuration)
		{
			return ERetractionState::WaitingForResponse;
		}
		check(ExpectedWorker.IsRemote());
		{
			const TRefCountPtr<FCookWorkerServer>* RemoteWorkerPtr = Director.FindRemoteWorkerInLock(ExpectedWorker);
			if (!RemoteWorkerPtr)
			{
				// The CookWorker aborted and we already reassigned all of its packages; stop waiting for a retraction
				// message from it.
				check(PackagesToRetract.IsEmpty()); // Otherwise WorkerWithResults would have been set
				ExpectedWorker = FWorkerId::Invalid();
				return ERetractionState::Idle;
			}
		}
		UE_CLOGF(!IsCookIgnoreTimeouts(), LogCook, Display,
			"%ls has not responded to a RetractionRequest message for %.1f seconds. Continuing to wait...",
			*Director.GetDisplayName(ExpectedWorker), static_cast<float>(CurrentTime - MessageSentTimeSeconds));
		LastWarnTimeSeconds = CurrentTime;
		return ERetractionState::WaitingForResponse;
	}

	// Convert names to packagedatas and collect results from all CookWorkers who sent a message.
	TArray<FPackageData*> PackageDatasToReassign;
	for (const TPair<FWorkerId, TArray<FName>>& Pair : PackagesToRetract)
	{
		TRefCountPtr<FCookWorkerServer> RemoteWorker;
		if (Pair.Key.IsRemote())
		{
			const TRefCountPtr<FCookWorkerServer>* FoundRemoteWorker = Director.FindRemoteWorkerInLock(Pair.Key);
			if (FoundRemoteWorker)
			{
				RemoteWorker = *FoundRemoteWorker;
			}
		}
		TArray<FPackageData*> WorkerPackageDatas;
		WorkerPackageDatas.Reserve(Pair.Value.Num());
		for (FName PackageName : Pair.Value)
		{
			FPackageData* PackageData = Director.COTFS.PackageDatas->FindPackageDataByPackageName(PackageName);
			if (PackageData)
			{
				WorkerPackageDatas.Add(PackageData);
			}
		}
		if (RemoteWorker)
		{
			// The worker(s) that sent the retraction message aborted all of the packages, so mark locally that they
			// have been aborted
			RemoteWorker->AbortAssignments(WorkerPackageDatas, ECookDirectorThread::SchedulerThread,
				Director.HeartbeatNumber, ENotifyRemote::LocalOnly);
		}
		PackageDatasToReassign.Append(MoveTemp(WorkerPackageDatas));
	}

	// Reassign the packages
	ERetractionResult Result = ReassignPackages(WorkerWithResults, PackageDatasToReassign);
	if (Result == ERetractionResult::NoneAvailable)
	{
		TOptional<int32> AssignedPackagesFence;
		if (WorkerWithResults.IsLocal())
		{
			AssignedPackagesFence.Emplace(Director.COTFS.PackageDatas->GetMonitor().GetMPCookAssignedFenceMarker());
		}
		else
		{
			const TRefCountPtr<FCookWorkerServer>* RemoteWorker = Director.FindRemoteWorkerInLock(WorkerWithResults);
			if (RemoteWorker)
			{
				AssignedPackagesFence.Emplace((*RemoteWorker)->GetPackagesAssignedFenceMarker());
			}
		}
		if (AssignedPackagesFence.IsSet())
		{
			WorkersUnavailableForRetract.Add(WorkerWithResults, *AssignedPackagesFence);
		}
	}

	// Mark that we are no longer waiting
	ExpectedWorker = FWorkerId::Invalid();
	WorkerWithResults = FWorkerId::Invalid();
	PackagesToRetract.Empty();

	// Return to WantToRetract state; that state will handle returning to idle if the retraction was sufficient
	return ERetractionState::WantToRetract;
}

void FCookDirector::FRetractionHandler::HandleRetractionMessage(FMPCollectorServerMessageContext& Context,
	bool bReadSuccessful, FRetractionResultsMessage&& Message)
{
	// Called from inside CommunicationLock
	if (!bReadSuccessful)
	{
		UE_LOGF(LogCook, Error,
			"Corrupt RetractionResultsMessage received from CookWorker %d. It will be ignored and packages may fail to cook.",
			Context.GetProfileId());
		return;
	}

	if (RetractionState != ERetractionState::WaitingForResponse)
	{
		UE_LOGF(LogCook, Warning, "Retractionmessage received from CookWorker %d when we were not expecting one.",
			Context.GetProfileId());
		InitializeForResultsMessage(Context.GetWorkerId());
		bool bUnusedHadStateChange;
		SetRetractionState(ERetractionState::WaitingForResponse, bUnusedHadStateChange);
	}

	UE_CLOGF(WorkerWithResults.IsValid(), LogCook, Error,
		"Unexpectedly received RetractionResults message from multiple CookWorkers. Merging the results.");
	WorkerWithResults = Context.GetWorkerId();
	PackagesToRetract.FindOrAdd(WorkerWithResults).Append(Message.ReturnedPackages);
}

FCookDirector::FRetractionHandler::ERetractionResult
FCookDirector::FRetractionHandler::ReassignPackages(const FWorkerId& FromWorker,
	TConstArrayView<FPackageData*> Packages)
{
	TArray<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers = Director.CopyRemoteWorkers();
	ECookPhase CookPhase = Director.COTFS.GetCookPhase();
	TConstArrayView<const ITargetPlatform*> SessionPlatforms = Director.COTFS.PlatformManager->GetSessionPlatforms();

	TArray<FWorkerId> WorkersRequiredByConstraint;
	TArray<FPackageData*> AssignmentPackages;
	for (FPackageData* PackageData : Packages)
	{
		EPackageState State = PackageData->GetState();
		if (State == EPackageState::Idle)
		{
			continue;
		}

		// Handle and log diagnostics for an unexpected edge case we have seen: retracted packages already have been
		// committed. Demote them to idle now rather than reassigning.
		int32 NumNeedCommitPlatforms = 0;
		for (const ITargetPlatform* SessionPlatform : SessionPlatforms)
		{
			FPackagePlatformData* PlatformData = PackageData->FindPlatformData(SessionPlatform);
			if (PlatformData && PlatformData->NeedsCommit(SessionPlatform, CookPhase))
			{
				++NumNeedCommitPlatforms;
			}
		}
		if (NumNeedCommitPlatforms == 0)
		{
			TStringBuilder<256> PlatformDataText;
			for (const TPair<const ITargetPlatform*, FPackagePlatformData>& PlatformPair : PackageData->GetPlatformDatas())
			{
				PlatformDataText.Appendf(TEXT("{ %s: Reachable=%s, Committed=%s }, "),
					PlatformPair.Key == CookerLoadingPlatformKey ? TEXT("CookerLoadingPlatform") : *PlatformPair.Key->PlatformName(),
					PlatformPair.Value.IsReachable(CookPhase == ECookPhase::Cook
						? EReachability::Runtime : EReachability::Build) ? TEXT("true") : TEXT("false"),
					PlatformPair.Value.IsCommitted() ? TEXT("true") : TEXT("false"));
			}
			UE_LOGF(LogCook, Warning,
				"InternalCookerError: Package %ls was retracted from %ls, but already has no platforms needing commit. Demoting the package to idle instead of reassinging. State = %ls. CookPhase = %ls. WorkerAssignment = %ls. [ %ls ]",
				*PackageData->GetPackageName().ToString(), *Director.GetDisplayName(FromWorker), LexToString(PackageData->GetState()),
				LexToString(CookPhase), *Director.GetDisplayName(PackageData->GetWorkerAssignment()), *PlatformDataText);
			PackageData->SetWorkerAssignment(FWorkerId::Invalid());
			Director.COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::AlreadyCooked);
			continue;
		}

		FWorkerId WorkerConstraint = PackageData->GetWorkerAssignmentConstraint();
		if (WorkerConstraint.IsValid())
		{
			if (!WorkerConstraint.IsLocal() && !LocalRemoteWorkers.FindByPredicate(
				[&WorkerConstraint](const TRefCountPtr<FCookWorkerServer>& X)
				{
					return X->GetWorkerId() == WorkerConstraint;
				}))
			{
				continue;
			}
			WorkersRequiredByConstraint.AddUnique(WorkerConstraint);
		}

		AssignmentPackages.Add(PackageData);
		// The package is no longer assigned to the worker, and the worker already knows it, so remove its assignment.
		// We have to remove the assignment so that we can reassign it down below; it is invalid to call
		// SetWorkerAssignment to a new worker without clearing the old worker first.
		PackageData->SetWorkerAssignment(FWorkerId::Invalid());
	}
	if (AssignmentPackages.IsEmpty())
	{
		UE_LOGF(LogCook, Display,
			"Retraction results message received from %ls; no packages were available for retraction.",
			*Director.GetDisplayName(FromWorker));
		Director.DisplayRemainingPackages();
		return ERetractionResult::NoneAvailable;
	}

	TArray<FWorkerId> WorkersToSplitOver = CalculateWorkersToSplitOver(AssignmentPackages.Num(), FromWorker,
		LocalRemoteWorkers);
	if (WorkersToSplitOver.IsEmpty())
	{
		// Send the packages back to the Director for reassignment
		TPackageDataMap<ESuppressCookReason>& RestartedRequests = Director.COTFS.PackageDatas->GetRequestQueue().GetRestartedRequests();
		for (FPackageData* PackageData : AssignmentPackages)
		{
			// UnStall the package to handle the case where it was previously in the save state on the LocalWorker
			// and was retracted to a remote worker, so we stalled it, but now we've pulled it back to the LocalWorker.
			PackageData->UnStall(ESendFlags::QueueAddAndRemove);
			// If the packagedata was unstalled back into the Save State, then keep it there, but otherwise it is in
			// the AssignedToWorker state so kick it back to readyrequest.
			if (PackageData->GetState() != EPackageState::SaveActive)
			{
				PackageData->SendToState(EPackageState::Request, ESendFlags::QueueRemove, EStateChangeReason::Retraction);
				RestartedRequests.Add(PackageData, ESuppressCookReason::RetractedByCookDirector);
			}
		}
		// MPCOOKTODO: Add a method to PumpRequests long enough to assign the packages
		UE_LOGF(LogCook, Display,
			"%d packages retracted from %ls. No workers are currently idle so the packages were assigned evenly to all CookWorkers.",
			AssignmentPackages.Num(), *Director.GetDisplayName(FromWorker));
		Director.DisplayRemainingPackages();
		return ERetractionResult::Retracted;
	}
	for (const FWorkerId& WorkerId : WorkersRequiredByConstraint)
	{
		WorkersToSplitOver.AddUnique(WorkerId);
	}

	TStringBuilder<256> WorkerListText;
	for (const FWorkerId& WorkerId : WorkersToSplitOver)
	{
		WorkerListText << Director.GetDisplayName(WorkerId) << TEXT(", ");
	}
	WorkerListText.RemoveSuffix(2);
	UE_LOGF(LogCook, Display, "%d packages retracted from %ls and distributed to idle workers { %ls }.",
		AssignmentPackages.Num(), *Director.GetDisplayName(FromWorker), *WorkerListText);

	TMap<FPackageData*, TArray<FPackageData*>> RequestGraph;
	TArray<FWorkerId> Assignments;
	Director.AssignRequests(MoveTemp(WorkersToSplitOver), LocalRemoteWorkers, AssignmentPackages, Assignments,
		MoveTemp(RequestGraph), false /* bInitialAssignment */);
	FRequestQueue& RequestQueue = Director.COTFS.PackageDatas->GetRequestQueue();
	bool bAssignedToLocal = false;
	for (int32 Index = 0; Index < AssignmentPackages.Num(); ++Index)
	{
		FPackageData* PackageData = AssignmentPackages[Index];
		FWorkerId Assignment = Assignments[Index];
		if (Assignment.IsInvalid())
		{
			Director.COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAdd,
				ESuppressCookReason::MultiprocessAssignmentError);
		}
		else if (Assignment.IsLocal())
		{
			// UnStall the package to handle the case where it was previously in the save state on the LocalWorker
			// and was retracted to a remote worker, so we stalled it, but now we've pulled it back to the LocalWorker.
			PackageData->UnStall(ESendFlags::QueueAddAndRemove);
			// If the packagedata was unstalled back into the Save State, then keep it there, but otherwise it is in
			// the AssignedToWorker state so kick it back to readyrequest.
			if (PackageData->GetState() != EPackageState::SaveActive)
			{
				PackageData->SendToState(EPackageState::Request, ESendFlags::QueueRemove, EStateChangeReason::Retraction);
				RequestQueue.AddReadyRequest(PackageData);
			}
			bAssignedToLocal = true;
		}
		else
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
			if (!GenerationHelper)
			{
				GenerationHelper = PackageData->GetParentGenerationHelper();
			}
			bool bShouldStall = false;
			if (GenerationHelper)
			{
				bShouldStall = GenerationHelper->ShouldRetractionStallRatherThanDemote(*PackageData);
			}
			if (bShouldStall)
			{
				PackageData->Stall(EPackageState::SaveStalledAssignedToWorker, ESendFlags::QueueAddAndRemove);
				// ShouldRetractionStallRatherThanDemote should not return true unless the Stall will succeed
				check(PackageData->GetState() == EPackageState::SaveStalledAssignedToWorker);
				UE_LOGF(LogCook, Display, "Retracting generated package %ls; it will remain in memory on the director's worker until the generator finishes saving.",
					*WriteToString<256>(PackageData->GetPackageName()));
			}
			else
			{
				PackageData->SendToState(EPackageState::AssignedToWorker, ESendFlags::QueueAddAndRemove, EStateChangeReason::Retraction);
			}
			PackageData->SetWorkerAssignment(Assignment);
			Director.bHasSentActions = true;
		}
	}
	Director.DisplayRemainingPackages();
	if (bAssignedToLocal)
	{
		// Clear the SoftGC diagnostic ExpectedNeverLoadPackages because we have new assigned packages
		// that we didn't consider during SoftGC
		Director.COTFS.PackageTracker->ClearExpectedNeverLoadPackages();
	}

	return ERetractionResult::Retracted;
}

TArray<FWorkerId> FCookDirector::FRetractionHandler::CalculateWorkersToSplitOver(int32 NumPackages,
	const FWorkerId& FromWorker, TConstArrayView<TRefCountPtr<FCookWorkerServer>> LocalRemoteWorkers)
{
	TArray<TPair<FWorkerId, int32>> WorkerNumPackages;
	if (FromWorker != FWorkerId::Local() && Director.bAllowLocalCooks)
	{
		WorkerNumPackages.Emplace(FWorkerId::Local(), Director.COTFS.NumMultiprocessLocalWorkerAssignments());
	}
	for (const TRefCountPtr<FCookWorkerServer>& RemoteWorker : LocalRemoteWorkers)
	{
		if (FromWorker != RemoteWorker->GetWorkerId())
		{
			WorkerNumPackages.Emplace(RemoteWorker->GetWorkerId(), RemoteWorker->NumAssignments());
		}
	}
	if (WorkerNumPackages.Num() == 0)
	{
		return TArray<FWorkerId>();
	}
	WorkerNumPackages.Sort([](const TPair<FWorkerId, int32>& A, const TPair<FWorkerId, int32>& B)
		{
			return A.Value < B.Value;
		});

	// Consider splitting the packages amonst the 1 lowest, 2 lowest, ... n lowest (not including the FromWorker)
	// Pick the value to split over based on whichever split group results in the lowest post split maximum
	// So splitting 500 over 0,1000,1000,1000 -> would give them all to the first, but splitting 500 over
	// 0, 100, 1000, 1000 would split them amongst the first two.
	int32 BestNumToSplitOver = 0;
	int32 BestPostSplitValue = 0;
	for (int32 NumToSplitOver = 1; NumToSplitOver <= WorkerNumPackages.Num(); ++NumToSplitOver)
	{
		int32 PostSplitValue = WorkerNumPackages[NumToSplitOver - 1].Value + NumPackages / NumToSplitOver;
		if (BestNumToSplitOver == 0 || PostSplitValue < BestPostSplitValue)
		{
			BestNumToSplitOver = NumToSplitOver;
			BestPostSplitValue = PostSplitValue;
		}
	}
	check(BestNumToSplitOver > 0);
	TArray<FWorkerId> Results;
	Results.Reserve(BestNumToSplitOver);
	for (const TPair<FWorkerId, int32>& Pair :
		TArrayView<TPair<FWorkerId, int32>>(WorkerNumPackages).Left(BestNumToSplitOver))
	{
		Results.Add(Pair.Key);
	}
	return Results;
}

void FRetractionRequestMessage::Write(FCbWriter& Writer) const
{
	Writer << "RequestedCount" << RequestedCount;
}
bool FRetractionRequestMessage::TryRead(FCbObjectView Object)
{
	return LoadFromCompactBinary(Object["RequestedCount"], RequestedCount);
}

FGuid FRetractionRequestMessage::MessageType(TEXT("7109E168E8A8405BA65F9E1E82571D1A"));

void FRetractionResultsMessage::Write(FCbWriter& Writer) const
{
	Writer << "ReturnedPackages" << ReturnedPackages;
}
bool FRetractionResultsMessage::TryRead(FCbObjectView Object)
{
	return LoadFromCompactBinary(Object["ReturnedPackages"], ReturnedPackages);
}

FGuid FRetractionResultsMessage::MessageType(TEXT("CBFB840A4FB94903A757C490514A4B86"));

void FDirectorEventMessage::Write(FCbWriter& Writer) const
{
	static_assert(sizeof(EDirectorEvent) <= sizeof(uint8), "We are storing it in a uint8");
	Writer << "T" << (uint8)Event;
}
bool FDirectorEventMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	uint8 EventInt = Object["T"].AsUInt8(static_cast<uint8>(EDirectorEvent::Count));
	if (EventInt < static_cast<uint8>(EDirectorEvent::Count))
	{
		Event = static_cast<EDirectorEvent>(EventInt);
	}
	else
	{
		bOk = false;
	}
	return bOk;
}

FGuid FDirectorEventMessage::MessageType(TEXT("58A03A38FEE045B08331BB8457AEBE35"));

}
