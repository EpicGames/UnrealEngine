// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookSockets.h"
#include "Cooker/CookTypes.h"
#include "Cooker/MPCollector.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Runnable.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Guid.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CookStats.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

#include <atomic>

class FCbObject;
class FCbWriter;
class FRunnableThread;
class UCookOnTheFlyServer;
namespace UE::Cook { class FCookWorkerServer; }
namespace UE::Cook { struct FAssignPackageExtraData; }
namespace UE::Cook { struct FCookWorkerProfileData; }
namespace UE::Cook { struct FDirectorEventMessage; }
namespace UE::Cook { struct FFileTransferMessage; }
namespace UE::Cook { struct FGeneratorEventMessage; }
namespace UE::Cook { struct FGenerationHelper; }
namespace UE::Cook { struct FHeartbeatMessage; }
namespace UE::Cook { struct FInitialConfigMessage; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FRetractionResultsMessage; }
namespace UE::Cook { struct FWorkerId; }

LLM_DECLARE_TAG(Cooker_MPCook);

namespace UE::Cook
{

/**
 * The categories of thread that can pump the communication with CookWorkers. It can be pumped either from
 * the cooker's scheduler thread (aka Unreal's game thread or main thread) or from a worker thread.
 */
enum class ECookDirectorThread : uint8
{
	SchedulerThread,
	CommunicateThread,
	Invalid,
};

/**
 * Timings for messages sent to CookWorker or CookDirector through the public Broadcast functions.
 * The timing indicates at which point in the network tick the message should be sent.
 */
enum class ECookBroadcastTiming : uint8
{
	Immediate,
	AfterAssignPackages,
};

/**
 * Helper for CookOnTheFlyServer that sends requests to CookWorker processes for load/save and merges
 * their replies into the local process's cook results.
 */
class FCookDirector
{
public:

	FCookDirector(UCookOnTheFlyServer& InCOTFS, int32 CookProcessCount, bool bInCookProcessCountSetByCommandLine);
	~FCookDirector();

	bool IsMultiprocessAvailable() const;
	void StartCook(const FBeginCookContext& Context);

	/**
	 * Assign the given requests out to CookWorkers (or keep on local COTFS), return the list of assignments.
	 * Input requests have been sorted by leaf to root load order.
	 */
	void AssignRequests(TArrayView<FPackageData*> Requests, TArray<FWorkerId>& OutAssignments,
		TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph);
	/** Notify the CookWorker that owns the cook of the package that the Director wants to take it back. */
	void RemoveFromWorker(FPackageData& PackageData);

	/**
	 * Send a message to all CookWorkers to match a state change on the Director.
	 * The message will be sent at the time indicated by the Timing argument; either Immediate, before any other messages
	 * from the director that are triggered after the call, or at the specified later time.
	 */
	void BroadcastMessage(const IMPCollectorMessage& Message,
		ECookBroadcastTiming Timing = ECookBroadcastTiming::Immediate);
	void BroadcastMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message,
		ECookBroadcastTiming Timing = ECookBroadcastTiming::Immediate);

	/**
	 * Increment the current heartbeat number, send a heartbeat message to all CookWorkers, and return the number in
	 * the broadcast. This number can then be queried for whether the CookWorkers have acknowledged it in a round trip
	 * message.
	 */
	int32 InsertBroadcastFence();
	/** Report whether all CookWorkers have acknowledged a previously inserted fence.. */
	bool IsBroadcastFencePassed(int32 Fence, TArray<FWorkerId>* OutPendingWorkers);

	/** Periodic tick function. Sends/Receives messages to CookWorkers. */
	void TickFromSchedulerThread();
	/** Periodic display function, called from CookOnTheFlyServer.UpdateDisplay. */
	void UpdateDisplayDiagnostics() const;
	/**
	 * Called when the COTFS Server has detected all packages are complete. Tells the CookWorkers to flush messages
	 * and exit.
	 */
	void PumpCookComplete(bool& bOutCompleted);
	/**
	 * Called when a session ends. The Director blocks on shutdown of all CookWorkers and returns state to before
	 * session started.
	 */
	void ShutdownCookSession();

	/** Enum specifying how CookWorker log output should be shown. */
	enum EShowWorker
	{
		CombinedLogs,
		SeparateLogs,
		SeparateWindows, // Implies SeparateLogs as well
	};
	EShowWorker GetShowWorkerOption() const { return ShowWorkerOption; }

	/** Register a Collector to receive messages of its MessageType from CookWorkers. */
	void Register(IMPCollector* Collector);
	/** Unegister a Collector that was registered. */
	void Unregister(IMPCollector* Collector);

	/** Data used by a CookWorkerServer to launch the remote process. */
	struct FLaunchInfo
	{
		EShowWorker ShowWorkerOption;
		FString CommandletExecutable;
		FString WorkerCommandLine;
	};
	FLaunchInfo GetLaunchInfo(FWorkerId WorkerId, int32 ProfileId);

	FString GetDisplayName(const FWorkerId& WorkerId, int32 PreferredWidth = -1) const;

	/** The message CookWorkerServer sends to the remote process once it is ready to connect. */
	const FInitialConfigMessage& GetInitialConfigMessage();

	/**
	 * This function is called from the Director's local Worker. It is for FileTransfers that have to be synchronized
	 * to avoid file write collisions, across all CookWorkers including the Director's local Worker.
	 * Messages from Remote workers instead go through HandleFileTransferMessage.
	 */
	void ReportFileTransferFromLocalWorker(FStringView TempFileName, FStringView TargetFileName);

	/**
	 * Report whether functions have been called on the director to send actions to a remote worker,
	 * e.g. AssignPackages. This flag is not true for messages about communication (e.g. InitialConfig), it is only
	 * true for messages that request or unblock work on the CookWorker and might cause a response from the worker.
	 * Used to decide whether we need to WaitOnFence during cook phase transitions.
	 */
	bool HasSentActions() const;
	void ClearHasSentActions();

private:
	enum class ELoadBalanceAlgorithm
	{
		Striped,
		CookBurden,
	};
	/** CookWorker connections that have not yet identified which CookWorker they are. */
	struct FPendingConnection
	{
		explicit FPendingConnection(FSocket* InSocket = nullptr)
		:Socket(InSocket)
		{
		}
		FPendingConnection(FPendingConnection&& Other);
		FPendingConnection(const FPendingConnection& Other) = delete;
		~FPendingConnection();

		FSocket* DetachSocket();

		FSocket* Socket = nullptr;
		UE::CompactBinaryTCP::FReceiveBuffer Buffer;
	};
	/** Struct that implements the FRunnable interface and forwards it to to named functions on this FCookDirector. */
	struct FRunnableShunt : public FRunnable
	{
		FRunnableShunt(FCookDirector& InDirector) : Director(InDirector) {}
		virtual uint32 Run() override;
		virtual void Stop() override;
		FCookDirector& Director;
	};
	class FRetractionHandler;

	struct FMachineResourceValues
	{
		uint64 MemoryMaxUsedPhysical = 0;
		uint64 MemoryMaxUsedVirtual = 0;
		uint64 MemoryMinFreeVirtual = 0;
		uint64 MemoryMinFreePhysical = 0;
		FGenericPlatformMemoryStats::EMemoryPressureStatus MemoryTriggerGCAtPressureLevel
			= FGenericPlatformMemoryStats::EMemoryPressureStatus::Unknown;
		int32 NumberOfCores = 0;
		int32 HyperThreadCount = 0;
	};

private:
	/** Helper for constructor parsing. */
	void ParseConfig(int32 CookProcessCount, bool& bOutValid);
	/** Initialization helper: create the listen socket. */
	bool TryCreateWorkerConnectSocket();
	/**
	 * Construct CookWorkerServers and communication thread if not yet constructed. The CookWorkerServers are
	 * constructed to Uninitialized; the worker process is created asynchronously during TickCommunication.
	 */
	void InitializeWorkers();
	/**
	 * Copy to snapshot variables the data required on the communication thread that can only be read from the
	 * scheduler thread.
	 */
	void ConstructReadonlyThreadVariables();
	/** Construct CookWorkerServers if necessary to replace workers that have crashed. */
	void RecreateWorkers();
	/** Reduce memory settings, cpusettings, and anything else that needs to be shared with CookWorkers. */
	void ActivateMachineResourceReduction();
	/** Restore the settings that were reduced in ActivateMachineResourceReduction. */
	void DeactivateMachineResourceReduction();
	/** Start the communication thread if not already started. */
	void LaunchCommunicationThread();
	/** Signal the communication thread to stop, wait for it to finish, and deallocate it. */
	void StopCommunicationThread();
	/** Entry point for the communication thread. */
	uint32 RunCommunicationThread();
	/**
	 * Execute a single frame of communication with CookWorkers: send/receive to all CookWorkers,
	 * including connecting, ongoing communication, and shutting down.
	 */
	void TickCommunication(ECookDirectorThread TickThread);
	/** Tick helper: tick any workers that have not yet finished initialization. */
	void TickWorkerConnects(ECookDirectorThread TickThread);
	/** Tick helper: tick any workers that are shutting down. */
	void TickWorkerShutdowns(ECookDirectorThread TickThread);
	/** The LogPath a worker process writes to. */
	FString GetWorkerLogFileName(int32 ProfileId);
	/** Get the commandline to launch a worker process with. */
	FString GetWorkerCommandLine(FWorkerId WorkerId, int32 ProfileId);
	/** Calls the configured LoadBalanceAlgorithm. Input Requests have been sorted by leaf to root load order. */
	void LoadBalance(TConstArrayView<FWorkerId> SortedWorkers, TArrayView<FPackageData*> Requests,
		TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments);
	/** Report whether it is time for a heartbeat message and update the timer data. */
	void TickHeartbeat(bool bForceHeartbeat, double CurrentTimeSeconds, bool& bOutSendHeartbeat,
		int32& OutHeartbeatNumber);
	/** Reset the IdleHeartbeatFence when new idle-breaking data comes in. */
	void ResetFinalIdleHeartbeatFence();
	/** Log the occurrence of a heartbeat message from a CookWorker. */
	void HandleHeartbeatMessage(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
		FHeartbeatMessage&& Message);
	void HandleFileTransferMessage(FCookWorkerServer& FromWorker, FFileTransferMessage&& Message);
	void MoveFileSynchronized(const TCHAR* TempFileName, const TCHAR* TargetFileName,
		FWorkerId WorkerId, uint32 WorkerProfileId);

	/** Move the given worker from active workers to the list of workers shutting down. */
	void AbortWorker(FWorkerId WorkerId, ECookDirectorThread TickThread);
	/** Send the given packages from an aborted worker back to the CookOnTheFlyServer for reassignment. */
	void ReassignAbortedPackages(TArray<FPackageData*>& PackagesToReassign);

	/**
	 * Periodically update whether (1) local server is done and (2) no results from cookworkers have come in.
	 * Send warning when it goes on too long.
	 */
	void SetWorkersStalled(bool bInWorkersStalled);
	/** Callback for CookStats system to log our stats. */
#if ENABLE_COOK_STATS
	void LogCookStats(FCookStatsManager::AddStatFuncRef AddStat);
#endif
	void AssignRequests(TArray<FWorkerId>&& InWorkers, TArray<TRefCountPtr<FCookWorkerServer>>& InRemoteWorkers, 
		TArrayView<FPackageData*> Requests, TArray<FWorkerId>& OutAssignments,
		TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, bool bInitialAssignment);

	TArray<TRefCountPtr<FCookWorkerServer>> CopyRemoteWorkers() const;
	void DisplayRemainingPackages() const;
	FString GetDisplayName(const FCookWorkerServer& RemoteWorker, int32 PreferredWidth=-1) const;
	const TRefCountPtr<FCookWorkerServer>* FindRemoteWorkerInLock(const FWorkerId& WorkerId) const;

	TMap<FPackageData*, FAssignPackageExtraData> GetAssignPackageExtraDatas(
		TConstArrayView<FPackageData*> Requests) const;
	TArray<FPackageData*> GetInfoPackagesForRequests(TConstArrayView<FPackageData*> Requests) const;

private:
	// Synchronization primitives that can be used from any thread
	mutable FCriticalSection CommunicationLock;
	FEventRef ShutdownEvent {EEventMode::ManualReset};

	// Data only accessible from the SchedulerThread
	FRunnableShunt RunnableShunt;
	FRunnableThread* CommunicationThread = nullptr;
	TArray<FCookWorkerProfileData> RemoteWorkerProfileDatas;
	TArray<FPendingConnection> PendingConnections;
	TUniquePtr<FCookWorkerProfileData> LocalWorkerProfileData;
	TArray<TPair<UE::CompactBinaryTCP::FMarshalledMessage, ECookBroadcastTiming>> QueuedBroadcasts;
	UCookOnTheFlyServer& COTFS;
	FMachineResourceValues SavedResources;
	double WorkersStalledStartTimeSeconds = 0.;
	double WorkersStalledWarnTimeSeconds = 0.;
	double LastTickTimeSeconds = 0.;
	double NextHeartbeatTimeSeconds = 0.;
	int32 HeartbeatNumber = 0;
	int32 FinalIdleHeartbeatFence = -1;
	bool bWorkersInitialized = false;
	bool bHasReducedMachineResources = false;
	bool bIsFirstAssignment = true;
	bool bCookCompleteSent = false;
	bool bWorkersStalled = false;
	bool bMultiprocessAvailable = false;
	bool bReceivingMessages = false;
	bool bForceNextHeartbeat = false;
	bool bCookProcessCountSetByCommandLine = false;
	bool bHasSentActions = false;

	// Data that is read-only while the CommunicationThread is active and is readable from any thread
	FBeginCookContextForWorker BeginCookContext;
	TMap<FGuid, TRefCountPtr<IMPCollector>> Collectors;
	TUniquePtr<FInitialConfigMessage> InitialConfigMessage;
	FString WorkerConnectAuthority;
	FString CommandletExecutablePath;
	int32 RequestedCookWorkerCount = 0;
	int32 WorkerConnectPort = 0;
	int32 CoreLimit = 0;
	EShowWorker ShowWorkerOption = EShowWorker::CombinedLogs;
	ELoadBalanceAlgorithm LoadBalanceAlgorithm = ELoadBalanceAlgorithm::CookBurden;
	/** Whether the director is allowed to cook any packages. True by default, false by commandline parameter. */
	bool bAllowLocalCooks = true;

	// Data only accessible from the CommunicationThread (or if the CommunicationThread is inactive)
	FSocket* WorkerConnectSocket = nullptr;

	// Data shared between SchedulerThread and CommunicationThread that can only be accessed inside CommunicationLock
	TMap<int32, TRefCountPtr<FCookWorkerServer>> RemoteWorkers;
	TMap<FCookWorkerServer*, TRefCountPtr<FCookWorkerServer>> ShuttingDownWorkers;
	TArray<FPackageData*> DeferredPackagesToReassign;
	TUniquePtr<FRetractionHandler> RetractionHandler;
	bool bWorkersActive = false;

	// FileTransfers (they have their own critical section to synchronize IO from multiple CookWorkers)
	mutable FCriticalSection FileTransferLock;

	friend class UE::Cook::FCookWorkerServer;
};

/** Parameters parsed from commandline for how a CookWorker connects to the CooKDirector. */
struct FDirectorConnectionInfo
{
	bool TryParseCommandLine();

	FString HostURI;
	int32 RemoteIndex = 0;
};

/** Message sent from a CookWorker to the Director to report that it is ready for setup messages and cooking. */
struct FWorkerConnectMessage : public IMPCollectorMessage
{
public:
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("WorkerConnectMessage"); }

public:
	int32 RemoteIndex = 0;
	static FGuid MessageType;
};

/**
 * Message sent from CookDirector to a CookWorker to cancel some of its assigned packages and return them
 * dispatch to idle workers.
 */
struct FRetractionRequestMessage : public IMPCollectorMessage
{
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("RetractionRequestMessage"); }

public:
	int32 RequestedCount = 0;
	static FGuid MessageType;
};

/**
 * Message sent from CookWorker to CookDirector identifying which assigned packages it chose to satisfy a
 * a RetractionRequest.
 */
struct FRetractionResultsMessage : public IMPCollectorMessage
{
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("RetractionResultsMessage"); }

public:
	TArray<FName> ReturnedPackages;
	static FGuid MessageType;
};

/** Director status Events that can be broadcast from CookDirector to CookWorkers. */
enum class EDirectorEvent : uint8
{
	KickBuildDependencies,
	Count,
};
/** Message sent from CookDirector to CookWorkers to notify them of an EDirectorEvent. */
struct FDirectorEventMessage : public IMPCollectorMessage
{
	explicit FDirectorEventMessage(EDirectorEvent InEvent = EDirectorEvent::Count)
		: Event(InEvent)
	{
	}

	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override
	{
		return MessageType;
	}
	virtual const TCHAR* GetDebugName() const override
	{
		return TEXT("DirectorEventMessage");
	}

public:
	EDirectorEvent Event;
	static FGuid MessageType;
};

} // namespace UE::Cook