// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookFunctionLibrary.h"

#include "Commandlets/Commandlet.h"
#include "Cooker/AsyncIODelete.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookPackageArtifacts.h"
#include "Cooker/CookSandbox.h"
#include "Cooker/LooseCookedPackageWriter.h"
#include "CookOnTheSide/CookLog.h"
#include "Editor/EditorEngine.h"
#include "HAL/FileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "LooseFilesCookArtifactReader.h"
#include "Serialization/BasePackageWriter.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/FieldPath.h"
#include "UObject/SavePackage.h"
#include "UObject/UnrealTypePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookFunctionLibrary)

extern UNREALED_API UEditorEngine* GEditor;

namespace UE::Private
{
	void SaveDepsToFile(const UE::Cook::FPackageArtifacts& Artifacts, const FString& Filename)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << "CookTestSnapshot" << Artifacts;
		Writer.EndObject();
		TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateDebugFileWriter(*Filename));
		Writer.Save(*FileArchive);
	}

	bool LoadDepsFromFile(UE::Cook::FPackageArtifacts& Artifacts, const FString& Filename)
	{
		TArray<uint8> Data;
		{
			TUniquePtr<FArchive> FileArchive(IFileManager::Get().CreateFileReader(*Filename));
			Data.SetNum(FileArchive->TotalSize());
			FileArchive->Serialize(Data.GetData(), Data.Num());
		}

		FSharedBuffer SharedBuffer = FSharedBuffer::MakeView(MakeMemoryView(Data)); // strange boilerplate, i don't understand it
		FCbObject CbObject(SharedBuffer);
		FCbField TestSnapshot = CbObject["CookTestSnapshot"];
		return LoadFromCompactBinary(TestSnapshot.AsObject(), Artifacts);
	}

	FString GetDepsFilename(const UObject* Object, const FString& DestinationSubfolder)
	{
		return FPaths::ConvertRelativePathToFull(
			FPaths::ProjectSavedDir() +
			FString(TEXT("Temp/")) + DestinationSubfolder + Object->GetPackage()->GetName() + TEXT(".cookdeps"));
	}
}

void UCookFunctionLibrary::CookAsset(UObject* Object, const FString& ForPlatform, const FString& DestinationSubfolder, const FString& CookCommandlineArgs)
{
	using namespace UE::Cook;
	if (Object == nullptr)
	{
		UE_LOGF(LogCook, Warning, "CookAsset expected an object to cook, but received nullptr");
		return;
	}

	ITargetPlatformManagerModule* PMM = GetTargetPlatformManager();
	if (PMM == nullptr)
	{
		return;
	}

	UPackage* Package = Cast<UPackage>(Object);
	if (!Package)
	{
		Package = Object->GetPackage();
	}

	if(Package == GetTransientPackage())
	{
		UE_LOGF(LogCook, Warning, 
			"CookAsset cannot cook the transient package: %ls", *Object->GetPathName());
		return;
	}

	const ITargetPlatform* TargetPlatform = PMM->FindTargetPlatform(*ForPlatform);
	if (!TargetPlatform)
	{
		TargetPlatform = PMM->GetRunningTargetPlatform();
		if (!TargetPlatform)
		{
			UE_LOGF(LogCook, Warning, 
				"Could not find any platform to cook for when requested to cook %ls", 
				*ForPlatform);
			return;
		}
		else
		{
			UE_LOGF(LogCook, Warning, 
				"Could not find requested platform %ls, fell back to %ls!", 
				*ForPlatform, *TargetPlatform->IniPlatformName());
		}
	}

	bool bUnversioned = false;
	if(!CookCommandlineArgs.IsEmpty())
	{
		TArray<FString> Tokens;
		TArray<FString> Switches;
		UCommandlet::ParseCommandLine(*CookCommandlineArgs, Tokens, Switches);
		if(Tokens.Num() > 0)
		{
			UE_LOGF(LogCook, Warning, 
				"CookAsset does not expect tokens - they have been discarded: %ls", 
				*FString::Join(Tokens, TEXT(" ")));
		}
		if(Switches.Num() > 0)
		{
			bUnversioned = (Switches.RemoveSingle(TEXT("UNVERSIONED")) == 1); 
			if(Switches.Num() > 0)
			{
				UE_LOGF(LogCook, Warning, 
					"CookAsset found switches it does not yet support - they have been discarded: %ls", 
					*FString::Join(Switches, TEXT(" ")));
			}
		}
	}

	const FString OutputName = FPaths::ConvertRelativePathToFull(
		FPaths::ProjectSavedDir() +
		FString(TEXT("Temp/")) + DestinationSubfolder + Package->GetName() + TEXT(".uasset"));

	// TODO: Create an ICookInfo provider that does not require UCookOnTheFlyServer and pass it in so we can collect
	// manual BuildDependencies.
	ICookInfo* CookInfo = nullptr; // TODO: NeedSimpleCookInfoClass
	FArchiveCookContext ArchiveCookContext(Package,
		UE::Cook::ECookType::ByTheBook,
		UE::Cook::ECookingDLC::No, // used only by UMaterialInterface::Serialize, we could expose via command line arguments
		TargetPlatform, CookInfo);
	FArchiveCookData CookData(*TargetPlatform, ArchiveCookContext);

	class FCookerInterface : public UE::PackageWriter::Private::ICookerInterface
	{
	public:
		virtual EPackageWriterResult CookerBeginCacheForCookedPlatformData(
			UE::PackageWriter::Private::FBeginCacheForCookedPlatformDataInfo& Info) override
		{
			return EPackageWriterResult::Success; // saving will fail if we don't say good things happened
		}
		virtual void RegisterDeterminismHelper(ICookedPackageWriter* PackageWriter, UObject* SourceObject,
			const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) override
		{
		}
		virtual bool IsDeterminismDebug() const override
		{
			return false;
		}
		virtual void WriteFileOnCookDirector(const UE::PackageWriter::Private::FWriteFileData& FileData,
			FMD5& AccumulatedHash, const TRefCountPtr<FPackageHashes>& PackageHashes,
			IPackageWriter::EWriteOptions WriteOptions) override
		{
			UE::PackageWriter::Private::HashAndWrite(FileData, AccumulatedHash, PackageHashes, WriteOptions);
		}
		virtual UE::Cook::ICookSandbox* GetCookSandbox()
		{
			return nullptr;
		}
	};
	FCookerInterface CookerInterface;

	// The AsyncIODelete object optimizes the cleaning of the saved/cook directory. Happily not relevant
	// for callers of this function, but an important optimization for the cook commandlet.
	TUniquePtr<FAsyncIODelete> AsyncIODelete = MakeUnique<FAsyncIODelete>(); 

	TArray<TSharedRef<IPlugin> > PluginsToRemap; // none?
	TUniquePtr<UE::Cook::FCookSandbox> SandboxFileObj = MakeUnique<UE::Cook::FCookSandbox>(OutputName, PluginsToRemap);
	FLooseCookedPackageWriter* LooseWriter = new FLooseCookedPackageWriter(
		OutputName, 
		OutputName, 
		TargetPlatform, 
		*AsyncIODelete, 
		*SandboxFileObj,
		MakeShared<FLooseFilesCookArtifactReader>());
	LooseWriter->SetCooker(&CookerInterface);

	// ~FSavePackageContext will delete the loose writer, hence naked new above:
	FSavePackageContext SaveContext(TargetPlatform, LooseWriter); 
	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Public;
	SaveArgs.bForceByteSwapping = (!TargetPlatform->IsLittleEndian()) ^ (!PLATFORM_LITTLE_ENDIAN);;
	SaveArgs.bWarnOfLongFilename = false;
	SaveArgs.SaveFlags = SAVE_AllowTimeout;
	SaveArgs.ArchiveCookData = &CookData;
	SaveArgs.bSlowTask = true;
	SaveArgs.SavePackageContext = &SaveContext;
	if(bUnversioned)
	{
		SaveArgs.SaveFlags |= SAVE_Unversioned;
	}

	TArray<TPair<ELogVerbosity::Type, FString>> Messages;
	FBuildResultDependenciesMap LoadDependencies = FBuildDependencySet::CollectLoadedPackage(Package, &Messages);

	// TODO: Execute the asynchronous transformation steps that packages go through after loading and before saving
	// BeginCacheForCookedPlatformData(); // Call once on each object
	// IsCachedCookedPlatformDataLoaded(); // Call on each object until it returns true

	// We'll need to support this if we want to support cook diffing or incrementalvalidate, 
	// but for now it's not needed:
	// SaveContext.PackageWriter->UpdateSaveArguments(SaveArgs);
	
	// The platform determines whether it wants editor only data, but lets restore the package
	// flag at the end of this function:
	const bool bWasFilterEditorOnly = Package->HasAllPackagesFlags(PKG_FilterEditorOnly);
	if(!TargetPlatform->HasEditorOnlyData())
	{
		Package->SetPackageFlags(PKG_FilterEditorOnly); 
	}
	
	ICookedPackageWriter::FBeginPackageInfo Info;
	Info.PackageName = Package->GetFName();
	Info.LooseFilePath = OutputName;
	LooseWriter->BeginPackage(Info);

	FSavePackageResultStruct Result = GEditor->Save(Package, Package->FindAssetInPackage(), *OutputName, SaveArgs);
	if (!Result.IsSuccessful())
	{
		UE_LOGF(LogCook, Warning, "Saving failed - asset not cooked!");
	}
	else
	{
		ICookedPackageWriter::FCommitPackageInfo CommitInfo;
		CommitInfo.Status = IPackageWriter::ECommitStatus::Success;
		CommitInfo.PackageName = Package->GetFName();
		CommitInfo.WriteOptions = IPackageWriter::EWriteOptions::Write;
		LooseWriter->CommitPackage(MoveTemp(CommitInfo)); // please clap

		// FGenerationHelper is used to support the creation of generated 
		// streamingcells from WorldPartition. Supporting streaming cells would require 
		// several other changes to this function, and is not the priority of this 
		// routine at this time.
		UE::Cook::FGenerationHelper* GenerationHelper = nullptr;
		bool bGenerated = false;
		Messages.Reset();

		// TODO: Call OnCookEvent(ECookEvent::PlatformLoadDependencies) on every object in the package, as is done in
		// FSaveCookedPackageContext::CalculateCookDependencies.
		// Doing this requires implementing the NeedSimpleCookInfoClass TODO first.
		FBuildResultDependenciesMap& BuildResultDependencies = Result.BuildResultDependencies;
		BuildResultDependencies.Append(LoadDependencies);
		// TODO: Populate RuntimeDependencies in the same way they are calculated in
		// FSaveCookedPackageContext::CalculateCookDependencies
		TArray<FName> RuntimeDependencies;

		FPackageArtifacts Deps = FPackageArtifacts::Collect(Package, TargetPlatform,
			MoveTemp(BuildResultDependencies), true /* bHasSaveResult */, Result.UntrackedSoftPackageReferences,
			GenerationHelper, bGenerated, MoveTemp(RuntimeDependencies), &Messages);
		if (!Deps.IsValid())
		{
			TStringBuilder<256> ErrorMessage;
			ErrorMessage << TEXT("Error collecting cook dependencies:");
			for (TPair<ELogVerbosity::Type, FString>& MessagePair : Messages)
			{
				ErrorMessage << TEXT("\n\t") << MessagePair.Value;
			}
			UE_LOGF(LogCook, Warning, "%ls", *ErrorMessage);
		}

		// persist cook dependencies along side the loose files we wrote above..
		const FString DepsFilename = UE::Private::GetDepsFilename(Object, DestinationSubfolder + "_" + ForPlatform);
		UE::Private::SaveDepsToFile(Deps, DepsFilename);

		UPackage::WaitForAsyncFileWrites();
	}

	if (bWasFilterEditorOnly)
	{
		Package->SetPackageFlags(PKG_FilterEditorOnly);
	}
	else
	{
		Package->ClearPackageFlags(PKG_FilterEditorOnly);
	}
}

