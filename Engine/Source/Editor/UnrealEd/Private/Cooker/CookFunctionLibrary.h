// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CookFunctionLibrary.generated.h"

UCLASS(transient)
class UCookFunctionLibrary : public UObject
{
	GENERATED_BODY()
public:
	/** 
	 * Writes the cooked version of the provided object's package into the Saved folder, in the subfolder
	 * defined by DestinationSubfolder. Extra arguments (such as -unversioned) can be provided by
	 * CookCommandlineArgs. This function is experimental and may not exactly match the behavior of
	 * the cook commandlet.
	 */
	UFUNCTION(BlueprintCallable, Category = "Development")
	static UNREALED_API void CookAsset(UObject* Object, const FString& ForPlatform, const FString& DestinationSubfolder, const FString& CookCommandlineArgs = TEXT("") );
};
