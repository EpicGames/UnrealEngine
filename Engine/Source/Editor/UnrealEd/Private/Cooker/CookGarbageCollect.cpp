// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookGarbageCollect.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPackagePreloader.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookTypes.h"
#include "Cooker/PackageTracker.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreGlobals.h"
#include "Engine/Engine.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/MemoryMisc.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Templates/RefCounting.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionHistory.h"
#include "UObject/Package.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include <atomic>

namespace UE::Cook
{

FScopeFindCookReferences::FScopeFindCookReferences(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
	, SoftGCGuard(UPackage::bSupportCookerSoftGC, true)
	, bNeedsConstructBuffer(COTFS.SoftGCPackageToObjectListBuffer.IsEmpty())
{
	if (bNeedsConstructBuffer)
	{
		ConstructSoftGCPackageToObjectList(COTFS.SoftGCPackageToObjectListBuffer);
	}
}

FScopeFindCookReferences::~FScopeFindCookReferences()
{
	if (bNeedsConstructBuffer)
	{
		UPackage::SoftGCPackageToObjectList.Empty();
		COTFS.SoftGCPackageToObjectListBuffer.Empty();
	}
}

FCookGCDiagnosticContext::~FCookGCDiagnosticContext()
{
	SetGCWithHistoryRequested(false);
}

bool FCookGCDiagnosticContext::NeedsDiagnosticSecondGC() const
{
	return bRequestGCWithHistory || bRequestFullGC;
}

bool FCookGCDiagnosticContext::CurrentGCHasHistory() const
{
	return bCurrentGCHasHistory;
}

bool FCookGCDiagnosticContext::TryRequestGCWithHistory()
{
#if ENABLE_GC_HISTORY
	if (!bRequestsAvailable || !bGCInProgress || bCurrentGCHasHistory)
	{
		return false;
	}
	SetGCWithHistoryRequested(true);
	return true;
#else
	return false;
#endif
}

bool FCookGCDiagnosticContext::TryRequestFullGC()
{
	if (!bRequestsAvailable || !bGCInProgress || bCurrentGCIsFull)
	{
		return false;
	}
	bRequestFullGC = true;
	return true;
}

void FCookGCDiagnosticContext::OnCookerStartCollectGarbage(UCookOnTheFlyServer& COTFS, uint32& ResultFlagsFromTick)
{
	bRequestsAvailable = true;

	bGCInProgress = true;
#if ENABLE_GC_HISTORY
	bCurrentGCHasHistory = FGCHistory::Get().GetHistorySize() > 0;
#else
	bCurrentGCHasHistory = false;
#endif
	if (bRequestFullGC)
	{
		COTFS.bGarbageCollectTypeSoft = false;
		ResultFlagsFromTick = ResultFlagsFromTick & ~UCookOnTheFlyServer::COSR_RequiresGC_Soft;
	}
	bCurrentGCIsFull = !COTFS.bGarbageCollectTypeSoft;
}

void FCookGCDiagnosticContext::OnCookerEndCollectGarbage(UCookOnTheFlyServer& COTFS, uint32& ResultFlagsFromTick)
{
	bGCInProgress = false;
	bCurrentGCHasHistory = false;
	bCurrentGCIsFull = false;
}

void FCookGCDiagnosticContext::OnEvaluateResultsComplete()
{
	SetGCWithHistoryRequested(false);
	bRequestFullGC = false;
}

void FCookGCDiagnosticContext::SetGCWithHistoryRequested(bool bValue)
{
#if ENABLE_GC_HISTORY
	if (bValue == bRequestGCWithHistory)
	{
		return;
	}

	if (bValue)
	{
		SavedGCHistorySize = FGCHistory::Get().GetHistorySize();
		if (SavedGCHistorySize < 1)
		{
			FGCHistory::Get().SetHistorySize(1);
		}
	}
	else
	{
		if (SavedGCHistorySize != FGCHistory::Get().GetHistorySize())
		{
			FGCHistory::Get().SetHistorySize(SavedGCHistorySize);
		}
		SavedGCHistorySize = 0;
	}
	bRequestGCWithHistory = bValue;
#endif
}

void FSoftGCHistory::AddDurationMeasurement(float DurationSeconds)
{
	int32 HistoryLength = DurationHistory.Num();
	while (HistoryLength >= MaxHistoryLength)
	{
		if (HistoryLength <= 1)
		{
			if (HistoryLength > 0)
			{
				DurationHistory.PopFront();
			}
			AverageDurationSeconds = 0.0;
			HistoryLength = 0;
		}
		else
		{
			AverageDurationSeconds = (AverageDurationSeconds * HistoryLength - DurationHistory.PopFrontValue()) / (HistoryLength - 1);
			--HistoryLength;
		}
	}
	if (HistoryLength < MaxHistoryLength)
	{
		if (HistoryLength == 0)
		{
			AverageDurationSeconds = DurationSeconds;
		}
		else
		{
			AverageDurationSeconds = (AverageDurationSeconds * HistoryLength + DurationSeconds) / (HistoryLength + 1);
		}
		DurationHistory.Add(DurationSeconds);
	}
}

bool FSoftGCHistory::IsTriggeringWithinBudget(UCookOnTheFlyServer& COTFS, double CurrentTimeSeconds, FString* OutDiagnostics)
{
	if (OutDiagnostics)
	{
		OutDiagnostics->Reset();
	}
	if (COTFS.CookedPackageCountSinceLastGC == 0)
	{
		return false;
	}
	float TimeSinceLastGCSeconds = CurrentTimeSeconds - COTFS.LastSoftGCTime;
	if (TimeSinceLastGCSeconds < COTFS.SoftGCMinimumPeriodSeconds)
	{
		// Don't allow triggering SoftGC too frequently, even if it is within budget. This prevents spam
		// from log messages that get printed every time garbage is collected.
		return false;
	}

	if (DurationHistory.Num() == 0)
	{
		if (OutDiagnostics)
		{
			*OutDiagnostics = TEXT("No duration data");
		}
		return true;
	}
	// TimeBudget/(Time + TimeBudget) == BudgetFraction
	// TimeBudget == (BudgetFraction/(1 - BudgetFraction))*Time
	if (COTFS.SoftGCTimeFractionBudget > .99f)
	{
		if (OutDiagnostics)
		{
			*OutDiagnostics = FString::Printf(
				TEXT("SoftGCTimeFractionBudget == %.3f, above threshold to always trigger"),
				COTFS.SoftGCTimeFractionBudget);
		}
		return true;
	}
	float CurrentTimeBudget = TimeSinceLastGCSeconds *
		COTFS.SoftGCTimeFractionBudget / (1 - COTFS.SoftGCTimeFractionBudget);
	if (CurrentTimeBudget >= AverageDurationSeconds)
	{
		if (OutDiagnostics)
		{
			*OutDiagnostics = FString::Printf(
				TEXT("SoftGCTimeFractionBudget == %.3f. TimeSinceLastGCSeconds == %.3f. CurrentTimeBudget == %.3f. ExpectedDuration == %.3f"),
				COTFS.SoftGCTimeFractionBudget, TimeSinceLastGCSeconds, CurrentTimeBudget, AverageDurationSeconds);
		}
		return true;
	}
	return false;
}

void ConstructSoftGCPackageToObjectList(TArray<UObject*>& PackageToObjectListBuffer)
{
	struct FPackageObjectPair
	{
		UPackage* Package;
		UObject* Object;
		bool operator<(const FPackageObjectPair& Other) const
		{
			if (Package != Other.Package)
				return Package < Other.Package;
			return Object < Other.Object;
		}
		bool operator==(const FPackageObjectPair& Other) const
		{
			return Object == Other.Object;
		}
	};

	PackageToObjectListBuffer.Empty();
	UPackage::SoftGCPackageToObjectList.Empty();

	// Iterate over all UObjects in memory (in parallel) and for each valid public object, get its package and add a FPackageObjectPair for it
	int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNum();
	int32 NumThreads = FMath::Clamp(FTaskGraphInterface::Get().GetNumWorkerThreads(), 1, MaxNumberOfObjects);
	int32 NumberOfObjectsPerThread = (MaxNumberOfObjects + NumThreads - 1) / NumThreads; // ceiling
	check(NumberOfObjectsPerThread * (NumThreads - 1) <= MaxNumberOfObjects);

	TArray<TArray<FPackageObjectPair>> ThreadContexts;
	ThreadContexts.SetNum(NumThreads);
	std::atomic<int32> PackagesNum{ 0 };

	ParallelFor(TEXT("ConstructSoftGCPackageToObjectList"), NumThreads, 1,
		[&ThreadContexts, &PackagesNum, NumberOfObjectsPerThread, NumThreads, MaxNumberOfObjects](int32 ThreadIndex)
		{
			TArray<FPackageObjectPair>& ThreadContext = ThreadContexts[ThreadIndex];
			int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
			int32 NumObjects = (ThreadIndex < (NumThreads - 1)) ? NumberOfObjectsPerThread : (MaxNumberOfObjects - (NumThreads - 1) * NumberOfObjectsPerThread);
			check(FirstObjectIndex + NumObjects <= MaxNumberOfObjects);

			for (int32 ObjectIndex = 0; ObjectIndex < NumObjects && (FirstObjectIndex + ObjectIndex) < MaxNumberOfObjects; ++ObjectIndex)
			{
				FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[FirstObjectIndex + ObjectIndex];
				if (!ObjectItem.GetObject())
				{
					continue;
				}
				if (ObjectItem.IsGarbage())
				{
					continue;
				}
				UObject* Object = static_cast<UObject*>(ObjectItem.GetObject());
				if (!Object->HasAnyFlags(RF_Public))
				{
					continue;
				}
				UPackage* Package = Object->GetPackage();
				if (!Package)
				{
					continue;
				}
				if (Package->HasAnyFlags(RF_Transient) || Package->HasAnyPackageFlags(PKG_CompiledIn))
				{
					// Skip any Transient packages (e.g. /Engine/Transient) and script packages
					// We only need to keep public objects alive in packages that could be saved.
					continue;
				}
				if (Object == Package)
				{
					PackagesNum.fetch_add(1, std::memory_order_relaxed);
				}
				ThreadContext.Add(FPackageObjectPair{ Package, Object });
			}
		});

	// Accumulate results from the parallel threads into a single array
	TArray<FPackageObjectPair> PackageObjectPairs = MoveTemp(ThreadContexts[0]);
	int32 PackageObjectPairsNum = PackageObjectPairs.Num();
	TArrayView<TArray<FPackageObjectPair>> RemainingThreadContexts = TArrayView<TArray<FPackageObjectPair>>(ThreadContexts).RightChop(1);
	for (TArray<FPackageObjectPair>& ThreadContext : RemainingThreadContexts)
	{
		PackageObjectPairsNum += ThreadContext.Num();
	}
	PackageObjectPairs.Reserve(PackageObjectPairsNum);
	for (TArray<FPackageObjectPair>& ThreadContext : RemainingThreadContexts)
	{
		PackageObjectPairs.Append(ThreadContext);
	}
	ThreadContexts.Empty();

	// Sort the array so that all objects for each package are together
	PackageObjectPairs.Sort();

	// Pull the UObject* out of the array of Pairs into a separate array of just UObject*,
	// and for each UPackage, add the ArrayView of UObjects matching that package into the UPackage::SoftGCPackageToObjectList.
	PackageToObjectListBuffer.SetNum(PackageObjectPairsNum);
	UObject** PackageToObjectListBufferPtr = PackageToObjectListBuffer.GetData();

	UPackage::SoftGCPackageToObjectList.Reserve(PackagesNum);
	int32 PreviousPackageStartIndex = 0;
	UPackage* PreviousPackage = nullptr;
	for (int32 Index = 0; Index < PackageObjectPairsNum; ++Index)
	{
		FPackageObjectPair& Pair = PackageObjectPairs[Index];
		if (Pair.Package != PreviousPackage)
		{
			if (Index > PreviousPackageStartIndex)
			{
				UPackage::SoftGCPackageToObjectList.Add(PreviousPackage,
					ObjectPtrWrap(TArrayView<UObject*>(PackageToObjectListBufferPtr + PreviousPackageStartIndex, Index - PreviousPackageStartIndex)));
			}
			PreviousPackage = Pair.Package;
			PreviousPackageStartIndex = Index;
		}
		PackageToObjectListBufferPtr[Index] = Pair.Object;
	}
	if (PackageObjectPairsNum > PreviousPackageStartIndex)
	{
		UPackage::SoftGCPackageToObjectList.Add(PreviousPackage,
			ObjectPtrWrap(TArrayView<UObject*>(PackageToObjectListBufferPtr + PreviousPackageStartIndex, PackageObjectPairsNum - PreviousPackageStartIndex)));
	}
}

} // namespace UE::Cook


void UCookOnTheFlyServer::PollGarbageCollection(UE::Cook::FTickStackData& StackData)
{
	NumObjectsHistory.AddInstance(GUObjectArray.GetObjectArrayNumMinusAvailable());
	NumPackagesHistory.AddInstance(PackageTracker->NumLoadedPackages());
	VirtualMemoryHistory.AddInstance(FPlatformMemory::GetStats().UsedVirtual);

	if (IsCookFlagSet(ECookInitializationFlags::TestCook))
	{
		StackData.ResultFlags |= COSR_RequiresGC | COSR_YieldTick;
		return;
	}
	if (PackagesPerGC > 0 && CookedPackageCountSinceLastGC > PackagesPerGC)
	{
		// if we are waiting on things to cache then ignore the PackagesPerGC
		if (!bSaveBusy)
		{
			StackData.ResultFlags |= COSR_RequiresGC | COSR_RequiresGC_PackageCount | COSR_YieldTick;
			return;
		}
	}
	if (IsCookOnTheFlyMode())
	{
		double CurrentTime = FPlatformTime::Seconds();
		if (IdleStatus == EIdleStatus::Done &&
			CurrentTime - IdleStatusStartTime > GetIdleTimeToGC() &&
			IdleStatusStartTime > GetLastGCTime())
		{
			StackData.ResultFlags |= COSR_RequiresGC | COSR_RequiresGC_Periodic | COSR_YieldTick;
			return;
		}
	}
}

bool UCookOnTheFlyServer::PumpHasExceededMaxMemory(uint32& OutResultFlags)
{
	if (GUObjectArray.GetObjectArrayEstimatedAvailable() < MinFreeUObjectIndicesBeforeGC)
	{
		UE_LOGF(LogCook, Display, "Running out of available UObject indices (%d remaining)", GUObjectArray.GetObjectArrayEstimatedAvailable());
		static bool bPerformedObjListWhenNearMaxObjects = false;
		if (GEngine && !bPerformedObjListWhenNearMaxObjects)
		{
			UE_LOGF(LogCook, Display, "Performing 'obj list' to show counts of types of objects due to low availability of UObject indices.");
			GEngine->Exec(nullptr, TEXT("OBJ LIST -COUNTSORT -SKIPMEMORYSIZE"));
			bPerformedObjListWhenNearMaxObjects = true;
		}
		OutResultFlags |= COSR_RequiresGC | COSR_RequiresGC_OOM | COSR_YieldTick;
		return true;
	}

	TStringBuilder<256> TriggerMessages;
	const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();
	
	bool bMinFreeTriggered = false;
	if (MemoryMinFreeVirtual > 0 || MemoryMinFreePhysical > 0)
	{
		// trigger GC if we have less than MemoryMinFreeVirtual OR MemoryMinFreePhysical
		// the check done in AssetCompilingManager is against the min of the two :
		//uint64 AvailableMemory = FMath::Min(MemStats.AvailablePhysical, MemStats.AvailableVirtual);
		// so for consistency the same check should be done here
		// you can get that by setting the MemoryMinFreeVirtual and MemoryMinFreePhysical config to be the same

		// AvailableVirtual is actually ullAvailPageFile (commit charge available)
		if (MemoryMinFreeVirtual > 0 && MemStats.AvailableVirtual < MemoryMinFreeVirtual)
		{
			TriggerMessages.Appendf(TEXT("\n  CookSettings.MemoryMinFreeVirtual: Available virtual memory %dMiB is less than %dMiB."),
				static_cast<uint32>(MemStats.AvailableVirtual / 1024 / 1024), static_cast<uint32>(MemoryMinFreeVirtual / 1024 / 1024));
			bMinFreeTriggered = true;
		}
		if (MemoryMinFreePhysical > 0 && MemStats.AvailablePhysical < MemoryMinFreePhysical)
		{
			TriggerMessages.Appendf(TEXT("\n  CookSettings.MemoryMinFreePhysical: Available physical memory %dMiB is less than %dMiB."),
				static_cast<uint32>(MemStats.AvailablePhysical / 1024 / 1024), static_cast<uint32>(MemoryMinFreePhysical / 1024 / 1024));
			bMinFreeTriggered = true;
		}
	}

	// if MemoryMaxUsed is set, we won't GC until at least that much mem is used
	// this can be useful if you demand that amount of memory as your min spec
	bool bMaxUsedTriggered = false;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (MemoryMaxUsedVirtual > 0 || MemoryMaxUsedPhysical > 0)
	{
		// check validity of trigger :
		// if the MaxUsed config exceeds the system memory, it can never be triggered and will prevent any GC :
		uint64 MaxMaxUsed = FMath::Max(MemoryMaxUsedVirtual,MemoryMaxUsedPhysical);
		if (MaxMaxUsed >= MemStats.TotalPhysical)
		{
			UE_CALL_ONCE([&]() {
				UE_LOGF(LogCook, Warning, "Warning MemoryMaxUsed condition is larger than total memory (%dMiB >= %dMiB).  System does not have enough memory to cook this project.",
				static_cast<uint32>(MaxMaxUsed / 1024 / 1024), static_cast<uint32>(MemStats.TotalPhysical / 1024 / 1024));
				});
		}

		if (MemoryMaxUsedVirtual > 0 && MemStats.UsedVirtual >= MemoryMaxUsedVirtual)
		{
			TriggerMessages.Appendf(TEXT("\n  CookSettings.MemoryMaxUsedVirtual: Used virtual memory %dMiB is greater than %dMiB."),
				static_cast<uint32>(MemStats.UsedVirtual / 1024 / 1024), static_cast<uint32>(MemoryMaxUsedVirtual / 1024 / 1024));
			bMaxUsedTriggered = true;
		}
		if (MemoryMaxUsedPhysical > 0 && MemStats.UsedPhysical >= MemoryMaxUsedPhysical)
		{
			TriggerMessages.Appendf(TEXT("\n  CookSettings.MemoryMaxUsedPhysical: Used physical memory %dMiB is greater than %dMiB."),
				static_cast<uint32>(MemStats.UsedPhysical / 1024 / 1024), static_cast<uint32>(MemoryMaxUsedPhysical / 1024 / 1024));
			bMaxUsedTriggered = true;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool bPeriodicTriggered = false;
	bool bPressureTriggered = false;
	if (MemoryTriggerGCAtPressureLevel != FPlatformMemoryStats::EMemoryPressureStatus::Unknown)
	{
		FPlatformMemoryStats::EMemoryPressureStatus PressureStatus = MemStats.GetMemoryPressureStatus();
		if (PressureStatus == FPlatformMemoryStats::EMemoryPressureStatus::Unknown)
		{
			UE_CALL_ONCE([&]() {
				UE_LOGF(LogCook, Warning,
				"MemoryPressureStatus is not available from the operating system. We may run out of memory due to lack of knowledge of when to collect garbage.");
				});
		}
		else
		{
			static_assert(FPlatformMemoryStats::EMemoryPressureStatus::Critical > FPlatformMemoryStats::EMemoryPressureStatus::Nominal,
				"We expect higher pressure to be higher integer values");
			int RequiredValue = static_cast<int>(MemoryTriggerGCAtPressureLevel);
			int CurrentValue = static_cast<int>(PressureStatus);
			if (CurrentValue >= RequiredValue)
			{
				bPressureTriggered = true;
				TriggerMessages.Appendf(TEXT("\n  Operating system has signalled that memory pressure is high."));
			}
		}
	}

	bool bTriggerGC = false;
	if (bMinFreeTriggered || bMaxUsedTriggered)
	{
		const bool bOnlyTriggerIfBothMinFreeAndMaxUsedTrigger = true;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!bOnlyTriggerIfBothMinFreeAndMaxUsedTrigger ||
			((bMinFreeTriggered || (MemoryMinFreeVirtual <= 0 && MemoryMinFreePhysical <= 0)) &&
				(bMaxUsedTriggered || (MemoryMaxUsedVirtual <= 0 && MemoryMaxUsedPhysical <= 0))))
		{
			bTriggerGC = true;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	if (bPressureTriggered)
	{
		bTriggerGC = true;
	}

	// If a normal GC was not triggered, check the SoftGC trigger conditions
	double CurrentTime = FPlatformTime::Seconds();
	bool bIsSoftGC = false;
	if (bUseSoftGC && IsDirectorCookByTheBook() && !IsCookingInEditor())
	{
		if (!bTriggerGC && SoftGCStartNumerator > 0)
		{
			if (SoftGCNextAvailablePhysicalTarget == -1) // Uninitialized
			{
				int32 StartNumerator = FMath::Max(SoftGCStartNumerator, 1);
				int32 Denominator = FMath::Max(SoftGCDenominator, 1);
				// e.g. Start the target at 5/10, and decrease it by 1/10 each time the target is reached
				SoftGCNextAvailablePhysicalTarget = (static_cast<int64>(MemStats.TotalPhysical) * StartNumerator)
					/ Denominator;
			}

			if (SoftGCNextAvailablePhysicalTarget < -1)
			{
				// No further targets, no further SoftGC
			}
			else if (static_cast<int64>(MemStats.AvailablePhysical) <= SoftGCNextAvailablePhysicalTarget)
			{
				constexpr float SoftGCInstigateCooldown = 5 * 60.f;
				CurrentTime = FPlatformTime::Seconds();
				if (LastSoftGCTime + SoftGCInstigateCooldown <= CurrentTime)
				{
					TriggerMessages.Appendf(TEXT("\n  CookSettings.SoftGCMemoryTrigger: Available physical memory %dMiB is less than the current target for SoftGC %dMiB."),
						static_cast<uint32>(MemStats.AvailablePhysical / 1024 / 1024), static_cast<uint32>(SoftGCNextAvailablePhysicalTarget / 1024 / 1024));
					bTriggerGC = true;
					bIsSoftGC = true;
				}
			}
		}

		if (!bTriggerGC && SoftGCTimeFractionBudget > 0.)
		{
			FString TriggerDiagnostics;
			if (SoftGCHistory->IsTriggeringWithinBudget(*this, CurrentTime, &TriggerDiagnostics))
			{
				TriggerMessages.Appendf(TEXT("\n  CookSettings.SoftGCTimeTrigger: Periodic triggering of SoftGC: %s."),
					*TriggerDiagnostics);
				bTriggerGC = true;
				bIsSoftGC = true;
				bPeriodicTriggered = true;
			}
		}
	}

	if (!bTriggerGC)
	{
		return false;
	}

	// Don't allow a second OOM GC (soft or normal) within the GC cooldown period after a full GC, because this can cause thrashing
	constexpr float GCCooldown = 60.f;
	if (!bPeriodicTriggered && LastFullGCTime + GCCooldown > CurrentTime)
	{
		if (!bIsSoftGC && !bWarnedExceededMaxMemoryWithinGCCooldown)
		{
			bWarnedExceededMaxMemoryWithinGCCooldown = true;
			// If we are in a cooldown period, return false.
			UE_LOGF(LogCook, Display, "Garbage collection triggers ignored: Out of memory condition has been detected, but is only %.0fs after the last GC. "
				"It will be prevented until %.0f seconds have passed and we may run out of memory.\n"
				"Garbage collection triggered by conditions: %ls",
				static_cast<float>(CurrentTime - LastFullGCTime), GCCooldown, TriggerMessages.ToString());
		}
		return false;
	}

	const TCHAR* TypeMessage = bIsSoftGC ? TEXT("Soft") : (IsCookFlagSet(ECookInitializationFlags::EnablePartialGC) ? TEXT("Partial") : TEXT("Full"));

	UE_LOGF(LogCook, Display, "Garbage collection triggered (%ls). Triggered by conditions:%ls",
		TypeMessage, TriggerMessages.ToString());
	OutResultFlags |= COSR_RequiresGC | COSR_YieldTick;
	OutResultFlags |= bPeriodicTriggered ? COSR_RequiresGC_Periodic : COSR_RequiresGC_OOM;
	if (bIsSoftGC)
	{
		OutResultFlags |= COSR_RequiresGC_Soft;
	}
	return true;
}

void UCookOnTheFlyServer::SetGarbageCollectType(uint32 ResultFlagsFromTick)
{
	bGarbageCollectTypeSoft = (ResultFlagsFromTick & COSR_RequiresGC_Soft);
	bSaveGCDataForEvaluation = true;
}

void UCookOnTheFlyServer::ClearGarbageCollectType()
{
	bGarbageCollectTypeSoft = false;
	bSaveGCDataForEvaluation = false;

	ClearGCData();
}

void UCookOnTheFlyServer::PreGarbageCollect()
{
	using namespace UE::Cook;

	if (!IsInSession())
	{
		PackageTracker->SetCollectingGarbage(true);
		return;
	}

	NumObjectsHistory.AddInstance(GUObjectArray.GetObjectArrayNumMinusAvailable());
	NumPackagesHistory.AddInstance(PackageTracker->NumLoadedPackages());
	VirtualMemoryHistory.AddInstance(FPlatformMemory::GetStats().UsedVirtual);
	TArray<UPackage*> GCKeepPackages;
	TArray<UE::Cook::FPackageData*> GCKeepPackageDatas;

#if COOK_CHECKSLOW_PACKAGEDATA
	// Verify that only packages in the saving states have pointers to objects
	PackageDatas->LockAndEnumeratePackageDatas([](const FPackageData* PackageData)
		{
			check(PackageData->IsInStateProperty(EPackageStateProperty::Saving) || !PackageData->HasReferencedObjects());
		});
#endif
	if (SavingPackageData)
	{
		check(SavingPackageData->GetPackage());
		GCKeepObjects.Add(SavingPackageData->GetPackage());
		GCKeepPackageDatas.Add(SavingPackageData);
	}

	// Notify every FGenerationHelper of the garbage collect
	PackageDatas->LockAndEnumeratePackageDatas([this, &GCKeepPackages, &GCKeepPackageDatas](FPackageData* PackageData)
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
			if (!GenerationHelper)
			{
				GenerationHelper = PackageData->GetParentGenerationHelper();
			}
			if (GenerationHelper)
			{
				bool bShouldDemote;
				GenerationHelper->PreGarbageCollect(GenerationHelper, *PackageData, GCKeepObjects, GCKeepPackages,
					GCKeepPackageDatas, bShouldDemote);
				if (bShouldDemote && PackageData->IsInStateProperty(EPackageStateProperty::Saving))
				{
					// Demote any Generated/Generator packages we called PreSave on so they call their PostSave before the GC
					// or prevent them from being garbage collected if the splitter wants to keep them referenced
					ReleaseCookedPlatformData(*PackageData, UE::Cook::EStateChangeReason::GeneratorPreGarbageCollected,
						EPackageState::Request);
				}
			}
			if (PackageData->GetIsCookLast() && PackageData->IsInStateProperty(EPackageStateProperty::Saving))
			{
				GCKeepPackages.Add(PackageData->GetPackage());
				GCKeepPackageDatas.Add(PackageData);
			}
		});
	
	// Find the packages that are waiting on async jobs to finish cooking data
	// and make sure that they are not garbage collected until the jobs have
	// completed.
	{
		TMap<FPackageData*, UPackage*> UniquePendingPackages;
		PackageDatas->ForEachPendingCookedPlatformData(
		[&UniquePendingPackages](const FPendingCookedPlatformData& PendingData)
		{
			if (UObject* Object = PendingData.Object.Get())
			{	
				if (UPackage* Package = Object->GetPackage())
				{
					UniquePendingPackages.Add(&PendingData.PackageData, Package);
				}	
			}
		});

		GCKeepPackages.Reserve(GCKeepPackages.Num() + UniquePendingPackages.Num());
		for (const TPair<FPackageData*,UPackage*>& Pair : UniquePendingPackages)
		{
			GCKeepPackages.Add(Pair.Value);
			GCKeepPackageDatas.Add(Pair.Key);
		}
	}

	// Prevent GC of any objects on which we are still waiting for IsCachedCookedPlatformData
	PackageDatas->ForEachPendingCookedPlatformData(
	[this](UE::Cook::FPendingCookedPlatformData& Pending)
	{
		if (!Pending.PollIsComplete())
		{
			UObject* Object = Pending.Object.Get();
			check(Object); // Otherwise PollIsComplete would have returned true
			GCKeepObjects.Add(Object);
		}
	});

	const bool bPartialGC = IsCookFlagSet(ECookInitializationFlags::EnablePartialGC);
	if (bGarbageCollectTypeSoft || bPartialGC)
	{
		// Keep referenced all packages in requestqueue, loadqueue, and savequeue, and any packages they depend on
		TArray<FName> Queue;
		TSet<FName> Visited;
		auto AddPackageName = [&Visited, &Queue](FName PackageName)
		{
			bool bAlreadyExists;
			Visited.Add(PackageName, &bAlreadyExists);
			if (!bAlreadyExists)
			{
				Queue.Add(PackageName);
			}
		};
		for (FPackageData* PackageData : PackageDatas->GetRequestQueue().GetReadyRequestsUrgent())
		{
			AddPackageName(PackageData->GetPackageName());
		}
		for (FPackageData* PackageData : PackageDatas->GetRequestQueue().GetReadyRequestsNormal())
		{
			AddPackageName(PackageData->GetPackageName());
		}
		for (FPackageData* PackageData : PackageDatas->GetLoadQueue())
		{
			AddPackageName(PackageData->GetPackageName());
		}
		for (FPackageData* PackageData : PackageDatas->GetSaveQueue())
		{
			AddPackageName(PackageData->GetPackageName());
		}
		for (FPackageData* PackageData : PackageDatas->GetSaveStalledSet())
		{
			AddPackageName(PackageData->GetPackageName());
		}

		TArray<FName> Dependencies;
		while (!Queue.IsEmpty())
		{
			FName PackageName = Queue.Pop();
			Dependencies.Reset();
			AssetRegistry->GetDependencies(PackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package,
				UE::AssetRegistry::EDependencyQuery::Hard);
			for (FName DependencyName : Dependencies)
			{
				AddPackageName(DependencyName);
			};
		}

		TSet<UPackage*> GCKeepPackagesSet;
		GCKeepPackagesSet.Append(GCKeepPackages);
		for (FName PackageName : Visited)
		{
			UPackage* Package = FindPackage(nullptr, *WriteToString<256>(PackageName));
			if (Package)
			{
				bool bAlreadyInSet;
				GCKeepPackagesSet.Add(Package, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					GCKeepPackages.Add(Package);
					FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(Package->GetFName());
					if (PackageData)
					{
						GCKeepPackageDatas.Add(PackageData);
					}
				}
			}
		}
		ExpectedFreedPackageNames.Empty(PackageTracker->NumLoadedPackages());
		PackageTracker->ForEachLoadedPackage(
			[this, &GCKeepPackagesSet](UPackage* Package)
			{
				if (!GCKeepPackagesSet.Contains(Package))
				{
					ExpectedFreedPackageNames.Add(Package->GetFName());
				}
			});
	}

	// Add packages to GCKeepObjects. 
	TArray<UObject*> ObjectsWithOuter;
	for (UPackage* Package : GCKeepPackages)
	{
		GCKeepObjects.Add(Package);
	}
	for (FPackageData* PackageData : GCKeepPackageDatas)
	{
		PackageData->SetKeepReferencedDuringGC(true);
	}

	// Add all public objects within every package in memory to the UPackage::SoftGCPackageToObjectList container,
	// so they will be kept in memory if the package is kept in memory.
	ConstructSoftGCPackageToObjectList(this->SoftGCPackageToObjectListBuffer);

	// We call arbitrary system-specific code through FPendingCookedPlatformData.PollIsComplete
	// -> IsCachedCookedPlatformDataLoaded above, and we need to continue responding to object reallocations
	// whenever we call system-specific code. So do not mark that we are ignoring deletions from GC until we
	// have finished calling into that system-specific code.
	PackageTracker->SetCollectingGarbage(true);
}

void UCookOnTheFlyServer::CookerAddReferencedObjects(FReferenceCollector& Collector)
{
	using namespace UE::Cook;

	// GCKeepObjects are the objects that we want to keep loaded but we only have a WeakPtr to
	Collector.AddReferencedObjects(GCKeepObjects);
}

void UCookOnTheFlyServer::PostGarbageCollect()
{
	using namespace UE::Cook;

	PackageTracker->SetCollectingGarbage(false);

	NumObjectsHistory.AddInstance(GUObjectArray.GetObjectArrayNumMinusAvailable());
	NumPackagesHistory.AddInstance(PackageTracker->NumLoadedPackages());
	VirtualMemoryHistory.AddInstance(FPlatformMemory::GetStats().UsedVirtual);

	TSet<UObject*> SaveQueueObjectsThatStillExist;

	// If garbage collection deleted a UPackage WHILE WE WERE SAVING IT, then we have problems.
	check(!SavingPackageData || SavingPackageData->GetPackage() != nullptr);

	// If there was a GarbageCollect after we already started calling BeginCacheCookedPlatformData, then we
	// have a list of the WeakObjectPtr to all objects in the package (FPackageData::CachedObjejectsInOuter)
	// and some of those objects may have been set to null. We declare a reference to prevent GC for the RF_Public
	// objects in that list, but we do not declare that reference for private objects. The private objects may
	// therefore have been deleted and set to null 
	// Side note: because objects can be marked as pending kill at any time and we use FObjectWeakPtr.Get(),
	// which returns null if pending kill, we need to skip nulls in the array at any point, not just after GC.
	// 
	// We do not want to prevent GC of private objects in case there is the expectation by some
	// systems (blueprints, licensee code) that removing references to an object during PreCollectGarbage will cause
	// it to be deleted by GC and be replaceable afterwards. We add any new private objects after the garbage collect
	// and continue with the save. Public objects have a different contract; they are not replaceable across a
	// GC because anything outside the package could be referring to them. So we keep them referenced. But GC may
	// force delete them despite our reference, and the package is then in an unknown state. If that happens we
	// demote the package back to request and start its load and save over.
	TArray<FPackageData*> Demotes;
	auto UpdateSavingPackageAfterGarbageCollect =
		[&Demotes, &SaveQueueObjectsThatStillExist](FPackageData* PackageData)
	{
		bool bOutDemote;
		PackageData->UpdateSaveAfterGarbageCollect(bOutDemote);
		if (bOutDemote)
		{
			Demotes.Add(PackageData);
		}
		else
		{
			// Mark that the objects for this package should be kept in CachedCookedPlatformData records
			for (FCachedObjectInOuter& CachedObjectInOuter : PackageData->GetCachedObjectsInOuter())
			{
				UObject* Object = CachedObjectInOuter.Object.Get();
				if (Object)
				{
					SaveQueueObjectsThatStillExist.Add(Object);
				}
			}
		}
	};
	for (FPackageData* PackageData : PackageDatas->GetSaveQueue())
	{
		UpdateSavingPackageAfterGarbageCollect(PackageData);
	}
	for (FPackageData* PackageData : PackageDatas->GetSaveStalledSet())
	{
		UpdateSavingPackageAfterGarbageCollect(PackageData);
	}
	for (FPackageData* PackageData : Demotes)
	{
		FGenerationHelper::ValidateSaveStalledState(*this, *PackageData, TEXT("PostGarbageCollect"));

		switch (PackageData->GetState())
		{
		case EPackageState::SaveActive:
			PackageData->SendToState(EPackageState::Request, ESendFlags::QueueRemove, EStateChangeReason::GarbageCollected);
			if (PackageData->GetIsCookLast())
			{
				// CookLast packages in SaveState have had their urgency removed. Add it back if we need to demote them.
				PackageData->SetUrgency(EUrgency::Blocking, ESendFlags::QueueNone);
			}
			PackageDatas->GetRequestQueue().AddRequest(PackageData, /* bForceUrgent */ true);
			break;
		case EPackageState::SaveStalledAssignedToWorker:
			PackageData->SendToState(EPackageState::AssignedToWorker, ESendFlags::QueueAddAndRemove, EStateChangeReason::GarbageCollected);
			break;
		case EPackageState::SaveStalledRetracted:
			DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::RetractedByCookDirector);
			break;
		default:
			checkf(false, TEXT("State %s not handled in a demoted package."), LexToString(PackageData->GetState()));
			break;
		}
	}

	// Mark that any objects in PendingCookedPlatformDatas should be kept in CachedCookedPlatformData records
	PackageDatas->ForEachPendingCookedPlatformData(
		[&SaveQueueObjectsThatStillExist](FPendingCookedPlatformData& CookedPlatformData)
		{
			UObject* Object = CookedPlatformData.Object.Get();
			if (Object)
			{
				SaveQueueObjectsThatStillExist.Add(Object);
			}
			else
			{
				CookedPlatformData.Release();
			}
		});

	// Remove objects that were deleted by garbage collection from our containers that track raw object pointers
	PackageDatas->CachedCookedPlatformDataObjectsPostGarbageCollect(SaveQueueObjectsThatStillExist);

	PackageDatas->LockAndEnumeratePackageDatas([this](FPackageData* PackageData)
	{
		if (TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper())
		{
			GenerationHelper->PostGarbageCollect(GenerationHelper, *GCDiagnosticContext);
		}
	});

	// Second pass over all PackageDatas, combine a few operations
	PackageDatas->LockAndEnumeratePackageDatas([](FPackageData* PackageData)
	{
		// Mark that the PackageData no longer needs to be keepreferenced.
		// This can only be done after all GenerationHelper->PostGarbageCollect have been called.
		PackageData->SetKeepReferencedDuringGC(false);

		// Reset the completion flags for FPreloadPackage, since the UPackage might be no longer loaded.
		TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
		if (Preloader)
		{
			Preloader->PostGarbageCollect();
		}

		// Free memory used by GetLoadDependencies for packages that have been garbage collected.
		// To avoid the expense of calling FindPackage on every package, only do this for packages that
		// are no longer in progress but still have loaddependencies.
		// We can not free LoadDependencies for PackageDatas that still have their package loaded, because
		// the package might need to be saved later for an additional platform, and we cannot correctly recreate the
		// package's LoadDependencies until after the package is garbagecollected and reexecutes Load.
		if (PackageData->GetLoadDependencies() && !PackageData->IsInProgress())
		{
			if (!FindPackage(nullptr, *WriteToString<256>(PackageData->GetPackageName())))
			{
				PackageData->ClearLoadDependencies();
			}
		}
	});

	// Only after running all possible callbacks that need our links for diagnostics, clear the list of temporary
	// references that we created for the garbage collection.
	if (!bSaveGCDataForEvaluation)
	{
		ClearGCData();
	}

	CookedPackageCountSinceLastGC = 0;

	// Whenever we collect garbage, reset the counter for how many busy reports with an
	// idle shadercompiler we need before we issue a warning
	bShaderCompilerWasActiveOnPreviousBusyReport = true;
}

void UCookOnTheFlyServer::ClearGCData()
{
	GCKeepObjects.Empty();
	UPackage::SoftGCPackageToObjectList.Empty();
	SoftGCPackageToObjectListBuffer.Empty();
}

bool UCookOnTheFlyServer::NeedsDiagnosticSecondGC() const
{
	return GCDiagnosticContext->NeedsDiagnosticSecondGC();
}

void UCookOnTheFlyServer::OnCookerStartCollectGarbage(uint32& ResultFlagsFromTick)
{
	GCDiagnosticContext->OnCookerStartCollectGarbage(*this, ResultFlagsFromTick);
}

void UCookOnTheFlyServer::OnCookerEndCollectGarbage(uint32& ResultFlagsFromTick)
{
	GCDiagnosticContext->OnCookerEndCollectGarbage(*this, ResultFlagsFromTick);
}

int32 UCookOnTheFlyServer::NumLoadedPackages() const
{
	return PackageTracker->NumLoadedPackages();
}

void UCookOnTheFlyServer::EvaluateGarbageCollectionResults(bool bWasDueToOOM, bool bWasPartialGC, uint32 ResultFlags,
	int32 NumObjectsBeforeGC, const FPlatformMemoryStats& MemStatsBeforeGC, int32 NumPackagesBeforeGC,
	const FGenericMemoryStats& AllocatorStatsBeforeGC,
	int32 NumObjectsAfterGC, const FPlatformMemoryStats& MemStatsAfterGC, int32 NumPackagesAfterGC,
	const FGenericMemoryStats& AllocatorStatsAfterGC,
	float GCDurationSeconds)
{
	using namespace UE::Cook;

	ON_SCOPE_EXIT
	{
		ExpectedFreedPackageNames.Empty();
		GCDiagnosticContext->OnEvaluateResultsComplete();
	};
	bWarnedExceededMaxMemoryWithinGCCooldown = false;
	LastGCTime = FPlatformTime::Seconds();
	bool bWasSoftGC = ResultFlags & COSR_RequiresGC_Soft;
	if (bWasSoftGC)
	{
		LastSoftGCTime = LastGCTime;
		if (SoftGCStartNumerator)
		{
			int32 StartNumerator = FMath::Max(SoftGCStartNumerator, 1);
			int32 Denominator = FMath::Max(SoftGCDenominator, 1);
			// Calculate the new SoftGCNextAvailablePhysicalTarget. Use the floor of NewAvailableMemory/Denominator,
			// unless we are already 50% of the way through that level, in which case use the next value below that
			int64 PhysicalMemoryQuantum = static_cast<int64>(MemStatsAfterGC.TotalPhysical) / Denominator;
			int32 NextTarget =
				static_cast<int64>(MemStatsAfterGC.AvailablePhysical - PhysicalMemoryQuantum / 2) / PhysicalMemoryQuantum;
			NextTarget = FMath::Min(NextTarget, StartNumerator);
			if (NextTarget <= 0)
			{
				SoftGCNextAvailablePhysicalTarget = -2; // disabled, no further targets
			}
			else
			{
				SoftGCNextAvailablePhysicalTarget = static_cast<int64>((MemStatsAfterGC.TotalPhysical) * NextTarget)
					/ Denominator;
			}
		}
	}
	else
	{
		LastSoftGCTime = LastGCTime;
		LastFullGCTime = LastGCTime;
	}
	if (SoftGCHistory)
	{
		SoftGCHistory->AddDurationMeasurement(GCDurationSeconds);
	}

	if (IsCookingInEditor())
	{
		return;
	}

	// Mark the packages we freed so we can give a warning to diagnose why they are still referenced if they
	// get loaded again.
	if (bWasSoftGC)
	{
		PackageTracker->AddExpectedNeverLoadPackages(ExpectedFreedPackageNames);
	}

	enum class EDisplayLevel : uint8
	{
		None,
		Simple,
		Full,
		FullWithDiagnostics,
	};

	static EDisplayLevel LevelGCOther = EDisplayLevel::None;
	static EDisplayLevel LevelGCSuccess = EDisplayLevel::None;
	// LevelGCFail defined below as the return value of ParseCommandLineAndReturnLevelGCFail.
	auto ParseCommandLineAndReturnLevelGCFail = []()
	{
		EDisplayLevel LevelGCFailLocal;
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		if (FLowLevelMemTracker::Get().IsEnabled())
		{
			LevelGCOther = EDisplayLevel::FullWithDiagnostics;
			LevelGCSuccess = EDisplayLevel::FullWithDiagnostics;
			LevelGCFailLocal = EDisplayLevel::FullWithDiagnostics;
		}
		else
#endif
		{
			LevelGCOther = EDisplayLevel::None;
			LevelGCSuccess = EDisplayLevel::Simple;
			LevelGCFailLocal = EDisplayLevel::Full;
		}
		// Command line overrides the defaults
		auto TryParseDisplayLevel = [](const TCHAR* FlagPattern, EDisplayLevel& OutLevel)
		{
			FString Text;
			bool bParsedValue = FParse::Value(FCommandLine::Get(), *FString::Printf(TEXT("-%s="), FlagPattern), Text);
			if (!bParsedValue && FParse::Param(FCommandLine::Get(), FlagPattern))
			{
				Text = TEXT("Full");
			}
			if (Text.IsEmpty())
			{
				return false;
			}
			for (const auto& Pair : std::initializer_list<TPair<EDisplayLevel, const ANSICHAR*>> {
				{ EDisplayLevel::None, "None" },
				{ EDisplayLevel::Simple, "Simple" },
				{ EDisplayLevel::FullWithDiagnostics, "Full" } })
			{
				if (Text == Pair.Value)
				{
					OutLevel = Pair.Key;
					return true;
				}
			}
			UE_LOGF(LogCook, Error, "Invalid commandline %ls argument for -%ls=%ls. Valid values are { None, Simple, Full }.",
				*Text, FlagPattern, *Text);
			return false;
		};

		EDisplayLevel LevelAll;
		if (TryParseDisplayLevel(TEXT("CookMemoryAnalysis"), LevelAll))
		{
			LevelGCOther = LevelAll;
			LevelGCSuccess = LevelAll;
			LevelGCFailLocal = LevelAll;
		}
		TryParseDisplayLevel(TEXT("CookMemoryAnalysis.GCOther"), LevelGCOther);
		TryParseDisplayLevel(TEXT("CookMemoryAnalysis.GCSuccess"), LevelGCSuccess);
		TryParseDisplayLevel(TEXT("CookMemoryAnalysis.GCFail"), LevelGCFailLocal);
		return LevelGCFailLocal;
	};
	static EDisplayLevel LevelGCFail = ParseCommandLineAndReturnLevelGCFail();

	if ((!bWasDueToOOM && LevelGCOther == EDisplayLevel::None)
		|| (bWasDueToOOM && LevelGCSuccess == EDisplayLevel::None && LevelGCFail == EDisplayLevel::None))
	{
		// No diagnostics requested for this case, early exit to avoid any further cost.
		return;
	}

	int64 NumObjectsMin = NumObjectsHistory.GetMinimum();
	int64 NumObjectsMax = NumObjectsHistory.GetMaximum();
	int64 NumObjectsFreed = NumObjectsBeforeGC - NumObjectsAfterGC;
	int64 NumObjectsCapacity = GUObjectArray.GetObjectArrayEstimatedAvailable() + GUObjectArray.GetObjectArrayNumMinusAvailable();
	int64 NumPackagesMin = NumPackagesHistory.GetMinimum();
	int64 NumPackagesMax = NumPackagesHistory.GetMaximum();
	int64 NumPackagesFreed = NumPackagesBeforeGC - NumPackagesAfterGC;
	int64 VirtualMemMin = VirtualMemoryHistory.GetMinimum();
	int64 VirtualMemMax = VirtualMemoryHistory.GetMaximum();
	int64 VirtualMemSpread = VirtualMemMax - VirtualMemMin;
	int64 VirtualMemBeforeGC = MemStatsBeforeGC.UsedVirtual;
	int64 VirtualMemAfterGC = MemStatsAfterGC.UsedVirtual;
	int64 VirtualMemFreed = MemStatsBeforeGC.UsedVirtual - MemStatsAfterGC.UsedVirtual;

	int64 ExpectedObjectsFreed = static_cast<int64>(MemoryExpectedFreedToSpreadRatio
		* static_cast<float>(NumObjectsBeforeGC - NumObjectsMin));
	double ExpectedMemFreed = MemoryExpectedFreedToSpreadRatio * (VirtualMemBeforeGC - VirtualMemMin);

	FString HeaderMessage = TEXT("GarbageCollection Results:");
	EDisplayLevel DisplayLevel = EDisplayLevel::None;
	bool bWasImpactful = true;
	if (!bWasDueToOOM)
	{
		DisplayLevel = LevelGCOther;
	}
	else
	{
		bWasImpactful =
			(NumObjectsFreed >= ExpectedObjectsFreed || NumObjectsBeforeGC - NumObjectsMin < ExpectedObjectsFreed) &&
			(VirtualMemFreed >= ExpectedMemFreed || VirtualMemBeforeGC - VirtualMemMin <= ExpectedMemFreed);
		if (!bWasImpactful)
		{
			HeaderMessage = TEXT("GarbageCollection Results: Garbage Collection was not very impactful.");
		}
		DisplayLevel = bWasImpactful ? LevelGCSuccess : LevelGCFail;
	}

	constexpr int32 BytesPerMeg = 1000000;
	if (DisplayLevel == EDisplayLevel::None)
	{
		// No text
	}
	else if (DisplayLevel == EDisplayLevel::Simple)
	{
		UE_LOG(LogCook, Display, TEXT("%s\n")
			TEXT("\tType: %s\n")
			TEXT("\tDuration: %.3fs\n")
			TEXT("\tNumObjects:\n")
			TEXT("\t\tCapacity:         %10lld\n")
			TEXT("\t\tBefore GC:        %10lld\n")
			TEXT("\t\tAfter GC:         %10lld\n")
			TEXT("\t\tFreed by GC:      %10.1f%%\n")
			TEXT("\tNumPackages:\n")
			TEXT("\t\tBefore GC:        %10lld\n")
			TEXT("\t\tAfter GC:         %10lld\n")
			TEXT("\t\tFreed by GC:      %10.1f%%\n")
			TEXT("\tVirtual Memory:\n")
			TEXT("\t\tBefore GC:        %10" INT64_FMT " MB\n")
			TEXT("\t\tAfter GC:         %10" INT64_FMT " MB\n")
			TEXT("\t\tFreed by GC:      %10.1f%%"),
			*HeaderMessage,
			(bWasSoftGC ? TEXT("Soft") : (bWasPartialGC ? TEXT("Partial") : TEXT("Full"))),
			GCDurationSeconds,
			NumObjectsCapacity, (int64)NumObjectsBeforeGC, (int64)NumObjectsAfterGC,
			100.f * ((float)NumObjectsFreed)/NumObjectsBeforeGC,
			(int64) NumPackagesBeforeGC, (int64) NumPackagesAfterGC,
			100.f * ((float)NumPackagesFreed)/NumPackagesBeforeGC,
			VirtualMemBeforeGC / BytesPerMeg, VirtualMemAfterGC / BytesPerMeg,
			100.f * ((float)VirtualMemFreed) / VirtualMemBeforeGC
		);
	}
	else
	{
		check(DisplayLevel == EDisplayLevel::Full || DisplayLevel == EDisplayLevel::FullWithDiagnostics);
		UE_LOGF(LogCook, Display, "%ls", *HeaderMessage);
		UE_LOGF(LogCook, Display, "\tMemoryAnalysis: General:\n"
			"\t\tType: %ls\n"
			"\tDuration: %.3fs",
			(bWasSoftGC ? TEXT("Soft") : (bWasPartialGC ? TEXT("Partial") : TEXT("Full"))),
			GCDurationSeconds);
		UE_LOG(LogCook, Display, TEXT("\tMemoryAnalysis: NumObjects:\n")
			TEXT("\t\tCapacity:         %10" INT64_FMT "\n")
			TEXT("\t\tProcess Min:      %10" INT64_FMT "\n")
			TEXT("\t\tProcess Max:      %10" INT64_FMT "\n")
			TEXT("\t\tBefore GC:        %10" INT64_FMT "\n")
			TEXT("\t\tAfter GC:         %10" INT64_FMT "\n")
			TEXT("\t\tFreed by GC:      %10.1f%%"),
			NumObjectsCapacity, NumObjectsMin, NumObjectsMax,
			(int64)NumObjectsBeforeGC, (int64)NumObjectsAfterGC,
			100.f * ((float)NumObjectsFreed)/NumObjectsBeforeGC);
		UE_LOG(LogCook, Display, TEXT("\tMemoryAnalysis: NumPackages:\n")
			TEXT("\t\tProcess Min:      %10" INT64_FMT "\n")
			TEXT("\t\tProcess Max:      %10" INT64_FMT "\n")
			TEXT("\t\tBefore GC:        %10" INT64_FMT "\n")
			TEXT("\t\tAfter GC:         %10" INT64_FMT "\n")
			TEXT("\t\tFreed by GC:      %10.1f%%"),
			NumPackagesMin, NumPackagesMax,
			(int64)NumPackagesBeforeGC, (int64)NumPackagesAfterGC,
			100.f * ((float)NumPackagesFreed)/NumPackagesBeforeGC);
		UE_LOG(LogCook, Display, TEXT("\tMemoryAnalysis: Virtual Memory:\n")
			TEXT("\t\tProcess Min:      %10" INT64_FMT " MB\n")
			TEXT("\t\tProcess Max:      %10" INT64_FMT " MB\n")
			TEXT("\t\tBefore GC:        %10" INT64_FMT " MB\n")
			TEXT("\t\tAfter GC:         %10" INT64_FMT " MB\n")
			TEXT("\t\tFreed by GC:      %10.1f%%"),
			VirtualMemMin / BytesPerMeg, VirtualMemMax / BytesPerMeg,
			VirtualMemBeforeGC / BytesPerMeg, VirtualMemAfterGC / BytesPerMeg,
			100.f * ((float)VirtualMemFreed)/VirtualMemBeforeGC);
		auto AllocatorStatsToString = [](const FGenericMemoryStats& AllocatorStats)
		{
			TStringBuilder<256> Writer;
			for (const TPair<FStringView, SIZE_T>& Item : AllocatorStats)
			{
				Writer << TEXT("\n\t\tItem ") << Item.Key << TEXT(" ") << (uint64)Item.Value;
			}
			return FString(*Writer);
		};
		UE_LOGF(LogCook, Display, "\tMemoryAnalysis: Allocator Stats Before:%ls",
			*AllocatorStatsToString(AllocatorStatsBeforeGC));
		UE_LOGF(LogCook, Display, "\tMemoryAnalysis: Allocator Stats After:%ls",
			*AllocatorStatsToString(AllocatorStatsAfterGC));

		if (DisplayLevel == EDisplayLevel::Full)
		{
			UE_LOGF(LogCook, Display,
				"Extended memory diagnostics are disabled. Run with -llm or -trace=memtag or -CookMemoryAnalysis for more diagnostics.");
		}
		else if (!bWasSoftGC)
		{
			UE_LOGF(LogCook, Display, "See log for package info and LLM tag diagnostics.");
			UE::Cook::DumpPackageReferencers(TEXT("Non-startup Packages"), CookByTheBookOptions->SessionStartupObjects);
			GLog->Logf(TEXT("Memory Analysis: LLM Tags:"));
#if ENABLE_LOW_LEVEL_MEM_TRACKER
			if (FLowLevelMemTracker::Get().IsEnabled())
			{
				FLowLevelMemTracker::Get().DumpToLog();
			}
			else
#endif
			{
				GLog->Logf(TEXT("LLM Tags are not displayed because llm is disabled. Run with -llm or -trace=memtag to see llm tags."));
			}
		}
		else // bWasSoftGC
		{
			// If some packages we expected to be freed were not freed, show the reference chains for why
			// they were not freed.
			TArray<UPackage*> PackagesReferencedOutsideOfCooker;
			for (FWeakObjectPtr& WeakPtr : CookByTheBookOptions->SessionStartupObjects)
			{
				UObject* Object = WeakPtr.Get();
				if (!Object)
				{
					continue;
				}
				UPackage* Package = Object->GetPackage();
				ExpectedFreedPackageNames.Remove(Package->GetFName());
			}
			PackageTracker->ForEachLoadedPackage(
				[this, &PackagesReferencedOutsideOfCooker](UPackage* Package)
				{
					if (ExpectedFreedPackageNames.Contains(Package->GetFName()))
					{
						PackagesReferencedOutsideOfCooker.Add(Package);
					}
				});
			if (PackagesReferencedOutsideOfCooker.Num() > 0)
			{
				UE::Cook::DumpPackageReferencers(TEXT("SoftGCPackages"),
					CookByTheBookOptions->SessionStartupObjects, PackagesReferencedOutsideOfCooker);
			}
		}
	}
}
