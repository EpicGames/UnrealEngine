// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/RingBuffer.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/UnrealTemplate.h"

class UCookOnTheFlyServer;
class UObject;

namespace UE::Cook
{

/**
 * Holds information about the cookers garbage collection status, and communicates requests from low level structures
 * back up to the CookCommandlet that is capable of acting on those requests with additional garbage collection
 * commands.
 */
class FCookGCDiagnosticContext
{
public:
	~FCookGCDiagnosticContext();

	bool NeedsDiagnosticSecondGC() const;
	bool CurrentGCHasHistory() const;

	/**
	 * Add a request to reexecute the current GC after all of the PostGarbageCollect calls run and
	 * control returns back to the caller of CollectGarbage, and with history turned on.
	 * Returns false if not currently in post-GC, or the garbage collect that just ran already had history.
	 */
	bool TryRequestGCWithHistory();
	/**
	 * Add a request to reexecute the current GC after all of the PostGarbageCollect calls run and
	 * control returns back to the caller of CollectGarbage, and with soft GC turned off.
	 * Returns false if not currently in post-GC, or the garbage collect that just ran already was a full GC.
	 */
	bool TryRequestFullGC();

	void OnCookerStartCollectGarbage(UCookOnTheFlyServer& COTFS, uint32& ResultFlagsFromTick);
	void OnCookerEndCollectGarbage(UCookOnTheFlyServer& COTFS, uint32& ResultFlagsFromTick);
	void OnEvaluateResultsComplete();

private:
	void SetGCWithHistoryRequested(bool bValue);
	
	int32 SavedGCHistorySize = 0;
	bool bRequestsAvailable = false;
	bool bGCInProgress = false;
	bool bRequestGCWithHistory = false;
	bool bRequestFullGC = false;
	bool bCurrentGCHasHistory = false;
	bool bCurrentGCIsFull = false;
};

/** Stores data over time for SoftGC that is used to throttle how frequently we trigger periodic SoftGC. */
class FSoftGCHistory
{
public:
	void AddDurationMeasurement(float DurationSeconds);
	bool IsTriggeringWithinBudget(UCookOnTheFlyServer& COTFS, double CurrentTimeSeconds, FString* OutDiagnostics);

private:
	TRingBuffer<float> DurationHistory;
	int MaxHistoryLength = 5;
	float AverageDurationSeconds = 0.f;
};

/** Scoped type to call Set/Clear SoftGCPackageToObjectListBuffer. */
struct FScopeFindCookReferences
{
	FScopeFindCookReferences(UCookOnTheFlyServer& InCOTFS);
	~FScopeFindCookReferences();

	UCookOnTheFlyServer& COTFS;
	TGuardValue<bool> SoftGCGuard;
	bool bNeedsConstructBuffer;
};

/**
 * For every package in memory, add a list of all of its public UObjects into the map
 * used in garbage collection: UPackage::SoftGCPackageToObjectList. This will cause all of
 * its public objects to be referenced if the UPackage is referenced.
 */
void ConstructSoftGCPackageToObjectList(TArray<UObject*>& PackageToObjectListBuffer);

} // namespace UE::Cook
