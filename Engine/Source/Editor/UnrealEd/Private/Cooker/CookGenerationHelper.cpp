// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookGenerationHelper.h"

#include "Algo/Unique.h"
#include "Cooker/CookActivePackage.h"
#include "AssetRegistry/AssetData.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookGarbageCollect.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookPackageArtifacts.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookWorkerServer.h"
#include "Cooker/IWorkerRequests.h"
#include "Cooker/PackageTracker.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/PlatformMath.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/Parse.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "UObject/ReferenceChainSearch.h"

//////////////////////////////////////////////////////////////////////////
// FPopulateContext

ICookPackageSplitter::FPopulateContext::FPopulateContext(UE::Cook::CookPackageSplitter::FPopulateContextData& InData)
	: Data(InData)
{
}

UPackage* ICookPackageSplitter::FPopulateContext::GetOwnerPackage() const
{
	return Data.OwnerPackage;
}

UObject* ICookPackageSplitter::FPopulateContext::GetOwnerObject() const
{
	return Data.OwnerObject;
}

TConstArrayView<ICookPackageSplitter::FGeneratedPackageForPopulate>& ICookPackageSplitter::FPopulateContext::GetGeneratedPackages()
{
	return Data.GeneratedPackages;
}

bool ICookPackageSplitter::FPopulateContext::IsCalledOnGenerator() const
{
	return Data.TargetGeneratedPackage == nullptr;
}

UPackage* ICookPackageSplitter::FPopulateContext::GetTargetPackage() const
{
	return Data.TargetGeneratedPackage ? Data.TargetGeneratedPackage->Package : Data.OwnerPackage;
}

const ICookPackageSplitter::FGeneratedPackageForPopulate* ICookPackageSplitter::FPopulateContext::GetTargetGeneratedPackage() const
{
	return Data.TargetGeneratedPackage;
}

void ICookPackageSplitter::FPopulateContext::ReportObjectToMove(UObject* Object)
{
	Data.ObjectsToMove.Add(Object);
}

void ICookPackageSplitter::FPopulateContext::ReportObjectsToMove(TConstArrayView<UObject*> Objects)
{
	Data.ObjectsToMove.Append(Objects);
}

void ICookPackageSplitter::FPopulateContext::ReportKeepReferencedPackage(UPackage* Package)
{
	Data.KeepReferencedPackages.Add(Package);
}

void ICookPackageSplitter::FPopulateContext::ReportKeepReferencedPackages(TConstArrayView<UPackage*> Packages)
{
	Data.KeepReferencedPackages.Append(Packages);
}

void ICookPackageSplitter::FPopulateContext::ReportSaveDependency(UE::Cook::FCookDependency CookDependency)
{
	Data.BuildResultDependencies.Add(UE::Cook::BuildResult::NAME_Save, MoveTemp(CookDependency));
}

bool ICookPackageSplitter::PopulateGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject,
	const TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& GeneratedPackages, TArray<UObject*>& OutObjectsToMove,
	TArray<UPackage*>& OutKeepReferencedPackages)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	return true;
}

bool ICookPackageSplitter::PreSaveGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject,
	const TArray<FGeneratedPackageForPopulate>& PlaceholderPackages, TArray<UPackage*>& OutKeepReferencedPackages)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	return true;
}

void ICookPackageSplitter::PostSaveGeneratorPackage(UPackage* OwnerPackage, UObject* OwnerObject)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

bool ICookPackageSplitter::PopulateGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
	const FGeneratedPackageForPopulate& GeneratedPackage, TArray<UObject*>& OutObjectsToMove,
	TArray<UPackage*>& OutKeepReferencedPackages)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	return true;
}

bool ICookPackageSplitter::PreSaveGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
	const FGeneratedPackageForPopulate& GeneratedPackage, TArray<UPackage*>& OutKeepReferencedPackages)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	return true;
};

void ICookPackageSplitter::PostSaveGeneratedPackage(UPackage* OwnerPackage, UObject* OwnerObject,
	const FGeneratedPackageForPopulate& GeneratedPackage)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bDeprecatedVirtualCalledAsExpected = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void ICookPackageSplitter::WarnIfDeprecatedVirtualNotCalled(const TCHAR* FunctionName)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	if (!bDeprecatedVirtualCalledAsExpected)
	{
		UE_LOGF(LogCook, Warning,
			"ICookPackageSplitter::%ls has been deprecated. Implement version that takes FPopulateContext instead.",
			FunctionName);
	}
	bDeprecatedVirtualCalledAsExpected = false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

namespace UE::Cook
{

//////////////////////////////////////////////////////////////////////////
// FGenerationHelper

FGenerationHelper::FGenerationHelper(FPackageData& InOwner)
: OwnerInfo(*this, InOwner, true /* bInGenerator */)
{
	UCookOnTheFlyServer& COTFS = InOwner.GetPackageDatas().GetCookOnTheFlyServer();
	TConstArrayView<const ITargetPlatform*> TargetPlatforms = COTFS.PlatformManager->GetSessionPlatforms();
	PlatformDatas.Reserve(TargetPlatforms.Num());
	OwnerInfo.PlatformDatas.Reserve(TargetPlatforms.Num());
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		PlatformDatas.Add(TargetPlatform);
		OwnerInfo.PlatformDatas.Add(TargetPlatform);
	}
}

FGenerationHelper::~FGenerationHelper()
{
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: Destructed.",
		*WriteToString<256>(GetOwner().GetPackageName()));

	NotifyCompletion(ICookPackageSplitter::ETeardown::Complete);
	GetOwner().OnGenerationHelperDestroyed(*this);
}

void FGenerationHelper::NotifyCompletion(ICookPackageSplitter::ETeardown Status)
{
	if (IsInitialized() && IsValid() && CookPackageSplitterInstance)
	{
		CookPackageSplitterInstance->Teardown(Status);
		CookPackageSplitterInstance.Reset();
	}
}

void FGenerationHelper::Initialize()
{
	if (InitializeStatus != EInitializeStatus::Uninitialized)
	{
		return;
	}

	FPackageData& OwnerPackageData = GetOwner();
	FName OwnerPackageName = OwnerPackageData.GetPackageName();
	UCookOnTheFlyServer& COTFS = OwnerPackageData.GetPackageDatas().GetCookOnTheFlyServer();
	UPackage* LocalOwnerPackage = FindOrLoadPackage(COTFS, OwnerPackageData);
	if (!LocalOwnerPackage)
	{
		InitializeStatus = EInitializeStatus::Invalid;
		return;
	}

	UObject* LocalSplitDataObject;
	UE::Cook::Private::FRegisteredCookPackageSplitter* LocalRegisteredSplitterType = nullptr;
	TUniquePtr<ICookPackageSplitter> LocalSplitter;

	// When asked to Initialize for cases outside of the generator's Save state, ignore the
	// RequiresCachedCookedPlatformDataBeforeSplit requirement before calling ShouldSplit.
	// MPCOOKTODO: This breaks a contract and we should fix it. We have worked around it for now by requiring
	// that RequiresCachedCookedPlatformDataBeforeSplit forces EGeneratedRequiresGenerator::Save, so that Initialize
	// is not called outside of the generator's Save state.
	constexpr bool bCookedPlatformDataIsLoaded = true;
	bool bNeedWaitForIsLoaded;

	SearchForRegisteredSplitDataObject(COTFS, OwnerPackageName, LocalOwnerPackage,
		TOptional<TConstArrayView<FCachedObjectInOuter>>(), LocalSplitDataObject, LocalRegisteredSplitterType,
		LocalSplitter, bCookedPlatformDataIsLoaded, bNeedWaitForIsLoaded);
	if (!LocalSplitDataObject || !LocalSplitter)
	{
		check(!bNeedWaitForIsLoaded);
		InitializeStatus = EInitializeStatus::Invalid;
		return;
	}

	Initialize(LocalSplitDataObject, LocalRegisteredSplitterType, MoveTemp(LocalSplitter));
}

void FGenerationHelper::Initialize(const UObject* InSplitDataObject,
	UE::Cook::Private::FRegisteredCookPackageSplitter* InRegisteredSplitterType,
	TUniquePtr<ICookPackageSplitter>&& InCookPackageSplitterInstance)
{
	check(InSplitDataObject);
	if (InitializeStatus != EInitializeStatus::Uninitialized)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: Redundant initialize ignored.",
			*WriteToString<256>(GetOwner().GetPackageName()));
		// If we already have a splitter, keep the old and throw out the new. The old one
		// still contains some state.
		return;
	}

	RegisteredSplitterType = InRegisteredSplitterType;
	CookPackageSplitterInstance = MoveTemp(InCookPackageSplitterInstance);
	InitializeStatus = EInitializeStatus::Valid;

	SplitDataObject = InSplitDataObject;
	SplitDataObjectName = FName(FStringView(InSplitDataObject->GetFullName()));
	bUseInternalReferenceToAvoidGarbageCollect =
		CookPackageSplitterInstance->UseInternalReferenceToAvoidGarbageCollect();
	bRequiresGeneratorPackageDestructBeforeResplit =
		CookPackageSplitterInstance->RequiresGeneratorPackageDestructBeforeResplit();
	DoesGeneratedRequireGeneratorValue =
		CookPackageSplitterInstance->DoesGeneratedRequireGenerator();

	// Workaround for our current inability to handle RequiresCachedCookedPlatformDataBeforeSplit when
	// calling Initialize or TryCreateValidParentGenerationHelper. We force EGeneratedRequiresGenerator::Save
	// in the RequiresCachedCookedPlatformDataBeforeSplit case, so that the generator is always initialized before
	// we call either of those functions. See the comments in TryCreateValidParentGenerationHelper and
	// FGenerationHelper::Initialize(void)
	if (RegisteredSplitterType->RequiresCachedCookedPlatformDataBeforeSplit()
		&& DoesGeneratedRequireGeneratorValue < ICookPackageSplitter::EGeneratedRequiresGenerator::Save)
	{
		DoesGeneratedRequireGeneratorValue = ICookPackageSplitter::EGeneratedRequiresGenerator::Save;
	}

	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: Initialized.", *WriteToString<256>(GetOwner().GetPackageName()));
}

void FGenerationHelper::InitializeAsInvalid()
{
	if (InitializeStatus != EInitializeStatus::Uninitialized)
	{
		return;
	}
	InitializeStatus = EInitializeStatus::Invalid;
}

void FGenerationHelper::Uninitialize()
{
	if (InitializeStatus != EInitializeStatus::Valid)
	{
		return;
	}
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: Uninitialized.", *WriteToString<256>(GetOwner().GetPackageName()));

	// Demote stalled packages; we will be garbage collecting so they no longer need to be preserved.
	// And we need to demote them so that they drop their references to the generation helper and allow it to be
	// deleted if no longer referenced
	DemoteStalledPackages(OwnerInfo.PackageData->GetPackageDatas().GetCookOnTheFlyServer(),
		false /* bFromAllSavesCompleted */);

	NotifyCompletion(ICookPackageSplitter::ETeardown::Complete);
	check(!CookPackageSplitterInstance);

	InitializeStatus = EInitializeStatus::Uninitialized;

	OwnerInfo.Uninitialize();
	SplitDataObject.Reset();
	SplitDataObjectName = NAME_None;
	RegisteredSplitterType = nullptr;
	// CookPackageSplitterInstance was set to null above
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		Info.Uninitialize();
	}
	OwnerPackage.Reset();
	ExternalActorDependencies.Empty();

	// Keep PlatformDatas, they are allowed in the uninitialized state.
	// Keep PlatformData.PreviousGeneratedPackages; they are allowed in the uninitialized state
	// PlatformData.ReferenceFromKeepForIncremental
	// PlatformData.ReferenceFromKeepForGeneratorSave
	// Keep PlatformData.NumSaved; it is allowed in the uninitialized state

	check(OwnerObjectsToMove.IsEmpty()); // We can not still be in the save state, so this should be empty
	// Do not modify the reference tracking variables
	// ReferenceFromKeepForQueueResults
	// ReferenceFromKeepForAllSavedOrGC
	// Keep MPCookNextAssignmentIndex; it is allowed in the uninitialized state
	// InitializeStatus was modified above
	// Keep DoesGeneratedRequireGeneratorValue; it is allowed in the uninitialized state
	// Keep bUseInternalReferenceToAvoidGarbageCollect; it is allowed in the uninitialized state
	// Keep bRequiresGeneratorPackageDestructBeforeResplit; it is allowed in the uninitialized state
	bReportedManifest = false;
	bSaveGenerator = true;
	// Keep bSkipSaveExistingGenerator; we need to preserve it across the uninitialized state
	bCurrentGCHasKeptGeneratorPackage = false;
	bCurrentGCHasKeptGeneratorKeepPackages = false;
	// Keep bKeepForAllSavedOrGC ; it is allowed in the uninitialized state
	// Keep bKeepForCompletedAllSavesMessage; it is allowed in the uninitialized state
	// Keep bNeedConfirmGeneratorPackageDestroyed; it is allowed in the uninitialized state
	// Keep bHasFinishedQueueGeneratedPackages; we need to preserve it across the uninitialized state
	// Keep bQueuedGeneratedPackagesFencePassed; we need to preserve it across the uninitialized state
	// Keep bDeferEvents; we need to preserve it across the uninitialized state
	// Keep bSentAllSavesCompleted; it is allowed in the uninitialized state
}

void FGenerationHelper::ModifyNumSaved(const ITargetPlatform* TargetPlatform, int32 Delta)
{
	FPlatformData& TargetPlatformData = FindCheckedPlatformData(TargetPlatform);
	TargetPlatformData.NumSaved += Delta;
	int32 NumAllSaved = PackagesToGenerate.Num() + 1;
	check(0 <= TargetPlatformData.NumSaved && TargetPlatformData.NumSaved <= NumAllSaved);
	if (TargetPlatformData.NumSaved != NumAllSaved)
	{
		return;
	}
	OnNumSavedUpdated();
}

void FGenerationHelper::OnNumSavedUpdated()
{
	if (bDeferEvents || bSentAllSavesCompleted)
	{
		return;
	}

	int32 NumAllSaved = PackagesToGenerate.Num() + 1;
	int32 NumSaved = 0;

	// Save events are combined for all platforms; early exit for now if any platform is incomplete.
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		FPlatformData& PlatformData = PlatformPair.Value;
		check(0 <= PlatformData.NumSaved && PlatformData.NumSaved <= NumAllSaved);
		if (PlatformData.NumSaved != NumAllSaved)
		{
			return;
		}
		NumSaved = PlatformData.NumSaved;
	}

	UCookOnTheFlyServer& COTFS = GetOwner().GetPackageDatas().GetCookOnTheFlyServer();
	// Only send OnAllSavesCompleted from director; clients have incomplete information and
	// could send it spuriously.
	// Additionally, only send it if we have completed queueing, to avoid sending it prematurely.
	// ModifyNumSaved(1 == 1) will occur when the generator package is incrementally skipped,
	// and ModifyNumSaved(k == k, k < expectednumber) can occur if we save some generated packages
	// (or mark them incrementally skippable) before getting the full list of packages from the worker
	// that called QueueGeneratedPackages.
	if (!COTFS.CookWorkerClient && !bHasFinishedQueueGeneratedPackages)
	{
		return;
	}

	FName PackageName = GetOwner().GetPackageName();
	bSentAllSavesCompleted = true;
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: All saves completed (%d/%d).",
		*WriteToString<256>(PackageName), NumSaved, NumAllSaved);
	if (!COTFS.CookWorkerClient)
	{
		if (COTFS.CookDirector)
		{
			FGeneratorEventMessage Message(EGeneratorEvent::AllSavesCompleted, PackageName);
			COTFS.CookDirector->BroadcastMessage(MoveTemp(Message), ECookBroadcastTiming::AfterAssignPackages);
		}
		OnAllSavesCompleted(COTFS);
	}
}

FGenerationHelper::FScopeDeferEvents::FScopeDeferEvents(const TRefCountPtr<FGenerationHelper>& InGenerationHelper)
	: GenerationHelper(InGenerationHelper)
{
	check(GenerationHelper);
	bOldDeferEvents = GenerationHelper->bDeferEvents;
	GenerationHelper->bDeferEvents = true;
}

FGenerationHelper::FScopeDeferEvents::~FScopeDeferEvents()
{
	GenerationHelper->bDeferEvents = bOldDeferEvents;
	if (!bOldDeferEvents)
	{
		GenerationHelper->OnNumSavedUpdated();
	}
}

void FGenerationHelper::OnAllSavesCompleted(UCookOnTheFlyServer& COTFS)
{
	// Caller is responsible for holding a reference that keeps *this from destructing if it clears
	// these references
	ClearKeepForCompletedAllSavesMessage();
	ClearKeepForAllSavedOrGC();

	// Demote stalled packages; we will no longer need to come back to them
	DemoteStalledPackages(COTFS, true /* bFromAllSavesCompleted */);
}

void FGenerationHelper::DemoteStalledPackages(UCookOnTheFlyServer& COTFS, bool bFromAllSavesCompleted)
{
	// For any packages that we stalled because they were retracted and assigned to another worker,
	// demote them now. But don't demote non-stalled packages, because doing so could demote the final
	// package that we just saved locally and still needs to finish its work in PumpSaves.
	auto ConditionalDemote = [&COTFS, &bFromAllSavesCompleted, this](FCookGenerationInfo& Info)
		{
			if (Info.PackageData->IsStalled())
			{
				FGenerationHelper::ValidateSaveStalledState(COTFS, *Info.PackageData, TEXT("DemoteStalledPackages"));
				if (Info.PackageData->GetState() == EPackageState::SaveStalledAssignedToWorker)
				{
					// If called from OnAllSavesCompleted on the CookDirector, then we should have no stalled packages;
					// they all should have been unstalled and demoted when saved. If we do have any, then log an error
					// and demote them to idle.
					if (bFromAllSavesCompleted)
					{
						int32 MinNumSaved = PackagesToGenerate.Num() + 1;
						for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
						{
							MinNumSaved = FMath::Min(MinNumSaved, PlatformPair.Value.NumSaved);
						}
						UE_LOGF(LogCook, Error,
							"Package %ls is still stalled on the CookDirector during FGenerationHelper::OnAllSavesCompleted. " "This is unexpected; all stalled packages on a GenerationHelper should have completed saving and therefore unstalled before OnAllSavesCompleted is called." "\n\tNumAllSaved == %d. NumSaved == %d. Info.HasSaved == %ls.",
							*Info.PackageData->GetPackageName().ToString(), PackagesToGenerate.Num() + 1, MinNumSaved,
							Info.HasSavedEveryPlatform() ? TEXT("true") : TEXT("false"));
						FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);

						COTFS.DemoteToIdle(*Info.PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::RetractedByCookDirector);
					}
					else
					{
						// Otherwise, when called on the CookDirector, demote them out of save but keep them in the assigned to worker state.
						Info.PackageData->SendToState(EPackageState::AssignedToWorker, ESendFlags::QueueAddAndRemove, EStateChangeReason::GarbageCollected);
						UE_LOGF(LogCookGenerationHelper, Verbose,
							"%ls generated package %ls: DemoteStalledPackage from SaveStalledAssignedToWorker to AssignedToWorker.",
							*WriteToString<256>(GetOwner().GetPackageName()),
							*WriteToString<256>(Info.PackageData->GetPackageName()));
					}
				}
				else
				{
					// Demoting stalled packages on a Client just returns them to Idle.
					UE_LOGF(LogCookGenerationHelper, Verbose, "%ls generated package %ls: DemoteStalledPackage from %ls to Idle.",
						*WriteToString<256>(GetOwner().GetPackageName()),
						*WriteToString<256>(Info.PackageData->GetPackageName()),
						LexToString(Info.PackageData->GetState()));
					COTFS.DemoteToIdle(*Info.PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::RetractedByCookDirector);
				}
			}
		};
	ConditionalDemote(OwnerInfo);
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		ConditionalDemote(Info);
	}
}

void FGenerationHelper::ValidateSaveStalledState(UCookOnTheFlyServer& COTFS, FPackageData& PackageData, const TCHAR* Caller)
{
	if (!PackageData.IsStalled())
	{
		return;
	}
	EPackageState ExpectedState = COTFS.CookDirector.IsValid() ? EPackageState::SaveStalledAssignedToWorker : EPackageState::SaveStalledRetracted;
	if (PackageData.GetState() != ExpectedState)
	{
		const TCHAR* Which = COTFS.CookDirector.IsValid() ? TEXT("the CookDirector") : TEXT("a CookWorker");
		UE_LOGF(LogCook, Error,
			"In %ls, package %ls is stalled on %ls, but is in state %ls. We expect stalled packages on %ls to be in state %ls.",
			Caller,
			*PackageData.GetPackageName().ToString(), Which, LexToString(PackageData.GetState()),
			Which, LexToString(ExpectedState));
		PackageData.SendToState(ExpectedState, ESendFlags::QueueNone, EStateChangeReason::Retraction);
	}
}

void FGenerationHelper::DiagnoseWhyNotShutdown()
{
	TStringBuilder<256> Lines;
	int32 MinNumSaved = PackagesToGenerate.Num() + 1;
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		MinNumSaved = FMath::Min(MinNumSaved, PlatformPair.Value.NumSaved);
	}

	int32 ExpectedNumSaved = PackagesToGenerate.Num() + 1;
	if (MinNumSaved != ExpectedNumSaved)
	{
		Lines.Appendf(TEXT("\tNumSaved == %d, ExpectedNumSaved == %d.\n"), MinNumSaved, ExpectedNumSaved);
	}
	UCookOnTheFlyServer& COTFS = GetOwner().GetPackageDatas().GetCookOnTheFlyServer();
	uint32 ExpectedRefCount = 1;
	auto TestInfo = [this, &Lines, &COTFS, &ExpectedRefCount](FCookGenerationInfo& Info)
		{
			if (Info.PackageData->GetState() != EPackageState::Idle)
			{
				Lines.Appendf(TEXT("\t%s%s is not idle; it is in state %d.\n"),
					Info.IsGenerator() ? TEXT("OwnerInfo") : TEXT("GeneratedPackage "),
					Info.IsGenerator() ? TEXT("") : *Info.GetPackageName(),
					static_cast<int32>(Info.PackageData->GetState()));
			}
			else
			{
				TArray< const ITargetPlatform*> MissingPlatforms;
				for (const ITargetPlatform* TargetPlatform : COTFS.PlatformManager->GetSessionPlatforms())
				{
					const FPackagePlatformData* PlatformData = Info.PackageData->GetPlatformDatas().Find(TargetPlatform);
					if (!PlatformData || PlatformData->GetCookResults() == ECookResult::NotAttempted)
					{
						MissingPlatforms.Add(TargetPlatform);
					}
				}
				if (!MissingPlatforms.IsEmpty())
				{
					TStringBuilder<256> MissingPlatformStr;
					if (MissingPlatforms.Num() != COTFS.PlatformManager->GetSessionPlatforms().Num())
					{
						MissingPlatformStr << TEXT(" for platforms { ");
						for (const ITargetPlatform* TargetPlatform : MissingPlatforms)
						{
							MissingPlatformStr << TargetPlatform->PlatformName() << TEXT(", ");
						}
						MissingPlatformStr.RemoveSuffix(2);
						MissingPlatformStr << TEXT(" }");
					}

					Lines.Appendf(TEXT("\t%s%s was not cooked%s. SuppressCookReason == %s.\n"),
						Info.IsGenerator() ? TEXT("OwnerInfo") : TEXT("GeneratedPackage "),
						Info.IsGenerator() ? TEXT("") : *Info.GetPackageName(),
						*MissingPlatformStr,
						LexToString(Info.PackageData->GetSuppressCookReason()));
				}
			}
			if (!Info.HasSavedEveryPlatform())
			{
				Lines.Appendf(TEXT("\t%s%s has not marked saved.\n"),
					Info.IsGenerator() ? TEXT("OwnerInfo") : TEXT("GeneratedPackage "),
					Info.IsGenerator() ? TEXT("") : *Info.GetPackageName());
			}
			if (!Info.IsGenerator() && Info.PackageData->GetParentGenerationHelper())
			{
				Lines.Appendf(TEXT("\tGeneratedPackage %s has ParentGenerationHelper set.\n"), *Info.GetPackageName());
				++ExpectedRefCount;
			}
		};
	TestInfo(GetOwnerInfo());
	// Do not call GetPackagesToGenerate as that would initialize.
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		TestInfo(Info);
	}

	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		FPlatformData& PlatformData = PlatformPair.Value;
		if (PlatformData.ReferenceFromKeepForIncremental)
		{
			Lines.Appendf(TEXT("\tReferenceFromKeepForIncremental is set for platform %s.\n"),
				*PlatformPair.Key->PlatformName());
			++ExpectedRefCount;
		}
		if (PlatformData.ReferenceFromKeepForGeneratorSave)
		{
			Lines.Appendf(TEXT("\tReferenceFromKeepForGeneratorSave is set for platform %s.\n"),
				*PlatformPair.Key->PlatformName());
			++ExpectedRefCount;
		}
	}
	if (ReferenceFromKeepForQueueResults)
	{
		Lines.Append(TEXT("\tReferenceFromKeepForQueueResults is set.\n"));
		++ExpectedRefCount;
	}
	if (bKeepForAllSavedOrGC)
	{
		Lines.Append(TEXT("\tbKeepForAllSavedOrGC is true.\n"));
	}
	if (bKeepForCompletedAllSavesMessage)
	{
		Lines.Append(TEXT("\tbKeepForCompletedAllSavesMessage is true.\n"));
	}
	if (ReferenceFromKeepForAllSavedOrGC)
	{
		if (!bKeepForAllSavedOrGC && !bKeepForCompletedAllSavesMessage)
		{
			Lines.Append(TEXT("\tReferenceFromKeepForAllSavedOrGC is set, despite bKeepForAllSavedOrGC and bKeepForCompletedAllSavesMessage being false.\n"));
		}
		++ExpectedRefCount;
	}
	if (GetRefCount() > ExpectedRefCount)
	{
		GetOwner().GetPackageDatas().LockAndEnumeratePackageDatas(
			[this, &ExpectedRefCount, &Lines](FPackageData* PackageData)
			{
				if (PackageData->GetParentGenerationHelper().GetReference() == this &&
					FindInfo(*PackageData) == nullptr)
				{
					Lines.Appendf(TEXT("\tGenerated package %s has ParentGenerationHelper set, but is not listed as a PackageToGenerate from the GenerationHelper.\n"),
						*PackageData->GetPackageName().ToString());
					++ExpectedRefCount;
				}
			});
	}
	if (GetRefCount() > ExpectedRefCount)
	{
		Lines.Appendf(TEXT("\tGetRefCount() has references from unknown sources. GetRefCount() == %u, ExpectedRefCount == %u.\n"),
			GetRefCount(), ExpectedRefCount);
	}

	if (Lines.Len() != 0)
	{
		FWorkerId WorkerId = GetWorkerIdThatSavedGenerator();
		Lines.Appendf(TEXT("\tGenerator: Saved on %s.\n"), *GetOwnerInfo().SavedOnWorker.ToString());
		for (FCookGenerationInfo& Info : PackagesToGenerate)
		{
			Lines.Appendf(TEXT("\tGeneratedPackage %s: Saved on %s.\n"), *Info.GetPackageName(),
				*Info.SavedOnWorker.ToString());
		}
	}
	else
	{
		Lines.Appendf(TEXT("\tDiagnoseWhyNotShutdown was called unexpectedly; GetRefCount() == 1 so this GenerationHelper should be shut down.\n"));
	}
	if (Lines.ToView().EndsWith(TEXT("\n")))
	{
		Lines.RemoveSuffix(1);
	}

	FString Message = FString::Printf(
		TEXT("GenerationHelper for package %s is still allocated%s at end of cooksession. This is unexpected and could indicate some generated packages are missing."),
		*GetOwner().GetPackageName().ToString(), IsInitialized() ? TEXT(" and initialized") : TEXT(""));

	if (IsInitialized())
	{
		UE_LOGF(LogCook, Error, "%ls", *Message);
	}
	else
	{
		UE_LOGF(LogCook, Warning, "%ls", *Message);
	}
	UE_LOGF(LogCook, Display, "Diagnostics:\n%ls", *Lines);
}

void FGenerationHelper::ForceUninitialize()
{
	TArray<FPackageData*> PackagesToDemote;
	auto TestInfo = [&PackagesToDemote](FCookGenerationInfo& Info)
		{
			if (Info.PackageData->GetState() != EPackageState::Idle)
			{
				PackagesToDemote.Add(Info.PackageData);
			}
		};
	TestInfo(GetOwnerInfo());
	for (FCookGenerationInfo& Info : GetPackagesToGenerate())
	{
		TestInfo(Info);
	}

	UCookOnTheFlyServer& COTFS = GetOwner().GetPackageDatas().GetCookOnTheFlyServer();
	for (FPackageData* PackageData : PackagesToDemote)
	{
		COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove, ESuppressCookReason::CookCanceled);
	}
	Uninitialize();
}

UPackage* FGenerationHelper::FindOrLoadPackage(UCookOnTheFlyServer& COTFS, FPackageData& OwnerPackageData)
{
	// This is the static helper function on FGenerationHelper that loads the package for any FPackageData; for the
	// member variable function that uses the cached pointer, see FindOrLoadOwnerPackage.
	FName OwnerPackageName = OwnerPackageData.GetPackageName();
	UPackage* Result = FindObjectFast<UPackage>(nullptr, OwnerPackageName);

	if (!Result || !Result->IsFullyLoaded())
	{
		COTFS.LoadPackageForCooking(OwnerPackageData, Result);
		if (!Result || !Result->IsFullyLoaded())
		{
			return nullptr;
		}
	}
	return Result;
}

void FGenerationHelper::SearchForRegisteredSplitDataObject(UCookOnTheFlyServer& COTFS,
	FName PackageName, UPackage* Package,
	TOptional<TConstArrayView<FCachedObjectInOuter>> CachedObjectsInOuter,
	UObject*& OutSplitDataObject, UE::Cook::Private::FRegisteredCookPackageSplitter*& OutRegisteredSplitter,
	TUniquePtr<ICookPackageSplitter>& OutSplitterInstance, bool bCookedPlatformDataIsLoaded,
	bool& bOutNeedWaitForIsLoaded)
{
	bOutNeedWaitForIsLoaded = false;
	OutSplitDataObject = nullptr;
	OutRegisteredSplitter = nullptr;
	OutSplitterInstance = nullptr;
	check(Package != nullptr || CachedObjectsInOuter.IsSet());

	UObject* LocalSplitDataObject = nullptr;
	Private::FRegisteredCookPackageSplitter* SplitterType = nullptr;
	TArray<Private::FRegisteredCookPackageSplitter*> FoundRegisteredSplitters;
	auto TryLookForSplitterOfObject =
		[&COTFS, PackageName, &FoundRegisteredSplitters, &SplitterType, &LocalSplitDataObject,
		bCookedPlatformDataIsLoaded, &bOutNeedWaitForIsLoaded](UObject* Obj)
		{
			FoundRegisteredSplitters.Reset();
			COTFS.RegisteredSplitDataClasses.MultiFind(Obj->GetClass(), FoundRegisteredSplitters);

			for (Private::FRegisteredCookPackageSplitter* SplitterForObject : FoundRegisteredSplitters)
			{
				if (!SplitterForObject)
				{
					continue;
				}
				if (SplitterForObject->RequiresCachedCookedPlatformDataBeforeSplit() && !bCookedPlatformDataIsLoaded)
				{
					bOutNeedWaitForIsLoaded = true;
					return false;
				}
				if (SplitterForObject && SplitterForObject->ShouldSplitPackage(Obj))
				{
					if (!Obj->HasAnyFlags(RF_Public))
					{
						UE_LOGF(LogCook, Error,
							"SplitterData object %ls must be publicly referenceable so we can keep them from being garbage collected",
							*Obj->GetFullName());
						return false;
					}

					if (SplitterType)
					{
						UE_LOGF(LogCook, Error,
							"Found more than one registered Cook Package Splitter for package %ls.",
							*PackageName.ToString());
						return false;
					}

					SplitterType = SplitterForObject;
					LocalSplitDataObject = Obj;
				}
			}
			return true;
		};

	if (CachedObjectsInOuter.IsSet())
	{
		// CachedObjectsInOuter might be set but empty for e.g. a generated package that has not been populated
		for (const FCachedObjectInOuter& CachedObjectInOuter : *CachedObjectsInOuter)
		{
			UObject* Obj = CachedObjectInOuter.Object.Get();
			if (!Obj)
			{
				continue;
			}
			if (!TryLookForSplitterOfObject(Obj))
			{
				return; // unable to complete the search, exit the entire search function
			}
		}
	}
	else
	{
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithOuter(Package, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects,
			RF_NoFlags, EInternalObjectFlags::Garbage);
		for (UObject* Obj : ObjectsInPackage)
		{
			if (!TryLookForSplitterOfObject(Obj))
			{
				return; // unable to complete the search, exit the entire search function
			}
		}
	}

	if (!SplitterType)
	{
		return;
	}

	// Create instance of CookPackageSplitter class
	ICookPackageSplitter* SplitterInstance = SplitterType->CreateInstance(Package, LocalSplitDataObject);
	if (!SplitterInstance)
	{
		UE_LOGF(LogCook, Error, "Error instantiating Cook Package Splitter %ls for object %ls.",
			*SplitterType->GetSplitterDebugName(), *LocalSplitDataObject->GetFullName());
		return;
	}

	OutSplitDataObject = LocalSplitDataObject;
	OutRegisteredSplitter = SplitterType;
	OutSplitterInstance.Reset(SplitterInstance);
}

void FGenerationHelper::ClearSelfReferences()
{
	// Any references we release might be the last reference and cause *this to be deleted,
	// so create a local reference to keep it alive until the end of the function.
	TRefCountPtr<FGenerationHelper> LocalRef(this);
	ClearKeepForIncrementalAllPlatforms();
	ClearKeepForGeneratorSaveAllPlatforms();
	ClearKeepForQueueResults();
	ClearKeepForAllSavedOrGC();
	ClearKeepForCompletedAllSavesMessage();
}

FCookGenerationInfo* FGenerationHelper::FindInfo(const FPackageData& PackageData)
{
	ConditionalInitialize();
	return FindInfoNoInitialize(PackageData);
}

FCookGenerationInfo* FGenerationHelper::FindInfoNoInitialize(const FPackageData& PackageData)
{
	if (&PackageData == &GetOwner())
	{
		return &OwnerInfo;
	}
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		if (Info.PackageData == &PackageData)
		{
			return &Info;
		}
	}
	return nullptr;
}

FCookGenerationInfo* FGenerationHelper::FindInfoNoInitialize(FName PackageName)
{
	if (PackageName == GetOwner().GetPackageName())
	{
		return &OwnerInfo;
	}
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		if (Info.PackageData->GetPackageName() == PackageName)
		{
			return &Info;
		}
	}
	return nullptr;
}

const FCookGenerationInfo* FGenerationHelper::FindInfo(const FPackageData& PackageData) const
{
	return const_cast<FGenerationHelper*>(this)->FindInfo(PackageData);
}

UObject* FGenerationHelper::GetWeakSplitDataObject()
{
	ConditionalInitialize();
	UObject* Result = SplitDataObject.Get();
	if (Result)
	{
		return Result;
	}

	FString ObjectPath = GetSplitDataObjectName().ToString();
	// SplitDataObjectName is a FullObjectPath; strip off the leading <ClassName> in
	// "<ClassName> <Package>.<Object>:<SubObject>"
	int32 ClassDelimiterIndex = -1;
	if (ObjectPath.FindChar(' ', ClassDelimiterIndex))
	{
		ObjectPath.RightChopInline(ClassDelimiterIndex + 1);
	}

	Result = FindObject<UObject>(nullptr, *ObjectPath);
	if (Result)
	{
		SplitDataObject = Result;
	}
	return Result;
}

UObject* FGenerationHelper::FindOrLoadSplitDataObject()
{
	if (!IsValid())
	{
		return nullptr;
	}
	UObject* Result = GetWeakSplitDataObject();
	if (Result)
	{
		return Result;
	}

	FPackageData& OwnerPackageData = GetOwner();
	FName OwnerPackageName = OwnerPackageData.GetPackageName();
	UCookOnTheFlyServer& COTFS = OwnerPackageData.GetPackageDatas().GetCookOnTheFlyServer();
	UPackage* LocalOwnerPackage;
	COTFS.LoadPackageForCooking(OwnerPackageData, LocalOwnerPackage);

	return GetWeakSplitDataObject();
}

UPackage* FGenerationHelper::GetOwnerPackage()
{
	UPackage* Result = OwnerPackage.Get();
	if (!Result && !OwnerPackage.GetEvenIfUnreachable())
	{
		OwnerPackage = FindObjectFast<UPackage>(nullptr, GetOwner().GetPackageName());
		Result = OwnerPackage.Get();
	}
	return Result;
}

UPackage* FGenerationHelper::FindOrLoadOwnerPackage(UCookOnTheFlyServer& COTFS)
{
	UPackage* Result = GetOwnerPackage();
	if (!Result)
	{
		Result = FindOrLoadPackage(COTFS, GetOwner());
	}
	return Result;
}

bool FGenerationHelper::TryReportGenerationManifest()
{
	if (bReportedManifest)
	{
		return true;
	}
	FPackageData& OwnerPackageData = GetOwner();
	FName OwnerPackageName = OwnerPackageData.GetPackageName();
	if (!IsValid())
	{
		// Unexpected, caller should not call in this case
		UE_LOGF(LogCook, Error,
			"TryReportGenerationManifest failed for package %ls: Called on an invalid FGenerationHelper.",
			*OwnerPackageName.ToString());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return false;
	}

	FPackageDatas& PackageDatas = OwnerPackageData.GetPackageDatas();
	UCookOnTheFlyServer& COTFS = PackageDatas.GetCookOnTheFlyServer();
	UObject* OwnerObject = FindOrLoadSplitDataObject();
	if (!OwnerObject)
	{
		// Unexpected, we found it earlier when we marked valid.
		UE_LOGF(LogCook, Error,
			"TryReportGenerationManifest failed for package %ls: Valid GenerationHelper but could not find OwnerObject.",
			*OwnerPackageName.ToString());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return false;
	}

	UPackage* LocalOwnerPackage = OwnerObject->GetPackage();

	ICookPackageSplitter::FGenerationManifest Manifest;
	{
		UE::Cook::FScopedCookActivePackage ScopedActivePackage(OwnerPackageName,
#if UE_WITH_OBJECT_HANDLE_TRACKING
			PackageAccessTrackingOps::NAME_CookerBuildObject
#else
			FName()
#endif
		);
		TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

		Manifest = GetCookPackageSplitterInstance()->ReportGenerationManifest(LocalOwnerPackage, OwnerObject);
		if (Manifest.GeneratedPackages.IsEmpty())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS;
			TArray<ICookPackageSplitter::FGeneratedPackage> GeneratorDatas;
			GeneratorDatas = GetCookPackageSplitterInstance()->GetGenerateList(LocalOwnerPackage, OwnerObject);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS;
			if (!GeneratorDatas.IsEmpty())
			{
				// Deprecated in UE 5.7
				UE_LOGF(LogCook, Warning,
					"%ls is overriding deprecated ICookPackageSplitter::GetGenerateList. Implement ReportGenerationManifest instead. - Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.",
					*GetRegisteredSplitterType()->GetSplitterDebugName());
				Manifest.GeneratedPackages = MoveTemp(GeneratorDatas);
			}
		}
	}

	bSaveGenerator = Manifest.bCookSaveGeneratorPackage && !bSkipSaveExistingGenerator;

	TMap<FPackageData*, int32> AlreadyExistingInfoPackageToIndex;
	int32 NumAlreadyExisting = PackagesToGenerate.Num();
	for (int32 ExistingIndex = 0; ExistingIndex < PackagesToGenerate.Num(); ++ExistingIndex)
	{
		AlreadyExistingInfoPackageToIndex.Add(PackagesToGenerate[ExistingIndex].PackageData, ExistingIndex);
	}
	PackagesToGenerate.Reserve(Manifest.GeneratedPackages.Num());

	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		PlatformPair.Value.NumSaved = 0;
	}
	for (ICookPackageSplitter::FGeneratedPackage& SplitterData : Manifest.GeneratedPackages)
	{
		if (!SplitterData.GetCreateAsMap().IsSet())
		{
			UE_LOGF(LogCook, Error,
				"PackageSplitter did not specify whether CreateAsMap is true for generated package. Splitter=%ls, Generated=%ls.",
				*this->GetSplitDataObjectName().ToString(), *OwnerPackageName.ToString());
			return false;
		}
		bool bCreateAsMap = *SplitterData.GetCreateAsMap();

		FString PackageName = ICookPackageSplitter::ConstructGeneratedPackageName(OwnerPackageName,
			SplitterData.RelativePath, SplitterData.GeneratedRootPath);
		const FName PackageFName(*PackageName);
		FPackageData* PackageData = PackageDatas.TryAddPackageDataByPackageName(PackageFName,
			false /* bRequireExists */, bCreateAsMap);
		if (!PackageData)
		{
			UE_LOGF(LogCook, Error,
				"PackageSplitter could not find mounted filename for generated packagepath. Splitter=%ls, Generated=%ls.",
				*this->GetSplitDataObjectName().ToString(), *PackageName);
			return false;
		}
		// No package should be generated by two different splitters.
		check(PackageData->GetParentGenerator().IsNone() ||
			PackageData->GetParentGenerator() == OwnerPackageName);
		PackageData->SetGenerated(OwnerPackageName);
		PackageData->SetDoesGeneratedRequireGenerator(DoesGeneratedRequireGeneratorValue);
		if (IFileManager::Get().FileExists(*PackageData->GetFileName().ToString()))
		{
			UE_LOGF(LogCook, Warning,
				"PackageSplitter specified a generated package that already exists in the workspace domain. Splitter=%ls, Generated=%ls.",
				*this->GetSplitDataObjectName().ToString(), *PackageName);
			return false;
		}

		FCookGenerationInfo* GeneratedInfo = nullptr;
		if (!AlreadyExistingInfoPackageToIndex.IsEmpty())
		{
			int32 ExistingIndex;
			if (AlreadyExistingInfoPackageToIndex.RemoveAndCopyValue(PackageData, ExistingIndex))
			{
				GeneratedInfo = &PackagesToGenerate[ExistingIndex];
			}
		}
		if (!GeneratedInfo)
		{
			GeneratedInfo = &PackagesToGenerate.Emplace_GetRef(*this, *PackageData, false /* bInGenerator */);
		}
		GeneratedInfo->RelativePath = MoveTemp(SplitterData.RelativePath);
		GeneratedInfo->GeneratedRootPath = MoveTemp(SplitterData.GeneratedRootPath);
		GeneratedInfo->PackageDependencies = MoveTemp(SplitterData.PackageDependencies);
		for (TArray<FAssetDependency>::TIterator Iter(GeneratedInfo->PackageDependencies); Iter; ++Iter)
		{
			if (Iter->Category != UE::AssetRegistry::EDependencyCategory::Package)
			{
				UE_LOGF(LogCook, Error,
					"PackageSplitter specified a dependency with category %d rather than category Package. Dependency will be ignored. Splitter=%ls, Generated=%ls.",
					(int32)Iter->Category, *this->GetSplitDataObjectName().ToString(), *PackageName);
				Iter.RemoveCurrent();
			}
			TStringBuilder<256> DependencyPackageName(InPlace, Iter->AssetId.PackageName);
			if (ICookPackageSplitter::IsUnderGeneratedPackageSubPath(DependencyPackageName))
			{
				UE_LOGF(LogCook, Error,
					"PackageSplitter specified a dependency for one generated package on another generated package. Only dependencies on non-generated packages are allowed. Dependency will be ignored. Splitter=%ls, Generated=%ls, Dependency=%ls.",
					*this->GetSplitDataObjectName().ToString(), *PackageName, *DependencyPackageName);
				Iter.RemoveCurrent();
			}
		}
		Algo::Sort(GeneratedInfo->PackageDependencies,
			[](const FAssetDependency& A, const FAssetDependency& B) { return A.LexicalLess(B); });
		GeneratedInfo->PackageDependencies.SetNum(Algo::Unique(GeneratedInfo->PackageDependencies));
		GeneratedInfo->SetIsCreateAsMap(bCreateAsMap);
		if (DoesGeneratedRequireGenerator() >= ICookPackageSplitter::EGeneratedRequiresGenerator::Save ||
			COTFS.MPCookGeneratorSplit == EMPCookGeneratorSplit::AllOnSameWorker)
		{
			PackageData->SetWorkerAssignmentConstraint(FWorkerId::Local());
		}

		// Copy hash from package splitter so it can partcipate in the package hash
		GeneratedInfo->GenerationHash = SplitterData.GenerationHash;

		GeneratedInfo->SetCopyChunkAssignmentFromGenerator(SplitterData.bCopyChunkAssignmentFromGenerator);

		// Create the Hash from the GenerationHash and Dependencies
		GeneratedInfo->CreatePackageHash();

		for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
		{
			FPlatformData& GeneratorPlatformData = PlatformPair.Value;
			FCookGenerationInfo::FPlatformData& InfoPlatformData = GeneratedInfo->FindCheckedPlatformData(PlatformPair.Key);
			GeneratorPlatformData.NumSaved += InfoPlatformData.HasSaved() ? 1 : 0;
		}
	}

	int32 MinNumSaved = PackagesToGenerate.Num() + 1;
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		FPlatformData& GeneratorPlatformData = PlatformPair.Value;
		FCookGenerationInfo::FPlatformData& InfoPlatformData = OwnerInfo.FindCheckedPlatformData(PlatformPair.Key);
		GeneratorPlatformData.NumSaved += InfoPlatformData.HasSaved() ? 1 : 0;
		MinNumSaved = FMath::Min(MinNumSaved, GeneratorPlatformData.NumSaved);
	}

	if (!AlreadyExistingInfoPackageToIndex.IsEmpty())
	{
		TArray<int32> UnusedExistingIndexes;
		for (TPair<FPackageData*, int32>& Pair : AlreadyExistingInfoPackageToIndex)
		{
			UnusedExistingIndexes.Add(Pair.Value);
			UE_LOGF(LogCook, Warning,
				"Unexpected generated package (local TryReportGenerationManifest). A remote cookworker reported generated package %ls for generator %ls," " but when TryReportGenerationManifest was called on the CookDirector, this package was not listed." " This is unexpected and causes minor performance problems in the cook.",
				*Pair.Key->GetPackageName().ToString(), *OwnerPackageData.GetPackageName().ToString());
		}
		Algo::Sort(UnusedExistingIndexes);
		for (int32 UnusedIndex : ReverseIterate(UnusedExistingIndexes))
		{
			PackagesToGenerate.RemoveAt(UnusedIndex);
		}
	}
	UE_LOGF(LogCookGenerationHelper, Verbose,
		"%ls: TryReportGenerationManifest: %d packages to generate, %d previously recorded, %d/%d already saved.",
		*WriteToString<256>(GetOwner().GetPackageName()), PackagesToGenerate.Num(),
		NumAlreadyExisting, MinNumSaved, PackagesToGenerate.Num() + 1);

	OnNumSavedUpdated();

	bReportedManifest = true;
	return true;
}

UObject* GetObjectFromObjectHolder(UObject*const& ObjectHolder)
{
	return ObjectHolder;
}
UObject* GetObjectFromObjectHolder(const FCachedObjectInOuter& ObjectHolder)
{
	return ObjectHolder.Object.Get();
}
template <typename ObjectHolderType>
TMap<UObject*, UPackage*> GetPackagesForObjects(TConstArrayView<ObjectHolderType> Objects)
{
	TMap<UObject*, UPackage*> PackageForObject;
	PackageForObject.Reserve(Objects.Num());
	TArray<UObject*, TInlineAllocator<10>> ChildStack;
	for (const ObjectHolderType& ObjectHolder : Objects)
	{
		check(ChildStack.IsEmpty());
		UObject* Object = GetObjectFromObjectHolder(ObjectHolder);
		if (!Object)
		{
			continue;
		}
		UPackage* Package = nullptr;
		for (;;)
		{
			UPackage** PackagePtr = PackageForObject.Find(Object);
			Package = PackagePtr ? *PackagePtr : nullptr;
			if (Package)
			{
				break;
			}

			Package = Cast<UPackage>(Object);
			if (Package)
			{
				PackageForObject.Add(Object, Package);
				break;
			}

			Package = Object->GetExternalPackage();
			if (Package)
			{
				PackageForObject.Add(Object, Package);
				break;
			}

			UObject* Outer = Object->GetOuter();
			if (!Outer)
			{
				Package = GetTransientPackage();
				PackageForObject.Add(Object, Package);
				break;
			}
			ChildStack.Add(Object);
			Object = Outer;
		}
		check(Package);
		for (UObject* ChildObject : ChildStack)
		{
			PackageForObject.Add(ChildObject, Package);
		}
		ChildStack.Reset();
	}
	return PackageForObject;
}

bool FGenerationHelper::TryCallPopulateGeneratorPackage(
	TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& InOutGeneratedPackagesForPopulate,
	bool bPopulateCallAllowed)
{
	if (OwnerInfo.HasCalledPopulate())
	{
		return true;
	}
	FPackageData& OwnerPackageData = GetOwner();
	FName OwnerPackageName = OwnerPackageData.GetPackageName();
	UCookOnTheFlyServer& COTFS = OwnerPackageData.GetPackageDatas().GetCookOnTheFlyServer();
	if (!bReportedManifest)
	{
		// Unexpected, caller should not call in this case
		UE_LOGF(LogCook, Error,
			"TryCallPopulateGeneratorPackage called for package %ls without a previous successful call to TryReportGenerationManifest.",
			*OwnerPackageName.ToString());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return false;
	}
	check(IsValid()); // Could not have set bReportedManifest=true without being valid. 
	UObject* LocalSplitDataObject = this->FindOrLoadSplitDataObject();
	if (!LocalSplitDataObject)
	{
		UE_LOGF(LogCook, Error,
			"Failed to call PopulateGeneratorPackage, CookPackageSplitter missing. Splitter=%ls",
			*GetSplitDataObjectName().ToString());
		return false;
	}
	UPackage* LocalOwnerPackage = LocalSplitDataObject->GetPackage();
	if (!COTFS.TryConstructGeneratedPackagesForPopulate(OwnerPackageData, *this, InOutGeneratedPackagesForPopulate))
	{
		UE_LOGF(LogCook, Error,
			"PackageSplitter unexpected failure: could not ConstructGeneratedPackagesForPreSave. Splitter=%ls",
			*GetSplitDataObjectName().ToString());
		return false;
	}

	// By Contract we do not call PopulateGeneratorPackage if not saving generator and not required by generated
	// packages, but we execute the rest of this function so that other GenerationHelper code can rely on the support
	// tasks for the Populate being done.
	UE::Cook::CookPackageSplitter::FPopulateContextData PopulateData;
	if (bPopulateCallAllowed)
	{
		UE::Cook::FScopedCookActivePackage ScopedActivePackage(OwnerPackageName,
#if UE_WITH_OBJECT_HANDLE_TRACKING
			PackageAccessTrackingOps::NAME_CookerBuildObject
#else
			FName()
#endif
		);
		TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

		// Get the list of Objects in the package. Read them from the PackageData if it already has them,
		// otherwise calculate them into a temporary buffer.
		TArray<FCachedObjectInOuter> ObjectsInOuterBuffer;
		TConstArrayView<FCachedObjectInOuter> ObjectsInOuter;
		if (OwnerPackageData.GetHasSaveCache())
		{
			ObjectsInOuter = OwnerPackageData.GetCachedObjectsInOuter();
		}
		else
		{
			OwnerPackageData.CreateObjectCache(ObjectsInOuterBuffer, LocalOwnerPackage);
			ObjectsInOuter = ObjectsInOuterBuffer;
		}

		ICookPackageSplitter::FPopulateContext PopulateContext(PopulateData);
		PopulateData.OwnerPackage = LocalOwnerPackage;
		PopulateData.OwnerObject = LocalSplitDataObject;
		PopulateData.GeneratedPackages = InOutGeneratedPackagesForPopulate;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		bool bPopulateSucceeded = CookPackageSplitterInstance->PopulateGeneratorPackage(PopulateData.OwnerPackage,
			PopulateData.OwnerObject, InOutGeneratedPackagesForPopulate, PopulateData.ObjectsToMove,
			PopulateData.KeepReferencedPackages);
		CookPackageSplitterInstance->WarnIfDeprecatedVirtualNotCalled(TEXT("PopulateGeneratorPackage"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
		bPopulateSucceeded = CookPackageSplitterInstance->PopulateGeneratorPackage(PopulateContext) && bPopulateSucceeded;

		if (!bPopulateSucceeded)
		{
			UE_LOGF(LogCook, Error, "CookPackageSplitter returned false from PopulateGeneratorPackage. Splitter=%ls",
				*GetSplitDataObjectName().ToString());
			return false;
		}
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: PopulateGeneratorPackage.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		// PopulateGenerator may have moved objects out of the package; detect them and add them to ObjectsToMove.
		bool bMovedObjects = false;
		TMap<UObject*, UPackage*> NewPackageForObject = GetPackagesForObjects(ObjectsInOuter);
		for (const TPair<UObject*, UPackage*>& PackagePair : NewPackageForObject)
		{
			UPackage* NewPackage = PackagePair.Value;
			check(NewPackage);
			if (NewPackage == LocalOwnerPackage)
			{
				continue;
			}
			FPackageData* NewPackageData = COTFS.PackageDatas->FindPackageDataByPackageName(NewPackage->GetFName());
			if (!NewPackageData)
			{
				// Object moved into a temporary or other package that the cooker does not track. The cooker will
				// treat it as still being in the package; SavePackage will make it an import if referenced.
				continue;
			}
			FCookGenerationInfo* Info = FindInfo(*NewPackageData);
			if (!Info)
			{
				// Object moved into another cookable package, but not a generated package of this generator.
				// During cook of the generator we will treat it as still being in the generator package; SavePackage
				// will make it an import if referenced.
				continue;
			}
			bMovedObjects = true;
			PopulateData.ObjectsToMove.Add(PackagePair.Key);
		}
		if (bMovedObjects)
		{
			// Enforce uniqueness on ObjectsToMove since we added without checking for duplicates
			PopulateData.ObjectsToMove.Sort();
			PopulateData.ObjectsToMove.SetNum(Algo::Unique(PopulateData.ObjectsToMove));
		}

		// If PopulateGenerator already moved any of the UObjects in the generator package out into generated packages,
		// keep those packages referenced until the generator package is destroyed, since we cannot reload
		// the generator package to recreate them until that happens. This behavior is only valid in the case where the
		// splitter has informed us !RequiresEmptyPackageBeforePopulate; otherwise it should not be accessing the
		// generated packages until their own populate calls.
		bMovedObjects = false;
		NewPackageForObject = GetPackagesForObjects<UObject*>(PopulateData.ObjectsToMove);
		for (const TPair<UObject*, UPackage*>& PackagePair : NewPackageForObject)
		{
			UPackage* NewPackage = PackagePair.Value;
			check(NewPackage);
			if (NewPackage == LocalOwnerPackage)
			{
				continue;
			}
			FPackageData* NewPackageData = COTFS.PackageDatas->FindPackageDataByPackageName(NewPackage->GetFName());
			if (!NewPackageData)
			{
				// Object moved into a temporary or other package that the cooker does not track. The cooker will
				// treat it as still being in the package; SavePackage will make it an import if referenced.
				continue;
			}
			FCookGenerationInfo* Info = FindInfo(*NewPackageData);
			if (!Info)
			{
				// Object moved into another cookable package, but not a generated package of this generator.
				// During cook of the generator we will treat it as still being in the generator package; SavePackage
				// will make it an import if referenced.
				continue;
			}
			check(Info != &OwnerInfo); // NewPackage != LocalOwnerPackage, so Info != OwnerInfo
			if (CookPackageSplitterInstance->RequiresEmptyPackageBeforePopulate(NewPackage))
			{
				UE_LOGF(LogCook, Error,
					"CookPackageSplitter moved an object from the generator package into a generated package during PopulateGeneratorPackage. " "This behavior is only allowed if RequiresEmptyPackageBeforePopulate returns false for the generated package. " "Edit the splitter code to either not move the object during Populate, or to return false for RequiresEmptyPackageBeforePopulate." "\n\tSplitter=%ls" "\n\tMoved Object=%ls",
					*GetSplitDataObjectName().ToString(), *PackagePair.Key->GetPathName());
			}
			else
			{
				bMovedObjects = true;
				PopulateData.KeepReferencedPackages.Add(NewPackage);
			}
		}
		if (bMovedObjects)
		{
			// Enforce uniqueness on KeepReferencedPackages since we added without checking for duplicates
			PopulateData.KeepReferencedPackages.Sort();
			PopulateData.KeepReferencedPackages.SetNum(Algo::Unique(PopulateData.KeepReferencedPackages));
		}

		OwnerObjectsToMove.Reserve(PopulateData.ObjectsToMove.Num());
		for (UObject* Object : PopulateData.ObjectsToMove)
		{
			if (Object)
			{
				OwnerObjectsToMove.Emplace(Object);
			}
		}
		OwnerInfo.AddKeepReferencedPackages(*this, PopulateData.KeepReferencedPackages);

		// Contract Point 1: We will not call populate again until the splitter has been destroyed
		// Contract Point 2: We will not call populate again without garbage collecting the generator package
		OwnerInfo.SetHasCalledPopulate(true);
		SetKeepForAllSavedOrGC();
	}
	else
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: PopulateGeneratorPackage Skipped because !IsSaveGenerator().",
			*WriteToString<256>(GetOwner().GetPackageName()));

	}

	return true;
}

bool FGenerationHelper::TryCallPopulateGeneratedPackage(UE::Cook::FCookGenerationInfo& Info,
	TArray<UObject*>& OutObjectsToMove)
{
	if (Info.HasCalledPopulate())
	{
		return true;
	}
	FPackageData& OwnerPackageData = GetOwner();
	FName OwnerPackageName = OwnerPackageData.GetPackageName();

	if (!bReportedManifest)
	{
		// Unexpected, caller should not call in this case
		UE_LOGF(LogCook, Error,
			"TryCallPopulateGeneratedPackage called for package %ls without a previous successful call to TryReportGenerationManifest.",
			*Info.GetPackageName());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return false;
	}
	check(IsValid()); // Could not have set bReportedManifest=true without being valid. 
	UObject* LocalSplitDataObject = this->FindOrLoadSplitDataObject();
	if (!LocalSplitDataObject)
	{
		UE_LOGF(LogCook, Error,
			"Failed to call TryCallPopulateGeneratedPackage, CookPackageSplitter missing. Splitter=%ls",
			*GetSplitDataObjectName().ToString());
		return false;
	}

	UPackage* Package = Info.PackageData->GetPackage();
	check(Package); // Caller checked this
	ICookPackageSplitter::FGeneratedPackageForPopulate SplitterInfo{ Info.RelativePath,
		Info.GeneratedRootPath, Package, Info.IsCreateAsMap() };

	UE::Cook::FScopedCookActivePackage ScopedActivePackage(OwnerPackageName,
#if UE_WITH_OBJECT_HANDLE_TRACKING
		PackageAccessTrackingOps::NAME_CookerBuildObject
#else
		FName()
#endif
	);
	TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

	UE::Cook::CookPackageSplitter::FPopulateContextData PopulateData;
	ICookPackageSplitter::FPopulateContext PopulateContext(PopulateData);
	PopulateData.OwnerPackage = GetOwnerPackage();
	PopulateData.OwnerObject = LocalSplitDataObject;
	PopulateData.TargetGeneratedPackage = &SplitterInfo;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	bool bPopulateSucceeded = CookPackageSplitterInstance->PopulateGeneratedPackage(
		// Bug in the deprecated API: First argument was supposed to be OwnerPackage, but we were passing in the target generated package
		// Keep passing in the target generated package in the deprecated version. This is fixed in the new API function.
		PopulateData.TargetGeneratedPackage->Package,
		PopulateData.OwnerObject, *PopulateData.TargetGeneratedPackage, PopulateData.ObjectsToMove,
		PopulateData.KeepReferencedPackages);
	CookPackageSplitterInstance->WarnIfDeprecatedVirtualNotCalled(TEXT("PopulateGeneratedPackage"));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	bPopulateSucceeded = CookPackageSplitterInstance->PopulateGeneratedPackage(PopulateContext) && bPopulateSucceeded;

	if (!bPopulateSucceeded)
	{
		UE_LOGF(LogCook, Error,
			"CookPackageSplitter returned false from PopulateGeneratedPackage. Splitter=%ls" "\nGeneratedPackage: %ls",
			*GetSplitDataObjectName().ToString(), *Info.GetPackageName());
		return false;
	}

	OutObjectsToMove.Append(PopulateData.ObjectsToMove);
	Info.AddKeepReferencedPackages(*this, PopulateData.KeepReferencedPackages);
	Info.BuildResultDependencies.Append(MoveTemp(PopulateData.BuildResultDependencies));

	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls generated package %ls: PopulateGeneratedPackage.",
		*WriteToString<256>(GetOwner().GetPackageName()),
		*WriteToString<256>(Info.PackageData->GetPackageName()));

	// Contract Point 1: We will not call populate again until the splitter has been destroyed
	// Contract Point 2: We will not call populate again without garbage collecting the generator package
	Info.SetHasCalledPopulate(true);
	SetKeepForAllSavedOrGC();
	return true;
}

void FGenerationHelper::StartOwnerSave()
{
	if (!IsValid())
	{
		return;
	}
	UE_LOGF(LogCook, Display, "Splitting Package %ls with splitter %ls acting on object %ls.",
		*WriteToString<256>(GetOwner().GetPackageName()),
		*GetRegisteredSplitterType()->GetSplitterDebugName(),
		*WriteToString<256>(GetSplitDataObjectName()));
	SetKeepForGeneratorSaveAllPlatforms();
}

void FGenerationHelper::StartQueueGeneratedPackages(UCookOnTheFlyServer& COTFS)
{
	if (!IsValid())
	{
		return;
	}
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: StartQueueGeneratedPackages.",
		*WriteToString<256>(GetOwner().GetPackageName()));

	NotifyStartQueueGeneratedPackages(COTFS, FWorkerId::Local());

	bool bLegacyBuildDependencies = COTFS.bLegacyBuildDependencies;
	bool bHasPreviousResultsForSomePlatform = false;
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		bHasPreviousResultsForSomePlatform |= !PlatformPair.Value.PreviousGeneratedPackages.IsEmpty();
	}

	if (bHasPreviousResultsForSomePlatform)
	{
		FPackageData& OwnerPackageData = GetOwner();
		TArray<const ITargetPlatform*, TInlineAllocator<1>> PlatformsToCook;
		OwnerPackageData.GetPlatformsNeedingCommit(PlatformsToCook, COTFS.GetCookPhase());

		for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
		{
			const ITargetPlatform* TargetPlatform = PlatformPair.Key;
			FPlatformData& PlatformData = PlatformPair.Value;
			TMap<FName, FAssetPackageData>& PreviousGeneratedPackages = PlatformData.PreviousGeneratedPackages;
			if (PreviousGeneratedPackages.IsEmpty())
			{
				continue;
			}

			if (!PlatformsToCook.Contains(TargetPlatform))
			{
				continue;
			}

			TSet<FName> RemainingPreviousPackages;
			RemainingPreviousPackages.Reserve(PreviousGeneratedPackages.Num());
			for (const TPair<FName, FAssetPackageData>& Pair : PreviousGeneratedPackages)
			{
				RemainingPreviousPackages.Add(Pair.Key);
			}

			int32 NumLegacyIterativeUnmodified = 0;
			int32 NumLegacyIterativeModified = 0;
			int32 NumLegacyIterativeRemoved = 0;
			int32 NumLegacyIterativePrevious = RemainingPreviousPackages.Num();

			for (FCookGenerationInfo& GeneratedInfo : PackagesToGenerate)
			{
				FName GeneratedPackageName = GeneratedInfo.PackageData->GetPackageName();
				FAssetPackageData* PreviousAssetData = PreviousGeneratedPackages.Find(GeneratedPackageName);
				RemainingPreviousPackages.Remove(GeneratedPackageName);
				if (PreviousAssetData)
				{
					if (bLegacyBuildDependencies)
					{
						bool bLegacyIterativeUnmodified;
						GeneratedInfo.LegacyIterativeCookValidateOrClear(*this, TargetPlatform,
							PreviousAssetData->GetPackageSavedHash(), bLegacyIterativeUnmodified);
						++(bLegacyIterativeUnmodified ? NumLegacyIterativeUnmodified : NumLegacyIterativeModified);
					}
					else
					{
						// Copy the current value for the package's hash into the PreviousPackageData, for use by
						// incremental cook's calculation in FRequestCluster::TryCalculateIncrementallyUnmodified
						PreviousAssetData->SetPackageSavedHash(GeneratedInfo.PackageHash);
					}
				}
			}
			if (!RemainingPreviousPackages.IsEmpty())
			{
				NumLegacyIterativeRemoved = RemainingPreviousPackages.Num();
				for (FName PreviousPackageName : RemainingPreviousPackages)
				{
					COTFS.DeleteOutputForPackage(PreviousPackageName, TargetPlatform);
				}
			}

			if (bLegacyBuildDependencies)
			{
				UE_LOGF(LogCook, Display,
					"Found %d cooked package(s) in package store for generator package %ls.",
					NumLegacyIterativePrevious, *WriteToString<256>(GetOwner().GetPackageName()));
				UE_LOGF(LogCook, Display, "Keeping %d. Recooking %d. Removing %d.",
					NumLegacyIterativeUnmodified, NumLegacyIterativeModified, NumLegacyIterativeRemoved);
			}
		}
	}
}

void FGenerationHelper::NotifyStartQueueGeneratedPackages(UCookOnTheFlyServer& COTFS, FWorkerId SourceWorkerId)
{
	// Note this function can be called on an uninitialized Generator; the generator is only needed
	// on the director so it can serve as the passer of messages. We have to keep ourselves referenced after
	// this call, until after we send EGeneratorEvent::QueuedGeneratedPackagesFencePassed, so that we don't destruct
	// and lose the information from SavedOnWorker or TryReportGenerationManifest.
	if (!COTFS.CookWorkerClient)
	{
		GetOwnerInfo().SavedOnWorker = SourceWorkerId;
		SetKeepForCompletedAllSavesMessage();
	}
	SetKeepForQueueResults();
}

bool FGenerationHelper::FDirectorAPI::HasStartedQueueGeneratedPackages() const
{
	return GenerationHelper.GetOwnerInfo().SavedOnWorker.IsValid();
}

void FGenerationHelper::EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS)
{
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: EndQueueGeneratedPackages.",
		*WriteToString<256>(GetOwner().GetPackageName()));

	bHasFinishedQueueGeneratedPackages = true;
	SetKeepForQueueResults();
	COTFS.WorkerRequests->EndQueueGeneratedPackages(COTFS, *this);
}

void FGenerationHelper::EndQueueGeneratedPackagesOnDirector(UCookOnTheFlyServer& COTFS, FWorkerId SourceWorkerId)
{
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: EndQueueGeneratedPackagesOnDirector.",
		*WriteToString<256>(GetOwner().GetPackageName()));

	// Note this function can be called on an uninitialized Generator; the generator is only needed
	// on the director so it can serve as the passer of messages.
	bHasFinishedQueueGeneratedPackages = true;
	// When we queued locally, this function is called after QueueDiscoveredPackage was called for each package.
	// When we queued on a remote CookWorker, the replication system from cookworker guarantees that all discovered
	// packages have been reported via TrackGeneratedPackageListedRemotely before we receive this function call
	// via the EGeneratorEvent::QueuedGeneratedPackages message (the package discovery messages are replicated
	// before the EGeneratorEvent). We therefore know that all generated packages have already been requested
	// or are in the discovery queue, so we can add a request fence listener now and know that when it is called
	// all generated packages have been queued and assigned.
	COTFS.PackageDatas->GetRequestQueue().AddRequestFenceListener(GetOwner().GetPackageName());
	SetKeepForQueueResults();

	// Setting OwnerInfo.SavedOnWorker and KeepForCompletedAllSavesMessage in response to this event is usually not
	// needed because they are set from NotifyStartQueueGeneratedPackages, but we set them anyway in case there is an
	// edge condition that skips those notifications.
	SetKeepForCompletedAllSavesMessage();
	GetOwnerInfo().SavedOnWorker = SourceWorkerId;

	// The save message for the owner may have come in before this GenerationHelper was created and thus
	// MarkPackageSavedRemotely was not called. Check for that case now and marked saved if so.
	FPackageData& OwnerPackageData = GetOwner();
	FCookGenerationInfo& LocalOwnerInfo = GetOwnerInfo();
	for (TPair<const ITargetPlatform*, FCookGenerationInfo::FPlatformData>& PlatformPair
		: LocalOwnerInfo.PlatformDatas)
	{
		const ITargetPlatform* TargetPlatform = PlatformPair.Key;
		if (OwnerPackageData.HasCookedPlatform(TargetPlatform, true /* bIncludeFailed */))
		{
			PlatformPair.Value.SetHasSaved(*this, LocalOwnerInfo, TargetPlatform, true /* bValue */, SourceWorkerId);
		}
	}
}

void FGenerationHelper::OnRequestFencePassed(UCookOnTheFlyServer& COTFS)
{
	// This function should only be called in response to a subscription that is sent from the cook director
	check(!COTFS.CookWorkerClient);

	const ITargetPlatform* FirstNotSkippedTargetPlatform = nullptr;
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		const ITargetPlatform* TargetPlatform = PlatformPair.Key;
		FPlatformData& PlatformData = PlatformPair.Value;
		if (OwnerInfo.FindCheckedPlatformData(TargetPlatform).IsIncrementallySkipped())
		{
			// PumpRequests has completed and we marked ourselves and all generated packages as incrementally skipped,
			// so we no longer need the PreviouslyCookedData or this entire GenerationHelper
			UE_LOGF(LogCookGenerationHelper, Verbose,
				"%ls: RequestFencePassed and found to be entirely incrementally skipped for %ls.",
				*WriteToString<256>(GetOwner().GetPackageName()), *TargetPlatform->PlatformName());
			PlatformData.ClearKeepForIncremental(*this, TargetPlatform);
			PlatformData.PreviousGeneratedPackages.Empty();
		}
		else
		{
			FirstNotSkippedTargetPlatform = FirstNotSkippedTargetPlatform
				? FirstNotSkippedTargetPlatform : TargetPlatform;
		}
	}

	if (bHasFinishedQueueGeneratedPackages)
	{
		// We have finished EndQueueGeneratedPackagesOnDirector, so all generated packages have been requested
		// and assigned to local ReadyRequests or to a CookWorker. Send OnQueuedGeneratedPackagesFencePassed
		// to ourselves and all cookworkers.

		// Call OnNumSavedUpdated to check for whether all packages have already been saved by the time we reach
		// the request fence. This can happen in incremental cooks, or in race conditions if we sent all packages
		// out for saving before receiving the EndQueueGeneratedPackagesOnDirector message.
		OnNumSavedUpdated();
		
		if (COTFS.CookDirector)
		{
			FName PackageName = GetOwner().GetPackageName();
			FGeneratorEventMessage Message(EGeneratorEvent::QueuedGeneratedPackagesFencePassed, PackageName);
			COTFS.CookDirector->BroadcastMessage(MoveTemp(Message), ECookBroadcastTiming::AfterAssignPackages);
		}
		OnQueuedGeneratedPackagesFencePassed(COTFS);
	}
	else
	{
		if (FirstNotSkippedTargetPlatform)
		{
			UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: RequestFencePassed, but not yet HasFinishedQueueGeneratedPackages.",
				*WriteToString<256>(GetOwner().GetPackageName()));
		}
	}
}

void FGenerationHelper::OnSkipSaveExistingGenerator()
{
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SkipSaveExistingGenerator.",
		*WriteToString<256>(GetOwner().GetPackageName()));
	bSkipSaveExistingGenerator = true;
	bSaveGenerator = false;
}

void FGenerationHelper::OnQueuedGeneratedPackagesFencePassed(UCookOnTheFlyServer& COTFS)
{
	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: RequestFencePassed, and HasFinishedQueueGeneratedPackages.",
		*WriteToString<256>(GetOwner().GetPackageName()));

	bQueuedGeneratedPackagesFencePassed = true;
	ClearKeepForQueueResults();
	// We no longer need PreviousGeneratedPackages or KeepForIncremental, because they are used only in 
	// StartQueueGeneratedPackages or the request cluster that they end up in in PumpRequests, both of which
	// are now finished. Clear them on the director and any CookWorkers that received them to free memory.
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		FPlatformData& PlatformData = PlatformPair.Value;
		PlatformData.ClearKeepForIncremental(*this, PlatformPair.Key);
		PlatformData.PreviousGeneratedPackages.Empty();
	}
}

UPackage* FGenerationHelper::TryCreateGeneratedPackage(FCookGenerationInfo& GeneratedInfo, bool bResetToEmpty)
{
	if (!IsValid())
	{
		return nullptr;
	}

	FPackageData& GeneratedPackageData = *GeneratedInfo.PackageData;
	const FString GeneratedPackageName = GeneratedPackageData.GetPackageName().ToString();
	UPackage* LocalOwnerPackage = FindOrLoadOwnerPackage(GetOwner().GetPackageDatas().GetCookOnTheFlyServer());
	if (!LocalOwnerPackage)
	{
		UE_LOGF(LogCook, Error,
			"TryCreateGeneratedPackage: could not load ParentGeneratorPackage %ls for GeneratedPackage %ls",
			*GetOwner().GetPackageName().ToString(), *GeneratedPackageName);
		return nullptr;
	}

	UPackage* GeneratedPackage = FindObject<UPackage>(nullptr, *GeneratedPackageName);
	if (GeneratedPackage)
	{
		// The package might have been created for the generator's presave, or it might have been created and abandoned
		// by an earlier save attempt of the generated package.
		// If bResetToEmpty then we are starting the populate of the generated package and we need to remove all objects
		// from the package. Generated packages are created empty and it is the job of the CookPackageSplitter to populate
		// them during save.
		if (bResetToEmpty && GetCookPackageSplitterInstance()->RequiresEmptyPackageBeforePopulate(GeneratedPackage))
		{
			TArray<UObject*> ExistingObjects;
			GetObjectsWithPackage(GeneratedPackage, ExistingObjects, EGetObjectsFlags::None);
			if (!ExistingObjects.IsEmpty())
			{
				UObject* TransientPackage = GetTransientPackage();
				for (UObject* Existing : ExistingObjects)
				{
					Existing->Rename(nullptr, TransientPackage, REN_DontCreateRedirectors);
				}
			}
		}
	}
	else
	{
#if ENABLE_COOK_STATS
		++DetailedCookStats::NumRequestedLoads;
#endif
		// Mark that we are the referencer of the package. This is not necessary for cook correctness, because we 
		// early exit GeneratedPackages from ProcessUnsolicitedPackages, but it is useful for debug tools to have
		// PackageTracker know the referencer of the package.
		UE_TRACK_REFERENCING_PACKAGE_SCOPED(GetOwner().GetPackageName(), PackageAccessTrackingOps::NAME_PostLoad);
		GeneratedPackage = CreatePackage(*GeneratedPackageName);
		GeneratedPackage->SetLoadedPath(FPackagePath::FromPackageNameChecked(GeneratedPackageName));

	}
	GeneratedPackage->SetSavedHash(GeneratedInfo.PackageHash);
	GeneratedPackage->SetPersistentGuid(LocalOwnerPackage->GetPersistentGuid());
	GeneratedPackage->SetPackageFlags(PKG_CookGenerated);
	GeneratedInfo.SetHasCreatedPackage(true);

	return GeneratedPackage;
}

void FGenerationHelper::FinishGeneratorPlatformSave(FPackageData& PackageData, bool bFirstPlatform,
	TArray<FAssetDependency>& OutPackageDependencies)
{
	ConditionalInitialize();

	FCookGenerationInfo* Info = &GetOwnerInfo();
	UCookOnTheFlyServer& COTFS = Info->PackageData->GetPackageDatas().GetCookOnTheFlyServer();

	// Set dependencies equal to the global AssetRegistry dependencies plus a dependency on
	// each generated package.
	COTFS.AssetRegistry->GetDependencies(PackageData.GetPackageName(), OutPackageDependencies,
		UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Game);

	OutPackageDependencies.Reserve(PackagesToGenerate.Num());
	for (FCookGenerationInfo& GeneratedInfo : GetPackagesToGenerate())
	{
		FAssetDependency& Dependency = OutPackageDependencies.Emplace_GetRef();
		Dependency.AssetId = FAssetIdentifier(GeneratedInfo.PackageData->GetPackageName());
		Dependency.Category = UE::AssetRegistry::EDependencyCategory::Package;
		// The dependencies we write on the generated packages should be soft, not build, and
		// game by default but optionally editor-only.
		// Soft because they're not necessarily hard imports and the generator's package does not need to load them
		// before it loads. The generator may have added them as imports, but we don't need to record that in the AR.
		// Not build because they they don't affect whether the generator package needs to be recooked.
		// Game by default because usually they are needed at runtime (and therefore mounted in the same chunk),
		// but some Generator classes may want to override that behavior for some generated packages
		// e.g. world partition may want streaming cells for a content bundle to be chunked with the content bundle).
		// See the comment on bCopyChunkAssignmentFromGenerator in ICookPackageSplitter, and see the use of this
		// dependency in FAssetRegistryGenerator::CalculateChunkIdsAndAssignToManifest.
		Dependency.Properties = UE::AssetRegistry::EDependencyProperty::None;
		if (GeneratedInfo.IsCopyChunkAssignmentFromGenerator())
		{
			Dependency.Properties |= UE::AssetRegistry::EDependencyProperty::Game;
		}
	}

	if (bFirstPlatform)
	{
		FetchExternalActorDependencies();
		COTFS.RecordExternalActorDependencies(GetExternalActorDependencies());
	}

	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: Generator package saved.",
		*WriteToString<256>(GetOwner().GetPackageName()));
}

void FGenerationHelper::FinishGeneratedPlatformSave(FPackageData& PackageData, const ITargetPlatform* TargetPlatform,
	FAssetPackageData& OutAssetPackageData, TArray<FAssetDependency>& OutDependencies,
	FBuildResultDependenciesMap& OutBuildResultDependencies)
{
	check(TargetPlatform);
	ConditionalInitialize();

	FCookGenerationInfo* Info = FindInfo(PackageData);
	if (!Info)
	{
		UE_LOGF(LogCook, Error, "GeneratedInfo missing for package %ls.",
			*PackageData.GetPackageName().ToString());
		return;
	}
	UCookOnTheFlyServer& COTFS = Info->PackageData->GetPackageDatas().GetCookOnTheFlyServer();

	FCookGenerationInfo::FPlatformData& InfoPlatformData = Info->FindCheckedPlatformData(TargetPlatform);

	// External packages are detected by their presence in the NeverCookPackageList 
	FThreadSafeSet<FName>& NeverCookPackageList = GetOwner().GetPackageDatas().GetCookOnTheFlyServer().PackageTracker->NeverCookPackageList; 
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	for (const FAssetDependency& Dependency : Info->PackageDependencies)
	{
		// Only dependencies flagged as Build add their own dependencies as runtime dependencies
		bool bIsBuildDependency = EnumHasAnyFlags(Dependency.Properties, UE::AssetRegistry::EDependencyProperty::Build);
		if (bIsBuildDependency)
		{
			TArray<FAssetDependency> PackageDependencies;
			AssetRegistry.GetDependencies(FAssetIdentifier(Dependency.AssetId.PackageName), PackageDependencies,
				UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Game);

			for (const FAssetDependency& PackageDependency : PackageDependencies)
			{
				TStringBuilder<256> PackageDependencyName(InPlace, PackageDependency.AssetId.PackageName);

				// TODO : This is a special case for world splitter. If the package is a level, ignore it. A level can contain another level. 
				// But they are already included in the generated package so there is no need to add them as runtime dependencies. It would trigger 
				// a recook of those levels which is unnecessary. We should find a better way to do this. One solution would be to have the world splitter
				// find the correct dependencies directly so I don't need any special case here.
				
				// If it's a script package, add it directly. Calling GetLocalFullPath to check if it's a map will trigger a warning.
				if (FPackageName::IsScriptPackage(PackageDependencyName))
				{
					OutDependencies.Add(PackageDependency);
					continue;
				}
				
				FString ExtensionStr;
				if (!AssetRegistry.DoesPackageExistOnDisk(PackageDependency.AssetId.PackageName, nullptr, &ExtensionStr))
				{
					// No need to add dependencies to packages that do not exist on disk (not even generated packages).
					continue;
				}
				EPackageExtension Extension = FPackagePath::ParseExtension(ExtensionStr);
				if (Extension == EPackageExtension::Map)
				{
					continue;
				}

				OutDependencies.Add(PackageDependency);
			}
		}

		// The world splitter flags every dependency as EditorOnly but I still want external actors added as runtime dependency.
		bool bNeverCookedPackage = NeverCookPackageList.Contains(Dependency.AssetId.PackageName);
		bool bIsGameDependency = EnumHasAnyFlags(Dependency.Properties, UE::AssetRegistry::EDependencyProperty::Game);
		if (bIsGameDependency || bNeverCookedPackage)
		{
			OutDependencies.Add(Dependency);
		}
	}

	// Add on the custom cook BuildResultDependencies declared for the generated package by the CookPackageSplitter.
	OutBuildResultDependencies = Info->BuildResultDependencies;

	// Update the AssetPackageData for each requested platform with Guid and ImportedClasses
	TSet<UClass*> PackageClasses;
	UPackage* Package = PackageData.GetPackage();
	check(Package);
	ForEachObjectWithPackage(Package, [&PackageClasses, Package](UObject* Object)
		{
			UClass* Class = Object->GetClass();
			if (!Class->IsInPackage(Package)) // Imported classes list does not include classes in the package
			{
				PackageClasses.Add(Object->GetClass());
			}
			return true;
		});
	TArray<FName> ImportedClasses;
	ImportedClasses.Reserve(PackageClasses.Num());
	for (UClass* Class : PackageClasses)
	{
		TStringBuilder<256> ClassPath;
		Class->GetPathName(nullptr, ClassPath);
		ImportedClasses.Add(FName(ClassPath));
	}
	ImportedClasses.Sort(FNameLexicalLess());

	FAssetPackageData& AssetPackageData = InfoPlatformData.AssetPackageData.Emplace();
	AssetPackageData.FileVersionUE = GPackageFileUEVersion;
	AssetPackageData.FileVersionLicenseeUE = GPackageFileLicenseeUEVersion;
	AssetPackageData.SetIsLicenseeVersion(FEngineVersion::Current().IsLicenseeVersion());
	// Currently assumes IoDispatcher however should we cook to loose files in the future this would need updating
	AssetPackageData.SetPackageLocation(FPackageName::EPackageLocationFilter::IoDispatcher);
	AssetPackageData.Extension = FPackagePath::ParseExtension(
		WriteToString<256>(PackageData.GetFileName()));
	AssetPackageData.SetPackageSavedHash(Info->PackageHash);
	AssetPackageData.ImportedClasses = MoveTemp(ImportedClasses);

	OutAssetPackageData = AssetPackageData;

	UE_LOGF(LogCookGenerationHelper, Verbose, "%ls generated package %ls: Generated package saved for platform %ls.",
		*WriteToString<256>(GetOwner().GetPackageName()),
		*WriteToString<256>(Info->PackageData->GetPackageName()), *TargetPlatform->PlatformName());
}

const FAssetPackageData* FGenerationHelper::GetAssetPackageData(FName PackageName, const ITargetPlatform* TargetPlatform)
{
	FPlatformData* PlatformData = PlatformDatas.Find(TargetPlatform);
	if (PlatformData)
	{
		const FAssetPackageData* Result = PlatformData->PreviousGeneratedPackages.Find(PackageName);
		if (Result)
		{
			return Result;
		}
	}

	FCookGenerationInfo* Info = FindInfoNoInitialize(PackageName);
	if (!Info || Info->IsGenerator())
	{
		return nullptr;
	}

	FCookGenerationInfo::FPlatformData* InfoPlatformData = Info->PlatformDatas.Find(TargetPlatform);
	if (InfoPlatformData)
	{
		const FAssetPackageData* Result = InfoPlatformData->AssetPackageData.GetPtrOrNull();
		if (Result)
		{
			return Result;
		}
	}
	return nullptr;
}

const FAssetPackageData* FGenerationHelper::GetAssetPackageDataAnyPlatform(FName PackageName)
{
	for (const TPair<const ITargetPlatform*, FPlatformData>& Pair : PlatformDatas)
	{
		const FAssetPackageData* Result = Pair.Value.PreviousGeneratedPackages.Find(PackageName);
		if (Result)
		{
			return Result;
		}
	}

	FCookGenerationInfo* Info = FindInfoNoInitialize(PackageName);
	if (!Info || Info->IsGenerator())
	{
		return nullptr;
	}

	for (const TPair<const ITargetPlatform*, FCookGenerationInfo::FPlatformData>& InfoPair : Info->PlatformDatas)
	{
		const FAssetPackageData* Result = InfoPair.Value.AssetPackageData.GetPtrOrNull();
		if (Result)
		{
			return Result;
		}
	}
	return nullptr;
}

UE::EditorDomain::FPackageDigest FGenerationHelper::GetPackageDigest(FName PackageName, const ITargetPlatform* TargetPlatform)
{
	const FAssetPackageData* AssetPackageData = GetAssetPackageData(PackageName, TargetPlatform);
	if (!AssetPackageData)
	{
		return UE::EditorDomain::FPackageDigest();
	}
	return UE::EditorDomain::CalculatePackageDigest(*AssetPackageData, PackageName);
}

void FGenerationHelper::ResetSaveState(FCookGenerationInfo& Info, UPackage* Package,
	EStateChangeReason ReleaseSaveReason, EPackageState NewState)
{
	ConditionalInitialize();

	// We release references to *this in this function so keep a local reference to avoid deletion during the function.
	TRefCountPtr<FGenerationHelper> LocalRefCount = this;

	if (Info.PackageData->GetSaveSubState() > ESaveSubState::Generation_CallPreSave)
	{
		UObject* SplitObject = GetWeakSplitDataObject();
		UPackage* LocalOwnerPackage = Info.IsGenerator() ? Package : GetOwnerPackage();
		if (!SplitObject || !Package || !LocalOwnerPackage)
		{
			UE_LOGF(LogCook, Warning,
				"PackageSplitter: %ls on %ls was GarbageCollected before we finished saving it. This prevents us from calling PostSave and may corrupt other packages that it altered during Populate. Splitter=%ls.",
				(!Package ? TEXT("UPackage") :
					(!LocalOwnerPackage ? TEXT("ParentGenerator UPackage") : TEXT("SplitDataObject"))),
				*Info.GetPackageName(),
				*GetSplitDataObjectName().ToString());
		}
		else
		{
			UE::Cook::FScopedCookActivePackage ScopedActivePackage(GetOwner().GetPackageName(),
#if UE_WITH_OBJECT_HANDLE_TRACKING
				PackageAccessTrackingOps::NAME_CookerBuildObject
#else
				FName()
#endif
			);
			TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

			UE::Cook::CookPackageSplitter::FPopulateContextData PopulateData;
			ICookPackageSplitter::FPopulateContext PopulateContext(PopulateData);
			PopulateData.OwnerPackage = LocalOwnerPackage;
			PopulateData.OwnerObject = SplitObject;
			ICookPackageSplitter* LocalSplitterInstance = GetCookPackageSplitterInstance();
			if (Info.IsGenerator())
			{
				// By contract, skip the call to PostSaveGeneratorPackage if !bCookSaveGeneratorPackage
				if (IsSaveGenerator())
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS;
					LocalSplitterInstance->PostSaveGeneratorPackage(PopulateData.OwnerPackage, PopulateData.OwnerObject);
					LocalSplitterInstance->WarnIfDeprecatedVirtualNotCalled(TEXT("PostSaveGeneratorPackage"));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS;
					LocalSplitterInstance->PostSaveGeneratorPackage(PopulateContext);
				}
			}
			else
			{
				ICookPackageSplitter::FGeneratedPackageForPopulate PopulateInfo;
				PopulateInfo.RelativePath = Info.RelativePath;
				PopulateInfo.GeneratedRootPath = Info.GeneratedRootPath;
				PopulateInfo.bCreatedAsMap = Info.IsCreateAsMap();
				PopulateInfo.Package = Package;
				PopulateData.TargetGeneratedPackage = &PopulateInfo;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS;
				LocalSplitterInstance->PostSaveGeneratedPackage(PopulateData.OwnerPackage, PopulateData.OwnerObject,
					*PopulateData.TargetGeneratedPackage);
				LocalSplitterInstance->WarnIfDeprecatedVirtualNotCalled(TEXT("PostSaveGeneratedPackage"));
				PRAGMA_ENABLE_DEPRECATION_WARNINGS;
				LocalSplitterInstance->PostSaveGeneratedPackage(PopulateContext);
			}
		}
	}

	if (IsTerminalStateChange(ReleaseSaveReason))
	{
		// The package's progress is completed and we will not come back to it; set state back to initial
		// state, mark the package as saved in our GenerationHelper data, and drop the ParentGenerationHelper
		// reference.
		if (Info.IsGenerator())
		{
			for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
			{
				const ITargetPlatform* TargetPlatform = PlatformPair.Key;
				FPlatformData& PlatformData = PlatformPair.Value;
				Info.FindCheckedPlatformData(TargetPlatform).SetHasSaved(*this, Info, TargetPlatform,
					true /* bValue */, FWorkerId::Local());

				// Now that we've finished saving, we know that we will not call QueueGeneratedPackages again, so we can
				// teardown incremental results as well
				PlatformData.ClearKeepForIncremental(*this, TargetPlatform);

				// We still need access to those to compare hashes
				//PlatformData.PreviousGeneratedPackages.Empty();

				// And also teardown data needed during save
				PlatformData.ClearKeepForGeneratorSave(*this, TargetPlatform);
			}
		}
		else
		{
			// For generated packages, SetHasSaved is called inside of SetParentGenerationHelper
			Info.PackageData->SetParentGenerationHelper(nullptr, ReleaseSaveReason, &Info);
		}
	}

	if (Info.HasTakenOverCachedCookedPlatformData())
	{
		// Demotion possibly causes a failure of our guarantees to CookPackageSplitter authors that we will keep the
		// UObjects in the packages referenced. But there are also some cases where it does not cause a problem.
		bool bDemotionAllowed = Info.PackageData->GetCachedObjectsInOuter().Num() == 0
			|| !IsUseInternalReferenceToAvoidGarbageCollect();
		// If the package is no longer cooking, the demotion is okay, because we will not be relying on the splitter
		// for any further work on it. The package is "no longer cooking" if it is moving into the idle or the
		// AssignedToWorker state. In the AssignedToWorker state we might get it again and need to cook it if it is
		// retracted back to us, but in the case of moving to AssignedToWorker state, we expect that is only happening
		// because this GenerationHelper is moving into the Uninitializ, and we will redo all of the splitter's
		// package loading later when we retract it and reinitialize this GenerationHelper.
		bDemotionAllowed = bDemotionAllowed ||
			NewState == EPackageState::Idle || NewState == EPackageState::AssignedToWorker;
		// Demotion is also allowed in cases of being kicked out of the save state and not expecting to come back.
		bDemotionAllowed = bDemotionAllowed || IsTerminalStateChange(ReleaseSaveReason)
			|| ReleaseSaveReason == EStateChangeReason::DoneForNow || ReleaseSaveReason == EStateChangeReason::Retraction;
		if (!bDemotionAllowed)
		{
			FString DeletedObjectInOuter(TEXT("<None>"));
			int32 NumNulls = 0;
			for (const TPair<UObject*, FCachedObjectInOuterGeneratorInfo>& Pair : Info.CachedObjectsInOuterInfo)
			{
				bool bStillPresent = Info.PackageData->GetCachedObjectsInOuter().ContainsByPredicate(
						[RawPtr = Pair.Key](const FCachedObjectInOuter& CachedObject)
						{
							return CachedObject.Object.Get() == RawPtr;
						});
				if (!bStillPresent)
				{
					if (NumNulls == 0)
					{
						DeletedObjectInOuter = Pair.Value.FullName;
					}
					++NumNulls;
				}
			}
			UPackage* LocalOwnerPackage = Info.IsGenerator() ? Package : GetOwnerPackage();

			UE_LOGF(LogCook, Error,
				"CookPackageSplitter failure: We are demoting a %ls package from save and removing our references that keep its objects loaded." "\nThis will allow the objects to be garbage collected and cause failures in the splitter which expects them to remain loaded." "\n\tPackage=%ls, Splitter=%ls, " "\n\tReleaseSaveReason=%ls, CurrentState=%ls, CurrentSubState=%ls, NewState=%ls," "\n\tPackagePtr=%ls, Package->IsFullyLoaded()=%ls," "\n\tOwnerPackagePtr=%ls, OwnerPackage->IsFullyLoaded()=%ls," "\n\tGetCachedObjectsInOuter->Num()==%d, Num nulls in GetCachedObjectsInOuter==%d, First null=='%ls'",
				Info.IsGenerator() ? TEXT("generator") : TEXT("generated"),
				*Info.GetPackageName(),
				*GetSplitDataObjectName().ToString(), LexToString(ReleaseSaveReason),
				LexToString(Info.PackageData->GetState()), LexToString(Info.PackageData->GetSaveSubState()),
				LexToString(NewState),
				Package == nullptr ? TEXT("<null>") : TEXT("<nonnull>"),
				*(::LexToString(Package == nullptr ? false : Package->IsFullyLoaded())),
				LocalOwnerPackage == nullptr ? TEXT("<null>") : TEXT("<nonnull>"),
				* (::LexToString(LocalOwnerPackage == nullptr ? false : LocalOwnerPackage->IsFullyLoaded())),
				Info.PackageData->GetCachedObjectsInOuter().Num(),
				NumNulls, *DeletedObjectInOuter);
			FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		}
		Info.CachedObjectsInOuterInfo.Empty();
		Info.SetHasTakenOverCachedCookedPlatformData(false);
	}
	else if (Info.CachedObjectsInOuterInfo.Num() > 0)
	{
		UE_LOGF(LogCook, Error,
			"CookPackageSplitter logic error: A %ls package has !Info.HasTakenOverCachedCookedPlatformData(), but Info.CachedObjectsInOuterInfo.Num() == %d > 0. This should be impossible.\n" "Package=%ls, PackageState=%ls, SaveSubState=%ls",
			Info.IsGenerator() ? TEXT("generator") : TEXT("generated"), Info.CachedObjectsInOuterInfo.Num(),
			*Info.GetPackageName(), LexToString(Info.PackageData->GetState()), LexToString(Info.PackageData->GetSaveSubState()));
		Info.CachedObjectsInOuterInfo.Empty();
	}

	Info.SetHasIssuedUndeclaredMovedObjectsWarning(false);

	// Clear KeepReferencedPackages; we no longer have a contract that we keep them referenced, except for the
	// generator. If the splitter requires EGeneratedRequiresGenerator::Populate, then we are required to keep them
	// referenced until all packages have saved as well, so we keep them referenced for the lifetime of the
	// GenerationHelper.
	if (!Info.IsGenerator() || DoesGeneratedRequireGenerator() <
		ICookPackageSplitter::EGeneratedRequiresGenerator::Populate)
	{
		Info.KeepReferencedPackages.Reset();
	}
	if (Info.IsGenerator())
	{
		OwnerObjectsToMove.Empty();
	}
}

bool FGenerationHelper::ShouldRetractionStallRatherThanDemote(FPackageData& PackageData)
{
	FCookGenerationInfo* Info = FindInfo(PackageData);
	if (Info)
	{
		if (PackageData.IsInStateProperty(EPackageStateProperty::Saving))
		{
			if (PackageData.GetSaveSubState() > ESaveSubState::Generation_PreMoveCookedPlatformData_WaitingForIsLoaded)
			{
				return true;
			}
		}
	}
	return false;
}


void FGenerationHelper::FetchExternalActorDependencies()
{
	if (!IsValid())
	{
		return;
	}

	// The Generator package declares all its ExternalActor dependencies in its AssetRegistry dependencies
	// The Generator's generated packages can also include ExternalActors from other maps due to level instancing,
	// these are included in the dependencies reported by the Generator for each GeneratedPackage in the data
	// returned from ReportGenerationManifest. These sets will overlap; take the union.
	ExternalActorDependencies.Reset();
	IAssetRegistry::GetChecked().GetDependencies(GetOwner().GetPackageName(), ExternalActorDependencies,
		UE::AssetRegistry::EDependencyCategory::Package);
	for (const FCookGenerationInfo& Info : PackagesToGenerate)
	{
		ExternalActorDependencies.Reserve(Info.GetDependencies().Num() + ExternalActorDependencies.Num());
		for (const FAssetDependency& Dependency : Info.GetDependencies())
		{
			ExternalActorDependencies.Add(Dependency.AssetId.PackageName);
		}
	}
	Algo::Sort(ExternalActorDependencies, FNameFastLess());
	ExternalActorDependencies.SetNum(Algo::Unique(ExternalActorDependencies));
	FPackageDatas& PackageDatas = this->GetOwner().GetPackageDatas();
	FThreadSafeSet<FName>& NeverCookPackageList =
		GetOwner().GetPackageDatas().GetCookOnTheFlyServer().PackageTracker->NeverCookPackageList;

	// We are supposed to collect only ExternalActor dependencies, but we collected every dependency from the
	// generated packages. Remove the packages that are not external actors, which we detect by being on-disk
	// PackageDatas that are marked as NeverCook
	ExternalActorDependencies.RemoveAll([&PackageDatas, &NeverCookPackageList](FName PackageName)
		{
			FPackageData* PackageData = PackageDatas.TryAddPackageDataByPackageName(PackageName);
			if (!PackageData)
			{
				return true;
			}
			bool bIsNeverCook = NeverCookPackageList.Contains(PackageData->GetPackageName());
			return !bIsNeverCook;
		});
	ExternalActorDependencies.Shrink();
}

void FGenerationHelper::SetPreviousGeneratedPackages(const ITargetPlatform* TargetPlatform, TMap<FName, FAssetPackageData>&& Packages)
{
	UE_LOGF(LogCookGenerationHelper, Verbose,
		"%ls: SetPreviousGeneratedPackages with %d generated packages for platform %ls.",
		*WriteToString<256>(GetOwner().GetPackageName()), Packages.Num(), *TargetPlatform->PlatformName());
	FPlatformData& PlatformData = FindCheckedPlatformData(TargetPlatform);
	PlatformData.SetKeepForIncremental(*this, TargetPlatform);
	PlatformData.PreviousGeneratedPackages = MoveTemp(Packages);
}

template <typename T>
static TArrayView<T*> AppendWeakPtrsToObjectPtrArray(TArray<T*>& Out, const TArray<TWeakObjectPtr<T>>& In)
{
	Out.Reserve(Out.Num() + In.Num());
	int32 StartIndex = Out.Num();
	for (const TWeakObjectPtr<T>& WeakPtr : In)
	{
		T* Object = WeakPtr.Get();
		if (Object)
		{
			Out.Add(Object);
		}
	}
	return TArrayView<T*>(Out).Mid(StartIndex);
}

void FGenerationHelper::PreGarbageCollect(const TRefCountPtr<FGenerationHelper>& RefcountHeldByCaller,
	FPackageData& PackageData, TArray<TObjectPtr<UObject>>& GCKeepObjects,
	TArray<UPackage*>& GCKeepPackages, TArray<FPackageData*>& GCKeepPackageDatas, bool& bOutShouldDemote)
{
	bOutShouldDemote = false;
	if (&PackageData == &GetOwner())
	{
		PreGarbageCollectGCLifetimeData();
	}
	if (!IsInitialized() || !IsValid())
	{
		return;
	}

	FCookGenerationInfo* InfoPtr = FindInfo(PackageData);
	if (!InfoPtr)
	{
		return;
	}
	FCookGenerationInfo& Info = *InfoPtr;

	if (!IsUseInternalReferenceToAvoidGarbageCollect() && !Info.PackageData->GetIsCookLast())
	{
		// If we don't have a contract to keep the packagedata referenced during GC, don't report
		// anything to garbage collection, and demote the package if it has progressed too far
		if (Info.PackageData->GetSaveSubState() > ESaveSubState::Generation_CallPreSave)
		{
			bOutShouldDemote = true;
		}
		return;
	}

	// When we have a contract to keep the packagedata referenced, keep its various object pointers referenced.

	// We have a contract that KeepReferencedPackages in any Info are kept referenced.
	bool bKeepingAnyObjects = false;
	bool bNeedsGeneratorPackage = false;
	FPackageDatas& PackageDatas = PackageData.GetPackageDatas();
	if (&Info == &OwnerInfo)
	{
		// Handled by bCurrentGCHasKeptGeneratorKeepPackages
	}
	else if (!Info.KeepReferencedPackages.IsEmpty())
	{
		bKeepingAnyObjects = true;
		for (UPackage* Package : AppendWeakPtrsToObjectPtrArray(GCKeepPackages, Info.KeepReferencedPackages))
		{
			FPackageData* PackageDataOfPackage = PackageDatas.FindPackageDataByPackageName(Package->GetFName());
			if (PackageDataOfPackage)
			{
				GCKeepPackageDatas.Add(PackageDataOfPackage);
			}
		}
	}
	if (!bCurrentGCHasKeptGeneratorKeepPackages)
	{
		bCurrentGCHasKeptGeneratorKeepPackages = true;
		if (!OwnerInfo.KeepReferencedPackages.IsEmpty())
		{
			bNeedsGeneratorPackage = true;
			for (UPackage* Package : AppendWeakPtrsToObjectPtrArray(GCKeepPackages, OwnerInfo.KeepReferencedPackages))
			{
				FPackageData* PackageDataOfPackage = PackageDatas.FindPackageDataByPackageName(Package->GetFName());
				if (PackageDataOfPackage)
				{
					GCKeepPackageDatas.Add(PackageDataOfPackage);
				}
			}
		}
	}

	// Keep the objects returned from GetObjectsToMove* functions referenced.
	if (Info.HasTakenOverCachedCookedPlatformData())
	{
		bKeepingAnyObjects = true;
		for (FCachedObjectInOuter& CachedObjectInOuter : Info.PackageData->GetCachedObjectsInOuter())
		{
			UObject* Object = CachedObjectInOuter.Object.Get();
			if (Object)
			{
				GCKeepObjects.Add(Object);
			}
		}
	}

	// Keep the generator and generated package referenced if we've passed the call to presave, or if we are keeping
	// any other objects referenced.
	if (bKeepingAnyObjects || Info.PackageData->GetSaveSubState() > ESaveSubState::Generation_CallPreSave)
	{
		bNeedsGeneratorPackage = true;
		if (&Info != &OwnerInfo)
		{
			UPackage* Package = Info.PackageData->GetPackage();
			if (!Package)
			{
				Package = FindObject<UPackage>(nullptr, *Info.PackageData->GetPackageName().ToString());
			}
			if (Package)
			{
				GCKeepPackages.Add(Package);
				GCKeepPackageDatas.Add(Info.PackageData);
			}
		}
	}

	if (bNeedsGeneratorPackage && !bCurrentGCHasKeptGeneratorPackage)
	{
		bCurrentGCHasKeptGeneratorPackage = true;
		UPackage* Package = OwnerInfo.PackageData->GetPackage();
		if (!Package)
		{
			Package = FindObject<UPackage>(nullptr, *OwnerInfo.PackageData->GetPackageName().ToString());
		}
		if (Package)
		{
			GCKeepPackages.Add(Package);
			GCKeepPackageDatas.Add(Info.PackageData);
		}
	}
}

void FGenerationHelper::PreGarbageCollectGCLifetimeData()
{
	// Starts at one because the caller of PreGarbageCollect has a ref
	uint32 HoldForGCRefCounts = 1;
	HoldForGCRefCounts += ReferenceFromKeepForAllSavedOrGC.IsValid() ? 1 : 0;
	// If the owner or any generated package is in progress and not stalled, do not uninitialize, because
	// the cooker might keep the packgae referenced (if it is e.g. in save state) even if the cooker does not
	// have a reference to the GenerationHelper from that package.
	// For stalled packages, if a generated package is stalled, we want to keep it in memory until GC,
	// but now that we have reached GC that stalled package is allowed to be demoted and released and does
	// not prevent uninitialize.
	// If the generator package is stalled, that's a complex case that we don't need to handle optimally;
	// just keep the entire generation helper referenced while the generator package is stalled.
	// Every stalled package will be holding a refcount; we need to subtract those refcounts when deciding
	// whether we have a reference from any non-stalled package.
	if (OwnerInfo.PackageData->IsStalled()
		|| OwnerInfo.PackageData->IsInProgress())
	{
		// Owner packagedata is stalled or in progress; do not uninitialize
		return;
	}
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		if (Info.PackageData->IsStalled())
		{
			if (Info.PackageData->GetParentGenerationHelper())
			{
				HoldForGCRefCounts += 1;
			}
		}
		else if (Info.PackageData->IsInProgress())
		{
			// Generated package is in progress and not stalled; do not uninitialize
			return;
		}
	}

	check(GetRefCount() >= HoldForGCRefCounts);
	if (GetRefCount() > HoldForGCRefCounts)
	{
		// Something else (generator save or generated package save, etc) is keeping us referenced
		// and we need to not allow destruction. Nothing further to do.
		return;
	}

	// We should either uninitialize or destroy after the garbage collect.
	// Usually we should not uninitialize unless the Generator package is going to be collected, but in this case we
	// are in a state where nothing in the cooker is depending on the package anymore (all generator and generated
	// packages are not in the save state or are stalled) so we do expect the generator package to be garbage
	// collected by the upcoming GC.
	// But for that to happen we have to drop our references to it from this FGenerationHelper, so we need
	// to uninitialize. Also mark that we should check for generator garbage collect in PostGarbageCollect.
	// Depending on the Splitter class, it may tolerate failure to GC the Generator package, in which case we
	// should not log this error.
	Uninitialize();
	bNeedConfirmGeneratorPackageDestroyed = IsRequiresGeneratorPackageDestructBeforeResplit();
}

void FGenerationHelper::PostGarbageCollectGCLifetimeData(FCookGCDiagnosticContext& Context)
{
	if (bNeedConfirmGeneratorPackageDestroyed)
	{
		VerifyGeneratorPackageGarbageCollected(Context);
		bNeedConfirmGeneratorPackageDestroyed = false;
	}

	if (!IsInitialized())
	{
		// ClearKeepForAllSavedOrGC is no longer required when Uninitialized after a GC
		// Note that this keep flag might be the last persistent reference to *this and *this will be deleted when the
		// caller of PostGarbageCollect drops its reference.
		ClearKeepForAllSavedOrGC();
	}
}

void FGenerationHelper::TrackGeneratedPackageListedRemotely(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
	const FIoHash& CurrentPackageHash)
{
	if (bReportedManifest)
	{
		if (!FindInfo(PackageData))
		{
			UE_LOGF(LogCook, Warning,
				"Unexpected generated package (discovery replication). A remote cookworker reported generated package %ls for generator %ls," " but when TryReportGenerationManifest was called on the CookDirector, this package was not listed." " This is unexpected and causes minor performance problems in the cook.",
				*PackageData.GetPackageName().ToString(), *GetOwner().GetPackageName().ToString());
		}
	}
	else
	{
		if (!FindInfoNoInitialize(PackageData))
		{
			bool bGenerator = false; // Cannot be the generator, if it were we would have found it
			PackagesToGenerate.Emplace(*this, PackageData, bGenerator);
		}
	}

	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		FPlatformData& PlatformData = PlatformPair.Value;
		FAssetPackageData* PreviousAssetData = PlatformData.PreviousGeneratedPackages.Find(PackageData.GetPackageName());
		if (PreviousAssetData)
		{
			// Copy the current value for the package's hash into the PreviousPackageData, for use by incremental cook's
			// calculation in FRequestCluster::TryCalculateIncrementallyUnmodified
			PreviousAssetData->SetPackageSavedHash(CurrentPackageHash);
		}
	}
}

void FGenerationHelper::MarkPackageSavedRemotely(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
	const ITargetPlatform* TargetPlatform, FWorkerId SourceWorkerId)
{
	FCookGenerationInfo* Info = FindInfoNoInitialize(PackageData);
	if (Info)
	{
		Info->FindCheckedPlatformData(TargetPlatform).SetHasSaved(*this, *Info, TargetPlatform,
			true /* bValue */, SourceWorkerId);
	}
}

void FGenerationHelper::MarkPackageIncrementallySkipped(FPackageData& PackageData,
	const ITargetPlatform* TargetPlatform, bool bIncrementallySkipped)
{
	FCookGenerationInfo* Info = FindInfoNoInitialize(PackageData);
	if (!Info)
	{
		check(&PackageData != &GetOwner()); // FindInfoNoInitialize should have succeeded on the owner.
		return;
	}

	bool bGenerator = &PackageData == &GetOwner();
	if (bGenerator)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose,
			"%ls: %ls",
			*WriteToString<256>(GetOwner().GetPackageName()),
			bIncrementallySkipped ? TEXT("generator found to be entirely incrementally skippable.")
			: TEXT("generator found to be NOT incrementally skippable. Generation will be reexecuted."));
	}
	else
	{
		UE_LOGF(LogCookGenerationHelper, Verbose,
			"%ls generated package %ls: %ls",
			*WriteToString<256>(GetOwner().GetPackageName()),
			*WriteToString<256>(PackageData.GetPackageName()),
			bIncrementallySkipped ? TEXT("generated package found to be incrementally skippable.")
			: TEXT("generated package found to be NOT incrementally skippable."));
	}
	if (bIncrementallySkipped)
	{
		FCookGenerationInfo::FPlatformData& PlatformData = Info->FindCheckedPlatformData(TargetPlatform);
		PlatformData.SetHasSaved(*this, *Info, TargetPlatform, true /* bValue */, FWorkerId::Local());
		PlatformData.SetIncrementallySkipped(true);
		if (bGenerator)
		{
			// The entire generator package has been skipped. Wait for the current cluster to complete
			// so we can mark all of our generated packages as skipped, but then clear the incremental data;
			// it will no longer be needed.
			GetOwner().GetPackageDatas().GetRequestQueue().AddRequestFenceListener(GetOwner().GetPackageName());
		}
	}
}

void FGenerationHelper::PostGarbageCollect(const TRefCountPtr<FGenerationHelper>& RefcountHeldByCaller, FCookGCDiagnosticContext& Context)
{
	PostGarbageCollectGCLifetimeData(Context);
	bCurrentGCHasKeptGeneratorPackage = false;
	bCurrentGCHasKeptGeneratorKeepPackages = false;
	if (!IsInitialized() || !IsValid())
	{
		return;
	}

	FPackageData& Owner = GetOwner();
	if (Owner.IsInStateProperty(EPackageStateProperty::Saving))
	{
		// If the package no longer exists, then UpdateSaveAfterGarbageCollect earlier in
		// UCookOnTheFlyServer::PostGarbageCollect should have demoted the package out of saving.
		// And if the package exists, then the SplitDataObject, which should be a public object within it, should
		// have been kept in memory by ConstructSoftGCPackageToObjectList. If the package or split object no longer
		// exist then we are in an invalid state and the savepackage might behave incorrectly.
		if (!Owner.GetPackage())
		{
			UPackage* FoundPackage = FindObject<UPackage>(nullptr, *Owner.GetPackageName().ToString());
			if (FoundPackage)
			{
				Owner.SetPackage(FoundPackage);
				UE_LOGF(LogCook, Warning,
					"CookPackageSplitter's package pointer was unexpectedly set to null by garbage collection while the package is still in the %ls state, %ls substate, but the package is still in memory." "\n\tPackage=%ls, Splitter=%ls.",
					LexToString(Owner.GetState()), LexToString(Owner.GetSaveSubState()),
					*Owner.GetPackageName().ToString(), *GetSplitDataObjectName().ToString());
			}
		}
		if (!Owner.GetPackage() || !GetWeakSplitDataObject())
		{
			UE_LOGF(LogCook, Error,
				"CookPackageSplitter's %ls was deleted by garbage collection while the package is still in the %ls state, %ls substate. This will break the generation." "\n\tPackage=%ls, Splitter=%ls.",
				(!Owner.GetPackage() ? TEXT("package") : TEXT("splitter object")),
				LexToString(Owner.GetState()), LexToString(Owner.GetSaveSubState()),
				*Owner.GetPackageName().ToString(), *GetSplitDataObjectName().ToString());
		}
	}
	else if (!IsUseInternalReferenceToAvoidGarbageCollect())
	{
		// After the Generator Package is saved, we drop our references to it and it can be garbage collected. If we
		// have any packages left to populate, our splitter contract requires that it be garbage collected because
		// we promise that the package is not partially GC'd during calls to PopulateGeneratedPackage or
		// PreSaveGeneratedPackage. The splitter can opt-out of this contract and keep it referenced itself.
		if (!Owner.IsInProgress() && !Owner.IsKeepReferencedDuringGC())
		{
			VerifyGeneratorPackageGarbageCollected(Context);
		}
	}

	bool bHasIssuedWarning = false;
	for (FCookGenerationInfo& Info : PackagesToGenerate)
	{
		if (FindObject<UPackage>(nullptr, *Info.PackageData->GetPackageName().ToString()))
		{
			if (!Info.PackageData->IsKeepReferencedDuringGC() && !Info.HasSavedEveryPlatform() && !bHasIssuedWarning)
			{
				UE_LOGF(LogCook, Warning,
					"PackageSplitter found a package it generated that was not removed from memory during garbage collection. This will cause errors later during population." "\n\tSplitter=%ls, Generated=%ls.", *GetSplitDataObjectName().ToString(),
					*Info.GetPackageName());
				
				{
					// Compute UCookOnTheFlyServer's references so they are gathered by OBJ REFS below 
					FScopeFindCookReferences(Owner.GetPackageDatas().GetCookOnTheFlyServer());

					StaticExec(nullptr, *FString::Printf(TEXT("OBJ REFS NAME=%s"),
						*Info.PackageData->GetPackageName().ToString()));
				}
				
				bHasIssuedWarning = true; // Only issue the warning once per GC
			}
		}
		else
		{
			Info.SetHasCreatedPackage(false);
		}
		for (TArray<TWeakObjectPtr<UPackage>>::TIterator Iter(Info.KeepReferencedPackages); Iter; ++Iter)
		{
			const TWeakObjectPtr<UPackage>& KeepPtr = *Iter;
			if (!KeepPtr.Get())
			{
				UE_LOGF(LogCook, Warning,
					"PackageSplitter returned a package in OutKeepReferencedPackages that the cooker tried to keep referenced, but it was removed by garbage collection anyway." " This might cause errors during save of the generated packages." "\n\tSplitter=%ls, Generated=%ls.",
					*GetSplitDataObjectName().ToString(),
					*Info.GetPackageName());
				Iter.RemoveCurrentSwap();
			}
		}
	}
}

void FGenerationHelper::VerifyGeneratorPackageGarbageCollected(FCookGCDiagnosticContext& Context)
{
	FString GeneratorPackageName = GetOwner().GetPackageName().ToString();
	UPackage* LocalOwnerPackage = FindObject<UPackage>(nullptr, *GeneratorPackageName);
	if (LocalOwnerPackage)
	{
		bool bWillRetry = false;
		bWillRetry = Context.TryRequestGCWithHistory() || bWillRetry;
		bWillRetry = Context.TryRequestFullGC() || bWillRetry;
		if (!bWillRetry)
		{
			// Might be called when uninitialized, so do not call GetSplitDataObjectNameIfAvailable
			FString Identifier;
			if (!SplitDataObjectName.IsNone())
			{
				Identifier = FString::Printf(TEXT("Splitter=%s"), *SplitDataObjectName.ToString());
			}
			else
			{
				Identifier = FString::Printf(TEXT("GeneratorPackage=%s"), *GeneratorPackageName);
			}
			UE_LOGF(LogCook, Error,
				"PackageSplitter found the Generator package still in memory after it should have been deleted by GC." "\n\tThis is unexpected since garbage has been collected and the package should have been unreferenced so it should have been collected, and will break population of Generated packages." "\n\tSplitter=%ls", *Identifier);
			EReferenceChainSearchMode SearchMode = EReferenceChainSearchMode::Shortest
				| EReferenceChainSearchMode::PrintAllResults
				| EReferenceChainSearchMode::FullChain;
			FReferenceChainSearch RefChainSearch(LocalOwnerPackage, SearchMode, ELogVerbosity::Display);
		}
	}
}

void FGenerationHelper::UpdateSaveAfterGarbageCollect(const FPackageData& PackageData, bool& bInOutDemote)
{
	if (!IsInitialized() || !IsValid())
	{
		return;
	}
	FCookGenerationInfo* Info = FindInfo(PackageData);
	if (!Info)
	{
		bInOutDemote = true;
		return;
	}

	if (!Info->IsGenerator())
	{
		UPackage* LocalOwnerPackage = OwnerPackage.Get();
		if (!LocalOwnerPackage || !LocalOwnerPackage->IsFullyLoaded())
		{
			if (IsUseInternalReferenceToAvoidGarbageCollect() && Info->HasTakenOverCachedCookedPlatformData())
			{
				UE_LOGF(LogCook, Error,
					"The generator package of a generated package was unloaded during garbage collection after the generated package started generating. " "This will cause errors during save of the generated package." "\n\tSplitter=%ls, Generated=%ls" "\n\tGeneratorPackage=%ls, GeneratorPackage->IsFullyLoaded()=%ls",
					*GetSplitDataObjectName().ToString(),
					*Info->PackageData->GetPackageName().ToString(),
					LocalOwnerPackage == nullptr ? TEXT("<null>") : TEXT("<nonnull>"),
					*(::LexToString(LocalOwnerPackage == nullptr ? false : LocalOwnerPackage->IsFullyLoaded())));
			}
			bInOutDemote = true;
			return;
		}
	}

	if (bInOutDemote && IsUseInternalReferenceToAvoidGarbageCollect() && Info->HasTakenOverCachedCookedPlatformData())
	{
		// No public objects should have been deleted; we are supposed to keep them referenced by keeping the package
		// referenced in UCookOnTheFlyServer::PreGarbageCollect, and the package keeping its public objects referenced
		// by UPackage::AddReferencedObjects. Since no public objects were deleted, our caller should not have
		// set bInOutDemote=true.
		// Allowing demotion after the splitter has started moving objects breaks our contract with the splitter
		// and can cause a crash. So log this as an error.
		// For better feedback, look in our extra data to identify the name of the public UObject that was deleted.
		FString DeletedObject;
		if (!PackageData.GetPackage())
		{
			DeletedObject = FString::Printf(TEXT("UPackage %s"), *PackageData.GetPackageName().ToString());
		}
		else
		{
			TSet<UObject*> ExistingObjectsAfterSave;
			for (const FCachedObjectInOuter& CachedObjectInOuter : PackageData.GetCachedObjectsInOuter())
			{
				UObject* Ptr = CachedObjectInOuter.Object.Get();
				if (Ptr)
				{
					ExistingObjectsAfterSave.Add(Ptr);
				}
			}

			for (const TPair<UObject*, FCachedObjectInOuterGeneratorInfo>& Pair : Info->CachedObjectsInOuterInfo)
			{
				if (Pair.Value.bPublic && !ExistingObjectsAfterSave.Contains(Pair.Key))
				{
					DeletedObject = Pair.Value.FullName;
					break;
				}
			}
			if (DeletedObject.IsEmpty())
			{
				if (!PackageData.GetPackage()->IsFullyLoaded())
				{
					DeletedObject = FString::Printf(TEXT("UPackage %s is no longer FullyLoaded"),
						*PackageData.GetPackageName().ToString());
				}
				else
				{
					DeletedObject = TEXT("<Unknown>");
				}
			}
		}
		UE_LOGF(LogCook, Error,
			"A %ls package had some of its UObjects deleted during garbage collection after it started generating. This will cause errors during save of the package." "\n\tDeleted object: %ls" "\n\tSplitter=%ls%ls",
			Info->IsGenerator() ? TEXT("Generator") : TEXT("Generated"),
			*DeletedObject,
			*GetSplitDataObjectName().ToString(),
			Info->IsGenerator() ? TEXT(".") : *FString::Printf(TEXT(", Generated=%s."),
				*Info->PackageData->GetPackageName().ToString()));
	}

	// Remove raw pointers from CachedObjectsInOuterInfo if they no longer exist in the weakpointers 
	// in CachedObjectsInOuter
	TSet<UObject*> CachedObjectsInOuterSet;
	for (FCachedObjectInOuter& CachedObjectInOuter : Info->PackageData->GetCachedObjectsInOuter())
	{
		UObject* Object = CachedObjectInOuter.Object.Get();
		if (Object)
		{
			CachedObjectsInOuterSet.Add(Object);
		}
	}
	for (TMap<UObject*, FCachedObjectInOuterGeneratorInfo>::TIterator Iter(Info->CachedObjectsInOuterInfo);
		Iter; ++Iter)
	{
		if (!CachedObjectsInOuterSet.Contains(Iter->Key))
		{
			Iter.RemoveCurrent();
		}
	}
}

FCookGenerationInfo::FCookGenerationInfo(FGenerationHelper& GenerationHelper, FPackageData& InPackageData, bool bInGenerator)
	: PackageData(&InPackageData)
	, bCreateAsMap(false), bHasCreatedPackage(false), bTakenOverCachedCookedPlatformData(false)
	, bIssuedUndeclaredMovedObjectsWarning(false), bGenerator(bInGenerator), bHasCalledPopulate(false)
	, bCopyChunkAssignmentFromGenerator(true)
{
	// The Info for the generator is created in the constructor of the GenerationHelper, and PlatformDatas is not yet readable.
	// The GenerationHelper constructor will set the PlatformDatas on this Info later on in its constructor.
	if (!bInGenerator)
	{
		for (TPair<const ITargetPlatform*, FGenerationHelper::FPlatformData>& PlatformPair
			: GenerationHelper.PlatformDatas)
		{
			PlatformDatas.Add(PlatformPair.Key);
		}
	}
}

void FCookGenerationInfo::Uninitialize()
{
	// Check that we have left the save state first, since other assertions assume we have left the save state
	checkf(PackageData->GetSaveSubState() == ESaveSubState::StartSave,
		TEXT("Cooker bug: Expected FCookGenerationInfo::Uninitialize to not be called for a package in an active save state, ")
		TEXT("but %s package %s has SaveSubState %s."),
		bGenerator ? TEXT("generator") : TEXT("generated"), *GetPackageName(),
		LexToString(PackageData->GetSaveSubState()));

	PackageHash.Reset();
	RelativePath.Empty();
	GeneratedRootPath.Empty();
	GenerationHash.Reset();
	PackageDependencies.Empty();
	BuildResultDependencies.Empty();
	// Keep PackageData; it is allowed in the uninitialized state
	KeepReferencedPackages.Empty();
	// CachedObjectsInOuterInfo should have been cleared before uninitialize, either by leaving the save state, or
	// calling ReleaseCookedPlatformData.
	checkf(CachedObjectsInOuterInfo.IsEmpty(),
		TEXT("Cooker bug: Expected FCookGenerationInfo::Uninitialize to not be called for a package in an active save state, ")
		TEXT("but %s package %s has non-empty CachedObjectsInOuterInfo. Package State == %s, SaveSubState == %s"),
		bGenerator ? TEXT("generator") : TEXT("generated"), *GetPackageName(),
		LexToString(PackageData->GetState()), LexToString(PackageData->GetSaveSubState()));
	// Keep SavedOnWorker; it is allowed in the uninitialized state

	// Keep PlatformDatas; they are allowed in the unitialized state
	// Keep PlatformData.AssetPackageData; it is allowed in the uninitialized state
	// Keep PlatformData.bHasSaved; it is allowed in the uninitialized state
	// Keep PlatformData.bIncrementallySkipped; it is allowed in the uninitialized state

	bCreateAsMap = false;
	bHasCreatedPackage = false;
	bTakenOverCachedCookedPlatformData = false;
	bIssuedUndeclaredMovedObjectsWarning = false;
	// Keep bGenerator; it is allowed in the uninitialized state
	bHasCalledPopulate = false;
}

void FCachedObjectInOuterGeneratorInfo::Initialize(UObject* Object)
{
	if (Object)
	{
		FullName = Object->GetFullName();
		bPublic = Object->HasAnyFlags(RF_Public);
	}
	else
	{
		FullName.Empty();
		bPublic = false;
	}

	bInitialized = true;
}

void FCookGenerationInfo::TakeOverCachedObjectsAndAddMoved(FGenerationHelper& GenerationHelper,
	TArray<FCachedObjectInOuter>& CachedObjectsInOuter, TArray<UObject*>& MovedObjects)
{
	CachedObjectsInOuterInfo.Reset();

	for (FCachedObjectInOuter& ObjectInOuter : CachedObjectsInOuter)
	{
		UObject* Object = ObjectInOuter.Object.Get();
		if (Object)
		{
			CachedObjectsInOuterInfo.FindOrAdd(Object).Initialize(Object);
		}
	}

	TArray<UObject*> ChildrenOfMovedObjects;
	for (UObject* Object : MovedObjects)
	{
		if (!IsValid(Object))
		{
			UE_LOGF(LogCook, Warning,
				"CookPackageSplitter found non-valid object %ls returned from %ls on Splitter %ls%ls. Ignoring it.",
				Object ? *Object->GetFullName() : TEXT("<null>"),
				IsGenerator() ? TEXT("PopulateGeneratorPackage") : TEXT("PopulateGeneratedPackage"),
				*GenerationHelper.GetSplitDataObjectName().ToString(),
				IsGenerator() ? TEXT("") : *FString::Printf(TEXT(", Package %s"),
					*PackageData->GetPackageName().ToString()));
			continue;
		}
		FCachedObjectInOuterGeneratorInfo& Info = CachedObjectsInOuterInfo.FindOrAdd(Object);
		if (!Info.bInitialized)
		{
			Info.Initialize(Object);
			Info.bMoved = true;
			Info.bMovedRoot = true;
			CachedObjectsInOuter.Emplace(Object);
			GetObjectsWithOuter(Object, ChildrenOfMovedObjects, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags,
				EInternalObjectFlags::Garbage);
		}
	}

	for (UObject* Object : ChildrenOfMovedObjects)
	{
		check(IsValid(Object));
		FCachedObjectInOuterGeneratorInfo& Info = CachedObjectsInOuterInfo.FindOrAdd(Object);
		if (!Info.bInitialized)
		{
			Info.Initialize(Object);
			Info.bMoved = true;
			CachedObjectsInOuter.Emplace(Object);
		}
	}

	SetHasTakenOverCachedCookedPlatformData(true);
}

EPollStatus FCookGenerationInfo::RefreshPackageObjects(FGenerationHelper& GenerationHelper, UPackage* Package,
	bool& bOutFoundNewObjects, ESaveSubState DemotionState)
{
	// As in FPackageData::RefreshObjectCache, we add any new objects but do not remove any removed objects.
	bOutFoundNewObjects = false;
	TArray<UObject*> CurrentObjectsInOuter;
	GetObjectsWithOuter(Package, CurrentObjectsInOuter, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags,
		EInternalObjectFlags::Garbage);

	TArray<FCachedObjectInOuter>& CachedObjectsInOuter = PackageData->GetCachedObjectsInOuter();
	UObject* FirstNewObject = nullptr;
	for (UObject* Object : CurrentObjectsInOuter)
	{
		FCachedObjectInOuterGeneratorInfo& Info = CachedObjectsInOuterInfo.FindOrAdd(Object);
		if (!Info.bInitialized)
		{
			Info.Initialize(Object);
			CachedObjectsInOuter.Emplace(Object);
			if (!FirstNewObject)
			{
				FirstNewObject = Object;
			}
		}
	}
	SetHasTakenOverCachedCookedPlatformData(true); // Set it again in case we unset it at any point
	bOutFoundNewObjects = FirstNewObject != nullptr;

	if (FirstNewObject != nullptr && DemotionState != ESaveSubState::Last)
	{
		PackageData->SetSaveSubState(DemotionState);
		if (++PackageData->GetNumRetriesBeginCacheOnObjects() > FPackageData::GetMaxNumRetriesBeginCacheOnObjects())
		{
			UE_LOGF(LogCook, Error,
				"Cooker has repeatedly tried to call BeginCacheForCookedPlatformData on all objects in a generated package, but keeps finding new objects.\n" "Aborting the save of the package; programmer needs to debug why objects keep getting added to the package.\n" "Splitter: %ls%ls. Most recent created object: %ls.",
				*GenerationHelper.GetSplitDataObjectName().ToString(),
				IsGenerator() ? TEXT("") : *FString::Printf(TEXT(", Package: %s"),
					*PackageData->GetPackageName().ToString()),
				*FirstNewObject->GetFullName());
			return EPollStatus::Error;
		}
	}
	return EPollStatus::Success;
}

void FGenerationHelper::OnAddPendingCookedPlatformData(UCookOnTheFlyServer& COTFS, FPackageData& AssignedPackageData,
	UObject* Obj)
{
	if (&AssignedPackageData != OwnerInfo.PackageData)
	{
		// We only need to coordinate saves if the assigned package is the generator package.
		return;
	}
	UPackage* AssignedPackage = AssignedPackageData.GetPackage();
	check(AssignedPackage); // Only be called for assigned packages in the save state.
	check(Obj); // Only called for non-null objects.
	UPackage* ParentPackage = Obj->GetPackage();
	if (ParentPackage == AssignedPackage)
	{
		// No action necessary if the object is within the assigned package.
		return;
	}
	FPackageData* ParentPackageData = COTFS.PackageDatas->FindPackageDataByPackageName(ParentPackage->GetFName());
	if (!ParentPackageData)
	{
		// Object is in a temporary or other package that the cooker does not track. The cooker will
		// treat it as still being in the assigned package; SavePackage will make it an import if referenced.
		return;
	}
	FCookGenerationInfo* ParentInfo = FindInfo(*ParentPackageData);
	if (!ParentInfo)
	{
		// Object moved into another cookable package, but not a generated package of this generator.
		// During cook of the generator we will treat it as still being in the generator package; SavePackage
		// will make it an import if referenced.
		return;
	}
	// ParentPackage != AssignedPackage and AssignedPackage == OwnerPackage, so ParentInfo != OwnerInfo
	check(ParentInfo != &OwnerInfo); 

	// Block any further operations on both the generator and generated package it moved into, until the Object is no
	// longer pending. We do this because we need to guarantee the cooker contract that we do not call
	// BeginCachedCookedPlatformData on it again until it has returned true from IsCachedCookedPlatformDataLoaded, and
	// operations on the generated package might find it in the generated package and call BeginCached.
	AddBlockingPendingCookedPlatformDataObject(Obj);
}

void FGenerationHelper::AddBlockingPendingCookedPlatformDataObject(UObject* Object)
{
	BlockingPendingCookedPlatformDatas.Add(FWeakObjectPtr(Object));
}

bool FGenerationHelper::HasBlockingPendingCookedPlatformDataObjects()
{
	FPackageDatas& PackageDatas = OwnerInfo.PackageData->GetPackageDatas();
	for (auto Iter = BlockingPendingCookedPlatformDatas.CreateIterator(); Iter; ++Iter)
	{
		UObject* Object = Iter->Get();
		if (Object && !PackageDatas.GetPendingCookedPlatformDatas(Object).IsEmpty())
		{
			return true;
		}
		Iter.RemoveCurrentSwap();
	}
	return false;
}

bool FGenerationHelper::ShouldKeepWaitingOnMovedPendingCookedPlatformData(FPendingCookedPlatformData& Pending,
	FPackageData& AssignedPackage, UObject* Object)
{
	check(Object);
	// Only allow generators to keep waiting on objects in other packages
	if (&AssignedPackage != OwnerInfo.PackageData)
	{
		return false;
	}
	UPackage* CurrentPackage = Object->GetPackage();
	check(CurrentPackage);
	FPackageData* CurrentPackageData =
		AssignedPackage.GetPackageDatas().FindPackageDataByPackageName(CurrentPackage->GetFName());
	if (!CurrentPackageData)
	{
		// Only wait on objects in cookable packages
		return false;
	}
	FCookGenerationInfo* CurrentInfo = FindInfo(*CurrentPackageData);
	if (!CurrentInfo)
	{
		// Only wait on objects in generated packages of this generator.
		return false;
	}
	return true;
}

void FCookGenerationInfo::AddKeepReferencedPackages(FGenerationHelper& GenerationHelper,
	TArray<UPackage*>& InKeepReferencedPackages)
{
	KeepReferencedPackages.Reserve(KeepReferencedPackages.Num() + InKeepReferencedPackages.Num());
	for (UPackage* Package : InKeepReferencedPackages)
	{
		TWeakObjectPtr<UPackage>& WeakPtr = KeepReferencedPackages.Emplace_GetRef(Package);
		if (!WeakPtr.Get())
		{
			UE_LOGF(LogCook, Warning,
				"PackageSplitter returned a package in OutKeepReferencedPackages that is already marked as garbage." " This might cause errors during save of the generated packages." "\n\tSplitter=%ls, Generated=%ls.",
				*GenerationHelper.GetSplitDataObjectName().ToString(),
				*GetPackageName());

			KeepReferencedPackages.Pop(EAllowShrinking::No);
		}
	}
}

void FCookGenerationInfo::CreatePackageHash()
{
	FBlake3 Blake3;
	Blake3.Update(&GenerationHash, sizeof(GenerationHash));
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	for (const FAssetDependency& Dependency : PackageDependencies)
	{
		TOptional<FAssetPackageData> DependencyData =
			AssetRegistry.GetAssetPackageDataCopy(Dependency.AssetId.PackageName);
		if (DependencyData)
		{
			Blake3.Update(&DependencyData->GetPackageSavedHash().GetBytes(),
				sizeof(decltype(DependencyData->GetPackageSavedHash().GetBytes())));
		}
	}
	PackageHash = FIoHash(Blake3.Finalize());
}

void FCookGenerationInfo::LegacyIterativeCookValidateOrClear(FGenerationHelper& GenerationHelper,
	const ITargetPlatform* TargetPlatform, const FIoHash& PreviousPackageHash,
	bool& bOutLegacyIterativeUnmodified)
{
	UCookOnTheFlyServer& COTFS = GenerationHelper.GetOwner().GetPackageDatas().GetCookOnTheFlyServer();
	bOutLegacyIterativeUnmodified = PreviousPackageHash == this->PackageHash;
	if (bOutLegacyIterativeUnmodified)
	{
		// If not directly modified, mark it as indirectly modified if any of its dependencies
		// were detected as modified during PopulateCookedPackages.
		for (const FAssetDependency& Dependency : this->PackageDependencies)
		{
			FPackageData* DependencyData =
				COTFS.PackageDatas->FindPackageDataByPackageName(Dependency.AssetId.PackageName);
			if (!DependencyData)
			{
				bOutLegacyIterativeUnmodified = false;
				break;
			}
			FPackagePlatformData* DependencyPlatformData = DependencyData->FindPlatformData(TargetPlatform);
			if (!DependencyPlatformData || !DependencyPlatformData->IsIncrementallyUnmodified())
			{
				bOutLegacyIterativeUnmodified = false;
				break;
			}
		}
	}

	if (bOutLegacyIterativeUnmodified)
	{
		PackageData->FindOrAddPlatformData(TargetPlatform).SetIncrementallyUnmodified(true);
	}
	bool bShouldLegacyIterativelySkip = bOutLegacyIterativeUnmodified;
	ICookedPackageWriter& PackageWriter = COTFS.FindOrCreatePackageWriter(TargetPlatform);
	ICookedPackageWriter::FUpdatePackageModifiedStatusContext Context;
	Context.PackageName = PackageData->GetPackageName();
	Context.bIncrementallyUnmodified = bOutLegacyIterativeUnmodified;
	Context.bPreviouslyCooked = true;
	Context.bInOutShouldIncrementallySkip = bShouldLegacyIterativelySkip;
	PackageWriter.UpdatePackageModifiedStatus(Context);
	bShouldLegacyIterativelySkip = Context.bInOutShouldIncrementallySkip;
	if (bShouldLegacyIterativelySkip)
	{
		PackageData->SetPlatformCooked(TargetPlatform, ECookResult::Succeeded);
		TConstArrayView<const ITargetPlatform*> SessionPlatforms = COTFS.PlatformManager->GetSessionPlatforms();
		if (SessionPlatforms.Num() > 0 && TargetPlatform == SessionPlatforms[0])
		{
			COOK_STAT(++DetailedCookStats::NumPackagesIncrementallySkipped);
		}
		// Declare the package to the EDLCookInfo verification so we don't warn about missing exports from it
		FEDLCookCheckerThreadState::Get().AddPackageWithUnknownExports(PackageData->GetPackageName());
	}
	else
	{
		COTFS.DeleteOutputForPackage(PackageData->GetPackageName(), TargetPlatform);
	}
}

namespace GenerationHelperPrivate
{

enum class ERequiredSaveOrder
{
	None,
	GeneratorFirst,
	GeneratedFirst,
};

ERequiredSaveOrder RequiredSaveOrder = ERequiredSaveOrder::None;

}

void FGenerationHelper::SetBeginCookConfigSettings()
{
	const TCHAR* CommandLine = FCommandLine::Get();

	FString SaveOrder;
	GConfig->GetString(TEXT("CookSettings"), TEXT("MPCookGeneratorSaveOrder"), SaveOrder, GEditorIni);
	FParse::Value(FCommandLine::Get(), TEXT("-MPCookGeneratorSaveOrder="), SaveOrder);
	if (SaveOrder == TEXT("GeneratorFirst"))
	{
		GenerationHelperPrivate::RequiredSaveOrder = GenerationHelperPrivate::ERequiredSaveOrder::GeneratorFirst;
	}
	else if (SaveOrder == TEXT("GeneratedFirst"))
	{
		GenerationHelperPrivate::RequiredSaveOrder = GenerationHelperPrivate::ERequiredSaveOrder::GeneratedFirst;
	}
	else
	{
		if (!SaveOrder.IsEmpty() && SaveOrder != TEXT("None"))
		{
			UE_LOGF(LogCook, Error,
				"Invalid setting Editor:[CookSettings]:MPCookGeneratorSaveOrder=%ls. Expected values are 'GeneratorFirst', 'GeneratedFirst', or 'None'. Falling back to default 'None'.",
				*SaveOrder);
		}
		GenerationHelperPrivate::RequiredSaveOrder = GenerationHelperPrivate::ERequiredSaveOrder::None;
	}
}

bool FGenerationHelper::IsGeneratorSavedFirst()
{
	return GenerationHelperPrivate::RequiredSaveOrder == GenerationHelperPrivate::ERequiredSaveOrder::GeneratorFirst;
}

bool FGenerationHelper::IsGeneratedSavedFirst()
{
	return GenerationHelperPrivate::RequiredSaveOrder == GenerationHelperPrivate::ERequiredSaveOrder::GeneratedFirst;
}

}