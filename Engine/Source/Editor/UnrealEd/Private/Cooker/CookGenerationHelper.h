// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Cooker/BuildResultDependenciesMap.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookTypes.h"
#include "Cooker/MPCollector.h"
#include "CookPackageSplitter.h"
#include "Hash/Blake3.h"
#include "Interfaces/ITargetPlatform.h"
#include "IO/IoHash.h"
#include "Misc/Optional.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"

class ITargetPlatform;
class UCookOnTheFlyServer;
namespace UE::Cook { class FCookGCDiagnosticContext; }
namespace UE::Cook { struct FGenerationHelper; }
namespace UE::EditorDomain { struct FPackageDigest; }

namespace UE::Cook::CookPackageSplitter
{

/** Data necessary to support the ICookPackageSplitter::FPopulateContext interface. */
struct FPopulateContextData
{
	TArray<UPackage*> KeepReferencedPackages;
	TArray<UObject*> ObjectsToMove;
	UE::Cook::FBuildResultDependenciesMap BuildResultDependencies;
	TConstArrayView<ICookPackageSplitter::FGeneratedPackageForPopulate> GeneratedPackages;
	UPackage* OwnerPackage = nullptr;
	UObject* OwnerObject = nullptr;
	const ICookPackageSplitter::FGeneratedPackageForPopulate* TargetGeneratedPackage = nullptr;
};

} // namespace UE::Cook::CookPackageSplitter::Private

namespace UE::Cook
{

/**
 * Extra information about CachedObjectsInOuter that a GenerationHelper needs to know for diagnostics.
 * The GenerationHelper constructs an associated array (aka TMap) of these structures when it takes
 * over the CachedObjectsInOuter.
 */
struct FCachedObjectInOuterGeneratorInfo
{
public:
	/** Object->GetFullName() before it was deleted. */
	FString FullName;
	/** Has Initialize been called on *this. */
	bool bInitialized = false;
	/** Object->GetFlags() had RF_Public when the info was initialized. */
	bool bPublic = false;
	/** bMovedRoot is true, or this is a child object of such a moved object. */
	bool bMoved = false;
	/** Splitter informed us that object was moved into this package from another package. */
	bool bMovedRoot = false;

public:
	void Initialize(UObject* Object);
};

/**
 * Struct used with ICookPackageSplitter, which contains the state information for the save of either the
 * Generator package or one of its Generated packages.
 */
struct FCookGenerationInfo
{
public:
	FCookGenerationInfo(FGenerationHelper& GenerationHelper, FPackageData& InPackageData, bool bInGenerator);

	bool IsCreateAsMap() const;
	void SetIsCreateAsMap(bool bValue);
	bool HasCreatedPackage() const;
	void SetHasCreatedPackage(bool bValue);
	bool HasSavedEveryPlatform() const;
	bool HasTakenOverCachedCookedPlatformData() const;
	void SetHasTakenOverCachedCookedPlatformData(bool bValue);
	bool HasIssuedUndeclaredMovedObjectsWarning() const;
	void SetHasIssuedUndeclaredMovedObjectsWarning(bool bValue);
	bool IsGenerator() const;
	void SetIsGenerator(bool bValue);
	bool HasCalledPopulate() const;
	void SetHasCalledPopulate(bool bValue);
	bool IsCopyChunkAssignmentFromGenerator() const;
	void SetCopyChunkAssignmentFromGenerator(bool bValue);

	/**
	 * Steal the list of cached objects to call BeginCacheForCookedPlatformData on from the PackageData,
	 * and add them to this. Also add NewObjects reported by the splitter that will be moved into the package.
	 */
	void TakeOverCachedObjectsAndAddMoved(FGenerationHelper& GenerationHelper,
		TArray<FCachedObjectInOuter>& CachedObjectsInOuter, TArray<UObject*>& MovedObjects);
	/**
	 * Fetch all the objects currently in the package and add them to the list of objects that need
	 * BeginCacheForCookedPlatformData.
	 * Reports whether new objects were found. If DemotionState is not ESaveSubState::Last,
	 * will SetSaveSubState back to DemotionState
	 * if new objects were found, and will error exit if this demotion has happened too many times.
	 */
	EPollStatus RefreshPackageObjects(FGenerationHelper& GenerationHelper, UPackage* Package,
		bool& bOutFoundNewObjects, ESaveSubState DemotionState);

	void AddKeepReferencedPackages(FGenerationHelper& GenerationHelper, TArray<UPackage*>& InKeepReferencedPackages);

	/** Create the hash for this generated package, based on dependencies and GenerationHash. */
	void CreatePackageHash();

	/**
	 * If the package has legacyiterative results from a previous cook that were not invalidated by dependency changes,
	 * test whether they need to be invalidated now and clear the results if so. Only called in legacyiterative;
	 * incremental cooks handle invalidation by querying the TargetDomainDigest during the RequestCluster.
	 */
	void LegacyIterativeCookValidateOrClear(FGenerationHelper& GeneratorHelper, const ITargetPlatform* TargetPlatform,
		const FIoHash& PreviousHash, bool& bOutLegacyIterativeUnmodified);

	TConstArrayView<FAssetDependency> GetDependencies() const;

	/** Return the packagename, for use in debug messages. Handles PackageData==nullptr by returning RelativePath. */
	FString GetPackageName() const;

	/**
	 * Reset this info to a state appropriate for an uninitialized GenerationHelper; all references to UObjects
	 * are dropped, but information necessary even when in the uninitialized GenerationHelper state is kept.
	 */
	void Uninitialize();

public:
	// When adding a new variable, add it to Uninitialize as well
	FIoHash PackageHash;
	FString RelativePath;
	FString GeneratedRootPath;
	FBlake3Hash GenerationHash;
	TArray<FAssetDependency> PackageDependencies;
	FBuildResultDependenciesMap BuildResultDependencies;
	/** Cannot be null, set in constructor */
	FPackageData* PackageData;
	TArray<TWeakObjectPtr<UPackage>> KeepReferencedPackages;
	TMap<UObject*, FCachedObjectInOuterGeneratorInfo> CachedObjectsInOuterInfo;
	FWorkerId SavedOnWorker = FWorkerId::Invalid();

private:
	struct FPlatformData
	{
		TOptional<FAssetPackageData> AssetPackageData;
		bool bHasSaved : 1;
		bool bIncrementallySkipped : 1;

		FPlatformData();
		bool IsIncrementallySkipped() const;
		void SetIncrementallySkipped(bool bValue);
		bool HasSaved() const;
		void SetHasSaved(FGenerationHelper& GenerationHelper, FCookGenerationInfo& Info,
			const ITargetPlatform* TargetPlatform, bool bValue, FWorkerId SourceWorkerId);
	};

private:
	FPlatformData& FindCheckedPlatformData(const ITargetPlatform* TargetPlatform);

private:

	TMap<const ITargetPlatform*, FPlatformData> PlatformDatas;
	bool bCreateAsMap : 1;
	bool bHasCreatedPackage : 1;
	bool bTakenOverCachedCookedPlatformData : 1;
	bool bIssuedUndeclaredMovedObjectsWarning : 1;
	bool bGenerator : 1;
	bool bHasCalledPopulate : 1;
	bool bCopyChunkAssignmentFromGenerator : 1;

	friend UE::Cook::FGenerationHelper;
};

/**
 * Helper that wraps an ICookPackageSplitter, gets/caches packages to generate, and is a reference-counted
 * collection of cached data and helper functions to save and list the generated packages.
 * 
 * TODO: MultiplatformCook: We do not yet support Packages that use a GenerationHelper being cooked for some platforms
 * in a multiplatformcook and not others. And if the package is not incrementally skipped for one platform then it is
 * not incrementally skipped for any platform. We don't support per-platform incremental cooks because allowing
 * per-platform results would require extra tests throughout PumpSaves and the events on GenerationHelper to check for
 * which platforms are enabled, and we don't have those yet. We don't support per-platform reachability for the same
 * reason, and additionally because reachability might be discovered for one platform after the save already started
 * for another platform, and we would need to handle the state transition to restart the generation with the new
 * platform accommodated. We will eventually need to implement both per-platform incremental cooks and per-platform
 * reachability to fully support per-platform customization of the cook graph. See TODO_COOKGENERATIONHELPER for
 * code blocks where we implement the workarounds for this todo.
 */
struct FGenerationHelper : public UE::Private::FQueryableRefCountedObject
{
public:
	/** Store the provided CookPackageSplitter and prepare the packages to generate. */
	FGenerationHelper(UE::Cook::FPackageData& InOwner);
	~FGenerationHelper();

	/**
	 * Early exits if already initialized. Otherwise, loads the package if not loaded and searches it for a
	 * splitter and creates the splitter.
	 * If load/search/creation fails, the GenerationHelper will be set to invalid @see IsValid.
	 *
	 * Unless otherwise stated, all public functions call Initialize if not already initialized.
	 */
	void Initialize();
	/**
	 * Version of Initialize that receives the splitterobject from the caller rather than needing to search
	 * for it, as an optimization for callers that have it already. If already initialized, the input Splitter
	 * will be destroyed and the function will early exit.
	 */
	void Initialize(const UObject* InSplitDataObject,
		UE::Cook::Private::FRegisteredCookPackageSplitter* InRegisteredSplitterType,
		TUniquePtr<ICookPackageSplitter>&& InSplitter);
	/** Version of Initialize that sets IsValid=false. */
	void InitializeAsInvalid();
	/**
	 * Clear all self references which keep this GenerationHelper from destructing. These could have been set by the
	 * SetKeep... functions. The Owner PackageData will keep a pointer to this until all references from self, from
	 * child generated PackageDatas, and other, are released. Note this means that we assert in the owner PackageData's
	 * destructor if any references remain; PackageDatas can not be deleted until all such references are cleared.
	 * Does not call Initialize.
	 */
	void ClearSelfReferences();

	/**
	 * GenerationHelpers can be created uninitialized at the start of cook, for incremental cooks. We initialize
	 * them on demand, which requires loading the package if not already loaded.
	 * IsInitialized reports whether initialization has been attempted. @see IsValid.
	 * Does not call Initialize.
	 */
	bool IsInitialized() const;
	/**
	 * Since GenerationHelpers can be created before we load the package (@see IsInitialized) we might incorrectly
	 * create one for a package that does not have a splitter. If so, when we load the package and discover that,
	 * we set status to invalid. When in this state, all public functions are still valid to call, but will be
	 * noops and will return empty data.
	 */
	bool IsValid();

	/**
	 * True if ReportGenerationManifest requested that the generator package should be cooksaved as well. Returns true
	 * if not initialized. Does not call Initialize. 
	 */
	bool IsSaveGenerator() const;
	/**
	 * True if we detected that we need to skip save of the generator package because it was already cooked in the
	 * basegame and we only need to cook any new generated packages that it has.
	 */
	bool IsSkipSaveExistingGenerator() const;
	/** Accessor for the packages to generate, empty if invalid or TryReportGenerationManifest was not yet called. */
	TArrayView<FCookGenerationInfo> GetPackagesToGenerate();
	/** Return the GenerationInfo used to save the generator package's UPackage. Does not call Initialize. */
	FCookGenerationInfo& GetOwnerInfo();
	const FCookGenerationInfo& GetOwnerInfo() const;
	/** Return owner FPackageData. Does not call Initialize. */
	FPackageData& GetOwner();
	/** Return the GenerationInfo for the given PackageData, or null if not found. */
	FCookGenerationInfo* FindInfo(const FPackageData& PackageData);
	const FCookGenerationInfo* FindInfo(const FPackageData& PackageData) const;
	/** Return CookPackageSplitter. Will return null if !IsValid.*/
	ICookPackageSplitter* GetCookPackageSplitterInstance() const;
	/** Return RegisteredSplitterType. Will return null if !IsValid. */
	UE::Cook::Private::FRegisteredCookPackageSplitter* GetRegisteredSplitterType() const;
	/** Return the SplitDataObject's FullObjectPath. Will return NAME_None if !IsValid. */
	const FName GetSplitDataObjectName() const;
	/** Return the Splitter's value for virtual bool UseInternalReferenceToAvoidGarbageCollect(). */
	bool IsUseInternalReferenceToAvoidGarbageCollect() const;
	/**
	 * Return the Splitter's value for virtual bool RequiresGeneratorPackageDestructBeforeResplit(). Returns false if not
	 * initialized. Does not call Initialize.
	 */
	bool IsRequiresGeneratorPackageDestructBeforeResplit() const;
	/** Return the Splitter's value for virtual bool DoesGeneratedRequireGenerator(). */
	ICookPackageSplitter::EGeneratedRequiresGenerator DoesGeneratedRequireGenerator() const;
	/** Return the pointer to the SplitDataObject. Returns null if not in memory or marked as garbage. */
	UObject* GetWeakSplitDataObject();
	/**
	 * Return our cached pointer to the SplitDataObject if set. If not set, load the package and find it. Can still
	 * return null if !IsValid or if not found even after loading the package.
	 */
	UObject* FindOrLoadSplitDataObject();
	/** Return the ObjectsToMove that were returned for the generator package. Does not call initialize. */
	TConstArrayView<FWeakObjectPtr> GetOwnerObjectsToMove() const;

	/** Find the OwnerPackage in memory, returns null if invalid or not already loaded. Does not call Initialize. */
	UPackage* GetOwnerPackage();
	/** Load the OwnerPackage if not already loaded. Can still return null if invalid or package fails to load. */
	UPackage* FindOrLoadOwnerPackage(UCookOnTheFlyServer& COTFS);
	/**
	 * Return a reference to ExternalActors that were discovered during save and stored on this.
	 * Does not call initialize.
	 */
	TConstArrayView<FName> GetExternalActorDependencies();
	/**
	 * Return the ExternalActors that were discovered during save and stored on this, and clear the values.
	 * Does not call initialize.
	 */
	TArray<FName> ReleaseExternalActorDependencies();
	/**
	 * Return whether PumpRequests on the director has processed all of the generated package requests, and we have
	 * done any followup configuration based on data about those generated packages in previous cooks.
	 */
	bool IsQueuedGeneratedPackagesFencePassed() const;

	/**
	 * Call the Splitter's ReportGenerationManifest and create the PackageDatas. Logs errors and returns false
	 * on failure.
	 */
	bool TryReportGenerationManifest();
	/**
	 * Call the Splitter's PopulateGeneratorPackage if not yet called. Assumes GenerateList has been called.
	 * This function executes support operations, needed for Populate and other calls, even if
	 * PopulateGeneratorPackage is not allowed to be called due to skipping of the Generator's package save.
	 * bPopulateCallAllowed=false executes those support functions without calling Populate.
	 *
	 * Logs errors and returns false on failure.
	 */
	bool TryCallPopulateGeneratorPackage(
		TArray<ICookPackageSplitter::FGeneratedPackageForPopulate>& InOutGeneratedPackagesForPresave,
		bool bPopulateCallAllowed = true);
	/**
	 * Call the Splitter's PopulateGeneratedPackage if not yet called. Assumes GenerateList has been called. Logs
	 * errors and returns false on failure.
	 */
	bool TryCallPopulateGeneratedPackage(UE::Cook::FCookGenerationInfo& Info, TArray<UObject*>& OutObjectsToMove);
	/**
	 * Mark that the SavePackage of the Owner is starting. Keeps a reference to keep the generator alive until save
	 * is finished.
	 */
	void StartOwnerSave();
	/** Update state before we queue the generated packages, e.g. mark whether packages are incrementally skippable. */
	void StartQueueGeneratedPackages(UCookOnTheFlyServer& COTFS);
	/**
	 * Mark that we have started queuing packages. Automatically called from StartQueueGeneratedPackages, but also
	 * is called on the director in response to a discovered generated package from a CookWorker. It sets the
	 * WorkerId so that it is available during assignment of the discovered generated packages.
	 */
	void NotifyStartQueueGeneratedPackages(UCookOnTheFlyServer& COTFS, FWorkerId SourceWorkerId);
	/** Update state after we queue generated packages; e.g. register to stay alive until the packages are assigned. */
	void EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS);
	/** Does not call Initialize. */
	void EndQueueGeneratedPackagesOnDirector(UCookOnTheFlyServer& COTFS, FWorkerId SourceWorkerId);
	/**
	 * Called on Director when the RequestFence added from SetKeepForIncremental or EndQueueGeneratedPackages has passed.
	 * Calls the proper followup events locally and on all Workers. Does not call Initialize.
	 */
	void OnRequestFencePassed(UCookOnTheFlyServer& COTFS);
	/**
	 * Called on Directors and CookWorkers when we discover that the generator was saved in the basegame and we should
	 * skip its save in this session but still save any new generatedpackages.
	 */
	void OnSkipSaveExistingGenerator();
	/**
	 * Called on Directors and Workers when the RequestFence added from EndQueueGeneratedPackages has passed.
	 * Does not call Initialize.
	 */
	void OnQueuedGeneratedPackagesFencePassed(UCookOnTheFlyServer& COTFS);
	/** Call CreatePackage and set package header data; or empty it and normalize it if it already exists. */
	UPackage* TryCreateGeneratedPackage(FCookGenerationInfo& GenerationInfo, bool bResetToEmpty);
	/**
	 * Called when a generator package is in FSaveCookedPackageContext::FinishPlatformSave.
	 * Calculates AssetRegistryData.
	 */
	void FinishGeneratorPlatformSave(FPackageData& PackageData, bool bFirstPlatform,
		TArray<FAssetDependency>& OutPackageDependencies);
	/**
	 * Called when a generated package is in FSaveCookedPackageContext::FinishPlatformSave.
	 * Calculates AssetRegistryData.
	 */
	void FinishGeneratedPlatformSave(FPackageData& PackageData, const ITargetPlatform* TargetPlatform,
		FAssetPackageData& OutAssetPackageData, TArray<FAssetDependency>& OutARDependencies,
		FBuildResultDependenciesMap& OutBuildResultDependencies);
	/**
	 * Return the AssetPackageData that was created for the given generated PackageName, or loaded for it from a
	 * previous cook. Will return nullptr if not loaded from a previous incremental cook and it has not yet been saved.
	 * Returns the AssetPackageData for the given platform, or for the shared data between platforms if nullptr.
	 * Does not call initialize.
	 */
	const FAssetPackageData* GetAssetPackageData(FName PackageName, const ITargetPlatform* TargetPlatform);
	/**
	 * Returns GetAssetPackageData for each known platform, returning the first non-null value, or null if no values
	 * found.
	 */
	const FAssetPackageData* GetAssetPackageDataAnyPlatform(FName PackageName);
	/**
	 * Calculate the PackageDigest for the given generated packagename and given platform based on its
	 * AssetPackageData (e.g. list of classes in the package) in a previous cook.
	 * TargetPlatform specifies which platform to look for results, or nullptr to look up results recorded for the
	 * shared data between platforms. Returns empty digest if not loaded from a previous incremental cook and it has not
	 * yet been saved.
	 * Does not call initialize.
	 */
	UE::EditorDomain::FPackageDigest GetPackageDigest(FName PackageName, const ITargetPlatform* TargetPlatform);

	/**
	 * Clear any data that should only be held when an FPackageData is in the save state, for the given Info. The given
	 * Info might specify the generator package or one of the generated packages.
	 */
	void ResetSaveState(FCookGenerationInfo& Info, UPackage* Package, EStateChangeReason ReleaseSaveReason,
		EPackageState NewState);

	/**
	 * Called when a package is retracted from a CookWorker, or from the Director's local worker. Default behavior
	 * for a retracted pacakge is to demote the package to idle, but if the state of the package is too advanced to
	 * recreate without a garbage collect, then the GenerationHelper instead needs to stall the package in case it
	 * is assigned back to this worker later.
	 */
	bool ShouldRetractionStallRatherThanDemote(FPackageData& PackageData);

	/**
	 * Record all dependencies from the generator package that are ExternalActor dependencies and
	 * store them on this.
	 */
	void FetchExternalActorDependencies();
	/** Incremental cook: store the list of generated packages from the last cook. Does not call Initialize. */
	void SetPreviousGeneratedPackages(const ITargetPlatform* TargetPlatform, TMap<FName, FAssetPackageData>&& Packages);
	/** Return the information set by SetPreviousGeneratedPackages if not yet cleared. Does not call Initialize. */
	const TMap<FName, FAssetPackageData>& GetPreviousGeneratedPackages(const ITargetPlatform* TargetPlatform) const;

	/**
	 * Callback during garbage collection. Does not call initialize. Caller must pass in a refcount to show
	 * a guarantee that clearing internal references will not delete before function return.
	 */
	void PreGarbageCollect(const TRefCountPtr<FGenerationHelper>& RefcountHeldByCaller,
		FPackageData& PackageData, TArray<TObjectPtr<UObject>>& GCKeepObjects,
		TArray<UPackage*>& GCKeepPackages, TArray<FPackageData*>& GCKeepPackageDatas, bool& bOutShouldDemote);
	/**
	 * Callback during garbage collection. Does not call initialize. Caller must pass in a refcount to show
	 * a guarantee that clearing internal references will not delete before function return.
	 */
	void PostGarbageCollect(const TRefCountPtr<FGenerationHelper>& RefcountHeldByCaller,
		FCookGCDiagnosticContext& Context);
	/**
	 * Called from PackageData function of the same name to decide whether to demote the package out of save.
	 * Does not call Initialize.
	 */
	void UpdateSaveAfterGarbageCollect(const FPackageData& PackageData, bool& bInOutDemote);

	// Self-references that keep this GenerationHelper in memory and referenced from the packages that use it until
	// the self-references are cleared. These Set/Clear functions do not call initialize.
	void SetKeepForIncremental(const ITargetPlatform* TargetPlatform);
	void SetKeepForIncrementalAllPlatforms();
	void ClearKeepForIncremental(const ITargetPlatform* TargetPlatform);
	void ClearKeepForIncrementalAllPlatforms();
	void SetKeepForQueueResults();
	void ClearKeepForQueueResults();
	bool IsWaitingForQueueResults() const;
	void SetKeepForGeneratorSave(const ITargetPlatform* TargetPlatform);
	void SetKeepForGeneratorSaveAllPlatforms();
	void ClearKeepForGeneratorSave(const ITargetPlatform* TargetPlatform);
	void ClearKeepForGeneratorSaveAllPlatforms();
	void SetKeepForAllSavedOrGC();
	void ClearKeepForAllSavedOrGC();
	void SetKeepForCompletedAllSavesMessage();
	void ClearKeepForCompletedAllSavesMessage();

	/**
	 * Helper for assignment of generated packages in MPCook. Return the id of the CookWorker that saved the
	 * generator, to decide where to assign the generated packages.
	 */
	FWorkerId GetWorkerIdThatSavedGenerator() const;
	/**
	 * Helper for assignment of generated packages in MPCook. A counter for assigned generated packages that is
	 * needed by some assignment schemes.
	 */
	int32& GetMPCookNextAssignmentIndex();

	/**
	 * Called for each of the generated packages that were discovered when TryReportGenerationManifest was called on a
	 * remote CookWorker. Does not call Initialize.
	 */
	void TrackGeneratedPackageListedRemotely(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
		const FIoHash& CurrentPackageHash);
	/**
	 * Called on the director when the generator or one of the generated packages was saved on a remote worker.
	 * Used to manage KeepForGCOrAllSaved lifetime. Does not call Initialize.
	 */
	void MarkPackageSavedRemotely(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
		const ITargetPlatform* TargetPlatform, FWorkerId SourceWorkerId);
	/** Called when a Package is demoted to idle, to update the saved count for it. Does not call Initialize. */
	void SetAllPlatformsSaved(FPackageData& PackageData, FCookGenerationInfo* Info);

	/** Scope object that blocks calls to OnAllSavesCompleted. Checks and calls it if necessary when destructed. */
	struct FScopeDeferEvents
	{
	public:
		FScopeDeferEvents(const TRefCountPtr<FGenerationHelper>& InGenerationHelper);
		~FScopeDeferEvents();
	private:
		TRefCountPtr<FGenerationHelper> GenerationHelper;
		bool bOldDeferEvents = false;
	};
	/**
	 * Called on the director when the generator or one of the generated packages was found to be incrementally skipped
	 * and marked already cooked in an incremental cook. Does not call Initialize.
	 */
	void MarkPackageIncrementallySkipped(FPackageData& PackageData, const ITargetPlatform* TargetPlatform,
		bool bIncrementallySkipped);
	/**
	 * Called on the director and every CookWorker when all saves have been completed; this indicates that
	 * some of our contract points are complete and the splitter can be destroyed. Does not call Initialize.
	 */
	void OnAllSavesCompleted(UCookOnTheFlyServer& COTFS);

	/** Diagnostics for shutdown errors. Report why the GenerationHelper is still referenced. */
	void DiagnoseWhyNotShutdown();
	/**
	 * Helper for shutdown errors. Force the GenerationHelper to uninitialize so that the CookPackageSplitter
	 * is shutdown correctly before the cooksession ends.
	 */
	void ForceUninitialize();

	/**
	 * Called when BeginCacheForCookedPlatformData called on a UObject, and IsCachedCookedPlatformDataLoaded does
	 * not immediately return true, for the given Object assigned to the given PackageData (but possibly a child of
	 * another package). The GenerationHelper will need to coordinate save fences if the object is a child
	 * of a different generated package owned by the generator than the one it is assigned to.
	 */
	void OnAddPendingCookedPlatformData(UCookOnTheFlyServer& COTFS, FPackageData& AssignedPackageData, UObject* Obj);
	/**
	 * Adds the object to a list that is checked before any save-state action of the generator or its generated
	 * packages. If the object still has an FPendingCookedPlatformData, the action is blocked and the package
	 * save is deferred as it would be if there were a FPendingCookedPlatformData assigned to the package. This
	 * is used for objects that move between generator and generated packages in edge-case flows that we
	 * don't support efficiently but need to handle for robustness.
	 */
	void AddBlockingPendingCookedPlatformDataObject(UObject* Object);
	/**
	 * Removes completed objects from the BlockingObjects list and returns whether any remain; it may leave
	 * some completed objects in the list if others are incomplete.
	 */
	bool HasBlockingPendingCookedPlatformDataObjects();
	/**
	 * A PendingCookedPlatformData has detected an object moved out of the given package with this generation helper.
	 * Return whether the pendingCookedPlatformData should keep waiting on it. If not we will stop polling it and will
	 * no longer let it block the save of its package.
	 */
	bool ShouldKeepWaitingOnMovedPendingCookedPlatformData(FPendingCookedPlatformData& Pending,
		FPackageData& AssignedPackage, UObject* Object);

	/** Helper function for Initialize and for TryCreateValidGenerationHelper. */
	static void SearchForRegisteredSplitDataObject(UCookOnTheFlyServer& COTFS, FName PackageName, UPackage* Package,
		TOptional<TConstArrayView<FCachedObjectInOuter>> CachedObjectsInOuter, UObject*& OutSplitDataObject,
		UE::Cook::Private::FRegisteredCookPackageSplitter*& OutRegisteredSplitterType,
		TUniquePtr<ICookPackageSplitter>& OutSplitterInstance, bool bCookedPlatformDataIsLoaded,
		bool& bOutNeedWaitForIsLoaded);
	/** Helper function for Initialize and for TryCreateValidGenerationHelper. */
	static UPackage* FindOrLoadPackage(UCookOnTheFlyServer& COTFS, FPackageData& OwnerPackageData);
	/** If PackageData is in a stalled state, verify that its state is valid for CookDirector or CookWorker. */
	static void ValidateSaveStalledState(UCookOnTheFlyServer& COTFS, FPackageData& PackageData, const TCHAR* Caller);

	/** Initialize settings from commandline and config files. */
	static void SetBeginCookConfigSettings();
	/** Debug setting that forces an otherwise unspecified order. Generator packages save before the generated do. */
	static bool IsGeneratorSavedFirst();
	/** Debug setting that forces an otherwise unspecified order. Generated packages save before their generator does. */
	static bool IsGeneratedSavedFirst();

	/**
	 * An API to access data that's only available on the CookDirector (or SingleProcess cook). This is present for
	 * readability at callsites, it just accesses data on the GenerationHelper.
	 */
	struct FDirectorAPI
	{
	public:
		/** Report whether NotifyStartQueueGeneratedPackages has been called. Returns false on CookWorkers. */
		bool HasStartedQueueGeneratedPackages() const;

	private:
		explicit FDirectorAPI(FGenerationHelper& InGenerationHelper);
		FDirectorAPI(const FDirectorAPI&) = delete;
		FDirectorAPI(FDirectorAPI&&) = delete;

		FGenerationHelper& GenerationHelper;

		friend struct FGenerationHelper;
	};
	FDirectorAPI GetDirectorAPI() const;

private:
	enum class EInitializeStatus : uint8
	{
		Uninitialized,
		Invalid,
		Valid,
	};
	struct FPlatformData
	{
		TMap<FName, FAssetPackageData> PreviousGeneratedPackages;
		TRefCountPtr<FGenerationHelper> ReferenceFromKeepForIncremental;
		TRefCountPtr<FGenerationHelper> ReferenceFromKeepForGeneratorSave;
		int32 NumSaved = 0;

		void SetKeepForIncremental(FGenerationHelper& GenerationHelper, const ITargetPlatform* TargetPlatform);
		void ClearKeepForIncremental(FGenerationHelper& GenerationHelper, const ITargetPlatform* TargetPlatform);
		void SetKeepForGeneratorSave(FGenerationHelper& GenerationHelper, const ITargetPlatform* TargetPlatform);
		void ClearKeepForGeneratorSave(FGenerationHelper& GenerationHelper, const ITargetPlatform* TargetPlatform);
	};

private:
	FPlatformData& FindCheckedPlatformData(const ITargetPlatform* TargetPlatform);
	const FPlatformData& FindCheckedPlatformData(const ITargetPlatform* TargetPlatform) const;
	void ConditionalInitialize() const;
	FCookGenerationInfo* FindInfoNoInitialize(const FPackageData& PackageData);
	FCookGenerationInfo* FindInfoNoInitialize(FName PackageName);
	void NotifyCompletion(ICookPackageSplitter::ETeardown Status);
	void PreGarbageCollectGCLifetimeData();
	void PostGarbageCollectGCLifetimeData(FCookGCDiagnosticContext& Context);
	void Uninitialize();
	void ModifyNumSaved(const ITargetPlatform* TargetPlatform, int32 Delta);
	void OnNumSavedUpdated();
	void VerifyGeneratorPackageGarbageCollected(FCookGCDiagnosticContext& Context);
	void DemoteStalledPackages(UCookOnTheFlyServer& COTFS, bool bFromAllSavesCompleted);

private:
	// When adding a new variable, add it to Uninitialize as well

	/** PackageData for the package that is being split */
	FCookGenerationInfo OwnerInfo;
	FWeakObjectPtr SplitDataObject;
	/** Name of the object that prompted the splitter creation */
	FName SplitDataObjectName;
	UE::Cook::Private::FRegisteredCookPackageSplitter* RegisteredSplitterType = nullptr;
	TUniquePtr<ICookPackageSplitter> CookPackageSplitterInstance;
	/** Recorded list of packages to generate from the splitter, and data we need about them */
	TArray<FCookGenerationInfo> PackagesToGenerate;
	TWeakObjectPtr<UPackage> OwnerPackage;
	TMap<const ITargetPlatform*, FPlatformData> PlatformDatas;
	TArray<FName> ExternalActorDependencies;
	TArray<FWeakObjectPtr> OwnerObjectsToMove;
	TArray<FWeakObjectPtr> BlockingPendingCookedPlatformDatas;
	TRefCountPtr<FGenerationHelper> ReferenceFromKeepForQueueResults;
	TRefCountPtr<FGenerationHelper> ReferenceFromKeepForAllSavedOrGC;
	int32 MPCookNextAssignmentIndex = 0;
	EInitializeStatus InitializeStatus = EInitializeStatus::Uninitialized;
	ICookPackageSplitter::EGeneratedRequiresGenerator DoesGeneratedRequireGeneratorValue =
		ICookPackageSplitter::EGeneratedRequiresGenerator::None;
	bool bUseInternalReferenceToAvoidGarbageCollect = false;
	bool bRequiresGeneratorPackageDestructBeforeResplit = false;
	bool bReportedManifest = false;
	bool bSaveGenerator = true;
	bool bSkipSaveExistingGenerator = false;
	bool bCurrentGCHasKeptGeneratorPackage = false;
	bool bCurrentGCHasKeptGeneratorKeepPackages = false;
	bool bKeepForAllSavedOrGC = false;
	bool bKeepForCompletedAllSavesMessage = false;
	bool bNeedConfirmGeneratorPackageDestroyed = false;
	bool bHasFinishedQueueGeneratedPackages = false;
	bool bQueuedGeneratedPackagesFencePassed = false;
	bool bDeferEvents = false;
	bool bSentAllSavesCompleted = false;

	friend FCookGenerationInfo;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline FCookGenerationInfo::FPlatformData& FCookGenerationInfo::FindCheckedPlatformData(
	const ITargetPlatform* TargetPlatform)
{
	FPlatformData* PlatformData = PlatformDatas.Find(TargetPlatform);
	if (!PlatformData)
	{
		UE_LOGF(LogCook, Error,
			"FCookGenerationInfo::FindCheckedPlatformData called for %ls on %ls, but doesn't already have it. This is a cook programming error.",
			*TargetPlatform->PlatformName(), *GetPackageName());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		PlatformData = &PlatformDatas.FindOrAdd(TargetPlatform);
	}
	return *PlatformData;
}

inline bool FCookGenerationInfo::IsCreateAsMap() const
{
	return bCreateAsMap;
}

inline void FCookGenerationInfo::SetIsCreateAsMap(bool bValue)
{
	bCreateAsMap = bValue;
}

inline bool FCookGenerationInfo::HasCreatedPackage() const
{
	return bHasCreatedPackage;

}
inline void FCookGenerationInfo::SetHasCreatedPackage(bool bValue)
{
	bHasCreatedPackage = bValue;
}

inline bool FCookGenerationInfo::HasSavedEveryPlatform() const
{
	bool bHasSaved = true;
	for (const TPair<const ITargetPlatform*, FPlatformData>& InfoPlatformPair : PlatformDatas)
	{
		bHasSaved &= InfoPlatformPair.Value.bHasSaved;
	}
	return bHasSaved;
}

inline bool FCookGenerationInfo::HasTakenOverCachedCookedPlatformData() const
{
	return bTakenOverCachedCookedPlatformData;
}

inline void FCookGenerationInfo::SetHasTakenOverCachedCookedPlatformData(bool bValue)
{
	bTakenOverCachedCookedPlatformData = bValue;
}

inline bool FCookGenerationInfo::HasIssuedUndeclaredMovedObjectsWarning() const
{
	return bIssuedUndeclaredMovedObjectsWarning;
}

inline void FCookGenerationInfo::SetHasIssuedUndeclaredMovedObjectsWarning(bool bValue)
{
	bIssuedUndeclaredMovedObjectsWarning = bValue;
}

inline bool FCookGenerationInfo::IsGenerator() const
{
	return bGenerator;
}

inline void FCookGenerationInfo::SetIsGenerator(bool bValue)
{
	bGenerator = bValue;
}

inline bool FCookGenerationInfo::HasCalledPopulate() const
{
	return bHasCalledPopulate;
}

inline void FCookGenerationInfo::SetHasCalledPopulate(bool bValue)
{
	bHasCalledPopulate = bValue;
}

inline bool FCookGenerationInfo::IsCopyChunkAssignmentFromGenerator() const
{
	return bCopyChunkAssignmentFromGenerator;
}

inline void FCookGenerationInfo::SetCopyChunkAssignmentFromGenerator(bool bValue)
{
	bCopyChunkAssignmentFromGenerator = bValue;
}

inline TConstArrayView<FAssetDependency> FCookGenerationInfo::GetDependencies() const
{
	return PackageDependencies;
}

inline FString FCookGenerationInfo::GetPackageName() const
{
	return PackageData->GetPackageName().ToString();
}

inline FCookGenerationInfo::FPlatformData::FPlatformData()
	: bHasSaved(false), bIncrementallySkipped(false)
{
}

inline bool FCookGenerationInfo::FPlatformData::IsIncrementallySkipped() const
{
	return bIncrementallySkipped;
}

inline void FCookGenerationInfo::FPlatformData::SetIncrementallySkipped(bool bValue)
{
	bIncrementallySkipped = bValue;
}

inline bool FCookGenerationInfo::FPlatformData::HasSaved() const
{
	return bHasSaved;
}

inline void FCookGenerationInfo::FPlatformData::SetHasSaved(FGenerationHelper& GenerationHelper,
	FCookGenerationInfo& Info, const ITargetPlatform* TargetPlatform, bool bValue, FWorkerId SourceWorkerId)
{
	if (bValue != bHasSaved)
	{
		bHasSaved = bValue;
		GenerationHelper.ModifyNumSaved(TargetPlatform, bValue ? 1 : -1);
		if (bHasSaved)
		{
			Info.SavedOnWorker = SourceWorkerId;
		}
	}
}

inline FGenerationHelper::FPlatformData& FGenerationHelper::FindCheckedPlatformData(
	const ITargetPlatform* TargetPlatform)
{
	FPlatformData* PlatformData = PlatformDatas.Find(TargetPlatform);
	if (!PlatformData)
	{
		UE_LOGF(LogCook, Error,
			"FGenerationHelper::FindCheckedPlatformData called for %ls on %ls, but doesn't already have it. This is a cook programming error.",
			*TargetPlatform->PlatformName(), *GetOwner().GetPackageName().ToString());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		PlatformData = &PlatformDatas.FindOrAdd(TargetPlatform);
	}
	return *PlatformData;
}

inline const FGenerationHelper::FPlatformData& FGenerationHelper::FindCheckedPlatformData(
	const ITargetPlatform* TargetPlatform) const
{
	return const_cast<FGenerationHelper&>(*this).FindCheckedPlatformData(TargetPlatform);
}

inline bool FGenerationHelper::IsInitialized() const
{
	return InitializeStatus != EInitializeStatus::Uninitialized;
}

inline void FGenerationHelper::ConditionalInitialize() const
{
	if (InitializeStatus == EInitializeStatus::Uninitialized)
	{
		// Use a const cast so we can call from getter functions that are otherwise const.
		const_cast<FGenerationHelper&>(*this).Initialize();
	}
}

inline bool FGenerationHelper::IsValid()
{
	ConditionalInitialize();
	return InitializeStatus == EInitializeStatus::Valid;
}

inline bool FGenerationHelper::IsSaveGenerator() const
{
	return bSaveGenerator;
}

inline bool FGenerationHelper::IsSkipSaveExistingGenerator() const
{
	return bSkipSaveExistingGenerator;
}

inline TArrayView<UE::Cook::FCookGenerationInfo> FGenerationHelper::GetPackagesToGenerate()
{
	ConditionalInitialize();
	return PackagesToGenerate;
}

inline UE::Cook::FCookGenerationInfo& FGenerationHelper::GetOwnerInfo()
{
	return OwnerInfo;
}

inline const UE::Cook::FCookGenerationInfo& FGenerationHelper::GetOwnerInfo() const
{
	return OwnerInfo;
}

inline UE::Cook::FPackageData& FGenerationHelper::GetOwner()
{
	return *OwnerInfo.PackageData;
}

inline ICookPackageSplitter* FGenerationHelper::GetCookPackageSplitterInstance() const
{
	ConditionalInitialize();
	return CookPackageSplitterInstance.Get();
}

inline UE::Cook::Private::FRegisteredCookPackageSplitter* FGenerationHelper::GetRegisteredSplitterType() const
{
	ConditionalInitialize();
	return RegisteredSplitterType;
}

inline const FName FGenerationHelper::GetSplitDataObjectName() const
{
	ConditionalInitialize();
	return SplitDataObjectName;
}

inline bool FGenerationHelper::IsUseInternalReferenceToAvoidGarbageCollect() const
{
	ConditionalInitialize();
	return bUseInternalReferenceToAvoidGarbageCollect;
}

inline bool FGenerationHelper::IsRequiresGeneratorPackageDestructBeforeResplit() const
{
	return bRequiresGeneratorPackageDestructBeforeResplit;
}

inline ICookPackageSplitter::EGeneratedRequiresGenerator FGenerationHelper::DoesGeneratedRequireGenerator() const
{
	ConditionalInitialize();
	return DoesGeneratedRequireGeneratorValue;
}

inline TConstArrayView<FWeakObjectPtr> FGenerationHelper::GetOwnerObjectsToMove() const
{
	return OwnerObjectsToMove;
}

inline TConstArrayView<FName> FGenerationHelper::GetExternalActorDependencies()
{
	return ExternalActorDependencies;
}

inline TArray<FName> FGenerationHelper::ReleaseExternalActorDependencies()
{
	TArray<FName> Result = MoveTemp(ExternalActorDependencies);
	ExternalActorDependencies.Empty();
	return Result;
}

inline bool FGenerationHelper::IsQueuedGeneratedPackagesFencePassed() const
{
	return bQueuedGeneratedPackagesFencePassed;
}

inline const TMap<FName, FAssetPackageData>& FGenerationHelper::GetPreviousGeneratedPackages(
	const ITargetPlatform* TargetPlatform) const
{
	return FindCheckedPlatformData(TargetPlatform).PreviousGeneratedPackages;
}

inline void FGenerationHelper::FPlatformData::SetKeepForIncremental(FGenerationHelper& GenerationHelper,
	const ITargetPlatform* TargetPlatform)
{
	if (!ReferenceFromKeepForIncremental)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SetKeepForIncremental for platform %ls.",
			*WriteToString<256>(GenerationHelper.GetOwner().GetPackageName()), *TargetPlatform->PlatformName());
		ReferenceFromKeepForIncremental = &GenerationHelper;
	}
}

inline void FGenerationHelper::SetKeepForIncremental(const ITargetPlatform* TargetPlatform)
{
	FindCheckedPlatformData(TargetPlatform).SetKeepForIncremental(*this, TargetPlatform);
}

inline void FGenerationHelper::SetKeepForIncrementalAllPlatforms()
{
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		PlatformPair.Value.SetKeepForIncremental(*this, PlatformPair.Key);
	}
}

inline void FGenerationHelper::FPlatformData::ClearKeepForIncremental(FGenerationHelper& GenerationHelper,
	const ITargetPlatform* TargetPlatform)
{
	if (ReferenceFromKeepForIncremental)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: ClearKeepForIncremental for platform %ls.",
			*WriteToString<256>(GenerationHelper.GetOwner().GetPackageName()), *TargetPlatform->PlatformName());
		ReferenceFromKeepForIncremental.SafeRelease();
	}
}

inline void FGenerationHelper::ClearKeepForIncremental(const ITargetPlatform* TargetPlatform)
{
	FindCheckedPlatformData(TargetPlatform).ClearKeepForIncremental(*this, TargetPlatform);
}

inline void FGenerationHelper::ClearKeepForIncrementalAllPlatforms()
{
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		PlatformPair.Value.ClearKeepForIncremental(*this, PlatformPair.Key);
	}
}

inline void FGenerationHelper::SetKeepForQueueResults()
{
	if (!ReferenceFromKeepForQueueResults)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SetKeepForQueueResults.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		ReferenceFromKeepForQueueResults = this;
	}
}

inline void FGenerationHelper::ClearKeepForQueueResults()
{
	if (ReferenceFromKeepForQueueResults)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: ClearKeepForQueueResults.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		ReferenceFromKeepForQueueResults.SafeRelease();
	}
}

inline bool FGenerationHelper::IsWaitingForQueueResults() const
{
	return ReferenceFromKeepForQueueResults.IsValid();
}

inline void FGenerationHelper::FPlatformData::SetKeepForGeneratorSave(FGenerationHelper& GenerationHelper,
	const ITargetPlatform* TargetPlatform)
{
	if (!ReferenceFromKeepForGeneratorSave)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SetKeepForGeneratorSave for platform %ls.",
			*WriteToString<256>(GenerationHelper.GetOwner().GetPackageName()), *TargetPlatform->PlatformName());
		ReferenceFromKeepForGeneratorSave = &GenerationHelper;
	}
}

inline void FGenerationHelper::SetKeepForGeneratorSave(const ITargetPlatform* TargetPlatform)
{
	FindCheckedPlatformData(TargetPlatform).SetKeepForGeneratorSave(*this, TargetPlatform);
}

inline void FGenerationHelper::SetKeepForGeneratorSaveAllPlatforms()
{
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		PlatformPair.Value.SetKeepForGeneratorSave(*this, PlatformPair.Key);
	}
}

inline void FGenerationHelper::FPlatformData::ClearKeepForGeneratorSave(FGenerationHelper& GenerationHelper,
	const ITargetPlatform* TargetPlatform)
{
	if (ReferenceFromKeepForGeneratorSave)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: ClearKeepForGeneratorSave for platform %ls.",
			*WriteToString<256>(GenerationHelper.GetOwner().GetPackageName()), *TargetPlatform->PlatformName());
		ReferenceFromKeepForGeneratorSave.SafeRelease();
	}
}

inline void FGenerationHelper::ClearKeepForGeneratorSave(const ITargetPlatform* TargetPlatform)
{
	FindCheckedPlatformData(TargetPlatform).ClearKeepForGeneratorSave(*this, TargetPlatform);
}

inline void FGenerationHelper::ClearKeepForGeneratorSaveAllPlatforms()
{
	for (TPair<const ITargetPlatform*, FPlatformData>& PlatformPair : PlatformDatas)
	{
		PlatformPair.Value.ClearKeepForGeneratorSave(*this, PlatformPair.Key);
	}
}

inline void FGenerationHelper::SetKeepForAllSavedOrGC()
{
	if (!bKeepForAllSavedOrGC)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SetKeepForAllSavedOrGC.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		ReferenceFromKeepForAllSavedOrGC = this;
		bKeepForAllSavedOrGC = true;
	}
	else
	{
		// If bKeepForAllSavedOrGC, ReferenceFromKeepForAllSavedOrGC must be true as well.
		check(ReferenceFromKeepForAllSavedOrGC);
	}
}

inline void FGenerationHelper::ClearKeepForAllSavedOrGC()
{
	if (bKeepForAllSavedOrGC)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: ClearKeepForAllSavedOrGC.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		bKeepForAllSavedOrGC = false;
		if (!bKeepForAllSavedOrGC && !bKeepForCompletedAllSavesMessage)
		{
			ReferenceFromKeepForAllSavedOrGC.SafeRelease();
		}
	}
	else
	{
		// ReferenceFromKeepForAllSavedOrGC must only be set if at least one of bKeepForAllSavedOrGC or bKeepForCompletedAllSavesMessage is set
		check(bKeepForAllSavedOrGC || bKeepForCompletedAllSavesMessage || !ReferenceFromKeepForAllSavedOrGC);
	}
}

inline void FGenerationHelper::SetKeepForCompletedAllSavesMessage()
{
	if (!bKeepForCompletedAllSavesMessage)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: SetKeepForCompletedAllSavesMessage.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		ReferenceFromKeepForAllSavedOrGC = this;
		bKeepForCompletedAllSavesMessage = true;
	}
	else
	{
		// If bKeepForCompletedAllSavesMessage, ReferenceFromKeepForAllSavedOrGC must be true as well.
		check(ReferenceFromKeepForAllSavedOrGC);
	}
}

inline void FGenerationHelper::ClearKeepForCompletedAllSavesMessage()
{
	if (bKeepForCompletedAllSavesMessage)
	{
		UE_LOGF(LogCookGenerationHelper, Verbose, "%ls: ClearKeepForCompletedAllSavesMessage.",
			*WriteToString<256>(GetOwner().GetPackageName()));

		bKeepForCompletedAllSavesMessage = false;
		if (!bKeepForAllSavedOrGC && !bKeepForCompletedAllSavesMessage)
		{
			ReferenceFromKeepForAllSavedOrGC.SafeRelease();
		}
	}
	else
	{
		// ReferenceFromKeepForAllSavedOrGC must only be set if at least one of bKeepForAllSavedOrGC or bKeepForCompletedAllSavesMessage is set
		check(bKeepForAllSavedOrGC || bKeepForCompletedAllSavesMessage || !ReferenceFromKeepForAllSavedOrGC);
	}
}

inline FWorkerId FGenerationHelper::GetWorkerIdThatSavedGenerator() const
{
	return OwnerInfo.SavedOnWorker;
}

inline int32& FGenerationHelper::GetMPCookNextAssignmentIndex()
{
	return MPCookNextAssignmentIndex;
}

inline void FGenerationHelper::SetAllPlatformsSaved(FPackageData& PackageData, FCookGenerationInfo* Info)
{
	if (!Info)
	{
		// Note that we expect this Find to succeed for generated packages even without initialization.
		// In the non-incremental case, the GenerationHelper must have already initialized,
		// or the generated PackageData would never exist. In the incremental case, the Info was added
		// during PopulateCookedSandbox to prepare the generator package for incremental cook.
		Info = FindInfoNoInitialize(PackageData);
	}
	if (Info)
	{
		for (TPair<const ITargetPlatform*, FCookGenerationInfo::FPlatformData>& PlatformPair : Info->PlatformDatas)
		{
			const ITargetPlatform* TargetPlatform = PlatformPair.Key;
			FCookGenerationInfo::FPlatformData& InfoPlatformData = PlatformPair.Value;
			InfoPlatformData.SetHasSaved(*this, *Info, TargetPlatform,
				true /* bValue */, FWorkerId::Local());
		}
	}
}

inline FGenerationHelper::FDirectorAPI FGenerationHelper::GetDirectorAPI() const
{
	return FDirectorAPI(const_cast<FGenerationHelper&>(*this));
}

inline FGenerationHelper::FDirectorAPI::FDirectorAPI(FGenerationHelper& InGenerationHelper)
	: GenerationHelper(InGenerationHelper)
{
}

} // namespace UE::Cook