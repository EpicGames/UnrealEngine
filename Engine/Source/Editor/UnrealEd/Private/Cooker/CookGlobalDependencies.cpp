// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookGlobalDependencies.h"

#include "Cooker/CookTypes.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreGlobals.h"
#include "Hash/Blake3.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Serialization/UnversionedPropertySerializationInternal.h"
#include "UObject/CoreRedirects.h"

namespace UE::Cook
{
	// This hash is calculated per platform from different settings and guids
	static TMap<const ITargetPlatform*, FBlake3Hash> GlobalHash;

	void CalculateGlobalDependenciesHash(const ITargetPlatform* Platform, const UCookOnTheFlyServer& COTFS)
	{
		UE::ConfigAccessTracking::FIgnoreScope Ignore;

		FBlake3 Hasher;

		UE_LOGF(LogCook, Display, "CalculateGlobalDependenciesHash(%ls):", *Platform->PlatformName());

		{
			Hasher.Update(&CookIncrementalVersion, sizeof(CookIncrementalVersion));
			UE_LOGF(LogCook, Display, "\tFGuid CookIncrementalVersion = %ls", *WriteToString<64>(CookIncrementalVersion));
		}

		{
			Hasher.Update(&ProjectCookIncrementalVersion, sizeof(ProjectCookIncrementalVersion));
			UE_LOGF(LogCook, Display, "\t[Editor]:CookSettings.ProjectCookIncrementalVersion = %ls", *WriteToString<64>(ProjectCookIncrementalVersion));
		}

		{
			bool bValue = false;
			GConfig->GetBool(TEXT("Core.System"), TEXT("CanStripEditorOnlyExportsAndImports"), bValue, GEngineIni);
			Hasher.Update(&bValue, sizeof(bValue));
			UE_LOGF(LogCook, Display, "\t[Engine]:Core.System.CanStripEditorOnlyExportsAndImports = %ls", bValue ? TEXT("True") : TEXT("False"));
		}

		{
			bool bValue = false;
			GConfig->GetBool(TEXT("Core.System"), TEXT("CanSkipEditorReferencedPackagesWhenCooking"), bValue, GEngineIni);
			Hasher.Update(&bValue, sizeof(bValue));
			UE_LOGF(LogCook, Display, "\t[Engine]:Core.System.CanSkipEditorReferencedPackagesWhenCooking = %ls", bValue ? TEXT("True") : TEXT("False"));
		}

		{
			bool bCanUseUnversionedPropertySerialization = CanUseUnversionedPropertySerialization(Platform);
			Hasher.Update(&bCanUseUnversionedPropertySerialization, sizeof(bCanUseUnversionedPropertySerialization));
			UE_LOGF(LogCook, Display, "\tCanUseUnversionedPropertySerialization = %ls", bCanUseUnversionedPropertySerialization ? TEXT("True") : TEXT("False"));
		}

		{
			TArray<FString> AssetList;
			GConfig->GetArray(TEXT("CookSettings"), TEXT("AutomaticOptionalInclusionAssetType"), AssetList, GEditorIni);
			for (const FString& AssetType : AssetList)
			{
				Hasher.Update(*AssetType, AssetType.NumBytesWithoutNull());
			}
			UE_LOGF(LogCook, Display, "\t[Editor]:CookSettings.AutomaticOptionalInclusionAssetType = %ls", *FString::Join(AssetList, TEXT(", ")));
		}

		{
			bool bIsUnversioned = COTFS.IsCookFlagSet(ECookInitializationFlags::Unversioned);
			Hasher.Update(&bIsUnversioned, sizeof(bIsUnversioned));
			UE_LOGF(LogCook, Display, "\tIsCookFlagSet(ECookInitializationFlags::Unversioned) = %ls", bIsUnversioned ? TEXT("True") : TEXT("False"));
		}

		{
			FBlake3 GlobalRedirectHasher;
			FCoreRedirects::AppendHashOfGlobalRedirects(GlobalRedirectHasher);

			FBlake3Hash GlobalRedirectHash = GlobalRedirectHasher.Finalize();
			Hasher.Update(&GlobalRedirectHash, sizeof(GlobalRedirectHash));

			FString GlobalRedirectHashString = LexToString(GlobalRedirectHash);
			UE_LOGF(LogCook, Display, "\tFCoreRedirects::AppendHashOfGlobalRedirects = %ls", *GlobalRedirectHashString);
		}

		FBlake3Hash Hash = Hasher.Finalize();
		UE_LOGF(LogCook, Display, "\tGlobalDependenciesHash(%ls) = %ls", *Platform->PlatformName(), *LexToString(Hash));
		GlobalHash.Add(Platform, Hash);
	}

	FBlake3Hash GetGlobalDependenciesHash(const ITargetPlatform* Platform)
	{
		FBlake3Hash* Hash = GlobalHash.Find(Platform);
		if (!Hash)
		{
			UE_LOGF(LogCook, Error, "CalculateGlobalDependenciesHash was not called for platform %ls", *Platform->PlatformName());
			return FBlake3Hash();
		}

		return *Hash;
	}
}
