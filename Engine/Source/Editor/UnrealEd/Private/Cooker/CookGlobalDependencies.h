// Copyright Epic Games, Inc. All Rights Reserved.

struct FBlake3Hash;
class ITargetPlatform;
class UCookOnTheFlyServer;

namespace UE::Cook
{
	/** 
	 * Calculate a hash representing the dependencies applied to every package.
	 * Some dependencies are per platform so this function needs to be called per platform cooked.
	 */
	void CalculateGlobalDependenciesHash(const ITargetPlatform* Platform, const UCookOnTheFlyServer& COTFS);

	FBlake3Hash GetGlobalDependenciesHash(const ITargetPlatform* Platform);
}
