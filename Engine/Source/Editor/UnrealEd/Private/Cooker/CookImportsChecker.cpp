// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookImportsChecker.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/TypeHash.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

LLM_DEFINE_TAG(EDLCookChecker);

FEDLCookChecker::FEDLNodeHash::FEDLNodeHash()
	: ObjectTypeData { nullptr }
	, NodeHashType(ENodeHashType::Object)
	, ObjectEvent(EObjectEvent::Create)
{
}

FEDLCookChecker::FEDLNodeHash::FEDLNodeHash(const FEDLNodeHash& Other)
	: ObjectTypeData{ nullptr }
	, NodeHashType(ENodeHashType::Object)
	, ObjectEvent(EObjectEvent::Create)
{
	*this = Other;
}

FEDLCookChecker::FEDLNodeHash::FEDLNodeHash(const TArray<FEDLNodeData>* InNodes, FEDLNodeID InNodeID,
	EObjectEvent InObjectEvent)
	: NodeTypeData { InNodes, InNodeID }
	, NodeHashType(ENodeHashType::Node)
	, ObjectEvent(InObjectEvent)
{
}

FEDLCookChecker::FEDLNodeHash::FEDLNodeHash(const TArray<FEDLNodeData>* InNodes, FEDLNodeID InParentNodeID,
	FName InObjectName, EObjectEvent InObjectEvent)
	: NameTypeData{ InObjectName, InNodes, InParentNodeID }
	, NodeHashType(ENodeHashType::NameAndParentNode)
	, ObjectEvent(InObjectEvent)
{
}

FEDLCookChecker::FEDLNodeHash::FEDLNodeHash(TObjectPtr<UObject> InObject, EObjectEvent InObjectEvent)
	: ObjectTypeData{ InObject }
	, NodeHashType(ENodeHashType::Object)
	, ObjectEvent(InObjectEvent)
{
}

bool FEDLCookChecker::FEDLNodeHash::operator==(const FEDLNodeHash& Other) const
{
	if (ObjectEvent != Other.ObjectEvent)
	{
		return false;
	}

	if (GetName() != Other.GetName())
	{
		return false;
	}
	FEDLCookChecker::FEDLNodeHash CurrentThis;
	FEDLCookChecker::FEDLNodeHash CurrentOther;
	bool bCurrentThisValid = TryGetParent(CurrentThis);
	bool bOtherThisValid = Other.TryGetParent(CurrentOther);

	while (bCurrentThisValid && bOtherThisValid)
	{
		if (CurrentThis.GetName() != CurrentOther.GetName())
		{
			return false;
		}
		bCurrentThisValid = CurrentThis.TryGetParent(CurrentThis);
		bOtherThisValid = CurrentOther.TryGetParent(CurrentOther);
	}
	return bCurrentThisValid == bOtherThisValid;
}

FEDLCookChecker::FEDLNodeHash& FEDLCookChecker::FEDLNodeHash::operator=(const FEDLNodeHash& Other)
{
	NodeHashType = Other.NodeHashType;
	switch (NodeHashType)
	{
	case ENodeHashType::Node:
		NodeTypeData.Nodes = Other.NodeTypeData.Nodes;
		NodeTypeData.NodeID = Other.NodeTypeData.NodeID;
		break;
	case ENodeHashType::Object:
		ObjectTypeData.Object = Other.ObjectTypeData.Object;
		break;
	case ENodeHashType::NameAndParentNode:
		NameTypeData.ObjectName = Other.NameTypeData.ObjectName;
		NameTypeData.Nodes = Other.NameTypeData.Nodes;
		NameTypeData.ParentID = Other.NameTypeData.ParentID;
		break;
	default:
		checkNoEntry();
		break;
	}
	ObjectEvent = Other.ObjectEvent;

	return *this;
}

uint32 GetTypeHash(const FEDLCookChecker::FEDLNodeHash& A)
{
	return FEDLCookChecker::FEDLNodeHash::GetTypeHashInternal(A);
}

uint32 FEDLCookChecker::FEDLNodeHash::GetTypeHashInternal(const FEDLCookChecker::FEDLNodeHash& A)
{
	auto GetTypeHashFromNodeOuterChain =
		[](uint32 Hash, const TArray<FEDLCookChecker::FEDLNodeData>& Nodes, FEDLCookChecker::FEDLNodeID ParentNodeID,
			FName ObjectName)
		{
			Hash = HashCombine(Hash, GetTypeHash(ObjectName));
			while (ParentNodeID != NodeIDInvalid)
			{
				const FEDLCookChecker::FEDLNodeData& ParentNode = Nodes[ParentNodeID];
				Hash = HashCombine(Hash, GetTypeHash(ParentNode.Name));
				ParentNodeID = ParentNode.ParentID;
			}
			return Hash;
		};

	uint32 Hash = 0;
	switch (A.NodeHashType)
	{
	case ENodeHashType::Node:
	{
		const TArray<FEDLCookChecker::FEDLNodeData>& Nodes = *A.NodeTypeData.Nodes;
		const FEDLCookChecker::FEDLNodeData& Node = Nodes[A.NodeTypeData.NodeID];
		Hash = GetTypeHashFromNodeOuterChain(Hash, Nodes, Node.ParentID, Node.Name);
		break;
	}
	case ENodeHashType::Object:
	{
		TObjectPtr<const UObject> Object = A.ObjectTypeData.Object;
		while (Object)
		{
			Hash = HashCombine(Hash, GetTypeHash(Object.GetFName()));
			Object = Object.GetOuter();
		}
		break;
	}
	case ENodeHashType::NameAndParentNode:
	{
		const TArray<FEDLCookChecker::FEDLNodeData>& Nodes = *A.NameTypeData.Nodes;
		Hash = GetTypeHashFromNodeOuterChain(Hash, Nodes, A.NameTypeData.ParentID, A.NameTypeData.ObjectName);
		break;
	}
	default:
		checkNoEntry();
		break;
	}

	return (Hash << 1) | (uint32)A.ObjectEvent;
}

FName FEDLCookChecker::FEDLNodeHash::GetName() const
{
	switch (NodeHashType)
	{
	case ENodeHashType::Node:
		return (*NodeTypeData.Nodes)[NodeTypeData.NodeID].Name;
	case ENodeHashType::Object:
		return ObjectTypeData.Object.GetFName();
	case ENodeHashType::NameAndParentNode:
		return NameTypeData.ObjectName;
	default:
		checkNoEntry();
		return NAME_None;
	}
}

bool FEDLCookChecker::FEDLNodeHash::TryGetParent(FEDLCookChecker::FEDLNodeHash& Parent) const
{
	EObjectEvent ParentObjectEvent = EObjectEvent::Create; // For purposes of parents, which is used only to get the ObjectPath, we always use the Create version of the node as the parent
	switch (NodeHashType)
	{
	case ENodeHashType::Node:
	{
		FEDLNodeID ParentID = (*NodeTypeData.Nodes)[NodeTypeData.NodeID].ParentID;
		if (ParentID != NodeIDInvalid)
		{
			Parent = FEDLNodeHash(NodeTypeData.Nodes, ParentID, ParentObjectEvent);
			return true;
		}
		return false;
	}
	case ENodeHashType::Object:
	{
		TObjectPtr<UObject> ParentObject = ObjectTypeData.Object.GetOuter();
		if (ParentObject)
		{
			Parent = FEDLNodeHash(ParentObject, ParentObjectEvent);
			return true;
		}
		return false;
	}
	case ENodeHashType::NameAndParentNode:
		if (NameTypeData.ParentID != NodeIDInvalid)
		{
			Parent = FEDLNodeHash(NameTypeData.Nodes, NameTypeData.ParentID, ParentObjectEvent);
			return true;
		}
		return false;
	default:
		checkNoEntry();
		return false;
	}
}

FEDLCookChecker::EObjectEvent FEDLCookChecker::FEDLNodeHash::GetObjectEvent() const
{
	return ObjectEvent;
}

void FEDLCookChecker::FEDLNodeHash::SetNodes(const TArray<FEDLNodeData>* InNodes)
{
	switch (NodeHashType)
	{
	case ENodeHashType::Node:
		NodeTypeData.Nodes = InNodes;
		break;
	case ENodeHashType::Object:
		break;
	case ENodeHashType::NameAndParentNode:
		NameTypeData.Nodes = InNodes;
		break;
	default:
		checkNoEntry();
		break;
	}
}

FEDLCookChecker::FEDLNodeData::FEDLNodeData(FEDLNodeID InID, FEDLNodeID InParentID, FName InName, EObjectEvent InObjectEvent)
	: Name(InName)
	, ID(InID)
	, ParentID(InParentID)
	, ObjectEvent(InObjectEvent)
	, bIsExport(false)
{
}

FEDLCookChecker::FEDLNodeData::FEDLNodeData(FEDLNodeID InID, FEDLNodeID InParentID, FName InName, FEDLNodeData&& Other)
	: Name(InName)
	, ID(InID)
	, ImportingPackagesSorted(MoveTemp(Other.ImportingPackagesSorted))
	, ParentID(InParentID)
	, ObjectEvent(Other.ObjectEvent)
	, bIsExport(Other.bIsExport)
{
	// Note that Other Name and ParentID must be unmodified, since they might still be needed for GetHashCode calls from children
	Other.ImportingPackagesSorted.Empty();
}

FEDLCookChecker::FEDLNodeHash FEDLCookChecker::FEDLNodeData::GetNodeHash(const FEDLCookChecker& Owner) const
{
	return FEDLNodeHash(&Owner.Nodes, ID, ObjectEvent);
}

FString FEDLCookChecker::FEDLNodeData::ToString(const FEDLCookChecker& Owner) const
{
	TStringBuilder<NAME_SIZE> Result;
	switch (ObjectEvent)
	{
	case EObjectEvent::Create:
		Result << TEXT("Create:");
		break;
	case EObjectEvent::Serialize:
		Result << TEXT("Serialize:");
		break;
	default:
		check(false);
		break;
	}
	AppendPathName(Owner, Result);
	return FString(Result);
}

void FEDLCookChecker::FEDLNodeData::AppendPathName(const FEDLCookChecker& Owner, FStringBuilderBase& Result) const
{
	if (ParentID != NodeIDInvalid)
	{
		const FEDLNodeData& ParentNode = Owner.Nodes[ParentID];
		ParentNode.AppendPathName(Owner, Result);
		bool bParentIsOutermost = ParentNode.ParentID == NodeIDInvalid;
		Result << (bParentIsOutermost ? TEXT(".") : SUBOBJECT_DELIMITER);
	}
	Name.AppendString(Result);
}

FName FEDLCookChecker::FEDLNodeData::GetPackageName(const FEDLCookChecker& Owner) const
{
	if (ParentID != NodeIDInvalid)
	{
		// @todo ExternalPackages: We need to store ExternalPackage pointers on the Node and return that
		return Owner.Nodes[ParentID].GetPackageName(Owner);
	}
	return Name;
}

void FEDLCookChecker::FEDLNodeData::Merge(FEDLCookChecker::FEDLNodeData&& Other)
{
	check(ObjectEvent == Other.ObjectEvent);
	bIsExport = bIsExport || Other.bIsExport;

	ImportingPackagesSorted.Append(Other.ImportingPackagesSorted);
	Algo::Sort(ImportingPackagesSorted, FNameFastLess());
	ImportingPackagesSorted.SetNum(Algo::Unique(ImportingPackagesSorted), EAllowShrinking::Yes);
}

FEDLCookCheckerThreadState::FEDLCookCheckerThreadState()
{
	Checker.SetActiveIfNeeded();

	FScopeLock CookCheckerInstanceLock(&FEDLCookChecker::CookCheckerInstanceCritical);
	FEDLCookChecker::CookCheckerInstances.Add(&Checker);
}

void FEDLCookChecker::SetActiveIfNeeded()
{
	bIsActive = !FParse::Param(FCommandLine::Get(), TEXT("DisableEDLCookChecker"));
}

void FEDLCookChecker::Reset()
{
	check(!UE::IsSavingPackage());

	Nodes.Reset();
	NodeHashToNodeID.Reset();
	NodePrereqs.Reset();
	bIsActive = false;
}

void FEDLCookChecker::AddImport(TObjectPtr<UObject> Import, UPackage* ImportingPackage)
{
	if (bIsActive)
	{
		if (!Import->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn))
		{
			LLM_SCOPE_BYTAG(EDLCookChecker);
			FEDLNodeID NodeId = FindOrAddNode(FEDLNodeHash(Import, EObjectEvent::Serialize));
			RecordImportFromPackage(NodeId, ImportingPackage->GetFName());
		}
	}
}

void FEDLCookChecker::RecordImportFromPackage(FEDLNodeID NodeId, FName ImportingPackageName)
{
	FEDLNodeData& NodeData = Nodes[NodeId];
	TArray<FName>& Sorted = NodeData.ImportingPackagesSorted;
	int32 InsertionIndex = Algo::LowerBound(Sorted, ImportingPackageName, FNameFastLess());
	if (InsertionIndex == Sorted.Num() || Sorted[InsertionIndex] != ImportingPackageName)
	{
		Sorted.Insert(ImportingPackageName, InsertionIndex);
	}
}

template <typename AddNodeType>
void FEDLCookChecker::AddImportExportNodeList(TConstArrayView<UE::Cook::FImportExportNode> NodeList, AddNodeType&& AddNode)
{
	// See the comment in FImportsCheckerData::ObjectListToNodeList, this is the same algorithm. Recursively
	// calculate and cache EDLNodes for parent nodes.
	TArray<int32, TInlineAllocator<10>> Stack;
	TArray<FEDLNodeID> NodeIDForExternalIndex;
	NodeIDForExternalIndex.SetNumUninitialized(NodeList.Num());
	for (FEDLNodeID& NodeID : NodeIDForExternalIndex)
	{
		NodeID = NodeIDInvalid;
	}
	for (int32 ExternalIndex = 0; ExternalIndex < NodeList.Num(); ++ExternalIndex)
	{
		check(Stack.IsEmpty());
		FEDLNodeID ParentNodeID = NodeIDInvalid;
		int32 CurrentExternalIndex = ExternalIndex;
		while (CurrentExternalIndex != -1)
		{
			FEDLNodeID& CurrentNodeID = NodeIDForExternalIndex[CurrentExternalIndex];
			if (CurrentNodeID == NodeIDInvalid)
			{
				const UE::Cook::FImportExportNode& Node = NodeList[CurrentExternalIndex];
				if (Node.ParentId != -1 && ParentNodeID == NodeIDInvalid)
				{
					Stack.Push(CurrentExternalIndex);
					CurrentExternalIndex = Node.ParentId;
					continue;
				}

				CurrentNodeID = FindOrAddNode(FEDLNodeHash(&Nodes, ParentNodeID,
					Node.ObjectName, EObjectEvent::Serialize));
				AddNode(Node, CurrentNodeID, ParentNodeID);
			}

			check(CurrentNodeID != NodeIDInvalid);
			ParentNodeID = CurrentNodeID;
			CurrentExternalIndex = !Stack.IsEmpty() ? Stack.Pop(EAllowShrinking::No) : -1;
		}
	}
}

void FEDLCookChecker::AddImports(TConstArrayView<UE::Cook::FImportExportNode> Imports, FName ImportingPackageName)
{
	if (!bIsActive)
	{
		return;
	}
	LLM_SCOPE_BYTAG(EDLCookChecker);

	AddImportExportNodeList(Imports,
		[ImportingPackageName, this]
		(const UE::Cook::FImportExportNode& ImportNode, FEDLNodeID NodeID, FEDLNodeID ParentNodeID)
		{
			RecordImportFromPackage(NodeID, ImportingPackageName);
		});
}

void FEDLCookChecker::AddExport(UObject* Export)
{
	if (bIsActive)
	{
		LLM_SCOPE_BYTAG(EDLCookChecker);
		FEDLNodeID SerializeID = FindOrAddNode(FEDLNodeHash(Export, EObjectEvent::Serialize));
		Nodes[SerializeID].bIsExport = true;
		FEDLNodeID CreateID = FindOrAddNode(FEDLNodeHash(Export, EObjectEvent::Create));
		Nodes[CreateID].bIsExport = true;

		// every export must be created before it can be serialized...
		// these arcs are implicit and not listed in any table.
		AddDependency(SerializeID, CreateID); 
	}
}

void FEDLCookChecker::AddExports(TConstArrayView<UE::Cook::FImportExportNode> Exports)
{
	if (!bIsActive)
	{
		return;
	}
	LLM_SCOPE_BYTAG(EDLCookChecker);

	AddImportExportNodeList(Exports,
		[this](const UE::Cook::FImportExportNode& ExportNode, FEDLNodeID SerializeID, FEDLNodeID ParentNodeID)
		{
			// AddImportExportNodeList added the Serialize node for us, we also need to add the Create node
			FEDLNodeID CreateID = FindOrAddNode(FEDLNodeHash(&Nodes, ParentNodeID,
				ExportNode.ObjectName, EObjectEvent::Create));

			Nodes[SerializeID].bIsExport = true;
			Nodes[CreateID].bIsExport = true;

			// every export must be created before it can be serialized...
			// these arcs are implicit and not listed in any table.
			AddDependency(SerializeID, CreateID);
		});
}

void FEDLCookChecker::Add(UE::Cook::FImportsCheckerData& ImportsCheckerData, FName PackageName)
{
	AddImports(ImportsCheckerData.Imports, PackageName);
	AddExports(ImportsCheckerData.Exports);
}

void FEDLCookChecker::AddArc(UObject* DepObject, bool bDepIsSerialize, UObject* Export, bool bExportIsSerialize)
{
	if (bIsActive)
	{
		LLM_SCOPE_BYTAG(EDLCookChecker);
		FEDLNodeID ExportID = FindOrAddNode(FEDLNodeHash(Export, bExportIsSerialize ? EObjectEvent::Serialize : EObjectEvent::Create));
		FEDLNodeID DepID = FindOrAddNode(FEDLNodeHash(DepObject, bDepIsSerialize ? EObjectEvent::Serialize : EObjectEvent::Create));
		AddDependency(ExportID, DepID);
	}
}

void FEDLCookChecker::AddPackageWithUnknownExports(FName LongPackageName)
{
	LLM_SCOPE_BYTAG(EDLCookChecker);
	if (bIsActive)
	{
		LLM_SCOPE_BYTAG(EDLCookChecker);
		PackagesWithUnknownExports.Add(LongPackageName);
	}
}


void FEDLCookChecker::AddDependency(FEDLNodeID SourceID, FEDLNodeID TargetID)
{
	// Add the edge; to save time don't check for uniqueness. Uniqueness is checked on demand, in
	// UniquifyNodePrereqsEdgeList.
	NodePrereqs.FindOrAdd(SourceID).Add(TargetID);
}

void FEDLCookChecker::StartSavingEDLCookInfoForVerification()
{
	LLM_SCOPE_BYTAG(EDLCookChecker);
	FScopeLock CookCheckerInstanceLock(&CookCheckerInstanceCritical);
	for (FEDLCookChecker* Checker : CookCheckerInstances)
	{
		Checker->Reset();
		Checker->SetActiveIfNeeded();
	}
}

void FEDLCookChecker::UniquifyNodePrereqsEdgeList(TArray<FEDLNodeID>& EdgeList)
{
	bool bNeedsShrink = false;

	// Avoid the cost of sort+unique if unnecessary, to reduce the cost if called multiple times.
	// This loop is similar to !Algo::IsSorted, but also returns true if any adjacent elements are equal.
	int32 NumToCompare = EdgeList.Num() - 1;
	for (int32 Index = 0; Index < NumToCompare; ++Index)
	{
		if (EdgeList[Index + 1] <= EdgeList[Index])
		{
			bNeedsShrink = true;
			break;
		}
	}
	if (bNeedsShrink)
	{
		Algo::Sort(EdgeList);
		EdgeList.SetNum(Algo::Unique(EdgeList));
	}
}

bool FEDLCookChecker::CheckForCyclesInner(TSet<FEDLNodeID>& Visited, TSet<FEDLNodeID>& Stack, const FEDLNodeID& Visit, FEDLNodeID& FailNode)
{
	bool bResult = false;
	if (Stack.Contains(Visit))
	{
		FailNode = Visit;
		bResult = true;
	}
	else
	{
		bool bWasAlreadyTested = false;
		Visited.Add(Visit, &bWasAlreadyTested);
		if (!bWasAlreadyTested)
		{
			Stack.Add(Visit);
			TArray<FEDLNodeID>* EdgeList = NodePrereqs.Find(Visit);
			if (EdgeList)
			{
				for (int32 EdgeListIndex = 0; !bResult && EdgeListIndex < EdgeList->Num(); ++EdgeListIndex)
				{
					bResult = CheckForCyclesInner(Visited, Stack, (*EdgeList)[EdgeListIndex], FailNode);
				}
			}
			Stack.Remove(Visit);
		}
	}
	UE_CLOGF(bResult && Stack.Contains(FailNode), LogSavePackage, Error, "Cycle Node %ls", *Nodes[Visit].ToString(*this));
	return bResult;
}

FEDLCookChecker::FEDLNodeID FEDLCookChecker::FindOrAddNode(const FEDLNodeHash& NodeHash)
{
	uint32 TypeHash = GetTypeHash(NodeHash);
	FEDLNodeID* NodeIDPtr = NodeHashToNodeID.FindByHash(TypeHash, NodeHash);
	if (NodeIDPtr)
	{
		return *NodeIDPtr;
	}

	FName Name = NodeHash.GetName();
	FEDLNodeHash ParentHash;
	FEDLNodeID ParentID = NodeHash.TryGetParent(ParentHash) ? FindOrAddNode(ParentHash) : NodeIDInvalid;
	FEDLNodeID NodeID = Nodes.Num();
	FEDLNodeData& NewNodeData = Nodes.Emplace_GetRef(NodeID, ParentID, Name, NodeHash.GetObjectEvent());
	NodeHashToNodeID.AddByHash(TypeHash, NewNodeData.GetNodeHash(*this), NodeID);
	return NodeID;
}

FEDLCookChecker::FEDLNodeID FEDLCookChecker::FindOrAddNode(FEDLNodeData&& NodeData, const FEDLCookChecker& OldOwnerOfNode, FEDLNodeID ParentIDInThis, bool& bNew)
{
	// Note that NodeData's Name and ParentID must be unmodified, since they might still be needed for GetHashCode calls from children

	FEDLNodeHash NodeHash = NodeData.GetNodeHash(OldOwnerOfNode);
	uint32 TypeHash = GetTypeHash(NodeHash);
	FEDLNodeID* NodeIDPtr = NodeHashToNodeID.FindByHash(TypeHash, NodeHash);
	if (NodeIDPtr)
	{
		bNew = false;
		return *NodeIDPtr;
	}

	FEDLNodeID NodeID = Nodes.Num();
	FEDLNodeData& NewNodeData = Nodes.Emplace_GetRef(NodeID, ParentIDInThis, NodeData.Name, MoveTemp(NodeData));
	NodeHashToNodeID.AddByHash(TypeHash, NewNodeData.GetNodeHash(*this), NodeID);
	bNew = true;
	return NodeID;
}

FEDLCookChecker::FEDLNodeID FEDLCookChecker::FindNode(const FEDLNodeHash& NodeHash)
{
	const FEDLNodeID* NodeIDPtr = NodeHashToNodeID.Find(NodeHash);
	return NodeIDPtr ? *NodeIDPtr : NodeIDInvalid;
}

void FEDLCookChecker::Merge(FEDLCookChecker&& Other)
{
	if (Nodes.Num() == 0)
	{
		Swap(Nodes, Other.Nodes);
		Swap(NodeHashToNodeID, Other.NodeHashToNodeID);
		Swap(NodePrereqs, Other.NodePrereqs);

		// Switch the pointers in all of the swapped data to point at this instead of Other
		for (TPair<FEDLNodeHash, FEDLNodeID>& KVPair : NodeHashToNodeID)
		{
			FEDLNodeHash& NodeHash = KVPair.Key;
			NodeHash.SetNodes(&Nodes);
		}
	}
	else
	{
		Other.NodeHashToNodeID.Empty(); // We will be invalidating the data these NodeHashes point to in the Other.Nodes loop, so empty the array now to avoid using it by accident

		TArray<FEDLNodeID> RemapIDs;
		RemapIDs.Reserve(Other.Nodes.Num());
		for (FEDLNodeData& NodeData : Other.Nodes)
		{
			FEDLNodeID ParentID;
			if (NodeData.ParentID == NodeIDInvalid)
			{
				ParentID = NodeIDInvalid;
			}
			else
			{
				// Parents should be earlier in the nodes list than children, since we always FindOrAdd the parent (and hence add it to the nodelist) when creating the child.
				// Since the parent is earlier in the nodes list, we have already transferred it, and its ID in this->Nodes is therefore RemapIDs[Other.ParentID]
				check(NodeData.ParentID < NodeData.ID);
				ParentID = RemapIDs[NodeData.ParentID];
			}

			bool bNew;
			FEDLNodeID NodeID = FindOrAddNode(MoveTemp(NodeData), Other, ParentID, bNew);
			if (!bNew)
			{
				Nodes[NodeID].Merge(MoveTemp(NodeData));
			}
			RemapIDs.Add(NodeID);
		}

		for (const TPair<FEDLNodeID, TArray<FEDLNodeID>>& Prereq : Other.NodePrereqs)
		{
			// Merge the edgelists; to save time don't check for uniqueness. Uniqueness is checked on demand, in
			// UniquifyNodePrereqsEdgeList.
			FEDLNodeID SourceID = RemapIDs[Prereq.Key];
			TArray<FEDLNodeID>& EdgeList = NodePrereqs.FindOrAdd(SourceID);
			for (FEDLNodeID OtherTargetID : Prereq.Value)
			{
				FEDLNodeID TargetID = RemapIDs[OtherTargetID];
				EdgeList.Add(TargetID);
			}
		}

		Other.NodePrereqs.Empty();
		Other.Nodes.Empty();
	}

	if (PackagesWithUnknownExports.Num() == 0)
	{
		Swap(PackagesWithUnknownExports, Other.PackagesWithUnknownExports);
	}
	else
	{
		PackagesWithUnknownExports.Reserve(Other.PackagesWithUnknownExports.Num());
		for (FName PackageName : Other.PackagesWithUnknownExports)
		{
			PackagesWithUnknownExports.Add(PackageName);
		}
		Other.PackagesWithUnknownExports.Empty();
	}
}

FEDLCookChecker FEDLCookChecker::AccumulateAndClear()
{
	FEDLCookChecker Accumulator;

	FScopeLock CookCheckerInstanceLock(&CookCheckerInstanceCritical);
	for (FEDLCookChecker* Checker : CookCheckerInstances)
	{
		if (Checker->bIsActive)
		{
			Accumulator.bIsActive = true;
			Accumulator.Merge(MoveTemp(*Checker));
			Checker->Reset();
			Checker->bIsActive = true;
		}
	}
	return Accumulator;
}

void FEDLCookChecker::Verify(const TFunction<void(UE::FLogRecord&& Record)>& MessageCallback,
	const TFunction<FString(FName PackageName)>& GetInstigatorChain,
	bool bFullReferencesExpected)
{
	LLM_SCOPE_BYTAG(EDLCookChecker);

	check(!UE::IsSavingPackage());
	FEDLCookChecker Accumulator = AccumulateAndClear();

	FString SeverityStr;
	GConfig->GetString(TEXT("CookSettings"), TEXT("CookContentMissingSeverity"), SeverityStr, GEditorIni);
	ELogVerbosity::Type MissingContentSeverity = ParseLogVerbosityFromString(SeverityStr);

	if (Accumulator.bIsActive)
	{
		double StartTime = FPlatformTime::Seconds();

		if (bFullReferencesExpected)
		{
			// imports to things that are not exports...
			for (const FEDLNodeData& NodeData : Accumulator.Nodes)
			{
				if (NodeData.bIsExport)
				{
					// The node is an export; imports of it are valid
					continue;
				}

				if (Accumulator.PackagesWithUnknownExports.Contains(NodeData.GetPackageName(Accumulator)))
				{
					// The node is an object in a package that exists, but for which we do not know the exports
					// because e.g. it was skipped by LegacyIterative in the current cook. Suppress warnings about it
					continue;
				}

				// Any imports of this non-exported node are an error; log them all if they exist
				if (NodeData.ImportingPackagesSorted.IsEmpty())
				{
					continue;
				}

				const FEDLNodeData* NodeDataOfExportPackage = &NodeData;
				while (NodeDataOfExportPackage->ParentID != NodeIDInvalid)
				{
					int32 ParentNodeIndex = static_cast<int32>(NodeDataOfExportPackage->ParentID);
					check(Accumulator.Nodes.IsValidIndex(ParentNodeIndex));
					NodeDataOfExportPackage = &Accumulator.Nodes[ParentNodeIndex];
				}

				const TCHAR* ReasonExportIsMissing = TEXT("");
				if (NodeDataOfExportPackage->bIsExport)
				{
					ReasonExportIsMissing = TEXT("the object was stripped out of the target package when saved");
				}
				else
				{
					ReasonExportIsMissing = TEXT("the target package was marked NeverCook or is not cookable for the target platform");
				}

				for (FName PackageName : NodeData.ImportingPackagesSorted)
				{
					UE::FLogRecord Record;
#if !NO_LOGGING
					Record.SetCategory(LogSavePackage.GetCategoryName());
#endif
					Record.SetVerbosity(MissingContentSeverity);
					Record.SetTime(UE::FLogTime::Now());
					Record.SetUtf8Format(UTF8TEXT("Content is missing from cook. Source package referenced an object in target package but {Reason}.")
						UTF8TEXT("\n\tSource package: {Source}")
						UTF8TEXT("\n\tTarget package: {Target}")
						UTF8TEXT("\n\tReferenced object: {ReferencedObject}")
						UTF8TEXT("\n\tSourceInstigator: {SourceInstigator}")
					);
					{
						FCbWriter Writer;
						Writer.BeginObject();
						Writer << "Reason" << ReasonExportIsMissing;
						Writer << "Source" << WriteToUtf8String<256>(PackageName);
						Writer << "Target" << WriteToUtf8String<256>(NodeDataOfExportPackage->Name);
						{
							TStringBuilder<256> ReferencedObjectStr;
							NodeData.AppendPathName(Accumulator, ReferencedObjectStr);
							Writer << "ReferencedObject" << ReferencedObjectStr;
						}
						Writer << "SourceInstigator" << GetInstigatorChain(PackageName);
						Writer.EndObject();
						Record.SetFields(Writer.Save().AsObject());
					}
					Record.SetFile(__FILE__);
					Record.SetLine(__LINE__);
					MessageCallback(MoveTemp(Record));
				}
			}
		}

		// cycles in the dep graph
		TSet<FEDLNodeID> Visited;
		TSet<FEDLNodeID> Stack;
		bool bHadCycle = false;
		for (TPair<FEDLNodeID, TArray<FEDLNodeID>>& NodePreqsPair : Accumulator.NodePrereqs)
		{
			UniquifyNodePrereqsEdgeList(NodePreqsPair.Value);
		}
		for (const FEDLNodeData& NodeData : Accumulator.Nodes)
		{
			if (!NodeData.bIsExport)
			{
				continue;
			}
			FEDLNodeID FailNode;
			if (Accumulator.CheckForCyclesInner(Visited, Stack, NodeData.ID, FailNode))
			{
				UE_LOGF(LogSavePackage, Error, "----- %ls contained a cycle (listed above).", *Accumulator.Nodes[FailNode].ToString(Accumulator));
				bHadCycle = true;
			}
		}
		if (bHadCycle)
		{
			UE_LOGF(LogSavePackage, Fatal, "EDL dep graph contained a cycle (see errors, above). This is fatal at runtime so it is fatal at cook time.");
		}
		UE_LOGF(LogSavePackage, Display, "Took %fs to verify the EDL loading graph.", float(FPlatformTime::Seconds() - StartTime));
	}
}

void FEDLCookChecker::MoveToCompactBinaryAndClear(FCbWriter& Writer, bool& bOutHasData)
{
	LLM_SCOPE_BYTAG(EDLCookChecker);

	bOutHasData = false;
	FEDLCookChecker Accumulator = AccumulateAndClear();
	if (!Accumulator.bIsActive)
	{
		return;
	}
	if (Accumulator.Nodes.IsEmpty() && Accumulator.NodePrereqs.IsEmpty() && Accumulator.PackagesWithUnknownExports.IsEmpty())
	{
		return;
	}
	bOutHasData = true;

	Accumulator.WriteToCompactBinary(Writer);
}

bool FEDLCookChecker::AppendFromCompactBinary(FCbFieldView Field)
{
	LLM_SCOPE_BYTAG(EDLCookChecker);
	FEDLCookChecker Instance;
	if (!Instance.ReadFromCompactBinary(Field))
	{
		return false;
	}
	FEDLCookChecker& CurrentChecker = FEDLCookCheckerThreadState::Get().Checker;
	CurrentChecker.Merge(MoveTemp(Instance));
	return true;
}

void FEDLCookChecker::WriteToCompactBinary(FCbWriter& Writer)
{
	Writer.BeginObject();
	{
		Writer.BeginArray("Nodes");
		for (const FEDLNodeData& Node : Nodes)
		{
			Writer << Node.Name;
			Writer << Node.ImportingPackagesSorted;
			Writer << Node.ParentID;
			Writer << static_cast<uint8>(Node.ObjectEvent);
			Writer << Node.bIsExport;
		}
		Writer.EndArray();
		Writer.BeginArray("NodePrereqs");
		// Transmit NumSources to use for reservation size
		uint32 NumSources = static_cast<uint32>(NodePrereqs.Num());
		Writer << NumSources;
		// Transmit the edgelist as (Source, NumEdgeList, EdgeListElements). Use a flat list of concatenated
		// objects to reduce transmission size.
		for (TPair<FEDLNodeID, TArray<FEDLNodeID>>& Pair : NodePrereqs)
		{
			Writer << static_cast<uint32>(Pair.Key);
			TArray<FEDLNodeID>& EdgeList = Pair.Value;
			UniquifyNodePrereqsEdgeList(EdgeList);
			uint32 NumDests = static_cast<uint32>(EdgeList.Num());
			Writer << NumDests;
			for (FEDLNodeID Dest : EdgeList)
			{
				Writer << static_cast<uint32>(Dest);
			}
		}
		Writer.EndArray();
		Writer.BeginArray("PackagesWithUnknownExports");
		for (FName PackageName : PackagesWithUnknownExports)
		{
			Writer << PackageName;
		}
		Writer.EndArray();
	}
	Writer.EndObject();
}

bool FEDLCookChecker::ReadFromCompactBinary(FCbFieldView Field)
{
	Reset();

	bool bSuccess = false;
	ON_SCOPE_EXIT
	{
		if (!bSuccess)
		{
			Reset();
		}
	};

	FCbFieldView NodesField = Field["Nodes"];
	const uint64 NumNodes = NodesField.AsArrayView().Num() / 5;
	if (NumNodes > MAX_int32)
	{
		return false;
	}
	Nodes.Reserve(static_cast<int32>(NumNodes));
	if (NodesField.HasError())
	{
		return false;
	}
	FCbFieldViewIterator NodeIter = NodesField.CreateViewIterator();
	while (NodeIter)
	{
		FEDLNodeID NodeID = Nodes.Num();
		FEDLNodeData& Node = Nodes.Emplace_GetRef();
		Node.ID = NodeID;
		if (!LoadFromCompactBinary(NodeIter, Node.Name)) { return false; }
		++NodeIter;
		if (!LoadFromCompactBinary(NodeIter, Node.ImportingPackagesSorted)) { return false; }
		++NodeIter;
		if (!LoadFromCompactBinary(NodeIter, Node.ParentID)) { return false; }
		++NodeIter;
		uint8 LocalObjectEvent;
		if (!LoadFromCompactBinary(NodeIter, LocalObjectEvent)) { return false; }
		if (LocalObjectEvent > static_cast<uint8>(EObjectEvent::Max)) { return false; }
		Node.ObjectEvent = static_cast<EObjectEvent>(LocalObjectEvent);
		++NodeIter;
		if (!LoadFromCompactBinary(NodeIter, Node.bIsExport)) { return false; }
		++NodeIter;
	}

	FCbFieldView PrereqsField = Field["NodePrereqs"];
	FCbFieldViewIterator PrereqsIter = PrereqsField.CreateViewIterator();
	uint32 NumSources = 0;
	if (!LoadFromCompactBinary(PrereqsIter++, NumSources)) { return false; }
	NodePrereqs.Reserve(static_cast<int32>(NumSources));
	while (PrereqsIter)
	{
		uint32 SourceAsInt;
		if (!LoadFromCompactBinary(PrereqsIter++, SourceAsInt)) { return false; }
		uint32 NumDests;
		if (!LoadFromCompactBinary(PrereqsIter++, NumDests)) { return false; }
		TArray<FEDLNodeID>& EdgeList = NodePrereqs.FindOrAdd(static_cast<FEDLNodeID>(SourceAsInt));
		EdgeList.Reserve(NumDests);
		for (uint32 DestIndex = 0; DestIndex < NumDests; ++DestIndex)
		{
			uint32 DestAsInt;
			if (!LoadFromCompactBinary(PrereqsIter++, DestAsInt)) { return false; }
			EdgeList.Add(DestAsInt);
		}
	}

	FCbFieldView PackagesWithUnknownExportsField = Field["PackagesWithUnknownExports"];
	const int64 NumPackagesWithUnknownExports = PackagesWithUnknownExportsField.AsArrayView().Num();
	if (NumPackagesWithUnknownExports > MAX_int32)
	{
		return false;
	}
	PackagesWithUnknownExports.Reserve(static_cast<int32>(NumPackagesWithUnknownExports));
	if (PackagesWithUnknownExportsField.HasError())
	{
		return false;
	}
	for (FCbFieldView PackageNameField : PackagesWithUnknownExportsField)
	{
		FName PackageName;
		if (!LoadFromCompactBinary(PackageNameField, PackageName))
		{
			return false;
		}
		PackagesWithUnknownExports.Add(PackageName);
	}

	for (const FEDLNodeData& Node : Nodes)
	{
		NodeHashToNodeID.Add(Node.GetNodeHash(*this), Node.ID);
	}
	bIsActive = !Nodes.IsEmpty() || !NodePrereqs.IsEmpty() || !PackagesWithUnknownExports.IsEmpty();

	bSuccess = true;
	return true;
}

FCriticalSection FEDLCookChecker::CookCheckerInstanceCritical;
TArray<FEDLCookChecker*> FEDLCookChecker::CookCheckerInstances;

namespace UE::Cook
{

void FImportExportNode::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << ObjectName;
	Writer << ParentId;
	Writer.EndArray();
}

bool FImportExportNode::TryLoad(const FCbFieldView& Field)
{
	FCbFieldViewIterator ElementView(Field.CreateViewIterator());
	if (!LoadFromCompactBinary(ElementView++, ObjectName))
	{
		return false;
	}
	if (!LoadFromCompactBinary(ElementView++, ParentId))
	{
		return false;
	}
	return true;
}

void FImportsCheckerData::Save(FCbWriter& Writer) const
{
	Writer.BeginObject();
	Writer << "Imports" << Imports;
	Writer << "Exports" << Exports;
	Writer.EndObject();
}

bool FImportsCheckerData::TryLoad(const FCbFieldView& Field)
{
	bool bImports = false;
	bool bExports = false;
	for (FCbFieldViewIterator ElementView(Field.CreateViewIterator()); ElementView; )
	{
		const FCbFieldViewIterator Last = ElementView;
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("Imports")))
		{
			bImports = true;
			if (!LoadFromCompactBinary(ElementView++, Imports))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("Exports")))
		{
			bExports = true;
			if (!LoadFromCompactBinary(ElementView++, Exports))
			{
				return false;
			}
		}
		if (ElementView == Last)
		{
			++ElementView;
		}
	}
	return bImports && bExports;
}

FImportsCheckerData FImportsCheckerData::FromObjectLists(TConstArrayView<UObject*> Imports,
	TConstArrayView<UObject*> Exports)
{
	FImportsCheckerData Result;
	TArray<UObject*> FilteredImports;
	FilteredImports.Reserve(Imports.Num());
	for (UObject* Import : Imports)
	{
		if (!Import->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn))
		{
			FilteredImports.Add(Import);
		}
	}
	Result.Imports = ObjectListToNodeList(FilteredImports);
	Result.Exports = ObjectListToNodeList(Exports);
	return Result;
}

TArray<FImportExportNode> FImportsCheckerData::ObjectListToNodeList(TConstArrayView<UObject*> Objects)
{
	TArray<FImportExportNode> Result;

	// Iterate each LeafObject in the array of Objects, and walk up the outer chain of each Object recursively adding
	// a node for each outer in the outer chain, and then add a node for the object as a child of the outer's node. If
	// any Outer (or even the LeafObject itself) has already been given a node, use the index of that node from the map
	// and stop walking up the stack. When walking up the outer chain we keep a stack of objects we are working on
	// beneath the current object in the outer chain, and when we reach the outermost or an already handled node, we
	// keep a parentindex variable which we set in previous loop iteration on the outer and use that as the recursive
	// result for the outer.
	TArray<UObject*, TInlineAllocator<10>> Stack;
	TMap<UObject*, int32> Map;
	Map.Reserve(Objects.Num());
	for (UObject* LeafObject : Objects)
	{
		check(Stack.IsEmpty());
		int32 ParentIndex = -1;
		UObject* Current = LeafObject;
		while (Current)
		{
			int32 CurrentIndex = -1;
			int32* CurrentIndexPtr = Map.Find(Current);
			if (CurrentIndexPtr)
			{
				CurrentIndex = *CurrentIndexPtr;
			}
			else
			{
				UObject* Outer = Current->GetOuter();
				if (Outer && ParentIndex == -1)
				{
					Stack.Push(Current);
					Current = Outer;
					continue;
				}

				CurrentIndex = Result.Num();
				FImportExportNode& CurrentNode = Result.Emplace_GetRef();
				check(Current); // Workaround StaticAnalyzer bug: spurious warning C28182: Dereferencing NULL pointer
				CurrentNode.ObjectName = Current->GetFName();
				CurrentNode.ParentId = ParentIndex;
				Map.Add(Current, CurrentIndex);
			}

			check(0 <= CurrentIndex && CurrentIndex < Result.Num());
			ParentIndex = CurrentIndex;
			Current = !Stack.IsEmpty() ? Stack.Pop(EAllowShrinking::No) : nullptr;
		}
	}

	return Result;
}

}
