// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "HAL/CriticalSection.h"
#include "HAL/ThreadSingleton.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

class FCbFieldView;
class FCbWriter;
class UPackage;
namespace UE { class FLogRecord; }

struct FEDLCookCheckerThreadState;

namespace UE::Cook
{

/**
 * Data describing an object's name in its outer hierarchy. Used so that we can persistently represent all the names in
 * a tree of object outers in a list that does not duplicate the string data between an object and
 * its outer.
 */
struct FImportExportNode
{
public:
	FName ObjectName;
	int32 ParentId = -1;

	void Save(FCbWriter& Writer) const;
	bool TryLoad(const FCbFieldView& Field);
private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FImportExportNode& Node)
	{
		Node.Save(Writer);
		return Writer;
	}
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FImportExportNode& Node)
	{
		return Node.TryLoad(Field);
	}
};

/**
 * Data about imports and exports from a package that can be stored in the oplog for incremental cooks for replay into
 * the CookImportsChecker when a package is skipped.
 */
struct FImportsCheckerData
{
public:
	TArray<FImportExportNode> Imports;
	TArray<FImportExportNode> Exports;

	bool IsEmpty() const;
	void Save(FCbWriter& Writer) const;
	bool TryLoad(const FCbFieldView& Field);

	static FImportsCheckerData FromObjectLists(TConstArrayView<UObject*> Imports,
		TConstArrayView<UObject*> Exports);
	static TArray<FImportExportNode> ObjectListToNodeList(TConstArrayView<UObject*> Objects);

private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FImportsCheckerData& Data)
	{
		Data.Save(Writer);
		return Writer;
	}
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FImportsCheckerData& Data)
	{
		return Data.TryLoad(Field);
	}
};

} // namespace UE::Cook

/**
 * Helper struct used during cooking to validate EDL dependencies
 */
struct FEDLCookChecker
{
	void SetActiveIfNeeded();

	void Reset();

	void AddImport(TObjectPtr<UObject> Import, UPackage* ImportingPackage);
	void AddImports(TConstArrayView<UE::Cook::FImportExportNode> Imports, FName ImportingPackageName);
	void AddExport(UObject* Export);
	void AddExports(TConstArrayView<UE::Cook::FImportExportNode> Exports);
	void Add(UE::Cook::FImportsCheckerData& ImportsCheckerData, FName PackageName);
	void AddArc(UObject* DepObject, bool bDepIsSerialize, UObject* Export, bool bExportIsSerialize);
	void AddPackageWithUnknownExports(FName LongPackageName);

	static void StartSavingEDLCookInfoForVerification();
	static void Verify(const TFunction<void(UE::FLogRecord&& Record)>& MessageCallback,
		const TFunction<FString(FName PackageName)>& GetInstigatorChain,
		bool bFullReferencesExpected);
	static void MoveToCompactBinaryAndClear(FCbWriter& Writer, bool& bOutHasData);
	static bool AppendFromCompactBinary(FCbFieldView Field);

private:
	static FEDLCookChecker AccumulateAndClear();
	void WriteToCompactBinary(FCbWriter& Writer);
	bool ReadFromCompactBinary(FCbFieldView Field);

	typedef uint32 FEDLNodeID;
	static const FEDLNodeID NodeIDInvalid = static_cast<FEDLNodeID>(-1);

	struct FEDLNodeData;
public: // FEDLNodeHash is public only so that GetTypeHash can be defined
	enum class EObjectEvent : uint8
	{
		Create,
		Serialize,
		Max = Serialize,
	};

	/**
	 * Wrapper around an FEDLNodeData (or around a UObject when searching for an FEDLNodeData corresponding to the UObject)
	 * that provides the hash-by-objectpath to lookup the FEDLNodeData for an objectpath.
	 */
	struct FEDLNodeHash
	{
		FEDLNodeHash(); // creates an uninitialized node; only use this to provide as an out parameter
		FEDLNodeHash(const TArray<FEDLNodeData>* InNodes, FEDLNodeID InNodeID, EObjectEvent InObjectEvent);
		FEDLNodeHash(const TArray<FEDLNodeData>* InNodes, FEDLNodeID InParentNodeID, FName InObjectName, EObjectEvent InObjectEvent);
		FEDLNodeHash(TObjectPtr<UObject> InObject, EObjectEvent InObjectEvent);
		FEDLNodeHash(const FEDLNodeHash& Other);
		bool operator==(const FEDLNodeHash& Other) const;
		FEDLNodeHash& operator=(const FEDLNodeHash& Other);
		friend uint32 GetTypeHash(const FEDLNodeHash& A);

		FName GetName() const;
		bool TryGetParent(FEDLNodeHash& Parent) const;
		EObjectEvent GetObjectEvent() const;
		void SetNodes(const TArray<FEDLNodeData>* InNodes);

	private:
		static uint32 GetTypeHashInternal(const FEDLNodeHash& A);

		struct FNodeTypeData
		{
			/**
			 * The array of nodes from the FEDLCookChecker; this is how we lookup the node for the NodeID.
			 * Because the FEDLNodeData are elements in an array which can resize and therefore reallocate the nodes, we cannot store the pointer to the node.
			 */
			const TArray<FEDLNodeData>* Nodes;
			/** The identifier for the FEDLNodeData this hash is wrapping. */
			FEDLNodeID NodeID;
		};
		struct FNameTypeData
		{
			FName ObjectName;
			/** The array of nodes from the FEDLCookChecker, the same as used in FNodeTypeData. */
			const TArray<FEDLNodeData>* Nodes;
			FEDLNodeID ParentID;
		};
		struct FObjectTypeData
		{
			TObjectPtr<const UObject> Object;
		};
		enum class ENodeHashType : uint8
		{
			Node,
			Object,
			NameAndParentNode,
		};
		union
		{
			FNodeTypeData NodeTypeData;
			FObjectTypeData ObjectTypeData;
			FNameTypeData NameTypeData;
		};
		ENodeHashType NodeHashType;
		EObjectEvent ObjectEvent;
	};

private:

	/**
	 * Node representing either the Create event or Serialize event of a UObject in the graph of runtime dependencies between UObjects.
	 */
	struct FEDLNodeData
	{
		// Note that order of the variables is important to reduce alignment waste in the size of FEDLNodeData.
		/** Name of the UObject represented by this node; full objectpath name is obtainable by traversing parent. */
		FName Name;
		/**
		 * Index of this node in the FEDLCookChecker's Nodes array. This index is used to provide a small-memory-usage
		 * identifier for the node.
		 */
		FEDLNodeID ID;
		/**
		 * Tracks references to this node's UObjects from other packages (which is the reverse of the references
		 * from each node that we track in NodePrereqs.)
		 * We only need this information from each package, so we track by package name instead of node id.
		 */
		TArray<FName> ImportingPackagesSorted;
		/**
		 * ID of the node representing the UObject parent of this node's UObject. NodeIDInvalid if the UObject has no parent.
		 * The ParentID always refers to the node for the Create event of the parent UObject.
		 */
		uint32 ParentID;
		/** True if this node represents the Serialize event on the UObject, false if it represents the Create event. */
		EObjectEvent ObjectEvent;
		/**
		 * True if the UObject represented by this node has been exported by a SavePackage call; used to verify that
		 * the imports requested by packages are present somewhere in the cook.
		 */
		bool bIsExport;

		FEDLNodeData() { /* Fields are uninitialized */ }
		FEDLNodeData(FEDLNodeID InID, FEDLNodeID InParentID, FName InName, EObjectEvent InObjectEvent);
		FEDLNodeData(FEDLNodeID InID, FEDLNodeID InParentID, FName InName, FEDLNodeData&& Other);
		FEDLNodeHash GetNodeHash(const FEDLCookChecker& Owner) const;

		FString ToString(const FEDLCookChecker& Owner) const;
		void AppendPathName(const FEDLCookChecker& Owner, FStringBuilderBase& Result) const;
		FName GetPackageName(const FEDLCookChecker& Owner) const;
		void Merge(FEDLNodeData&& Other);
	};

	FEDLNodeID FindOrAddNode(const FEDLNodeHash& NodeLookup);
	FEDLNodeID FindOrAddNode(FEDLNodeData&& NodeData, const FEDLCookChecker& OldOwnerOfNode, FEDLNodeID ParentIDInThis, bool& bNew);
	FEDLNodeID FindNode(const FEDLNodeHash& NodeHash);
	template <typename AddNodeType>
	void AddImportExportNodeList(TConstArrayView<UE::Cook::FImportExportNode> NodeList, AddNodeType&& AddNode);
	void RecordImportFromPackage(FEDLNodeID NodeId, FName ImportingPackageName);
	void Merge(FEDLCookChecker&& Other);
	bool CheckForCyclesInner(TSet<FEDLNodeID>& Visited, TSet<FEDLNodeID>& Stack, const FEDLNodeID& Visit, FEDLNodeID& FailNode);
	void AddDependency(FEDLNodeID SourceID, FEDLNodeID TargetID);
	static void UniquifyNodePrereqsEdgeList(TArray<FEDLNodeID>& EdgeList);

	/**
	 * All the FEDLNodeDatas that have been created for this checker. These are allocated as elements of an array rather than pointers to reduce cputime and
	 * memory due to many small allocations, and to provide index-based identifiers. Nodes are not deleted until the checker is reset.
	 */
	TArray<FEDLNodeData> Nodes;
	/** A map to lookup the node for a UObject or for the corresponding node in another thread's FEDLCookChecker. */
	TMap<FEDLNodeHash, FEDLNodeID> NodeHashToNodeID;
	/**
	 * The graph of dependencies between nodes. Format is a map from Source node to a custom-format edgelist. 
	 * The custom format edge-list is an optionally (sorted and unique) array of Dest nodes. We make it sorted and unique
	 * when we read it. This format is performant for our use case, which is a period of unique Source->Target writes
	 * with no reads followed by a period of reads with no writes. Size of the graph can be very large on large projects
	 * (20 million vertices, 100 million edges); we use this format to prevent int32 overflow of the allocation size
	 * of the map buckets and to otherwise be as memory-efficient as possible.
	 */
	TMap<FEDLNodeID, TArray<FEDLNodeID>> NodePrereqs;
	/**
	 * Packages that were cooked with LegacyIterative and therefore have an unknown set of exports.
	 * We suppress warnings for exports missing from these packages.
	 */
	TSet<FName> PackagesWithUnknownExports;
	/** True if the EDLCookChecker should be active; it is turned off if the runtime will not be using EDL. */
	bool bIsActive = false;

	/** When cooking with concurrent saving, each thread has its own FEDLCookChecker, and these are merged after the cook is complete. */
	static FCriticalSection CookCheckerInstanceCritical;
	static TArray<FEDLCookChecker*> CookCheckerInstances;

	friend FEDLCookCheckerThreadState;
};

/** Per-thread accessor for writing EDL dependencies to global FEDLCookChecker storage. */
struct FEDLCookCheckerThreadState : public TThreadSingleton<FEDLCookCheckerThreadState>
{
	FEDLCookCheckerThreadState();

	void AddImport(TObjectPtr<UObject> Import, UPackage* ImportingPackage)
	{
		Checker.AddImport(Import, ImportingPackage);
	}
	void AddImports(TConstArrayView<UE::Cook::FImportExportNode> Imports, FName ImportingPackageName)
	{
		Checker.AddImports(Imports, ImportingPackageName);
	}
	void AddExport(UObject* Export)
	{
		Checker.AddExport(Export);
	}
	void AddExports(TConstArrayView<UE::Cook::FImportExportNode> Exports)
	{
		Checker.AddExports(Exports);
	}
	void Add(UE::Cook::FImportsCheckerData& ImportsCheckerData, FName PackageName)
	{
		Checker.Add(ImportsCheckerData, PackageName);
	}
	void AddArc(UObject* DepObject, bool bDepIsSerialize, UObject* Export, bool bExportIsSerialize)
	{
		Checker.AddArc(DepObject, bDepIsSerialize, Export, bExportIsSerialize);
	}
	void AddPackageWithUnknownExports(FName LongPackageName)
	{
		Checker.AddPackageWithUnknownExports(LongPackageName);
	}

private:
	FEDLCookChecker Checker;
	friend TThreadSingleton<FEDLCookCheckerThreadState>;
	friend FEDLCookChecker;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::Cook
{

inline bool FImportsCheckerData::IsEmpty() const
{
	return Imports.IsEmpty() && Exports.IsEmpty();
}

} // namespace UE::Cook
