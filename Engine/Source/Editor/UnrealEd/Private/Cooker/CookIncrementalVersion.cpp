// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookTypes.h"

#include "Misc/Guid.h"

namespace UE::Cook
{

// Engine configuration is in cpp: change the value of CookIncrementalVersion to a new guid when all packages in all
// platforms in all projects need to be invalidated in the next incremental cook.

FGuid CookIncrementalVersion(0xFFA12FDC, 0xC8221FE2, 0x7757627A, 0x12F1D3A7);

// Project configuration is in ini: change the value of ProjectCookIncrementalVersion in
// Editor.ini:[CookSettings]:ProjectCookIncrementalVersion to a new guid when all packages in all platforms in a
// single project need to be invalidated in the next incremental cook.

FGuid ProjectCookIncrementalVersion(0, 0, 0, 0);

}
