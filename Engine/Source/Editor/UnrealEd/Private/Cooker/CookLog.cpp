// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookOnTheSide/CookLog.h"

#include "Cooker/CookLogPrivate.h"

#include "Containers/Utf8String.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookWorkerClient.h"
#include "CoreGlobals.h"
#include "Logging/StructuredLog.h"
#include "Logging/StructuredLogFormat.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CompactBinarySerialization.h"

DEFINE_LOG_CATEGORY(LogCook);
DEFINE_LOG_CATEGORY(LogCookStats);
DEFINE_LOG_CATEGORY(LogCookStatus);
DEFINE_LOG_CATEGORY(LogCookList);

FName LogCookName(TEXT("LogCook"));

namespace UE::Cook
{

FCbWriter& operator<<(FCbWriter& Writer, const FReplicatedLogData& LogData)
{
	// Serializing as an array of unnamed fields and using the quantity of fields
	// as the discriminator between structured and unstructured log data.
	Writer.BeginArray();
	if (LogData.LogDataVariant.IsType<FReplicatedLogData::FUnstructuredLogData>())
	{
		const FReplicatedLogData::FUnstructuredLogData& UnstructuredLogData = LogData.LogDataVariant.Get<FReplicatedLogData::FUnstructuredLogData>();
		Writer << UnstructuredLogData.Category;
		uint8 Verbosity = static_cast<uint8>(UnstructuredLogData.Verbosity);
		Writer << Verbosity;
		Writer << UnstructuredLogData.Message;
	}
	else if (LogData.LogDataVariant.IsType<FCbObject>())
	{
		Writer << LogData.LogDataVariant.Get<FCbObject>();
	}
	else
	{
		checkNoEntry();
	}
	Writer.EndArray();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FReplicatedLogData& OutLogData)
{
	bool bOk = true;
	FCbArrayView ArrayView = Field.AsArrayView();
	switch (ArrayView.Num())
	{
	case 3:
	{
		OutLogData.LogDataVariant.Emplace<FReplicatedLogData::FUnstructuredLogData>();
		FReplicatedLogData::FUnstructuredLogData& UnstructuredLogData = OutLogData.LogDataVariant.Get<FReplicatedLogData::FUnstructuredLogData>();
		FCbFieldViewIterator It = ArrayView.CreateViewIterator();
		bOk = LoadFromCompactBinary(*It++, UnstructuredLogData.Category) & bOk;
		uint8 Verbosity;
		if (LoadFromCompactBinary(*It++, Verbosity))
		{
			UnstructuredLogData.Verbosity = static_cast<ELogVerbosity::Type>(Verbosity);
		}
		else
		{
			bOk = false;
			UnstructuredLogData.Verbosity = static_cast<ELogVerbosity::Type>(0);
		}
		bOk = LoadFromCompactBinary(*It++, UnstructuredLogData.Message) & bOk;
		break;
	}
	case 1:
	{
		OutLogData.LogDataVariant.Emplace<FCbObject>();
		FCbObject& StructuredLogData = OutLogData.LogDataVariant.Get<FCbObject>();
		FCbFieldViewIterator It = ArrayView.CreateViewIterator();
		if (It->IsObject())
		{
			StructuredLogData = FCbObject::Clone(It->AsObjectView());
		}
		else
		{
			bOk = false;
		}
		break;
	}
	default:
		bOk = false;
	}
	return bOk;
}

class FLogHandler : public FOutputDevice, public ILogHandler
{
public:
	explicit FLogHandler(UCookOnTheFlyServer& InCOTFS);
	virtual ~FLogHandler();

	virtual void ReplayLogsFromIncrementallySkipped(FName PackageName,
		TConstArrayView<FReplicatedLogData> LogMessages) override;
	virtual void ReplayLogFromCookWorker(FReplicatedLogData&& LogData, int32 CookWorkerProfileId) override;
	virtual void ConditionalPruneReplay() override;
	virtual void FlushIncrementalCookLogs() override;

	// FOutputDevice
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category,
		const double Time) override;
	virtual void SerializeRecord(const FLogRecord& Record) override;
	virtual void Flush() override;
	virtual bool CanBeUsedOnAnyThread() const override;
	virtual bool CanBeUsedOnMultipleThreads() const override;

private:
	struct FQueuedLog
	{
		FName ActivePackage;
		FReplicatedLogData LogData;
	};

	void Marshal(FReplicatedLogData& OutData, FStringView Message, ELogVerbosity::Type Verbosity,
		const FName& Category);
	void Marshal(FReplicatedLogData& OutData, const FLogRecord& LogRecord);
	void UnMarshalAndLog(const FReplicatedLogData& LogData,
		TFunctionRef<bool(FName, const FString&)> MessagePassesFilter,
		TFunctionRef<bool(const FString&, FString&)> TryTransformMessage);
	bool UnMarshal(FCbFieldView Field, FLogRecord& OutLogRecord,
		TFunctionRef<bool(FName, const FString&)> MessagePassesFilter,
		TFunctionRef<bool(const FString&, FString&)> TryTransformMessage);

	template <UE::CCharType CharType>
	void ReportActiveLog(FReplicatedLogData&& LogData, TStringView<CharType> FormatMessage, ELogVerbosity::Type Verbosity);
	void RecordLogForIncrementalCook(FReplicatedLogData&& LogData, ELogVerbosity::Type Verbosity);
	void RecordLogForIncrementalCookGameThreadPortion(FName PackageName, FReplicatedLogData&& LogData);

	void PruneReplay();

private:
	UCookOnTheFlyServer& COTFS;
	bool bRegistered = false;

	FCriticalSection QueuedLogsForIncrementalCookLock;
	TArray<FQueuedLog> QueuedLogsForIncrementalCook;

	FCriticalSection TableLock;
	TArray<FString> StringTable;
	TArray<FUtf8String> Utf8StringTable;
	TArray<FUniqueLogTemplate> TemplateTable;
};

FLogHandler::FLogHandler(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{
	check(!bRegistered);
	check(GLog);
	GLog->AddOutputDevice(this);
	bRegistered = true;
}

FLogHandler::~FLogHandler()
{
	PruneReplay();

	if (bRegistered)
	{
		if (GLog)
		{
			GLog->RemoveOutputDevice(this);
		}
		bRegistered = false;
	}
}

void FLogHandler::FlushIncrementalCookLogs()
{
	TArray<FQueuedLog> LocalQueuedLogs;
	{
		FScopeLock ScopeLock(&QueuedLogsForIncrementalCookLock);
		LocalQueuedLogs = MoveTemp(QueuedLogsForIncrementalCook);
	}

	for (FQueuedLog& QueuedLog : LocalQueuedLogs)
	{
		RecordLogForIncrementalCookGameThreadPortion(QueuedLog.ActivePackage, MoveTemp(QueuedLog.LogData));
	}
}

void FLogHandler::ReplayLogsFromIncrementallySkipped(FName PackageName,
	TConstArrayView<FReplicatedLogData> LogMessages)
{
	// Replays only come from MarkPackageIncrementallySkipped, which happens only on the CookDirector, during
	// CookRequestCluster traversal. We rely on that, and do not report whether messages from CookWorkers
	// came from a replay or an active log, we always assume they came from active logs. So we currently forbid
	// replay on CookWorkers.
	check(!COTFS.CookWorkerClient);

	if (LogMessages.IsEmpty())
	{
		return;
	}

	auto MessagePassesFilter = [](FName Category, const FString& Message)
		{
			return true;
		};
	auto TryTransformMessage = [](const FString& In, FString& Out)
		{
			Out = FString::Printf(TEXT("[IncrementalReplay]: %s"), *In);
			return true;
		};
	UE_LOGF(LogCook, Display, "IncrementalReplay of messages for %ls", *WriteToString<256>(PackageName))
	for (const FReplicatedLogData& LogMessage : LogMessages)
	{
		UnMarshalAndLog(LogMessage, MessagePassesFilter, TryTransformMessage);
	}
}

void FLogHandler::ReplayLogFromCookWorker(FReplicatedLogData&& LogData, int32 CookWorkerProfileId)
{
	auto MessagePassesFilter = [](FName Category, const FString& Message)
		{
			// Do not spam heartbeat messages into the CookDirector log
			return Category != LogCookName || !Message.Contains(HeartbeatCategoryText);
		};
	auto TryTransformMessage = [CookWorkerProfileId](const FString& In, FString& Out)
		{
			Out = FString::Printf(TEXT("[CookWorker %d]: %s"), CookWorkerProfileId, *In);
			return true;
		};
	UnMarshalAndLog(LogData, MessagePassesFilter, TryTransformMessage);
}

void FLogHandler::ConditionalPruneReplay()
{
	// Flush if the tables in the serialization context have exceeded 100 entries
	const int32 TableSizeToFlushAt = 100;
	if ((StringTable.Num() > TableSizeToFlushAt)
		|| (Utf8StringTable.Num() > TableSizeToFlushAt)
		|| (TemplateTable.Num() > TableSizeToFlushAt))
	{
		PruneReplay();
	}
}

void FLogHandler::PruneReplay()
{
	// We are going to drop data from our tables that might be pointed to from logs still pending in GLog. So Flush
	// logs before we prune the tables.
	if (!StringTable.IsEmpty() || !Utf8StringTable.IsEmpty() || !TemplateTable.IsEmpty())
	{
		// NOTE: We only call FlushThreadedLogs on GLog even though we might serialize structured logs via GLog or GWarn.
		// GWarn is an output device, but GLog is a an output redirector, and only the redirector has/needs FlushThreadedLogs.
		// Output devices are expected to not use any pointer on a structured log record after completion of the SerializeRecord call.
		GLog->FlushThreadedLogs();
	}

	StringTable.Empty();
	Utf8StringTable.Empty();
	TemplateTable.Empty();
}

void FLogHandler::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity,
	const FName& Category)
{
	FReplicatedLogData SerializedData;
	FStringView FormatString(V);
	Marshal(SerializedData, V, Verbosity, Category);
	ReportActiveLog(MoveTemp(SerializedData), FormatString, Verbosity);
}

void FLogHandler::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity,
	const FName& Category, const double Time)
{
	Serialize(V, Verbosity, Category);
}

void FLogHandler::SerializeRecord(const UE::FLogRecord& Record)
{
	FReplicatedLogData SerializedData;
	Marshal(SerializedData, Record);
	if (const UTF8CHAR* Utf8Format = Record.GetUtf8Format())
	{
		ReportActiveLog<UTF8CHAR>(MoveTemp(SerializedData), Utf8Format, Record.GetVerbosity());
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		ReportActiveLog<TCHAR>(MoveTemp(SerializedData), Record.GetFormat(), Record.GetVerbosity());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}
}

void FLogHandler::Flush()
{
	PruneReplay();
}

bool FLogHandler::CanBeUsedOnAnyThread() const
{
	return true;
}

bool FLogHandler::CanBeUsedOnMultipleThreads() const
{
	return true;
}

void FLogHandler::Marshal(FReplicatedLogData& OutData, FStringView Message,
	ELogVerbosity::Type Verbosity, const FName& Category)
{
	OutData.LogDataVariant.Emplace<FReplicatedLogData::FUnstructuredLogData>();
	FReplicatedLogData::FUnstructuredLogData& OutVal
		= OutData.LogDataVariant.Get<FReplicatedLogData::FUnstructuredLogData>();
	OutVal.Message = Message;
	OutVal.Category = Category;
	OutVal.Verbosity = Verbosity;
}

void FLogHandler::Marshal(FReplicatedLogData& OutData, const FLogRecord& LogRecord)
{
	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "S";
	Writer.BeginArray();
	Writer << LogRecord.GetCategory();
	Writer << static_cast<uint8>(LogRecord.GetVerbosity());
	Writer << LogRecord.GetTime().GetUtcTime();
	if (const UTF8CHAR* Utf8Format = LogRecord.GetUtf8Format())
	{
		Writer << Utf8Format;
	}
	else
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		Writer << LogRecord.GetFormat();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}
	Writer << LogRecord.GetFields();
	Writer << LogRecord.GetFile();
	Writer << LogRecord.GetLine();
	Writer << LogRecord.GetTextNamespace();
	Writer << LogRecord.GetTextKey();
	Writer.EndArray();
	Writer.EndObject();
	FCbObject Object = Writer.Save().AsObject();
	OutData.LogDataVariant.Emplace<FCbObject>(MoveTemp(Object));
}

void FLogHandler::UnMarshalAndLog(const FReplicatedLogData& LogData,
	TFunctionRef<bool(FName, const FString&)> MessagePassesFilter,
	TFunctionRef<bool(const FString&, FString&)> TryTransformMessage)
{
	if (const FReplicatedLogData::FUnstructuredLogData* UnStructuredLogData
		= LogData.LogDataVariant.TryGet<FReplicatedLogData::FUnstructuredLogData>())
	{
		if (!MessagePassesFilter(UnStructuredLogData->Category, UnStructuredLogData->Message))
		{
			return;
		}
		const FString* SerializedString = &UnStructuredLogData->Message;
		FString TransformedString;
		if (TryTransformMessage(*SerializedString, TransformedString))
		{
			SerializedString = &TransformedString;
		}

		FMsg::Logf(__FILE__, __LINE__, UE_FNAME_TO_LOG_CATEGORY_NAME(UnStructuredLogData->Category), UnStructuredLogData->Verbosity,
			TEXT("%s"), **SerializedString);
	}
	else if (const FCbObject* StructuredLogObject = LogData.LogDataVariant.TryGet<FCbObject>())
	{
		FLogRecord LogRecord;
		if (UnMarshal((*StructuredLogObject)["S"], LogRecord, MessagePassesFilter, TryTransformMessage))
		{
			FOutputDevice* LogOverride = nullptr;
			switch (LogRecord.GetVerbosity())
			{
			case ELogVerbosity::Error:
			case ELogVerbosity::Warning:
			case ELogVerbosity::Display:
			case ELogVerbosity::SetColor:
				LogOverride = GWarn;
				break;
			default:
				break;
			}
			if (LogOverride)
			{
				LogOverride->SerializeRecord(LogRecord);
			}
			else
			{
				GLog->SerializeRecord(LogRecord);
			}
		}
	}
	else
	{
		checkNoEntry();
	}
}

bool FLogHandler::UnMarshal(FCbFieldView Field, FLogRecord& OutLogRecord,
	TFunctionRef<bool(FName, const FString&)> MessagePassesFilter,
	TFunctionRef<bool(const FString&, FString&)> TryTransformMessage)
{
	bool bOk = true;
	FCbFieldViewIterator It = Field.CreateViewIterator();
	FName Category;
	if (LoadFromCompactBinary(*It++, Category))
	{
		OutLogRecord.SetCategory(Category);
	}
	else
	{
		bOk = false;
	}
	if (uint8 Verbosity; LoadFromCompactBinary(*It++, Verbosity) && Verbosity < ELogVerbosity::NumVerbosity)
	{
		OutLogRecord.SetVerbosity(static_cast<ELogVerbosity::Type>(Verbosity));
	}
	else
	{
		bOk = false;
	}
	if (FDateTime Time; LoadFromCompactBinary(*It++, Time))
	{
		OutLogRecord.SetTime(FLogTime::FromUtcTime(Time));
	}
	else
	{
		bOk = false;
	}
	if (FString SerializedString;
		LoadFromCompactBinary(*It++, SerializedString) && MessagePassesFilter(Category, SerializedString))
	{
		FString TransformedString;
		if (TryTransformMessage(SerializedString, TransformedString))
		{
			SerializedString = MoveTemp(TransformedString);
		}
		FUtf8String& FormatString = Utf8StringTable.Emplace_GetRef(SerializedString);
		OutLogRecord.SetUtf8Format(*FormatString);
	}
	else
	{
		bOk = false;
	}

	FCbObject Object(FCbObject::Clone(It->AsObjectView()));
	OutLogRecord.SetFields(MoveTemp(Object));
	bOk = !It->HasError() && bOk;
	It++;

	if (TUtf8StringBuilder<64> FileStringBuilder; LoadFromCompactBinary(*It++, FileStringBuilder))
	{
		FUtf8String& FileString = Utf8StringTable.AddDefaulted_GetRef();
		FileString = FileStringBuilder.ToString();
		OutLogRecord.SetFile((const ANSICHAR*)*FileString);
	}
	else
	{
		bOk = false;
	}
	if (int32 Line; LoadFromCompactBinary(*It++, Line))
	{
		OutLogRecord.SetLine(Line);
	}
	else
	{
		bOk = false;
	}
	if (FString TextNamespaceString; LoadFromCompactBinary(*It++, TextNamespaceString))
	{
		if (!TextNamespaceString.IsEmpty())
		{
			OutLogRecord.SetTextNamespace(*StringTable.Emplace_GetRef(MoveTemp(TextNamespaceString)));
		}
		else
		{
			OutLogRecord.SetTextNamespace(nullptr);
		}
	}
	else
	{
		bOk = false;
	}
	bool bHasTextKey = false;
	if (FString TextKeyString; LoadFromCompactBinary(*It++, TextKeyString))
	{
		if (!TextKeyString.IsEmpty())
		{
			bHasTextKey = true;
			OutLogRecord.SetTextKey(*StringTable.Emplace_GetRef(MoveTemp(TextKeyString)));
		}
		else
		{
			OutLogRecord.SetTextKey(nullptr);
		}
	}
	else
	{
		bOk = false;
	}

	if (bHasTextKey)
	{
		FLogTemplate* LogTemplate = TemplateTable.Emplace_GetRef(OutLogRecord.GetTextNamespace(), OutLogRecord.GetTextKey(), OutLogRecord.GetUtf8Format()).Get();
		OutLogRecord.SetTemplate(LogTemplate);
	}
	else
	{
		FLogTemplate* LogTemplate = TemplateTable.Emplace_GetRef(OutLogRecord.GetUtf8Format()).Get();
		OutLogRecord.SetTemplate(LogTemplate);
	}

	return bOk;
}

template <UE::CCharType CharType>
void FLogHandler::ReportActiveLog(FReplicatedLogData&& LogData, TStringView<CharType> FormatMessage,
	ELogVerbosity::Type Verbosity)
{
	if (COTFS.CookWorkerClient)
	{
		COTFS.CookWorkerClient->ReportLogMessage(LogData);
	}
	else if (COTFS.CookDirector)
	{
		if (FormatMessage.StartsWith(LITERAL(CharType, "[CookWorker")))
		{
			// Do not store logs from CookWorkers; only the CookWorker saving the package needs to store those logs.
			return;
		}
	}
	RecordLogForIncrementalCook(MoveTemp(LogData), Verbosity);
}

void FLogHandler::RecordLogForIncrementalCook(UE::Cook::FReplicatedLogData&& LogData,
	ELogVerbosity::Type LogVerbosity)
{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	// Note that this function can be called from any thread. Only threadsafe data only can be accessed.
	if (LogVerbosity > ELogVerbosity::Warning)
	{
		// Only warnings and errors are recorded; we don't want to spam display logs and they would waste memory to record.
		return;
	}
	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData
		= PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (!AccumulatedScopeData)
	{
		return;
	}
	FName ActivePackage = AccumulatedScopeData->PackageName;
	if (ActivePackage.IsNone())
	{
		return;
	}

	if (!IsInGameThread())
	{
		// The rest of the function requires access to schedulerthread-only data, so queue it.
		FScopeLock ScopeLock(&QueuedLogsForIncrementalCookLock);
		QueuedLogsForIncrementalCook.Emplace(ActivePackage, MoveTemp(LogData));
	}
	else
	{
		RecordLogForIncrementalCookGameThreadPortion(ActivePackage, MoveTemp(LogData));
	}
#endif // UE_WITH_PACKAGE_ACCESS_TRACKING
}

void FLogHandler::RecordLogForIncrementalCookGameThreadPortion(FName ActivePackage, FReplicatedLogData&& LogData)
{
	if (!COTFS.IsInSession())
	{
		// It's illegal to call GetSessionPlatforms below before the cook session has started.
		// We don't need to record errors before session started for incremental cook, because they come from startup
		// packages and will be replayed on every cook anyway without our intervention.
		return;
	}
	FPackageData* PackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(ActivePackage);
	if (!PackageData)
	{
		return;
	}

	// We want to avoid wasting memory for packages if they have already saved, which we can do because we will not
	// have an opportunity to save the data for them anyway so it causes no change in behavior.
	if (PackageData->HasAllCommittedPlatforms(COTFS.PlatformManager->GetSessionPlatforms()))
	{
		return;
	}

	PackageData->AddLogMessage(MoveTemp(LogData));
}

ILogHandler* CreateLogHandler(UCookOnTheFlyServer& COTFS)
{
	return new FLogHandler(COTFS);
}

} // namespace UE::Cook