// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookLog.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/TVariant.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class UCookOnTheFlyServer;

DECLARE_LOG_CATEGORY_EXTERN(LogCookList, Log, All);

namespace UE::Cook
{

/** Stores the data passed into FOutputDevice::Serialize, for replication to the CookDirector. */
struct FReplicatedLogData
{
	struct FUnstructuredLogData
	{
		FString Message;
		FName Category;
		ELogVerbosity::Type Verbosity;
	};
	TVariant<FUnstructuredLogData, FCbObject> LogDataVariant;
};
FCbWriter& operator<<(FCbWriter& Writer, const FReplicatedLogData& LogData);
bool LoadFromCompactBinary(FCbFieldView Field, FReplicatedLogData& OutLogData);

/**
 * The Cooker's listener to log messages. It passes the logs onto CookWorkerClient for reporting to CookDirector in
 * MPCook, and it stores the log messages on the active package for storage in incremental cooks.
 */
class ILogHandler
{
public:
	virtual ~ILogHandler() = default;

	virtual void ReplayLogsFromIncrementallySkipped(FName PackageName,
		TConstArrayView<FReplicatedLogData> LogMessages) = 0;
	virtual void ReplayLogFromCookWorker(FReplicatedLogData&& LogData, int32 CookWorkerProfileId) = 0;
	virtual void ConditionalPruneReplay() = 0;
	virtual void FlushIncrementalCookLogs() = 0;
};

ILogHandler* CreateLogHandler(UCookOnTheFlyServer& COTFS);

constexpr FStringView HeartbeatCategoryText(TEXTVIEW("CookWorkerHeartbeat:"));

} // namespace UE::Cook
