// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookTypes.h"
#include "CoreTypes.h"
#include "ShaderCompiler.h"

struct FODSCRequestPayload;
class ICookedPackageWriter;
class ITargetPlatform;

namespace UE { namespace Cook
{

using FRecompileShaderCompletedCallback = TFunction<void()>;

using FPrecookedFileList = TMap<FString, FDateTime>;

/**
 * Cook package request.
 */
struct FCookPackageRequest
{
	/** The platform to cook for. */
	FName PlatformName;
	/* Asset filename to cook. */
	FString Filename;
	/** Completion callback. */
	FCompletionCallback CompletionCallback;
};

/**
 * Recompile shader(s) request.
 */
struct FRecompileShaderRequest
{
	/** The arguments to configure shader compilation for this request. */
	FShaderRecompileData RecompileArguments;
	/** Completion callback. */
	FRecompileShaderCompletedCallback CompletionCallback;
};

/**
 * Cook-on-the-fly server interface used by the request manager.
 */
class ICookOnTheFlyServer
{
public:
	virtual ~ICookOnTheFlyServer() = default;

	/** Returns the cooker sandbox directory path. */
	virtual FString GetSandboxDirectory() const = 0;
	
	/** Request cooker set up and be ready to cook PlatformName. */
	virtual const ITargetPlatform* AddPlatform(FName PlatformName, bool& bOutAlreadyInitialized) = 0;

	/** Remove platform from the cook-on-the-fly session. */
	virtual void RemovePlatform(FName PlatformName) = 0;

	/** Report whether the current thread is the cooker's scheduler thread. */
	virtual bool IsSchedulerThread() const = 0;

	/**
	 * Returns all unsolicited files that have been produced as a result of a cook request.
	 *
	 * @param PlatformName The platform name.
	 * @param Filename The filename.
	 * @param Filename Whether the filename is a cookable asset.
	 * @param OutUnsolicitedFiles All unsolicited filename(s) returned to the caller.
	 */
	virtual void GetUnsolicitedFiles(
		const FName& PlatformName,
		const FString& Filename,
		const bool bIsCookable,
		TArray<FString>& OutUnsolicitedFiles) = 0;

	/**
	 * Enqueue a new cook request.
	 *
	 * @param CookPackageRequest Cook package request parameters.
	 */
	virtual bool EnqueueCookRequest(FCookPackageRequest CookPackageRequest) = 0;

	virtual void MarkPackageDirty(const FName& PackageName) = 0;

	/**
	 * Returns the package store writer for the specified platform.
	 */
	virtual ICookedPackageWriter& GetPackageWriter(const ITargetPlatform* TargetPlatform) = 0;
};

/**
 * The cook-on-the-fly request manager.
 *
 * Responsible for managing cook-on-the-fly requests from connected client(s).
 */
class ICookOnTheFlyRequestManager
{
public:
	virtual ~ICookOnTheFlyRequestManager() = default;

	/** Initialze the request manager. */
	virtual bool Initialize() = 0;

	virtual void Tick() = 0;

	/** Shutdown the request manager. */
	virtual void Shutdown() = 0;

	/** Called when cooker has completed setup for a platform and the AssetRegistry is available */
	virtual void OnSessionStarted(FName PlatformName, bool bFirstSessionInThisProcess) {}

	/** Called when a new package is generated */
	virtual void OnPackageGenerated(const FName& PackageName) = 0;

	/** Returns true if the cooker should use package discovery and urgency to schedule requests */
	virtual bool ShouldUseLegacyScheduling() = 0;
};

}} // namespace UE::Cook
