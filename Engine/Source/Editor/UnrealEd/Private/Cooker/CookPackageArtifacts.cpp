// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookPackageArtifacts.h"

#include "Algo/Sort.h"
#include "Cooker/CookArtifact.h"
#include "Algo/Unique.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Cooker/CookAssetRegistryAccessTracker.h"
#include "Cooker/CookConfigAccessTracker.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDependencyContext.h"
#include "Cooker/CookEvents.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookGlobalDependencies.h"
#include "Cooker/ObjectRedirectCookDependency.h"
#include "Cooker/PackageBuildDependencyTracker.h"
#include "CookOnTheSide/CookLog.h"
#include "CookPackageSplitter.h"
#include "DerivedDataBuildDefinition.h"
#include "DerivedDataBuildKey.h"
#include "EditorDomain/EditorDomain.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/Blake3.h"
#include "IO/IoChunkId.h"
#include "IO/PackageId.h"
#include "Misc/Guid.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerLoad.h"

namespace UE::Cook
{

// Bump PackageArtifactsVersion version when the serialization of PackageArtifacts has changed and we want to add
// backwards compatibility rather than invalidating everything.
constexpr uint32 PackageArtifactsVersion = 0x00000004;

static const FUtf8StringView PackageArtifactsAttachmentKey = UTF8TEXTVIEW("meta.cook.artifacts");
static const FUtf8StringView BuildDefinitionsAttachmentKey = UTF8TEXTVIEW("meta.cook.builddefinitions");
static const FUtf8StringView ImportsCheckerAttachmentKey = UTF8TEXTVIEW("meta.cook.importexport");
static const FUtf8StringView LogMessagesAttachmentKey = UTF8TEXTVIEW("meta.cook.logs");

bool FBuildDependencySet::HasKeyMatch(FName PackageName, const ITargetPlatform* TargetPlatform,
	FGenerationHelper* GenerationHelper)
{
	if (!bValid)
	{
		return false;
	}
	if (StoredKey.IsZero())
	{
		return false;
	}
	if (CurrentKey.IsZero())
	{
		ECurrentKeyResult Result = TryCalculateCurrentKey(PackageName, TargetPlatform, GenerationHelper,
			false /* bUpdateValues */);
		if (Result != ECurrentKeyResult::Success)
		{
			CurrentKey.Reset();
			return false;
		}
	}
	return CurrentKey == StoredKey;
}

FBuildDependencySet::ECurrentKeyResult FBuildDependencySet::TryCalculateCurrentKey(FName PackageName,
	const ITargetPlatform* TargetPlatform, FGenerationHelper* GenerationHelper,
	bool bUpdateValues, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (PackageName.IsNone())
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("PackageName is not set."));
		return ECurrentKeyResult::Error;
	}
	if (!AssetRegistry)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("AssetRegistry is unavailable."));
		return ECurrentKeyResult::Error;
	}
	FEditorDomain* EditorDomain = FEditorDomain::Get();
	if (!EditorDomain)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("EditorDomain is unavailable."));
		return ECurrentKeyResult::Error;
	}
	
	FBlake3 KeyBuilder;

	FBlake3Hash GlobalDependencies = GetGlobalDependenciesHash(TargetPlatform);
	KeyBuilder.Update(&GlobalDependencies, sizeof(GlobalDependencies));

	ECurrentKeyResult Result = ECurrentKeyResult::Success;
	FCookDependencyContext Context(&KeyBuilder,
		[&Result, OutMessages](ELogVerbosity::Type Verbosity, FString&& Message, bool bInvalidated)
		{
			if (OutMessages)
			{
				OutMessages->Emplace(Verbosity, MoveTemp(Message));
			}

			if (bInvalidated)
			{
				ECurrentKeyResult NewResult = Verbosity <= ELogVerbosity::Error
					? ECurrentKeyResult::Error : ECurrentKeyResult::Invalidated;
				Result = FMath::Max(Result, NewResult);
			}
		},
		PackageName);

	// The BuildDependencies have already been sorted, and the CookDependency sort
	// function sorts CookDependencies of the same type together. This allows us
	// to create batches for CookDependencies with Update functions that benefit
	// from being updated in batches.
	int32 NumDependencies = Dependencies.Num();
	for (int32 BatchStart = 0; BatchStart < NumDependencies; )
	{
		ECookDependency BatchCategory = Dependencies[BatchStart].GetType();
		int32 BatchEnd = BatchStart + 1;
		while (BatchEnd < NumDependencies && Dependencies[BatchEnd].GetType() == BatchCategory)
		{
			++BatchEnd;
		}
		TArrayView<FCookDependency> Batch(Dependencies.GetData() + BatchStart, BatchEnd - BatchStart);
		BatchStart += Batch.Num();

		// Some CookDependency types can not handle Update being called, because their Update relies on functions
		// only available outside of the CoreUObject module. Handle those types.
		switch (BatchCategory)
		{
		case ECookDependency::Package:
		{
			for (FCookDependency& PackageDependency : Batch)
			{
				FName DependencyPackageName = PackageDependency.GetPackageName();
				UE::EditorDomain::FPackageDigest PackageDigest = EditorDomain->GetPackageDigest(DependencyPackageName);
				if (!PackageDigest.IsSuccessful() && GenerationHelper)
				{
					PackageDigest = GenerationHelper->GetPackageDigest(DependencyPackageName, TargetPlatform);
				}
				if (PackageDigest.IsSuccessful())
				{
					FBlake3Hash ConvertedHash = FCookDependency::ConvertToHash(PackageDigest.Hash);
					if (bUpdateValues)
					{
						PackageDependency.SetValue(ConvertedHash);
					}
					KeyBuilder.Update(&ConvertedHash.GetBytes(), sizeof(ConvertedHash.GetBytes()));
					
					continue;
				}
				else if (PackageDigest.Status == UE::EditorDomain::FPackageDigest::EStatus::DoesNotExistInAssetRegistry)
				{
					FBlake3Hash Hash = FBlake3Hash::Zero;
					if (bUpdateValues)
					{
						PackageDependency.SetValue(Hash);
					}

					KeyBuilder.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));
					continue;
				}

				Context.LogError(FString::Printf(
					TEXT("PackageDependency failed: Could not create PackageDigest for %s: %s"),
					*DependencyPackageName.ToString(), *PackageDigest.GetStatusString()));
			}
			break;
		}

		case ECookDependency::Config:
		{
#if UE_WITH_CONFIG_TRACKING
			using namespace UE::ConfigAccessTracking;
			FCookConfigAccessTracker& ConfigTracker = FCookConfigAccessTracker::Get();
#endif
			for (FCookDependency& ConfigDependency : Batch)
			{
				FString Value;
#if UE_WITH_CONFIG_TRACKING
				Value = ConfigTracker.GetValue(ConfigDependency.GetConfigAccessData());
#endif
				uint8 Marker = 0;
				KeyBuilder.Update(&Marker, sizeof(Marker));
				if (!Value.IsEmpty())
				{
					FBlake3Hash ConvertedHash = FCookDependency::ConvertToHash(FUtf8String(Value));
					if (bUpdateValues)
					{
						ConfigDependency.SetValue(ConvertedHash);
					}
					KeyBuilder.Update(&ConvertedHash.GetBytes(), sizeof(ConvertedHash.GetBytes()));
				}
			}
			break;
		}

		case ECookDependency::NativeClass:
		{
			UE::EditorDomain::TryAppendClassDigests(Batch, KeyBuilder, Context, bUpdateValues);
			break;
		}

		case ECookDependency::RedirectionTarget:
		{
			TArray<FName, TInlineAllocator<10>> PackageNames;
			PackageNames.Reserve(Batch.Num());
			for (const FCookDependency& RedirectionDependency : Batch)
			{
				PackageNames.Add(RedirectionDependency.GetPackageName());
			}

			TArray<FBlake3Hash> Hashes;
			Hashes.SetNum(PackageNames.Num());
			FCoreRedirects::GetHashOfRedirectsAffectingPackages(PackageNames, Hashes);

			for (int Index = 0; Index < Hashes.Num(); ++Index)
			{
				const FBlake3Hash& Hash = Hashes[Index];
				FCookDependency& RedirectionDependency = Batch[Index];

				if (bUpdateValues)
				{
					RedirectionDependency.SetValue(Hash);
				}
				KeyBuilder.Update(&Hash, sizeof(Hash));
			}

			break;
		}

		default:
			for (FCookDependency& BatchDependency : Batch)
			{
				FBlake3Hash Hash;
				BatchDependency.UpdateHash(Context, &Hash);
				if (bUpdateValues)
				{
					BatchDependency.SetValue(Hash);
				}
			}
			break;
		}
	}

	if (Result != ECurrentKeyResult::Error) // -V547
	{
		CurrentKey = KeyBuilder.Finalize();
	}
	return Result;
}

bool FBuildDependencySet::TryGetModifiedDependencies(FName PackageName, const ITargetPlatform* TargetPlatform,
	FGenerationHelper* GenerationHelper, TArray<FString>& OutModifiedDependencies) const
{
	if (!bValid)
	{
		return false;
	}
	if (StoredKey.IsZero())
	{
		return false;
	}
	if (PackageName.IsNone())
	{
		return false;
	}
	FEditorDomain* EditorDomain = FEditorDomain::Get();
	if (!EditorDomain)
	{
		return false;
	}

	for (const FCookDependency& Dependency : Dependencies)
	{
		// Some CookDependency types can not handle Update being called, because their Update relies on functions
		// only available outside of the CoreUObject module. Handle those types.
		switch (Dependency.GetType())
		{
		case ECookDependency::Package:
		{
			FName DependencyPackageName = Dependency.GetPackageName();
			UE::EditorDomain::FPackageDigest PackageDigest = EditorDomain->GetPackageDigest(DependencyPackageName);
			if (!PackageDigest.IsSuccessful() && GenerationHelper)
			{
				PackageDigest = GenerationHelper->GetPackageDigest(DependencyPackageName, TargetPlatform);
			}
			if (!PackageDigest.IsSuccessful())
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: Could not get current PackageDigest"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier()));
			}
			else
			{
				FBlake3Hash CurrentValue = FCookDependency::ConvertToHash(PackageDigest.Hash);
				if (CurrentValue != Dependency.GetHashValue())
				{
					OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: %s -> %s"),
						LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(),
						*LexToString(Dependency.GetHashValue()), *LexToString(CurrentValue)));
				}
			}
			break;
		}

		case ECookDependency::Config:
		{
#if UE_WITH_CONFIG_TRACKING
			using namespace UE::ConfigAccessTracking;
			FCookConfigAccessTracker& ConfigTracker = FCookConfigAccessTracker::Get();
#endif
			FString Value;
#if UE_WITH_CONFIG_TRACKING
			Value = ConfigTracker.GetValue(Dependency.GetConfigAccessData());
#endif
			FBlake3Hash CurrentValue = FCookDependency::ConvertToHash(FUtf8String(Value));
			if (CurrentValue != Dependency.GetHashValue())
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: %s -> %s"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(),
					*LexToString(Dependency.GetHashValue()), *LexToString(CurrentValue)));
			}
			break;
		}

		case ECookDependency::NativeClass:
		{
			FStringView ClassPath = Dependency.GetClassPath();
			TOptional<FBlake3Hash> CurrentValue = UE::EditorDomain::TryGetClassDigest(FTopLevelAssetPath(ClassPath));
			if (!CurrentValue)
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: Could not get current ClassDigest"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier()));
			}
			else if (*CurrentValue != Dependency.GetHashValue())
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: %s -> %s"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(),
					*LexToString(Dependency.GetHashValue()), *LexToString(*CurrentValue)));
			}
			break;
		}

		case ECookDependency::RedirectionTarget:
		{
			FName DependencyPackageName = Dependency.GetPackageName();

			TArray<FName, TInlineAllocator<1>> PackageNames;
			PackageNames.Add(DependencyPackageName);
			TArray<FBlake3Hash> Hashes;
			Hashes.SetNum(PackageNames.Num());
			FCoreRedirects::GetHashOfRedirectsAffectingPackages(PackageNames, Hashes);

			FBlake3Hash& CurrentValue = Hashes[0];
			if (CurrentValue != Dependency.GetHashValue())
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: %s -> %s"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(),
					*LexToString(Dependency.GetHashValue()), *LexToString(CurrentValue)));
			}
			break;
		}

		default:
		{
			FBlake3 KeyBuilder;
			FCookDependencyContext Context(&KeyBuilder,
				[&OutModifiedDependencies, &Dependency](ELogVerbosity::Type Verbosity, FString&& Message, bool bInvalidated)
				{
					if (bInvalidated)
					{
						OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: ReportInvalidated was called : %s"),
							LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(), *Message));
					}
				},
				PackageName);
	
			FBlake3Hash CurrentValue;
			Dependency.UpdateHash(Context, &CurrentValue);
			if (CurrentValue != Dependency.GetHashValue())
			{
				OutModifiedDependencies.Add(FString::Printf(TEXT("%s: %s: %s -> %s"),
					LexToString(Dependency.GetType()), *Dependency.GetDebugIdentifier(),
					*LexToString(Dependency.GetHashValue()), *LexToString(CurrentValue)));
			}
			break;
		}
		}
	}
	return true;
}

void FBuildDependencySet::Empty()
{
	Dependencies.Empty();
	StoredKey = FIoHash::Zero;
	CurrentKey = FIoHash::Zero;
	bValid = false;
}

bool FBuildDependencySet::TryLoad(FCbFieldView InFieldView)
{
	Empty();

	for (FCbFieldViewIterator FieldView(InFieldView.CreateViewIterator()); FieldView; )
	{
		const FCbFieldViewIterator Last = FieldView;
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("Name")))
		{
			if (!LoadFromCompactBinary(FieldView++, Name))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("StoredKey")))
		{
			if (!LoadFromCompactBinary(FieldView++, StoredKey))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("Dependencies")))
		{
			if (!LoadFromCompactBinary(FieldView++, Dependencies))
			{
				return false;
			}
		}
		if (FieldView == Last)
		{
			++FieldView;
		}
	}
	bValid = true;
	return true;
}

void FBuildDependencySet::Save(FCbWriter& Writer) const
{
	Writer.BeginObject();
	Writer << "Name" << Name;
	Writer << "StoredKey" << StoredKey;
	if (!Dependencies.IsEmpty())
	{
		Writer << "Dependencies" << Dependencies;
	}
	Writer.EndObject();
}

FBuildResultDependenciesMap FBuildDependencySet::CollectLoadedPackage(const UPackage* Package,
	TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	FBuildResultDependenciesMap ResultDependencies;
	TArray<FName> UnusedRuntimeDependencies;
	if (!TryCollectInternal(ResultDependencies, UnusedRuntimeDependencies, OutMessages,
		BuildResult::NAME_Load, Package, nullptr /* TargetPlatform */,
		TConstArrayView<FName>() /* UntrackedSoftPackageReferences */, nullptr /* GenerationHelper */,
		false /* bGenerated */))
	{
		return FBuildResultDependenciesMap();
	}

	// Sort and remove duplicates in the results from TryCollectInternal.
	for (TPair<FName, TArray<FCookDependency>>& ResultPair : ResultDependencies)
	{
		Algo::Sort(ResultPair.Value);
		ResultPair.Value.SetNum(Algo::Unique(ResultPair.Value), EAllowShrinking::Yes);
	}
	return ResultDependencies;
}

enum class EPackageMountPoint
{
	Transient,
	Script,
	Content,
	GeneratedContent,
};
static EPackageMountPoint GetPackageMountPoint(FName PackageName)
{
	TStringBuilder<256> StringBuffer;
	PackageName.ToString(StringBuffer);
	if (
		// Some packages get renamed to "TrashedPackage" during blueprint compilation and are no longer valid for
		// saving but might have been dereferenced by TObjectPtr during PostLoad/PreSave. We need to discard these
		// packages, which we can do by requiring a valid package name; all valid packages start with /MountPoint/.
		!StringBuffer.ToView().StartsWith('/') ||
		// Ignore /Memory and /Temp packages
		FPackageName::IsMemoryPackage(StringBuffer) ||
		FPackageName::IsTempPackage(StringBuffer) ||
		FPackageName::IsInEngineTransientPackages(StringBuffer))
	{
		return EPackageMountPoint::Transient;
	}
	if (FPackageName::IsScriptPackage(StringBuffer))
	{
		return EPackageMountPoint::Script;
	}
	if (ICookPackageSplitter::IsUnderGeneratedPackageSubPath(StringBuffer))
	{
		return EPackageMountPoint::GeneratedContent;
	}
	return EPackageMountPoint::Content;
}

static void JoinMessagesIntoErrorReason(FStringBuilderBase& OutText,
	TArray<TPair<ELogVerbosity::Type, FString>>& Messages)
{
	if (Messages.IsEmpty())
	{
		OutText << TEXT(".");
	}
	else
	{
		OutText << TEXT(":");
		for (TPair<ELogVerbosity::Type, FString>& MessagePair : Messages)
		{
			OutText << TEXT("\n\t") << MessagePair.Value;
		}
	}
}

/**
 * Go through all the imports of a package using the LinkerLoad and create an object redirector dependency for each import.
 * 
 * Here is an example of why we need this : Plugin 1 from DLC1 has package A, plugin 2 from DLC2 has package B. Package A contains an object 
 * redirector A.A -> B.B. Package C from DLC2 has an import (a pointer) to B.B.
 * When doing a multiproduct cook of package C, plugin 1 is not loaded because it is not needed for DLC2. The dependencies of package C will
 * contain a redirector dependency for package B. This dependency will track any redirector where B is a source or target of a redirection. In this
 * case there is no redirector so the hash will be equivalent to 0.
 * Now if I run a regular cook, I have all the plugins loaded. So when I cook package C, the redirector dependency for package B will find the object
 * redirect package A and use it to compute the hash. The dependencies are now different from the multiproduct cook.
 * 
 */
static void CollectObjectRedirectDependencies(const UPackage* Package, TArray<FCookDependency>& Dependencies)
{
	const FLinkerLoad* LinkerLoad = Package->GetLinker();
	if (!LinkerLoad)
	{
		return;
	}
	
	TSet<FName> UniqueImportNames;
	for(int ArrayIndex = 0; ArrayIndex < LinkerLoad->ImportMap.Num(); ++ArrayIndex)
	{
		FPackageIndex Index = FPackageIndex::FromImport(ArrayIndex);

		FString PathName;
		LinkerLoad->BuildPathName(PathName, Index);

		FSoftObjectPath ObjectPath(PathName);
		FName ImportPackageName = ObjectPath.GetLongPackageFName();
		
		if (ImportPackageName.IsNone())
		{
			continue;
		}

		bool bAlreadyInSet = false;
		UniqueImportNames.Add(ImportPackageName, &bAlreadyInSet);

		if (bAlreadyInSet)
		{
			continue;
		}

		EPackageMountPoint MountPoint = GetPackageMountPoint(ImportPackageName);
		switch (MountPoint)
		{

		case EPackageMountPoint::Content:
			Dependencies.Add(UE::Cook::CreateObjectRedirectDependency(ImportPackageName));
			break;


		case EPackageMountPoint::Transient:
		case EPackageMountPoint::Script:
		case EPackageMountPoint::GeneratedContent:
		default:
			break;
		}
	}
}

bool FBuildDependencySet::TryCollectInternal(FBuildResultDependenciesMap& InOutResultDependencies,
	TArray<FName>& InOutRuntimeDependencies, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages,
	FName DefaultBuildResult, const UPackage* Package, const ITargetPlatform* TargetPlatform,
	TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper, bool bGenerated)
{
	if (!Package)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("Invalid null package."));
		return false;
	}
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (!AssetRegistry)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("AssetRegistry is unavailable."));
		return false;
	}
	FEditorDomain* EditorDomain = FEditorDomain::Get();
	if (!EditorDomain)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("EditorDomain is unavailable."));
		return false;
	}

	TArray<FCookDependency> DefaultResultDependencies;
	// Skip the multiple reallocations for an array that grows from 0 to 128, for performance, but then
	// reallocate according to normal TArray growth to reduce spike memory use.
	DefaultResultDependencies.Empty(128);

	FName PackageName = Package->GetFName();
	DefaultResultDependencies.Add(FCookDependency::Package(PackageName));

#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FPackageBuildDependencyTracker& Tracker = FPackageBuildDependencyTracker::Get();

	if (Tracker.IsEnabled())
	{
		TArray<TPair<FBuildDependencyAccessData, FResultProjectionList>> AccessDatas
			= Tracker.GetAccessDatas(PackageName);

		for (TPair<FBuildDependencyAccessData, FResultProjectionList>& Pair : AccessDatas)
		{
			FBuildDependencyAccessData& AccessData = Pair.Key;
			if (AccessData.TargetPlatform == TargetPlatform || AccessData.TargetPlatform == nullptr)
			{
				FResultProjectionList& ProjectionList = Pair.Value;
				constexpr bool bAutoTransitiveDependenciesEnabled = false;
				if (!bAutoTransitiveDependenciesEnabled)
				{
					// We have not yet enabled marking the auto-added dependencies from TObjectPtr resolve
					// as transitive, because it causes a performance regression and we are still working on fixing the
					// regression.
					DefaultResultDependencies.Add(FCookDependency::Package(AccessData.ReferencedPackage));
				}
				else if (ProjectionList.bHasAll)
				{
					// TObjectPtr UE::BuildProjection::All dependencies are added as transitive build dependencies.
					// We have to do this to be conservative, since we do not know which bytes from the target are
					// dependended upon and which of the target's build dependencies influence those bytes.
					DefaultResultDependencies.Add(FCookDependency::TransitiveBuild(AccessData.ReferencedPackage));
				}
				else
				{
					for (FTopLevelAssetPath ClassPath : ProjectionList.Classes)
					{
						DefaultResultDependencies.Add(FCookDependency::NativeClass(WriteToString<256>(ClassPath)));
					}

					for (FName ResultProjection : ProjectionList.ResultProjections)
					{
						if (ResultProjection == UE::Cook::ResultProjection::PackageAndClass)
						{
							DefaultResultDependencies.Add(FCookDependency::Package(AccessData.ReferencedPackage));
						}
						else
						{
							if (OutMessages)
							{
								OutMessages->Emplace(ELogVerbosity::Error, FString::Printf(
									TEXT("When saving %s, found ResultProjection %s, which is a system-specific ResultProjection, and this is not yet implemented. ")
									TEXT("Find the call to UE_COOK_RESULTPROJECTION_SCOPED passing in this name and remove it."),
									*PackageName.ToString(), *ResultProjection.ToString()));
							}
							return false;
						}
					}
				}
			}
		}
	}
	else
#endif
	{
		// When PackageAccessTracking is disabled, defensively treat all asset dependencies as transitive build
		// dependencies.
		TArray<FName> AssetDependencies;
		if (!bGenerated)
		{
			AssetRegistry->GetDependencies(PackageName, AssetDependencies,
				UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Game);
			for (FName AssetDependency : AssetDependencies)
			{
				DefaultResultDependencies.Add(FCookDependency::TransitiveBuild(AssetDependency));
			}
		}
	}

	// For dependencies which don't exist on disk, add a package dependency. This will allow us to detect the 
	// need to recook if that dependency is later recreated on disk. 
	TArray<FName> AssetDependencies;
	AssetRegistry->GetDependencies(PackageName, AssetDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
	AssetRegistry->GetDependencies(PackageName, AssetDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Soft);

	for (FName AssetDependency : AssetDependencies)
	{
		FString DependencyName = AssetDependency.ToString();
		if (!FPackageName::IsScriptPackage(DependencyName) && !AssetRegistry->DoesPackageExistOnDisk(AssetDependency))
		{
			DefaultResultDependencies.Add(FCookDependency::Package(AssetDependency));
		}
	}

#if UE_WITH_CONFIG_TRACKING
	{
		using namespace UE::ConfigAccessTracking;
		FCookConfigAccessTracker& ConfigTracker = FCookConfigAccessTracker::Get();
		if (ConfigTracker.IsEnabled())
		{
			TArray<FConfigAccessData> ConfigKeys = ConfigTracker.GetPackageRecords(PackageName, TargetPlatform);
			for (const FConfigAccessData& ConfigKey : ConfigKeys)
			{
				DefaultResultDependencies.Add(FCookDependency::Config(ConfigKey));
			}
		}
	}
#endif

	{
		UE::CookAssetRegistryAccessTracker::FCookAssetRegistryAccessTracker& AssetRegistryTracker = UE::CookAssetRegistryAccessTracker::FCookAssetRegistryAccessTracker::Get();
		TArray<UE::CookAssetRegistryAccessTracker::FRecord> Records = AssetRegistryTracker.GetRecords(PackageName, TargetPlatform);
		for (const UE::CookAssetRegistryAccessTracker::FRecord& Record : Records)
		{
			DefaultResultDependencies.Add(FCookDependency::AssetRegistryQuery(Record.Filter));

			if (Record.bAddPackageDependencies)
			{
				// Add as package dependency each package from the result of the query.
				TSet<FName> ResultPackageNames;
				AssetRegistry->EnumerateAssets(Record.Filter, [&ResultPackageNames](const FAssetData& AssetData)
					{
						ResultPackageNames.Add(AssetData.PackageName);
						return true;
					}, UE::AssetRegistry::EEnumerateAssetsFlags::None);

				for (const FName& ResultPackageName : ResultPackageNames)
				{
					DefaultResultDependencies.Add(FCookDependency::Package(ResultPackageName));
				}
			}
		}
	}

	if (!UntrackedSoftPackageReferences.IsEmpty())
	{
		TArray<FCookDependency>& SaveDependencies = InOutResultDependencies.FindOrAdd(BuildResult::NAME_Save);
		for (FName SoftPackageReference : UntrackedSoftPackageReferences)
		{
			SaveDependencies.Add(FCookDependency::RedirectionTarget(SoftPackageReference));
		}
	}

	CollectObjectRedirectDependencies(Package, DefaultResultDependencies);

	// Put the Dependencies we have collected onto the requested DefaultBuildResult
	InOutResultDependencies.FindOrAdd(DefaultBuildResult).Append(MoveTemp(DefaultResultDependencies));

	// If we have any RuntimeDependencies, they will cause some BuildDependencies in the SaveBuildResult, so add a SaveBuildResult
	// output if we don't already have one.
	if (!InOutRuntimeDependencies.IsEmpty())
	{
		InOutResultDependencies.FindOrAdd(BuildResult::NAME_Save);
	}

	// All Dependencies have been gathered. Format the lists for TryCalculateCurrentKey and for storage.

	for (TPair<FName, TArray<FCookDependency>>& ResultPair : InOutResultDependencies)
	{
		TArray<FCookDependency>& ResultDependencies = ResultPair.Value;
		// Settings dependencies - Expand transitive dependencies on SettingsObjects into the list of dependencies
		// recorded for that settings object.
		TSet<const UObject*> SettingsDependencies;
		ResultDependencies.RemoveAllSwap([&SettingsDependencies](const FCookDependency& Dependency)
			{
				if (Dependency.GetType() == ECookDependency::SettingsObject)
				{
					SettingsDependencies.Add(Dependency.GetSettingsObject());
					return true;
				}
				return false;
			},
			EAllowShrinking::No);
		for (const UObject* SettingsObject : SettingsDependencies)
		{
			// We rely on the object to be rooted because we use its pointer as a key for the lifetime of
			// the cook process, so it being garbage collected and something else allocated on the same
			// pointer would break our key. IsRooted should have been validated by
			// FCookDependency::SettingsObject.
			check(SettingsObject->IsRooted());
			FCookDependencyGroups::FRecordedDependencies& IncludeDependencies =
				FCookDependencyGroups::Get().FindOrCreate(reinterpret_cast<UPTRINT>(SettingsObject));
			if (!IncludeDependencies.bInitialized)
			{
				IncludeDependencies.Dependencies = CollectSettingsObject(SettingsObject,
					&IncludeDependencies.Messages);
				IncludeDependencies.bInitialized = true;
			}
			if (!IncludeDependencies.Dependencies.IsValid())
			{
				if (OutMessages)
				{
					TStringBuilder<256> ErrorText;
					ErrorText << TEXT("Dependencies for SettingsObject ") << SettingsObject->GetPathName()
						<< TEXT(" are unavailable");
					JoinMessagesIntoErrorReason(ErrorText, IncludeDependencies.Messages);
					OutMessages->Emplace(ELogVerbosity::Error, FString(*ErrorText));
				}
				return false;
			}

			for (const FCookDependency& IncludeDependency : IncludeDependencies.Dependencies.GetDependencies())
			{
				// Recursive SettingsDependencies are not allowed. We haven't needed them yet, and not supporting them allows
				// prevents the need for cycle detection.
				if (IncludeDependency.GetType() == ECookDependency::SettingsObject)
				{
					if (OutMessages)
					{
						OutMessages->Emplace(ELogVerbosity::Error, FString::Printf(
							TEXT("Settings dependency on object %s, but that object has a recursive Settings dependency on %s, and recursive Settings dependencies are not supported."),
							*SettingsObject->GetPathName(), *IncludeDependency.GetSettingsObject()->GetPathName()));
					}
					return false;
				}
				ResultDependencies.Add(IncludeDependency);
			}
		}

		// Process some rules for Package dependencies
		TSet<FName> RedirectionTargets;
		for (TArray<FCookDependency>::TIterator Iter(ResultDependencies); Iter; ++Iter)
		{
			FCookDependency& Dependency = *Iter;
			if (Dependency.GetType() == ECookDependency::TransitiveBuild ||
				Dependency.GetType() == ECookDependency::Package)
			{
				FName DependencyName = Dependency.GetPackageName();
				// Remove transitive dependencies to self, for performance. But keep the package dependency to
				// self; every cooked package has its EditorDomain package as a dependency.
				if (Dependency.GetType() == ECookDependency::TransitiveBuild && DependencyName == PackageName)
				{
					Iter.RemoveCurrentSwap();
					continue;
				}

				// We do not hash dependencies to non-content packages (e.g. temp, memory, script),
				// so remove package or transitive package dependencies to them.
				EPackageMountPoint MountPoint = GetPackageMountPoint(DependencyName);
				if (MountPoint != EPackageMountPoint::Content && MountPoint != EPackageMountPoint::GeneratedContent)
				{
					Iter.RemoveCurrentSwap();
					continue;
				}

				// Remove dependencies to generated packages, except for a generated package's dependency to itself.
				// We do not yet support the availability of the digest of other generated packages when requested from
				// the savepackage and dependency collection of a generated package or a generator; 
				// the digests only become available when the target generated package is saved, which can happen after
				// the save of the packages that refer to it.
				if (MountPoint == EPackageMountPoint::GeneratedContent && DependencyName != PackageName)
				{
					Iter.RemoveCurrentSwap();
					continue;
				}

				bool bPackageExistOnDisk = AssetRegistry->DoesPackageExistOnDisk(DependencyName);
				if (!bPackageExistOnDisk)
				{
					TStringBuilder<256> DependencyNameStr(InPlace, DependencyName);
					UPackage* DependencyPackage = FindPackage(nullptr, *DependencyNameStr);

					if (DependencyPackage && DependencyPackage->HasAnyPackageFlags(PKG_NewlyCreated))
					{
						// If the package is a newly created package (in-memory package) then ignore it.
						// In-memory packages are ignored because we can't compute their digest. Only packages on disk have a digest.
						Iter.RemoveCurrentSwap();
						continue;
					}
					// else the package is not on disk, in-memory and not newly created. This can happen for packages
					// with soft dependencies which have been deleted from disk - register it to the dependencies
					// so that we can later detect if it's been re-added and recook packages that depend on it.
				}

				// Package dependencies of all kinds (Runtime, Build, TransitiveBuild) also cause RedirectionTarget
				// dependencies.
				RedirectionTargets.Add(DependencyName);

				// Deprecated TransitiveBuildAndRuntime dependencies can also cause runtime dependencies;
				// convert them to separate dependencies now.
				if (Dependency.GetType() == ECookDependency::TransitiveBuild && ResultPair.Key == BuildResult::NAME_Save)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS;
					if (Dependency.IsAlsoAddRuntimeDependency())
					{
						InOutRuntimeDependencies.Add(DependencyName);
						// Remove the IsAlsoRuntimeDependency flag
						Dependency = FCookDependency::TransitiveBuild(DependencyName);
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS;
				}
			}
		}

		if (ResultPair.Key == BuildResult::NAME_Save)
		{
			// Pull transient packages out of the runtime dependencies for performance; we don't need them for
			// deciding what gets cooked. 
			// Runtime dependencies also cause RedirectionTarget dependencies, so record those.
			for (TArray<FName>::TIterator Iter(InOutRuntimeDependencies); Iter; ++Iter)
			{
				FName DependencyPackageName = *Iter;
				EPackageMountPoint MountPoint = GetPackageMountPoint(DependencyPackageName);
				switch (MountPoint)
				{
				case EPackageMountPoint::GeneratedContent:
				case EPackageMountPoint::Content:
					// Keep it
					RedirectionTargets.Add(DependencyPackageName);
					break;

				// Do not add redirection dependencies for native package. Those dependencies are too wide and already
				// included in schema hash or package digest.
				case EPackageMountPoint::Script:
					break;

				default:
					Iter.RemoveCurrentSwap();
					break;
				}
			}
		}

		// Put all the extra redirection dependencies into BuildDependencies
		for (FName RedirectionTarget : RedirectionTargets)
		{
			ResultDependencies.Add(FCookDependency::RedirectionTarget(RedirectionTarget));
			ResultDependencies.Add(UE::Cook::CreateObjectRedirectDependency(RedirectionTarget));
		}
	}
	return true;
}

FBuildDependencySet FBuildDependencySet::CollectSettingsObject(const UObject* Object,
	TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	if (!Object)
	{
		if (OutMessages)
		{
			OutMessages->Emplace(ELogVerbosity::Error, TEXT("Invalid null Object."));
		}
		return FBuildDependencySet();
	}

	UClass* Class = Object->GetClass();
	if (!Class->HasAnyClassFlags(CLASS_Config | CLASS_PerObjectConfig))
	{
		if (OutMessages)
		{
			OutMessages->Emplace(ELogVerbosity::Error, FString::Printf(
				TEXT("Class %s is not a config class."), *Class->GetPathName()));
		}
		return FBuildDependencySet();
	}
	if (!Class->HasAnyClassFlags(CLASS_PerObjectConfig) && Object != Class->GetDefaultObject())
	{
		if (OutMessages)
		{
			OutMessages->Emplace(ELogVerbosity::Error, FString::Printf(
				TEXT("Class %s is not a per-object-config class."), *Class->GetPathName()));
		}
		return FBuildDependencySet();
	}

	TArray<FCookDependency> BuildDependencies;
	TArray<UE::ConfigAccessTracking::FConfigAccessData> ConfigDatas;
	const_cast<UObject*>(Object)->LoadConfig(nullptr /* ConfigClass */, nullptr /* Filename */,
		UE::LCPF_None /* PropagationFlags */, nullptr /* PropertyToLoad */, &ConfigDatas);
	BuildDependencies.Reserve(ConfigDatas.Num() + 1);
	for (const UE::ConfigAccessTracking::FConfigAccessData& ConfigData : ConfigDatas)
	{
		BuildDependencies.Add(UE::Cook::FCookDependency::Config(ConfigData));
	}

	// In addition to adding the config dependencies, add a dependency on the class schema. If the current class has
	// config fields A,B,C, we add dependencies on those config values. But if the class header is modified to have
	// additional config field D then we need to rebuild packages that depend on it to record the new dependency on D.
	UClass* NativeClass = Class;
	while (NativeClass && !NativeClass->IsNative())
	{
		NativeClass = NativeClass->GetSuperClass();
	}
	if (NativeClass)
	{
		BuildDependencies.Add(UE::Cook::FCookDependency::NativeClass(NativeClass));
	}

	Algo::Sort(BuildDependencies);
	BuildDependencies.SetNum(Algo::Unique(BuildDependencies));
	FBuildDependencySet Result;
	Result.SetNormalizedDependencies(MoveTemp(BuildDependencies));
	Result.SetValid(true);
	return Result;
}

FPackageArtifacts::FPackageArtifacts()
{
	LoadBuildDependencies.SetName(BuildResult::NAME_Load);
	SaveBuildDependencies.SetName(BuildResult::NAME_Save);
}

FBuildDependencySet& FPackageArtifacts::FindOrAddBuildDependencySet(FName ResultName)
{
	if (ResultName == BuildResult::NAME_Save)
	{
		return SaveBuildDependencies;
	}
	else if (ResultName == BuildResult::NAME_Load)
	{
		return LoadBuildDependencies;
	}
	else
	{
		// Not yet implemented
		check(false);
		return SaveBuildDependencies;
	}
}

FBuildDependencySet* FPackageArtifacts::FindBuildDependencySet(FName ResultName)
{
	if (ResultName == BuildResult::NAME_Save)
	{
		return &SaveBuildDependencies;
	}
	else if (ResultName == BuildResult::NAME_Load)
	{
		return &LoadBuildDependencies;
	}
	else
	{
		return nullptr;
	}
}

bool FPackageArtifacts::HasSaveResults() const
{
	return bHasSaveResults;
}

bool FPackageArtifacts::HasKeyMatch(const ITargetPlatform* TargetPlatform, FGenerationHelper* GenerationHelper)
{
	return bValid && SaveBuildDependencies.HasKeyMatch(PackageName, TargetPlatform, GenerationHelper);
}

bool FPackageArtifacts::TryGetModifiedDependencies(const ITargetPlatform* TargetPlatform,
	FGenerationHelper* GenerationHelper, TArray<FString>& OutModifiedDependencies) const
{
	if (!bValid)
	{
		return false;
	}
	return SaveBuildDependencies.TryGetModifiedDependencies(PackageName, TargetPlatform, GenerationHelper,
		OutModifiedDependencies);
}

TOptional<bool> FPackageArtifacts::GetIsPackageModified() const
{
	if (!bValid)
	{
		return TOptional<bool>();
	}
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (!AssetRegistry)
	{
		return TOptional<bool>();
	}

	TOptional<FAssetPackageData> CurrentValue = AssetRegistry->GetAssetPackageDataCopy(PackageName);
	if (!CurrentValue)
	{
		return TOptional<bool>();
	}
	return TOptional<bool>(CurrentValue->GetPackageSavedHash() != StoredPackageSavedHash);
}

FBuildDependencySet::ECurrentKeyResult FPackageArtifacts::TryCalculateCurrentKey(const ITargetPlatform* TargetPlatform,
	FGenerationHelper* GenerationHelper, bool bUpdateValues, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	return SaveBuildDependencies.TryCalculateCurrentKey(PackageName, TargetPlatform, GenerationHelper, bUpdateValues,
		OutMessages);
}

void FPackageArtifacts::Empty()
{
	SaveBuildDependencies.Empty();
	LoadBuildDependencies.Empty();
	RuntimeDependencies.Empty();
	PackageName = FName();
	bHasSaveResults = false;
	bValid = false;
}

}

bool LoadFromCompactBinary(FCbObjectView ObjectView, UE::Cook::FPackageArtifacts& Artifacts)
{
	using namespace UE::Cook;

	Artifacts.Empty();
	int32 Version = -1;

	for (FCbFieldViewIterator FieldView(ObjectView.CreateViewIterator()); FieldView; )
	{
		const FCbFieldViewIterator Last = FieldView;
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("Version")))
		{
			Version = FieldView.AsInt32();
			if ((FieldView++).HasError() || Version != PackageArtifactsVersion)
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("HasSaveResults")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.bHasSaveResults))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("SaveBuildDependencies")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.SaveBuildDependencies))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("LoadBuildDependencies")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.LoadBuildDependencies))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("RuntimeDependencies")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.RuntimeDependencies))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("PackageSavedHash")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.StoredPackageSavedHash))
			{
				return false;
			}
		}
		if (FieldView.GetName().Equals(UTF8TEXTVIEW("IoStoreChunkId")))
		{
			if (!LoadFromCompactBinary(FieldView++, Artifacts.IoStoreChunkId))
			{
				return false;
			}
		}
		if (FieldView == Last)
		{
			++FieldView;
		}
	}
	if (Version == -1)
	{
		return false;
	}
	Artifacts.bValid = true;
	return true;
}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FPackageArtifacts& Artifacts)
{
	using namespace UE::Cook;

	Writer.BeginObject();
	Writer << "Version" << PackageArtifactsVersion;
	Writer << "HasSaveResults" << Artifacts.bHasSaveResults;
	if (Artifacts.SaveBuildDependencies.IsValid())
	{
		Writer << "SaveBuildDependencies" << Artifacts.SaveBuildDependencies;
	}
	if (Artifacts.LoadBuildDependencies.IsValid())
	{
		Writer << "LoadBuildDependencies" << Artifacts.LoadBuildDependencies;
	}
	if (!Artifacts.RuntimeDependencies.IsEmpty())
	{
		Writer << "RuntimeDependencies" << Artifacts.RuntimeDependencies;
	}
	Writer << "PackageSavedHash" << Artifacts.StoredPackageSavedHash;
	Writer << "IoStoreChunkId" << Artifacts.IoStoreChunkId;

	Writer.EndObject();
	return Writer;
}

namespace UE::Cook
{

FCookDependencyGroups& FCookDependencyGroups::Get()
{
	static FCookDependencyGroups Singleton;
	return Singleton;
}

FCookDependencyGroups::FRecordedDependencies& FCookDependencyGroups::FindOrCreate(UPTRINT Key)
{
	return Groups.FindOrAdd(Key);
}

FBuildDefinitionList FBuildDefinitionList::Collect(const UPackage* Package, const ITargetPlatform* TargetPlatform,
	TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	using namespace UE::DerivedData;

	FBuildDefinitionList Result;

	// TODO_BuildDefinitionList: Calculate and store BuildDefinitionList on the PackageData, or collect it here from some other source.
	if (Result.Definitions.IsEmpty())
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("Not yet implemented"));
		return FBuildDefinitionList();
	}

	TArray<FBuildDefinition>& Defs = Result.Definitions;
	Algo::Sort(Defs, [](const FBuildDefinition& A, const FBuildDefinition& B)
		{
			return A.GetKey().Hash < B.GetKey().Hash;
		});

	return Result;
}

void FBuildDefinitionList::Empty()
{
	Definitions.Empty();
}

}

bool LoadFromCompactBinary(FCbObject&& Object, UE::Cook::FBuildDefinitionList& Definitions)
{
	using namespace UE::DerivedData;

	FCbField DefinitionsField = Object["BuildDefinitions"];
	FCbArray DefinitionsArrayField = DefinitionsField.AsArray();
	if (DefinitionsField.HasError())
	{
		return false;
	}
	TArray<FBuildDefinition>& Defs = Definitions.Definitions;
	Defs.Empty(DefinitionsArrayField.Num());
	for (FCbField& BuildDefinitionObj : DefinitionsArrayField)
	{
		FOptionalBuildDefinition BuildDefinition = FBuildDefinition::Load(TEXTVIEW("TargetDomainBuildDefinitionList"),
			BuildDefinitionObj.AsObject());
		if (!BuildDefinition)
		{
			Defs.Empty();
			return false;
		}
		Defs.Add(MoveTemp(BuildDefinition).Get());
	}

	return true;
}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FBuildDefinitionList& Definitions)
{
	using namespace UE::DerivedData;

	Writer.BeginObject();
	Writer.BeginArray("BuildDefinitions");
	for (const FBuildDefinition& BuildDefinition : Definitions.Definitions)
	{
		BuildDefinition.Save(Writer);
	}
	Writer.EndArray();
	return Writer;
}

namespace UE::Cook
{

/** Wrapper around TArray<FLogReplicationData> so we can serialize is as FCbObject instead of FCbArray. */
struct FLogMessagesArray
{
public:
	TArray<FReplicatedLogData>& Array;

public:
	FLogMessagesArray(TArray<FReplicatedLogData>& InLogMessages)
		: Array(InLogMessages)
	{
	}

private:
	friend bool LoadFromCompactBinary(FCbFieldView FieldView, UE::Cook::FLogMessagesArray& LogMessages)
	{
		return LoadFromCompactBinary(FieldView["Logs"], LogMessages.Array);
	}
	friend FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FLogMessagesArray& LogMessages)
	{
		Writer.BeginObject();
		Writer << "Logs" << const_cast<FLogMessagesArray&>(LogMessages).Array;
		Writer.EndObject();
		return Writer;
	}
};

void FIncrementalCookAttachments::Empty()
{
	Artifacts.Empty();
	BuildDefinitions.Empty();
}

template <typename WritableType>
static void AddAttachment(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context,
	WritableType&& Writable, FUtf8StringView AttachmentKey)
{
	FCbWriter Writer;
	Writer << Writable;
	Context.SetAttachment(FName(AttachmentKey), Writer.Save().AsObject(), Artifact::Attachment);
}

void FIncrementalCookAttachments::AppendCommitAttachments(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context)
{
	if (Artifacts.IsValid())
	{
		AddAttachment(Context, Artifacts, PackageArtifactsAttachmentKey);
	}
	if (!BuildDefinitions.Definitions.IsEmpty())
	{
		AddAttachment(Context, BuildDefinitions, BuildDefinitionsAttachmentKey);
	}
	if (!ImportsCheckerData.IsEmpty())
	{
		AddAttachment(Context, ImportsCheckerData, ImportsCheckerAttachmentKey);
	}
	if (!LogMessages.IsEmpty())
	{
		AddAttachment(Context, FLogMessagesArray(LogMessages), LogMessagesAttachmentKey);
	}
}

FIncrementalCookAttachments FIncrementalCookAttachments::Collect(const UPackage* Package,
	const ITargetPlatform* TargetPlatform, FBuildResultDependenciesMap&& InResultDependencies,
	bool bHasSaveResult, TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper,
	bool bGenerated, TArray<FName>&& RuntimeDependencies,
	TConstArrayView<UObject*> Imports, TConstArrayView<UObject*> Exports,
	TConstArrayView<UE::SavePackageUtilities::FPreloadDependency> PreloadDependencies,
	TConstArrayView<FReplicatedLogData> InLogMessages)
{
	FIncrementalCookAttachments Result;

	Result.CommitStatus = IPackageWriter::ECommitStatus::NotCommitted;

	TArray<TPair<ELogVerbosity::Type, FString>> Messages;
	Result.Artifacts = FPackageArtifacts::Collect(Package, TargetPlatform, MoveTemp(InResultDependencies),
		bHasSaveResult, UntrackedSoftPackageReferences, GenerationHelper, bGenerated,
		MoveTemp(RuntimeDependencies), &Messages);
	if (!Result.Artifacts.IsValid())
	{
		TStringBuilder<256> LogText;
		LogText << TEXT("Could not collect PackageArtifacts for package '") << Package->GetFName() << TEXT("'");
		JoinMessagesIntoErrorReason(LogText, Messages);

		// INCREMENTALCOOK_TODO: This error occurs due to dependencies on _Verse.
		// Raise Verbosity to Error once that is fixed.
		UE_LOGF(LogCook, Verbose, "%ls", *LogText);
	}

	Result.BuildDefinitions = FBuildDefinitionList::Collect(Package, TargetPlatform);
	Result.ImportsCheckerData = UE::Cook::FImportsCheckerData::FromObjectLists(Imports, Exports);
	Result.LogMessages = InLogMessages;

	return Result;
}

void FIncrementalCookAttachments::Fetch(TArrayView<FPackageIncrementalCookId> PackageIds,
	const ITargetPlatform* TargetPlatform, ICookedPackageWriter* PackageWriter,
	TFunction<void(FName PackageName, FIncrementalCookAttachments&& Result)>&& Callback)
{
	using namespace UE::TargetDomain;

	if (!TargetPlatform && !GEditorDomainOplog)
	{
		for (const FPackageIncrementalCookId& Id : PackageIds)
		{
			Callback(Id.PackageName, FIncrementalCookAttachments());
		}
		return;
	}

	struct FInProgressResult
	{
		FIncrementalCookAttachments Result;
		EWhereCooked WhereCooked = EWhereCooked::ThisSession;
		int ReceivedAttachmentCount = 0;
	};
	TMap<FName, FInProgressResult> InProgressResults;
	InProgressResults.Reserve(PackageIds.Num());

	TArray<FName> SessionOplogNames;
	TArray<FName> BaseGameOplogNames;
	TArray<FName> EditorDomainNames;
	if (TargetPlatform)
	{
		check(PackageWriter);
		for (const FPackageIncrementalCookId& Id : PackageIds)
		{
			InProgressResults.FindOrAdd(Id.PackageName).WhereCooked = Id.WhereCooked;
			switch (Id.WhereCooked)
			{
			case EWhereCooked::ThisSession:
				SessionOplogNames.Add(Id.PackageName);
				break;
			case EWhereCooked::BaseGame:
				BaseGameOplogNames.Add(Id.PackageName);
				break;
			case EWhereCooked::ExtraReleaseVersionAssets:
				UE_CALL_ONCE([&]
					{
						UE_LOGF(LogCook, Warning,
							"CookDependency fetch was requested for package %ls with EWhereCooked::ExtraReleaseVersionAssets, which is not handled. "
							"TransitiveBuildDepencies will be missing and false positive incremental skips could result.",
							*Id.PackageName.ToString());
					});
				Callback(Id.PackageName, FIncrementalCookAttachments());
				break;
			default:
				UE_CALL_ONCE([&]
					{
						UE_LOGF(LogCook, Warning,
							"CookDependency fetch was requested for package %ls with EWhereCooked value %d, which is not handled. "
							"TransitiveBuildDepencies will be missing and false positive incremental skips could result.",
							*Id.PackageName.ToString(), EnumToUnderlyingType(Id.WhereCooked));
					});
				Callback(Id.PackageName, FIncrementalCookAttachments());
				break;
			}
		}
	}
	else
	{
		for (const FPackageIncrementalCookId& Id : PackageIds)
		{
			// EditorDomainNames are called with AttachmentsForIncrementalSkipAndBuildDependencies
			check(Id.WhereCooked == EWhereCooked::ThisSession);
			InProgressResults.FindOrAdd(Id.PackageName).WhereCooked = EWhereCooked::ThisSession;
			EditorDomainNames.Add(Id.PackageName);
		}
	}

	FUtf8StringView AttachmentsForIncrementalSkipAndBuildDependenciesArray[] =
	{ PackageArtifactsAttachmentKey, ImportsCheckerAttachmentKey, BuildDefinitionsAttachmentKey,
		LogMessagesAttachmentKey };
	TArrayView<FUtf8StringView> AttachmentsForIncrementalSkipAndBuildDependencies(
		AttachmentsForIncrementalSkipAndBuildDependenciesArray);
	FUtf8StringView AttachmentsForBuildDependenciesArray[] =
	{ PackageArtifactsAttachmentKey, ImportsCheckerAttachmentKey };
	TArrayView<FUtf8StringView> AttachmentsForBuildDependencies(AttachmentsForBuildDependenciesArray);

	TFunction<void(FName , FUtf8StringView, FCbObject&&)> OnOplogAttachment =
		[PackageWriter, Callback = MoveTemp(Callback), AttachmentsForIncrementalSkipAndBuildDependenciesNum = AttachmentsForIncrementalSkipAndBuildDependencies.Num(),
		AttachmentsForBuildDependenciesNum = AttachmentsForBuildDependencies.Num(),
		InProgressResults = MoveTemp(InProgressResults)]
		(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment) mutable
		{
			FInProgressResult& InProgressResult = InProgressResults.FindOrAdd(PackageName);
			InProgressResult.ReceivedAttachmentCount++;
			if (AttachmentKey == PackageArtifactsAttachmentKey)
			{
				if (PackageWriter && InProgressResult.WhereCooked == EWhereCooked::ThisSession)
				{
					InProgressResult.Result.CommitStatus = PackageWriter->GetCommitStatus(PackageName);
				}
				else
				{
					InProgressResult.Result.CommitStatus = Attachment ? IPackageWriter::ECommitStatus::Success
						: IPackageWriter::ECommitStatus::NotCommitted;
				}

				if (LoadFromCompactBinary(MoveTemp(Attachment), InProgressResult.Result.Artifacts))
				{
					InProgressResult.Result.Artifacts.PackageName = PackageName;
				}
			}
			else if (AttachmentKey == BuildDefinitionsAttachmentKey)
			{
				LoadFromCompactBinary(MoveTemp(Attachment), InProgressResult.Result.BuildDefinitions);
			}
			else if (AttachmentKey == ImportsCheckerAttachmentKey)
			{
				LoadFromCompactBinary(Attachment.AsFieldView(), InProgressResult.Result.ImportsCheckerData);
			}
			else if (AttachmentKey == LogMessagesAttachmentKey)
			{
				FLogMessagesArray LogMessagesArray(InProgressResult.Result.LogMessages);
				LoadFromCompactBinary(Attachment.AsFieldView(), LogMessagesArray);
			}

			int32 RequestedAttachmentNum = InProgressResult.WhereCooked == EWhereCooked::ThisSession
				? AttachmentsForIncrementalSkipAndBuildDependenciesNum : AttachmentsForBuildDependenciesNum;
			if (InProgressResult.ReceivedAttachmentCount == RequestedAttachmentNum)
			{
				Callback(PackageName, MoveTemp(InProgressResult.Result));
			}
		};
	

	if (!SessionOplogNames.IsEmpty())
	{
		auto UniqueCallback = [OnOplogAttachment](FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)
			{
				OnOplogAttachment(PackageName, AttachmentKey, MoveTemp(Attachment));
			};
		PackageWriter->GetOplogAttachments(SessionOplogNames,
			AttachmentsForIncrementalSkipAndBuildDependencies, MoveTemp(UniqueCallback));
	}
	if (!BaseGameOplogNames.IsEmpty())
	{
		auto UniqueCallback = [OnOplogAttachment](FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)
			{
				OnOplogAttachment(PackageName, AttachmentKey, MoveTemp(Attachment));
			};
		PackageWriter->GetBaseGameOplogAttachments(BaseGameOplogNames, AttachmentsForBuildDependencies,
			MoveTemp(UniqueCallback));
	}
	if (!EditorDomainNames.IsEmpty())
	{
		auto UniqueCallback = [OnOplogAttachment](FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)
			{
				OnOplogAttachment(PackageName, AttachmentKey, MoveTemp(Attachment));
			};
		GEditorDomainOplog->GetOplogAttachments(EditorDomainNames,
			AttachmentsForIncrementalSkipAndBuildDependencies, MoveTemp(UniqueCallback));
	}
}

FPackageArtifacts FPackageArtifacts::Collect(const UPackage* Package, const ITargetPlatform* TargetPlatform,
	FBuildResultDependenciesMap&& InResultDependencies, bool bHasSaveResult,
	TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper, bool bGenerated,
	TArray<FName>&& InRuntimeDependencies, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages)
{
	if (!Package)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("Invalid null package."));
		return FPackageArtifacts();
	}
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (!AssetRegistry)
	{
		if (OutMessages) OutMessages->Emplace(ELogVerbosity::Error, TEXT("AssetRegistry is unavailable."));
		return FPackageArtifacts();
	}
	FName PackageName = Package->GetFName();

	// Append AssetRegistry dependencies as runtime dependencies, only for non-generated pacakges. The equivalent for
	// generated packages comes from the generator's ICookPackageSplitter functions and this function receives them via
	// InRuntimeDependencies.
	if (!bGenerated)
	{
		TArray<FName> AssetDependencies;
		AssetRegistry->GetDependencies(PackageName, AssetDependencies,
			UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Game);
		InRuntimeDependencies.Append(MoveTemp(AssetDependencies));
	}

	// Collect the save's BuildDependencies, and pass in our runtimedependencies for read/write.
	if (!FBuildDependencySet::TryCollectInternal(InResultDependencies, InRuntimeDependencies, OutMessages,
		BuildResult::NAME_Save, Package, TargetPlatform, UntrackedSoftPackageReferences, GenerationHelper, bGenerated))
	{
		return FPackageArtifacts();
	}
	// Sort and remove duplicates in the results from TryCollectInternal.
	for (TPair<FName, TArray<FCookDependency>>& BuildResultPair : InResultDependencies)
	{
		TArray<FCookDependency>& ResultDependencies = BuildResultPair.Value;
		Algo::Sort(ResultDependencies);
		ResultDependencies.SetNum(Algo::Unique(ResultDependencies));
	}

	FPackageArtifacts Result;
	Result.PackageName = PackageName;
	Result.bHasSaveResults = bHasSaveResult;
	TOptional<FAssetPackageData> AssetPackageData = AssetRegistry->GetAssetPackageDataCopy(PackageName);
	Result.StoredPackageSavedHash = AssetPackageData ? AssetPackageData->GetPackageSavedHash() : FIoHash();
	Result.IoStoreChunkId = CreatePackageDataChunkId(FPackageId::FromName(PackageName));

	// Store input+collected RuntimeDependencies on the result
	Algo::Sort(InRuntimeDependencies, FNameLexicalLess());
	InRuntimeDependencies.SetNum(Algo::Unique(InRuntimeDependencies), EAllowShrinking::Yes);
	Result.RuntimeDependencies = MoveTemp(InRuntimeDependencies);

	// Store the collected LoadBuildDependencies on the result
	TArray<FCookDependency>& LoadDependencies = InResultDependencies.FindOrAdd(BuildResult::NAME_Load);
	Result.LoadBuildDependencies.SetName(BuildResult::NAME_Load);
	Result.LoadBuildDependencies.SetNormalizedDependencies(MoveTemp(LoadDependencies));
		Result.LoadBuildDependencies.TryCalculateCurrentKey(PackageName, TargetPlatform,
			nullptr /* GenerationHelper */, true /* bUpdateValues */, nullptr /* OutMessages */);
		Result.LoadBuildDependencies.StoreCurrentKey();
		Result.LoadBuildDependencies.SetValid(true);

	// Copy LoadBuildDependencies onto SaveBuildDependencies
	// TODO: Add a transitive builddependency from SaveBuildDependencies to LoadBuildDependencies rather than duplicating.
	TArray<FCookDependency>& SaveDependencies = InResultDependencies.FindOrAdd(BuildResult::NAME_Save);
	SaveDependencies.Append(Result.LoadBuildDependencies.GetDependencies());
	Algo::Sort(SaveDependencies);
	SaveDependencies.SetNum(Algo::Unique(SaveDependencies), EAllowShrinking::Yes);

	// Store the collected SaveDependencies on the result
	Result.SaveBuildDependencies.SetName(BuildResult::NAME_Save);
	Result.SaveBuildDependencies.SetNormalizedDependencies(MoveTemp(SaveDependencies));
	FBuildDependencySet::ECurrentKeyResult CurrentKeyResult = Result.SaveBuildDependencies.TryCalculateCurrentKey(
		PackageName, TargetPlatform, GenerationHelper, true /* bUpdateValues */, OutMessages);
	if (CurrentKeyResult == FBuildDependencySet::ECurrentKeyResult::Error)
	{
		return FPackageArtifacts();
	}
	Result.SaveBuildDependencies.StoreCurrentKey();
	Result.SaveBuildDependencies.SetValid(true);
	Result.bValid = true;

	return Result;
}

} // namespace UE::Cook
