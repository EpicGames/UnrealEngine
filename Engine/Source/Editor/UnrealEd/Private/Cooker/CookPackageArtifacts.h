// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DerivedDataBuildDefinition.h"
#include "IO/IoChunkId.h"
#include "IO/IoHash.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Optional.h"
#include "Serialization/PackageWriter.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"

class FCbObject;
class FCbObjectView;
class FCbWriter;
class ITargetPlatform;
class UObject;
class UPackage;
struct FSavePackageResultStruct;
template <typename FuncType> class TUniqueFunction;

namespace UE::Cook { class FPackageArtifacts; }
namespace UE::Cook { class ICookInfo; }
namespace UE::Cook { struct FBuildDefinitionList; }
namespace UE::Cook { struct FGenerationHelper; }
namespace UE::Cook::Artifact { struct FAppendPackageMetaDataContext; }
namespace UE::Cook { struct FIncrementalCookAttachments; }

bool LoadFromCompactBinary(FCbObjectView ObjectView, UE::Cook::FPackageArtifacts& CookAttachments);
FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FPackageArtifacts& CookAttachments);
bool LoadFromCompactBinary(FCbObject&& Object, UE::Cook::FBuildDefinitionList& Definitions);
FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FBuildDefinitionList& Definitions);

namespace UE::Cook
{

/**
 * A list of dependencies that affect the build of a BuildResult. BuildResults can be a package load, a package save,
 * or a system-specific set of data that is produced alongside the package load or save.
 *
 * In the build operation for the cook of a package, the load of the package is recorded as a BuildResult with no
 * payload (it has an implicit payload which is the loaded package as it exists in memory, but we do not store that
 * payload in the oplog). The dependencies for that BuildResult are the most commonly used source for transitive build
 * dependencies.
 * 
 * The second BuildResult in a package's cook is the bytes of the saved package. That BuildResult stores some of its
 * payload - the package bytes - as a special payload which is not stored in the BuildResult itself, but rather is
 * stored externally in the oplog. It stores the rest of its payload - the runtime dependencies - in the
 * FPackageArtifacts. The BuildDependencies of the save BuildResult are used to decide whether the package can be
 * incrementally skipped.
 *
 * System-specific BuildResults are not saved during the cook and each one must be recalculated on demand during the
 * cook by each dependent package that incorporates their data, and stored in the package data of the owning package
 * until it gets garbage collected and the data has to be recreated. The dependencies of that operation are stored in
 * the FPackageArtifacts for the owning package and can be used as a build dependency for the owning package and the
 * dependent packages.
 *
 * TODO: Add a system to preserve build results along with the dependencies.
 */
class FBuildDependencySet
{
public:
	/**
	 * True if the structure has been calculated or Set since last reset. False if the stucture is default or has
	 * been reset.
	 */
	bool IsValid() const;

	/** Name of the build result owning the DependencySet, used for lookup by transitive build dependencies. */
	FName GetName() const;
	void SetName(FName InName);

	/** The list of dependencies in the BuildDependencySet. */
	const TArray<FCookDependency>& GetDependencies() const;
	/**
	 * When constructing the set, the dependencies must be normalized - rules processed, sorted, made unique
	 * by the caller before being set into the BuildDependencySet.
	 */
	void SetNormalizedDependencies(TArray<FCookDependency> InDependencies);
	/** Copy CurrentKey into StoredKey, called after SetNormalizedDependencies before Save. */
	void StoreCurrentKey();
	/** Sets IsValid to the given argument, called after all values have been written by the Caller before Save */
	void SetValid(bool bInValid);

	/** Helper function to filter GetDependencies for the transitive build dependencies. */
	template <typename AllocatorType>
	void GetTransitiveDependencies(TArray<FName, AllocatorType>& OutDependencies) const;

	/** Return the Key that was hashed from the BuildDependencySet in the cookprocess that created it. */
	const FIoHash& GetStoredKey() const;
	/**
	 * Return the Key that was hashed from the BuildDependencySet in the current cookprocess.
	 * Will be the zero-hash if not yet calculated.
	 */
	const FIoHash& GetCurrentKey() const;
	/** Call TryCalculateCurrentKey if not yet called, and return whether StoredKey == CurrentKey. */
	bool HasKeyMatch(FName PackageName, const ITargetPlatform* TargetPlatform, FGenerationHelper* GenerationHelper);
	/**
	 * Report a StringDescriptor for each dependency that was found to be modified, for writing log output
	 * about why the BuildDependencySet's current key is different than its stored key.
	 * 
	 * @return True if data is valid and modified dependencies can be calculated, else false. Will return
	 *         true but with an empty list if not modified.
	 */
	bool TryGetModifiedDependencies(FName PackageName, const ITargetPlatform* TargetPlatform,
		FGenerationHelper* GenerationHelper, TArray<FString>& OutModifiedDependencies) const;
	enum class ECurrentKeyResult
	{
		Success,
		Invalidated,
		Error,
	};
	/**
	 * Calculate the current key(s) from the Dependencies and store it in GetCurrentKey().
	 *
	 * @param GenerationHelper - If non-null, provides the lookup for the AssetPackageData of generated packages.
	 *                           Must be provided if any generated packages are in the dependencies.
	 * @param bUpdateValues - If true, also updates the stored value on each CookDependency.
	 */
	ECurrentKeyResult TryCalculateCurrentKey(FName PackageName, const ITargetPlatform* TargetPlatform,
		FGenerationHelper* GenerationHelper, bool bUpdateValues,
		TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages = nullptr);

	/** Clear data (except Name) and free memory. */
	void Empty();

	bool TryLoad(FCbFieldView FieldView);
	void Save(FCbWriter& Writer) const;

	/**
	 * Read dependencies for the given targetplatform of the given package out of global dependency trackers
	 * that have recorded its data during the package's load operations in the current cook session.
	 */
	static FBuildResultDependenciesMap CollectLoadedPackage(const UPackage* Package,
		TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages = nullptr);

	/**
	 * Internal helper for FBuildDependencySet::Collect and FPackageArtifacts::Collect. Handles all arguments
	 * used by either of them, and returns a map of BuildResultDependencies. Each returned map entry contains the
	 * dependencies for a BuildResult (e.g. NAME_Save); those dependencies are not yet sorted or unique.
	 * 
	 * @param DefaultBuildResult - into which build result (NAME_Load or NAME_Save) detected dependencies should be
	 *                             added onto, for each dependency that does not already have a buildresult specified.
	 * @param TargetPlatform - If null, collects dependencies reported by all platforms. If non-null, only collects
	 *                         dependencies reported for the given platform or reported as platform-agnostic.
	 * @param SaveResult - If non-null, contains data reported from the SavePackage call.
	 * @param GenerationHelper - If non-null, provides the lookup for the AssetPackageData of generated packages.
	 *                           Must be provided if any generated packages are in the dependencies.
	 */
	static bool TryCollectInternal(FBuildResultDependenciesMap& InOutResultDependencies,
		TArray<FName>& InOutRuntimeDependencies, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages,
		FName DefaultBuildResult, const UPackage* Package, const ITargetPlatform* TargetPlatform,
		TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper, bool bGenerated);

	/**
	 * Collect the dependencies referenced by a given settingsobject from e.g. config. Globally cached for the current
	 * process.
	 */
	static FBuildDependencySet CollectSettingsObject(const UObject* Object,
		TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages);

	// Hidden friend operators
private:
	friend bool LoadFromCompactBinary(FCbFieldView FieldView, UE::Cook::FBuildDependencySet& DependencySet)
	{
		return DependencySet.TryLoad(FieldView);
	}
	friend FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FBuildDependencySet& DependencySet)
	{
		DependencySet.Save(Writer);
		return Writer;
	}

private:
	/* Name used to look up the BuildResult for transitive dependencies and data derived from it. */
	FName Name;
	/**
	 * The dependencies that impact the creation of the BuildResult that owns this set.
	 * These dependencies are normalized and sorted before storage.
	 */
	TArray<UE::Cook::FCookDependency> Dependencies;
	/** The hash of the dependencies that was calculated in the cooksession that created the BuildResult. */
	FIoHash StoredKey;
	/** The hash of the dependencies that was calculated during the current cook session. */
	FIoHash CurrentKey;
	bool bValid = false;
};

/**
 * Non-runtime data recorded about each package and stored in the cook Oplog as attachments to the package.
 * Includes BuildResults built from the package that can be used for future incremental cooks, and the dependencies
 * discovered for those BuildResults while the package was loading and cook-saving.
 *
 * Notes about the dependencies:
 * All dependencies except for those marked Runtime contribute to the BuildResult's TargetDomain Key. If HasKeyMatch
 * returns false after fetching this structure for a package at the beginning of cook, then the package is not
 * incrementally skippable and needs to be recooked, and this structure needs to be recalculated for the package.
 *
 * Runtime fields on the dependencies are used to inform the cook of discovered softreferences that need to be added to
 * the cook when the package is cooked.
 */
class FPackageArtifacts
{
public:
	FPackageArtifacts();

	/**
	 * True if the structure has been calculated or fetched and accurately reports dependencies and
	 * key for the package. False if the stucture is default, has been reset, or was marked invalid.
	 */
	bool IsValid() const;

	FBuildDependencySet& FindOrAddBuildDependencySet(FName ResultName);
	FBuildDependencySet* FindBuildDependencySet(FName ResultName);

	/** Get all of the runtime dependencies reported by the package, both script and content. */
	const TArray<FName>& GetRuntimeDependencies() const;
	/** Return runtime dependencies reported by the package that are content packages; script packages are removed. */
	template <typename AllocatorType>
	void GetRuntimeContentDependencies(TArray<FName, AllocatorType>& OutDependencies) const;

	bool HasSaveResults() const;
	FName GetPackageName() const;

	/**
	 * Calculate the current key(s) from the BuildDependencies stored on this PackageArtifacts, and store it in
	 * GetCurrentKey().
	 * 
	 * @param TargetPlatform - Which targetplatform we are collecting for.
	 * @param GenerationHelper - If non-null, provides the lookup for the AssetPackageData of generated packages.
	 *                           Must be provided if any generated packages are in the dependencies.
	 * @param bUpdateValues - If true, also updates the stored value on each CookDependency.
	 */
	FBuildDependencySet::ECurrentKeyResult TryCalculateCurrentKey(const ITargetPlatform* TargetPlatform,
		FGenerationHelper* GenerationHelper, bool bUpdateValues,
		TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages = nullptr);
	void Empty();

	// Construction functions
	/**
	 * Read dependencies for the given targetplatform of the given package out of global dependency trackers that have
	 * recorded its data for the package's save operations, and combine those with the given previously recorded
	 * LoadDependencies to create the complete PackageArtifacts.
	 *
	 * @param LoadDependencies - If non-null, will be copied into LoadDependencies on the result.
	 * @param GenerationHelper - If non-null, provides the lookup for the AssetPackageData of generated packages.
	 *                           Must be provided if any generated packages are in the dependencies.
	 */
	static FPackageArtifacts Collect(const UPackage* Package, const ITargetPlatform* TargetPlatform,
		FBuildResultDependenciesMap&& InResultDependencies, bool bHasSaveResult,
		TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper, bool bGenerated,
		TArray<FName>&& InRuntimeDependencies, TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages = nullptr);

	// Fetch function to load the dependencies from a PackageStore is not yet implemented independently for
	// this structure. Use FIncrementalCookAttachments instead.

	// Legacy API before FBuildDependencySet. TODO: Change callers to use FindBuildResult. */
	const TArray<FCookDependency>& GetBuildDependencies() const;
	template <typename AllocatorType>
	void GetTransitiveBuildDependencies(TArray<FName, AllocatorType>& OutDependencies) const;
	const FIoHash& GetStoredKey() const;
	const FIoHash& GetCurrentKey() const;
	bool HasKeyMatch(const ITargetPlatform* TargetPlatform, FGenerationHelper* GenerationHelper);

	/**
	 * Report a StringDescriptor for each dependency that was found to be modified, for writing log output
	 * about why the HasKeyMatch returned false.
	 * 
	 * @return True if data is valid and modified dependencies can be calculated, else false. Will return
	 *         true but with an empty list if not modified.
	 */
	bool TryGetModifiedDependencies(const ITargetPlatform* TargetPlatform, FGenerationHelper* GenerationHelper,
		TArray<FString>& OutModifiedDependencies) const;

	/**
	 * Return whether the current Package's PackageSavedHash (from its package header on disk) is different than
	 * it was during the previous cook that recorded the PackageArtifacts.
	 *
	 * @return An optional with true if modified, false if unmodified, and unset if data is invalid and whether
	 *         it is modified is therefore unknown.
	 */
	TOptional<bool> GetIsPackageModified() const;

private:
	FBuildDependencySet LoadBuildDependencies;
	FBuildDependencySet SaveBuildDependencies;
	TArray<FName> RuntimeDependencies;
	FIoHash StoredPackageSavedHash;
	FIoChunkId IoStoreChunkId;
	FName PackageName;
	bool bHasSaveResults = false;
	bool bValid = false;

	friend bool ::LoadFromCompactBinary(FCbObjectView ObjectView, FPackageArtifacts& CookAttachments);
	friend FCbWriter& ::operator<<(FCbWriter& Writer, const FPackageArtifacts& CookAttachments);
	friend FIncrementalCookAttachments;
};

/**
 * Non-persistent cache of groups of cookdependencies. Dependencies to a CookDependencyGroup are not persistently
 * recorded into the oplog, instead we make a copy of all of their dependencies and append those dependencies onto
 * the CookDependencies that are written for a package.
 *
 * Example: The cookdependencies used by the CDO of a settings object that itself is configured by config values.
 *          The settings object's class's schema and the list of config settings are included in the cookdependencies.
 */
class FCookDependencyGroups
{
public:
	struct FRecordedDependencies
	{
		FBuildDependencySet Dependencies;
		TArray<TPair<ELogVerbosity::Type, FString>> Messages;
		bool bInitialized = false;
	};

	static FCookDependencyGroups& Get();
	FRecordedDependencies& FindOrCreate(UPTRINT Key);

private:
	TMap<UPTRINT, FRecordedDependencies> Groups;
};

/** Wrapper around TArray<FBuildDefinition>, used to provide custom functions for compactbinary, collection, and fetch */
struct FBuildDefinitionList
{
public:
	TArray<UE::DerivedData::FBuildDefinition> Definitions;

	void Empty();

	/** Collect DDC BuildDefinitions that were issued from the load/save of the given package and platform. */
	static FBuildDefinitionList Collect(const UPackage* Package, const ITargetPlatform* TargetPlatform,
		TArray<TPair<ELogVerbosity::Type, FString>>* OutMessages = nullptr);

private:
	friend bool ::LoadFromCompactBinary(FCbObject&& ObjectView, FBuildDefinitionList& Definitions);
	friend FCbWriter& ::operator<<(FCbWriter& Writer, const FBuildDefinitionList& Definitions);
};

/** All of the metadata that is written/read to the oplog for the incremental cook of a package. */
struct FIncrementalCookAttachments
{
	FPackageArtifacts Artifacts;
	FBuildDefinitionList BuildDefinitions;
	FImportsCheckerData ImportsCheckerData;
	TArray<FReplicatedLogData> LogMessages;
	IPackageWriter::ECommitStatus CommitStatus = IPackageWriter::ECommitStatus::NotCommitted;

	void Empty();
	void AppendCommitAttachments(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context);

	static void Fetch(TArrayView<FPackageIncrementalCookId> PackageIds, const ITargetPlatform* TargetPlatform,
		ICookedPackageWriter* PackageWriter,
		TFunction<void(FName PackageName, FIncrementalCookAttachments&& Result)>&& Callback);
	static FIncrementalCookAttachments Collect(const UPackage* Package,
		const ITargetPlatform* TargetPlatform, FBuildResultDependenciesMap&& InResultDependencies,
		bool bHasSaveResult, TConstArrayView<FName> UntrackedSoftPackageReferences, FGenerationHelper* GenerationHelper,
		bool bGenerated, TArray<FName>&& RuntimeDependencies,
		TConstArrayView<UObject*> Imports, TConstArrayView<UObject*> Exports,
		TConstArrayView<UE::SavePackageUtilities::FPreloadDependency> PreloadDependencies,
		TConstArrayView<FReplicatedLogData> LogMessages);
};

} // namespace UE::Cook


////////////////////////////////
// Inline Implementations
////////////////////////////////

namespace UE::Cook
{

inline bool FBuildDependencySet::IsValid() const
{
	return bValid;
}

inline FName FBuildDependencySet::GetName() const
{
	return Name;
}

inline void FBuildDependencySet::SetName(FName InName)
{
	Name = InName;
}

inline const TArray<FCookDependency>& FBuildDependencySet::GetDependencies() const
{
	return Dependencies;
}

inline void FBuildDependencySet::SetNormalizedDependencies(TArray<FCookDependency> InDependencies)
{
	Dependencies = MoveTemp(InDependencies);
}

inline void FBuildDependencySet::StoreCurrentKey()
{
	StoredKey = CurrentKey;
}

inline void FBuildDependencySet::SetValid(bool bInValid)
{
	bValid = bInValid;
}

template <typename AllocatorType>
inline void FBuildDependencySet::GetTransitiveDependencies(TArray<FName, AllocatorType>& OutDependencies) const
{
	for (const FCookDependency& BuildDependency : Dependencies)
	{
		if (BuildDependency.GetType() == ECookDependency::TransitiveBuild)
		{
			OutDependencies.Add(BuildDependency.GetPackageName());
		}
	}
}

inline const FIoHash& FBuildDependencySet::GetStoredKey() const
{
	return StoredKey;
}

inline const FIoHash& FBuildDependencySet::GetCurrentKey() const
{
	return CurrentKey;
}

inline bool FPackageArtifacts::IsValid() const
{
	return bValid;
}

inline const TArray<FName>& FPackageArtifacts::GetRuntimeDependencies() const
{
	return RuntimeDependencies;
}

template <typename AllocatorType>
inline void FPackageArtifacts::GetRuntimeContentDependencies(TArray<FName, AllocatorType>& OutDependencies) const
{
	OutDependencies.Reserve(OutDependencies.Num() + RuntimeDependencies.Num());
	for (FName RuntimeDependency : RuntimeDependencies)
	{
		if (!FPackageName::IsScriptPackage(WriteToString<256>(RuntimeDependency)))
		{
			OutDependencies.Add(RuntimeDependency);
		}
	}
}

inline FName FPackageArtifacts::GetPackageName() const
{
	return PackageName;
}

inline const TArray<FCookDependency>& FPackageArtifacts::GetBuildDependencies() const
{
	return SaveBuildDependencies.GetDependencies();
}

template <typename AllocatorType>
inline void FPackageArtifacts::GetTransitiveBuildDependencies(TArray<FName, AllocatorType>& OutDependencies) const
{
	return SaveBuildDependencies.GetTransitiveDependencies(OutDependencies);
}

inline const FIoHash& FPackageArtifacts::GetStoredKey() const
{
	return SaveBuildDependencies.GetStoredKey();
}

inline const FIoHash& FPackageArtifacts::GetCurrentKey() const
{
	return SaveBuildDependencies.GetCurrentKey();
}

} // namespace UE::Cook
