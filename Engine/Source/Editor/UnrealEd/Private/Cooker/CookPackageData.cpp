// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookPackageData.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetCompilingManager.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "CompactBinaryTCP.h"
#include "Cooker/CookActivePackage.h"
#include "Cooker/BuildResultDependenciesMap.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookPackagePreloader.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookRequestCluster.h"
#include "Cooker/CookWorkerClient.h"
#include "Cooker/IWorkerRequests.h"
#include "Cooker/PackageTracker.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Containers/StringView.h"
#include "Engine/Console.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Serialization/CompactBinaryWriter.h"
#include "ShaderCompiler.h"
#include "UObject/Object.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"

#include <atomic>

namespace UE::Cook
{

float GPollAsyncPeriod = .100f;
static FAutoConsoleVariableRef CVarPollAsyncPeriod(
	TEXT("cook.PollAsyncPeriod"),
	GPollAsyncPeriod,
	TEXT("Minimum time in seconds between PollPendingCookedPlatformDatas."),
	ECVF_Default);
	
//////////////////////////////////////////////////////////////////////////
// FPackageData
FPackagePlatformData::FPackagePlatformData()
	: Reachability((uint8)EReachability::None), ReachabilityVisitedByCluster((uint8)EReachability::None)
	, bSaveTimedOut(0), bCookable(1), bExplorable(1), bExplorableOverride(0) , IncrementallyUnmodified(0)
	, bIncrementallySkipped(0), WhereCooked((uint8)EWhereCooked::ThisSession), bRegisteredForCachedObjectsInOuter(0), bCommitted(0)
	, CookResults((uint8)ECookResult::NotAttempted)
{
}

void FPackagePlatformData::ResetReachable(EReachability InReachability)
{
	ClearReachability(InReachability);
	ClearVisitedByCluster(InReachability);
	if (EnumHasAnyFlags(InReachability, EReachability::Runtime))
	{
		SetCookable(true);
		SetExplorable(true);
	}
}

void FPackagePlatformData::MarkAsExplorable()
{
	ResetReachable(EReachability::Runtime);
	SetExplorableOverride(true);
}

void FPackagePlatformData::MarkCommittableForWorker(EReachability InReachability, FCookWorkerClient& CookWorkerClient)
{
	AddReachability(InReachability);
	AddVisitedByCluster(InReachability);
	if (EnumHasAnyFlags(InReachability, EReachability::Runtime))
	{
		SetExplorable(true);
		SetCookable(true);
	}
	SetCommitted(false);
	SetCookResults(ECookResult::NotAttempted);
}

bool FPackagePlatformData::NeedsCommit(const ITargetPlatform* PlatformItBelongsTo, EReachability InReachability) const
{
	return !IsCommitted()
		&& PlatformItBelongsTo != CookerLoadingPlatformKey
		&& IsReachable(InReachability)
		&& !(InReachability == EReachability::Runtime && !IsCookable());
}

FPackageData::FPackageData(FPackageDatas& PackageDatas, const FName& InPackageName, const FName& InFileName)
	: ParentGenerationHelper(nullptr), PackageName(InPackageName), FileName(InFileName), PackageDatas(PackageDatas)
	, Instigator(EInstigator::NotYetRequested), BuildInstigator(EInstigator::NotYetRequested)
	, Urgency(static_cast<uint32>(EUrgency::Normal)), bIsCookLast(0)
	, bIsVisited(0)
	, bHasSaveCache(0), bPrepareSaveFailed(0), bPrepareSaveRequiresGC(0)
	, MonitorCookResult((uint8)ECookResult::NotAttempted)
	, bGenerated(0), bKeepReferencedDuringGC(0)
	, bWasCookedThisSession(0)
	, DoesGeneratedRequireGeneratorValue(static_cast<uint32>(ICookPackageSplitter::EGeneratedRequiresGenerator::None))
	, bHasReplayedLogMessages(0), bHasBeenLoadedForCooking(0)
{
	SetState(EPackageState::Idle);
	SetSaveSubState(ESaveSubState::StartSave);
	SetSuppressCookReason(ESuppressCookReason::NotSuppressed);

	SendToState(EPackageState::Idle, ESendFlags::QueueAdd, EStateChangeReason::Discovered);
}

FPackageData::~FPackageData()
{
	// ClearReferences should have been called earlier, but call it here in case it was missed
	ClearReferences();
	// We need to send OnLastCookedPlatformRemoved message to the monitor, so call SetPlatformsNotCooked
	ClearCookResults();
	// Update the monitor's counters and call exit functions
	SendToState(EPackageState::Idle, ESendFlags::QueueNone, EStateChangeReason::CookerShutdown);

	// FPackageDatas guarantees that all references to GenerationHelper are removed before any PackageDatas are deleted.
	// We rely on that so that we can be sure that when this PackageData is being deleted, its GenerationHelper - which
	// assumes the FPackageData lifetime exceeds its own - has already been deleted.
	check(GenerationHelper == nullptr);
	// FPackageDatas guarantees that all references to PackagePreloaders are removed before any PackageDatas are deleted.
	// We rely on that so that we can be sure that when this PackageData is being deleted, its PackagePreloader - which
	// assumes the FPackageData lifetime exceeds its own - has already been deleted.
	check(PackagePreloader == nullptr);
}

void FPackageData::ClearReferences()
{
	if (GenerationHelper)
	{
		GenerationHelper->ClearSelfReferences();
	}
	SetParentGenerationHelper(nullptr, EStateChangeReason::CookerShutdown);
	if (PackagePreloader)
	{
		PackagePreloader->Shutdown(); // Clears references to any other preloaders
	}
	ClearDiscoveredDependencies();
}

int32 FPackageData::GetPlatformsNeedingCommitNum(EReachability Reachability) const
{
	int32 Result = 0;
	for (const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Value.NeedsCommit(Pair.Key, Reachability))
		{
			++Result;
		}
	}
	return Result;
}

bool FPackageData::IsPlatformVisitedByCluster(const ITargetPlatform* Platform, EReachability InReachability) const
{
	const FPackagePlatformData* PlatformData = FindPlatformData(Platform);
	return PlatformData && PlatformData->IsVisitedByCluster(InReachability);
}

bool FPackageData::HasReachablePlatforms(EReachability InReachability,
	const TArrayView<const ITargetPlatform* const>& Platforms) const
{
	if (Platforms.Num() == 0)
	{
		return true;
	}
	if (PlatformDatas.Num() == 0)
	{
		return false;
	}

	for (const ITargetPlatform* QueryPlatform : Platforms)
	{
		const FPackagePlatformData* PlatformData = PlatformDatas.Find(QueryPlatform);
		if (!PlatformData || !PlatformData->IsReachable(InReachability))
		{
			return false;
		}
	}
	return true;
}

bool FPackageData::AreAllReachablePlatformsVisitedByCluster(EReachability InReachability) const
{
	for (const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (EnumHasAllFlags(Pair.Value.GetReachability(), InReachability) && !Pair.Value.IsVisitedByCluster(InReachability))
		{
			return false;
		}
	}
	return true;
}

const TArray<const ITargetPlatform*>& FPackageData::GetSessionPlatformsInternal(UCookOnTheFlyServer& COTFS)
{
	return COTFS.PlatformManager->GetSessionPlatforms();
}

void FPackageData::AddReachablePlatforms(FRequestCluster& RequestCluster, EReachability InReachability,
	TConstArrayView<const ITargetPlatform*> Platforms, FInstigator&& InInstigator)
{
	AddReachablePlatformsInternal(*this, InReachability, Platforms, MoveTemp(InInstigator));
}

void FPackageData::AddReachablePlatformsInternal(FPackageData& PackageData, EReachability InReachability,
	TConstArrayView<const ITargetPlatform*> Platforms, FInstigator&& InInstigator)
{
	// This is a static helper function to make it impossible to make a typo and use this->Instigator instead of
	// InInstigator
	bool bSessionPlatformModified = false;
	for (const ITargetPlatform* Platform : Platforms)
	{
		FPackagePlatformData& PlatformData = PackageData.FindOrAddPlatformData(Platform);
		bSessionPlatformModified |= (Platform != CookerLoadingPlatformKey && !PlatformData.IsReachable(InReachability));
		PlatformData.AddReachability(InReachability);
	}
	if (bSessionPlatformModified)
	{
		PackageData.SetInstigatorInternal(InReachability, MoveTemp(InInstigator));
	}
}

void FPackageData::QueueAsDiscovered(FInstigator&& InInstigator, FDiscoveredPlatformSet&& ReachablePlatforms,
	EUrgency InUrgency)
{
	QueueAsDiscoveredInternal(*this, MoveTemp(InInstigator), MoveTemp(ReachablePlatforms), InUrgency);
}

void FPackageData::QueueAsDiscoveredInternal(FPackageData& PackageData, FInstigator&& InInstigator,
	FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency InUrgency)
{
	// This is a static helper function to make it impossible to make a typo and use this->Instigator instead of
	// InInstigator
	FPackageDatas& LocalPackageDatas = PackageData.PackageDatas;
	FRequestQueue& RequestQueue = LocalPackageDatas.GetRequestQueue();
	UCookOnTheFlyServer& COTFS = LocalPackageDatas.GetCookOnTheFlyServer();
	if (InInstigator.Category != EInstigator::BuildDependency)
	{
		TRingBuffer<FDiscoveryQueueElement>& Queue = RequestQueue.GetDiscoveryQueue();
		if (COTFS.GetCookPhase() == ECookPhase::BuildDependencies)
		{
			UE_LOGF(LogCook, Warning,
				"Package was added to the runtime discovery queue after starting BuildDependencies phase."
				"\n\tPackage: %ls", *PackageData.GetPackageName().ToString());
			constexpr int32 MaxCount = 5;
			static int32 Count = 0;
			if (Count++ < MaxCount)
			{
				FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
			}
		}
		Queue.Add(FDiscoveryQueueElement{ &PackageData, MoveTemp(InInstigator), MoveTemp(ReachablePlatforms), InUrgency });
	}
	else
	{
		// Build dependencies always immediately mark the package as being reachable, rather than needing to wait for
		// the discoveryqueue. Waiting for the discovery queue is only necessary for runtime dependencies because
		// we need to know whether the package was expected.
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> BufferPlatforms;
		TConstArrayView<const ITargetPlatform*> PlatformArray =
			ReachablePlatforms.GetPlatforms(COTFS, &InInstigator, TConstArrayView<const ITargetPlatform*>(),
				EReachability::Build, BufferPlatforms);
		bool bHasNewPlatforms = !PackageData.HasAllCommittedPlatforms(PlatformArray);
		if (bHasNewPlatforms)
		{
			AddReachablePlatformsInternal(PackageData, EReachability::Build, PlatformArray, MoveTemp(InInstigator));

			// If we have already kicked build dependencies, send the package to the discovery queue.
			// Otherwise it will be added to the discoveryqueue when we kick build dependencies, if it hasn't been committed by then.
			if (COTFS.GetCookPhase() == ECookPhase::BuildDependencies)
			{
				TRingBuffer<FPackageData*>& Queue = RequestQueue.GetBuildDependencyDiscoveryQueue();
				Queue.Add(&PackageData);
			}
		}
	}
}

void FPackageData::SetUrgency(EUrgency NewUrgency, ESendFlags SendFlags, bool bAllowUrgencyInIdle)
{
	if (GetUrgency() == NewUrgency)
	{
		return;
	}

	// It is illegal to SetUrgency to above normal when in the Idle state, unless the caller explicitly takes
	// responsibility for changing the state immediately afterwards.
	check(bAllowUrgencyInIdle || IsInProgress() || NewUrgency == EUrgency::Normal);
	// For SendFlags when setting urgency, only AddAndRemove or None are supported
	check(SendFlags == ESendFlags::QueueAddAndRemove || SendFlags == ESendFlags::QueueNone);

	EUrgency OldUrgency = GetUrgency();
	Urgency = static_cast<uint32>(NewUrgency);
	if (SendFlags == ESendFlags::QueueAddAndRemove)
	{
		UpdateContainerUrgency(OldUrgency, NewUrgency);
	}
	PackageDatas.GetMonitor().OnUrgencyChanged(*this, OldUrgency, NewUrgency);
}

void FPackageData::SetIsCookLast(bool bValue)
{
	bool bWasCookLast = GetIsCookLast();
	if (bWasCookLast != bValue)
	{
		bIsCookLast = static_cast<uint32>(bValue);
		PackageDatas.GetMonitor().OnCookLastChanged(*this);
	}
}

void FPackageData::SetInstigator(FRequestCluster& Cluster, EReachability InReachability, FInstigator&& InInstigator)
{
	SetInstigatorInternal(InReachability, MoveTemp(InInstigator));
}

void FPackageData::SetInstigator(FCookWorkerClient& Client, EReachability InReachability, FInstigator&& InInstigator)
{
	SetInstigatorInternal(InReachability, MoveTemp(InInstigator));
}

void FPackageData::SetInstigator(FGenerationHelper& InHelper, EReachability InReachability, FInstigator&& InInstigator)
{
	SetInstigatorInternal(InReachability, MoveTemp(InInstigator));
}

void FPackageData::SetInstigatorInternal(EReachability InReachability, FInstigator&& InInstigator)
{
	if ((InReachability == EReachability::Runtime && this->Instigator.Category == EInstigator::NotYetRequested)
		|| (InReachability == EReachability::Build && this->BuildInstigator.Category == EInstigator::NotYetRequested))
	{
		OnPackageDataFirstMarkedReachable(InReachability, MoveTemp(InInstigator));
	}
}

void FPackageData::ClearInProgressData(EStateChangeReason StateChangeReason)
{
	SetUrgency(EUrgency::Normal, ESendFlags::QueueNone);
	CompletionCallback = FCompletionCallback();
	if (GenerationHelper)
	{
		// ClearKeepForGeneratorSaveAllPlatforms might drop the last reference to the GenerationHelper, and delete
		// it out from under the ClearKeepForGeneratorSaveAllPlatforms, which is not supported by
		// ClearKeepForGeneratorSaveAllPlatforms, so keep it referenced across that call.
		TRefCountPtr<FGenerationHelper> KeepReferenced = GenerationHelper;
		// ClearKeepForGeneratorSave is called when finishing the save state, but not when demoting out of the save
		// state after a garbage collect. Call it here in case we cancel the save of the packagedata after demotion.
		// The other self-references (incremental, queued packages) should persist even when the packagedata is not
		// in progress.
		GenerationHelper->ClearKeepForGeneratorSaveAllPlatforms();
	}
	SetParentGenerationHelper(nullptr, StateChangeReason);

	// Clear data that is no longer needed when we have comitted all platforms
	if (HasAllCommittedPlatforms(PackageDatas.GetCookOnTheFlyServer().PlatformManager->GetSessionPlatforms()))
	{
		ClearLogMessages();
	}
}

void FPackageData::SetPlatformsCooked(
	const TConstArrayView<const ITargetPlatform*> TargetPlatforms,
	const TConstArrayView<ECookResult> Result,
	const bool bInWasCookedThisSession)
{
	check(TargetPlatforms.Num() == Result.Num());
	for (int32 n = 0; n < TargetPlatforms.Num(); ++n)
	{
		SetPlatformCooked(TargetPlatforms[n], Result[n], bInWasCookedThisSession);
	}
}

void FPackageData::SetPlatformsCooked(
	const TConstArrayView<const ITargetPlatform*> TargetPlatforms, 
	ECookResult Result,
	const bool bInWasCookedThisSession)
{
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		SetPlatformCooked(TargetPlatform, Result, bInWasCookedThisSession);
	}
}

void FPackageData::SetPlatformCooked(
	const ITargetPlatform* TargetPlatform, 
	ECookResult CookResult, 
	const bool bInWasCookedThisSession)
{
	check(TargetPlatform != nullptr);
	bWasCookedThisSession |= bInWasCookedThisSession && (CookResult == ECookResult::Succeeded);

	bool bNewCookAttemptedValue = (CookResult != ECookResult::NotAttempted);
	bool bModifiedCookAttempted = false;
	bool bHasAnyOtherCookAttempted = false;
	bool bExists = false;
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Key == TargetPlatform)
		{
			bExists = true;
			bModifiedCookAttempted = bModifiedCookAttempted | (Pair.Value.IsCookAttempted() != bNewCookAttemptedValue);
			Pair.Value.SetCookResults(CookResult);
			// Clear the SaveTimedOut when get a cook result, in case we save again later and need to allow retry again
			Pair.Value.SetSaveTimedOut(false);
		}
		else
		{
			bHasAnyOtherCookAttempted = bHasAnyOtherCookAttempted | Pair.Value.IsCookAttempted();
		}
	}

	if (!bExists && bNewCookAttemptedValue)
	{
		FPackagePlatformData& Value = PlatformDatas.FindOrAdd(TargetPlatform);
		Value.SetCookResults(CookResult);
		Value.SetSaveTimedOut(false);
		bModifiedCookAttempted = true;
	}

	if (bModifiedCookAttempted && !bHasAnyOtherCookAttempted)
	{
		if (bNewCookAttemptedValue)
		{
			PackageDatas.GetMonitor().OnFirstCookedPlatformAdded(*this, CookResult);
		}
		else
		{
			bWasCookedThisSession = false;
			PackageDatas.GetMonitor().OnLastCookedPlatformRemoved(*this);
		}
	}
}

void FPackageData::SetPlatformCommitted(const ITargetPlatform* TargetPlatform)
{
	check(TargetPlatform != nullptr);
	FPackagePlatformData& Value = PlatformDatas.FindOrAdd(TargetPlatform);
	Value.SetCommitted(true);
}

void FPackageData::ClearCookResults(const TConstArrayView<const ITargetPlatform*> TargetPlatforms)
{
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		ClearCookResults(TargetPlatform);
	}
}

void FPackageData::ResetReachable(EReachability InReachability)
{
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		Pair.Value.ResetReachable(InReachability);
	}
}

void FPackageData::ClearCookResults()
{
	bool bModifiedCookAttempted = false;
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		bModifiedCookAttempted = bModifiedCookAttempted | Pair.Value.IsCookAttempted();
		Pair.Value.SetCookResults(ECookResult::NotAttempted);
		Pair.Value.SetCommitted(false);
		Pair.Value.SetSaveTimedOut(false);
	}
	if (bModifiedCookAttempted)
	{
		bWasCookedThisSession = false;
		PackageDatas.GetMonitor().OnLastCookedPlatformRemoved(*this);
	}
	SetSuppressCookReason(ESuppressCookReason::NotSuppressed);
	bHasReplayedLogMessages = false;
	bHasBeenLoadedForCooking = false;
}

void FPackageData::ClearCookResults(const ITargetPlatform* TargetPlatform)
{
	bool bHasAnyOthers = false;
	bool bModifiedCookAttempted = false;
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Key == TargetPlatform)
		{
			bModifiedCookAttempted = bModifiedCookAttempted | Pair.Value.IsCookAttempted();
			Pair.Value.SetCookResults(ECookResult::NotAttempted);
			Pair.Value.SetCommitted(false);
			Pair.Value.SetSaveTimedOut(false);
		}
		else
		{
			bHasAnyOthers = bHasAnyOthers | Pair.Value.IsCookAttempted();
		}
	}
	if (bModifiedCookAttempted && !bHasAnyOthers)
	{
		bWasCookedThisSession = false;
		PackageDatas.GetMonitor().OnLastCookedPlatformRemoved(*this);
		bHasReplayedLogMessages = false;
		bHasBeenLoadedForCooking = false;
	}
}

const TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>>&
FPackageData::GetPlatformDatas() const
{
	return PlatformDatas;
}

TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>>&
FPackageData::GetPlatformDatasConstKeysMutableValues()
{
	return PlatformDatas;
}

FPackagePlatformData& FPackageData::FindOrAddPlatformData(const ITargetPlatform* TargetPlatform)
{
	check(TargetPlatform != nullptr);
	return PlatformDatas.FindOrAdd(TargetPlatform);
}

FPackagePlatformData* FPackageData::FindPlatformData(const ITargetPlatform* TargetPlatform)
{
	return PlatformDatas.Find(TargetPlatform);
}

const FPackagePlatformData* FPackageData::FindPlatformData(const ITargetPlatform* TargetPlatform) const
{
	return PlatformDatas.Find(TargetPlatform);
}

bool FPackageData::HasAnyCookedPlatform() const
{
	return Algo::AnyOf(PlatformDatas,
		[](const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair)
		{
			return Pair.Key != CookerLoadingPlatformKey && Pair.Value.IsCookAttempted();
		});
}

bool FPackageData::HasAnyCommittedPlatforms() const
{
	return Algo::AnyOf(PlatformDatas,
		[](const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair)
		{
			return Pair.Key != CookerLoadingPlatformKey && Pair.Value.IsCommitted();
		});
}

bool FPackageData::HasAnyCookedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms,
	bool bIncludeFailed) const
{
	if (PlatformDatas.Num() == 0)
	{
		return false;
	}

	for (const ITargetPlatform* QueryPlatform : Platforms)
	{
		if (HasCookedPlatform(QueryPlatform, bIncludeFailed))
		{
			return true;
		}
	}
	return false;
}

bool FPackageData::HasAllCookedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms,
	bool bIncludeFailed) const
{
	if (Platforms.Num() == 0)
	{
		return true;
	}
	if (PlatformDatas.Num() == 0)
	{
		return false;
	}

	for (const ITargetPlatform* QueryPlatform : Platforms)
	{
		if (!HasCookedPlatform(QueryPlatform, bIncludeFailed))
		{
			return false;
		}
	}
	return true;
}

bool FPackageData::HasCookedPlatform(const ITargetPlatform* Platform, bool bIncludeFailed) const
{
	ECookResult Result = GetCookResults(Platform);
	return (Result == ECookResult::Succeeded) | ((Result != ECookResult::NotAttempted) & (bIncludeFailed != 0));
}

ECookResult FPackageData::GetCookResults(const ITargetPlatform* Platform) const
{
	const FPackagePlatformData* PlatformData = PlatformDatas.Find(Platform);
	if (PlatformData)
	{
		return PlatformData->GetCookResults();
	}
	return ECookResult::NotAttempted;
}

bool FPackageData::HasAllCommittedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms) const
{
	if (Platforms.Num() == 0)
	{
		return true;
	}
	if (PlatformDatas.Num() == 0)
	{
		return false;
	}

	for (const ITargetPlatform* QueryPlatform : Platforms)
	{
		if (!HasCommittedPlatform(QueryPlatform))
		{
			return false;
		}
	}
	return true;
}

bool FPackageData::HasCommittedPlatform(const ITargetPlatform* Platform) const
{
	const FPackagePlatformData* PlatformData = PlatformDatas.Find(Platform);
	if (PlatformData)
	{
		return PlatformData->IsCommitted();
	}
	return false;
}

UPackage* FPackageData::GetPackage() const
{
	return Package.Get();
}

void FPackageData::SetPackage(UPackage* InPackage)
{
	Package = InPackage;
}

EPackageState FPackageData::GetState() const
{
	return static_cast<EPackageState>(State);
}

/** Boilerplate-reduction struct that defines all multi-state properties and sets them based on the given state. */
struct FStateProperties
{
	EPackageStateProperty Properties;
	explicit FStateProperties(EPackageState InState)
	{
		switch (InState)
		{
		case EPackageState::Idle:
			Properties = EPackageStateProperty::None;
			break;
		case EPackageState::Request:
			Properties = EPackageStateProperty::InProgress;
			break;
		case EPackageState::AssignedToWorker:
			Properties = EPackageStateProperty::InProgress | EPackageStateProperty::AssignedToWorkerProperty;
			break;
		case EPackageState::Load:
			Properties = EPackageStateProperty::InProgress;
			break;
		case EPackageState::SaveActive:
			Properties = EPackageStateProperty::InProgress | EPackageStateProperty::Saving;
			break;
		case EPackageState::SaveStalledRetracted:
			Properties = EPackageStateProperty::InProgress | EPackageStateProperty::Saving;
			break;
		case EPackageState::SaveStalledAssignedToWorker:
			Properties = EPackageStateProperty::InProgress | EPackageStateProperty::Saving
				| EPackageStateProperty::AssignedToWorkerProperty;
			break;
		default:
			check(false);
			Properties = EPackageStateProperty::None;
			break;
		}
	}
};

void FPackageData::SendToState(EPackageState NextState, ESendFlags SendFlags, EStateChangeReason ReleaseSaveReason)
{
	EPackageState OldState = GetState();
	switch (OldState)
	{
	case EPackageState::Idle:
		OnExitIdle();
		break;
	case EPackageState::Request:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetRequestQueue().Remove(this) == 1);
		}
		OnExitRequest();
		break;
	case EPackageState::AssignedToWorker:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetAssignedToWorkerSet().Remove(this) == 1);
		}
		OnExitAssignedToWorker();
		break;
	case EPackageState::Load:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetLoadQueue().Remove(this) == 1);
		}
		OnExitLoad();
		break;
	case EPackageState::SaveActive:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetSaveQueue().Remove(this) == 1);
		}
		OnExitSaveActive();
		break;
	case EPackageState::SaveStalledRetracted:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetSaveStalledSet().Remove(this) == 1);
		}
		OnExitSaveStalledRetracted();
		break;
	case EPackageState::SaveStalledAssignedToWorker:
		if (!!(SendFlags & ESendFlags::QueueRemove))
		{
			ensure(PackageDatas.GetSaveStalledSet().Remove(this) == 1);
		}
		OnExitSaveStalledAssignedToWorker();
		break;
	default:
		check(false);
		break;
	}

	FStateProperties OldProperties(OldState);
	FStateProperties NewProperties(NextState);
	// Exit state properties from highest to lowest; enter state properties from lowest to highest.
	// This ensures that properties that rely on earlier properties are constructed later and torn down earlier
	// than the earlier properties.
	for (EPackageStateProperty Iterator = EPackageStateProperty::Max;
		Iterator >= EPackageStateProperty::Min;
		Iterator = static_cast<EPackageStateProperty>(static_cast<uint32>(Iterator) >> 1))
	{
		if (((OldProperties.Properties & Iterator) != EPackageStateProperty::None) &
			((NewProperties.Properties & Iterator) == EPackageStateProperty::None))
		{
			switch (Iterator)
			{
			case EPackageStateProperty::InProgress:
				OnExitInProgress(ReleaseSaveReason);
				break;
			case EPackageStateProperty::Saving:
				OnExitSaving(ReleaseSaveReason, NextState);
				break;
			case EPackageStateProperty::AssignedToWorkerProperty:
				OnExitAssignedToWorkerProperty();
				break;
			default:
				check(false);
				break;
			}
		}
	}
	for (EPackageStateProperty Iterator = EPackageStateProperty::Min;
		Iterator <= EPackageStateProperty::Max;
		Iterator = static_cast<EPackageStateProperty>(static_cast<uint32>(Iterator) << 1))
	{
		if (((OldProperties.Properties & Iterator) == EPackageStateProperty::None) &
			((NewProperties.Properties & Iterator) != EPackageStateProperty::None))
		{
			switch (Iterator)
			{
			case EPackageStateProperty::InProgress:
				OnEnterInProgress();
				break;
			case EPackageStateProperty::Saving:
				OnEnterSaving();
				break;
			case EPackageStateProperty::AssignedToWorkerProperty:
				OnEnterAssignedToWorkerProperty();
				break;
			default:
				check(false);
				break;
			}
		}
	}

	if (NextState != EPackageState::Idle && NextState != EPackageState::Request)
	{
		// Add more diagnostics for an unexpected edge case we have seen: retracted packages already have been
		// committed. In-progress packages after the request state have a contract that they have a platform
		// needing commit; give a warning if this contract is broken during a state transition.
		int32 NumNeedCommitPlatforms = 0;
		ECookPhase CookPhase = PackageDatas.GetCookOnTheFlyServer().GetCookPhase();
		for (const TPair<const ITargetPlatform*, FPackagePlatformData>& PlatformPair : GetPlatformDatas())
		{
			if (PlatformPair.Key == CookerLoadingPlatformKey)
			{
				continue;
			}
			if (PlatformPair.Value.NeedsCommit(PlatformPair.Key, CookPhase))
			{
				++NumNeedCommitPlatforms;
			}
		}
		if (NumNeedCommitPlatforms == 0)
		{
			TStringBuilder<256> PlatformDataText;
			for (const TPair<const ITargetPlatform*, FPackagePlatformData>& PlatformPair : GetPlatformDatas())
			{
				PlatformDataText.Appendf(TEXT("{ %s: Reachable=%s, Committed=%s }, "),
					PlatformPair.Key == CookerLoadingPlatformKey ? TEXT("CookerLoadingPlatform") : *PlatformPair.Key->PlatformName(),
					PlatformPair.Value.IsReachable(CookPhase == ECookPhase::Cook
						? EReachability::Runtime : EReachability::Build) ? TEXT("true") : TEXT("false"),
					PlatformPair.Value.IsCommitted() ? TEXT("true") : TEXT("false"));
			}
			UE_LOGF(LogCook, Warning,
				"InternalCookerError: Package %ls is entering InProgress state %ls, but already has no platforms needing commit. CookPhase = %ls. [ %ls ]",
				*GetPackageName().ToString(), LexToString(NextState), LexToString(CookPhase), *PlatformDataText);
			UE_CALL_ONCE([]()
				{
					FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
				});
		}
	}

	SetState(NextState);
	switch (NextState)
	{
	case EPackageState::Idle:
		OnEnterIdle();
		break;
	case EPackageState::Request:
		OnEnterRequest();
		if (((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone))
		{
			PackageDatas.GetRequestQueue().AddRequest(this);
		}
		break;
	case EPackageState::AssignedToWorker:
		OnEnterAssignedToWorker();
		if (((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone))
		{
			PackageDatas.GetAssignedToWorkerSet().Add(this);
		}
		break;
	case EPackageState::Load:
		OnEnterLoad();
		if ((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone)
		{
			PackageDatas.GetLoadQueue().Add(this);
		}
		break;
	case EPackageState::SaveActive:
		OnEnterSaveActive();
		if (((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone))
		{
			if (GetUrgency() > EUrgency::Normal)
			{
				PackageDatas.GetSaveQueue().AddFront(this);
			}
			else
			{
				PackageDatas.GetSaveQueue().Add(this);
			}
		}
		break;
	case EPackageState::SaveStalledRetracted:
		OnEnterSaveStalledRetracted();
		if (((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone))
		{
			PackageDatas.GetSaveStalledSet().Add(this);
		}
		break;
	case EPackageState::SaveStalledAssignedToWorker:
		OnEnterSaveStalledAssignedToWorker();
		if (((SendFlags & ESendFlags::QueueAdd) != ESendFlags::QueueNone))
		{
			PackageDatas.GetSaveStalledSet().Add(this);
		}
		break;
	default:
		check(false);
		break;
	}

	PackageDatas.GetMonitor().OnStateChanged(*this, OldState);
}

void FPackageData::UpdateContainerUrgency(EUrgency OldUrgency, EUrgency NewUrgency)
{
	switch (GetState())
	{
	case EPackageState::Idle:
		// Urgency does not affect behavior in the Idle state
		break;
	case EPackageState::Request:
		PackageDatas.GetRequestQueue().UpdateUrgency(this, OldUrgency, NewUrgency);
		break;
	case EPackageState::AssignedToWorker:
		// Urgency does not affect behavior in the AssignedToWorker state
		break;
	case EPackageState::Load:
		PackageDatas.GetLoadQueue().UpdateUrgency(this, OldUrgency, NewUrgency);
		break;
	case EPackageState::SaveActive:
		if (NewUrgency > EUrgency::Normal)
		{
			FPackageDataQueue& Queue = PackageDatas.GetSaveQueue();
			if (Queue.Remove(this) > 0)
			{
				Queue.AddFront(this);
			}
		}
		break;
	case EPackageState::SaveStalledRetracted:
		// Urgency does not affect behavior in stalled states
		break;
	case EPackageState::SaveStalledAssignedToWorker:
		// Urgency does not affect behavior in stalled states
		break;
	default:
		check(false);
		break;
	}

	// The Package preloader can be active in any state, and is contained in the LoadQueue.
	// If it exists and we did not already call UpdateUrgency on the LoadQueue, then call it.
	if (GetState() != EPackageState::Load && GetPackagePreloader())
	{
		PackageDatas.GetLoadQueue().UpdateUrgency(this, OldUrgency, NewUrgency);
	}
}

void FPackageData::Stall(EPackageState TargetState, ESendFlags SendFlags)
{
	switch (TargetState)
	{
	case EPackageState::SaveStalledAssignedToWorker:
	case EPackageState::SaveStalledRetracted:
		if (GetState() != EPackageState::SaveActive)
		{
			return;
		}
		break;
	default:
		return;
	}

	SendToState(TargetState, SendFlags, EStateChangeReason::Retraction);
}

void FPackageData::UnStall(ESendFlags SendFlags)
{
	EPackageState TargetState = EPackageState::Idle;

	switch (GetState())
	{
	case EPackageState::SaveStalledAssignedToWorker:
	case EPackageState::SaveStalledRetracted:
		TargetState = EPackageState::SaveActive;
		break;
	default:
		return;
	}

	UE_LOGF(LogCook, Display, "Unstalling package %ls; it will resume saving from the point at which it was retracted.",
		*WriteToString<256>(GetPackageName()));
	SendToState(TargetState, SendFlags, EStateChangeReason::Retraction);
}

bool FPackageData::IsStalled() const
{
	switch (GetState())
	{
	case EPackageState::SaveStalledAssignedToWorker:
	case EPackageState::SaveStalledRetracted:
		return true;
	default:
		return false;
	}
}

void FPackageData::CheckInContainer() const
{
	switch (GetState())
	{
	case EPackageState::Idle:
		break;
	case EPackageState::Request:
		check(PackageDatas.GetRequestQueue().Contains(this));
		break;
	case EPackageState::AssignedToWorker:
		check(PackageDatas.GetAssignedToWorkerSet().Contains(this));
		break;
	case EPackageState::Load:
		check(PackageDatas.GetLoadQueue().Contains(this));
		break;
	case EPackageState::SaveActive:
		// The save queue is huge and often pushed at end. Check last element first and then scan.
		check(PackageDatas.GetSaveQueue().Num() && (PackageDatas.GetSaveQueue().Last() == this
			|| Algo::Find(PackageDatas.GetSaveQueue(), this)));
		break;
	case EPackageState::SaveStalledRetracted:
		check(PackageDatas.GetSaveStalledSet().Contains(this));
		break;
	case EPackageState::SaveStalledAssignedToWorker:
		check(PackageDatas.GetSaveStalledSet().Contains(this));
		break;
	default:
		check(false);
		break;
	}
}

bool FPackageData::IsInProgress() const
{
	return IsInStateProperty(EPackageStateProperty::InProgress);
}

bool FPackageData::IsInStateProperty(EPackageStateProperty Property) const
{
	return (FStateProperties(GetState()).Properties & Property) != EPackageStateProperty::None;
}

void FPackageData::OnEnterIdle()
{
	// Note that this might be on construction of the PackageData
}

void FPackageData::OnExitIdle()
{
}

void FPackageData::OnEnterRequest()
{
}

void FPackageData::OnExitRequest()
{
}

void FPackageData::OnEnterAssignedToWorker()
{
	if (IsGenerated())
	{
		// Clear the referencecount that we added in OnEnterInProgress; we don't want to keep the GenerationHelper
		// referenced for the entire duration of assigned packages running on other CookWorkers. If this package gets
		// retracted and moved into LoadState locally, we will recreate the GenerationHelper if necessary.
		// Since we have set the ParentGenerationHelper to null, we can no automatically longer report to the
		// GenerationHelper that the package has saved when it transitions to Idle. Reporting to the GenerationHelper
		// that this FPackageData has saved is now the responsibility of the CookWorkerServer's RecordResults function.
		SetParentGenerationHelper(nullptr, EStateChangeReason::Retraction);
	}
}

void FPackageData::SetWorkerAssignment(FWorkerId InWorkerAssignment, ESendFlags SendFlags)
{
	if (WorkerAssignment.IsValid())
	{
		checkf(InWorkerAssignment.IsInvalid(),
			TEXT("Package %s is being assigned to worker %d while it is already assigned to worker %d."),
			*GetPackageName().ToString(), WorkerAssignment.GetRemoteIndex(), WorkerAssignment.GetRemoteIndex());
		if (EnumHasAnyFlags(SendFlags, ESendFlags::QueueRemove))
		{
			PackageDatas.GetCookOnTheFlyServer().NotifyRemovedFromWorker(*this);
		}
		WorkerAssignment = FWorkerId::Invalid();
	}
	else
	{
		if (InWorkerAssignment.IsValid())
		{
			checkf(IsInStateProperty(EPackageStateProperty::AssignedToWorkerProperty),
				TEXT("Package %s is being assigned to worker %d while in state %s, which is not an AssignedToWorker state. This is invalid."),
				*GetPackageName().ToString(), GetWorkerAssignment().GetRemoteIndex(), LexToString(GetState()));
		}
		WorkerAssignment = InWorkerAssignment;
	}
}

void FPackageData::OnExitAssignedToWorker()
{
}

void FPackageData::OnEnterLoad()
{
	TRefCountPtr<FPackagePreloader> Local = CreatePackagePreloader();
	Local->SetSelfReference();
	check(PackagePreloader);
}

void FPackageData::OnExitLoad()
{
	check(PackagePreloader); // Guaranteed by OnEnterLoad
	PackagePreloader->OnPackageLeaveLoadState();
	PackagePreloader->ClearSelfReference();
	// PackagePreloader might now be nullptr
}

void FPackageData::OnEnterSaveActive()
{
}

void FPackageData::OnExitSaveActive()
{
}

void FPackageData::OnEnterSaveStalledRetracted()
{
}

void FPackageData::OnExitSaveStalledRetracted()
{
}

void FPackageData::OnEnterSaveStalledAssignedToWorker()
{
}

void FPackageData::OnExitSaveStalledAssignedToWorker()
{
}

void FPackageData::OnEnterInProgress()
{
	PackageDatas.GetMonitor().OnInProgressChanged(*this, true);
	if (IsGenerated())
	{
		// Keep a refcount to the ParentGenerationHelper until we are saved so that it does not destruct
		// and waste time reconstructing when we reach the LoadQueue.
		GetOrFindParentGenerationHelper();
	}
}

void FPackageData::OnExitInProgress(EStateChangeReason StateChangeReason)
{
	PackageDatas.GetMonitor().OnInProgressChanged(*this, false);
	UE::Cook::FCompletionCallback LocalCompletionCallback(MoveTemp(GetCompletionCallback()));
	if (LocalCompletionCallback)
	{
		LocalCompletionCallback(this);
	}
	ClearInProgressData(StateChangeReason);
}

void FPackageData::OnEnterSaving()
{
	check(GetPackage() != nullptr && GetPackage()->IsFullyLoaded());
	check(GetLoadDependencies() != nullptr);

	check(!HasPrepareSaveFailed());
	CheckObjectCacheEmpty();
	CheckCookedPlatformDataEmpty();
}

void FPackageData::OnExitSaving(EStateChangeReason ReleaseSaveReason, EPackageState NewState)
{
	PackageDatas.GetCookOnTheFlyServer().ReleaseCookedPlatformData(*this, ReleaseSaveReason, NewState);
	ClearObjectCache();
	SetHasPrepareSaveFailed(false);
	SetIsPrepareSaveRequiresGC(false);
	SetPackage(nullptr);
}

void FPackageData::OnPackageDataFirstMarkedReachable(EReachability InReachability, FInstigator&& InInstigator)
{
	if (InReachability == EReachability::Runtime)
	{
		TracePackage(GetPackageName().ToUnstableInt(), GetPackageName().ToString());
		Instigator = MoveTemp(InInstigator);
		PackageDatas.DebugInstigator(*this);
		PackageDatas.UpdateThreadsafePackageData(*this);
	}
	else
	{
		check(InReachability == EReachability::Build);
		BuildInstigator = MoveTemp(InInstigator);
	}
}

void FPackageData::OnEnterAssignedToWorkerProperty()
{
}

void FPackageData::OnExitAssignedToWorkerProperty()
{
	SetWorkerAssignment(FWorkerId::Invalid());
}

void FPackageData::SetState(EPackageState NextState)
{
	State = static_cast<uint32>(NextState);
}

FCompletionCallback& FPackageData::GetCompletionCallback()
{
	return CompletionCallback;
}

void FPackageData::AddCompletionCallback(TConstArrayView<const ITargetPlatform*> TargetPlatforms, 
	FCompletionCallback&& InCompletionCallback)
{
	if (!InCompletionCallback)
	{
		return;
	}

	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		FPackagePlatformData* PlatformData = FindPlatformData(TargetPlatform);
		// Adding a completion callback is only allowed after marking the requested platforms as runtime reachable
		check(PlatformData);
		check(PlatformData->IsReachable(EReachability::Runtime));
		// Adding a completion callback is only allowed after putting the PackageData in progress.
		// If it's not in progress because it already finished the desired platforms, that is allowed.
		check(IsInProgress() || PlatformData->IsCookAttempted() || !PlatformData->IsCookable());
	}

	if (IsInProgress())
	{
		// We don't yet have a mechanism for calling two completion callbacks.
		// CompletionCallbacks only come from external requests, and it should not be possible to request twice,
		// so a failed check here shouldn't happen.
		check(!CompletionCallback);
		CompletionCallback = MoveTemp(InCompletionCallback);
	}
	else
	{
		// Already done; call the completioncallback immediately
		InCompletionCallback(this);
	}
}

TRefCountPtr<FPackagePreloader> FPackageData::GetPackagePreloader() const
{
	return TRefCountPtr<FPackagePreloader>(PackagePreloader);
}

TRefCountPtr<FPackagePreloader> FPackageData::CreatePackagePreloader()
{
	if (PackagePreloader)
	{
		return TRefCountPtr<FPackagePreloader>(PackagePreloader);
	}
	TRefCountPtr<FPackagePreloader> Result(new FPackagePreloader(*this));
	PackagePreloader = Result.GetReference();
	return Result;
}

void FPackageData::OnPackagePreloaderDestroyed(FPackagePreloader& InPackagePreloader)
{
	check(PackagePreloader == &InPackagePreloader);
	PackagePreloader = nullptr;
}

const FBuildResultDependenciesMap* FPackageData::GetLoadDependencies() const
{
	return LoadDependencies.Get();
}

void FPackageData::CreateLoadDependencies()
{
	if (!LoadDependencies)
	{
		UPackage* LocalPackage = Package.Get();
		checkf(LocalPackage != nullptr,
			TEXT("CreateLoadDependencies failed for package %s because this->Package == nullptr. It can only be called after the Package has been set."),
			*GetPackageName().ToString());
		LoadDependencies.Reset(new FBuildResultDependenciesMap(FBuildDependencySet::CollectLoadedPackage(LocalPackage)));
	}
}

void FPackageData::ClearLoadDependencies()
{
	LoadDependencies.Reset();
}

TArray<FCachedObjectInOuter>& FPackageData::GetCachedObjectsInOuter()
{
	return CachedObjectsInOuter;
}

const TArray<FCachedObjectInOuter>& FPackageData::GetCachedObjectsInOuter() const
{
	return CachedObjectsInOuter;
}

void FPackageData::CheckObjectCacheEmpty() const
{
	check(CachedObjectsInOuter.Num() == 0);
	check(!GetHasSaveCache());
}

void FPackageData::CreateObjectCache()
{
	if (GetHasSaveCache())
	{
		return;
	}

	UPackage* LocalPackage = GetPackage();
	if (LocalPackage && LocalPackage->IsFullyLoaded())
	{
		CreateObjectCache(CachedObjectsInOuter, LocalPackage);

		for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
		{
			FPackagePlatformData& PlatformData = Pair.Value;
			check(!PlatformData.IsRegisteredForCachedObjectsInOuter());
			if (PlatformData.NeedsCooking(Pair.Key))
			{
				PlatformData.SetRegisteredForCachedObjectsInOuter(true);
			}
		}

		SetHasSaveCache(true);
	}
	else
	{
		check(false);
	}
}

void FPackageData::CreateObjectCache(TArray<FCachedObjectInOuter>& OutBuffer, UPackage* InPackage)
{
	TArray<UObject*> ObjectsInOuter;
	// Ignore garbage objects; they will not be serialized out so we don't need to call
	// BeginCacheForCookedPlatformData on them
	GetObjectsWithOuter(InPackage, ObjectsInOuter, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags,
		EInternalObjectFlags::Garbage);
	OutBuffer.Reset(ObjectsInOuter.Num());
	for (UObject* Object : ObjectsInOuter)
	{
		FWeakObjectPtr ObjectWeakPointer(Object);
		// GetObjectsWithOuter with Garbage filtered out should only return valid-for-weakptr objects
		check(ObjectWeakPointer.Get());
		OutBuffer.Emplace(ObjectWeakPointer);
	}
}

static TArray<UObject*> SetDifference(TArray<UObject*>& A, TArray<UObject*>& B)
{
	Algo::Sort(A); // Don't use TArray.Sort, it sorts pointers as references and we want to sort them as pointers
	Algo::Sort(B);
	int32 ANum = A.Num();
	int32 BNum = B.Num();
	UObject** AData = A.GetData();
	UObject** BData = B.GetData();

	// Always move to the smallest next element from the two remaining lists and if it's in one set and not the
	// other add it to the output if in A or skip it if in B.
	int32 AIndex = 0;
	int32 BIndex = 0;
	TArray<UObject*> AMinusB;
	while (AIndex < ANum && BIndex < BNum)
	{
		if (AData[AIndex] == BData[BIndex])
		{
			++AIndex;
			++BIndex;
			continue;
		}
		if (AData[AIndex] < BData[BIndex])
		{
			AMinusB.Add(AData[AIndex++]);
		}
		else
		{
			++BIndex;
		}
	}

	// When we reach the end of B, all remaining elements of A are not in B.
	while (AIndex < ANum)
	{
		AMinusB.Add(AData[AIndex++]);
	}
	return AMinusB;
}

EPollStatus FPackageData::RefreshObjectCache(bool& bOutFoundNewObjects)
{
	check(Package.Get() != nullptr);

	TArray<UObject*> OldObjects;
	OldObjects.Reserve(CachedObjectsInOuter.Num());
	for (FCachedObjectInOuter& Object : CachedObjectsInOuter)
	{
		UObject* ObjectPtr = Object.Object.Get();
		if (ObjectPtr)
		{
			OldObjects.Add(ObjectPtr);
		}
	}
	TArray<UObject*> CurrentObjects;
	GetObjectsWithOuter(Package.Get(), CurrentObjects, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags,
		EInternalObjectFlags::Garbage);

	// We add any new objects but do not remove any objects moved out of the package, and do not remove any objects
	// deleted by garbage collection. FCookGenerationHelper relies on being able to store objects in the generator's
	// object cache that it needs to call BeginCache/IsCached on but that might be in other packages such as
	// ExternalActor packages that own the object on disk or generated packages the object has been moved into.
	// Objects deleted by garbage collection are detected elsewhere and cause control flow decisions.
	TArray<UObject*> NewObjects = SetDifference(CurrentObjects, OldObjects);
	bOutFoundNewObjects = NewObjects.Num() > 0;
	if (bOutFoundNewObjects)
	{
		CachedObjectsInOuter.Reserve(CachedObjectsInOuter.Num() + NewObjects.Num());
		for (UObject* Object : NewObjects)
		{
			FWeakObjectPtr ObjectWeakPointer(Object);
			// GetObjectsWithOuter with Garbage filtered out should only return valid-for-weakptr objects
			check(ObjectWeakPointer.Get());
			CachedObjectsInOuter.Emplace(MoveTemp(ObjectWeakPointer));
		}
		// GetCookedPlatformDataNextIndex is already where it should be, pointing at the first of the objects we have
		// added. Caller is responsible for changing state back to calling BeginCacheForCookedPlatformData.

		if (++GetNumRetriesBeginCacheOnObjects() > FPackageData::GetMaxNumRetriesBeginCacheOnObjects())
		{
			UE_LOGF(LogCook, Error,
				"Cooker has repeatedly tried to call BeginCacheForCookedPlatformData on all objects in the package, but keeps finding new objects.\n" "Aborting the save of the package; programmer needs to debug why objects keep getting added to the package.\n" "Package: %ls. Most recent created object: %ls.",
				*GetPackageName().ToString(), *NewObjects[0]->GetFullName());
			return EPollStatus::Error;
		}
	}
	return EPollStatus::Success;
}

void FPackageData::ClearObjectCache()
{
	// Note we do not need to remove objects in CachedObjectsInOuter from CachedCookedPlatformDataObjects
	// That removal is handled by ReleaseCookedPlatformData, and the caller is responsible for calling
	// ReleaseCookedPlatformData before calling ClearObjectCache
	CachedObjectsInOuter.Empty();
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		Pair.Value.SetRegisteredForCachedObjectsInOuter(false);
	}
	SetHasSaveCache(false);
}

const int32& FPackageData::GetNumPendingCookedPlatformData() const
{
	return NumPendingCookedPlatformData;
}

int32& FPackageData::GetNumPendingCookedPlatformData()
{
	return NumPendingCookedPlatformData;
}

const int32& FPackageData::GetCookedPlatformDataNextIndex() const
{
	return CookedPlatformDataNextIndex;
}

int32& FPackageData::GetCookedPlatformDataNextIndex()
{
	return CookedPlatformDataNextIndex;
}

int32& FPackageData::GetNumRetriesBeginCacheOnObjects()
{
	return NumRetriesBeginCacheOnObject;
}

int32 FPackageData::GetMaxNumRetriesBeginCacheOnObjects()
{
	return 10;
}

void FPackageData::SetSaveSubState(ESaveSubState Value)
{
	if (Value != ESaveSubState::StartSave && !IsInStateProperty(EPackageStateProperty::Saving))
	{
		UE_LOGF(LogCook, Error, "SetSaveSubState(%ls) called from invalid PackageState %ls. The call will be ignored",
			LexToString(Value), LexToString(GetState()));
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	SaveSubState = static_cast<uint32>(Value);
}

void FPackageData::SetSaveSubStateComplete(ESaveSubState Value)
{
	if (Value < ESaveSubState::Last)
	{
		Value = static_cast<ESaveSubState>(static_cast<uint32>(Value) + 1);
	}
	SetSaveSubState(Value);
}

void FPackageData::CheckCookedPlatformDataEmpty() const
{
	check(GetCookedPlatformDataNextIndex() <= 0);
	check(GetSaveSubState() <= ESaveSubState::StartSave);
}

void FPackageData::ClearCookedPlatformData()
{
	CookedPlatformDataNextIndex = -1;
	NumRetriesBeginCacheOnObject = 0;
	// Note that GetNumPendingCookedPlatformData is not cleared; it persists across Saves and CookSessions
	// Caller is responsible for calling SetSaveSubState(ESaveSubState::StartSave);
}

void FPackageData::OnRemoveSessionPlatform(const ITargetPlatform* Platform)
{
	PlatformDatas.Remove(Platform);
	if (DiscoveredDependencies)
	{
		DiscoveredDependencies->Remove(Platform);
	}
}

bool FPackageData::HasReferencedObjects() const
{
	return Package != nullptr || CachedObjectsInOuter.Num() > 0;
}

void FPackageData::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	typedef TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>> MapType;
	MapType NewPlatformDatas;
	NewPlatformDatas.Reserve(PlatformDatas.Num());
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& ExistingPair : PlatformDatas)
	{
		ITargetPlatform*const* NewKeyPtr = Remap.Find(ExistingPair.Key);
		ITargetPlatform* NewKey = NewKeyPtr ? *NewKeyPtr : const_cast<ITargetPlatform*>(ExistingPair.Key);
		check(NewKey);
		NewPlatformDatas.FindOrAdd(NewKey) = MoveTemp(ExistingPair.Value);

		if (DiscoveredDependencies)
		{
			TMap<FPackageData*, EInstigator>* OldValue = DiscoveredDependencies->Find(ExistingPair.Key);
			if (OldValue)
			{
				TMap<FPackageData*, EInstigator> MovedValue = MoveTemp(*OldValue);
				DiscoveredDependencies->Remove(ExistingPair.Key);
				DiscoveredDependencies->Add(NewKey, MoveTemp(MovedValue));
			}
		}
	}

	// The save state (and maybe more in the future) by contract can depend on the order of the request platforms
	// remaining unchanged. If we change that order due to the remap, we need to demote back to request.
	if (IsInProgress() && GetState() != EPackageState::Request)
	{
		bool bDemote = false;
		MapType::TConstIterator OldIter = PlatformDatas.CreateConstIterator();
		MapType::TConstIterator NewIter = NewPlatformDatas.CreateConstIterator();
		for (; OldIter; ++OldIter, ++NewIter)
		{
			if (OldIter.Key() != NewIter.Key())
			{
				bDemote = true;
			}
		}
		if (bDemote)
		{
			SendToState(EPackageState::Request, ESendFlags::QueueAddAndRemove, EStateChangeReason::ForceRecook);
		}
	}
	PlatformDatas = MoveTemp(NewPlatformDatas);
}

void FPackageData::UpdateSaveAfterGarbageCollect(bool& bOutDemote)
{
	bOutDemote = false;
	if (!IsInStateProperty(EPackageStateProperty::Saving))
	{
		return;
	}

	// Reexecute PrepareSave if we already completed it; we need to refresh our CachedObjectsInOuter list
	// and call BeginCacheOnCookedPlatformData on any new objects.
	if (GetSaveSubState() >= ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded)
	{
		SetSaveSubState(ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded);
	}

	if (GetPackage() == nullptr || !GetPackage()->IsFullyLoaded())
	{
		bOutDemote = true;
	}
	else
	{
		for (FCachedObjectInOuter& CachedObjectInOuter : CachedObjectsInOuter)
		{
			if (CachedObjectInOuter.Object.Get() == nullptr)
			{
				// Deleting a public object puts the package in an invalid state; demote back to request
				// and load/save it again
				bool bPublicDeleted = !!(CachedObjectInOuter.ObjectFlags & RF_Public);
				bOutDemote |= bPublicDeleted;
			}
		}
	}

	if (GenerationHelper)
	{
		GenerationHelper->UpdateSaveAfterGarbageCollect(*this, bOutDemote);
	}
	else if (IsGenerated())
	{
		if (!ParentGenerationHelper)
		{
			bOutDemote = true;
		}
		else
		{
			ParentGenerationHelper->UpdateSaveAfterGarbageCollect(*this, bOutDemote);
		}
	}
}

TRefCountPtr<UE::Cook::FGenerationHelper> FPackageData::GetGenerationHelper() const
{
	return GenerationHelper;
}

void FPackageData::SetGenerated(FName InParentGenerator)
{
	bGenerated = true;
	ParentGenerator = InParentGenerator;
}

TRefCountPtr<FGenerationHelper> FPackageData::GetParentGenerationHelper() const
{
	return ParentGenerationHelper;
}

void FPackageData::SetParentGenerationHelper(FGenerationHelper* InGenerationHelper,
	EStateChangeReason StateChangeReason, FCookGenerationInfo* InfoOfPackageInGenerator)
{
	check(InGenerationHelper == nullptr || IsGenerated());
	check(!(ParentGenerationHelper && InGenerationHelper) || ParentGenerationHelper == InGenerationHelper);

	if (ParentGenerationHelper && !InGenerationHelper && IsTerminalStateChange(StateChangeReason))
	{
		// The package's progress is completed and we will not come back to it; report the package was saved.
		ParentGenerationHelper->SetAllPlatformsSaved(*this, InfoOfPackageInGenerator);
	}
	ParentGenerationHelper = InGenerationHelper;
}

TRefCountPtr<FGenerationHelper> FPackageData::GetOrFindParentGenerationHelper()
{
	if (ParentGenerationHelper)
	{
		return ParentGenerationHelper;
	}
	if (!IsGenerated())
	{
		return nullptr;
	}

	FPackageData* OwnerPackageData = PackageDatas.FindPackageDataByPackageName(GetParentGenerator());
	if (!OwnerPackageData)
	{
		return nullptr;
	}

	SetParentGenerationHelper(OwnerPackageData->GetGenerationHelper(), EStateChangeReason::Requested);
	return ParentGenerationHelper;
}

TRefCountPtr<FGenerationHelper> FPackageData::GetOrFindParentGenerationHelperNoCache()
{
	if (ParentGenerationHelper)
	{
		return ParentGenerationHelper;
	}
	if (!IsGenerated())
	{
		return nullptr;
	}

	FPackageData* OwnerPackageData = PackageDatas.FindPackageDataByPackageName(GetParentGenerator());
	if (!OwnerPackageData)
	{
		return nullptr;
	}

	return OwnerPackageData->GetGenerationHelper();
}

TRefCountPtr<FGenerationHelper> FPackageData::TryCreateValidParentGenerationHelper()
{
	if (ParentGenerationHelper)
	{
		if (!ParentGenerationHelper->IsValid())
		{
			SetParentGenerationHelper(nullptr, EStateChangeReason::Requested);
		}
		return ParentGenerationHelper;
	}
	if (!IsGenerated())
	{
		return nullptr;
	}

	FPackageData* OwnerPackageData = PackageDatas.FindPackageDataByPackageName(GetParentGenerator());
	if (!OwnerPackageData)
	{
		return nullptr;
	}

	// MPCOOKTODO: We need to support calling BeginCacheForCookedPlatformData/IsCachedCookedPlatformData
	// on all objects in the generator package if they have not already been called, if 
	// RequiresCachedCookedPlatformDataBeforeSplit. For now we workaround our inability to do this
	// by forcing EGeneratedRequiresGenerator::Save.
	constexpr bool bCookedPlatformDataIsLoaded = true;
	bool bNeedWaitForIsLoaded;
	ParentGenerationHelper = OwnerPackageData->TryCreateValidGenerationHelper(bCookedPlatformDataIsLoaded,
		bNeedWaitForIsLoaded);
	check(ParentGenerationHelper.IsValid() || !bNeedWaitForIsLoaded);

	return ParentGenerationHelper;
}

TRefCountPtr<FGenerationHelper> FPackageData::CreateUninitializedGenerationHelper()
{
	if (GenerationHelper)
	{
		return GenerationHelper;
	}
	TRefCountPtr<UE::Cook::FGenerationHelper> Result = new UE::Cook::FGenerationHelper(*this);
	GenerationHelper = Result.GetReference();
	return Result;
}

TRefCountPtr<UE::Cook::FGenerationHelper> FPackageData::TryCreateValidGenerationHelper(
	bool bCookedPlatformDataIsLoaded, bool& bOutNeedWaitForIsLoaded)
{
	bOutNeedWaitForIsLoaded = false;

	if (GenerationHelper && GenerationHelper->IsInitialized())
	{
		if (!GenerationHelper->IsValid())
		{
			// The GenerationHelper is not valid; we can get here if it was created from incremental cook data but this
			// package is no longer a generator after syncing. If it has any self-references, clear them so that it
			// will delete and this non-generator package will set the usual GenerationHelper=nullptr value.
			GenerationHelper->ClearSelfReferences(); // Might set GenerationHelper=nullptr
			// this->GenerationHelper might still be non-null, if there are some generated packages
			// that still have a pointer to it. This will only happen in error-handling edge cases, but we
			// need to check for invalid GenerationHelper everwhere we use them to cover this case.
			// Our contract for TryCreateValidGenerationHelper this case is we return nullptr.
			return nullptr;
		}
		return GenerationHelper;
	}

	UCookOnTheFlyServer& COTFS = PackageDatas.GetCookOnTheFlyServer();
	UE::Cook::Private::FRegisteredCookPackageSplitter* RegisteredSplitterType = nullptr;
	TUniquePtr<ICookPackageSplitter> CookPackageSplitterInstance;
	UObject* SplitDataObject = nullptr;
	UPackage* LocalPackage = GetPackage();
	if (!LocalPackage)
	{
		LocalPackage = FGenerationHelper::FindOrLoadPackage(COTFS, *this);
	}
	if (LocalPackage)
	{
		TOptional<TConstArrayView<FCachedObjectInOuter>> LocalCachedObjectsInOuter;
		if (GetHasSaveCache())
		{
			LocalCachedObjectsInOuter.Emplace(GetCachedObjectsInOuter());
		}
		FGenerationHelper::SearchForRegisteredSplitDataObject(COTFS, GetPackageName(),
			LocalPackage, LocalCachedObjectsInOuter, SplitDataObject, RegisteredSplitterType,
			CookPackageSplitterInstance, bCookedPlatformDataIsLoaded, bOutNeedWaitForIsLoaded);
	}
	TRefCountPtr<UE::Cook::FGenerationHelper> Result = GenerationHelper;
	if (!SplitDataObject || !CookPackageSplitterInstance)
	{
		if (Result)
		{
			// Mark that GenerationHelper is invalid, and clear its references and return nullptr; see comment above.
			Result->InitializeAsInvalid(); // cannot set GenerationHelper=nullptr because we have a local refcount.
			Result->ClearSelfReferences();
		}
		return nullptr;
	}
	else
	{
		if (!Result)
		{
			Result = new UE::Cook::FGenerationHelper(*this);
			GenerationHelper = Result.GetReference();
		}

		Result->Initialize(SplitDataObject, RegisteredSplitterType, MoveTemp(CookPackageSplitterInstance));
		return Result;
	}
}

TRefCountPtr<UE::Cook::FGenerationHelper> FPackageData::GetGenerationHelperIfValid()
{
	if (GenerationHelper && GenerationHelper->IsValid())
	{
		return GenerationHelper;
	}
	return nullptr;
}

void FPackageData::OnGenerationHelperDestroyed(FGenerationHelper& InGenerationHelper)
{
	check(GenerationHelper == &InGenerationHelper);
	GenerationHelper = nullptr;
}

FConstructPackageData FPackageData::CreateConstructData()
{
	FConstructPackageData ConstructData;
	ConstructData.PackageName = PackageName;
	ConstructData.NormalizedFileName = FileName;
	return ConstructData;
}

void FPackageData::AddDiscoveredDependency(const FDiscoveredPlatformSet& Platforms, FPackageData* Dependency,
	EInstigator Category)
{
	TConstArrayView<const ITargetPlatform*> PlatformArray;
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> BufferPlatforms;

	if (Platforms.GetSource() == EDiscoveredPlatformSet::CopyFromInstigator)
	{
		BufferPlatforms.Add(nullptr); // PlatformAgnostic platform
		PlatformArray = BufferPlatforms;
	}
	else
	{
		UCookOnTheFlyServer& COTFS = PackageDatas.GetCookOnTheFlyServer();
		PlatformArray = Platforms.GetPlatforms(COTFS, nullptr,
			TConstArrayView<const ITargetPlatform*>(), EReachability::Runtime, BufferPlatforms);
		if (PlatformArray.Num() == COTFS.PlatformManager->GetSessionPlatforms().Num())
		{
			BufferPlatforms.SetNum(1, EAllowShrinking::No);
			BufferPlatforms[0] = nullptr; // PlatformAgnostic platform
			PlatformArray = BufferPlatforms;
		}
	}

	if (!DiscoveredDependencies)
	{
		DiscoveredDependencies = MakeUnique<TMap<const ITargetPlatform*, TMap<FPackageData*, EInstigator>>>();
	}

	for (const ITargetPlatform* TargetPlatform : PlatformArray)
	{
		TMap<FPackageData*, EInstigator>& PlatformDependencies = DiscoveredDependencies->FindOrAdd(TargetPlatform);
		EInstigator& ExistingEdgeType = PlatformDependencies.FindOrAdd(Dependency, Category);

		// Overwrite the previous edge type with the new edge type if the new edge type is higher priority.
		if (Category == EInstigator::ForceExplorableSaveTimeSoftDependency)
		{
			ExistingEdgeType = Category;
		}
	}
}

void FPackageData::ClearDiscoveredDependencies()
{
	DiscoveredDependencies.Reset();
}

TMap<FPackageData*, EInstigator>& FPackageData::CreateOrGetDiscoveredDependencies(const ITargetPlatform* TargetPlatform)
{
	if (!DiscoveredDependencies)
	{
		DiscoveredDependencies = MakeUnique<TMap<const ITargetPlatform*, TMap<FPackageData*, EInstigator>>>();
	}
	return DiscoveredDependencies->FindOrAdd(TargetPlatform);
}

TMap<FPackageData*, EInstigator>* FPackageData::GetDiscoveredDependencies(const ITargetPlatform* TargetPlatform)
{
	if (!DiscoveredDependencies)
	{
		return nullptr;
	}
	return DiscoveredDependencies->Find(TargetPlatform);
}

void FPackageData::AddLogMessage(FReplicatedLogData&& LogData)
{
	if (!LogMessages)
	{
		LogMessages.Reset(new TArray<FReplicatedLogData>());
	}
	LogMessages->Add(MoveTemp(LogData));
}

TConstArrayView<FReplicatedLogData> FPackageData::GetLogMessages() const
{
	if (!LogMessages)
	{
		return TConstArrayView<FReplicatedLogData>();
	}
	return *LogMessages;
}

void FPackageData::ClearLogMessages()
{
	LogMessages.Reset();
}

} // namespace UE::Cook

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FConstructPackageData& PackageData)
{
	Writer.BeginObject();
	Writer << "P" << PackageData.PackageName;
	Writer << "F" << PackageData.NormalizedFileName;
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FConstructPackageData& PackageData)
{
	LoadFromCompactBinary(Field["P"], PackageData.PackageName);
	LoadFromCompactBinary(Field["F"], PackageData.NormalizedFileName);
	return !PackageData.PackageName.IsNone() && !PackageData.NormalizedFileName.IsNone();
}

namespace UE::Cook
{

//////////////////////////////////////////////////////////////////////////
// FPendingCookedPlatformData


FPendingCookedPlatformData::FPendingCookedPlatformData(UObject* InObject, const ITargetPlatform* InTargetPlatform,
	FPackageData& InPackageData, bool bInNeedsResourceRelease, UCookOnTheFlyServer& InCookOnTheFlyServer)
	: Object(InObject), TargetPlatform(InTargetPlatform), PackageData(InPackageData)
	, CookOnTheFlyServer(InCookOnTheFlyServer),	CancelManager(nullptr), ClassName(InObject->GetClass()->GetFName())
	, bHasReleased(false), bNeedsResourceRelease(bInNeedsResourceRelease)
{
	check(InObject);
	PackageData.GetNumPendingCookedPlatformData() += 1;
}

FPendingCookedPlatformData::FPendingCookedPlatformData(FPendingCookedPlatformData&& Other)
	: Object(Other.Object), TargetPlatform(Other.TargetPlatform), PackageData(Other.PackageData)
	, CookOnTheFlyServer(Other.CookOnTheFlyServer), CancelManager(Other.CancelManager), ClassName(Other.ClassName)
	, UpdatePeriodMultiplier(Other.UpdatePeriodMultiplier), bHasReleased(Other.bHasReleased)
	, bNeedsResourceRelease(Other.bNeedsResourceRelease)
{
	Other.Object = nullptr;
	Other.bHasReleased = true;
}

FPendingCookedPlatformData::~FPendingCookedPlatformData()
{
	Release();
}

bool FPendingCookedPlatformData::PollIsComplete()
{
	if (bHasReleased)
	{
		return true;
	}

	UObject* LocalObject = Object.Get();
	if (!LocalObject)
	{
		Release();
		return true;
	}
	UCookOnTheFlyServer& COTFS = PackageData.GetPackageDatas().GetCookOnTheFlyServer();
	if (COTFS.RouteIsCachedCookedPlatformDataLoaded(PackageData, LocalObject, TargetPlatform,
		nullptr /* ExistingEvent */))
	{
		Release();
		return true;
	}

	// If something (another object's BeginCacheForCookedPlatformData, maybe) has marked the object as
	// garbage, or renamed it out of the package, then we no longer need to wait on it.
	// We might have removed the packagedata from the save state and no longer have a cached UPackage* on it,
	// so compare current package vs original package by name instead of pointer.
	FName CurrentPackageName = LocalObject->GetPackage()->GetFName();
	if (CurrentPackageName != PackageData.GetPackageName())
	{
		// Exception: allow Generator packages to keep waiting on objects that they have moved into generated packages.
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper();
		if (!GenerationHelper ||
			!GenerationHelper->ShouldKeepWaitingOnMovedPendingCookedPlatformData(*this, PackageData, LocalObject))
		{
			UE_LOGF(LogCook, Display,
				"We were waiting for IsCachedCookedPlatformData to return true for %ls in package %ls, but that object has been moved out of the package. We will stop waiting on it.",
				*LocalObject->GetFullName(), *PackageData.GetPackageName().ToString());
			Release();
			return true;
		}
	}
	if (LocalObject->HasAnyFlags(EObjectFlags::RF_MirroredGarbage))
	{
		UE_LOGF(LogCook, Display,
			"We were waiting for IsCachedCookedPlatformData to return true for %ls, but that object is now marked for garbage. We will stop waiting on it.",
			*LocalObject->GetFullName());
		Release();
		return true;
	}

#if DEBUG_COOKONTHEFLY
	UE_LOGF(LogCook, Display, "%ls isn't cached yet", *LocalObject->GetFullName());
#endif
	/*if ( LocalObject->IsA(UMaterial::StaticClass()) )
	{
		if (GShaderCompilingManager->HasShaderJobs() == false)
		{
			UE_LOGF(LogCook, Warning,
			"Shader compiler is in a bad state!  Shader %ls is finished compile but shader compiling manager did not notify shader.  ",
				*LocalObject->GetPathName());
		}
	}*/
	return false;
}

void FPendingCookedPlatformData::Release()
{
	if (bHasReleased)
	{
		return;
	}

	if (bNeedsResourceRelease)
	{
		int32* CurrentAsyncCache = CookOnTheFlyServer.CurrentAsyncCacheForType.Find(ClassName);
		// bNeedsRelease should not have been set if the AsyncCache does not have an entry for the class
		check(CurrentAsyncCache != nullptr);
		*CurrentAsyncCache += 1;
	}

	PackageData.GetNumPendingCookedPlatformData() -= 1;
	check(PackageData.GetNumPendingCookedPlatformData() >= 0);
	if (CancelManager)
	{
		CancelManager->Release(*this);
		CancelManager = nullptr;
	}

	Object = nullptr;
	bHasReleased = true;
}

void FPendingCookedPlatformData::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	TargetPlatform = Remap[TargetPlatform];
}

void FPendingCookedPlatformData::ClearCachedCookedPlatformData(UObject* Object, FPackageData& PackageData,
	bool bCompletedSuccesfully)
{
	FPackageDatas& PackageDatas = PackageData.GetPackageDatas();
	UCookOnTheFlyServer& COTFS = PackageDatas.GetCookOnTheFlyServer();
	FMapOfCachedCookedPlatformDataState& CCPDs = PackageDatas.GetCachedCookedPlatformDataObjects();

	uint32 ObjectKeyHash = FMapOfCachedCookedPlatformDataState::KeyFuncsType::GetKeyHash(Object);
	FCachedCookedPlatformDataState* CCPDState = CCPDs.FindByHash(ObjectKeyHash, Object);
	if (!CCPDState)
	{
		return;
	}

	CCPDState->ReleaseFrom(&PackageData);
	if (!CCPDState->IsReferenced())
	{
		for (const TPair<const ITargetPlatform*, ECachedCookedPlatformDataEvent>&
			PlatformPair : CCPDState->PlatformStates)
		{
			Object->ClearCachedCookedPlatformData(PlatformPair.Key);
		}

		// ClearAllCachedCookedPlatformData and WillNeverCacheCookedPlatformDataAgain are not used in editor
		if (!COTFS.IsCookingInEditor())
		{
			Object->ClearAllCachedCookedPlatformData();
			if (bCompletedSuccesfully && COTFS.IsDirectorCookByTheBook())
			{
				Object->WillNeverCacheCookedPlatformDataAgain();
			}
		}

		CCPDs.RemoveByHash(ObjectKeyHash, Object);
	}
};


//////////////////////////////////////////////////////////////////////////
// FPendingCookedPlatformDataCancelManager


void FPendingCookedPlatformDataCancelManager::Release(FPendingCookedPlatformData& Data)
{
	--NumPendingPlatforms;
	if (NumPendingPlatforms <= 0)
	{
		check(NumPendingPlatforms == 0);
		UObject* LocalObject = Data.Object.Get();
		if (LocalObject)
		{
			FPendingCookedPlatformData::ClearCachedCookedPlatformData(LocalObject, Data.PackageData,
				false /* bCompletedSuccesfully */);
		}
		delete this;
	}
}


//////////////////////////////////////////////////////////////////////////
// FPackageDataMonitor
FPackageDataMonitor::FPackageDataMonitor()
{
	FMemory::Memset(NumUrgentInState, 0);
	FMemory::Memset(NumCookLastInState, 0);
}

int32 FPackageDataMonitor::GetNumUrgent(EUrgency UrgencyLevel) const
{
	check(EUrgency::Min <= UrgencyLevel && UrgencyLevel <= EUrgency::Max);
	int32 UrgencyIndex = static_cast<uint32>(UrgencyLevel) - static_cast<uint32>(EUrgency::Min);
	int32 NumUrgent = 0;
	for (EPackageState State = EPackageState::Min;
		State <= EPackageState::Max;
		State = static_cast<EPackageState>(static_cast<uint32>(State) + 1))
	{
		int32 StateIndex = static_cast<uint32>(State) - static_cast<uint32>(EPackageState::Min);
		NumUrgent += NumUrgentInState[StateIndex][UrgencyIndex];
	}
	return NumUrgent;
}

int32 FPackageDataMonitor::GetNumCookLast() const
{
	int32 Num = 0;
	for (EPackageState State = EPackageState::Min;
		State <= EPackageState::Max;
		State = static_cast<EPackageState>(static_cast<uint32>(State) + 1))
	{
		Num += NumCookLastInState[static_cast<uint32>(State) - static_cast<uint32>(EPackageState::Min)];
	}
	return Num;
}

int32 FPackageDataMonitor::GetNumUrgent(EPackageState InState, EUrgency UrgencyLevel) const
{
	check(EUrgency::Min <= UrgencyLevel && UrgencyLevel <= EUrgency::Max);
	int32 UrgencyIndex = static_cast<uint32>(UrgencyLevel) - static_cast<uint32>(EUrgency::Min);
	check(EPackageState::Min <= InState && InState <= EPackageState::Max);
	int32 StateIndex = static_cast<uint32>(InState) - static_cast<uint32>(EPackageState::Min);
	return NumUrgentInState[StateIndex][UrgencyIndex];
}

int32 FPackageDataMonitor::GetNumCookLast(EPackageState InState) const
{
	check(EPackageState::Min <= InState && InState <= EPackageState::Max);
	int32 StateIndex = static_cast<uint32>(InState) - static_cast<uint32>(EPackageState::Min);
	return NumCookLastInState[StateIndex];
}

int32 FPackageDataMonitor::GetNumPreloadAllocated() const
{
	return NumPreloadAllocated;
}

int32 FPackageDataMonitor::GetNumInProgress() const
{
	return NumInProgress;
}

int32 FPackageDataMonitor::GetNumCooked(ECookResult Result) const
{
	return NumCooked[(uint8)Result];
}

void FPackageDataMonitor::OnInProgressChanged(FPackageData& PackageData, bool bInProgress)
{
	NumInProgress += bInProgress ? 1 : -1;
	check(NumInProgress >= 0);
}

void FPackageDataMonitor::OnPreloadAllocatedChanged(FPackageData& PackageData, bool bPreloadAllocated)
{
	NumPreloadAllocated += bPreloadAllocated ? 1 : -1;
	check(NumPreloadAllocated >= 0);
}

void FPackageDataMonitor::OnFirstCookedPlatformAdded(FPackageData& PackageData, ECookResult CookResult)
{
	check(CookResult != ECookResult::NotAttempted);
	if (PackageData.GetMonitorCookResult() == ECookResult::NotAttempted)
	{
		PackageData.SetMonitorCookResult(CookResult);
		++NumCooked[(uint8)CookResult];
	}
}

void FPackageDataMonitor::OnLastCookedPlatformRemoved(FPackageData& PackageData)
{
	ECookResult CookResult = PackageData.GetMonitorCookResult();
	if (CookResult != ECookResult::NotAttempted)
	{
		--NumCooked[(uint8)CookResult];
		PackageData.SetMonitorCookResult(ECookResult::NotAttempted);
	}
}

void FPackageDataMonitor::OnUrgencyChanged(FPackageData& PackageData, EUrgency OldUrgency, EUrgency NewUrgency)
{
	TrackUrgentRequests(PackageData.GetState(), OldUrgency, -1);
	TrackUrgentRequests(PackageData.GetState(), NewUrgency, 1);
}

void FPackageDataMonitor::OnCookLastChanged(FPackageData& PackageData)
{
	int32 Delta = PackageData.GetIsCookLast() ? 1 : -1;
	TrackCookLastRequests(PackageData.GetState(), Delta);
}

void FPackageDataMonitor::OnStateChanged(FPackageData& PackageData, EPackageState OldState)
{
	EPackageState NewState = PackageData.GetState();
	EUrgency Urgency = PackageData.GetUrgency();
	if (Urgency > EUrgency::Normal)
	{
		TrackUrgentRequests(OldState, Urgency, -1);
		TrackUrgentRequests(NewState, Urgency, 1);
	}
	if (PackageData.GetIsCookLast())
	{
		TrackCookLastRequests(OldState, -1);
		TrackCookLastRequests(NewState, 1);
	}
	bool bOldStateAssignedToLocal = OldState != EPackageState::Idle &&
		!EnumHasAnyFlags(FStateProperties(OldState).Properties, EPackageStateProperty::AssignedToWorkerProperty);
	bool bNewStateAssignedToLocal = NewState != EPackageState::Idle &&
		!EnumHasAnyFlags(FStateProperties(NewState).Properties, EPackageStateProperty::AssignedToWorkerProperty);
	if (bOldStateAssignedToLocal != bNewStateAssignedToLocal)
	{
		++(bNewStateAssignedToLocal ? MPCookAssignedFenceMarker : MPCookRetiredFenceMarker);
	}
}

void FPackageDataMonitor::TrackUrgentRequests(EPackageState State, EUrgency Urgency, int32 Delta)
{
	if (State == EPackageState::Idle || Urgency == EUrgency::Normal)
	{
		// We don't track urgency count in idle, and we don't track normal urgency count.
		return;
	}
	check(EPackageState::Min <= State && State <= EPackageState::Max);
	check(EUrgency::Min <= Urgency && Urgency <= EUrgency::Max);

	int32 StateIndex = static_cast<uint32>(State) - static_cast<uint32>(EPackageState::Min);
	int32 UrgencyIndex = static_cast<uint32>(Urgency) - static_cast<uint32>(EUrgency::Min);
	NumUrgentInState[StateIndex][UrgencyIndex] += Delta;
	check(NumUrgentInState[StateIndex][UrgencyIndex] >= 0);
}

void FPackageDataMonitor::TrackCookLastRequests(EPackageState State, int32 Delta)
{
	check(EPackageState::Min <= State && State <= EPackageState::Max);
	if (State != EPackageState::Idle)
	{
		NumCookLastInState[static_cast<uint32>(State) - static_cast<uint32>(EPackageState::Min)] += Delta;
		check(NumCookLastInState[static_cast<uint32>(State) - static_cast<uint32>(EPackageState::Min)] >= 0);
	}
}

int32 FPackageDataMonitor::GetMPCookAssignedFenceMarker() const
{
	return MPCookAssignedFenceMarker;
}

int32 FPackageDataMonitor::GetMPCookRetiredFenceMarker() const
{
	return MPCookRetiredFenceMarker;
}

//////////////////////////////////////////////////////////////////////////
// FPackageDatas

IAssetRegistry* FPackageDatas::AssetRegistry = nullptr;

FPackageDatas::FPackageDatas(UCookOnTheFlyServer& InCookOnTheFlyServer)
	: CookOnTheFlyServer(InCookOnTheFlyServer)
	, LastPollAsyncTime(0)
{
	Allocator.SetMinBlockSize(1024);
	Allocator.SetMaxBlockSize(65536);
}

void FPackageDatas::SetBeginCookConfigSettings(FStringView CookShowInstigator)
{
	ShowInstigatorPackageData = nullptr;
	if (!CookShowInstigator.IsEmpty())
	{
		FString LocalPath;
		FString PackageName;
		if (!FPackageName::TryConvertToMountedPath(CookShowInstigator, &LocalPath, &PackageName,
			nullptr, nullptr, nullptr))
		{
			UE_LOGF(LogCook, Fatal, "-CookShowInstigator argument %.*ls is not a mounted filename or packagename",
				CookShowInstigator.Len(), CookShowInstigator.GetData());
		}
		else
		{
			FName PackageFName(*PackageName);
			ShowInstigatorPackageData = TryAddPackageDataByPackageName(PackageFName);
			if (!ShowInstigatorPackageData)
			{
				UE_LOGF(LogCook, Fatal, "-CookShowInstigator argument %.*ls could not be found on disk",
					CookShowInstigator.Len(), CookShowInstigator.GetData());
			}
		}
	}
}

FPackageDatas::~FPackageDatas()
{
	Clear();
}

void FPackageDatas::OnAssetRegistryGenerated(IAssetRegistry& InAssetRegistry)
{
	AssetRegistry = &InAssetRegistry;
}

FString FPackageDatas::GetReferencerName() const
{
	return TEXT("CookOnTheFlyServer");
}

void FPackageDatas::AddReferencedObjects(FReferenceCollector& Collector)
{
	return CookOnTheFlyServer.CookerAddReferencedObjects(Collector);
}

FPackageData& FPackageDatas::FindOrAddPackageData(const FName& PackageName, const FName& NormalizedFileName)
{
	{
		FReadScopeLock ExistenceReadLock(ExistenceLock);
		FPackageData** PackageDataMapAddr = PackageNameToPackageData.Find(PackageName);
		if (PackageDataMapAddr != nullptr)
		{
			FPackageData& ExistingPackageData = **PackageDataMapAddr;
			FPackageData** FileNameMapAddr = FileNameToPackageData.Find(NormalizedFileName);
			if (FileNameMapAddr == nullptr)
			{
				// Since we have an existing package, it should have registered its filename. Therefore the only
				// way the current filename could be unregistered is if the existing package's filename is different.
				FString ExistingFileNameStr = ExistingPackageData.GetFileName().ToString();
				FString NewFileNameStr = NormalizedFileName.ToString();
				FileNameMapAddr = FileNameToPackageData.Find(ExistingPackageData.GetFileName());
				checkf(ExistingPackageData.GetFileName() != NormalizedFileName,
					TEXT("InternalCookerError: FindOrAddPackageData found existing package at PackageName, and NormalizedFileName is not registered, and so we expected ExistingPackage.FileName != NormalizedFileName, but it is equal.")
					TEXT("\n\tPackageName = %s, NormalizedFileName = %s"), *PackageName.ToString(), *NormalizedFileName.ToString());
				checkf(FileNameMapAddr != nullptr,
					TEXT("InternalCookerError: FindOrAddPackageData found existing package at PackageName with a different FileName, but that existing package's FileName is not registered.")
					TEXT("\n\tPackageName = %s, NormalizedFileName = %s, ExistingFileName = %s"),
					*PackageName.ToString(), *NewFileNameStr, *ExistingFileNameStr);

				if (FPaths::GetBaseFilename(ExistingFileNameStr, false /* bRemovePath */)
					!= FPaths::GetBaseFilename(NewFileNameStr, false /* bRemovePath */))
				{
					// If the filename's Path or BaseNameWithoutExtension are different, that's a discrepency we don't handle.
					checkf(ExistingFileNameStr == NewFileNameStr,
						TEXT("Package %s is being added with filename %s, but it already exists with filename with different path %s, ")
						TEXT("and it is not present in FileNameToPackageData map under the new name."),
						*PackageName.ToString(), *NewFileNameStr, *ExistingFileNameStr);
				}
				else
				{
					// We handle the extension being different, but only in the special case that the package does not need to be saved.
					// If we are responsible for saving the package then we need to know which extension is correct, so we are broken.
					bool bAllPreviouslyCooked = true;
					for (const ITargetPlatform* TargetPlatform
						: GetCookOnTheFlyServer().PlatformManager->GetSessionPlatforms())
					{
						if (ExistingPackageData.FindOrAddPlatformData(TargetPlatform).GetWhereCooked()
							== EWhereCooked::ThisSession)
						{
							bAllPreviouslyCooked = false;
						}
					}
					if (bAllPreviouslyCooked)
					{
						// Silently accept the existing extension; we will not need it since we are not cooking it.
					}
					else
					{
						// We don't handle different extensions when we need to cook the package, but it will probably
						// not cause a crash elsewhere. Log it as an error and print the callstack.
						UE_LOGF(LogCook, Error,
							"Package %ls is being added with filename %ls, but it already exists with filename with different extension %ls. Keeping the old extension. This is unexpected and the output of the cook might be invalid.",
							*PackageName.ToString(), *NewFileNameStr, *ExistingFileNameStr);
						FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
					}
					// MPCook TODO: Send a correction to all CookWorkers?
				}
			}
			checkf(*FileNameMapAddr == &ExistingPackageData,
				TEXT("Package %s is being added with filename %s, but that filename maps to a different package %s."),
				*PackageName.ToString(), *NormalizedFileName.ToString(),
				*(*FileNameMapAddr)->GetPackageName().ToString());
			return ExistingPackageData;
		}

		checkf(FileNameToPackageData.Find(NormalizedFileName) == nullptr,
			TEXT("Package \"%s\" and package \"%s\" share the same filename \"%s\"."),
			*PackageName.ToString(), *(*FileNameToPackageData.Find(NormalizedFileName))->GetPackageName().ToString(),
			*NormalizedFileName.ToString());
	}
	return CreatePackageData(PackageName, NormalizedFileName);
}

FPackageData* FPackageDatas::FindPackageDataByPackageName(const FName& PackageName)
{
	if (PackageName.IsNone())
	{
		return nullptr;
	}

	FReadScopeLock ExistenceReadLock(ExistenceLock);
	FPackageData** PackageDataMapAddr = PackageNameToPackageData.Find(PackageName);
	return PackageDataMapAddr ? *PackageDataMapAddr : nullptr;
}

FPackageData* FPackageDatas::TryAddPackageDataByPackageName(const FName& PackageName, bool bRequireExists,
	bool bCreateAsMap)
{
	if (PackageName.IsNone())
	{
		return nullptr;
	}

	{
		FReadScopeLock ExistenceReadLock(ExistenceLock);
		FPackageData** PackageDataMapAddr = PackageNameToPackageData.Find(PackageName);
		if (PackageDataMapAddr != nullptr)
		{
			return *PackageDataMapAddr;
		}
	}

	FName FileName = LookupFileNameOnDisk(PackageName, bRequireExists, bCreateAsMap);
	if (FileName.IsNone())
	{
		// This will happen if PackageName does not exist on disk
		return nullptr;
	}
	{
		FReadScopeLock ExistenceReadLock(ExistenceLock);
		checkf(FileNameToPackageData.Find(FileName) == nullptr,
			TEXT("Package \"%s\" and package \"%s\" share the same filename \"%s\"."),
			*PackageName.ToString(), *(*FileNameToPackageData.Find(FileName))->GetPackageName().ToString(),
			*FileName.ToString());
	}
	return &CreatePackageData(PackageName, FileName);
}

FPackageData& FPackageDatas::AddPackageDataByPackageNameChecked(const FName& PackageName, bool bRequireExists,
	bool bCreateAsMap)
{
	FPackageData* PackageData = TryAddPackageDataByPackageName(PackageName, bRequireExists, bCreateAsMap);
	check(PackageData);
	return *PackageData;
}

FPackageData* FPackageDatas::FindPackageDataByFileName(const FName& InFileName)
{
	FName FileName(GetStandardFileName(InFileName));
	if (FileName.IsNone())
	{
		return nullptr;
	}

	FReadScopeLock ExistenceReadLock(ExistenceLock);
	FPackageData** PackageDataMapAddr = FileNameToPackageData.Find(FileName);
	return PackageDataMapAddr ? *PackageDataMapAddr : nullptr;
}

FPackageData* FPackageDatas::TryAddPackageDataByFileName(const FName& InFileName)
{
	return TryAddPackageDataByStandardFileName(GetStandardFileName(InFileName));
}

FPackageData* FPackageDatas::TryAddPackageDataByStandardFileName(const FName& FileName, bool bExactMatchRequired,
	FName* OutFoundFileName)
{
	FName FoundFileName = FileName;
	ON_SCOPE_EXIT{ if (OutFoundFileName) { *OutFoundFileName = FoundFileName; } };
	if (FileName.IsNone())
	{
		return nullptr;
	}

	{
		FReadScopeLock ExistenceReadLock(ExistenceLock);
		FPackageData** PackageDataMapAddr = FileNameToPackageData.Find(FileName);
		if (PackageDataMapAddr != nullptr)
		{
			return *PackageDataMapAddr;
		}
	}

	FName ExistingFileName;
	FName PackageName = LookupPackageNameOnDisk(FileName, bExactMatchRequired, ExistingFileName);
	if (PackageName.IsNone())
	{
		return nullptr;
	}
	if (ExistingFileName.IsNone())
	{
		if (!bExactMatchRequired)
		{
			FReadScopeLock ExistenceReadLock(ExistenceLock);
			FPackageData** PackageDataMapAddr = PackageNameToPackageData.Find(PackageName);
			if (PackageDataMapAddr != nullptr)
			{
				FoundFileName = (*PackageDataMapAddr)->GetFileName();
				return *PackageDataMapAddr;
			}
		}
		UE_LOGF(LogCook, Warning,
			"Unexpected failure to cook filename '%ls'. It is mapped to PackageName '%ls', but does not exist on disk and we cannot verify the extension.",
			*FileName.ToString(), *PackageName.ToString());
		return nullptr;
	}
	FoundFileName = ExistingFileName;
	return &CreatePackageData(PackageName, ExistingFileName);
}

FPackageData& FPackageDatas::CreatePackageData(FName PackageName, FName FileName)
{
	check(!PackageName.IsNone());
	check(!FileName.IsNone());

	FWriteScopeLock ExistenceWriteLock(ExistenceLock);
	FPackageData*& ExistingByPackageName = PackageNameToPackageData.FindOrAdd(PackageName);
	FPackageData*& ExistingByFileName = FileNameToPackageData.FindOrAdd(FileName);
	if (ExistingByPackageName)
	{
		// The other CreatePackageData call should have added the FileName as well
		check(ExistingByFileName == ExistingByPackageName);
		return *ExistingByPackageName;
	}
	// If no other CreatePackageData added the PackageName, then they should not have added
	// the FileName either
	check(!ExistingByFileName);
	FPackageData* PackageData = Allocator.NewElement(*this, PackageName, FileName);
	ExistingByPackageName = PackageData;
	ExistingByFileName = PackageData;
	return *PackageData;
}

FPackageData& FPackageDatas::AddPackageDataByFileNameChecked(const FName& FileName)
{
	FPackageData* PackageData = TryAddPackageDataByFileName(FileName);
	check(PackageData);
	return *PackageData;
}

FName FPackageDatas::GetFileNameByPackageName(FName PackageName, bool bRequireExists, bool bCreateAsMap)
{
	FPackageData* PackageData = TryAddPackageDataByPackageName(PackageName, bRequireExists, bCreateAsMap);
	return PackageData ? PackageData->GetFileName() : NAME_None;
}

bool FPackageDatas::TryGetNamesByFlexName(FName PackageOrFileName, FName* OutPackageName, FName* OutFileName,
	bool bRequireExists, bool bCreateAsMap)
{
	FString Buffer = PackageOrFileName.ToString();
	if (!FPackageName::TryConvertFilenameToLongPackageName(Buffer, Buffer))
	{
		return false;
	}
	FName PackageName = FName(Buffer);
	FName FileName = GetFileNameByPackageName(PackageName, bRequireExists, bCreateAsMap);
	if (FileName.IsNone())
	{
		return false;
	}
	if (OutPackageName)
	{
		*OutPackageName = PackageName;
	}
	if (OutFileName)
	{
		*OutFileName = FileName;
	}
	return true;
}

FName FPackageDatas::LookupFileNameOnDisk(FName PackageName, bool bRequireExists, bool bCreateAsMap)
{
	FString FilenameOnDisk;
	if (TryLookupFileNameOnDisk(PackageName, FilenameOnDisk))
	{
	}
	else if (!bRequireExists)
	{
		FString Extension = bCreateAsMap ? FPackageName::GetMapPackageExtension() :
			FPackageName::GetAssetPackageExtension();
		if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName.ToString(), FilenameOnDisk, Extension))
		{
			return NAME_None;
		}
	}
	else
	{
		return NAME_None;
	}
	FilenameOnDisk = FPaths::ConvertRelativePathToFull(FilenameOnDisk);
	FPaths::MakeStandardFilename(FilenameOnDisk);
	return FName(FilenameOnDisk);
}

bool FPackageDatas::TryLookupFileNameOnDisk(FName PackageName, FString& OutFileName)
{
	FString PackageNameStr = PackageName.ToString();

	// Verse packages are editor-generated in-memory packages which don't have a corresponding 
	// asset file (yet). However, we still want to cook these packages out, producing cooked 
	// asset files for packaged projects.
	if (FPackageName::IsVersePackage(PackageNameStr))
	{
		if (FindPackage(/*Outer =*/nullptr, *PackageNameStr))
		{
			if (!FPackageName::TryConvertLongPackageNameToFilename(PackageNameStr, OutFileName,
				FPackageName::GetAssetPackageExtension()))
			{
				UE_LOGF(LogCook, Warning,
					"Package %ls exists in memory but its PackageRoot is not mounted. It will not be cooked.",
					*PackageNameStr);
				return false;
			}
			return true;
		}
		// else, the cooker could be responding to a NotifyUObjectCreated() event, and the object hasn't
		// been fully constructed yet (missing from the FindObject() list) -- in this case, we've found 
		// that the linker loader is creating a dummy object to fill a referencing import slot, not loading
		// the proper object (which means we want to ignore it).
	}

	if (!AssetRegistry)
	{
		return FPackageName::DoesPackageExist(PackageNameStr, &OutFileName, false /* InAllowTextFormats */);
	}
	else
	{
		FString PackageExtension;
		if (!AssetRegistry->DoesPackageExistOnDisk(PackageName, nullptr, &PackageExtension))
		{
			return false;
		}

		return FPackageName::TryConvertLongPackageNameToFilename(PackageNameStr, OutFileName, PackageExtension);
	}
}

FName FPackageDatas::LookupPackageNameOnDisk(FName NormalizedFileName, bool bExactMatchRequired, FName& FoundFileName)
{
	FoundFileName = NormalizedFileName;
	if (NormalizedFileName.IsNone())
	{
		return NAME_None;
	}
	FString Buffer = NormalizedFileName.ToString();
	if (!FPackageName::TryConvertFilenameToLongPackageName(Buffer, Buffer))
	{
		return NAME_None;
	}
	FName PackageName = FName(*Buffer);

	FName DiscoveredFileName = LookupFileNameOnDisk(PackageName, true /* bRequireExists */, false /* bCreateAsMap */);
	if (DiscoveredFileName == NormalizedFileName || !bExactMatchRequired)
	{
		FoundFileName = DiscoveredFileName;
		return PackageName;
	}
	else
	{
		// Either the file does not exist on disk or NormalizedFileName did not match its format or extension
		return NAME_None;
	}
}

FName FPackageDatas::GetStandardFileName(FName FileName)
{
	FString FileNameString(FileName.ToString());
	FPaths::MakeStandardFilename(FileNameString);
	return FName(FileNameString);
}

FName FPackageDatas::GetStandardFileName(FStringView InFileName)
{
	FString FileName(InFileName);
	FPaths::MakeStandardFilename(FileName);
	return FName(FileName);
}

void FPackageDatas::AddExistingPackageDatasForPlatform(TConstArrayView<FConstructPackageData> ExistingPackages,
	const ITargetPlatform* TargetPlatform, bool bExpectPackageDatasAreNew, int32& OutPackageDataFromBaseGameNum)
{
	int32 NumPackages = ExistingPackages.Num();
	if (NumPackages == 0)
	{
		return;
	}

	// Make the list unique
	TArray<FConstructPackageData> UniqueArray(ExistingPackages);
	Algo::Sort(UniqueArray, [](const FConstructPackageData& A, const FConstructPackageData& B)
		{
			return A.PackageName.FastLess(B.PackageName);
		});
	UniqueArray.SetNum(Algo::Unique(UniqueArray, [](const FConstructPackageData& A, const FConstructPackageData& B)
		{
			return A.PackageName == B.PackageName;
		}));
	ExistingPackages = UniqueArray;

	FWriteScopeLock ExistenceWriteLock(ExistenceLock);
	if (bExpectPackageDatasAreNew)
	{
		Allocator.ReserveDelta(NumPackages);
		FileNameToPackageData.Reserve(FileNameToPackageData.Num() + NumPackages);
		PackageNameToPackageData.Reserve(PackageNameToPackageData.Num() + NumPackages);
	}

	// Create the PackageDatas and mark them as cooked
	for (const FConstructPackageData& ConstructData : ExistingPackages)
	{
		FName PackageName = ConstructData.PackageName;
		FName NormalizedFileName = ConstructData.NormalizedFileName;
		check(!PackageName.IsNone());
		check(!NormalizedFileName.IsNone());

		FPackageData*& PackageData = FileNameToPackageData.FindOrAdd(NormalizedFileName, nullptr);
		if (!PackageData)
		{
			// create the package data (copied from CreatePackageData)
			FPackageData* NewPackageData = Allocator.NewElement(*this, PackageName, NormalizedFileName);
			FPackageData* ExistingByPackageName = PackageNameToPackageData.FindOrAdd(PackageName, NewPackageData);
			// If no other CreatePackageData added the FileName, then they should not have added
			// the PackageName either
			check(ExistingByPackageName == NewPackageData);

			PackageData = NewPackageData;
		}
		PackageData->SetPlatformCooked(TargetPlatform, ECookResult::Succeeded, /*bWasCookedThisSession=*/false);
		FPackagePlatformData& PlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
		PlatformData.SetWhereCooked(EWhereCooked::BaseGame);
	}
	OutPackageDataFromBaseGameNum += ExistingPackages.Num();
}

FPackageData* FPackageDatas::UpdateFileName(FName PackageName)
{
	FWriteScopeLock ExistenceWriteLock(ExistenceLock);

	FPackageData** PackageDataAddr = PackageNameToPackageData.Find(PackageName);
	if (!PackageDataAddr)
	{
		FName NewFileName = LookupFileNameOnDisk(PackageName);
		check(NewFileName.IsNone() || !FileNameToPackageData.Find(NewFileName));
		return nullptr;
	}
	FPackageData* PackageData = *PackageDataAddr;
	FName OldFileName = PackageData->GetFileName();
	bool bIsMap = FPackageName::IsMapPackageExtension(*FPaths::GetExtension(OldFileName.ToString()));
	FName NewFileName = LookupFileNameOnDisk(PackageName, false /* bRequireExists */, bIsMap);
	if (OldFileName == NewFileName)
	{
		return PackageData;
	}
	if (NewFileName.IsNone())
	{
		UE_LOGF(LogCook, Error, "Cannot update FileName for package %ls because the package is no longer mounted.",
			*PackageName.ToString())
			return PackageData;
	}

	check(!OldFileName.IsNone());
	FPackageData* ExistingByFileName;
	ensure(FileNameToPackageData.RemoveAndCopyValue(OldFileName, ExistingByFileName));
	check(ExistingByFileName == PackageData);

	PackageData->SetFileName(NewFileName);
	FPackageData* AddedByFileName = FileNameToPackageData.FindOrAdd(NewFileName, PackageData);
	check(AddedByFileName == PackageData);

	return PackageData;
}

FThreadsafePackageData::FThreadsafePackageData()
	: bInitialized(false)
	, bHasLoggedDiscoveryWarning(false)
	, bHasLoggedDependencyWarning(false)
{
}

void FPackageDatas::UpdateThreadsafePackageData(const FPackageData& PackageData)
{
	UpdateThreadsafePackageData(PackageData.GetPackageName(),
		[&PackageData](FThreadsafePackageData& ThreadsafeData, bool bNew)
		{
			ThreadsafeData.Instigator = PackageData.GetInstigator(EReachability::Runtime);
			ThreadsafeData.Generator = PackageData.GetParentGenerator();
		});
}

int32 FPackageDatas::GetNumCooked()
{
	int32 Count = 0;
	for (uint8 CookResult = 0; CookResult < (uint8)ECookResult::Count; ++CookResult)
	{
		Count += Monitor.GetNumCooked((ECookResult)CookResult);
	}
	return Count;
}

int32 FPackageDatas::GetNumCooked(ECookResult CookResult)
{
	return Monitor.GetNumCooked(CookResult);
}

void FPackageDatas::GetCommittedPackagesForPlatform(const ITargetPlatform* Platform,
	TArray<FPackageData*>& SucceededPackages,
	TArray<FPackageData*>& FailedPackages)
{
	LockAndEnumeratePackageDatas(
	[Platform, &SucceededPackages, &FailedPackages](FPackageData* PackageData)
	{
		FPackagePlatformData* PlatformData = PackageData->FindPlatformData(Platform);
		if (PlatformData && PlatformData->IsCommitted())
		{
			ECookResult CookResults = PackageData->GetCookResults(Platform);
			(CookResults == ECookResult::Succeeded ? SucceededPackages : FailedPackages).Add(PackageData);
		}
	});
}

void FPackageDatas::Clear()
{
	FWriteScopeLock ExistenceWriteLock(ExistenceLock);
	PendingCookedPlatformDataLists.Empty(); // These destructors will read/write PackageDatas
	RequestQueue.Empty();
	SaveQueue.Empty();
	AssignedToWorkerSet.Empty();
	SaveStalledSet.Empty();
	PackageNameToPackageData.Empty();
	FileNameToPackageData.Empty();
	CachedCookedPlatformDataObjects.Empty();
	{
		// All references must be cleared before any PackageDatas are destroyed
		EnumeratePackageDatasWithinLock([](FPackageData* PackageData)
		{
			PackageData->ClearReferences();
		});
		EnumeratePackageDatasWithinLock([](FPackageData* PackageData)
		{
			PackageData->~FPackageData();
		});
		Allocator.Empty();
	}

	ShowInstigatorPackageData = nullptr;
}

void FPackageDatas::ClearCookedPlatforms()
{
	LockAndEnumeratePackageDatas([](FPackageData* PackageData)
	{
		PackageData->ResetReachable(EReachability::All);
		PackageData->ClearCookResults();
	});
}

void FPackageDatas::ClearCookResultsForPackages(const TSet<FName>& InPackages, const ITargetPlatform* TargetPlatform,
	int32& InOutNumBaseGamePackages)
{
	int32 AffectedPackagesCount = 0;
	LockAndEnumeratePackageDatas([InPackages, &AffectedPackagesCount, TargetPlatform](FPackageData* PackageData)
		{
			const FName& PackageName = PackageData->GetPackageName();
			if (InPackages.Contains(PackageName))
			{
				bool bCookAttempted;
				{
					FPackagePlatformData& PlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
					PlatformData.SetWhereCooked(EWhereCooked::ThisSession);
					bCookAttempted = PlatformData.IsCookAttempted();
				}
				if (bCookAttempted)
				{
					PackageData->ClearCookResults(TargetPlatform);
					AffectedPackagesCount++;
				}
			}
		});

	UE_LOGF(LogCook, Display,
		"Cleared the cook results of %d packages because ClearCookResultsForPackages requested them to be recooked.",
		AffectedPackagesCount);
	InOutNumBaseGamePackages -= AffectedPackagesCount;
}

void FPackageDatas::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform)
{
	LockAndEnumeratePackageDatas([TargetPlatform](FPackageData* PackageData)
	{
		PackageData->OnRemoveSessionPlatform(TargetPlatform);
	});
}

constexpr int32 PendingPlatformDataReservationSize = 128;
constexpr int32 PendingPlatformDataMaxUpdatePeriod = 16;

void FPackageDatas::AddPendingCookedPlatformData(FPendingCookedPlatformData&& Data)
{
	if (PendingCookedPlatformDataLists.IsEmpty())
	{
		PendingCookedPlatformDataLists.Emplace();
		PendingCookedPlatformDataLists.Last().Reserve(PendingPlatformDataReservationSize );
	}
	PendingCookedPlatformDataLists.First().Add(MoveTemp(Data));
	++PendingCookedPlatformDataNum;
}

TArray<FPendingCookedPlatformData*> FPackageDatas::GetPendingCookedPlatformDatas(UObject* Object)
{
	check(Object);

	TArray<FPendingCookedPlatformData*> Results;
	ForEachPendingCookedPlatformData([&Results, Object](FPendingCookedPlatformData& Data)
		{
			if (Data.Object.GetEvenIfUnreachable() == Object)
			{
				Results.Add(&Data);
			}
		});
	return Results;
}

void FPackageDatas::PollPendingCookedPlatformDatas(bool bForce, double& LastCookableObjectTickTime,
	int32& OutNumRetired)
{
	OutNumRetired = 0;
	if (PendingCookedPlatformDataNum == 0)
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	if (!bForce)
	{
		// ProcessAsyncResults and IsCachedCookedPlatformDataLoaded can be expensive to call
		// Cap the frequency at which we call them. We only update the last poll time at completion
		// so that we don't suddenly saturate the game thread by making derived data key strings
		// when the time to do the polls increases to GPollAsyncPeriod.
		if (CurrentTime < LastPollAsyncTime + GPollAsyncPeriod)
		{
			return;
		}
	}
	LastPollAsyncTime = CurrentTime;

	// PendingPlatformDataLists is a rotating list of lists of PendingPlatformDatas
	// The first list contains all of the PendingPlatformDatas that we should poll on this tick
	// The nth list is all of the PendingPlatformDatas that we should poll after N more ticks
	// Each poll period we pull the front list off and all other lists move frontwards by 1.
	// New PendingPlatformDatas are inserted into the first list, to be polled in the next poll period
	// When a PendingPlatformData signals it is not ready after polling, we increase its poll period
	// exponentially - we double it.
	// A poll period of N times the default poll period means we insert it into the Nth list in
	// PendingPlatformDataLists.
	FPendingCookedPlatformDataContainer List = PendingCookedPlatformDataLists.PopFrontValue();
	if (!bForce && List.IsEmpty())
	{
		return;
	}

	if (bForce)
	{
		// When we are forced, because the caller has an urgent package to save, call ProcessAsyncResults
		// with a small timeslice in case we need to process shaders to unblock the package
		constexpr float TimeSlice = 0.01f;
		GShaderCompilingManager->ProcessAsyncResults(TimeSlice, false /* bBlockOnGlobalShaderCompletion */);
	}

	// Future proof as each callback thread gets its own thread_local scope so that the thread_local state
	// (GCookActivePackageName, FInheritedContextExtensionScope) is always manipulated on the correct thread.
	// The atomic counter tracks the total number of open scopes across all threads so we can assert they are
	// all closed by the time ProcessAsyncTasks returns.
	static thread_local TOptional<UE::Cook::FScopedCookActivePackage> ActivePackageScope;
	std::atomic<int32> OpenScopeCount{ 0 };
	FDelegateHandle EventHandle = FAssetCompilingManager::Get().OnPackageScopeEvent().AddLambda(
		[&OpenScopeCount](UPackage* Package, bool bEntering)
		{
			if (bEntering)
			{
				// We need the scope to always be properly balanced
				check(!ActivePackageScope.IsSet());
				ActivePackageScope.Emplace(Package->GetFName(),
#if UE_WITH_OBJECT_HANDLE_TRACKING
					PackageAccessTrackingOps::NAME_CookerBuildObject
#else
					FName()
#endif
				);
				OpenScopeCount.fetch_add(1, std::memory_order_relaxed);
			}
			else
			{
				ActivePackageScope.Reset();
				OpenScopeCount.fetch_sub(1, std::memory_order_relaxed);
			}
		});
	FAssetCompilingManager::Get().ProcessAsyncTasks(true);
	FAssetCompilingManager::Get().OnPackageScopeEvent().Remove(EventHandle);
	// Validate that we don't have any open scopes left after exiting
	check(OpenScopeCount.load(std::memory_order_relaxed) == 0);

	if (LastCookableObjectTickTime + TickCookableObjectsFrameTime <= CurrentTime)
	{
		UE_SCOPED_COOKTIMER(TickCookableObjects);
		FTickableCookObject::TickObjects(static_cast<float>(CurrentTime - LastCookableObjectTickTime),
			false /* bTickComplete */);
		LastCookableObjectTickTime = CurrentTime;
	}

	if (!bForce)
	{
		for (FPendingCookedPlatformData& Data : List)
		{
			if (Data.PollIsComplete())
			{
				// We are destructing all elements of List after the for loop is done; we leave
				// the completed Data on List to be destructed.
				--PendingCookedPlatformDataNum;
				++OutNumRetired;
			}
			else
			{
				Data.UpdatePeriodMultiplier = FMath::Clamp(Data.UpdatePeriodMultiplier*2, 1,
					PendingPlatformDataMaxUpdatePeriod);
				int32 ContainerIndex = Data.UpdatePeriodMultiplier - 1;
				while (PendingCookedPlatformDataLists.Num() <= ContainerIndex)
				{
					PendingCookedPlatformDataLists.Emplace();
					PendingCookedPlatformDataLists.Last().Reserve(PendingPlatformDataReservationSize);
				}
				PendingCookedPlatformDataLists[ContainerIndex].Add(MoveTemp(Data));
			}
		}
	}
	else
	{
		// When called with bForce, we poll all PackageDatas in all lists, and do not update
		// any PollPeriods.
		PendingCookedPlatformDataLists.AddFront(MoveTemp(List));
		for (FPendingCookedPlatformDataContainer& ForceList : PendingCookedPlatformDataLists)
		{
			for (int32 Index = 0; Index < ForceList.Num(); )
			{
				FPendingCookedPlatformData& Data = ForceList[Index];
				if (Data.PollIsComplete())
				{
					ForceList.RemoveAtSwap(Index, EAllowShrinking::No);
					--PendingCookedPlatformDataNum;
					++OutNumRetired;
				}
				else
				{
					++Index;
				}
			}
		}
	}
}

void FPackageDatas::ClearCancelManager(FPackageData& PackageData)
{
	ForEachPendingCookedPlatformData(
		[&PackageData](FPendingCookedPlatformData& PendingCookedPlatformData)
		{
			if (&PendingCookedPlatformData.PackageData == &PackageData)
			{
				if (!PendingCookedPlatformData.PollIsComplete())
				{
					// Abandon it
					PendingCookedPlatformData.Release();
				}
			}
		});
}

void FPackageDatas::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	LockAndEnumeratePackageDatas([&Remap](FPackageData* PackageData)
	{
		PackageData->RemapTargetPlatforms(Remap);
	});
	ForEachPendingCookedPlatformData([&Remap](FPendingCookedPlatformData& CookedPlatformData)
	{
		CookedPlatformData.RemapTargetPlatforms(Remap);
	});
}

void FPackageDatas::DebugInstigator(FPackageData& PackageData)
{
	if (ShowInstigatorPackageData == &PackageData)
	{
		TArray<FInstigator> Chain = CookOnTheFlyServer.GetInstigatorChain(PackageData.GetPackageName());
		UE_LOGF(LogCook, Display, "Instigator chain of %ls: %ls",
			*PackageData.GetPackageName().ToString(), *UE::Cook::InstigatorChainToString(Chain));
	}
	UpdateThreadsafePackageData(PackageData);
}

void FRequestQueue::Empty()
{
	RestartedRequests.Empty();
	DiscoveryQueue.Empty();
	BuildDependencyDiscoveryQueue.Empty();
	RequestClusters.Empty();
	RequestFencePackageListeners.Empty();
	NormalRequests.Empty();
	UrgentRequests.Empty();
}

bool FRequestQueue::IsEmpty() const
{
	return Num() == 0;
}

uint32 FRequestQueue::Num() const
{
	uint32 Count = RestartedRequests.Num() + ReadyRequestsNum();
	for (const TUniquePtr<FRequestCluster>& RequestCluster : RequestClusters)
	{
		Count += RequestCluster->NumPackageDatas();
	}
	return Count;
}

bool FRequestQueue::Contains(const FPackageData* InPackageData) const
{
	FPackageData* PackageData = const_cast<FPackageData*>(InPackageData);
	if (RestartedRequests.Contains(PackageData) || NormalRequests.Contains(PackageData) ||
		UrgentRequests.Contains(PackageData))
	{
		return true;
	}
	for (const TUniquePtr<FRequestCluster>& RequestCluster : RequestClusters)
	{
		if (RequestCluster->Contains(PackageData))
		{
			return true;
		}
	}
	return false;
}

uint32 FRequestQueue::RemoveRequestExceptFromCluster(FPackageData* PackageData, FRequestCluster* ExceptFromCluster)
{
	uint32 OriginalNum = Num();
	RestartedRequests.Remove(PackageData);
	NormalRequests.Remove(PackageData);
	UrgentRequests.Remove(PackageData);
	for (TUniquePtr<FRequestCluster>& RequestCluster : RequestClusters)
	{
		if (RequestCluster.Get() != ExceptFromCluster)
		{
			RequestCluster->RemovePackageData(PackageData);
		}
	}
	uint32 Result = OriginalNum - Num();
	check(Result == 0 || Result == 1);
	return Result;
}

uint32 FRequestQueue::RemoveRequest(FPackageData* PackageData)
{
	return RemoveRequestExceptFromCluster(PackageData, nullptr);
}

uint32 FRequestQueue::Remove(FPackageData* PackageData)
{
	return RemoveRequest(PackageData);
}

bool FRequestQueue::IsReadyRequestsEmpty() const
{
	return ReadyRequestsNum() == 0;
}

bool FRequestQueue::HasRequestsToExplore() const
{
	return !RequestClusters.IsEmpty() | !RestartedRequests.IsEmpty() | !DiscoveryQueue.IsEmpty()
		| !BuildDependencyDiscoveryQueue.IsEmpty() | !RequestFencePackageListeners.IsEmpty();
}

uint32 FRequestQueue::ReadyRequestsNum() const
{
	return UrgentRequests.Num() + NormalRequests.Num();
}

FPackageData* FRequestQueue::PopReadyRequest()
{
	FPackageData* PackageData = UrgentRequests.Pop();
	if (PackageData)
	{
		return PackageData;
	}
	PackageData = NormalRequests.Pop();
	if (PackageData)
	{
		return PackageData;
	}
	return nullptr;
}

void FRequestQueue::AddRequest(FPackageData* PackageData, bool bForceUrgent)
{
	RestartedRequests.Add(PackageData);
}

void FRequestQueue::AddReadyRequest(FPackageData* PackageData, bool bForceUrgent)
{
	if (bForceUrgent || PackageData->GetUrgency() > EUrgency::Normal)
	{
		UrgentRequests.Add(PackageData);
	}
	else
	{
		NormalRequests.Add(PackageData);
	}
}

void FRequestQueue::UpdateUrgency(FPackageData* PackageData, EUrgency OldUrgency, EUrgency NewUrgency)
{
	if (OldUrgency == EUrgency::Normal)
	{
		if (NormalRequests.Remove(PackageData) > 0)
		{
			UrgentRequests.Add(PackageData);
		}
	}
	else
	{
		if (UrgentRequests.Remove(PackageData) > 0)
		{
			NormalRequests.Add(PackageData);
		}
	}
	// The other subcontainers do not handle urgency types differently
}

void FRequestQueue::AddRequestFenceListener(FName PackageName)
{
	RequestFencePackageListeners.Add(PackageName);
}

void FRequestQueue::NotifyRequestFencePassed(FPackageDatas& PackageDatas)
{
	for (FName PackageName : RequestFencePackageListeners)
	{
		FPackageData* PackageData = PackageDatas.FindPackageDataByPackageName(PackageName);
		if (PackageData)
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
			if (GenerationHelper)
			{
				GenerationHelper->OnRequestFencePassed(PackageDatas.GetCookOnTheFlyServer());
			}
		}
	}
	RequestFencePackageListeners.Empty();
}

FPoppedPackageDataScope::FPoppedPackageDataScope(FPackageData& InPackageData)
#if COOK_CHECKSLOW_PACKAGEDATA
	: PackageData(InPackageData)
#endif
{
}

#if COOK_CHECKSLOW_PACKAGEDATA
FPoppedPackageDataScope::~FPoppedPackageDataScope()
{
	PackageData.CheckInContainer();
}
#endif

const TCHAR* LexToString(ECachedCookedPlatformDataEvent Value)
{
	switch (Value)
	{
	case ECachedCookedPlatformDataEvent::None:
		return TEXT("None");
	case ECachedCookedPlatformDataEvent::BeginCacheForCookedPlatformDataCalled:
		return TEXT("BeginCacheForCookedPlatformDataCalled");
	case ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedCalled:
		return TEXT("IsCachedCookedPlatformDataLoadedCalled");
	case ECachedCookedPlatformDataEvent::IsCachedCookedPlatformDataLoadedReturnedTrue:
		return TEXT("IsCachedCookedPlatformDataLoadedReturnedTrue");
	case ECachedCookedPlatformDataEvent::ClearCachedCookedPlatformDataCalled:
		return TEXT("ClearCachedCookedPlatformDataCalled");
	case ECachedCookedPlatformDataEvent::ClearAllCachedCookedPlatformDataCalled:
		return TEXT("ClearAllCachedCookedPlatformDataCalled");
	default: return TEXT("Invalid");
	}
}

void FPackageDatas::CachedCookedPlatformDataObjectsPostGarbageCollect(
	const TSet<UObject*>& SaveQueueObjectsThatStillExist)
{
	for (TMap<UObject*, FCachedCookedPlatformDataState>::TIterator Iter(this->CachedCookedPlatformDataObjects);
		Iter; ++Iter)
	{
		if (!SaveQueueObjectsThatStillExist.Contains(Iter->Key))
		{
			Iter.RemoveCurrent();
		}
	}
}

void FPackageDatas::CachedCookedPlatformDataObjectsOnDestroyedOutsideOfGC(const UObject* DestroyedObject)
{
	CachedCookedPlatformDataObjects.Remove(DestroyedObject);
}

void FCachedCookedPlatformDataState::AddRefFrom(FPackageData* PackageData)
{
	// Most objects will only be referenced by a single package.
	// The exceptions:
	//   1) Generator Packages that move the object from the generator into a generated
	//   2) Bugs
	// Even in case (1), the number of referencers will be 2.
	// We therefore for now just use a flat array and AddUnique, to minimize memory and performance in the 
	// usual case on only a single referencer.
	PackageDatas.AddUnique(PackageData);
}

void FCachedCookedPlatformDataState::ReleaseFrom(FPackageData* PackageData)
{
	PackageDatas.Remove(PackageData);
}

bool FCachedCookedPlatformDataState::IsReferenced() const
{
	return !PackageDatas.IsEmpty();
}

void FCachedCookedPlatformDataState::Construct(UObject* Object)
{
	WeakPtr = Object;
	bInitialized = true;
}

FCachedCookedPlatformDataState& FMapOfCachedCookedPlatformDataState::Add(
	UObject* Object, const FCachedCookedPlatformDataState& Value)
{
	FCachedCookedPlatformDataState* Existing = &Super::FindOrAdd(Object);
	*Existing = Value;
	if (!Existing->bInitialized)
	{
		Existing->Construct(Object);
	}
	return *Existing;
}

FCachedCookedPlatformDataState& FMapOfCachedCookedPlatformDataState::FindOrAdd(UObject* Object)
{
	FCachedCookedPlatformDataState* Existing = &Super::FindOrAdd(Object);
	if (!Existing->bInitialized)
	{
		Existing->Construct(Object);
	}
	if (!Existing->WeakPtr.Get())
	{
		Remove(Object);
		Existing = &Super::FindOrAdd(Object);
	}
	return *Existing;
}

FCachedCookedPlatformDataState* FMapOfCachedCookedPlatformDataState::Find(UObject* Object)
{
	FCachedCookedPlatformDataState* Existing = Super::Find(Object);
	if (!Existing)
	{
		return nullptr;
	}
	if (!Existing->WeakPtr.Get())
	{
		Remove(Object);
		return nullptr;
	}
	return Existing;
}

FCachedCookedPlatformDataState& FMapOfCachedCookedPlatformDataState::FindOrAddByHash(uint32 KeyHash, UObject* Object)
{
	FCachedCookedPlatformDataState* Existing = &Super::FindOrAddByHash(KeyHash, Object);
	if (!Existing->bInitialized)
	{
		Existing->Construct(Object);
	}
	if (!Existing->WeakPtr.Get())
	{
		Remove(Object);
		Existing = &Super::FindOrAddByHash(KeyHash, Object);
	}
	return *Existing;
}

FCachedCookedPlatformDataState* FMapOfCachedCookedPlatformDataState::FindByHash(uint32 KeyHash, UObject* Object)
{
	FCachedCookedPlatformDataState* Existing = Super::FindByHash(KeyHash, Object);
	if (!Existing)
	{
		return nullptr;
	}
	if (!Existing->WeakPtr.Get())
	{
		Remove(Object);
		return nullptr;
	}
	return Existing;
}

} // namespace UE::Cook
