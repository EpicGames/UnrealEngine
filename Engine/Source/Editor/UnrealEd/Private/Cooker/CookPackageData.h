// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "Containers/SortedMap.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookTypes.h"
#include "Cooker/FifoSet.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "IO/IoHash.h"
#include "Math/NumericLimits.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TypedBlockAllocator.h"
#include "UObject/GCObject.h"
#include "UObject/ICookInfo.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReferenceCollector;
class ITargetPlatform;
class FCbFieldView;
class FCbWriter;
class UCookOnTheFlyServer;
class UObject;
class UPackage;
namespace UE::Cook { class FCookWorkerClient; }
namespace UE::Cook { class FPackagePreloader; }
namespace UE::Cook { class FRequestCluster; }
namespace UE::Cook { struct FBuildResultDependenciesMap; }
namespace UE::Cook { struct FConstructPackageData; }
namespace UE::Cook { struct FDiscoveredPlatformSet; }
namespace UE::Cook { struct FReplicatedLogData; }

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FConstructPackageData& PackageData);
bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FConstructPackageData& PackageData);

namespace UE::Cook
{

class FPackageDataQueue;
class FRequestCluster;
struct FGenerationHelper;
struct FPackageData;
struct FPackageDataMonitor;
struct FPendingCookedPlatformDataCancelManager;

/**
 * Events in the lifetime of an object related to BeginCacheForCookedPlatformData. Used by the cooker
 * to track which calls have been made and still need to be made.
 */
enum class ECachedCookedPlatformDataEvent : uint8
{
	None,
	BeginCacheForCookedPlatformDataCalled,
	IsCachedCookedPlatformDataLoadedCalled,
	IsCachedCookedPlatformDataLoadedReturnedTrue,
	ClearCachedCookedPlatformDataCalled,
	ClearAllCachedCookedPlatformDataCalled,
};
const TCHAR* LexToString(ECachedCookedPlatformDataEvent);
/**
 * BeginCachedForCookedPlatformData state about an object - which packages owned it (not always the same
 * one when PackageGenerators are involved) and the per-platform state for ECachedCookedPlatformDataEvent.
 */
struct FCachedCookedPlatformDataState
{
	void Construct(UObject* Object);
	void AddRefFrom(FPackageData* PackageData);
	void ReleaseFrom(FPackageData* PackageData);
	bool IsReferenced() const;

	/**
	* The weakpointer that was constructed from the UObject* key when we added the data to the cache.
	* If this pointer no longer equals the key, then the UObject* at the memory position given by the key
	* has been destroyed (and possibly a new object allocated into its same memory) and we should clear the
	* the cache state.
	*/
	FWeakObjectPtr WeakPtr;
	/**
	 * The packages that have called any of the BeginCacheForCookedPlatformData family of function
	 * on this object. Usually only a single 1, sometimes 2, see comment in AddPackageData.
	 */
	TArray<FPackageData*, TInlineAllocator<2>> PackageDatas;
	/** The per-platform state of which BeginCacheForCookedPlatformData events have been passed. */
	TMap<const ITargetPlatform*, ECachedCookedPlatformDataEvent> PlatformStates;
	bool bInitialized = false;
};

class FMapOfCachedCookedPlatformDataState : public TMap<UObject*, FCachedCookedPlatformDataState>
{
public:
	using Super = TMap<UObject*, FCachedCookedPlatformDataState>;
	using Super::TMap;

	FCachedCookedPlatformDataState& Add(UObject* Object, const FCachedCookedPlatformDataState& Value);
	FCachedCookedPlatformDataState& FindOrAdd(UObject* Object);
	FCachedCookedPlatformDataState* Find(UObject* Object);
	FCachedCookedPlatformDataState& FindOrAddByHash(uint32 TypeHash, UObject* Object);
	FCachedCookedPlatformDataState* FindByHash(uint32 TypeHash, UObject* Object);
};

/**
 * Objects we searched for in the Package being saved; we need to execute various operations on
 * all of these objects, most notably BeginCacheForCookedPlatformData family of functions.
 * We keep track of a WeakPtr to the object along with its flags in case it gets deleted and we
 * need to decide how to respond to the deletion it based on what its flags were.
 */
struct FCachedObjectInOuter
{
	FWeakObjectPtr Object;
	EObjectFlags ObjectFlags;

	FCachedObjectInOuter(UObject* InObject = nullptr)
	{
		Object = InObject;
		ObjectFlags = InObject ? InObject->GetFlags() : RF_NoFlags;
	}
	FCachedObjectInOuter(FWeakObjectPtr&& InObject)
	{
		Object = MoveTemp(InObject);
		UObject* Ptr = Object.Get(true /* bEvenIfPendingKill */);
		ObjectFlags = Ptr ? Ptr->GetFlags() : RF_NoFlags;
	}
	FCachedObjectInOuter(const FWeakObjectPtr& InObject)
	{
		Object = InObject;
		UObject* Ptr = Object.Get(true /* bEvenIfPendingKill */);
		ObjectFlags = Ptr ? Ptr->GetFlags() : RF_NoFlags;
	}
};

/** Flags specifying the behavior of FPackageData::SendToState */
enum class ESendFlags : uint8
{
	/**
	 * PackageData will not be removed from queue for its old state and will not be added to queue for its new state.
	 * Caller is responsible for remove and add.
	 */
	QueueNone = 0x0,
	/**
	 * PackageData will be removed from the queue of its old state.
	 * If this flag is missing, caller is responsible for removing from the old state's queue.
	 */
	QueueRemove = 0x1,
	/**
	 * PackageData will be added to queue for its next state.
	 * If this flag is missing, caller is responsible for adding to queue.
	 */
	QueueAdd = 0x2,
	/**
	 * PackageData will be removed from the queue of its old state and added to the queue of its new state.
	 * This may be wasteful, if the caller can add or remove more efficiently.
	 */
	QueueAddAndRemove = QueueAdd | QueueRemove,
};
ENUM_CLASS_FLAGS(ESendFlags);

/**
 * Reachability can be set for multiple properties. Reachability of each property is initially assigned to a package
 * from initial requests, by e.g. the AssetManager requests in StartCookByTheBook, and it transitively is assigned to
 * dependencies of the packages to which it is assigned.
 */
enum class EReachability : uint8
{
	None,
	/**
	 * A transitive build dependency from a cooked package. Build dependencies are commitable even if the package is
	 * not cookable.
	 */
	Build = 0x01,
	/**
	 * Reachable as a runtime dependency; the package should be saved and made available at runtime. Packages
	 * that are runtime reachable might still be not cooked if they are not cookable due to  e.g. NeverCook settings.
	 */
	Runtime = 0x02,
	MaxBit = Runtime,
	All = Build | Runtime,
	NumBits = FPlatformMath::ConstExprCeilLogTwo(EReachability::MaxBit) + 1,
};
ENUM_CLASS_FLAGS(EReachability);

/** Data necessary to create an FPackageData without checking the disk, for e.g. AddExistingPackageDatasForPlatform */
struct FConstructPackageData
{
	friend FCbWriter& ::operator<<(FCbWriter& Writer, const FConstructPackageData& PackageData);
	friend bool ::LoadFromCompactBinary(FCbFieldView Field, FConstructPackageData& PackageData);

	FName PackageName;
	FName NormalizedFileName;
};

// This should be a constexpr, but some compilers give an error that 0x1 is an unevaluable pointer value
#define CookerLoadingPlatformKey ((ITargetPlatform*)0x1)

/** Data about a platform that has been interacted with (marked reachable, etc) by a PackageData. */
struct FPackagePlatformData
{
	FPackagePlatformData();

	/**
	 * Get or modify the Reachability for specified reachability bits. @see UE::Cook::EReachability.
	 */
	EReachability GetReachability() const;
	bool IsReachable(EReachability InReachability) const;
	void AddReachability(EReachability InReachability);
	void ClearReachability(EReachability InReachability);

	/** Query/Modify whether the package has passed through a RequestCluster for all of the given Reachability bits. */
	bool IsVisitedByCluster(EReachability InReachability) const;
	void AddVisitedByCluster(EReachability InReachability);
	void ClearVisitedByCluster(EReachability InReachability);

	/** UPackage::Save was called on the package, but timedout and needs to be retried. */
	bool IsSaveTimedOut() const;
	void SetSaveTimedOut(bool bValue);

	/** Whether the package is allowed to cook. Initially true, may be set false during Cluster search. */
	bool IsCookable() const;
	void SetCookable(bool bValue);

	/**
	 * Whether the package is searchable for transitive dependencies during Cluster evaluation.
	 * This can be true even if the package is not cookable. Initially true, may be set false during Cluster search
	 */
	bool IsExplorable() const;
	void SetExplorable(bool bValue);

	/**
	 * Whether the package, which might be set as not explorable from IsRequestCookable, is set to explorable
	 * based on conditions discovered during the cook.
	 */
	bool IsExplorableOverride() const;
	void SetExplorableOverride(bool bValue);

	/** All flags modified by reachability calculations for the given Reachability bits are returned to default. */
	void ResetReachable(EReachability InReachability);
	/**
	 * Mark platform as ExplorableOverride=true and reset all data necessary to reexplore it, including reachability.
	 * Caller is responsible for marking it again as reachable.
	 */
	void MarkAsExplorable();

	/** Called on CookWorkers to indicate reachable,cookable,etc for packages sent from Director. */
	void MarkCommittableForWorker(EReachability InReachability, FCookWorkerClient& CookWorkerClient);

	/**
	 * Returns whether SetIncrementallyUnmodified has been called in the current cook session.
	 * IncrementallyUnmodified like a TOptional tracks an unset state in addition to its value if set.
	 */
	bool IsIncrementallyUnmodifiedSet() const;
	/**
	 * The package was found to be unmodified in the current incremental cook.
	 * Returns false if !IsIncrementallyUnmodifiedSet().
	 */
	bool IsIncrementallyUnmodified() const;
	void SetIncrementallyUnmodified(bool bValue);
	/** Restore the value to its unset state (e.g. beginning a new cook session). */
	void ClearIncrementallyUnmodified();

	/**
	 * Report which session cooked this package, if it was cooked. For packages cooked in the current session, or
	 * or for packages not yet cooked, this is EWhereCooked::CurrentSession. For packages encountered during DLC
	 * cooks but cooked in a different session, this result gives the category of that other session's relationship
	 * to the current session, e.g. the BaseGame cook.
	 */
	EWhereCooked GetWhereCooked() const;
	void SetWhereCooked(EWhereCooked Value);

	/**
	 * The package was skipped in the current incremental cook. This might not be equal to IncrementallyUnmodified,
	 * depending on packagewriter).
	 */
	bool IsIncrementallySkipped() const;
	void SetIncrementallySkipped(bool bValue);

	ECookResult GetCookResults() const;
	bool IsCookAttempted() const;
	bool IsCookSucceeded() const;
	void SetCookResults(ECookResult Value);

	bool IsCommitted() const;
	void SetCommitted(bool bValue);

	/** Return if we need to commit for the given reachability - reachable but not yet cooked/committed. */
	bool NeedsCommit(const ITargetPlatform* PlatformItBelongsTo, EReachability InReachability) const;
	bool NeedsCommit(const ITargetPlatform* PlatformItBelongsTo, ECookPhase CookPhase) const;

	/** Return if we need to cook the package (Reachable for Runtime and not yet cooked), aka NeedsCommit(Runtime). */
	bool NeedsCooking(const ITargetPlatform* PlatformItBelongsTo) const;

	bool IsRegisteredForCachedObjectsInOuter() const;
	void SetRegisteredForCachedObjectsInOuter(bool bValue);

	/** Only read/written on CookWorkers */
	bool IsReportedToDirector();
	void SetReportedToDirector(bool bValue);

private:
	uint32 Reachability : (int)EReachability::NumBits;
	uint32 ReachabilityVisitedByCluster : (int)EReachability::NumBits;
	uint32 bSaveTimedOut : 1;
	uint32 bCookable : 1;
	uint32 bExplorable : 1;
	uint32 bExplorableOverride : 1;
	uint32 IncrementallyUnmodified : 2;
	uint32 bIncrementallySkipped : 1;
	uint32 WhereCooked : (int)EWhereCooked::NumBits;
	uint32 bRegisteredForCachedObjectsInOuter : 1;
	uint32 bReportedToDirector : 1;
	uint32 bCommitted : 1;
	uint32 CookResults : (int)ECookResult::NumBits;
};

/**
 * Contains all the information the cooker uses for a package, during request, load, or save.  Once allocated, this
 * structure is never deallocated or moved for a given package; it is deallocated only when the CookOnTheFlyServer
 * is destroyed.
 *
 * PlatformDatas - Per-platform information about the package's cook state. Whether it has been marked reachable,
 *   whether it has been cooked and what the result was, etc.
 *   Direct write access to the platformdata is pseudo-private: it's available for convenience of the RequestCluster,
*    but when modifiying it the package may need to change state, so calling code should either update the state itself
*    or should call the functions on FPackageData that take a TargetPlatform and handle modifying the state.

 * Cooked platforms - Platforms are marked cooked if they have been saved in the lifetime of the 
 *   CookOnTheFlyServer; note this extends outside of the current CookOnTheFlyServer session for in-editor cooks.
 *   Cooked platforms also store the CookResults - success,error,other.
 *   Other: cooked platforms can be added to a PackageData for reasons other than normal Save, such as when a Package
 *   is marked not cookable for a Platform and its "Cooked" operation is therefore a noop.
 *   Cooked platforms can be cleared for a PackageData if the Package is modified, e.g. during editor operations
 *   when the CookOnTheFlyServer is run from the editor.
 *
 * Package - The package pointer corresponding to this PackageData.
 *   Contract for the lifetime of this field is that it is only set after the Package has passed through the load
 *   state, and it is cleared when the package returns to idle or is demoted to an earlier state.
 *   At all other times it is nullptr.
 *
 * State - the state of this PackageData in the CookOnTheFlyServer's current session. See the definition of
 *   EPackageState for a description of each state. Contract for the value of this field includes membership in the
 *   corresponding queue such as SaveQueue, and the presence or absence of state-specific data. Since modifying the
 *   State can put the Package into an invalid state, direct write access is private; SendToState handles enforcing
 *   the contracts.

 * PendingCookedPlatformData - a counter for how many objects in the package have had
 *   BeginCacheForCookedPlatformData called and have not yet completed. This is used to block the package from
 *   saving until all objects have finished their cache. It is also used to block the package from starting new
 *   BeginCacheForCookedPlatformData calls until all pending calls from a previous canceled save have completed.
 *   The lifetime of this counter extends for the lifetime of the PackageData; it is shared across
 *   CookOnTheFlyServer sessions.
 *
 * CookedPlatformDataNextIndex - Index for the next Object in CachedObjectsInOuter that needs to have
 *   BeginCacheForCookedPlatformData called on it for the current PackageSave. This field is only >= 0 during
 *   the save state; it is cleared to -1 when successfully or unsucessfully leaving the save state. 
 *
 * Other fields with explanation inline
*/
struct FPackageData
{
public:
	FPackageData(FPackageDatas& PackageDatas, const FName& InPackageName, const FName& InFileName);
	~FPackageData();
	static void* operator new(size_t Size)
	{
		checkf(false, TEXT("PackageDatas should be allocated using FPackageDatas.Allocator"));
		return FMemory::Malloc(Size);
	}
	static void* operator new(size_t Size, void* PlacementNewPtr)
	{
		return PlacementNewPtr;
	}
	static void operator delete(void* Ptr, size_t Size)
	{
		checkf(false, TEXT("PackageDatas should be freed using FPackageDatas.Allocator"));
	}

	/**
	 * ClearReferences is called on every PackageData before any packageDatas are deleted,
	 * so references are still valid during ClearReferences
	 */
	void ClearReferences();

	FPackageData(const FPackageData& other) = delete;
	FPackageData(FPackageData&& other) = delete;
	FPackageData& operator=(const FPackageData& other) = delete;
	FPackageData& operator=(FPackageData&& other) = delete;

	FPackageDatas& GetPackageDatas() const;

	/** Return a copy of Package->GetName(). It is derived from the FileName if necessary, and is never modified. */
	const FName& GetPackageName() const;
	/**
	 * Return a copy of the FileName containing the package, normalized as returned from MakeStandardFilename.
	 * This field may change from e.g. *.umap to *.uasset if LoadPackage loads a different FileName for the
	 * requested PackageName.
	 */
	const FName& GetFileName() const;

	/**
	 * Get the LeafToRootRank that was found for this PackageData, if set. Returns MAX_uint32 if not yet set.
	 * This value is not replicated to CookWorkers; each CookWorker calculates it individually during Pumploads.
	 */
	uint32 GetLeafToRootRank() const;
	/** Set the LeafToRootRank for this package data. */
	void SetLeafToRootRank(uint32 Value);

	/** Reset OutPlatforms and copy NeedsCommit platforms into it, for the given CookPhase (@see NeedsCommit). */
	template <typename ArrayType>
	void GetPlatformsNeedingCommit(ArrayType& OutPlatforms, ECookPhase CookPhase) const;
	/** Number of platforms that would be returned by GetPlatformsNeedingCommit. */
	int32 GetPlatformsNeedingCommitNum(ECookPhase CookPhase) const;

	/** Reset OutPlatforms and copy NeedsCommit platforms into it, for the given Reachability (@see NeedsCommit). */
	template <typename ArrayType>
	void GetPlatformsNeedingCommit(ArrayType& OutPlatforms, EReachability Reachability) const;
	/** Number of platforms that would be returned by GetPlatformsNeedingCommit. */
	int32 GetPlatformsNeedingCommitNum(EReachability Reachability) const;

	/** Reset OutPlatforms and copy current set of reachable platforms into it. */
	template <typename ArrayType>
	void GetReachablePlatforms(EReachability InReachability, ArrayType& OutPlatforms) const;

	/**
	 * Return true if and only if the Platform has been visited by a cluster (and hence is reachable and explored)
	 * for all of the requested Reachability bits.
	 */
	bool IsPlatformVisitedByCluster(const ITargetPlatform* Platform, EReachability InReachability) const;

	/**
	 * Return true if and only if every element of Platforms is currently reachable for all of the given
	 * reachability bits. Returns true if Platforms is empty.
	 */
	bool HasReachablePlatforms(EReachability InReachability,
		const TArrayView<const ITargetPlatform* const>& Platforms) const;

	/**
	 * Return whether all Platforms that have been marked reachable for the given reachability property have also been
	 * explored in an FRequestCluster for that reachability.
	 */
	bool AreAllReachablePlatformsVisitedByCluster(EReachability InReachability) const;

	/** Get/Set the urgency of a package. Always EUrgency::Normal for pacakges that are not in progress. */
	EUrgency GetUrgency() const;
	void SetUrgency(EUrgency NewUrgency, ESendFlags SendFlags, bool bAllowUrgencyInIdle = false);
	/** Set the Urgency iff the newurgency is greater than current. */
	void RaiseUrgency(EUrgency NewUrgency, ESendFlags SendFlags, bool bAllowUrgencyInIdle = false);

	/** Accessor for RequestClusters to add reachable platforms directly without modifying dependent data. */
	void AddReachablePlatforms(FRequestCluster& RequestCluster, EReachability InReachability,
		TConstArrayView<const ITargetPlatform*> Platforms, FInstigator&& InInstigator);

	/** Add the given reachable platforms to this PackageData and send it back to Request state for exploration. */
	void QueueAsDiscovered(FInstigator&& InInstigator, FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency InUrgency);

	/**
	 * Clear all the inprogress variables from the current PackageData. It is invalid to call this except when
	 * the PackageData is transitioning out of InProgress.
	 */
	void ClearInProgressData(EStateChangeReason StateChangeReason);

	/**
	 * FindOrAdd each TargetPlatform and set its flags: CookAttempted=true, Succeeded=<given>.
	 * In version that takes two arrays, TargetPlatforms and Succeeded must be the same length.
	 */
	void SetPlatformsCooked(const TConstArrayView<const ITargetPlatform*> TargetPlatforms,
		const TConstArrayView<ECookResult> Succeeded, bool bInWasCookedThisSession = true);
	void SetPlatformsCooked(const TConstArrayView<const ITargetPlatform*> TargetPlatforms,
		ECookResult Result, bool bInWasCookedThisSession = true);
	void SetPlatformCooked(const ITargetPlatform* TargetPlatform, ECookResult Result,
		bool bInWasCookedThisSession = true);
	void SetPlatformCommitted(const ITargetPlatform* TargetPlatform);

	/**
	 * FindOrAdd each TargetPlatform and set its flags: CookAttempted=false.
	 * In Version that takes no TargetPlatform, CookAttempted is cleared from all existing platforms.
	 */
	void ClearCookResults(const TConstArrayView<const ITargetPlatform*> TargetPlatforms);
	void ClearCookResults();
	void ClearCookResults(const ITargetPlatform* TargetPlatform);
	/** Clear reachable and related fields from all platforms for the given reachability bits. */
	void ResetReachable(EReachability InReachability);

	/** Access the information about platforms interacted with by *this. */
	const TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>>& GetPlatformDatas() const;
	TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>>&
		GetPlatformDatasConstKeysMutableValues();

	/** Add a platform if not already existing and return a writable pointer to its flags. */
	FPackagePlatformData& FindOrAddPlatformData(const ITargetPlatform* TargetPlatform);

	/** Find a platform if it exists and return a writable pointer to its flags. */
	FPackagePlatformData* FindPlatformData(const ITargetPlatform* TargetPlatform);

	/** Find a platform if it exists and return a writable pointer to its flags. */
	const FPackagePlatformData* FindPlatformData(const ITargetPlatform* TargetPlatform) const;

	/** Return true if and only if at least one platform has been cooked. */
	bool HasAnyCookedPlatform() const;

	/** Return true if and only if at least one platform has been committed. */
	bool HasAnyCommittedPlatforms() const;

	/**
	 * Return true if and only if at least one element of Platforms has been cooked, and with its
	 * succeeded flag set to true if bIncludeFailed is false. Returns false if Platforms is empty.
	 */
	bool HasAnyCookedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms, bool bIncludeFailed) const;
	/**
	 * Return true if and only if every element of Platforms has been cooked, and with its succeeded
	 * flag set to true if bIncludeFailed is false. Returns true if Platforms is empty.
	 */
	bool HasAllCookedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms, bool bIncludeFailed) const;
	/**
	 * Return true if and only if the given Platform has been cooked, and with its succeeded flag
	 * set to true if bIncludeFailed is false.
	 */
	bool HasCookedPlatform(const ITargetPlatform* Platform, bool bIncludeFailed) const;
	/**
	 * Return the CookResult for the given platform.  If the platform has not been cooked,
	 * returns ECookResult::NotAttempted, otherwise returns whatever result has been set.
	 */
	ECookResult GetCookResults(const ITargetPlatform* Platform) const;
	/** Get/Set the SuppressCookReason for the package. Defaults to ESuppressCookReason::NotSuppressed. */
	ESuppressCookReason GetSuppressCookReason() const;
	/** Get/Set the SuppressCookReason for the package. Defaults to ESuppressCookReason::NotSuppressed. */
	void SetSuppressCookReason(ESuppressCookReason Reason);
	/** Return true iff every element of Platforms has been committed. Returns true if Platforms is empty. */
	bool HasAllCommittedPlatforms(const TArrayView<const ITargetPlatform* const>& Platforms) const;
	/** Return true if and only if the given Platform has been committed. */
	bool HasCommittedPlatform(const ITargetPlatform* Platform) const;

	/**
	 * Return the package pointer. By contract it will be non-null if and only if the PackageData's state is
	 * >= EPackageState::Load.
	 */
	UPackage* GetPackage() const;
	/* Set the package pointer. Caller is responsible for maintaining the contract for this field. */
	void SetPackage(UPackage* InPackage);

	/** Return the current PackageState */
	EPackageState GetState() const;
	/**
	 * Set the PackageData's state to the given state, remove and add of from the appropriate queues, and destroy,
	 * create, and verify the appropriate state-specific data.

	 * @param NextState The destination state
	 * @param SendFlags Behavior for how the PackageData should be added/removed from the queues corresponding to
	 *                  the new and old states. Callers may want to manage queue membership directly for better
	 *                  performance; removing from the middle is more expensive than popping from the front.
	 *                  See definition of ESendFlags for a description of the behavior controlled by SendFlags.
	 * @param ReleaseSaveReason Explanation for why the state is changing, used for debugging.
	 */
	void SendToState(EPackageState NextState, ESendFlags SendFlags, EStateChangeReason ReleaseSaveReason);

	/**
	 * Stall the package into the target stalled state, if it is in a valid source state for the target stalled state.
	 * Does nothing if not in a valid source state. SendFlags are passed into SendToState.
	 */
	void Stall(EPackageState TargetState, ESendFlags SendFlags);

	/**
	 * If the package is in a stalled state, returns the package to the active state that is a source state for that
	 * stalled state. Does nothing if not in a valid source state.SendFlags are passed into SendToState.
	 */
	void UnStall(ESendFlags SendFlags);

	/** Return whether the package is in one of the stalled states. */
	bool IsStalled() const;

	/* Debug-only code to assert that this PackageData is contained by the container matching its current state. */
	void CheckInContainer() const;
	/**
	 * Return true if and only if this PackageData is InProgress in the current CookOnTheFlyServer session.
	 * Some data is allocated/destroyed/verified when moving in and out of InProgress.
	 * InProgress means the CookOnTheFlyServer will in the future decide to cook, failtocook, or skip the PackageData.
	 */
	bool IsInProgress() const;

	/** Return true if the Package's current state is in the given Property Group */
	bool IsInStateProperty(EPackageStateProperty Property) const;

	/*
	 * CompletionCallback - A callback that is called when this PackageData next transitions from InProgress to not
	 * InProgress because of cook success, failure, skip, or cancel.
	 */
	/** Get a reference to the currently set callback, to e.g. move it into a local variable during execution. */
	FCompletionCallback& GetCompletionCallback();
	/**
	 * Add the given callback into this PackageData's callback field. It is invalid to call this function with a
	 * non-empty callback if this PackageData already has a CompletionCallback.
	 */
	void AddCompletionCallback(TConstArrayView<const ITargetPlatform*> TargetPlatforms,
		FCompletionCallback&& InCompletionCallback);

	/** Get/Set whether the package has been marked for cooking after all other packages, via -cooklast */
	void SetIsCookLast(bool bValue);
	bool GetIsCookLast() const;

	/**
	 * Get/Set a visited flag used when searching graphs of PackageData. User of the graph is responsible for 
	 * setting the bIsVisited flag back to empty when graph operations are done.
	 */
	bool GetIsVisited() const;
	void SetIsVisited(bool bValue);

	/**
	 * Return the LoadDependencies if they have been created, otherwise nullptr. LoadDependencies are stored in the
	 * incremental cook oplog and are used by CookRequestCluster to test invalidation of previous cookresults for the
	 * package and for other packages that have the package as a transitive build dependency.
	 */
	const FBuildResultDependenciesMap* GetLoadDependencies() const;
	/** Calculate the LoadDependencies if not already created. */
	void CreateLoadDependencies();
	void ClearLoadDependencies();

	/**
	 * The list of objects inside the package.  Only non-empty during saving; it is populated on demand by
	 * TryCreateObjectCache and is cleared when leaving the save state.
	 */
	TArray<FCachedObjectInOuter>& GetCachedObjectsInOuter();
	const TArray<FCachedObjectInOuter>& GetCachedObjectsInOuter() const;
	template <typename ArrayType>
	/** The list of platforms that were recorded as NeedsCooking when CachedObjeObjectsInOuter was recorded. */
	void GetCachedObjectsInOuterPlatforms(ArrayType& OutPlatforms) const;

	/** Validate that the CachedObjectsInOuter-dependent variables are empty, when entering save. */
	void CheckObjectCacheEmpty() const;
	/** Populate CachedObjectsInOuter if not already populated. Invalid to call except when in the save state. */
	void CreateObjectCache();
	/** Populate OutBuffer with all objects from the given Package; filters out objects not relevant to the cook.  */
	static void CreateObjectCache(TArray<FCachedObjectInOuter>& OutBuffer, UPackage* Package);
	/**
	 * Look for new Objects that were created during BeginCacheForCookedPlatformData calls, and if found add
	 * them to the ObjectCache and set state so that we call BeginCacheForCookedPlatformData on the new objects.
	 * ErrorExits if this creation of new objects happens too many times.
	 */
	EPollStatus RefreshObjectCache(bool& bOutFoundNewObjects);
	/** Clear the CachedObjectsInOuter list, when e.g. leaving the save state. */
	void ClearObjectCache();

	const int32& GetNumPendingCookedPlatformData() const;
	int32& GetNumPendingCookedPlatformData();
	const int32& GetCookedPlatformDataNextIndex() const;
	int32& GetCookedPlatformDataNextIndex();
	int32& GetNumRetriesBeginCacheOnObjects();
	static int32 GetMaxNumRetriesBeginCacheOnObjects();

	/** Get/Set the flag for whether CachedObjectsInOuter is populated. Always false except during save state. */
	bool GetHasSaveCache() const;
	void SetHasSaveCache(bool Value);

	/**
	 * Get/Set the SubState within EPackageState::Save. Outside of EPackageStateProperty::Saving it is always
	 * ESaveSubState::StartSave, and is ignored and logs an error if attempting to set to any other value.
	 */
	ESaveSubState GetSaveSubState() const;
	void SetSaveSubState(ESaveSubState Value);
	/** Set the SaveSubState to the next state after Value. */
	void SetSaveSubStateComplete(ESaveSubState Value);

	/**
	 * Check whether savestate contracts on the PackageData were invalidated by by e.g. garbage collection.
	 * Request demotion if so unless we have a contract to keep it, in which case it is fixed up.
	 */
	void UpdateSaveAfterGarbageCollect(bool& bOutDemote);

	/** Get/Set the flag for whether PrepareSave has been called and returned an error. */
	bool HasPrepareSaveFailed() const;
	void SetHasPrepareSaveFailed(bool bValue);

	bool IsPrepareSaveRequiresGC() const;
	void SetIsPrepareSaveRequiresGC(bool bValue);

	/** Validate that the BeginCacheForCookedPlatformData-dependent fields are empty, when entering save. */
	void CheckCookedPlatformDataEmpty() const;
	/**
	 * Clear the BeginCacheForCookedPlatformData-dependent fields, when leaving save.
	 * Caller must have already executed any required cancellation steps to avoid dangling pending operations.
	 */
	void ClearCookedPlatformData();

	/** Get/Set the Monitor's flag that counts whether this PackageData has finished cooking and with what result. */
	ECookResult GetMonitorCookResult() const;
	void SetMonitorCookResult(ECookResult Value);

	/** Remove all data about the given platform from all fields in this PackageData. */
	void OnRemoveSessionPlatform(const ITargetPlatform* Platform);

	/** Report whether this PackageData holds references to UObjects and would be affected by GarbageCollection. */
	bool HasReferencedObjects() const;

	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	// The PackagePreloader holds data necessary to preload and load the UPackage for this PackageData.
	// It is used during this PackageData's load state, but also during the load of packages that have
	// a transitive import of it, so it is refcounted.
	/** Return the PackagePreloader if it already exists, otherwise return nullptr. */
	TRefCountPtr<FPackagePreloader> GetPackagePreloader() const;
	/** Create the PackagePreloader if it does not already exist and return a non-null TRefCountPtr to it. */
	TRefCountPtr<FPackagePreloader> CreatePackagePreloader();
	/** Helper function for ~FPackagePreloader: clear the pointer this->PackagePreloader. */
	void OnPackagePreloaderDestroyed(FPackagePreloader& InPackagePreloader);

	// GenerationHelper is set on packages that are generator packages: they generate other packages
	// during cook. The PackageData for the generator package has a pointer to the GenerationHelper to look
	// it up, but does not keep it in memory. It is kept in memory by its internal state and by
	// ParentGenerationHelper references from its generated packages.
	/** Return the GenerationHelper if it already exists, otherwise return nullptr. */
	TRefCountPtr<FGenerationHelper> GetGenerationHelper() const;
	/**
	 * Return the GenerationHelper if it already exists and is initialized and is valid. If not initialized,
	 * load the package to initialize it. If it is not valid after initialization, return nullptr.
	 */
	TRefCountPtr<FGenerationHelper> GetGenerationHelperIfValid();
	/** Return the GenerationHelper if it already exists, otherwise create it without calling Initialize. */
	TRefCountPtr<FGenerationHelper> CreateUninitializedGenerationHelper();
	/**
	 * Return the GenerationHelper if it already exists, otherwise load the package and check all the objects
	 * to see whether any of them have a registered CookPackageSplitter. If a split object exists, create the
	 * GenerationHelper and return it. Otherwise return nullptr. If the splitter requires it, also return nullptr
	 * if the caller needs to call IsCachedCookedPlatformDataLoaded before creation.
	 */
	TRefCountPtr<FGenerationHelper> TryCreateValidGenerationHelper(bool bCookedPlatformDataIsLoaded,
		bool& bOutNeedWaitForIsLoaded);
	/** Helper function for ~FGenerationHelper: clear the pointer this->GenerationHelper. */
	void OnGenerationHelperDestroyed(FGenerationHelper& InGenerationHelper);
	/** Return whether the PackageData is a generated package created by a owning generator PackageData. */
	bool IsGenerated() const;
	/** Mark that the PackageData is a generated package created by a ParentGenerator. */
	void SetGenerated(FName InParentGenerator);
	/** Return the name of the generator package that generates this package, or NAME_None if not IsGenerated. */
	FName GetParentGenerator() const;
	/**
	 * Set the owning generator PackageData. Only valid to call with non-null if SetGenerated() has been called. Keeps
	 * the GenerationHelper referenced until SetParentGenerationHelper(nullptr) is called.
	 */
	void SetParentGenerationHelper(FGenerationHelper* InGenerationHelper, EStateChangeReason StateChangeReason,
		FCookGenerationInfo* InfoOfPackageInGenerator = nullptr);
	/** Return the ParentGenerator's GenerationHelper if the pointer to it has already been set on this. */
	TRefCountPtr<FGenerationHelper> GetParentGenerationHelper() const;
	/**
	 * Return the ParentGenerator's GenerationHelper if the pointer to it has already been set on this, otherwise
	 * look for it on the ParentGenerator by calling GetGenerationHelper, and if found, store a reference to it
	 * and return it, otherwise return null.
	 */
	TRefCountPtr<FGenerationHelper> GetOrFindParentGenerationHelper();
	/**
	 * Return the ParentGenerator's GenerationHelper if the pointer to it has already been set on this, otherwise
	 * look for it on the ParentGenerator by calling GetGenerationHelper, return the found generator or null.
	 * Does not cache the found generator on this if found.
	 */
	TRefCountPtr<FGenerationHelper> GetOrFindParentGenerationHelperNoCache();
	/**
	 * Return the ParentGenerator's GenerationHelper if the pointer to it has already been set on this, otherwise
	 * try to find or create it by calling TryCreateValidGenerationHelper on the ParentGenerator.
	 */
	TRefCountPtr<FGenerationHelper> TryCreateValidParentGenerationHelper();

	/**
	 * Get/Set the package's parent's CookPackageSplitter's value for DoesGeneratedRequireGenerator.
	 * Should only be called for generated packages.
	 */
	ICookPackageSplitter::EGeneratedRequiresGenerator DoesGeneratedRequireGenerator() const;
	void SetDoesGeneratedRequireGenerator(ICookPackageSplitter::EGeneratedRequiresGenerator Value);

	/**
	 * Return the instigator for this package for the given reachability bits. The Instigator is the first code
	 * location or referencing package that causes the package to enter the requested state for the given
	 * reachability bit. If multiple bits of Reachability are specified, the most important one that has an
	 * instigator is returned. EReachability::None is invalid.
	 */
	const FInstigator& GetInstigator(EReachability InReachability) const;
	bool HasInstigator(EReachability InReachability) const;
	/** Setting the instigator is mostly private - it should only be done during clustering. */
	void SetInstigator(FRequestCluster& Cluster, EReachability InReachability, FInstigator&& InInstigator);
	void SetInstigator(FCookWorkerClient& Client, EReachability InReachability, FInstigator&& InInstigator);
	void SetInstigator(FGenerationHelper& InHelper, EReachability InReachability, FInstigator&& InInstigator);

	/** Get whether COTFS is keeping this package referenced referenced during GC. */
	bool IsKeepReferencedDuringGC() const;
	/** Set whether COTFS is keeping this package referenced referenced during GC. */
	void SetKeepReferencedDuringGC(bool Value);

	/** Return whether the package was cooked during this session */
	bool GetWasCookedThisSession() const;

	/** For MultiProcessCooks, Get the id of the worker this Package is assigned to; InvalidId means owned by local. */
	FWorkerId GetWorkerAssignment() const;
	/**
	 * Set the id of the worker this Package is assigned to. If value changes from Valid to Invalid and SendFlags
	 * includes QueueRemove, also calls NotifyRemovedFromWorker.
	 */
	void SetWorkerAssignment(FWorkerId InWorkerAssignment, ESendFlags SendFlags = ESendFlags::QueueAddAndRemove);
	/** Get the workerid that is the only worker allowed to cook this package; InvalidId means no constraint. */
	FWorkerId GetWorkerAssignmentConstraint() const;
	/** Set the workerid that is the only worker allowed to cook this package; default is InvalidId; */
	void SetWorkerAssignmentConstraint(FWorkerId InWorkerAssignment)
	{
		WorkerAssignmentConstraint = InWorkerAssignment;
	}

	/** Marshall this PackageData to a ConstructData that is used later or on a remote machine to reconstruct it. */
	FConstructPackageData CreateConstructData();

	/** Storage for dependencies discovered for the package during cook that are not reported by the AssetRegistry. */
	void AddDiscoveredDependency(const FDiscoveredPlatformSet& Platforms, FPackageData* Dependency,
		EInstigator Category);
	void ClearDiscoveredDependencies();
	TMap<FPackageData*, EInstigator>& CreateOrGetDiscoveredDependencies(const ITargetPlatform* TargetPlatform);
	TMap<FPackageData*, EInstigator>* GetDiscoveredDependencies(const ITargetPlatform* TargetPlatform);

	/** Storage for warnings and errors about the package discovered during cook; these are saved into the oplog. */
	void AddLogMessage(FReplicatedLogData&& LogData);
	TConstArrayView<FReplicatedLogData> GetLogMessages() const;
	void ClearLogMessages();
	bool HasReplayedLogMessages() const;
	void SetHasReplayedLogMessages(bool bValue);
	bool HasBeenLoadedForCooking() const;
	void SetHasBeenLoadedForCooking(bool Value);

	/**
	 * Return the platforms for which the given Package has been marked reachable.
	 * If the package does not exist, return the COTFS's list of Session platforms
	 */
	template <typename ArrayType>
	static void GetReachablePlatformsForInstigator(EReachability InReachability, UCookOnTheFlyServer& COTFS,
		FPackageData* InInstigator, ArrayType& Platforms);
	template <typename ArrayType>
	static void GetReachablePlatformsForInstigator(EReachability InReachability, UCookOnTheFlyServer& COTFS,
		FName InInstigator, ArrayType& Platforms);
private:
	friend struct UE::Cook::FPackageDatas;

	void UpdateContainerUrgency(EUrgency OldUrgency, EUrgency NewUrgency);
	void SetInstigatorInternal(EReachability InReachability, FInstigator&& InInstigator);
	static const TArray<const ITargetPlatform*>& GetSessionPlatformsInternal(UCookOnTheFlyServer& COTFS);
	static void AddReachablePlatformsInternal(FPackageData& PackageData, EReachability InReachability,
		TConstArrayView<const ITargetPlatform*> Platforms, FInstigator&& InInstigator);
	static void QueueAsDiscoveredInternal(FPackageData& PackageData, FInstigator&& InInstigator,
		FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency InUrgency);

	/**
	 * Set the FileName of the file that contains the package. This member is private because FPackageDatas
	 * keeps a map from FileName to PackageData that needs to be updated in sync with it.
	 */
	void SetFileName(const FName& InFileName);

	/**
	 * Set the State of this PackageDAta in the CookOnTheFlyServer's session. This member is private because it
	 * needs to be updated in sync with other contract data.
	 */
	void SetState(EPackageState NextState);

private:
	/**
	 * Helper function to call the given EdgeFunction (e.g. OnExitInProgress)
	 * when a property changes from true to false.
	 */
	typedef void (FPackageData::*FEdgeFunction)();
	inline void UpdateDownEdge(bool bOld, bool bNew, const FEdgeFunction& EdgeFunction);
	/**
	 * Helper function to call the given EdgeFunction (e.g. OnEnterInProgress)\
	 * when a property changes from false to true.
	 */
	inline void UpdateUpEdge(bool bOld, bool bNew, const FEdgeFunction& EdgeFunction);

	/* Entry/Exit gates for PackageData states, used to enforce state contracts and free unneeded memory. */
	void OnEnterIdle();
	void OnExitIdle();
	void OnEnterRequest();
	void OnExitRequest();
	void OnEnterAssignedToWorker();
	void OnExitAssignedToWorker();
	void OnEnterLoad();
	void OnExitLoad();
	void OnEnterSaveActive();
	void OnExitSaveActive();
	void OnEnterSaveStalledRetracted();
	void OnExitSaveStalledRetracted();
	void OnEnterSaveStalledAssignedToWorker();
	void OnExitSaveStalledAssignedToWorker();
	/* Entry/Exit gates for Properties shared between multiple states */
	void OnExitInProgress(EStateChangeReason StateChangeReason);
	void OnEnterInProgress();
	void OnExitSaving(EStateChangeReason ReleaseSaveReason, EPackageState NewState);
	void OnEnterSaving();
	void OnExitAssignedToWorkerProperty();
	void OnEnterAssignedToWorkerProperty();

	void OnPackageDataFirstMarkedReachable(EReachability InReachability, FInstigator&& InInstigator);

	FGenerationHelper* GenerationHelper = nullptr;
	TRefCountPtr<FGenerationHelper> ParentGenerationHelper;
	/** Data for each platform that has been interacted with by *this. */
	TSortedMap<const ITargetPlatform*, FPackagePlatformData, TInlineAllocator<1>> PlatformDatas;

	TArray<FCachedObjectInOuter> CachedObjectsInOuter;
	FCompletionCallback CompletionCallback;
	TUniquePtr<TMap<const ITargetPlatform*, TMap<FPackageData*, EInstigator>>> DiscoveredDependencies;
	TUniquePtr<FBuildResultDependenciesMap> LoadDependencies;
	TUniquePtr<TArray<FReplicatedLogData>> LogMessages;
	FName PackageName;
	FName FileName;
	FName ParentGenerator;

	TWeakObjectPtr<UPackage> Package;
	/** The one-per-CookOnTheFlyServer owner of this PackageData. */
	FPackageDatas& PackageDatas;
	FPackagePreloader* PackagePreloader = nullptr;
	uint32 LeafToRootRank = MAX_uint32;
	int32 NumPendingCookedPlatformData = 0;
	int32 CookedPlatformDataNextIndex = -1;
	int32 NumRetriesBeginCacheOnObject = 0;
	FInstigator Instigator;
	FInstigator BuildInstigator;

	FWorkerId WorkerAssignment = FWorkerId::Invalid();
	FWorkerId WorkerAssignmentConstraint = FWorkerId::Invalid();
	uint32 State : int32(EPackageState::BitCount);
	uint32 SaveSubState : int32(ESaveSubState::BitCount);
	uint32 SuppressCookReason : int32(ESuppressCookReason::BitCount);
	uint32 Urgency : int32(EUrgency::BitCount);
	uint32 bIsCookLast : 1;
	uint32 bIsVisited : 1;
	uint32 bHasSaveCache : 1;
	uint32 bPrepareSaveFailed : 1;
	uint32 bPrepareSaveRequiresGC : 1;
	uint32 MonitorCookResult : (int) ECookResult::NumBits;
	uint32 bGenerated : 1;
	uint32 bKeepReferencedDuringGC : 1;
	uint32 bWasCookedThisSession : 1;
	static_assert(static_cast<uint32>(ICookPackageSplitter::EGeneratedRequiresGenerator::Count) <= 4, "We are storing Enum value in 2 bits");
	uint32 DoesGeneratedRequireGeneratorValue : 2;
	uint32 bHasReplayedLogMessages : 1;
	uint32 bHasBeenLoadedForCooking : 1;
};

/**
 * Stores information about the pending action in response to a single call to BeginCacheForCookedPlatformData that
 * was made on a given object for the given platform, when saving the given PackageData.
 * This instance will remain alive until the object returns true from IsCachedCookedPlatformDataLoaded.
 * If the PackageData's save was canceled, this struct also becomes responsible for cleanup of the cached data by
 * calling ClearAllCachedCookedPlatformData.
 */
struct FPendingCookedPlatformData
{
	FPendingCookedPlatformData(UObject* InObject, const ITargetPlatform* InTargetPlatform,
		FPackageData& InPackageData, bool bInNeedsResourceRelease, UCookOnTheFlyServer& InCookOnTheFlyServer);
	FPendingCookedPlatformData(FPendingCookedPlatformData&& Other);
	FPendingCookedPlatformData(const FPendingCookedPlatformData& Other) = delete;
	~FPendingCookedPlatformData();
	FPendingCookedPlatformData& operator=(const FPendingCookedPlatformData& Other) = delete;
	FPendingCookedPlatformData& operator=(const FPendingCookedPlatformData&& Other) = delete;

	/** Helper for both pending and synchronous; call ClearCachedCookedPlatformData and related teardowns. */
	static void ClearCachedCookedPlatformData(UObject* Object, FPackageData& PackageData, bool bCompletedSuccesfully);

	/**
	 * Call IsCachedCookedPlatformDataLoaded on the object if it has not already returned true.
	 * If IsCachedCookedPlatformDataLoaded returns true, this function releases all held resources related to the
	 * pending call, and returns true. Otherwise takes no action and returns false.
	 * Returns true and early exits if IsCachedCookedPlatformDataLoaded has already returned true.
	 */
	bool PollIsComplete();
	/** Release all held resources related to the pending call, if they have not already been released. */
	void Release();

	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	/** The object with the pending call. */
	FWeakObjectPtr Object;
	/** The platform that was passed to BeginCacheForCookedPlatformData. */
	const ITargetPlatform* TargetPlatform;
	/** The PackageData that owns the call; the pending count needs to be updated on this PackageData. */
	FPackageData& PackageData;
	/** Backpointer to the CookOnTheFlyServer to allow releasing of resources for the pending call. */
	UCookOnTheFlyServer& CookOnTheFlyServer;
	/**
	 * Non-null only in the case of a cancel. Used to synchronize release of shared resources used by all
	 * FPendingCookedPlatformData for the various TargetPlatforms of a given object.
	 */
	FPendingCookedPlatformDataCancelManager* CancelManager;
	/* Saved copy of the ClassName to use for resource releasing. */
	FName ClassName;
	/** Polling performance field: how many UpdatePeriods should we wait before polling again. */
	int32 UpdatePeriodMultiplier = 1;
	/** Flag for whether we have executed the release. */
	bool bHasReleased;
	/**
	 * Flag for whether the CookOnTheFlyServer requires resource tracking for the object's
	 * BeginCacheForCookedPlatformData call.
	 */
	bool bNeedsResourceRelease;
};

/**
 * Stores information about all of the FPendingCookedPlatformData for a given object, so that resources shared by
 * all of the FPendingCookedPlatformData can be released after they are all released.
 */
struct FPendingCookedPlatformDataCancelManager
{
	/** The number of FPendingCookedPlatformData for the given object that are still pending. */
	int32 NumPendingPlatforms;
	/** Decrement the reference count, and if it has reached 0, release the resources and delete *this. */
	void Release(FPendingCookedPlatformData& Data);
};

using FPackageDataSortedSet = TFifoSet<FPackageData*>;

/**
 * The container class for PackageData pointers that are InProgress in a CookOnTheFlyServer. These containers
 * most frequently do queue push/pop operations, but also commonly need to support iteration.
 */
class FPackageDataQueue : public TRingBuffer<FPackageData*>
{
	using TRingBuffer<FPackageData*>::TRingBuffer;
};

/**
 * A monitor class held by an FPackageDatas to provide reporting and decision making based on aggregated-data
 * across all InProgress or completed FPackageData.
 */
struct FPackageDataMonitor
{
public:
	FPackageDataMonitor();

	/** Report the number of FPackageData that are in any non-idle state and need action by CookOnTheFlyServer. */
	int32 GetNumInProgress() const;
	int32 GetNumPreloadAllocated() const;
	/** Report the number of packages that have cooked any platform. Used by CookCommandlet progress reporting. */
	int32 GetNumCooked(ECookResult CookResult) const;
	/**
	 * Report the number of FPackageData that are currently set to the given urgency level.
	 * Used to check if a Pump function needs to exit to handle urgent PackageData in other states.
	 */
	int32 GetNumUrgent(EUrgency UrgencyLevel) const;
	/**
	 * Report the number of FPackageData that are in the given state and are set to the given urgency level. Only
	 * valid to call on states that are in the InProgress set, such as Save.
	 * Used to prioritize scheduler actions.
	 */
	int32 GetNumUrgent(EPackageState InState, EUrgency UrgencyLevel) const;
	/** Report the number of CookLast packages. */
	int32 GetNumCookLast() const;
	/** Report the number of CookLast packages in the given state. */
	int32 GetNumCookLast(EPackageState InState) const;

	/** Callback called from FPackageData when it transitions to or from inprogress. */
	void OnInProgressChanged(FPackageData& PackageData, bool bInProgress);
	void OnPreloadAllocatedChanged(FPackageData& PackageData, bool bPreloadAllocated);
	/**
	 * Callback called from FPackageData when it has set a platform to CookAttempted=true and it does not have
	 * any others cooked.
	 */
	void OnFirstCookedPlatformAdded(FPackageData& PackageData, ECookResult CookResult);
	/**
	 * Callback called from FPackageData when it has set a platform to CookAttempted=false and it does not have
	 * any others cooked.
	 */
	void OnLastCookedPlatformRemoved(FPackageData& PackageData);
	/** Callback called from FPackageData when it has changed its urgency. */
	void OnUrgencyChanged(FPackageData& PackageData, EUrgency OldUrgency, EUrgency NewUrgency);
	/** Callback called from FPackageData when it has changed its value of IsCookLast. */
	void OnCookLastChanged(FPackageData& PackageData);
	/** Callback called from FPackageData when it has changed its state. */
	void OnStateChanged(FPackageData& PackageData, EPackageState OldState);

	int32 GetMPCookAssignedFenceMarker() const;
	int32 GetMPCookRetiredFenceMarker() const;

private:
	/** Increment or decrement the NumUrgent counter for the given state. */
	void TrackUrgentRequests(EPackageState State, EUrgency Urgency, int32 Delta);
	/** Increment or decrement the NumCookLast counter for the given state. */
	void TrackCookLastRequests(EPackageState State, int32 Delta);

	int32 NumInProgress = 0;
	int32 NumCooked[(uint8)ECookResult::Count]{};
	int32 NumPreloadAllocated = 0;
	int32 NumUrgentInState[static_cast<uint32>(EPackageState::Count)][static_cast<uint32>(EUrgency::Count)];
	int32 NumCookLastInState[static_cast<uint32>(EPackageState::Count)];
	int32 MPCookAssignedFenceMarker = 0;
	int32 MPCookRetiredFenceMarker = 0;
};

struct FDiscoveryQueueElement
{
	FPackageData* PackageData;
	FInstigator Instigator;
	FDiscoveredPlatformSet ReachablePlatforms;
	UE::Cook::EUrgency Urgency;
};

/**
 * A container for FPackageDatas in the Request state. This container needs to support fast find and remove,
 * RequestClusters, staging for packages not yet in request clusters, and a FIFO for ready requests
 * using AddRequest/PopRequest that is overridden for urgent requests to push them to the front.
 */
class FRequestQueue
{
public:
	bool IsEmpty() const;
	uint32 Num() const;
	uint32 Remove(FPackageData* PackageData);
	bool Contains(const FPackageData* PackageData) const;
	void Empty();

	void AddRequest(FPackageData* PackageData, bool bForceUrgent=false);

	bool HasRequestsToExplore() const;
	uint32 ReadyRequestsNum() const;
	bool IsReadyRequestsEmpty() const;
	FPackageData* PopReadyRequest();
	void AddReadyRequest(FPackageData* PackageData, bool bForceUrgent = false);
	uint32 RemoveRequest(FPackageData* PackageData);
	uint32 RemoveRequestExceptFromCluster(FPackageData* PackageData, FRequestCluster* ExceptFromCluster);
	void UpdateUrgency(FPackageData* PackageData, EUrgency bOldUrgency, EUrgency NewUrgency);

	TPackageDataMap<ESuppressCookReason>& GetRestartedRequests();
	/**
	 * Unlike non-discovery containers on PackageData, GetDiscoveryQueue and GetBuildDependencyDiscoveryQueue
	 * are not ownership containers.
	 * PackageDatas in the DiscoveryQueues are PackageDatas we need to look at - in the right order during
	 * PumpRequests - they can be in any state and are owned by another container (or are in the idle state).
	 */
	TRingBuffer<FDiscoveryQueueElement>& GetDiscoveryQueue();
	TRingBuffer<FPackageData*>& GetBuildDependencyDiscoveryQueue();
	TRingBuffer<TUniquePtr<FRequestCluster>>& GetRequestClusters();
	FPackageDataSortedSet& GetReadyRequestsUrgent();
	FPackageDataSortedSet& GetReadyRequestsNormal();
	/**
	 * Add an FPackageData by name that will be notified when all packages that were present in the DiscoveryQueue
	 * before the fence was created have been assigned or demoted to idle.
	 */
	void AddRequestFenceListener(FName PackageName);
	/** Called when the DiscoveryQueue has been flushed and fence listeners can therefore be notified. */
	void NotifyRequestFencePassed(FPackageDatas& PackageDatas);

private:
	TPackageDataMap<ESuppressCookReason> RestartedRequests;
	TRingBuffer<FDiscoveryQueueElement> DiscoveryQueue;
	TRingBuffer<FPackageData*> BuildDependencyDiscoveryQueue;
	TRingBuffer<TUniquePtr<FRequestCluster>> RequestClusters; // Not trivially relocatable, so must be TUniquePtr
	TSet<FName> RequestFencePackageListeners;
	FPackageDataSortedSet UrgentRequests;
	FPackageDataSortedSet NormalRequests;
};

/** A wrapper around a TRefCountPtr<FPackagePreloader> that defines operator< for FPackagePreloaderPriorityQueue. */
struct FPackagePreloaderPriorityWrapper
{
	TRefCountPtr<FPackagePreloader> Payload;
	bool operator<(const FPackagePreloaderPriorityWrapper& Other) const;
};

/**
 * Priority queue for FPackagePreloaders in the PendingKick substate, prioritized mostly by LeafToRoot order,
 * but with various exceptions. Controls which PendingKick preloader will next be kicked.
 */
class FPackagePreloaderPriorityQueue
{
public:
	bool IsEmpty() const;
	void Add(TRefCountPtr<FPackagePreloader> Preloader);
	void Remove(const TRefCountPtr<FPackagePreloader>& Preloader);
	TRefCountPtr<FPackagePreloader> PopFront();

private:
	TArray<FPackagePreloaderPriorityWrapper> Heap;
};

/**
 * Container for FPackageDatas in the Load state. Has a single InProgress container, and  multiple subqueues which
 * contain pointers to PackagePreloaders of packages which might be requested because they're in the load state, or
 * requested because even though they are in another state (e.g. request or idle), one of the packages in the load
 * state imports them so we want to preload and load them for better load performance of the referencer package.
 */
class FLoadQueue
{
public:
	bool IsEmpty();
	int32 Num() const;
	void Add(FPackageData* PackageData);
	bool Contains(const FPackageData* PackageData) const;
	uint32 Remove(FPackageData* PackageData);
	void UpdateUrgency(FPackageData* PackageData, EUrgency bOldUrgency, EUrgency NewUrgency);
	TSet<FPackageData*>::TRangedForIterator begin();
	TSet<FPackageData*>::TRangedForIterator end();

	TRingBuffer<FPackageData*> Inbox;
	FPackagePreloaderPriorityQueue PendingKicks;
	TSet<TRefCountPtr<FPackagePreloader>> ActivePreloads;
	TRingBuffer<TRefCountPtr<FPackagePreloader>> ReadyForLoads;
	TSet<FPackageData*> InProgress;
};

/** Data duplicated from FPackageData that is stored separately for read/write from any thread. */
struct FThreadsafePackageData
{
	FInstigator Instigator;
	FName Generator;
	bool bInitialized : 1;
	bool bHasLoggedDiscoveryWarning : 1;
	bool bHasLoggedDependencyWarning : 1;

	FThreadsafePackageData();
};

typedef TArray<FPendingCookedPlatformData> FPendingCookedPlatformDataContainer;

/*
 * Class that manages the list of all PackageDatas for a CookOnTheFlyServer. PackageDatas is an associative
 * array for extra data about a package (e.g. the cook results) that is needed by the CookOnTheFlyServer.
 * FPackageData are allocated once and never destroyed or moved until the CookOnTheFlyServer is destroyed.
 * Memory on the FPackageData is allocated and deallocated as necessary for its current state.
 * FPackageData are mapped by PackageName and by FileName.
 * This class also manages all non-temporary references to FPackageData such as the SaveQueue and RequeustQueue.
*/
struct FPackageDatas : public FGCObject
{
public:
	FPackageDatas(UCookOnTheFlyServer& InCookOnTheFlyServer);
	~FPackageDatas();
	/** Called when the initial AssetRegistry search is done and it can be used to determine package existence. */
	static void OnAssetRegistryGenerated(IAssetRegistry& InAssetRegistry);

	/** Called each time BeginCook is called, to initialize settings from config */
	void SetBeginCookConfigSettings(FStringView CookShowInstigator);

	/** FGCObject interface function - return a debug name describing this FGCObject. */
	virtual FString GetReferencerName() const override;
	/**
	 * FGCObject interface function - add the objects referenced by this FGCObject to the ReferenceCollector.
	 * This class forwards the query on to the CookOnTheFlyServer.
	 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Return the Monitor used to report aggregated information about FPackageDatas. */
	FPackageDataMonitor& GetMonitor();
	/** Return the backpointer to the CookOnTheFlyServer */
	UCookOnTheFlyServer& GetCookOnTheFlyServer();

	/** Return and increment the RoofToLankRank that should be assigned to the next PackageData needing one. */
	uint32 GetNextLeafToRootRank();
	/** Reset the LeafToRootRank index back to 0; called when the cook is reset. */
	void ResetLeafToRootRank();

	/**
	 * Return the RequestQueue used by the CookOnTheFlyServer. The RequestQueue is the mostly-FIFO list of
	 * PackageData that need to be cooked.
	 */
	FRequestQueue& GetRequestQueue();

	/** Return the Set that holds unordered all PackageDatas that are in the AssignedToWorker state. */
	TSet<FPackageData*>& GetAssignedToWorkerSet();

	/**
	 * Return the LoadQueue used by CookOnTheFlyServer. Container for packages in the Load state and for various
	 * data that manages their passage through the state.
	 */
	FLoadQueue& GetLoadQueue();
	/**
	 * Return the SaveQueue used by the CookOnTheFlyServer. The SaveQueue is the performance-sorted list of
	 * PackageData that have been loaded and need to start or are only part way through saving.
	 */
	FPackageDataQueue& GetSaveQueue();

	/** Return the Set that holds unordered all PackageDatas that are in one of the SaveStalled states. */
	TSet<FPackageData*>& GetSaveStalledSet();

	/**
	Return the PackageData for the given PackageName and FileName; no validation is done on the names.
	 * Creates the PackageData if it does not already exist.
	 */
	FPackageData& FindOrAddPackageData(const FName& PackageName, const FName& NormalizedFileName);

	/** Return the PackageData with the given PackageName if one exists, otherwise return nullptr. */
	FPackageData* FindPackageDataByPackageName(const FName& PackageName);
	/**
	 * Return a pointer to the PackageData for the given PackageName. If one does not already exist,
	 * find its FileName on disk and create the PackageData.  Will fail if the path is not mounted,
	 * or if the file does not exist and bRequireExists is true. If it fails, returns nullptr.
	 * 
	 * @param bRequireExists If true, returns nullptr if the PackageData does not already exist and the
	 *                       package does not exist on disk in the Workspace Domain.
	 *                       If false, creates the PackageData so long as the package path is mounted.
	 * @param bCreateAsMap Only used if the PackageData does not already exist, the package does not exist
	 *                     on disk and bRequireExists is false. If true the extension is set to .umap,
	 *                     if false, it is set to .uasset.
	 */
	FPackageData* TryAddPackageDataByPackageName(const FName& PackageName, bool bRequireExists = true,
		bool bCreateAsMap = false);
	/**
	 * Return a reference to the PackageData for the given PackageName. If one does not already exist,
	 * find its FileName on disk and create the PackageData. Will fail if the path is not mounted,
	 * or if the file does not exist and bRequireExists is true. If it fails, this function will assert.
	 *
	 * @param bRequireExists If true, asserts if the PackageData does not already exist and the
	 *                       package does not exist on disk in the Workspace Domain.
	 *                       If false, creates the PackageData so long as the package path is mounted.
	 * @param bCreateAsMap Only used if the PackageData does not already exist, the package does not exist
	 *                     on disk and bRequireExists is false. If true the extension is set to .umap,
	 *                     if false, it is set to .uasset.
	 */
	FPackageData& AddPackageDataByPackageNameChecked(const FName& PackageName, bool bRequireExists = true,
		bool bCreateAsMap = false);

	void UpdateThreadsafePackageData(const FPackageData& PackageData);
	/** Callback == void (*Callback)(FThreadsafePackageData& Value, bool bNew) */
	template<typename CallbackType>
	void UpdateThreadsafePackageData(FName PackageName, CallbackType&& Callback);
	TOptional<FThreadsafePackageData> FindThreadsafePackageData(FName PackageName);
	/**
	 * Return the PackageData with the given FileName if one exists, otherwise return nullptr.
	 */
	FPackageData* FindPackageDataByFileName(const FName& InFileName);
	/**
	 * Return a pointer to the PackageData for the given FileName.
	 * If one does not already exist, verify the FileName on disk and create the PackageData.
	 * If no filename exists for the package on disk, return nullptr.
	 */
	FPackageData* TryAddPackageDataByFileName(const FName& InFileName);
	/**
	 * Return a pointer to the PackageData for the given FileName.
	 * If one does not already exist, verify the FileName on disk and create the PackageData.
	 * If no filename exists for the package on disk, return nullptr.
	 * FileName must have been normalized by FPackageDatas::GetStandardFileName.
	 * 
	 * @param bExactMatchRequired If true, returns true even if the FileName is not an exact match, e.g.
	 *                            because it is missing the extension.
	 * @param FoundFileName If non-null, will be set to the discovered FileName, only useful if !bExactMatchRequired
	 */
	FPackageData* TryAddPackageDataByStandardFileName(const FName& InFileName, bool bExactMatchRequired=true,
		FName* OutFoundFileName=nullptr);
	/**
	 * Return a reference to the PackageData for the given FileName.
	 * If one does not already exist, verify the FileName on disk and create the PackageData. Asserts if FileName
	 * does not exist; caller is claiming it does.
	 */
	FPackageData& AddPackageDataByFileNameChecked(const FName& FileName);

	/**
	 * Return the local path for the given packagename.
	 * 
	 * @param PackageName The LongPackageName of the Package
	 * @param bRequireExists If true, fails if the package does not exist on disk in the Workspace Domain.
	 *                       If false, returns the filename it would have so long as the package path is mounted.
	 * @param bCreateAsMap Only used if bRequireExists is false and the path does not exist. If true
	 *                     the extension is set to .umap, if false, it is set to .uasset.
	 * @return Local WorkspaceDomain path in FPaths::MakeStandardFilename form, or NAME_None if it does not exist.
	 */
	FName GetFileNameByPackageName(FName PackageName, bool bRequireExists = true, bool bCreateAsMap = false);

	/**
	 * Return the local path for the given LongPackageName or unnormalized localpath.
	 *
	 * @param PackageName The name or local path for the package.
	 * @param bRequireExists If true, fails if the package does not exist on disk in the Workspace Domain.
	 *                       If false, returns the filename it would have so long as the package path is mounted.
	 * @param bCreateAsMap Only used if bRequireExists is false and the path does not exist. If true
	 *                     the extension is set to .umap, if false, it is set to .uasset.
	 * @return Local WorkspaceDomain path in FPaths::MakeStandardFilename form, or NAME_None if it does not exist.
	 */
	bool TryGetNamesByFlexName(FName PackageOrFileName, FName* OutPackageName = nullptr, FName* OutFileName = nullptr,
		bool bRequireExists = true, bool bCreateAsMap = false);

	/**
	 * Uncached; reads the AssetRegistry and disk to find the filename for the given PackageName.
	 * This is the same function that the other caching lookup functions use internally.
	 * It can be called from multiple threads on a batch of files to accelerate the lookup, and
	 * the results passed to FindOrAddPackageData(PackageName, FileName) which will then skip the lookup.
	 * 
	 * @param bRequireExists If true, fails if the package does not exist on disk in the Workspace Domain.
	 *                       If false, returns the filename it would have so long as the package path is mounted.
	 * @param bCreateAsMap Only used if bRequireExists is false and the path does not exist. If true
	 *                     the extension is set to .umap, if false, it is set to .uasset.
	 * @return The FPaths::MakeStandardFilename format of the localpath for the packagename, or NAME_None if
	 *         it does not exist.
	 */
	static FName LookupFileNameOnDisk(FName PackageName, bool bRequireExists = true, bool bCreateAsMap = false);
	/** Normalize the given FileName for use in looking up the cached data associated with the FileName. */
	static FName GetStandardFileName(FName FileName);
	/** Normalize the given FileName for use in looking up the cached data associated with the FileName. */
	static FName GetStandardFileName(FStringView FileName);

	/** Create and mark-cooked a batch of PackageDatas, used by DLC for cooked-in-earlier-release packages. */
	void AddExistingPackageDatasForPlatform(TConstArrayView<FConstructPackageData> ExistingPackages,
		const ITargetPlatform* TargetPlatform, bool bExpectPackageDatasAreNew, int32& OutPackageDataFromBaseGameNum);

	/**
	 * Try to find the PackageData for the given PackageName.
	 * If it exists, change the PackageData's FileName if current FileName is different and update the map to it.
	 * This is called in response to the package being moved in the editor or if we attempted to load a FileName
	 * and got redirected to another FileName.
	 * Returns the PackageData if it exists.
	 */
	FPackageData* UpdateFileName(FName PackageName);

	/** Report the number of packages that have cooked any platform. Used by cook commandlet progress reporting. */
	int32 GetNumCooked();
	int32 GetNumCooked(ECookResult CookResult);
	/**
	 * Append to SucceededPackages all packages that have cooked any platform with success, and to
	 * FailedPackages all packages that were committed for any platform with other results. The output variables are
	 * permitted to point to the same array.
	 */
	void GetCommittedPackagesForPlatform(const ITargetPlatform* Platform, TArray<FPackageData*>& SucceededPackages,
		TArray<FPackageData*>& FailedPackages);

	/**
	 * Delete all PackageDatas and free all other memory used by this FPackageDatas.
	 * For performance reasons, should only be called on destruction.
	 */
	void Clear();
	/** Set all platforms to not cooked in all PackageDatas. Used to e.g. invalidate previous cooks. */
	void ClearCookedPlatforms();
	/** Set PackageDatas that are in the given set to not cooked for the given platform. */
	void ClearCookResultsForPackages(const TSet<FName>& InPackages, const ITargetPlatform* TargetPlatform,
		int32& InOutNumBaseGamePackages);
	/** Remove all data about the given platform from all PackageDatas and other memory used by *this. */
	void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform);

	/** Enumerate PendingPlatformDatas: the list of pending calls to BeginCacheForCookedPlatformData. */
	template <typename FunctionType>
	void ForEachPendingCookedPlatformData(const FunctionType& Function);
	int32 GetPendingCookedPlatformDataNum() const;
	void AddPendingCookedPlatformData(FPendingCookedPlatformData&& Data);
	/**
	 * Return pointers to the PendingCookedPlatformDatas for the given object, if any exist. These
	 * pointers are to TArray elements and must not be dereferenced after any modifications to
	 * PendingCookedPlatformData: e.g. AddPendingCookedPlatformData or PollPendingCookedPlatformDatas.
	 */
	TArray<FPendingCookedPlatformData*> GetPendingCookedPlatformDatas(UObject* Object);

	/**
	 * Iterate over all elements in PendingCookedPlatformDatas and check whether they have completed,
	 * releasing their resources and pending count if so.
	 */
	void PollPendingCookedPlatformDatas(bool bForce, double& LastCookableObjectTickTime, int32& OutNumRetired);
	void ClearCancelManager(FPackageData& PackageData);

	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	/** Called when a PackageData assigns its instigator, for debugging. */
	void DebugInstigator(FPackageData& PackageData);

	/** Enter the required locks and enumerate all created PackageDatas. */
	template <typename CallbackType>
	void LockAndEnumeratePackageDatas(CallbackType&& Callback);

	FMapOfCachedCookedPlatformDataState& GetCachedCookedPlatformDataObjects();
	void CachedCookedPlatformDataObjectsPostGarbageCollect(const TSet<UObject*>& SaveQueueObjectsThatStillExist);
	void CachedCookedPlatformDataObjectsOnDestroyedOutsideOfGC(const UObject* DestroyedObject);

private:
	/**
	 * Construct a new FPackageData with the given PackageName and FileName and store references to it in the maps.
	 * New FPackageData are always created in the Idle state.
	 */
	FPackageData& CreatePackageData(FName PackageName, FName FileName);
	/** Called from within ExistenceLock, enumerate all created PackageDatas. */
	template <typename CallbackType>
	void EnumeratePackageDatasWithinLock(CallbackType&& Callback);
	/** Return whether a filename exists for the package on disk, and return it, unnormalized. */
	static bool TryLookupFileNameOnDisk(FName PackageName, FString& OutFileName);
	/** Return the corresponding PackageName if the normalized filename exists on disk. */
	static FName LookupPackageNameOnDisk(FName NormalizedFileName, bool bExactMatchRequired, FName& FoundFileName);

	/** Allocator for PackageDatas Guarded by ExistenceLock. */
	TTypedBlockAllocatorFreeList<FPackageData> Allocator;
	FPackageDataMonitor Monitor;
	/** Guarded by ExistenceLock */
	TMap<FName, FPackageData*> PackageNameToPackageData;
	/** Guarded by ExistenceLock */
	TMap<FName, FPackageData*> FileNameToPackageData;
	/* Guarded by ExistenceLock. Duplicates information on FPackageData, but can be read/write from any thread. */
	TMap<FName, FThreadsafePackageData> ThreadsafePackageDatas;
	TRingBuffer<FPendingCookedPlatformDataContainer> PendingCookedPlatformDataLists;
	FMapOfCachedCookedPlatformDataState CachedCookedPlatformDataObjects;
	uint32 NextLeafToRootRank = 0;
	int32 PendingCookedPlatformDataNum = 0;
	FRequestQueue RequestQueue;
	TSet<FPackageData*> AssignedToWorkerSet;
	TSet<FPackageData*> SaveStalledSet;
	FLoadQueue LoadQueue;
	FPackageDataQueue SaveQueue;
	UCookOnTheFlyServer& CookOnTheFlyServer;
	mutable FRWLock ExistenceLock;
	FPackageData* ShowInstigatorPackageData = nullptr;
	double LastPollAsyncTime;

	static IAssetRegistry* AssetRegistry;
};

/**
 * A debug-only scope class to confirm that each FPackageData removed from a container during a Pump function
 * is added to the container for its new state before leaving the Pump function.
 */
struct FPoppedPackageDataScope
{
	explicit FPoppedPackageDataScope(FPackageData& InPackageData);

#if COOK_CHECKSLOW_PACKAGEDATA
	~FPoppedPackageDataScope();

	FPackageData& PackageData;
#endif
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline EReachability FPackagePlatformData::GetReachability() const
{
	return (EReachability)Reachability;
}

inline bool FPackagePlatformData::IsReachable(EReachability InReachability) const
{
	return EnumHasAllFlags((EReachability)Reachability, InReachability);
}

inline void FPackagePlatformData::AddReachability(EReachability InReachability)
{
	Reachability |= (uint8)InReachability;
}

inline void FPackagePlatformData::ClearReachability(EReachability InReachability)
{
	Reachability &= ~(uint8)InReachability;
}

inline bool FPackagePlatformData::IsVisitedByCluster(EReachability InReachability) const
{
	return EnumHasAllFlags((EReachability)ReachabilityVisitedByCluster, InReachability);
}

inline void FPackagePlatformData::AddVisitedByCluster(EReachability InReachability)
{
	ReachabilityVisitedByCluster |= (uint8)InReachability;
}

inline void FPackagePlatformData::ClearVisitedByCluster(EReachability InReachability)
{
	ReachabilityVisitedByCluster &= ~((uint8)InReachability);
}

inline bool FPackagePlatformData::IsSaveTimedOut() const
{
	return bSaveTimedOut != 0;
}

inline void FPackagePlatformData::SetSaveTimedOut(bool bValue)
{
	bSaveTimedOut = (uint32)bValue;
}

inline bool FPackagePlatformData::IsCookable() const
{
	return bCookable != 0;
}

inline void FPackagePlatformData::SetCookable(bool bValue)
{
	bCookable = (uint32)bValue;
}

inline bool FPackagePlatformData::IsExplorable() const
{
	return bExplorable != 0;
}

inline void FPackagePlatformData::SetExplorable(bool bValue)
{
	bExplorable = (uint32)bValue;
}

inline bool FPackagePlatformData::IsExplorableOverride() const
{
	return bExplorableOverride != 0;
}

inline void FPackagePlatformData::SetExplorableOverride(bool bValue)
{
	bExplorableOverride = (uint32)bValue;
}

inline bool FPackagePlatformData::IsIncrementallyUnmodifiedSet() const
{
	return IncrementallyUnmodified != 0;
}

inline bool FPackagePlatformData::IsIncrementallyUnmodified() const
{
	return IncrementallyUnmodified == 2;
}

inline void FPackagePlatformData::SetIncrementallyUnmodified(bool bValue)
{
	IncrementallyUnmodified = bValue ? 2 : 1;
}

inline void FPackagePlatformData::ClearIncrementallyUnmodified()
{
	IncrementallyUnmodified = 0;
}

inline bool FPackagePlatformData::IsIncrementallySkipped() const
{
	return bIncrementallySkipped != 0;
}

inline void FPackagePlatformData::SetIncrementallySkipped(bool bValue)
{
	bIncrementallySkipped = (uint32)bValue;
}

inline EWhereCooked FPackagePlatformData::GetWhereCooked() const
{
	return static_cast<EWhereCooked>(WhereCooked);
}

inline void FPackagePlatformData::SetWhereCooked(EWhereCooked Value)
{
	WhereCooked = static_cast<uint32>(Value);
}

inline ECookResult FPackagePlatformData::GetCookResults() const
{
	return (ECookResult)CookResults;
}

inline bool FPackagePlatformData::IsCookAttempted() const
{
	return CookResults != (uint32)ECookResult::NotAttempted;
}

inline bool FPackagePlatformData::IsCookSucceeded() const
{
	return CookResults == (uint32)ECookResult::Succeeded;
}

inline void FPackagePlatformData::SetCookResults(ECookResult Value)
{
	// ECookResult::Invalid is only used in replication and is not allowed in FPackagePlatformData
	check(Value != ECookResult::Invalid);
	CookResults = (uint32)Value;
	if (Value == ECookResult::Succeeded || Value == ECookResult::Failed)
	{
		SetCommitted(true);
	}
}

inline bool FPackagePlatformData::IsCommitted() const
{
	return bCommitted != 0;
}

inline void FPackagePlatformData::SetCommitted(bool bValue)
{
	bCommitted = (uint32)bValue;
	SetReportedToDirector(false);
}

inline bool FPackagePlatformData::NeedsCooking(const ITargetPlatform* PlatformItBelongsTo) const
{
	return NeedsCommit(PlatformItBelongsTo, EReachability::Runtime);
}

inline bool FPackagePlatformData::NeedsCommit(const ITargetPlatform* PlatformItBelongsTo, ECookPhase CookPhase) const
{
	return NeedsCommit(PlatformItBelongsTo,
		CookPhase == ECookPhase::Cook ? EReachability::Runtime : EReachability::Build);
}

inline bool FPackagePlatformData::IsRegisteredForCachedObjectsInOuter() const
{
	return bRegisteredForCachedObjectsInOuter != 0;
}

inline void FPackagePlatformData::SetRegisteredForCachedObjectsInOuter(bool bValue)
{
	bRegisteredForCachedObjectsInOuter = bValue;
}

inline bool FPackagePlatformData::IsReportedToDirector()
{
	return bReportedToDirector != 0;
}

inline void FPackagePlatformData::SetReportedToDirector(bool bValue)
{
	bReportedToDirector = (uint32)bValue;
}

inline FPackageDatas& FPackageData::GetPackageDatas() const
{
	return PackageDatas;
}

inline const FName& FPackageData::GetPackageName() const
{
	return PackageName;
}

inline const FName& FPackageData::GetFileName() const
{
	return FileName;
}

inline void FPackageData::SetFileName(const FName& InFileName)
{
	FileName = InFileName;
}

inline uint32 FPackageData::GetLeafToRootRank() const
{
	return LeafToRootRank;
}

inline void FPackageData::SetLeafToRootRank(uint32 Value)
{
	LeafToRootRank = Value;
}

template <typename ArrayType>
inline void FPackageData::GetPlatformsNeedingCommit(ArrayType& OutPlatforms, ECookPhase CookPhase) const
{
	return GetPlatformsNeedingCommit(OutPlatforms,
		CookPhase == ECookPhase::Cook ? EReachability::Runtime : EReachability::Build);
}

inline int32 FPackageData::GetPlatformsNeedingCommitNum(ECookPhase CookPhase) const
{
	return GetPlatformsNeedingCommitNum(
		CookPhase == ECookPhase::Cook ? EReachability::Runtime : EReachability::Build);
}

template <typename ArrayType>
inline void FPackageData::GetPlatformsNeedingCommit(ArrayType& OutPlatforms, EReachability Reachability) const
{
	OutPlatforms.Reset(PlatformDatas.Num());
	for (const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Value.NeedsCommit(Pair.Key, Reachability))
		{
			OutPlatforms.Add(Pair.Key);
		}
	}
}

template <typename ArrayType>
inline void FPackageData::GetReachablePlatforms(EReachability InReachability, ArrayType& OutPlatforms) const
{
	OutPlatforms.Reset(PlatformDatas.Num());
	for (const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Value.IsReachable(InReachability))
		{
			OutPlatforms.Add(Pair.Key);
		}
	}
}

template <typename ArrayType>
void FPackageData::GetReachablePlatformsForInstigator(EReachability InReachability, UCookOnTheFlyServer& COTFS,
	FName InInstigator, ArrayType& Platforms)
{
	return GetReachablePlatformsForInstigator(InReachability, COTFS,
		COTFS.PackageDatas->TryAddPackageDataByPackageName(InInstigator), Platforms);
}

template <typename ArrayType>
void FPackageData::GetReachablePlatformsForInstigator(EReachability InReachability, UCookOnTheFlyServer& COTFS,
	UE::Cook::FPackageData* InInstigator, ArrayType& Platforms)
{
	if (InInstigator)
	{
		InInstigator->GetReachablePlatforms(InReachability, Platforms);
	}
	else
	{
		const TArray<const ITargetPlatform*>& SessionPlatforms = GetSessionPlatformsInternal(COTFS);
		Platforms.Reset(SessionPlatforms.Num() + 1);
		Platforms.Append(SessionPlatforms);
	}
}

inline EUrgency FPackageData::GetUrgency() const
{
	return static_cast<EUrgency>(Urgency);
}

inline void FPackageData::RaiseUrgency(EUrgency NewUrgency, ESendFlags SendFlags, bool bAllowUrgencyInIdle)
{
	if (NewUrgency > GetUrgency())
	{
		SetUrgency(NewUrgency, SendFlags, bAllowUrgencyInIdle);
	}
}

inline ESuppressCookReason FPackageData::GetSuppressCookReason() const
{
	return static_cast<ESuppressCookReason>(SuppressCookReason);
}

inline void FPackageData::SetSuppressCookReason(ESuppressCookReason Reason)
{
	SuppressCookReason = static_cast<uint32>(Reason);
}

inline bool FPackageData::GetIsCookLast() const
{
	return bIsCookLast != 0;
}

inline bool FPackageData::GetIsVisited() const
{
	return bIsVisited != 0;
}

inline void FPackageData::SetIsVisited(bool bValue)
{
	bIsVisited = static_cast<uint32>(bValue);
}

template <typename ArrayType>
inline void FPackageData::GetCachedObjectsInOuterPlatforms(ArrayType& OutPlatforms) const
{
	OutPlatforms.Reset(PlatformDatas.Num());
	for (const TPair<const ITargetPlatform*, FPackagePlatformData>& Pair : PlatformDatas)
	{
		if (Pair.Value.IsRegisteredForCachedObjectsInOuter())
		{
			OutPlatforms.Add(Pair.Key);
		}
	}
}

inline bool FPackageData::GetHasSaveCache() const
{
	return static_cast<bool>(bHasSaveCache);
}

inline void FPackageData::SetHasSaveCache(bool Value)
{
	bHasSaveCache = Value != 0;
}

inline ESaveSubState FPackageData::GetSaveSubState() const
{
	return static_cast<ESaveSubState>(SaveSubState);
}

inline bool FPackageData::HasPrepareSaveFailed() const
{
	return static_cast<bool>(bPrepareSaveFailed);
}

inline void FPackageData::SetHasPrepareSaveFailed(bool bValue)
{
	bPrepareSaveFailed = bValue != 0;
}

inline bool FPackageData::IsPrepareSaveRequiresGC() const
{
	return bPrepareSaveRequiresGC;
}

inline void FPackageData::SetIsPrepareSaveRequiresGC(bool bValue)
{
	bPrepareSaveRequiresGC = bValue != 0;
}

inline ECookResult FPackageData::GetMonitorCookResult() const
{
	return (ECookResult)MonitorCookResult;
}

inline void FPackageData::SetMonitorCookResult(ECookResult Value)
{
	MonitorCookResult = (uint8)Value;
}

inline bool FPackageData::IsGenerated() const
{
	return static_cast<bool>(bGenerated);
}

inline FName FPackageData::GetParentGenerator() const
{
	return ParentGenerator;
}

inline ICookPackageSplitter::EGeneratedRequiresGenerator FPackageData::DoesGeneratedRequireGenerator() const
{
	return static_cast<ICookPackageSplitter::EGeneratedRequiresGenerator>(DoesGeneratedRequireGeneratorValue);
}

inline void FPackageData::SetDoesGeneratedRequireGenerator(ICookPackageSplitter::EGeneratedRequiresGenerator Value)
{
	DoesGeneratedRequireGeneratorValue = static_cast<uint32>(Value);
}

inline const FInstigator& FPackageData::GetInstigator(EReachability InReachability) const
{
	if (InReachability == EReachability::None)
	{
		checkf(false, TEXT("Invalid argument EReachability::None."));
		return Instigator;
	}
	if (EnumHasAnyFlags(InReachability, EReachability::Runtime)
		&& Instigator.Category != EInstigator::NotYetRequested)
	{
		return Instigator;
	}
	if (EnumHasAnyFlags(InReachability, EReachability::Build)
		&& BuildInstigator.Category != EInstigator::NotYetRequested)
	{
		return BuildInstigator;
	}

	// return a const reference to the (empty) Instigator for the most important set bit.
	if (EnumHasAnyFlags(InReachability, EReachability::Runtime))
	{
		return Instigator;
	}
	return BuildInstigator;
}

inline bool FPackageData::HasInstigator(EReachability InReachability) const
{
	if (InReachability == EReachability::None)
	{
		checkf(false, TEXT("Invalid argument EReachability::None."));
		return false;
	}
	if (EnumHasAnyFlags(InReachability, EReachability::Runtime)
		&& Instigator.Category != EInstigator::NotYetRequested)
	{
		return true;
	}
	if (EnumHasAnyFlags(InReachability, EReachability::Build)
		&& BuildInstigator.Category != EInstigator::NotYetRequested)
	{
		return true;
	}
	return false;
}

inline bool FPackageData::IsKeepReferencedDuringGC() const
{
	return static_cast<bool>(bKeepReferencedDuringGC);
}

inline void FPackageData::SetKeepReferencedDuringGC(bool Value)
{
	bKeepReferencedDuringGC = Value != 0;
}

inline bool FPackageData::GetWasCookedThisSession() const
{
	return static_cast<bool>(bWasCookedThisSession);
}

inline bool FPackageData::HasReplayedLogMessages() const
{
	return static_cast<bool>(bHasReplayedLogMessages);
}

inline void FPackageData::SetHasReplayedLogMessages(bool Value)
{
	bHasReplayedLogMessages = Value != 0;
}

inline bool FPackageData::HasBeenLoadedForCooking() const
{
	return static_cast<bool>(bHasBeenLoadedForCooking);
}

inline void FPackageData::SetHasBeenLoadedForCooking(bool Value)
{
	bHasBeenLoadedForCooking = Value != 0;
}

inline FWorkerId FPackageData::GetWorkerAssignment() const
{
	return WorkerAssignment;
}

inline FWorkerId FPackageData::GetWorkerAssignmentConstraint() const
{
	return WorkerAssignmentConstraint;
}

typedef void (FPackageData::* FEdgeFunction)();
inline void FPackageData::UpdateDownEdge(bool bOld, bool bNew, const FEdgeFunction& EdgeFunction)
{
	if ((bOld != bNew) & bOld)
	{
		(this->*EdgeFunction)();
	}
}

inline void FPackageData::UpdateUpEdge(bool bOld, bool bNew, const FEdgeFunction& EdgeFunction)
{
	if ((bOld != bNew) & bNew)
	{
		(this->*EdgeFunction)();
	}
}

inline TPackageDataMap<ESuppressCookReason>& FRequestQueue::GetRestartedRequests()
{
	return RestartedRequests;
}

inline TRingBuffer<FDiscoveryQueueElement>& FRequestQueue::GetDiscoveryQueue()
{
	return DiscoveryQueue;
}

inline TRingBuffer<FPackageData*>& FRequestQueue::GetBuildDependencyDiscoveryQueue()
{
	return BuildDependencyDiscoveryQueue;
}

inline TRingBuffer<TUniquePtr<FRequestCluster>>& FRequestQueue::GetRequestClusters()
{
	return RequestClusters;
}

inline FPackageDataSortedSet& FRequestQueue::GetReadyRequestsUrgent()
{
	return UrgentRequests;
}

inline FPackageDataSortedSet& FRequestQueue::GetReadyRequestsNormal()
{
	return NormalRequests;
}

inline FPackageDataMonitor& FPackageDatas::GetMonitor()
{
	return Monitor;
}

inline UCookOnTheFlyServer& FPackageDatas::GetCookOnTheFlyServer()
{
	return CookOnTheFlyServer;
}

inline uint32 FPackageDatas::GetNextLeafToRootRank()
{
	return NextLeafToRootRank++;
}

inline void FPackageDatas::ResetLeafToRootRank()
{
	NextLeafToRootRank = 0;
}

inline FRequestQueue& FPackageDatas::GetRequestQueue()
{
	return RequestQueue;
}

inline TSet<FPackageData*>& FPackageDatas::GetAssignedToWorkerSet()
{
	return AssignedToWorkerSet;
}

inline FLoadQueue& FPackageDatas::GetLoadQueue()
{
	return LoadQueue;
}

inline TSet<FPackageData*>& FPackageDatas::GetSaveStalledSet()
{
	return SaveStalledSet;
}

inline FPackageDataQueue& FPackageDatas::GetSaveQueue()
{
	return SaveQueue;
}

template<typename CallbackType>
inline void FPackageDatas::UpdateThreadsafePackageData(FName PackageName, CallbackType&& Callback)
{
	FWriteScopeLock ExistenceWriteLock(ExistenceLock);
	FThreadsafePackageData& Value = ThreadsafePackageDatas.FindOrAdd(PackageName);
	bool bNew = false;
	if (!Value.bInitialized)
	{
		Value.bInitialized = true;
		bNew = true;
	}
	Callback(Value, bNew);
}

inline TOptional<FThreadsafePackageData> FPackageDatas::FindThreadsafePackageData(FName PackageName)
{
	FReadScopeLock ExistenceReadLock(ExistenceLock);
	FThreadsafePackageData* Value = ThreadsafePackageDatas.Find(PackageName);
	return Value ? TOptional<FThreadsafePackageData>(*Value) : TOptional<FThreadsafePackageData>();
}

template <typename FunctionType>
inline void FPackageDatas::ForEachPendingCookedPlatformData(const FunctionType& Function)
{
	for (FPendingCookedPlatformDataContainer& Container : PendingCookedPlatformDataLists)
	{
		for (FPendingCookedPlatformData& Data : Container)
		{
			Function(Data);
		}
	}
}

inline int32 FPackageDatas::GetPendingCookedPlatformDataNum() const
{
	return PendingCookedPlatformDataNum;
}

template <typename CallbackType>
inline void FPackageDatas::LockAndEnumeratePackageDatas(CallbackType&& Callback)
{
	FReadScopeLock ExistenceReadLock(ExistenceLock);
	EnumeratePackageDatasWithinLock(Forward<CallbackType>(Callback));
}

inline FMapOfCachedCookedPlatformDataState& FPackageDatas::GetCachedCookedPlatformDataObjects()
{
	return CachedCookedPlatformDataObjects;
}

template <typename CallbackType>
inline void FPackageDatas::EnumeratePackageDatasWithinLock(CallbackType&& Callback)
{
	Allocator.EnumerateAllocations(Forward<CallbackType>(Callback));
}

} // namespace UE::Cook