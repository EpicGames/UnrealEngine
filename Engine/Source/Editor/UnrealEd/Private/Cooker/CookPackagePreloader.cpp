// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookPackagePreloader.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreGlobals.h"
#include "EditorDomain/EditorDomain.h"
#include "HAL/PlatformMath.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackagePath.h"
#include "Misc/PreloadableFile.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

namespace UE::Cook
{

bool FPackagePreloader::bConfigInitialized = false;
bool FPackagePreloader::bAllowPreloadImports = true;

void FPackagePreloader::InitializeConfig()
{
	bConfigInitialized = true;

	bAllowPreloadImports = true;
	FParse::Bool(FCommandLine::Get(), TEXT("-CookPreloadImports="), bAllowPreloadImports);
}

FPackagePreloader::~FPackagePreloader()
{
	// To avoid incorrect behavior, we have to set ESendFlags::QueueNone, otherwise a TRefCountPtr will be created
	// and invalidly call ~FPackagePreloader when the TRefCountPtr goes out of scope.
	SendToState(EPreloaderState::Inactive, ESendFlags::QueueNone);
	PackageData.OnPackagePreloaderDestroyed(*this);
}

void FPackagePreloader::Shutdown()
{
	TRefCountPtr<FPackagePreloader> LocalRef(this);
	SendToState(EPreloaderState::Inactive, ESendFlags::QueueAddAndRemove);
}

template <typename ShouldKeepFunc, typename ReportAndIsContinueFunc>
void FPackagePreloader::TraverseImportGraph(ShouldKeepFunc&& ShouldKeep, ReportAndIsContinueFunc&& ReportAndIsContinue,
	bool bAllowGather)
{
	// VisitState should only ever be changed from Unvisited during the execution below, and we change it back
	// before exiting.
	check(VisitState == EGraphVisitState::Unvisited);

	struct FStackData
	{
		FPackagePreloader* Preloader = nullptr;
		int32 NextImport = 0;

		FStackData(FPackagePreloader* InPreloader)
			: Preloader(InPreloader)
		{
		}
	};
	TArray<FStackData, TInlineAllocator<16>> Stack;
	TArray<FPackagePreloader*, TInlineAllocator<128>> VisitedList;

	// Depth first search over the import graph
	Stack.Emplace(this);
	while (!Stack.IsEmpty())
	{
		FStackData& Top = Stack.Last();

		// When the stackdata's NextImport is 0, that means we just pushed it onto the stack,
		// and we need to execute the initial setup
		if (Top.NextImport == 0)
		{
			if (Top.Preloader->VisitState != EGraphVisitState::Unvisited)
			{
				// Inprogress or already visited; ignore this link
				Stack.Pop(EAllowShrinking::No);
				continue;
			}

			if (!ShouldKeep(*Top.Preloader))
			{
				// Caller does not want us to report or explore this one
				Top.Preloader->VisitState = EGraphVisitState::Visited;
				VisitedList.Add(Top.Preloader);
				Stack.Pop(EAllowShrinking::No);
				continue;
			}

			if (!ReportAndIsContinue(*Top.Preloader))
			{
				// Caller requested we stop searching; break out of the graph search
				break;
			}

			// Gather the imports from the AssetRegistry if not already gathered, keeping only the unloaded ones.
			if (bAllowGather)
			{
				Top.Preloader->GatherUnloadedImports();
			}

			// Mark that we are on the stack and are traversing imports
			Top.Preloader->VisitState = EGraphVisitState::InProgress;
			VisitedList.Add(Top.Preloader);

			// Fallthrough to after the if/else and examine the first import. 
		}

		// Examine the next import, if we have not yet reached the end
		if (Top.NextImport < Top.Preloader->UnloadedImports.Num())
		{
			FPackagePreloader* Import = Top.Preloader->UnloadedImports[Top.NextImport++];
			Stack.Emplace(Import);
			// Top is now possibly a dangling pointer if the stack reallocated; do not access it
			continue;
		}

		// Finish the visit of Top
		Top.Preloader->VisitState = EGraphVisitState::Visited;
		Stack.Pop(EAllowShrinking::No);
	}

	// Clear all the VisitState variables we modified
	for (FPackagePreloader* PackagePreloader : VisitedList)
	{
		PackagePreloader->VisitState = EGraphVisitState::Unvisited;
	}
}

void FPackagePreloader::GetNeedsLoadPreloadersInImportTree(TArray<TRefCountPtr<FPackagePreloader>>& OutPreloaders)
{
	TraverseImportGraph(
		[](FPackagePreloader& Preloader) -> bool // ShouldKeep
		{
			// Traverse every import; TraverseImportGraph already filters the imports by NeedsLoad 
			return true;
		},
		[&OutPreloaders](FPackagePreloader& Preloader) -> bool // ReportAndIsContinue
		{
			OutPreloaders.Add(&Preloader);
			return true; // Continue iterating through the entire unloaded import graph
		},
		true /* bAllowGather */
	);
}

void FPackagePreloader::GatherUnloadedImports()
{
	if (IsImportsGathered())
	{
		return;
	}
	SetIsImportsGathered(true);

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	TArray<FName> ImportNames;
	FPackageDatas& PackageDatas = PackageData.GetPackageDatas();

	AssetRegistry.GetDependencies(PackageData.GetPackageName(), ImportNames,
		UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
	UnloadedImports.Reset(ImportNames.Num());
	for (FName ImportName : ImportNames)
	{
		FPackageData* ImportData = PackageDatas.TryAddPackageDataByPackageName(ImportName);
		if (!ImportData)
		{
			continue;
		}

		TRefCountPtr<FPackagePreloader> ImportPreloader = ImportData->GetPackagePreloader();
		if (!ImportPreloader)
		{
			// Optimization: If the ImportPackage does not have a Packagedata, check whether it is already loaded
			// before paying the expense of creating a PackagePreloader which we would then immediately delete.
			if (FPackagePreloader::IsPackageLoaded(*ImportData))
			{
				continue;
			}
			ImportPreloader = ImportData->CreatePackagePreloader();
		}
		else
		{
			if (!ImportPreloader->NeedsLoad())
			{
				continue;
			}
		}
		UnloadedImports.Add(MoveTemp(ImportPreloader));
	}
}

void FPackagePreloader::EmptyImports()
{
	UnloadedImports.Empty();
	SetIsImportsGathered(false);
}

bool FPackagePreloader::TryPreload(FString* OutFailReason)
{
	bool bTreatPackageAsLoaded = WasLoadAttempted() || IsPackageLoaded();
	if (bTreatPackageAsLoaded)
	{
		if (AsyncRequest || IsPreloaded())
		{
			if (AsyncRequest && !AsyncRequest->IsCompleted())
			{
				// In case of async loading, the object can be found while still being asynchronously serialized, we need
				// to wait until the callback is called and the async request is completely done.
				if (OutFailReason)
				{
					*OutFailReason = TEXT("Package is loaded but AsyncRequest is not completed");
				}
				return false;
			}

			// If the package has already loaded, then we no longer need the preloaded data
			ClearPreload();
		}
		SetIsPreloadAttempted(true);
		return true;
	}
	else
	{
		if (IsPreloadAttempted())
		{
			return true;
		}
		else if (AsyncRequest && AsyncRequest->IsFailed())
		{
			SetIsPreloadAttempted(true);
			return true;
		}
	}

	if (PackageData.IsGenerated())
	{
		// Deferred populate generated packages are loaded from their generator, not from disk
		ClearPreload();
		SetIsPreloadAttempted(true);
		return true;
	}
	if (IsAsyncLoadingMultithreaded())
	{
		bool bResetOldAsyncRequest = false;
		if (AsyncRequest && AsyncRequest->IsCompleted() && !IsPackageLoaded())
		{
			// The AsyncRequest previously finished, but we garbage collected and removed the package from memory
			// before we had a chance to test IsPackageLoaded. Restart the AsyncRequest.
			AsyncRequest.Reset();
			// We will bump it back up down below.
			PackageData.GetPackageDatas().GetMonitor().OnPreloadAllocatedChanged(PackageData, false);
			bResetOldAsyncRequest = true;
			if (OutFailReason)
			{
				*OutFailReason = TEXT("AsyncRequest was completed but package is not loaded");
			}
		}

		if (!AsyncRequest.IsValid())
		{
			// Declare the package as being referenced by the ScriptPackageNameEngine; we look for this in
			// ProcessUnsolicitedPackages so we can avoid adding an error for the package being loaded without a
			// referencer.
			static FName ScriptPackageNameEngine(TEXT("/Script/Engine"));
			UE_TRACK_REFERENCING_PACKAGE_SCOPED(ScriptPackageNameEngine, PackageAccessTrackingOps::NAME_Load);

			PackageData.GetPackageDatas().GetMonitor().OnPreloadAllocatedChanged(PackageData, true);
			AsyncRequest = MakeShared<FAsyncRequest>();
			int32 RequestID = LoadPackageAsync(
				PackageData.GetFileName().ToString(),
				FLoadPackageAsyncDelegate::CreateLambda(
					[AsyncRequest = AsyncRequest, PackageName = this->PackageData.GetPackageName()]
					(const FName&, UPackage* LoadedPackage, EAsyncLoadingResult::Type)
					{
						if (!LoadedPackage)
						{
							// PackageLoad failed, mark the preload as complete; UCookOnTheFlyServer::LoadPackageInQueue will
							// call LoadPackage and handle the error.
							AsyncRequest->SetFailed();
						}
						else if (LoadedPackage->GetFName() != PackageName)
						{
							// The PackageName is not the name that we loaded. This can happen due to CoreRedirects.
							// Do not attempt to handle this here in the preloader; just mark the preload complete.
							// UCookOnTheFlyServer::LoadPackageInQueue will call LoadPackage and will detect the renamed
							// package there and will handle the update of the cooker.
							AsyncRequest->SetFailed();
						}
						else if (!FPackagePreloader::IsPackageLoaded(PackageName))
						{
							UE_LOGF(LogCook, Error,
								"Package %ls finished loading asynchronously, but IsPackageLoaded returned false. We will try to recover, but this error is unexpected.",
								*PackageName.ToString());
							AsyncRequest->SetFailed();
						}
						else
						{
							AsyncRequest->SetSucceeded();
						}
					}
				),
				32 /* Use arbitrary higher priority for preload as we're going to need them very soon */
			);
			AsyncRequest->SetRequestID(RequestID);

			if (OutFailReason && !bResetOldAsyncRequest)
			{
				*OutFailReason = TEXT("AsyncRequest was not yet created");
			}
		}
		else
		{
			if (OutFailReason)
			{
				*OutFailReason = TEXT("AsyncRequest is not yet complete");
			}
		}

		// always return false so we continue to check the status of the load until IsPackageLoaded().
		return false;
	}
	if (!PreloadableFile.Get())
	{
		if (FEditorDomain* EditorDomain(FEditorDomain::Get());
			EditorDomain && EditorDomain->IsReadingPackages())
		{
			EditorDomain->PrecachePackageDigest(PackageData.GetPackageName());
		}
		TStringBuilder<NAME_SIZE> FileNameString;
		PackageData.GetFileName().ToString(FileNameString);
		PreloadableFile.Set(MakeShared<FPreloadableArchive>(FileNameString.ToString()), PackageData);
		PreloadableFile.Get()->InitializeAsync([this]()
			{
				TStringBuilder<NAME_SIZE> FileNameString;
				// Note this async callback has an read of PackageData->GetFilename and a write of this->PreloadableFileOpenResult
				// outside of a critical section. This read and write is allowed because GetFilename does
				// not change until the PackageData is destructed, and the destructor does not run and other threads do not read
				// or write PreloadableFileOpenResult until after PreloadableFile.Get() has finished initialization
				// and this callback is therefore complete.
				// The code that accomplishes that waiting is in TryPreload (via IsInitialized) and ClearPreload
				// (via ReleaseCache)
				PackageData.GetFileName().ToString(FileNameString);
				FPackagePath PackagePath = FPackagePath::FromLocalPath(FileNameString);
				FOpenPackageResult Result = IPackageResourceManager::Get().OpenReadPackage(PackagePath);
				if (Result.Archive)
				{
					this->PreloadableFileOpenResult.CopyMetaData(Result);
				}
				return Result.Archive.Release();
			},
			FPreloadableFile::Flags::PreloadHandle | FPreloadableFile::Flags::Prime);
	}
	const TSharedPtr<FPreloadableArchive>& FilePtr = PreloadableFile.Get();
	if (!FilePtr->IsInitialized())
	{
		if (PackageData.GetUrgency() == EUrgency::Blocking)
		{
			// For blocking requests, wait on them to finish preloading rather than letting them run asynchronously
			// and coming back to them later
			FilePtr->WaitForInitialization();
			check(FilePtr->IsInitialized());
		}
		else
		{
			if (OutFailReason)
			{
				*OutFailReason = TEXT("Waiting on PreloadableFile async load");
			}

			return false;
		}
	}
	if (FilePtr->TotalSize() < 0)
	{
		UE_LOGF(LogCook, Warning, "Failed to find file when preloading %ls.",
			*PackageData.GetFileName().ToString());
		SetIsPreloadAttempted(true);
		PreloadableFile.Reset(PackageData);
		PreloadableFileOpenResult = FOpenPackageResult();
		return true;
	}

	TStringBuilder<NAME_SIZE> FileNameString;
	PackageData.GetFileName().ToString(FileNameString);
	if (!IPackageResourceManager::TryRegisterPreloadableArchive(FPackagePath::FromLocalPath(FileNameString),
		FilePtr, PreloadableFileOpenResult))
	{
		UE_LOGF(LogCook, Warning, "Failed to register %ls for preload.",
			*PackageData.GetFileName().ToString());
		SetIsPreloadAttempted(true);
		PreloadableFile.Reset(PackageData);
		PreloadableFileOpenResult = FOpenPackageResult();
		return true;
	}

	SetIsPreloaded(true);
	SetIsPreloadAttempted(true);
	return true;
}

void FPackagePreloader::ClearPreload()
{
	if (AsyncRequest)
	{
		if (!AsyncRequest->IsCompleted())
		{
			FlushAsyncLoading(AsyncRequest->GetRequestID());
			check(AsyncRequest->IsCompleted());
		}
		PackageData.GetPackageDatas().GetMonitor().OnPreloadAllocatedChanged(PackageData, false);
		AsyncRequest.Reset();
	}

	const TSharedPtr<FPreloadableArchive>& FilePtr = PreloadableFile.Get();
	if (IsPreloaded())
	{
		check(FilePtr);
		TStringBuilder<NAME_SIZE> FileNameString;
		PackageData.GetFileName().ToString(FileNameString);
		if (IPackageResourceManager::UnRegisterPreloadableArchive(FPackagePath::FromLocalPath(FileNameString)))
		{
			UE_LOGF(LogCook, Display,
				"PreloadableFile was created for %ls but never used. This is wasteful and bad for cook performance.",
				*PackageData.GetPackageName().ToString());
		}
		FilePtr->ReleaseCache(); // ReleaseCache to conserve memory if the Linker still has a pointer to it
	}
	else
	{
		check(!FilePtr || !FilePtr->IsCacheAllocated());
	}

	PreloadableFile.Reset(PackageData);
	PreloadableFileOpenResult = FOpenPackageResult();
	SetIsPreloaded(false);
	SetIsPreloadAttempted(false);
}

void FPackagePreloader::PostGarbageCollect()
{
	// Reevaluate imports.
	SetIsImportsGathered(false);

	// Reevaluate variables that depend on whether our package is loaded.
	SetLoadAttempted(false);

	if (!AsyncRequest && !PreloadableFile.Get())
	{
		// If we have no preload data, we might have marked that we are done preloading becuase the package already
		// exists. Call ClearPreload so we reevaluate whether the package exists
		ClearPreload();
	}
	else
	{
		if (AsyncRequest)
		{
			// The AsyncRequest should have been flushed (and then either kept in memory or garbage collected),
			// so clear the preload data
			ClearPreload();
		}

		if (PreloadableFile.Get() && IsPreloaded() && !PreloadableFile.Get()->HasValidData())
		{
			// If we finished preloading the file, then we registered it, and it might have been consumed by the loaded
			// package, but then the loaded package GC'd. In that case we need to clear the PreloadableFile data so we
			// can restart it when necessary the next load of the package. And if the package already exists in memory then
			// we don't need the preloaded data, so it's okay to free it.
			// If we didn't register it, or we registered it but it has not yet been consumed, then we don't need to free it.
			ClearPreload();
		}
	}

	// If state is past ActivePreload, move back to ActivePreload to reevaluate whether we're ready.
	if (State > EPreloaderState::ActivePreload)
	{
		SendToState(EPreloaderState::ActivePreload, ESendFlags::QueueAddAndRemove);
	}
}

void FPackagePreloader::OnPackageLeaveLoadState()
{
	// Caller guarantees that a ref count is held during this function, so no need for a local refcount.

	if (HasInitializedRequestedLoads())
	{
		SetHasInitializedRequestedLoads(false);

		// There should be at least one CountFromRequestedLoads due to a request from *this.
		check(CountFromRequestedLoads > 0);
		// Don't allow the triggering of a state transition on this during the for loop.
		IncrementCountFromRequestedLoads();

		for (TRefCountPtr<FPackagePreloader>& Other : RequestedLoads)
		{
			Other->DecrementCountFromRequestedLoads();
		}
		RequestedLoads.Empty();

		DecrementCountFromRequestedLoads(); // This decrement might kick *this back to Inactive.
	}
}

bool FPackagePreloader::IsPackageLoaded() const
{
	return IsPackageLoaded(PackageData.GetPackageName());
}

bool FPackagePreloader::IsPackageLoaded(const FPackageData& InPackageData)
{
	return IsPackageLoaded(InPackageData.GetPackageName());
}

bool FPackagePreloader::IsPackageLoaded(FName InPackageName)
{
	return FindObjectFast<UPackage>(nullptr, InPackageName) != nullptr;
}

void FPackagePreloader::IncrementCountFromRequestedLoads()
{
	++CountFromRequestedLoads;
}

void FPackagePreloader::DecrementCountFromRequestedLoads()
{
	check(CountFromRequestedLoads > 0); // Assert we do not have an unbalanced decrement.
	--CountFromRequestedLoads;

	if (CountFromRequestedLoads == 0 && GetState() != EPreloaderState::Inactive)
	{
		SendToState(EPreloaderState::Inactive, ESendFlags::QueueAddAndRemove);
	}
}

void FPackagePreloader::SetRequestedLoads(TArray<TRefCountPtr<FPackagePreloader>>&& InRequestedLoads,
	bool bMakeActive)
{
	FPackageDatas& PackageDatas = PackageData.GetPackageDatas();
	check(RequestedLoads.IsEmpty()); // This function is only for setting from empty.

	// Our contract specifies that InRequestedLoads is in RootToLeaf order, so traverse it
	// backwards to set the LeafToRoot rank
	for (TRefCountPtr<FPackagePreloader>& NeedsLoadPreloader : ReverseIterate(InRequestedLoads))
	{
		FPackageData& NeedsLoadData(NeedsLoadPreloader->GetPackageData());
		if (NeedsLoadData.GetLeafToRootRank() == MAX_uint32)
		{
			NeedsLoadData.SetLeafToRootRank(PackageDatas.GetNextLeafToRootRank());
		}

		if (NeedsLoadPreloader->GetState() == EPreloaderState::Inactive && bMakeActive)
		{
			NeedsLoadPreloader->SendToState(EPreloaderState::PendingKick, ESendFlags::QueueAddAndRemove);
		}

		NeedsLoadPreloader->IncrementCountFromRequestedLoads();
	}
	RequestedLoads = MoveTemp(InRequestedLoads);
}

void FPackagePreloader::OnExitActive()
{
	ClearPreload();
	EmptyImports();
}

bool FPackagePreloader::NeedsLoad()
{
	if (WasLoadAttempted())
	{
		return false;
	}
	if (!IsPackageLoaded(PackageData))
	{
		return true;
	}
	// We might not be done preloading even if the package exists. Calling TryPreload(false) will ClearPreload
	// and return true unless we're still waiting on asynchronous post-loads to complete.
	return !TryPreload();
}

bool FPackagePreloader::IsHigherPriorityThan(const FPackagePreloader& Other) const
{
	if (PackageData.GetUrgency() != Other.PackageData.GetUrgency())
	{
		return PackageData.GetUrgency() > Other.PackageData.GetUrgency();
	}

	// Leaves are higher priority because we want them to be already preloaded (or even
	// better, completely loaded) when we load the package that imports them (and is therefore
	// more rootwards).
	return PackageData.GetLeafToRootRank() < Other.PackageData.GetLeafToRootRank();
}

void FPackagePreloader::SendToState(EPreloaderState NewState, ESendFlags SendFlags)
{
	TRefCountPtr<FPackagePreloader> KeepRemovalResident;
	FLoadQueue& LoadQueue = PackageData.GetPackageDatas().GetLoadQueue();
	if (EnumHasAnyFlags(SendFlags, ESendFlags::QueueRemove))
	{
		KeepRemovalResident = this;
		switch (State)
		{
		case EPreloaderState::Inactive:
			break;
		case EPreloaderState::PendingKick:
			LoadQueue.PendingKicks.Remove(this);
			break;
		case EPreloaderState::ActivePreload:
			LoadQueue.ActivePreloads.Remove(this);
			break;
		case EPreloaderState::ReadyForLoad:
			LoadQueue.ReadyForLoads.Remove(this);
			break;
		default:
			checkNoEntry();
			break;
		}
	}
	bool bActive = State != EPreloaderState::Inactive;
	bool bNewActive = NewState != EPreloaderState::Inactive;
	if (bActive != bNewActive)
	{
		if (!bNewActive)
		{
			OnExitActive();
		}
	}

	State = NewState;

	if (EnumHasAnyFlags(SendFlags, ESendFlags::QueueAdd))
	{
		switch (State)
		{
		case EPreloaderState::Inactive:
			break;
		case EPreloaderState::PendingKick:
			LoadQueue.PendingKicks.Add(this);
			break;
		case EPreloaderState::ActivePreload:
			LoadQueue.ActivePreloads.Add(this);
			break;
		case EPreloaderState::ReadyForLoad:
			LoadQueue.ReadyForLoads.Add(this);
			break;
		default:
			checkNoEntry();
			break;
		}
	}
}

bool FPackagePreloader::PumpLoadsTryStartInboxPackage(UCookOnTheFlyServer& COTFS)
{
	using namespace UE::Cook;

	TRingBuffer<FPackageData*>& Inbox = COTFS.PackageDatas->GetLoadQueue().Inbox;
	if (Inbox.IsEmpty())
	{
		return false;
	}

	FPackageData* PoppedPackageData = Inbox.PopFrontValue();
	check(PoppedPackageData->GetState() == EPackageState::Load);
	TRefCountPtr<FPackagePreloader> Preloader = PoppedPackageData->GetPackagePreloader();
	check(Preloader);
	Preloader->SetIsInInbox(false);

	// A required invariant for any preloader moved into an active state is that it has a count from the packages in
	// load state that are requesting it. Assert that we satisfy that invariant for *this during this function.
	ON_SCOPE_EXIT
	{
		if (Preloader->GetState() != EPreloaderState::Inactive)
		{
			check(Preloader->GetCountFromRequestedLoads() > 0);
		}
	};

	if (COTFS.TryCreateRequestCluster(*PoppedPackageData))
	{
		return true;
	}

	// If the package is already ready for loading, or we otherwise want to skip preloading for it,
	// skip the preload containers and put in the ReadyLoads container
	if (!COTFS.bPreloadingEnabled || Preloader->IsPackageLoaded()
		|| PoppedPackageData->GetUrgency() == EUrgency::Blocking)
	{
		if (Preloader->GetState() != EPreloaderState::ReadyForLoad)
		{
			if (!Preloader->HasInitializedRequestedLoads())
			{
				Preloader->SetHasInitializedRequestedLoads(true);
				Preloader->SetRequestedLoads(TArray<TRefCountPtr<FPackagePreloader>>({ Preloader }),
					false /* bMakeActive */);
			}
			Preloader->SendToState(EPreloaderState::ReadyForLoad, ESendFlags::QueueAddAndRemove);
		}
		return true;
	}

	if (!Preloader->HasInitializedRequestedLoads())
	{
		Preloader->SetHasInitializedRequestedLoads(true);

		TArray<TRefCountPtr<FPackagePreloader>> NeedsLoadPreloaders;
		if (bAllowPreloadImports)
		{
			Preloader->GetNeedsLoadPreloadersInImportTree(NeedsLoadPreloaders);
		}
		else
		{
			NeedsLoadPreloaders.Add(Preloader);
		}
		check(!NeedsLoadPreloaders.IsEmpty() && NeedsLoadPreloaders[0].GetReference() == Preloader);
		Preloader->SetRequestedLoads(MoveTemp(NeedsLoadPreloaders));
		// Should have been set to active by SetRequestedLoads
		check(Preloader->GetState() != EPreloaderState::Inactive);
	}
	else if (Preloader->GetState() == EPreloaderState::Inactive)
	{
		// Edgecase: we've already initialized loads, but the preloader is inactive and not loaded somehow. Put it
		// directly into ReadyForLoad since its not clear that it needs preloading.
		if (!Preloader->HasInitializedRequestedLoads())
		{
			Preloader->SetHasInitializedRequestedLoads(true);
			Preloader->SetRequestedLoads(TArray<TRefCountPtr<FPackagePreloader>>({ Preloader }),
				false /* bMakeActive */);
		}
		Preloader->SendToState(EPreloaderState::ReadyForLoad, ESendFlags::QueueAddAndRemove);
	}

	return true;
}

bool FPackagePreloader::PumpLoadsTryKickPreload(UCookOnTheFlyServer& COTFS)
{
	using namespace UE::Cook;

	FPackagePreloaderPriorityQueue& PendingKicks = COTFS.PackageDatas->GetLoadQueue().PendingKicks;
	if (PendingKicks.IsEmpty())
	{
		return false;
	}
	FPackageDataMonitor& Monitor = COTFS.PackageDatas->GetMonitor();
	if (Monitor.GetNumPreloadAllocated() >= static_cast<int32>(COTFS.MaxPreloadAllocated))
	{
		return false;
	}

	TRefCountPtr<FPackagePreloader> Preloader = PendingKicks.PopFront();
	Preloader->TryPreload();
	Preloader->SendToState(EPreloaderState::ActivePreload, ESendFlags::QueueAdd);

	return true;
}

bool FPackagePreloader::PumpLoadsIsReadyToLeavePreload(TRefCountPtr<FPackagePreloader>* OutBlocker,
	FString* OutBlockerReason)
{
	// Once we are added to ActivePreloads, we stop caring about the preload status of all of our imports
	// that are not actively preloading.
	// The imports that we depend on should have been added to ActivePreloads before us, and we either check
	// them because they are still active, or we don't need to check them because they already finished preloading
	// and moved past the ActivePreloads state.
	// In the case of a cycle, or if one of our imports was demoted out of ActivePreloads somehow, we
	// need to proceed despite the package possibly not being preloaded, so we don't get stuck in a cycle that
	// we can't preload all elements of.

	TRefCountPtr<FPackagePreloader> Blocker;
	TraverseImportGraph(
		[](FPackagePreloader& Preloader) -> bool // ShouldKeep
		{
			// Only look at the imports that are in ActivePreload, per the comment above. Most notably, this includes
			// ourself, at the root of the import tree.
			return Preloader.GetState() == EPreloaderState::ActivePreload;
		},
		[&Blocker, OutBlockerReason](FPackagePreloader& Preloader) -> bool // ReportAndIsContinue
		{
			if (!Preloader.TryPreload(OutBlockerReason))
			{
				Blocker = &Preloader;
				return false; // Once we find we will return false, stop searching.
			}
			return true; // Keep searching.
		},
		false /* bAllowGather */
	);
	bool bBlocked = Blocker.IsValid();
	if (OutBlocker)
	{
		*OutBlocker = MoveTemp(Blocker);
	}
	return !bBlocked;
}

void FPackagePreloader::PumpLoadsMarkLoadAttemptComplete()
{
	SetLoadAttempted(true);
	// Caller is responsible for having removed *this from the ReadyForLoad container.
	SendToState(EPreloaderState::Inactive, ESendFlags::QueueNone);
}

void FPackagePreloader::FTrackedPreloadableFilePtr::Set(TSharedPtr<FPreloadableArchive>&& InPtr, FPackageData& Owner)
{
	Reset(Owner);
	if (InPtr)
	{
		Ptr = MoveTemp(InPtr);
		Owner.GetPackageDatas().GetMonitor().OnPreloadAllocatedChanged(Owner, true);
	}
}

void FPackagePreloader::FTrackedPreloadableFilePtr::Reset(FPackageData& Owner)
{
	if (Ptr)
	{
		Owner.GetPackageDatas().GetMonitor().OnPreloadAllocatedChanged(Owner, false);
		Ptr.Reset();
	}
}

bool FPackagePreloaderPriorityWrapper::operator<(const FPackagePreloaderPriorityWrapper& Other) const
{
	check(Payload.IsValid() && Other.Payload.IsValid());
	// Higher Priority -> comes earlier in the queue -> has lower index -> is less than
	return Payload->IsHigherPriorityThan(*Other.Payload);
}

bool FPackagePreloaderPriorityQueue::IsEmpty() const
{
	return Heap.IsEmpty();
}

void FPackagePreloaderPriorityQueue::Add(TRefCountPtr<FPackagePreloader> Preloader)
{
	Heap.HeapPush(FPackagePreloaderPriorityWrapper{ MoveTemp(Preloader) });
}

void FPackagePreloaderPriorityQueue::Remove(const TRefCountPtr<FPackagePreloader>& Preloader)
{
	int32 Index = Heap.IndexOfByPredicate([&Preloader](const FPackagePreloaderPriorityWrapper& Element)
		{
			return Element.Payload == Preloader;
		});
	if (Index != INDEX_NONE)
	{
		Heap.HeapRemoveAt(Index, EAllowShrinking::No);
	}
}

TRefCountPtr<FPackagePreloader> FPackagePreloaderPriorityQueue::PopFront()
{
	FPackagePreloaderPriorityWrapper Wrapper;
	Heap.HeapPop(Wrapper, EAllowShrinking::No);
	return MoveTemp(Wrapper.Payload);
}

bool FLoadQueue::IsEmpty()
{
	return InProgress.IsEmpty();
}

int32 FLoadQueue::Num() const
{
	return InProgress.Num();
}

void FLoadQueue::Add(FPackageData* PackageData)
{
	bool bAlreadyInSet;
	// The Package must be in the LoadState to be added to the container, and OnEnterLoading
	// guarantees a refcount exists on the Preloader. We rely on this, because we need to store
	// IsInInbox on the Preloader.
	TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
	check(Preloader);

	InProgress.Add(PackageData, &bAlreadyInSet);
	if (!bAlreadyInSet)
	{
		Inbox.Add(PackageData);
		Preloader->SetIsInInbox(true);
	}
}

bool FLoadQueue::Contains(const FPackageData* PackageData) const
{
	return InProgress.Contains(PackageData);
}

uint32 FLoadQueue::Remove(FPackageData* PackageData)
{
	uint32 Result = static_cast<uint32>(InProgress.Remove(PackageData));
	if (Result == 0)
	{
		return 0;
	}

	TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
	if (Preloader && Preloader->IsInInbox())
	{
		Inbox.Remove(PackageData);
		Preloader->SetIsInInbox(false);
	}
	// This Remove function is not responsible for removing the PackageData's Preloader from the sub containers for
	// preloaders. That responsibility is complicated and the work that needs to be done for it upon leaving the load
	// state is done by FPackageData::OnExitLoad.
	return Result;
}

void FLoadQueue::UpdateUrgency(FPackageData* PackageData, EUrgency bOldUrgency, EUrgency NewUrgency)
{
	TRefCountPtr<FPackagePreloader> Preloader = PackageData->GetPackagePreloader();
	if (!Preloader)
	{
		// Urgency does not impact state for packages that haven't reached a preloader state yet
		return;
	}
	switch (Preloader->GetState())
	{
	case EPreloaderState::Inactive:
		// Urgency does not impact state for packages that are inactive
		break;
	case EPreloaderState::PendingKick:
		PendingKicks.Remove(Preloader);
		PendingKicks.Add(Preloader);
		break;
	case EPreloaderState::ActivePreload:
		break;
	case EPreloaderState::ReadyForLoad:
		ReadyForLoads.Remove(Preloader);
		ReadyForLoads.AddFront(Preloader);
		break;
	default:
		checkNoEntry();
		break;
	}
}

TSet<FPackageData*>::TRangedForIterator FLoadQueue::begin()
{
	return InProgress.begin();
}

TSet<FPackageData*>::TRangedForIterator FLoadQueue::end()
{
	return InProgress.end();
}

}