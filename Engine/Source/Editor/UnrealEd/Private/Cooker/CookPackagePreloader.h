// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/CookPackageData.h"
#include "Cooker/CookTypes.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "UObject/PackageResourceManager.h"

#include <atomic>

class UCookOnTheFlyServer;

namespace UE::Cook
{

/**
 * Helper class for the Load state on FPackageData. Also supports loads of other packages that depend on the owner
 * package. Handles preloading or asyncloading the package before we call LoadPackage on it.
 */
class FPackagePreloader : public FRefCountedObject
{
public:
	FPackagePreloader(FPackageData& InPackageData);
	~FPackagePreloader();

	FPackageData& GetPackageData();

	EPreloaderState GetState() const;

	/** Free any IO requests or buffers, and remove all references to any other FPackagePreloaders. */
	void Shutdown();

	/** Set the SelfReference that keeps this object in memory while the owner package is using it. */
	void SetSelfReference();
	/** Clear the SelfReference; the owner package is done using this object. */
	void ClearSelfReference();

	/** Clear our completion records; we need to reevaluate them since packages have been removed from memory. */
	void PostGarbageCollect();
	/** Called when the owner package exits the loadstate; clears reference counts of requested imports. */
	void OnPackageLeaveLoadState();
	/** Return whether the Preloader's package is loaded. */
	bool IsPackageLoaded() const;
	/** Return whether the given PackageData's package is loaded. */
	static bool IsPackageLoaded(const FPackageData& InPackageData);
	static bool IsPackageLoaded(FName InPackageName);
	int32 GetCountFromRequestedLoads();

	/**
	 * State transition function that calls OnEnter/OnExit functions and (optionally) removes and adds the
	 * PackagePreloader to the container for its state.
	 */
	void SendToState(EPreloaderState NewState, ESendFlags SendFlags);

	// UCookOnTheFlyServer::PumpLoads helper functions, see comments in PumpLoads.
	static bool PumpLoadsTryStartInboxPackage(UCookOnTheFlyServer& COTFS);
	static bool PumpLoadsTryKickPreload(UCookOnTheFlyServer& COTFS);
	bool PumpLoadsIsReadyToLeavePreload(TRefCountPtr<FPackagePreloader>* OutBlocker = nullptr, FString* OutBlockerReason = nullptr);
	void PumpLoadsMarkLoadAttemptComplete();

private:
	/**
	 * The number of active PreloadableFiles is tracked globally; wrap the PreloadableFile in a struct that
	 * guarantees we always update the counter when changing it
	 */
	struct FTrackedPreloadableFilePtr
	{
		const TSharedPtr<FPreloadableArchive>& Get() { return Ptr; }
		void Set(TSharedPtr<FPreloadableArchive>&& InPtr, FPackageData& PackageData);
		void Reset(FPackageData& PackageData);
	private:
		TSharedPtr<FPreloadableArchive> Ptr;
	};

	/**
	 * Structure used when we are doing async packageloads rather than just preloading. Holds the
	 * request data that gets written from the AsyncLoading thread.
	 */
	struct FAsyncRequest
	{
	public:
		bool IsSucceeded() const;
		bool IsFailed() const;
		bool IsCompleted() const;

		void SetSucceeded();
		void SetFailed();

		int32 GetRequestID() const;
		void SetRequestID(int32 InRequestID);

	private:
		enum ECompletionState : uint8
		{
			Incomplete,
			Failed,
			Succeeded
		};
		int32 RequestID{ 0 };
		std::atomic<ECompletionState> CompletionState{ ECompletionState::Incomplete };
	};

	/** Used during DFS of the import graph. */
	enum class EGraphVisitState : uint8
	{
		Unvisited,
		InProgress,
		Visited,
	};

private:
	bool HasInitializedRequestedLoads() const;
	void SetHasInitializedRequestedLoads(bool bValue);
	bool IsInInbox() const;
	void SetIsInInbox(bool bValue);
	bool WasLoadAttempted() const;
	void SetLoadAttempted(bool bValue);
	bool IsPreloadAttempted() const;
	void SetIsPreloadAttempted(bool bValue);
	bool IsPreloaded() const;
	void SetIsPreloaded(bool bValue);
	bool IsImportsGathered() const;
	void SetIsImportsGathered(bool bValue);

	void SetState(EPreloaderState Value);
	/** Clear any allocated preload data. */
	void ClearPreload();
	void EmptyImports();
	/** Gather imports from the AssetRegistry, but skip Preloads that return true for NeedsLoad. */
	void GatherUnloadedImports();

	/**
	 * Traverse the import tree of this Preloader, optionally gathering imports from the asset registry if they have not
	 * already been gathered. For each package found, if loaded skip it, otherwise add it to the list of imports needed
	 * by the referencer, and make its PackagePreloader active in the loadqueue.
	 * 
	 * Caller can specify skipping a given import and not reporting it or its transitive imports.
	 * Caller provides a Report function that is called for each reported import.
	 * Packages are reported in root-to-leaf order. If a cycle is present, order in the cycle is arbitrary.
	 * 
	 * bool ShouldKeep(FPackagePreloader&) -> should Report be called and the preloader be explored.
	 * bool ReportAndIsContinue(FPackagePreloader&) -> should the search continue.
	 * 
	 * If bAllowGather is true, caller must activate all inactive reported PackagePreloaders, otherwise
	 * they will hold imports in the inactive state which is not valid.
	 */
	template <typename ShouldKeepFunc, typename ReportAndIsContinueFunc>
	void TraverseImportGraph(ShouldKeepFunc&& ShouldKeep, ReportAndIsContinueFunc&& ReportAndIsContinue,
		bool bAllowGather);
	/**
	 * TraverseImportGraph and return all the discovered needs-loading imports. The output list is
	 * in root to leaf order.
	 */
	void GetNeedsLoadPreloadersInImportTree(TArray<TRefCountPtr<FPackagePreloader>>& OutPreloaders);

	/**
	 * Try to preload just the file for this PackagePreloader's owner package, does not consider imports.
	 * Return true if preloading is complete (succeeded or failed or was skipped).
	 */
	bool TryPreload(FString* OutFailReason = nullptr);
	
	/** Increment the countfromrequestedloads. When it is decremented back to zero, the preload+load is cancelled. */
	void IncrementCountFromRequestedLoads();
	/** Decrement the countfromrequestedloads. When it is reaches zero, the preload+load is cancelled. */
	void DecrementCountFromRequestedLoads();
	/**
	 * Set the list of packages that are in the import tree of this one and have not loaded yet.
	 * Each one will have its CountFromRequestedLoads incremented to keep it in an active state until
	 * done loading.
	 * InRequestedLoads must be in Root to Leaf order; that order is used to define the priority of
	 * packages in PendingKick queue (leafwards packages are kicked earlier).
	 */
	void SetRequestedLoads(TArray<TRefCountPtr<FPackagePreloader>>&& InRequestedLoads, bool bMakeActive = true);
	/**
	 * Called when a PackagePreloader leaves the active state (regardless of what state its Package is in) to
	 * clear the resources used during preloading.
	 */
	void OnExitActive();
	/**
	 * Report whether the Preloader's package still needs to be loaded.
	 * Performance sideeffect: ClearPreload if it no longer needs to load.
	 */
	bool NeedsLoad();
	/** Used to decide which PendingKick PackagePreloaders should next be preloaded. */
	bool IsHigherPriorityThan(const FPackagePreloader& Other) const;

	static void InitializeConfig();

private:
	FTrackedPreloadableFilePtr PreloadableFile;
	FOpenPackageResult PreloadableFileOpenResult;
	FPackageData& PackageData;
	TRefCountPtr<FPackagePreloader> SelfReference;
	TSharedPtr<FAsyncRequest> AsyncRequest;
	TArray<TRefCountPtr<FPackagePreloader>> UnloadedImports;
	TArray<TRefCountPtr<FPackagePreloader>> RequestedLoads;
	int32 CountFromRequestedLoads = 0;
	EPreloaderState State = EPreloaderState::Inactive;
	EGraphVisitState VisitState = EGraphVisitState::Unvisited;
	bool bIsPreloadAttempted = false;
	bool bIsPreloaded = false;
	bool bImportsGathered = false;
	bool bIsInInbox = false;
	bool bHasInitializedRequestedLoads = false;
	bool bWasLoadAttempted = false;

	static bool bConfigInitialized;
	static bool bAllowPreloadImports;

	friend UE::Cook::FLoadQueue;
	friend UE::Cook::FPackagePreloaderPriorityWrapper;
};

}


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::Cook
{

inline FPackagePreloader::FPackagePreloader(FPackageData& InPackageData)
: PackageData(InPackageData)
{
	if (!bConfigInitialized)
	{
		InitializeConfig();
	}
}

inline FPackageData& FPackagePreloader::GetPackageData()
{
	return PackageData;
}

inline EPreloaderState FPackagePreloader::GetState() const
{
	return State;
}

inline void FPackagePreloader::SetState(EPreloaderState Value)
{
	State = Value;
}

inline void FPackagePreloader::SetSelfReference()
{
	SelfReference = this;
}

inline void FPackagePreloader::ClearSelfReference()
{
	SelfReference.SafeRelease();
}

inline int32 FPackagePreloader::GetCountFromRequestedLoads()
{
	return CountFromRequestedLoads;
}

inline bool FPackagePreloader::HasInitializedRequestedLoads() const
{
	return bHasInitializedRequestedLoads;
}

inline void FPackagePreloader::SetHasInitializedRequestedLoads(bool bValue)
{
	bHasInitializedRequestedLoads = bValue;
}

inline bool FPackagePreloader::IsInInbox() const
{
	return bIsInInbox;
}

inline void FPackagePreloader::SetIsInInbox(bool bValue)
{
	bIsInInbox = bValue;
}

inline bool FPackagePreloader::WasLoadAttempted() const
{
	return bWasLoadAttempted;
}

inline void FPackagePreloader::SetLoadAttempted(bool bValue)
{
	bWasLoadAttempted = bValue;
}

inline bool FPackagePreloader::IsPreloadAttempted() const
{
	return bIsPreloadAttempted;
}

inline void FPackagePreloader::SetIsPreloadAttempted(bool bValue)
{
	bIsPreloadAttempted = bValue;
}

inline bool FPackagePreloader::IsPreloaded() const
{
	return bIsPreloaded;
}

inline void FPackagePreloader::SetIsPreloaded(bool bValue)
{
	bIsPreloaded = bValue;
}

inline bool FPackagePreloader::IsImportsGathered() const
{
	return bImportsGathered;
}

inline void FPackagePreloader::SetIsImportsGathered(bool bValue)
{
	bImportsGathered = bValue;
}

inline bool FPackagePreloader::FAsyncRequest::IsSucceeded() const
{
	return CompletionState.load() == ECompletionState::Succeeded;
}

inline bool FPackagePreloader::FAsyncRequest::IsFailed() const
{
	return CompletionState.load() == ECompletionState::Failed;
}

inline bool FPackagePreloader::FAsyncRequest::IsCompleted() const
{
	return CompletionState.load() != ECompletionState::Incomplete;
}

inline void FPackagePreloader::FAsyncRequest::SetSucceeded()
{
	CompletionState = ECompletionState::Succeeded;
}

inline void FPackagePreloader::FAsyncRequest::SetFailed()
{
	CompletionState = ECompletionState::Failed;
}

inline int32 FPackagePreloader::FAsyncRequest::GetRequestID() const
{
	return RequestID;
}

inline void FPackagePreloader::FAsyncRequest::SetRequestID(int32 InRequestID)
{
	RequestID = InRequestID;
}

}