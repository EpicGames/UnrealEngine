// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookPackageSplitter.h"

#if WITH_EDITOR
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"

const TCHAR* ICookPackageSplitter::GetGeneratedPackageSubPath()
{
	return FPackageName::GetGeneratedPackageSubPath();
}

bool ICookPackageSplitter::IsUnderGeneratedPackageSubPath(FStringView FileOrLongPackagePath)
{
	return FPackageName::IsUnderGeneratedPackageSubPath(FileOrLongPackagePath);
}

FString ICookPackageSplitter::ConstructGeneratedPackageName(FName OwnerPackageName, FStringView RelPath,
	FStringView GeneratedRootOverride)
{
	FString PackageRoot;
	if (GeneratedRootOverride.IsEmpty())
	{
		PackageRoot = OwnerPackageName.ToString();
	}
	else
	{
		PackageRoot = GeneratedRootOverride;
	}
	return FPaths::RemoveDuplicateSlashes(FString::Printf(TEXT("/%s/%s/%.*s"),
		*PackageRoot, GetGeneratedPackageSubPath(), RelPath.Len(), RelPath.GetData()));
}

namespace UE::Cook::Private
{

static TLinkedList<FRegisteredCookPackageSplitter*>* GRegisteredCookPackageSplitterList = nullptr;

FRegisteredCookPackageSplitter::FRegisteredCookPackageSplitter()
: GlobalListLink(this)
{
	GlobalListLink.LinkHead(GetRegisteredList());
}

FRegisteredCookPackageSplitter::~FRegisteredCookPackageSplitter()
{
	GlobalListLink.Unlink();
}

TLinkedList<FRegisteredCookPackageSplitter*>*& FRegisteredCookPackageSplitter::GetRegisteredList()
{
	return GRegisteredCookPackageSplitterList;
}

void FRegisteredCookPackageSplitter::ForEach(TFunctionRef<void(FRegisteredCookPackageSplitter*)> Func)
{
	for (TLinkedList<FRegisteredCookPackageSplitter*>::TIterator It(GetRegisteredList()); It; It.Next())
	{
		Func(*It);
	}
}

}

#endif