// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookPlatformManager.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Cooker/CookTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTLS.h"
#include "HAL/PlatformTime.h"
#include "IWorkerRequests.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopeRWLock.h"

namespace UE::Cook
{

class FRWScopeLockConditional
{
public:
	FRWScopeLockConditional(FRWLock& InLockObject, FRWScopeLockType InLockType, bool bInNeedsLock)
		:LockObject(InLockObject)
		, LockType(InLockType)
		, bNeedsLock(bInNeedsLock)
	{
		if (bNeedsLock)
		{
			if (LockType != SLT_ReadOnly)
			{
				LockObject.WriteLock();
			}
			else
			{
				LockObject.ReadLock();
			}
		}
	}
	~FRWScopeLockConditional()
	{
		if (bNeedsLock)
		{
			if (LockType != SLT_ReadOnly)
			{
				LockObject.WriteUnlock();
			}
			else
			{
				LockObject.ReadUnlock();
			}
		}
	}

private:
	UE_NONCOPYABLE(FRWScopeLockConditional);

	FRWLock& LockObject;
	FRWScopeLockType LockType;
	bool bNeedsLock;
};

FPlatformData::FPlatformData()
	: LastReferenceTime(0.0)
	, ReferenceCount(0)
{
}

uint32 FPlatformManager::IsInPlatformsLockTLSSlot = FPlatformTLS::InvalidTlsSlot;

void FPlatformManager::InitializeTls()
{
	IsInPlatformsLockTLSSlot = FPlatformTLS::AllocTlsSlot();
}

bool FPlatformManager::IsInPlatformsLock()
{
	return FPlatformTLS::GetTlsValue(IsInPlatformsLockTLSSlot) != 0;
}

void FPlatformManager::SetIsInPlatformsLock(bool bValue)
{
	FPlatformTLS::SetTlsValue(IsInPlatformsLockTLSSlot, bValue ? (void*)0x1 : (void*)0x0);
}

FPlatformManager::~FPlatformManager()
{
	{
		FRWScopeLock PlatformDatasScopeLock(PlatformDatasLock, FRWScopeLockType::SLT_Write);
		for (TPair<const ITargetPlatform*, FPlatformData*>& kvpair : PlatformDatas)
		{
			FPlatformData* Value = kvpair.Value;
			delete Value;
		}
		PlatformDatas.Empty();
		PlatformDatasByName.Empty();
	}
}

const TArray<FPlatformId>& FPlatformManager::GetSessionPlatforms() const
{
	checkf(IsSchedulerThread(), TEXT("Access to SessionPlatforms on non-scheduler thread that persists outside of function scope is not yet implemented."));
	checkf(bHasSelectedSessionPlatforms, TEXT("Calling GetSessionPlatforms or (any of the top level cook functions that call it) without first calling SelectSessionPlatforms is invalid"));
	return SessionPlatforms;
}

int32 FPlatformManager::GetNumSessionPlatforms() const
{
	checkf(IsSchedulerThread() || IsInPlatformsLock(), TEXT("Access to SessionPlatforms is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));
	return SessionPlatforms.Num();
}

bool FPlatformManager::HasSessionPlatform(FPlatformId TargetPlatform) const
{
	const bool bIsSchedulerThread = IsSchedulerThread();
	checkf(bIsSchedulerThread || IsInPlatformsLock(), TEXT("Looking up platforms by PlatformId is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));
	FRWScopeLockConditional ScopeLock(SessionLock, FRWScopeLockType::SLT_ReadOnly, !bIsSchedulerThread);
	return SessionPlatforms.Contains(TargetPlatform);
}

void FPlatformManager::SelectSessionPlatforms(UCookOnTheFlyServer& COTFS,
	const TArrayView<FPlatformId const>& TargetPlatforms)
{
	checkf(IsSchedulerThread(), TEXT("Writing to SessionPlatforms is only allowed from the scheduler thread."));
	for (FPlatformId TargetPlatform : TargetPlatforms)
	{
		CreatePlatformData(COTFS, TargetPlatform);
	}

	FRWScopeLock ScopeLock(SessionLock, FRWScopeLockType::SLT_Write);
	SessionPlatforms.Empty(TargetPlatforms.Num());
	SessionPlatforms.Append(TargetPlatforms.GetData(), TargetPlatforms.Num());
	COTFS.bSessionRunning = true;
	bHasSelectedSessionPlatforms = true;
}

void FPlatformManager::ClearSessionPlatforms(UCookOnTheFlyServer& COTFS)
{
	checkf(IsSchedulerThread(), TEXT("Writing to SessionPlatforms is only allowed from the scheduler thread."));
	FRWScopeLock ScopeLock(SessionLock, FRWScopeLockType::SLT_Write);
	SessionPlatforms.Empty();
	COTFS.bSessionRunning = false;
	bHasSelectedSessionPlatforms = false;
}

void FPlatformManager::AddSessionPlatform(UCookOnTheFlyServer& COTFS, FPlatformId TargetPlatform)
{
	checkf(IsSchedulerThread(), TEXT("Writing to SessionPlatforms is only allowed from the scheduler thread."));
	CreatePlatformData(COTFS, TargetPlatform);

	{
		FRWScopeLock ScopeLock(SessionLock, FRWScopeLockType::SLT_ReadOnly);
		if (SessionPlatforms.Contains(TargetPlatform))
		{
			return;
		}
	}

	if (COTFS.bSessionRunning)
	{
		COTFS.OnBeforePlatformAddedToSession(TargetPlatform);
	}

	{
		FRWScopeLock ScopeLock(SessionLock, FRWScopeLockType::SLT_Write);
		if (SessionPlatforms.Contains(TargetPlatform))
		{
			return;
		}
		SessionPlatforms.Add(TargetPlatform);
		COTFS.bSessionRunning = true;
		bHasSelectedSessionPlatforms = true;
	}

	if (COTFS.bSessionRunning)
	{
		COTFS.OnPlatformAddedToSession(TargetPlatform);
	}
}

FPlatformData* FPlatformManager::GetPlatformData(FPlatformId Platform)
{
	const bool bIsSchedulerThread = IsSchedulerThread();
	checkf(bIsSchedulerThread || IsInPlatformsLock(), TEXT("Reading FPlatformData on non-scheduler threads is only allowed when inside a ReadLockPlatforms scope."));
	FPlatformData** Existing = PlatformDatas.Find(Platform);
	return Existing ? *Existing : nullptr;
}

FPlatformData* FPlatformManager::GetPlatformDataByName(FName PlatformName)
{
	const bool bIsSchedulerThread = IsSchedulerThread();
	checkf(bIsSchedulerThread || IsInPlatformsLock(), TEXT("Reading FPlatformData on non-scheduler threads is only allowed when inside a ReadLockPlatforms scope."));
	FPlatformData** Existing = PlatformDatasByName.Find(PlatformName);
	return Existing ? *Existing : nullptr;
}

FPlatformData& FPlatformManager::CreatePlatformData(UCookOnTheFlyServer& COTFS, const ITargetPlatform* Platform)
{
	check(Platform != nullptr);
	FPlatformData** ExistingPlatformData = PlatformDatas.Find(Platform);
	if (ExistingPlatformData)
	{
		return **ExistingPlatformData;
	}

	checkf(IsSchedulerThread(), TEXT("Writing to FPlatformData is only allowed from the scheduler thread."));
	FName PlatformName(Platform->PlatformName());
	checkf(!PlatformName.IsNone(), TEXT("Invalid ITargetPlatform with an empty name"));
	{
		FRWScopeLock PlatformDatasScopeLock(PlatformDatasLock, FRWScopeLockType::SLT_Write);

		FPlatformData*& PlatformData = PlatformDatas.Add(Platform);
		PlatformData = new FPlatformData;
		PlatformDatasByName.Add(PlatformName, PlatformData);

		// We could get the non-const ITargetPlatform* from the global PlatformTargetModule,
		// so this const cast is just a performance shortcut rather than a contract break
		ITargetPlatform* NonConstPlatform = const_cast<ITargetPlatform*>(Platform);
		PlatformData->TargetPlatform = NonConstPlatform;
		PlatformData->PlatformName = PlatformName;

		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		AssetRegistry.InitializeSerializationOptions(PlatformData->AssetRegistryDevelopmentSaveOptions, Platform,
			// This set of filters applies to the DevelopmentAssetRegistry created by AssetRegistryGenerator, so pass
			// in ForDevelopment. The AssetRegistryState will be filtered further as ForGame when the
			// AssetRegistryGenerator creates the RuntimeAssetRegistry from the DevelopmentAssetRegistry during
			// FAssetRegistryGenerator::SaveAssetRegistry.
			UE::AssetRegistry::ESerializationTarget::ForDevelopment);
		PlatformData->AssetRegistryDevelopmentSaveOptions.bKeepDevelopmentAssetRegistryTags = true;
		if (COTFS.IsForceNoFilterAssetsFromAssetRegistry())
		{
			PlatformData->AssetRegistryDevelopmentSaveOptions.DisableFilters();
		}

		return *PlatformData;
	}
}

bool FPlatformManager::IsPlatformInitialized(FPlatformId Platform) const
{
	checkf(IsSchedulerThread() || IsInPlatformsLock(), TEXT("Looking up platforms by PlatformId is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));
	const FPlatformData*const* PlatformData = PlatformDatas.Find(Platform);
	if (!PlatformData)
	{
		return false;
	}
	return (*PlatformData)->bIsSandboxInitialized;
}

void FPlatformManager::SetArePlatformsPrepopulated(bool bValue)
{
	checkf(IsSchedulerThread(), TEXT("Get/SetArePlatformsPrepopulated is only allowed from the scheduler thread."));
	bArePlatformsPrepopulated = bValue;
}

bool FPlatformManager::GetArePlatformsPrepopulated() const
{
	checkf(IsSchedulerThread(), TEXT("Get/SetArePlatformsPrepopulated is only allowed from the scheduler thread."));
	return bArePlatformsPrepopulated;
}

void FPlatformManager::PruneUnreferencedSessionPlatforms(UCookOnTheFlyServer& CookOnTheFlyServer)
{
	checkf(IsSchedulerThread(), TEXT("Writing to SessionPlatforms is only allowed from the scheduler thread."));
	const double SecondsToLive = 5.0 * 60;

	// OldestKeepTime is constructed to less than -SecondsToLive, so we can robustly detect not-yet-initialized
	double OldestKeepTime = -1.0e10;
	TArray<const ITargetPlatform*, TInlineAllocator<1>> RemovePlatforms;

	for (TPair<const ITargetPlatform*, FPlatformData*>& kvpair : PlatformDatas)
	{
		FPlatformData* PlatformData = kvpair.Value;
		if (PlatformData->LastReferenceTime > 0. && PlatformData->ReferenceCount == 0)
		{
			// We have a platform that we need to check for pruning.
			// Initialize the OldestKeepTime so we can check whether the platform has aged out.
			if (OldestKeepTime < -SecondsToLive)
			{
				const double CurrentTimeSeconds = FPlatformTime::Seconds();
				OldestKeepTime = CurrentTimeSeconds - SecondsToLive;
			}

			// Note that this loop is outside of the critical section, for performance.
			// If we find any candidates for pruning we have to check them again once inside the critical section.
			if (kvpair.Value->LastReferenceTime < OldestKeepTime)
			{
				RemovePlatforms.Add(kvpair.Key);
			}
		}
	}

	if (RemovePlatforms.Num() > 0)
	{
		FRWScopeLock Lock(SessionLock, FRWScopeLockType::SLT_Write);

		for (const ITargetPlatform* TargetPlatform : RemovePlatforms)
		{
			FPlatformData* PlatformData = *PlatformDatas.Find(TargetPlatform);
			if (PlatformData->LastReferenceTime > 0. && PlatformData->ReferenceCount == 0
				&& PlatformData->LastReferenceTime < OldestKeepTime)
			{
				int32 RemovedIndex = SessionPlatforms.IndexOfByKey(TargetPlatform);
				check(RemovedIndex != INDEX_NONE);

				// Mark that the platform no longer needs to be inspected for pruning because we have removed it
				// from CookOnTheFly's SessionPlatforms
				PlatformData->LastReferenceTime = 0.;

				// Remove the SessionPlatform
				CookOnTheFlyServer.OnRemoveSessionPlatform(TargetPlatform, RemovedIndex);

				SessionPlatforms.RemoveAt(RemovedIndex);
				if (SessionPlatforms.Num() == 0)
				{
					CookOnTheFlyServer.bSessionRunning = false;
					bHasSelectedSessionPlatforms = false;
				}
			}
		}
	}
}

void FPlatformManager::AddRefCookOnTheFlyPlatform(FName PlatformName, UCookOnTheFlyServer& CookOnTheFlyServer)
{
	checkf(IsSchedulerThread() || IsInPlatformsLock(), TEXT("AddRefCookOnTheFlyPlatform is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));

	FPlatformData* PlatformData = GetPlatformDataByName(PlatformName);
	checkf(PlatformData != nullptr, TEXT("Unrecognized Platform %s"), *PlatformName.ToString());
	++PlatformData->ReferenceCount;

	if (!HasSessionPlatform(PlatformData->TargetPlatform))
	{
		CookOnTheFlyServer.WorkerRequests->AddCookOnTheFlyCallback([PlatformName,
			LocalCookOnTheFlyServer = &CookOnTheFlyServer, this]()
			{
				ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformName.ToString());
				if (TargetPlatform)
				{
					// We might get multiple AddRef calls that add this callback before the first one reaches
					// StartCookOnTheFlySessionFromGameThread, so check again whether some earlier request
					// has already added the sessionplatform
					if (!HasSessionPlatform(TargetPlatform))
					{
						LocalCookOnTheFlyServer->StartCookOnTheFlySessionFromGameThread(TargetPlatform);
					}
				}
			});
	}
}

void FPlatformManager::ReleaseCookOnTheFlyPlatform(FName PlatformName)
{
	checkf(IsSchedulerThread() || IsInPlatformsLock(), TEXT("ReleaseCookOnTheFlyPlatform is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));
		
	FPlatformData* PlatformData = GetPlatformDataByName(PlatformName);
	checkf(PlatformData != nullptr, TEXT("Unrecognized Platform %s"), *PlatformName.ToString());
	check(PlatformData->ReferenceCount > 0);
	--PlatformData->ReferenceCount;
	PlatformData->LastReferenceTime = FPlatformTime::Seconds();
}

TMap<ITargetPlatform*, ITargetPlatform*> FPlatformManager::RemapTargetPlatforms()
{
	checkf(IsSchedulerThread(), TEXT("Writing to PlatformDatas is only allowed from the scheduler thread."));
	TMap<ITargetPlatform*, ITargetPlatform*> Remap;

	ITargetPlatformManagerModule* PlatformManager = GetTargetPlatformManager();
	TMap<const ITargetPlatform*, FPlatformData*> NewPlatformDatas;

	{
		FRWScopeLock PlatformDatasScopeLock(PlatformDatasLock, FRWScopeLockType::SLT_Write);
		for (TPair<const ITargetPlatform*, FPlatformData*>& kvpair : PlatformDatas)
		{
			ITargetPlatform* Old = const_cast<ITargetPlatform*>(kvpair.Key);
			FPlatformData& Data = *kvpair.Value;
			FName PlatformName = Data.PlatformName;

			ITargetPlatform* New = PlatformManager->FindTargetPlatform(PlatformName.ToString());
			checkf(New, TEXT("TargetPlatform %s has been removed from the list of TargetPlatforms from ITargetPlatformManagerModule after cooking has started; this case is not handled."),
				*PlatformName.ToString());
			Data.TargetPlatform = New;

			Remap.Add(Old, New);
			NewPlatformDatas.Add(New, &Data);
		}
		// Note that PlatformDatasByName is unchanged, since it maps PlatformName -> PlatformData*,
		// and neither of those have changed. The values inside the FPlatformData may have changed, though,
		// so we need to hold the lock during this function.
		Swap(PlatformDatas, NewPlatformDatas);

		{
			FRWScopeLock SessionScopeLock(SessionLock, FRWScopeLockType::SLT_Write);
			RemapArrayElements(SessionPlatforms, Remap);
		}
	}

	return Remap;
}

FPlatformManager::FReadScopeLock::FReadScopeLock(FPlatformManager& InPlatformManager)
	:PlatformManager(InPlatformManager)
{
	bAttached = true;
	PlatformManager.PlatformDatasLock.ReadLock();
	check(!IsInPlatformsLock());
	SetIsInPlatformsLock(true);
}

FPlatformManager::FReadScopeLock::FReadScopeLock(FReadScopeLock&& Other)
	:PlatformManager(Other.PlatformManager)
{
	bAttached = Other.bAttached;
	Other.bAttached = false;
}

FPlatformManager::FReadScopeLock::~FReadScopeLock()
{
	if (bAttached)
	{
		SetIsInPlatformsLock(false);
		PlatformManager.PlatformDatasLock.ReadUnlock();
		bAttached = false;
	}
}

FPlatformManager::FReadScopeLock FPlatformManager::ReadLockPlatforms()
{
	return FReadScopeLock(*this);
}

}
