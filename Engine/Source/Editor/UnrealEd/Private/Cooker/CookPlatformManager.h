// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetRegistryState.h"
#include "Commandlets/AssetRegistryGenerator.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CookTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Templates/Atomic.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class ITargetPlatform;
class UCookOnTheFlyServer;
namespace UE::Cook { class IAssetRegistryReporter; }

namespace UE::Cook
{

	/*
	 * Struct to hold data about each platform we have encountered in the cooker.
	 * Fields in this struct persist across multiple CookByTheBook sessions.
	 * All fields on this struct are readable only from either the scheduler thread or within a ReadLock on
	 * non-scheduler thread.
	 * Some fields on this struct are SingleThreadWrite fields - written only from the scheduler thread.
	 * These are unsynchronized other than SingleThreadWrite readlocks.
	 * The remaining fields are read/write from any thread, and are TAtomics.
	 */
	struct FPlatformData
	{
		FPlatformData();

		/* Name of the Platform, a cache of FName(ITargetPlatform->PlatformName()) */
		FName PlatformName;

		/**
		 * The TargetPlatform returned from TargetPlatformManagerModule.
		 * This may change to a different pointer if Invalidate is called on the TargetPlatformManagerModule, so it is
		 * only valid to dereference on the ScheduleThread since invalidation occurs on the ScheduleThread.
		 */
		ITargetPlatform* TargetPlatform;

		/*
		* Pointer to the platform-specific RegistryGenerator for this platform.  If already constructed we can take a
		* faster refresh path on future sessions.
		*/
		TUniquePtr<FAssetRegistryGenerator> RegistryGenerator;
		TUniquePtr<UE::Cook::IAssetRegistryReporter> RegistryReporter;
		FAssetRegistrySerializationOptions AssetRegistryDevelopmentSaveOptions;

		/*
		* Whether BeginCookSandbox has been called for this platform.  If we have already initialized the sandbox we
		* can take a faster refresh path on future sessions.
		*/
		bool bIsSandboxInitialized = false;

		/** Whether BeginCookSandbox specified a full, non-incremental build for this platform. */
		bool bFullBuild = false;
		/**
		 * Whether BeginCookSandBox specied non-full, and is in an incremental mode that allows using previous results.
		 * -diffonly is the expected case where bFullBuild=false but bAllowIncrementalResults=false.
		 */
		bool bAllowIncrementalResults = true;

		/**
		 * If true we are cooking with legacyiterative, from results in a shared build (e.g. from buildfarm) rather
		 * than from our previous cook.
		 */
		bool bLegacyIterativeSharedBuild = false;

		/** If true we are a CookWorker, and we are working on a Sandbox directory that has already been populated by a remote Director process. */
		bool bWorkerOnSharedSandbox = false;

		/*
		 * The last FPlatformTime::GetSeconds() at which this platform was requested in a CookOnTheFly request.
		 * If equal to 0, the platform was not requested in a CookOnTheFly since the last clear.
		 */
		TAtomic<double> LastReferenceTime;

		/* The count of how many active CookOnTheFly requests are currently using the Platform. */
		TAtomic<uint32> ReferenceCount;
	};

	typedef const ITargetPlatform* FPlatformId;

	/*
	 * Information about the platforms (a) known and (b) active for the current cook session in the UCookOnTheFlyServer.
	 * This structure is SingleThreadWrite-ThreadSafe.  All read accesses from threads other than the SchedulerThread
	 * enter read locks (or require read locks held). 
	 * Write accesses enter write locks, and assert if not on the SchedulerThread.
	*/
	struct FPlatformManager
	{
	public:
		~FPlatformManager();

		/** Returns the TargetPlatforms that are active for the CurrentCookByTheBook session or CookOnTheFly request. */
		const TArray<const ITargetPlatform*>& GetSessionPlatforms() const;
		/**
		 * Calls the Callback with each platform in GetSessionPlatforms. This is the accessor that must be used when
		 * trying to read the SessionPlatforms from an async thread in an FPlatformManager::FReadScopeLock.
		 */
		template <typename CallbackType>
		void EnumerateSessionPlatforms(CallbackType&& Callback) const;
		int32 GetNumSessionPlatforms() const;

		/** Return whether the given platform is already in the list of platforms for the current session. */
		bool HasSessionPlatform(FPlatformId TargetPlatform) const;

		/** Specify the TargetPlatforms to use for the currently CookByTheBook session or CookOnTheFly request. */
		void SelectSessionPlatforms(UCookOnTheFlyServer& COTFS, const TArrayView<FPlatformId const>& TargetPlatforms);

		/**
		 * Mark that the TargetPlatforms are no longer set; reading them without first calling
		 * SelectSessionPlatforms is an error.
		 */
		void ClearSessionPlatforms(UCookOnTheFlyServer& COTFS);

		/** Add @param TargetPlatform to the session platforms if not already present. */
		void AddSessionPlatform(UCookOnTheFlyServer& COTFS, FPlatformId TargetPlatform);

		/**
		 * Get The PlatformData for the given Platform.
		 * Guaranteed to return non-null for any Platform in the current list of SessionPlatforms.
		 */
		FPlatformData* GetPlatformData(FPlatformId Platform);

		/**
		 * Get The PlatformData for the given Platform, looked up by name since NetworkThread does not have
		 * FPlatformId to start out with.
		 */
		FPlatformData* GetPlatformDataByName(FName PlatformName);

		/** Create if not already created the necessary platform-specific data for the given platform. */
		FPlatformData& CreatePlatformData(UCookOnTheFlyServer& COTFS, const ITargetPlatform* Platform);

		/** Return whether platform-specific setup steps have run in the current UCookOnTheFlyServer. */
		bool IsPlatformInitialized(FPlatformId Platform) const;

		/**
		 * Get/Set the Prepopulated flag. If platforms are prepopulated, then we need to repopulate the list
		 * whenever targetplatforms are invalidated.
		 */
		void SetArePlatformsPrepopulated(bool bValue);
		bool GetArePlatformsPrepopulated() const;

		/**
		 * Platforms requested in CookOnTheFly requests are added to the list of SessionPlatforms, and some packages
		 * (e.g. unsolicited packages) are cooked against all session packages.
		 * To have good performance in the case where multiple targetplatforms are being requested over time from the
		 * same CookOnTheFly server, we prune platforms from the list of active session platforms if they haven't been
		 * requested in a while.
		 */
		void PruneUnreferencedSessionPlatforms(UCookOnTheFlyServer& CookOnTheFlyServer);

		/**
		 * Increment the counter indicating the current platform is requested in an active CookOnTheFly request.
		 * Add it to the SessionPlatforms if not already present.
		 */
		void AddRefCookOnTheFlyPlatform(FName PlatformName, UCookOnTheFlyServer& CookOnTheFlyServer);

		/** Decrement the counter indicating the current platform is being used in a CookOnTheFly request. */
		void ReleaseCookOnTheFlyPlatform(FName TargetPlatform);

		/**
		 * Called from CookOnTheFlyServer when TargetPlatformManagerModule reports that ITargetPlatform* have been
		 * invalided. Constructs a map from the old ITargetPlatform* to the new ITargetPlatform* using the PlatformName
		 * that we cached from the old ITargetPlatform* to identify the matching new one. Modifies all internal
		 * ITargetPlatform* values and returns the map to use for other stored copies of ITargetPlatform*.
		 */
		TMap<ITargetPlatform*, ITargetPlatform*> RemapTargetPlatforms();

		/* 
		 * When using FPlatformManager from a thread other than the scheduler thread (e.g. HandleNetworkFileServer
		 * functions), callers must lock the list of PlatformDatas using ReadLockPlatforms.
		 */
		struct FReadScopeLock
		{
			~FReadScopeLock();
			FReadScopeLock(FReadScopeLock&& Other);

		private:
			FReadScopeLock(FPlatformManager& InPlatformManager);
			FReadScopeLock(const FReadScopeLock& Other) = delete;

			bool bAttached = false;
			FPlatformManager& PlatformManager;
			friend struct FPlatformManager;
		};
		FReadScopeLock ReadLockPlatforms();

		/** Initialize storage for the global ThreadLocalStorage used by this class. */
		static void InitializeTls();

	private:
		static bool IsInPlatformsLock();
		static void SetIsInPlatformsLock(bool bValue);
		static uint32 IsInPlatformsLockTLSSlot;

		/** A collection of flags and other data we maintain for each platform we have encountered in any session. */
		TMap<FPlatformId, FPlatformData*> PlatformDatas;
		TMap<FName, FPlatformData*> PlatformDatasByName;

		/*
		 * RWLock used to guard PlatformDatas, PlatformDatasByName, and the validity of FPlatformIds. These fields are
		 * writable only on the SchedulerThread, so SchedulerThreadread operations can skip taking this lock.
		 * To prevent deadlocks, if PlatformDatasLock and SessionLock are held at the same time in a block of code,
		 * then PlatformDatasLock must be entered before entering SessionLock.
		 */
		mutable FRWLock PlatformDatasLock;

		/**
		 * A collection of Platforms that are active for the current CookByTheBook session or CookOnTheFly request.
		 * Used so we can refer to "all platforms" without having to store a list on every FileRequest.
		 * Writing to the list of active session platforms requires a CriticalSection, because it is read (under
		 * critical section) on NetworkFileServer threads.
		 */
		TArray<FPlatformId> SessionPlatforms;

		/**
		 * RWLock used to guard SessionPlatforms. SessionPlatforms can only be written from the SchedulerThread, so
		 * SchedulerThread read operations can skip taking this lock.
		 * To prevent deadlocks, if PlatformDatasLock and SessionLock are held at the same time in a block of code,
		 * then PlatformDatasLock must be entered before entering SessionLock.
		 */
		mutable FRWLock SessionLock;
		
		bool bArePlatformsPrepopulated = false;

		/**
		 * Duplicate of COTFS.bSessionRunning. Used for assertions on other threads. It is invalid to attempt to cook
		 * if session platforms have not been selected.
		 */
		bool bHasSelectedSessionPlatforms = false;
	};

} // namespace UE::Cook


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


template <typename Value>
void RemapMapKeys(TMap<const ITargetPlatform*, Value>& Map, const TMap<ITargetPlatform*,
	ITargetPlatform*>& Remap)
{
	TMap<const ITargetPlatform*, Value> NewMap;
	NewMap.Reserve(Map.Num());
	for (TPair<const ITargetPlatform*, Value>& OldPair : Map)
	{
		NewMap.Add(Remap[OldPair.Key], MoveTemp(OldPair.Value));
	}
	Swap(Map, NewMap);
}

inline void RemapArrayElements(TArray<const ITargetPlatform*>& Array, const TMap<ITargetPlatform*,
	ITargetPlatform*>& Remap)
{
	for (const ITargetPlatform*& Old : Array)
	{
		Old = Remap[Old];
	}
}


namespace UE::Cook
{

template <typename CallbackType>
void FPlatformManager::EnumerateSessionPlatforms(CallbackType&& Callback) const
{
	checkf(IsSchedulerThread() || IsInPlatformsLock(), TEXT("Access to SessionPlatforms is only legal on non-scheduler threads when inside a ReadLockPlatforms scope."));
	checkf(bHasSelectedSessionPlatforms, TEXT("Calling GetSessionPlatforms or (any of the top level cook functions that call it) without first calling SelectSessionPlatforms is invalid"));
	for (const ITargetPlatform* TargetPlatform : SessionPlatforms)
	{
		Callback(TargetPlatform);
	}
}

} // namespace UE::Cook
