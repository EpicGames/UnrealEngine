// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookProfiling.h" // Keep as first include to satisfy UBT
#include "CookProfilingPrivate.h" 

#include "Algo/GraphConvert.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookLog.h"
#include "CoreGlobals.h"
#include "DerivedDataBuildRemoteExecutor.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/OutputDevice.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "PackageBuildDependencyTracker.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Templates/Casts.h"
#include "Templates/Greater.h"
#include "UObject/GCObject.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"

#if OUTPUT_COOKTIMING || ENABLE_COOK_STATS
#include "ProfilingDebugging/ScopedTimers.h"
#endif

#if OUTPUT_COOKTIMING
#include <Containers/AllocatorFixedSizeFreeList.h>
#endif

#if ENABLE_COOK_STATS
#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "IAnalyticsProviderET.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheUsageStats.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Experimental/ZenServerInterface.h"
#endif

#if OUTPUT_COOKTIMING
struct FHierarchicalTimerInfo
{
public:
	FHierarchicalTimerInfo(const FHierarchicalTimerInfo& InTimerInfo) = delete;
	FHierarchicalTimerInfo(FHierarchicalTimerInfo&& InTimerInfo) = delete;

	explicit FHierarchicalTimerInfo(const char* InName, uint16 InId)
	:	Id(InId)
	,	Name(InName)
	{
	}

	~FHierarchicalTimerInfo()
	{
		ClearChildren();
	}

	void ClearChildren()
	{
		for (FHierarchicalTimerInfo* Child = FirstChild; Child;)
		{
			FHierarchicalTimerInfo* NextChild = Child->NextSibling;

			DestroyAndFree(Child);

			Child = NextChild;
		}
		FirstChild = nullptr;
	}
	FHierarchicalTimerInfo* GetChild(uint16 InId, const char* InName)
	{
		for (FHierarchicalTimerInfo* Child = FirstChild; Child;)
		{
			if (Child->Id == InId)
				return Child;

			Child = Child->NextSibling;
		}

		FHierarchicalTimerInfo* Child = AllocNew(InName, InId);

		Child->NextSibling	= FirstChild;
		FirstChild			= Child;

		return Child;
	}
	

	uint32							HitCount = 0;
	uint16							Id = 0;
	bool							IncrementDepth = true;
	double							Length = 0;
	const char*						Name;

	FHierarchicalTimerInfo*			FirstChild = nullptr;
	FHierarchicalTimerInfo*			NextSibling = nullptr;

private:
	static FHierarchicalTimerInfo*	AllocNew(const char* InName, uint16 InId);
	static void						DestroyAndFree(FHierarchicalTimerInfo* InPtr);
};

static TAllocatorFixedSizeFreeList<sizeof(FHierarchicalTimerInfo), 256> TimerInfoAllocator;
static FHierarchicalTimerInfo RootTimerInfo("Root", 0);
static FHierarchicalTimerInfo* CurrentTimerInfo = &RootTimerInfo;

FHierarchicalTimerInfo* FHierarchicalTimerInfo::AllocNew(const char* InName, uint16 InId)
{
	return new(TimerInfoAllocator.Allocate()) FHierarchicalTimerInfo(InName, InId);
}

void FHierarchicalTimerInfo::DestroyAndFree(FHierarchicalTimerInfo* InPtr)
{
	InPtr->~FHierarchicalTimerInfo();
	TimerInfoAllocator.Free(InPtr);
}

FScopeTimer::FScopeTimer(int InId, const char* InName, bool IncrementScope /*= false*/ )
{
	checkSlow(IsInGameThread());

	HierarchyTimerInfo = CurrentTimerInfo->GetChild(static_cast<uint16>(InId), InName);

	HierarchyTimerInfo->IncrementDepth = IncrementScope;

	PrevTimerInfo		= CurrentTimerInfo;
	CurrentTimerInfo	= HierarchyTimerInfo;
}

void FScopeTimer::Start()
{
	if (StartTime)
	{
		return;
	}

	StartTime = FPlatformTime::Cycles64();
}

void FScopeTimer::Stop()
{
	if (!StartTime)
	{
		return;
	}

	HierarchyTimerInfo->Length += FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
	++HierarchyTimerInfo->HitCount;

	StartTime = 0;
}

FScopeTimer::~FScopeTimer()
{
	Stop();

	check(CurrentTimerInfo == HierarchyTimerInfo);
	CurrentTimerInfo = PrevTimerInfo;
}

void OutputHierarchyTimers(const FHierarchicalTimerInfo* TimerInfo, int32 Depth)
{
	FString TimerName(TimerInfo->Name);

	static const TCHAR LeftPad[] = TEXT("                                ");
	const SIZE_T PadOffset = FMath::Max<int>(UE_ARRAY_COUNT(LeftPad) - 1 - Depth * 2, 0);

	UE_LOGF(LogCookStats, Display, "  %ls%ls: %.3fs (%u)", &LeftPad[PadOffset], *TimerName, TimerInfo->Length, TimerInfo->HitCount);

	// We need to print in reverse order since the child list begins with the most recently added child

	TArray<const FHierarchicalTimerInfo*> Stack;

	for (const FHierarchicalTimerInfo* Child = TimerInfo->FirstChild; Child; Child = Child->NextSibling)
	{
		Stack.Add(Child);
	}

	const int32 ChildDepth = Depth + TimerInfo->IncrementDepth;

	for (size_t i = Stack.Num(); i > 0; --i)
	{
		OutputHierarchyTimers(Stack[i - 1], ChildDepth);
	}
}

void OutputHierarchyTimers()
{
	UE_LOGF(LogCookStats, Display, "Hierarchy Timer Information:");

	OutputHierarchyTimers(&RootTimerInfo, 0);
}

void ClearHierarchyTimers()
{
	RootTimerInfo.ClearChildren();
}

#endif

#if ENABLE_COOK_STATS
namespace DetailedCookStats
{
	// Cook.Profile.Params
	FString CookProject;
	FString CookCultures;
	FString CookLabel;
	FString TargetPlatforms;
	bool IsCookAll = false;
	bool IsCookOnTheFly = false;
	bool IsIncrementalCook = false;
	bool IsIterativeCook = false;
	bool IsFastCook = false;
	bool IsUnversioned = false;

	// Cook.Profile
	double CookStartTime;
	double CookWallTimeSec;
	double StartupWallTimeSec;
	double StartCookByTheBookTimeSec;
	double BlockOnAssetRegistryTimeSec;
	double GameCookModificationDelegateTimeSec;
	double TickCookOnTheSideTimeSec;
	double TickCookOnTheSideLoadPackagesTimeSec;
	double TickCookOnTheSideSaveCookedPackageTimeSec;
	double TickCookOnTheSideResolveRedirectorsTimeSec;
	double TickCookOnTheSidePrepareSaveTimeSec;
	double TickLoopGCTimeSec;
	double TickLoopRecompileShaderRequestsTimeSec;
	double TickLoopShaderProcessAsyncResultsTimeSec;
	double TickLoopProcessDeferredCommandsTimeSec;
	double TickLoopTickCommandletStatsTimeSec;
	double TickLoopFlushRenderingCommandsTimeSec;
	double ShaderFlushTimeSec;
	double ValidationTimeSec;

	// CookOnTheFlyServer.Package.Load
	int32 NumRequestedLoads = 0;
	std::atomic<int32> NumDetectedLoads{ 0 };
	
	// CookOnTheFlyServer.Package.Save
	uint32 NumPackagesIncrementallySkipped = 0;
	
	//"CookOnTheFlyServer.QueueSize"
	int32 PeakRequestQueueSize = 0;
	int32 PeakLoadQueueSize = 0;
	int32 PeakSaveQueueSize = 0;
	
	TMap<FTopLevelAssetPath, uint32> NumPackagesIncrementallyModifiedByClass;

	FCookStatsManager::FAutoRegisterCallback RegisterStats([](FCookStatsManager::AddStatFuncRef AddStat)
		{

#define COOK_PROFILE_STAT_TEXT(name) TEXT(#name), name

			AddStat(TEXT("Cook.Profile.Params"),
				FCookStatsManager::CreateKeyValueArray(
					COOK_PROFILE_STAT_TEXT(CookProject),
					COOK_PROFILE_STAT_TEXT(CookCultures),
					COOK_PROFILE_STAT_TEXT(CookLabel),
					COOK_PROFILE_STAT_TEXT(TargetPlatforms),
					COOK_PROFILE_STAT_TEXT(IsCookAll),
					COOK_PROFILE_STAT_TEXT(IsCookOnTheFly),
					COOK_PROFILE_STAT_TEXT(IsIncrementalCook),
					COOK_PROFILE_STAT_TEXT(IsIterativeCook),
					COOK_PROFILE_STAT_TEXT(IsFastCook),
					COOK_PROFILE_STAT_TEXT(IsUnversioned)
					)
			);

			AddStat(TEXT("CookOnTheFlyServer.Package.Load"),
				FCookStatsManager::CreateKeyValueArray(
					TEXT("NumRequestedLoads"), NumRequestedLoads,
					TEXT("NumPackagesLoaded"), NumDetectedLoads.load(),
					TEXT("NumInlineLoads"), NumDetectedLoads - NumRequestedLoads
				)
			);

			AddStat(TEXT("CookOnTheFlyServer.Package.Save"),
				FCookStatsManager::CreateKeyValueArray(
					TEXT("NumPackagesIncrementallySkipped"), NumPackagesIncrementallySkipped
				)
			);

			AddStat(TEXT("CookOnTheFlyServer.QueueSize"),
				FCookStatsManager::CreateKeyValueArray(
					TEXT("PeakRequest"), PeakRequestQueueSize,
					TEXT("PeakLoad"), PeakLoadQueueSize,
					TEXT("PeakSave"), PeakSaveQueueSize
				)
			);

			// Hierarchical profiling stats
			const FString StatName(TEXT("Cook.Profile"));
#define ADD_COOK_STAT_FLT(Path, Name) AddStat(StatName, FCookStatsManager::CreateKeyValueArray(TEXT("Path"), TEXT(Path), TEXT(#Name), Name))

			ADD_COOK_STAT_FLT(" 0", CookWallTimeSec);
			ADD_COOK_STAT_FLT(" 0. 0", StartupWallTimeSec);
			double CookProcessingTime = CookWallTimeSec - StartupWallTimeSec;
			ADD_COOK_STAT_FLT(" 0. 1", CookProcessingTime);
			ADD_COOK_STAT_FLT(" 0. 1. 1", StartCookByTheBookTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 1. 0", BlockOnAssetRegistryTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 1. 1", GameCookModificationDelegateTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 2", TickCookOnTheSideTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 2. 0", TickCookOnTheSideLoadPackagesTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 2. 1", TickCookOnTheSideSaveCookedPackageTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 2. 1. 0", TickCookOnTheSideResolveRedirectorsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 2. 2", TickCookOnTheSidePrepareSaveTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 3", TickLoopGCTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 4", TickLoopRecompileShaderRequestsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 5", TickLoopShaderProcessAsyncResultsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 6", TickLoopProcessDeferredCommandsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 7", TickLoopTickCommandletStatsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 8", TickLoopFlushRenderingCommandsTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 9", ShaderFlushTimeSec);
			ADD_COOK_STAT_FLT(" 0. 1. 10", ValidationTimeSec);
		});

	void SendLogCookStats(ECookMode::Type CookMode)
	{
		if (IsCookingInEditor(CookMode))
		{
			return;
		}

		/** Used to store profile data for custom logging. */
		struct FCookProfileData
		{
		public:
			FCookProfileData(FString InPath, FString InKey, FString InValue) : Path(MoveTemp(InPath)), Key(MoveTemp(InKey)), Value(MoveTemp(InValue)) {}
			FString Path;
			FString Key;
			FString Value;
		};

		// instead of printing the usage stats generically, we capture them so we can log a subset of them in an easy-to-read way.
		TArray<FCookProfileData> CookProfileData;
		TArray<FString> StatCategories;
		TMap<FString, TArray<FCookStatsManager::StringKeyValue>> StatsInCategories;

		/** this functor will take a collected cooker stat and log it out using some custom formatting based on known stats that are collected.. */
		auto LogStatsFunc = [&CookProfileData, &StatCategories, &StatsInCategories]
		(const FString& StatName, const TArray<FCookStatsManager::StringKeyValue>& StatAttributes)
			{
				// Some stats will use custom formatting to make a visibly pleasing summary.
				bool bStatUsedCustomFormatting = false;

				if (StatName == TEXT("DDC.Usage"))
				{
					// Don't even log this detailed DDC data. It's mostly only consumable by ingestion into pivot tools.
					bStatUsedCustomFormatting = true;
				}
				else if (StatName.EndsWith(TEXT(".Usage"), ESearchCase::IgnoreCase))
				{
					// These are gathered through GatherResourceStats.
					bStatUsedCustomFormatting = true;
				}
				else if (StatName == TEXT("DDC.Summary"))
				{
					bStatUsedCustomFormatting = true;
				}
				else if (StatName == TEXT("Cook.Profile"))
				{
					if (StatAttributes.Num() >= 2)
					{
						CookProfileData.Emplace(StatAttributes[0].Value, StatAttributes[1].Key, StatAttributes[1].Value);
					}
					bStatUsedCustomFormatting = true;
				}

				// if a stat doesn't use custom formatting, just spit out the raw info.
				if (!bStatUsedCustomFormatting)
				{
					TArray<FCookStatsManager::StringKeyValue>& StatsInCategory = StatsInCategories.FindOrAdd(StatName);
					if (StatsInCategory.Num() == 0)
					{
						StatCategories.Add(StatName);
					}
					StatsInCategory.Append(StatAttributes);
				}
			};

		FDerivedDataCacheSummaryStats DDCSummaryStats;
		GetDerivedDataCacheRef().GatherSummaryStats(DDCSummaryStats);

		TArray<FDerivedDataCacheResourceStat> DDCResourceUsageStats;
		GetDerivedDataCacheRef().GatherResourceStats(DDCResourceUsageStats);

		FCookStatsManager::LogCookStats(LogStatsFunc);

		TMap<FString, FDerivedDataCacheResourceStat> DDCStatsPerAssetType;
		CookProfiling::CollectAccumulatedDDCStats(DDCResourceUsageStats, DDCStatsPerAssetType);

		FString CookStatsFileName;
		FString CookStatsJsonString;
		TSharedPtr<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> CookStatsWriter{};
		if (FParse::Value(FCommandLine::Get(), TEXT("-CookStatsFile="), CookStatsFileName))
		{
			uint32 MultiprocessId = UE::GetMultiprocessId();
			if (MultiprocessId == 0)
			{
				CookStatsWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&CookStatsJsonString).ToSharedPtr();
				CookStatsWriter->WriteObjectStart();
			}
			else
			{
				// Suppress the file creation on CookWorkers
				// TODO: Replicate the information back to the CookDirector instead, UE-185774
			}
		}

		UE_LOGF(LogCookStats, Display, "Misc Cook Stats");
		UE_LOGF(LogCookStats, Display, "===============");
		for (FString& StatCategory : StatCategories)
		{
			UE_LOGF(LogCookStats, Display, "%ls", *StatCategory);
			TArray<FCookStatsManager::StringKeyValue>& StatsInCategory = StatsInCategories.FindOrAdd(StatCategory);

			// log each key/value pair, with the equal signs lined up.
			for (const FCookStatsManager::StringKeyValue& StatKeyValue : StatsInCategory)
			{
				UE_LOGF(LogCookStats, Display, "    %ls=%ls", *StatKeyValue.Key, *StatKeyValue.Value);
			}

			if (CookStatsWriter)
			{
				CookStatsWriter->WriteObjectStart(StatCategory);
				for (const FCookStatsManager::StringKeyValue& StatKeyValue : StatsInCategory)
				{
					CookStatsWriter->WriteValue(*StatKeyValue.Key, *StatKeyValue.Value);
				}
				CookStatsWriter->WriteObjectEnd();
			}
		}

		// DDC Usage stats are custom formatted, and the above code just accumulated them into a TSet. Now log it with our special formatting for readability.
		if (CookProfileData.Num() > 0)
		{
			UE_LOGF(LogCookStats, Display, "");
			UE_LOGF(LogCookStats, Display, "Cook Profile");
			UE_LOGF(LogCookStats, Display, "============");
			for (const auto& ProfileEntry : CookProfileData)
			{
				UE_LOGF(LogCookStats, Display, "%ls.%ls=%ls", *ProfileEntry.Path, *ProfileEntry.Key, *ProfileEntry.Value);
			}

			if (CookStatsWriter)
			{
				CookStatsWriter->WriteObjectStart("CookProfile");
				for (const auto& ProfileEntry : CookProfileData)
				{
					CookStatsWriter->WriteObjectStart(ProfileEntry.Key);
					CookStatsWriter->WriteValue(TEXT("Path"), ProfileEntry.Path);
					CookStatsWriter->WriteValue(TEXT("Value"), ProfileEntry.Value);
					CookStatsWriter->WriteObjectEnd();
				}
				CookStatsWriter->WriteObjectEnd();
			}
		}
		if (DDCSummaryStats.Stats.Num() > 0)
		{
			UE_LOGF(LogCookStats, Display, "");
			UE_LOGF(LogCookStats, Display, "DDC Summary Stats");
			UE_LOGF(LogCookStats, Display, "=================");
			for (const auto& Attr : DDCSummaryStats.Stats)
			{
				UE_LOGF(LogCookStats, Display, "%-16ls=%10ls", *Attr.Key, *Attr.Value);
			}

			if (CookStatsWriter)
			{
				CookStatsWriter->WriteObjectStart("DDCSummaryStats");
				for (const auto& Attr : DDCSummaryStats.Stats)
				{
					CookStatsWriter->WriteValue(*Attr.Key, *Attr.Value);
				}
				CookStatsWriter->WriteObjectEnd();
			}
		}

		DumpDerivedDataBuildRemoteExecutorStats();

		if (!DDCResourceUsageStats.IsEmpty())
		{
			Algo::SortBy(DDCResourceUsageStats, [](const FDerivedDataCacheResourceStat& Stat) { return Stat.BuildTimeSec + Stat.LoadTimeSec; }, TGreater());

			UE_LOGF(LogCookStats, Display, "");
			UE_LOGF(LogCookStats, Display, "DDC Resource Stats");
			UE_LOGF(LogCookStats, Display, "=======================================================================================================");
			UE_LOGF(LogCookStats, Display, "Asset Type                          Total Time (Sec)  GameThread Time (Sec)  Assets Built  MB Processed");
			UE_LOGF(LogCookStats, Display, "----------------------------------  ----------------  ---------------------  ------------  ------------");
			for (const FDerivedDataCacheResourceStat& Stat : DDCResourceUsageStats)
			{
				UE_LOGF(LogCookStats, Display, "%-34ls  %16.2f  %21.2f  %12lld  %12.2f",
					*Stat.AssetType, Stat.LoadTimeSec + Stat.BuildTimeSec, Stat.GameThreadTimeSec,
					Stat.BuildCount, Stat.LoadSizeMB + Stat.BuildSizeMB);
			}

			if (CookStatsWriter)
			{
				CookStatsWriter->WriteObjectStart("DDCResourceStats");
				for (const FDerivedDataCacheResourceStat& Stat : DDCResourceUsageStats)
				{
					CookStatsWriter->WriteObjectStart(*Stat.AssetType);
					CookStatsWriter->WriteValue(TEXT("TotalTimeSec"), Stat.LoadTimeSec + Stat.BuildTimeSec);
					CookStatsWriter->WriteValue(TEXT("GameThreadTimeSec"), Stat.GameThreadTimeSec);
					CookStatsWriter->WriteValue(TEXT("AssetsBuilt"), Stat.BuildCount);
					CookStatsWriter->WriteValue(TEXT("MBProcessed"), Stat.LoadSizeMB + Stat.BuildSizeMB);
					CookStatsWriter->WriteObjectEnd();
				}
				CookStatsWriter->WriteObjectEnd();
			}
		}

		if (!DDCStatsPerAssetType.IsEmpty())
		{
			UE_LOGF(LogCookStats, Display, "");
			UE_LOGF(LogCookStats, Display, "Accumulated DDC Resource Stats Per Asset Type");
			UE_LOGF(LogCookStats, Display, "=======================================================================================================");

			for (const TPair< FString, FDerivedDataCacheResourceStat>& Stat : DDCStatsPerAssetType)
			{
				UE_LOGF(LogCookStats, Display, "%-34ls  %16.2f  %21.2f  %12lld  %12.2f",
					*Stat.Key, Stat.Value.LoadTimeSec + Stat.Value.BuildTimeSec, Stat.Value.GameThreadTimeSec,
					Stat.Value.BuildCount, Stat.Value.LoadSizeMB + Stat.Value.BuildSizeMB);
			}

			if (CookStatsWriter)
			{
				CookStatsWriter->WriteObjectStart("DDCAccumulatedStats");
				for (const TPair< FString, FDerivedDataCacheResourceStat>& Stat : DDCStatsPerAssetType)
				{
					CookStatsWriter->WriteObjectStart(*Stat.Key);
					CookStatsWriter->WriteValue(TEXT("LoadTimeSec"), Stat.Value.LoadTimeSec);
					CookStatsWriter->WriteValue(TEXT("LoadSizeMB"), Stat.Value.LoadSizeMB);
					CookStatsWriter->WriteValue(TEXT("LoadCount"), Stat.Value.LoadCount);
					CookStatsWriter->WriteValue(TEXT("BuildTimeSec"), Stat.Value.BuildTimeSec);
					CookStatsWriter->WriteValue(TEXT("BuildSizeMB"), Stat.Value.BuildSizeMB);
					CookStatsWriter->WriteValue(TEXT("BuildCount"), Stat.Value.BuildCount);
					CookStatsWriter->WriteValue(TEXT("TotalCount"), Stat.Value.TotalCount);
					CookStatsWriter->WriteValue(TEXT("Efficiency"), Stat.Value.Efficiency);
					CookStatsWriter->WriteValue(TEXT("GameThreadTimeSec"), Stat.Value.GameThreadTimeSec);
					CookStatsWriter->WriteValue(TEXT("TotalTimeSec"), Stat.Value.LoadTimeSec + Stat.Value.BuildTimeSec);
					CookStatsWriter->WriteObjectEnd();
				}
				CookStatsWriter->WriteObjectEnd();
			}
		}

		DumpBuildDependencyTrackerStats();

		if (UE::Virtualization::IVirtualizationSystem::IsInitialized())
		{
			UE::Virtualization::IVirtualizationSystem::Get().DumpStats();
		}

		if (CookStatsWriter)
		{
			FShaderCompilerStats ShaderCompilerStats;
			GShaderCompilingManager->GetLocalStats(ShaderCompilerStats);

			TSharedPtr<FJsonObject> JsonShaderCompilerStats = ShaderCompilerStats.ToJson();
			FJsonSerializer::Serialize(
				MakeShared<FJsonValueObject>(JsonShaderCompilerStats),
				TEXT("ShaderCompilerStats"),
				CookStatsWriter.ToSharedRef(),
				false
			);
		}

		if (!CookStatsFileName.IsEmpty() && CookStatsWriter)
		{
			CookStatsWriter->WriteObjectEnd();
			CookStatsWriter->Close();
			TUniquePtr<FArchive> CookStatsJsonFile(IFileManager::Get().CreateFileWriter(*CookStatsFileName));
			if (!CookStatsJsonFile)
			{
				UE_LOGF(LogCookStats, Warning, "Could not write to CookStatsFile %ls.", *CookStatsFileName);
			}
			else
			{
				CookStatsJsonFile->Serialize(TCHAR_TO_ANSI(*CookStatsJsonString), CookStatsJsonString.Len());
				CookStatsJsonFile->Close();
			}
		}
	}
};

void OutputIncrementalCookStats()
{
	using namespace DetailedCookStats;

	uint32 NumPackagesIncrementallyModifiedByClassSum = 0;
	TArray<TPair<FTopLevelAssetPath, uint32>> ClassAndNums = NumPackagesIncrementallyModifiedByClass.Array();
	ClassAndNums.RemoveAllSwap(
		[&NumPackagesIncrementallyModifiedByClassSum](const TPair<FTopLevelAssetPath, uint32>& Pair)
		{
			NumPackagesIncrementallyModifiedByClassSum += Pair.Value;
			return Pair.Value == 0;
		});
	if (NumPackagesIncrementallyModifiedByClassSum > 0)
	{
		Algo::Sort(ClassAndNums,
			[](const TPair<FTopLevelAssetPath, uint32>& A, const TPair<FTopLevelAssetPath, uint32>& B)
			{
				if (A.Value != B.Value)
				{
					return A.Value > B.Value;
				}
				return A.Key.Compare(B.Key) < 0;
			});

		UE_LOGF(LogCookStats, Display, "Incrementally Modified Counts for Each Class:");
		UE_LOGF(LogCookStats, Display, "\tTotal, %u", NumPackagesIncrementallyModifiedByClassSum);
		for (const TPair<FTopLevelAssetPath, uint32>& Pair : ClassAndNums)
		{
			UE_LOGF(LogCookStats, Display, "\t%ls, %u",
				Pair.Key.IsValid() ? *Pair.Key.ToString() : TEXT("<UnknownClass>"), Pair.Value);
		}
	}
}
#endif

namespace UE::Cook
{

/** The various ways objects can be referenced that keeps them in memory. */
enum class EObjectReferencerType : uint8
{
	Unknown = 0,
	Rooted,
	GCObjectRef,
	Referenced,
};

struct FObjectGraphProfileData;

/**
 * Data for how an object is referenced in the DumpObjClassList graph search,
 * including the type of reference and the vertex of the referencer.
 */
struct FObjectReferencer
{
	FObjectReferencer() = default;
	explicit FObjectReferencer(EObjectReferencerType InLinkType, Algo::Graph::FVertex InVertexArgument = Algo::Graph::InvalidVertex)
	{
		Set(InLinkType, InVertexArgument);
	}

	Algo::Graph::FVertex GetVertexArgument() const
	{
		return VertexArgument;
	}
	EObjectReferencerType GetLinkType()
	{
		return LinkType;
	}
	void Set(EObjectReferencerType InLinkType, Algo::Graph::FVertex InVertexArgument = Algo::Graph::InvalidVertex)
	{
		switch (InLinkType)
		{
		case EObjectReferencerType::GCObjectRef:
			check(InVertexArgument != Algo::Graph::InvalidVertex);
			break;
		case EObjectReferencerType::Referenced:
			check(InVertexArgument != Algo::Graph::InvalidVertex);
			break;
		default:
			break;
		}

		VertexArgument = InVertexArgument;
		LinkType = InLinkType;
	}
	void ToString(FStringBuilderBase& Builder, FObjectGraphProfileData& ProfileData);

private:
	Algo::Graph::FVertex VertexArgument = Algo::Graph::InvalidVertex;
	EObjectReferencerType LinkType = EObjectReferencerType::Unknown;
};

struct FObjectGraphProfileData
{
	/** The list of UObjects found from a TObectIterator */
	TArray<UObject*> AllObjects;
	/** We assign FVertex N <-> AllObjects[N]; this field records the reverse map. */
	TMap<UObject*, Algo::Graph::FVertex> VertexOfObject;
	/** Element N records whether AllObjects[N] is not one of InitialObjects */
	TBitArray<> IsNew;
	/** The first reason found that AllObjects[n] is still referenced. */
	TArray<FObjectReferencer> AliveReason;
	/** The first rooted vertex found that has a reference chain to AllObjects[n]. */
	TArray<Algo::Graph::FVertex> RootOfVertex;
	/** The referencenames reported by FGCObject::GGCObjectReferencer for why it refers to objects. */
	TArray<FString> AllGCObjectNames;
	/** We assign (FVertex NumObjects+N) <-> AllGCObjectNames[N]; this field records the reverse map. */
	TMap<FString, Algo::Graph::FVertex> GCObjectNameToVertex;
	/** Buffer of edges used for ObjectGraph */
	TArray64<Algo::Graph::FVertex> ObjectGraphBuffer;
	/** ObjectGraph constructed from the edges between vertices defined by serialization references between objects. */
	TArray<TConstArrayView<Algo::Graph::FVertex>> ObjectGraph;
	/** Total number of vertices, both Objects and GCObjectNames */
	int32 NumVertices;
	/** Number of object vertices. The first GCObjectName vertex starts after this number. */
	int32 NumObjectVertices;
	/** Number of GCObjectName vertices. */
	int32 NumGCObjectNameVertices;
	/** The vertex that is assigned to FGCObject::GGCObjectReferencer. */
	Algo::Graph::FVertex GCObjectReferencerVertex;

	void AppendVertexName(Algo::Graph::FVertex Vertex, FStringBuilderBase& Builder)
	{
		if (Vertex < 0)
		{
			Builder << TEXT("InvalidVertex");
		}
		else if (Vertex < NumObjectVertices)
		{
			AllObjects[Vertex]->GetPathName(nullptr, Builder);
		}
		else if (Vertex - NumObjectVertices < NumGCObjectNameVertices)
		{
			Builder << TEXT("FGCObject ") << AllGCObjectNames[Vertex - NumObjectVertices];
		}
		else
		{
			Builder << TEXT("InvalidVertex");
		}
	}
};

void FObjectReferencer::ToString(FStringBuilderBase& Builder, FObjectGraphProfileData& ProfileData)
{
	switch (GetLinkType())
	{
	case EObjectReferencerType::Unknown:
		Builder << TEXT("<Unknown>");
		break;
	case EObjectReferencerType::Rooted:
		Builder << TEXT("<Rooted>");
		break;
	case EObjectReferencerType::GCObjectRef:
	{
		check(VertexArgument != Algo::Graph::InvalidVertex);
		check(ProfileData.NumObjectVertices <= VertexArgument && VertexArgument < ProfileData.NumObjectVertices + ProfileData.NumGCObjectNameVertices);
		ProfileData.AppendVertexName(VertexArgument, Builder);
		break;
	}
	case EObjectReferencerType::Referenced:
	{
		check(VertexArgument != Algo::Graph::InvalidVertex);
		check(VertexArgument < ProfileData.NumObjectVertices);
		ProfileData.AppendVertexName(VertexArgument, Builder);
		break;
	}
	default:
		checkNoEntry();
		break;
	}
}
/** An ObjectReferenceCollector to pass to Object->Serialize to collect references into an array. */
class FArchiveGetReferences : public FArchiveUObject
{
public:
	FArchiveGetReferences(UObject* Object, TArray<UObject*>& OutReferencedObjects)
		:ReferencedObjects(OutReferencedObjects)
	{
		ArIsObjectReferenceCollector = true;
		ArIgnoreOuterRef = false;
		SetShouldSkipCompilingAssets(false);
		Object->Serialize(*this);
	}

	FArchive& operator<<(UObject*& Object)
	{
		if (Object)
		{
			ReferencedObjects.Add(Object);
		}

		return *this;
	}
private:
	TArray<UObject*>& ReferencedObjects;
};

/**
 *  A ReferenceFinder used only when serializing FGCObject::GGCObjectReferencer.
 * It captures the referencerName from GGCObjectReferencer for each UObject passed to it.
 */
class FGCObjectReferencerFinder : public FReferenceFinder
{
public:

	FGCObjectReferencerFinder(TArray<UObject*>& InObjectArray, TMap<UObject*, FString>& InObjectReferencerNames)
		: FReferenceFinder(InObjectArray)
		, ObjectReferencerNames(InObjectReferencerNames)
	{
	}

	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		// Avoid duplicate entries.
		if (InObject != NULL)
		{
			// Many places that use FReferenceFinder expect the object to not be const.
			UObject* Object = const_cast<UObject*>(InObject);
			// do not add or recursively serialize objects that have already been added
			bool bAlreadyExists;
			ObjectSet.Add(Object, &bAlreadyExists);
			if (!bAlreadyExists)
			{
				check(Object->IsValidLowLevel());
				ObjectArray.Add(Object);
				FString ReferencerName;
				FGCObject::GGCObjectReferencer->GetReferencerName(Object, ReferencerName, true /* bOnlyIfAddingReferenced */);
				if (!ReferencerName.IsEmpty())
				{
					ObjectReferencerNames.Add(Object, MoveTemp(ReferencerName));
				}
			}
		}
	}

private:
	TMap<UObject*, FString>& ObjectReferencerNames;
	FGCObject* CurrentlySerializingObject;
};

/**
 * Given the list of AllObjects from e.g. a TObjectIterator, use serialization and other methods from Garbage Collection
 * to find all the dependencies of each Object.
 * Return the dependencies as a normalized graph in the style of GraphConvert.h, with the vertex of each object defined
 * by AllObjects and ObjectToVertex.
 */
void ConstructObjectGraph(TConstArrayView<UObject*> AllObjects,
	const TMap<UObject*, Algo::Graph::FVertex>& ObjectToVertex, TArray64<Algo::Graph::FVertex>& OutGraphBuffer,
	TArray<TConstArrayView<Algo::Graph::FVertex>>& OutGraph, TMap<UObject*, FString>& OutGCObjectReferencerNames)
{
	using namespace Algo::Graph;

	TArray<TArray<FVertex>> LooseEdges;
	int32 NumVertices = AllObjects.Num();
	LooseEdges.SetNum(NumVertices);
	TArray<UObject*> TargetObjects;
	int32 NumEdges = 0;
	OutGCObjectReferencerNames.Reset();

	for (FVertex SourceVertex = 0; SourceVertex < NumVertices; ++SourceVertex)
	{
		UObject* SourceObject = AllObjects[SourceVertex];
		TargetObjects.Reset();
		{
			if (SourceObject == FGCObject::GGCObjectReferencer)
			{
				FGCObjectReferencerFinder Collector(TargetObjects, OutGCObjectReferencerNames);
				UGCObjectReferencer::AddReferencedObjects(FGCObject::GGCObjectReferencer, Collector);
			}
			else
			{
				FReferenceFinder Collector(TargetObjects);
				FArchiveGetReferences Ar(SourceObject, TargetObjects);
				if (SourceObject->GetClass())
				{
					SourceObject->GetClass()->CallAddReferencedObjects(SourceObject, Collector);
				}
				// TODO: Handle elements in the token stream not covered by serialize, such as GetOuter()
				// In the meantime we handle Outer explicitly.
				UObject* Outer = SourceObject->GetOuter();
				if (Outer)
				{
					TargetObjects.Add(Outer);
				}
			}
		}

		if (TargetObjects.Num())
		{
			Algo::Sort(TargetObjects);
			TargetObjects.SetNum(Algo::Unique(TargetObjects), EAllowShrinking::No);
			TArray<FVertex>& TargetVertices = LooseEdges[SourceVertex];
			TargetVertices.Reserve(TargetObjects.Num());
			for (UObject* TargetObject : TargetObjects)
			{
				const FVertex* TargetVertex = ObjectToVertex.Find(TargetObject);
				if (TargetVertex && *TargetVertex != SourceVertex)
				{
					TargetVertices.Add(*TargetVertex);
				}
			}
			NumEdges += TargetVertices.Num();
		}
	}
	OutGraphBuffer.Empty(NumEdges);
	OutGraph.Empty(NumVertices);
	for (FVertex SourceVertex = 0; SourceVertex < NumVertices; ++SourceVertex)
	{
		TArray<FVertex>& InEdges = LooseEdges[SourceVertex];
		TConstArrayView<FVertex>& OutEdges = OutGraph.Emplace_GetRef();
		OutEdges = TConstArrayView<FVertex>(OutGraphBuffer.GetData() + OutGraphBuffer.Num(), InEdges.Num());
		OutGraphBuffer.Append(InEdges);
	}
}

void ConstructObjectGraphProfileData(TConstArrayView<FWeakObjectPtr> InitialObjects, FObjectGraphProfileData& OutProfileData)
{
	using namespace Algo::Graph;
	// Get the list of Objects
	OutProfileData.AllObjects.Reset();
	for (FThreadSafeObjectIterator Iter; Iter; ++Iter)
	{
		UObject* Object = *Iter;
		if (!Object)
		{
			continue;
		}
		OutProfileData.AllObjects.Add(Object);
	}

	// Convert Objects to Algo::Graph::FVertex to reduce graph search memory
	OutProfileData.NumObjectVertices = OutProfileData.AllObjects.Num();
	OutProfileData.NumVertices = OutProfileData.NumObjectVertices;
	OutProfileData.VertexOfObject.Reset();
	for (FVertex Vertex = 0; Vertex < OutProfileData.NumObjectVertices; ++Vertex)
	{
		OutProfileData.VertexOfObject.Add(OutProfileData.AllObjects[Vertex], Vertex);
	}

	// Store for each vertex whether the vertex is new - not in InitialObjects
	OutProfileData.IsNew.Init(true, OutProfileData.NumObjectVertices);
	for (const FWeakObjectPtr& InitialObjectWeak : InitialObjects)
	{
		UObject* InitialObject = InitialObjectWeak.Get();
		if (InitialObject)
		{
			FVertex* Vertex = OutProfileData.VertexOfObject.Find(InitialObject);
			if (Vertex)
			{
				OutProfileData.IsNew[*Vertex] = false;
			}
		}
	}

	// Serialize objects to get dependencies and use them to create the ObjectGraph
	TMap<UObject*, FString> GCObjectReferencerNames;
	ConstructObjectGraph(OutProfileData.AllObjects, OutProfileData.VertexOfObject,
		OutProfileData.ObjectGraphBuffer, OutProfileData.ObjectGraph,
		GCObjectReferencerNames);

	OutProfileData.GCObjectReferencerVertex = InvalidVertex;
	OutProfileData.AliveReason.SetNum(OutProfileData.NumObjectVertices);
	OutProfileData.RootOfVertex.SetNumUninitialized(OutProfileData.NumObjectVertices);
	for (FVertex& Root : OutProfileData.RootOfVertex)
	{
		Root = InvalidVertex;
	}

	// Mark the objects that are rooted by IsRooted, and find the special GCObjectReferencerVertex
	for (FVertex Vertex = 0; Vertex < OutProfileData.NumObjectVertices; ++Vertex)
	{
		UObject* Object = OutProfileData.AllObjects[Vertex];
		if (Object->IsRooted())
		{
			OutProfileData.AliveReason[Vertex].Set(EObjectReferencerType::Rooted);
			OutProfileData.RootOfVertex[Vertex] = Vertex;
		}
		if (Object == FGCObject::GGCObjectReferencer)
		{
			OutProfileData.GCObjectReferencerVertex = Vertex;
		}
	}
	check(OutProfileData.GCObjectReferencerVertex != InvalidVertex);

	// Mark the objects that are rooted by GCObjectReferencerVertex, and construct a synthetic vertex
	// for each of the referencer names reported by GCObjectReferencerVertex.
	OutProfileData.GCObjectNameToVertex.Reset();
	OutProfileData.AllGCObjectNames.Reset();
	FString UnknownReferencer(TEXT("<Unknown>"));
	for (FVertex Vertex : OutProfileData.ObjectGraph[OutProfileData.GCObjectReferencerVertex])
	{
		if (OutProfileData.AliveReason[Vertex].GetLinkType() == EObjectReferencerType::Unknown)
		{
			UObject* Object = OutProfileData.AllObjects[Vertex];
			FString* ReferencerName = &UnknownReferencer;
			if (Object)
			{
				ReferencerName = GCObjectReferencerNames.Find(Object);
				if (!ReferencerName)
				{
					ReferencerName = &UnknownReferencer;
				}
			}
			FVertex& ReferencerVertex = OutProfileData.GCObjectNameToVertex.FindOrAdd(*ReferencerName);
			if (ReferencerVertex == (FVertex)0) // Having value 0 means it was newly added by FindOrAdd
			{
				ReferencerVertex = OutProfileData.NumVertices++;
				OutProfileData.AllGCObjectNames.Add(*ReferencerName);
				check(OutProfileData.NumVertices == OutProfileData.AllObjects.Num() + OutProfileData.AllGCObjectNames.Num());
			}
			OutProfileData.AliveReason[Vertex].Set(EObjectReferencerType::GCObjectRef, ReferencerVertex);
			OutProfileData.RootOfVertex[Vertex] = ReferencerVertex;
		}
	}
	OutProfileData.NumObjectVertices = OutProfileData.AllObjects.Num();
	OutProfileData.NumGCObjectNameVertices = OutProfileData.AllGCObjectNames.Num();

	// Do a DFS to mark the referencer and root of all non-rooted objects
	TArray<FVertex> Stack;
	for (FVertex PotentialRoot = 0; PotentialRoot < OutProfileData.NumObjectVertices; ++PotentialRoot)
	{
		if (PotentialRoot == OutProfileData.GCObjectReferencerVertex ||
			(OutProfileData.AliveReason[PotentialRoot].GetLinkType() != EObjectReferencerType::Rooted &&
				OutProfileData.AliveReason[PotentialRoot].GetLinkType() != EObjectReferencerType::GCObjectRef))
		{
			continue;
		}
		FVertex RootVertex = OutProfileData.RootOfVertex[PotentialRoot];

		Stack.Reset();
		Stack.Add(PotentialRoot);
		while (!Stack.IsEmpty())
		{
			FVertex SourceVertex = Stack.Pop(EAllowShrinking::No);
			for (FVertex TargetVertex : OutProfileData.ObjectGraph[SourceVertex])
			{
				if (OutProfileData.AliveReason[TargetVertex].GetLinkType() == EObjectReferencerType::Unknown)
				{
					OutProfileData.AliveReason[TargetVertex].Set(EObjectReferencerType::Referenced, SourceVertex);
					OutProfileData.RootOfVertex[TargetVertex] = RootVertex;
					Stack.Add(TargetVertex);
				}
			}
		}
	}
}

void DumpObjClassList(TConstArrayView<FWeakObjectPtr> InitialObjects)
{
	using namespace Algo::Graph;

	FOutputDevice& LogAr = *(GLog);
	FObjectGraphProfileData ProfileData;
	ConstructObjectGraphProfileData(InitialObjects, ProfileData);

	// Count how many new objects of each class there are, and store all root objects that keep them in memory
	struct FClassInfo
	{
		TMap<FVertex, int32> Roots;
		int32 Count = 0;
		UClass* Class = nullptr;
	};
	TMap<UClass*, FClassInfo> ClassInfos;
	for (FVertex Vertex = 0; Vertex < ProfileData.NumObjectVertices; ++Vertex)
	{
		// Ignore non-new objects
		if (!ProfileData.IsNew[Vertex] || Vertex == ProfileData.GCObjectReferencerVertex)
		{
			continue;
		}
		FObjectReferencer Link = ProfileData.AliveReason[Vertex];
		EObjectReferencerType LinkType = Link.GetLinkType();
		// Ignore objects that have AliveReason unknown. This can occur if the objects were rooted during garbage
		// collection but then asynchronous work RemovedThemFromRoot in between GC finishing and our call to IsRooted.
		if (LinkType == EObjectReferencerType::Unknown)
		{
			continue;
		}
		UClass* Class = ProfileData.AllObjects[Vertex]->GetClass();
		if (!Class || !Class->IsNative())
		{
			continue;
		}
		FClassInfo& ClassInfo = ClassInfos.FindOrAdd(Class);
		ClassInfo.Class = Class;
		ClassInfo.Roots.FindOrAdd(ProfileData.RootOfVertex[Vertex], 0)++;
		ClassInfo.Count++;
	}

	TArray<FClassInfo> ClassInfoArray;
	ClassInfoArray.Reserve(ClassInfos.Num());
	for (TPair<UClass*, FClassInfo>& Pair : ClassInfos)
	{
		ClassInfoArray.Add(MoveTemp(Pair.Value));
	}
	Algo::Sort(ClassInfoArray, [](const FClassInfo& A, const FClassInfo& B)
		{
			return FTopLevelAssetPath(A.Class).Compare(FTopLevelAssetPath(B.Class)) < 0;
		});


	LogAr.Logf(TEXT("Memory Analysis: New Objects of each class and the top roots keeping them alive:"));
	LogAr.Logf(TEXT("\t%6s %s"), TEXT("Count"), TEXT("ClassPath"));
	LogAr.Logf(TEXT("\t\t%6s %s"), TEXT("Count"), TEXT("RootObjectAndChain"));
	TStringBuilder<1024> RootObjectString;
	constexpr int32 MaxRootCount = 2;
	TArray<TPair<FVertex, int32>, TInlineAllocator<MaxRootCount + 1>> MaxRoots;
	for (FClassInfo& ClassInfo : ClassInfoArray)
	{
		MaxRoots.Reset();
		for (TPair<FVertex, int32>& RootPair : ClassInfo.Roots)
		{
			for (int32 IndexFromMax = 0; IndexFromMax < MaxRootCount; ++IndexFromMax)
			{
				if (MaxRoots.Num() <= IndexFromMax || MaxRoots[IndexFromMax].Value < RootPair.Value)
				{
					MaxRoots.Insert(RootPair, IndexFromMax);
					break;
				}
			}
			if (MaxRoots.Num() > MaxRootCount)
			{
				MaxRoots.Pop(EAllowShrinking::No);
			}
		}
		LogAr.Logf(TEXT("\t%6d %s"), ClassInfo.Count, *ClassInfo.Class->GetPathName());
		for (TPair<FVertex, int32>& RootPair : MaxRoots)
		{
			RootObjectString.Reset();
			RootObjectString.Appendf(TEXT("\t\t%6d: "), RootPair.Value);
			ProfileData.AppendVertexName(RootPair.Key, RootObjectString);
			if (RootPair.Key < ProfileData.NumObjectVertices)
			{
				FObjectReferencer Link = ProfileData.AliveReason[RootPair.Key];
				while (Link.GetLinkType() == EObjectReferencerType::Referenced)
				{
					RootObjectString << TEXT(" <- ");
					Link.ToString(RootObjectString, ProfileData);
					Link = ProfileData.AliveReason[Link.GetVertexArgument()];
				}
				RootObjectString << TEXT(" <- ");
				Link.ToString(RootObjectString, ProfileData);
			}
			LogAr.Logf(TEXT("%s"), *RootObjectString);
		}
	}
}

void DumpPackageReferencers(const TCHAR* DebugNameOfPackageList,
	TConstArrayView<FWeakObjectPtr> InitialObjects, TConstArrayView<UPackage*> Packages)
{
	using namespace Algo::Graph;

	FOutputDevice& LogAr = *(GLog);
	FObjectGraphProfileData ProfileData;
	ConstructObjectGraphProfileData(TConstArrayView<FWeakObjectPtr>(), ProfileData);

	TArray<UPackage*> AllPackagesInMemory;
	if (Packages.IsEmpty())
	{
		// Ignore packages containing any of the InitialObjects
		TSet<UPackage*> InitialPackages;
		for (const FWeakObjectPtr& WeakObject : InitialObjects)
		{
			UObject* Object = WeakObject.Get();
			if (Object)
			{
				InitialPackages.Add(Object->GetPackage());
			}
		}
		for (TObjectIterator<UPackage> Iter; Iter; ++Iter)
		{
			UPackage* Package = *Iter;
			if (FPackageName::IsScriptPackage(WriteToString<256>(Package->GetFName())))
			{
				continue;
			}
			if (InitialPackages.Contains(Package))
			{
				continue;
			}
			AllPackagesInMemory.Add(Package);
		}
		Packages = AllPackagesInMemory;
	}

	// Store the root keeping each package in memory and calculate the memory size used by each package.
	struct FPackageDumpData
	{
		UPackage* Package = nullptr;
		FVertex Vertex = InvalidVertex;
		FVertex RootVertex = InvalidVertex;
		uint64 Size = 0;
	};
	struct FRootDumpData
	{
		FVertex Vertex = InvalidVertex;
		TArray<FPackageDumpData*> Packages;
		uint64 Size = 0;
	};
	uint64 TotalPackageSize = 0;
	TMap<UPackage*, TUniquePtr<FPackageDumpData>> PackageDatas;
	TMap<FVertex, FRootDumpData> RootDatas;
	for (UPackage* Package : Packages)
	{
		TUniquePtr<FPackageDumpData>& PackageData = PackageDatas.FindOrAdd(Package);
		if (!PackageData)
		{
			PackageData.Reset(new FPackageDumpData);
			PackageData->Package = Package;
		}
		FVertex* FoundVertex = ProfileData.VertexOfObject.Find(Package);
		PackageData->Vertex = FoundVertex ? *FoundVertex : InvalidVertex;
		PackageData->RootVertex = PackageData->Vertex != InvalidVertex
			? ProfileData.RootOfVertex[PackageData->Vertex] : InvalidVertex;

		PackageData->Size += Package->GetClass()->GetStructureSize();
		PackageData->Size += Package->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
		ForEachObjectWithPackage(Package, [&PackageData](UObject* Object)
			{
				PackageData->Size += Object->GetClass()->GetStructureSize();
				PackageData->Size += Object->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
				return true;
			}, EGetObjectsFlags::IncludeNestedObjects);

		FRootDumpData& RootData = RootDatas.FindOrAdd(PackageData->RootVertex);
		if (RootData.Packages.IsEmpty())
		{
			RootData.Vertex = PackageData->RootVertex;
		}
		RootData.Packages.Add(PackageData.Get());
		RootData.Size += PackageData->Size;
		TotalPackageSize += PackageData->Size;
	}

	LogAr.Logf(TEXT("Memory Analysis: Referencers of %s"), DebugNameOfPackageList);
	if (RootDatas.Num() == 0)
	{
		LogAr.Logf(TEXT("No %s found to report"), DebugNameOfPackageList);
		return;
	}
	LogAr.Logf(TEXT("%d Referencers are holding in memory %d packages totalling %d MiB in size."),
		RootDatas.Num(), PackageDatas.Num(), (int32)(TotalPackageSize / 1024 / 1024));

	// Sort the roots by size
	RootDatas.ValueSort([&ProfileData](const FRootDumpData& A, const FRootDumpData& B)
		{
			if (A.Size != B.Size)
			{
				return A.Size > B.Size;
			}
			TStringBuilder<256> AName;
			TStringBuilder<256> BName;
			A.Vertex != InvalidVertex ? ProfileData.AppendVertexName(A.Vertex, AName)
				: (void) AName.Append(TEXT("<Unknown>"));
			B.Vertex != InvalidVertex ? ProfileData.AppendVertexName(B.Vertex, BName)
				: (void) BName.Append(TEXT("<Unknown>"));
			return AName.ToView().Compare(BName) < 0;
		});

	uint64 MaxSize = 0;
	int32 NumSoFar = 0;
	constexpr float MinimumRelativeSizeToDisplay = 0.10f;
	constexpr int PackagesPerRoot = 5;
	for (TPair<FVertex, FRootDumpData>& Pair : RootDatas)
	{
		FRootDumpData& RootData = Pair.Value;
		++NumSoFar;
		if (MaxSize == 0)
		{
			MaxSize = RootData.Size; // Elements are sorted by size, first element in iteration is the largest.
		}
		if (RootData.Size < (uint64)(MinimumRelativeSizeToDisplay * MaxSize))
		{
			LogAr.Logf(TEXT("\t... (%d more referencers each responsible for <= %d MiB in size)"),
				RootDatas.Num() - NumSoFar, (int32)(RootData.Size / 1024 / 1024));
			break;
		}

		TStringBuilder<256> ReferencerName;
		RootData.Vertex != InvalidVertex ? ProfileData.AppendVertexName(RootData.Vertex, ReferencerName)
			: (void)ReferencerName.Append(TEXT("<Unknown>"));
		LogAr.Logf(TEXT("\t%s"), *ReferencerName);
		check(!RootData.Packages.IsEmpty()); // Creation of each rootdata includes adding a package to it.
		Algo::Sort(RootData.Packages, [](FPackageDumpData* A, FPackageDumpData* B)
			{
				if (A->Size != B->Size)
				{
					return A->Size < B->Size;
				}
				return A->Package->GetFName().LexicalLess(B->Package->GetFName());
			});
		FPackageDumpData* ExamplePackage = RootData.Packages[0];
		if (ExamplePackage->Vertex != InvalidVertex)
		{
			TStringBuilder<256> Chain;
			FObjectReferencer Link = ProfileData.AliveReason[ExamplePackage->Vertex];
			ProfileData.AllObjects[ExamplePackage->Vertex]->GetFullName(Chain);
			while (Link.GetLinkType() == EObjectReferencerType::Referenced)
			{
				Chain << TEXT(" <- ");
				Link.ToString(Chain, ProfileData);
				Link = ProfileData.AliveReason[Link.GetVertexArgument()];
			}
			Chain << TEXT(" <- ");
			Link.ToString(Chain, ProfileData);
			LogAr.Logf(TEXT("\t\tExample Chain: %s"), *Chain);
		}
		LogAr.Logf(TEXT("\t\t%d MiB, %d Packages"), (int32)(RootData.Size / 1024 / 1024), RootData.Packages.Num());
		int32 NumPackagesSoFar = 0;
		uint64 SizeSoFar = 0;
		for (FPackageDumpData* PackageData : RootData.Packages)
		{
			++NumPackagesSoFar;
			if (NumPackagesSoFar > PackagesPerRoot)
			{
				LogAr.Logf(TEXT("\t\t\t... (%d more packages totalling %d MiB in size)"),
					RootData.Packages.Num() - NumPackagesSoFar, (int32)((RootData.Size - SizeSoFar) / 1024 / 1024));
				break;
			}
			SizeSoFar += PackageData->Size;
			LogAr.Logf(TEXT("\t\t\t%4d MiB: %s"),
				(int32)(PackageData->Size / 1024 / 1024), *WriteToString<256>(PackageData->Package->GetFName()));
		}
	}
}

} // namespace UE::Cook

#if ENABLE_COOK_STATS
namespace CookProfiling
{
	const char* AssetTypesToAccumulate[] =
	{
		"AnimationSequence",
		"Audio",
		"CardRepresentation",
		"DistanceField",
		"MaterialShader",
		"MaterialTranslation",
		"NiagaraScript",
		"SkeletalMesh",
		"StaticMesh",
		"Texture",
		"Other"
	};

	void CollectAccumulatedDDCStats(
		TArray<FDerivedDataCacheResourceStat> DDCResourceUsageStats,
		TMap<FString, FDerivedDataCacheResourceStat>& StatsPerAssetType)
	{
		for (const char* KnownAssetType : CookProfiling::AssetTypesToAccumulate)
		{
			StatsPerAssetType.Add(KnownAssetType, FDerivedDataCacheResourceStat());
		}

		FDerivedDataCacheResourceStat* OtherAccumulator = StatsPerAssetType.Find("Other");
		for (const FDerivedDataCacheResourceStat& Stat : DDCResourceUsageStats)
		{
			FDerivedDataCacheResourceStat* Accumulator = StatsPerAssetType.Find(Stat.AssetType);

			if (!Accumulator)
			{
				Accumulator = OtherAccumulator;
			}

			*Accumulator += Stat;
		}
	}

	void CollectAttributesForAccumulatedDDCUsageStats(
		TMap<FString, FDerivedDataCacheResourceStat> StatsPerAssetType,
		TArray<FAnalyticsEventAttribute>& Attributes)
	{
		const FString BaseName = TEXT("DDCAccumulatedStats_");
		Attributes.Reserve(StatsPerAssetType.Num() * 10); // pre allocate ten attributes per asset type

		for (auto& StatForAsset : StatsPerAssetType)
		{
			FDerivedDataCacheResourceStat& Stat = StatForAsset.Value;
			FString AttrName = BaseName + StatForAsset.Key;
			Attributes.Emplace(AttrName + TEXT("LoadTimeSec"), Stat.LoadTimeSec);
			Attributes.Emplace(AttrName + TEXT("LoadSizeMB"), Stat.LoadSizeMB);
			Attributes.Emplace(AttrName + TEXT("LoadCount"), Stat.LoadCount);
			Attributes.Emplace(AttrName + TEXT("BuildTimeSec"), Stat.BuildTimeSec);
			Attributes.Emplace(AttrName + TEXT("BuildSizeMB"), Stat.BuildSizeMB);
			Attributes.Emplace(AttrName + TEXT("BuildCount"), Stat.BuildCount);
			Attributes.Emplace(AttrName + TEXT("TotalCount"), Stat.TotalCount);
			Attributes.Emplace(AttrName + TEXT("Efficiency"), Stat.Efficiency);
			Attributes.Emplace(AttrName + TEXT("GameThreadTimeSec"), Stat.GameThreadTimeSec);
			Attributes.Emplace(AttrName + TEXT("TotalTimeSec"), Stat.LoadTimeSec + Stat.BuildTimeSec);
		}
	}

	void GatherAccumulatedDDCStats(
		TArray<FAnalyticsEventAttribute>& Attributes)
	{
		TArray<FDerivedDataCacheResourceStat> DDCResourceUsageStats;
		GetDerivedDataCacheRef().GatherResourceStats(DDCResourceUsageStats);

		TMap<FString, FDerivedDataCacheResourceStat> StatsPerAssetType;
		CollectAccumulatedDDCStats(DDCResourceUsageStats, StatsPerAssetType);

		CollectAttributesForAccumulatedDDCUsageStats(StatsPerAssetType, Attributes);
	}
};

#endif // ENABLE_COOK_STATS