// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreTypes.h"
#include "ProfilingDebugging/CookStats.h"
#include "Stats/Stats.h"

#include <atomic>

#define OUTPUT_COOKTIMING ENABLE_COOK_STATS
#define PROFILE_NETWORK 0

#if ENABLE_COOK_STATS
#include "Containers/Map.h"
#include "UObject/TopLevelAssetPath.h"
#endif

#if OUTPUT_COOKTIMING
#include "Trace/Trace.inl"
#include "ProfilingDebugging/FormatArgsTrace.h"
#include "ProfilingDebugging/ScopedTimers.h"
#endif

struct FWeakObjectPtr;

#if OUTPUT_COOKTIMING

void OutputHierarchyTimers();
void OutputIncrementalCookStats();
void ClearHierarchyTimers();

struct FHierarchicalTimerInfo;

struct FScopeTimer
{
public:
	FScopeTimer(const FScopeTimer&) = delete;
	FScopeTimer(FScopeTimer&&) = delete;

	FScopeTimer(int InId, const char* InName, bool IncrementScope = false );

	void Start();
	void Stop();

	~FScopeTimer();

private:
	uint64					StartTime = 0;
	FHierarchicalTimerInfo* HierarchyTimerInfo;
	FHierarchicalTimerInfo* PrevTimerInfo;
};

#define UE_CREATE_HIERARCHICAL_COOKTIMER(name, incrementScope) \
			FScopeTimer UE_JOIN(__HierarchicalCookTimerScope, __LINE__)(__COUNTER__, #name, incrementScope); 
#define UE_CREATE_TEXT_HIERARCHICAL_COOKTIMER(name, incrementScope) \
			FScopeTimer UE_JOIN(__HierarchicalCookTimerScope, __LINE__)(__COUNTER__, name, incrementScope);

// Emits trace events denoting scope/lifetime of an activity on the cooking channel.
#define UE_SCOPED_COOKTIMER(name)						TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(name, CookChannel)
// More expensive version that takes a dynamic TCHAR*
#define UE_SCOPED_TEXT_COOKTIMER(name)					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(name, CookChannel)

#define UE_SCOPED_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_COOKTIMER(name)
// More expensive version that takes a dynamic TCHAR*
#define UE_SCOPED_TEXT_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_TEXT_COOKTIMER(name)

// Emits trace events denoting scope/lifetime of an activity on the cooking channel.  Also creates a named hierarchical timer that can be aggregated and reported at cook completion.
#define UE_SCOPED_HIERARCHICAL_COOKTIMER(name) \
			UE_CREATE_HIERARCHICAL_COOKTIMER(name, true); \
			UE_JOIN(__HierarchicalCookTimerScope, __LINE__).Start(); \
			UE_SCOPED_COOKTIMER(name)
// More expensive version that takes a dynamic TCHAR*
#define UE_SCOPED_TEXT_HIERARCHICAL_COOKTIMER(name) \
			UE_CREATE_TEXT_HIERARCHICAL_COOKTIMER(name, true); \
			UE_JOIN(__HierarchicalCookTimerScope, __LINE__).Start(); \
			UE_SCOPED_TEXT_COOKTIMER(name)
// Version that takes a separate variable for where the duration should be stored
#define UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_HIERARCHICAL_COOKTIMER(name)
// More expensive version that takes a separate variable for where the duration should be stored and a dynamic TCHAR*
#define UE_SCOPED_TEXT_HIERARCHICAL_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_TEXT_HIERARCHICAL_COOKTIMER(name)

#define UE_CUSTOM_COOKTIMER_LOG Cpu

// Emits trace events denoting scope/lifetime of an activity on the cooking channel.
#define UE_SCOPED_CUSTOM_COOKTIMER(name)				UE_TRACE_LOG_SCOPED_T(UE_CUSTOM_COOKTIMER_LOG, name, CookChannel)
#define UE_SCOPED_CUSTOM_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_CUSTOM_COOKTIMER(name)

// Emits trace events denoting scope/lifetime of an activity on the cooking channel.  Also creates a named hierarchical timer that can be aggregated and reported at cook completion.
#define UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER(name) \
			UE_CREATE_HIERARCHICAL_COOKTIMER(name, true); \
			UE_JOIN(__HierarchicalCookTimerScope, __LINE__).Start(); \
			UE_SCOPED_CUSTOM_COOKTIMER(name)
#define UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER_AND_DURATION(name, durationStorage) \
			FScopedDurationTimer UE_JOIN(__CookTimerScope, __LINE__)(durationStorage); \
			UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER(name)
#define UE_ADD_CUSTOM_COOKTIMER_META(name, key, value) << name.key(value)

#else
#define UE_SCOPED_COOKTIMER(name)
#define UE_SCOPED_TEXT_COOKTIMER(name)
#define UE_SCOPED_COOKTIMER_AND_DURATION(name, durationStorage)
#define UE_SCOPED_TEXT_COOKTIMER_AND_DURATION(name, durationStorage)
#define UE_SCOPED_HIERARCHICAL_COOKTIMER(name)
#define UE_SCOPED_TEXT_HIERARCHICAL_COOKTIMER(name)
#define UE_SCOPED_HIERARCHICAL_COOKTIMER_AND_DURATION(name, durationStorage)
#define UE_SCOPED_TEXT_HIERARCHICAL_COOKTIMER_AND_DURATION(name, durationStorage)

#define UE_CUSTOM_COOKTIMER_LOG Cpu
#define UE_SCOPED_CUSTOM_COOKTIMER(name)
#define UE_SCOPED_CUSTOM_COOKTIMER_AND_DURATION(name, durationStorage)
#define UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER(name)
#define UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER_AND_DURATION(name, durationStorage)
#define UE_ADD_CUSTOM_COOKTIMER_META(name, key, value)

inline void OutputHierarchyTimers() {}
inline void ClearHierarchyTimers() {}
#endif

#if ENABLE_COOK_STATS
namespace DetailedCookStats
{
// Cook.Profile.Params
extern FString CookProject;
extern FString CookCultures;
extern FString CookLabel;
extern FString TargetPlatforms;
extern bool IsCookAll;
extern bool IsCookOnTheFly;
extern bool IsIncrementalCook;
extern bool IsIterativeCook;
extern bool IsFastCook;
extern bool IsUnversioned;

// Cook.Profile
extern double CookStartTime;
extern double CookWallTimeSec;
extern double StartupWallTimeSec;
extern double StartCookByTheBookTimeSec;
extern double TickCookOnTheSideTimeSec;
extern double TickCookOnTheSideLoadPackagesTimeSec;
extern double TickCookOnTheSideResolveRedirectorsTimeSec;
extern double TickCookOnTheSideSaveCookedPackageTimeSec;
extern double TickCookOnTheSidePrepareSaveTimeSec;
extern double BlockOnAssetRegistryTimeSec;
extern double GameCookModificationDelegateTimeSec;
extern double TickLoopGCTimeSec;
extern double TickLoopRecompileShaderRequestsTimeSec;
extern double TickLoopShaderProcessAsyncResultsTimeSec;
extern double TickLoopProcessDeferredCommandsTimeSec;
extern double TickLoopTickCommandletStatsTimeSec;
extern double TickLoopFlushRenderingCommandsTimeSec;
extern double ShaderFlushTimeSec;
extern double ValidationTimeSec;

// CookOnTheFlyServer.Package.Load
extern int32 NumRequestedLoads;
extern std::atomic<int32> NumDetectedLoads;

// CookOnTheFlyServer.Package.Save
extern uint32 NumPackagesIncrementallySkipped;

//"CookOnTheFlyServer.QueueSize"
extern int32 PeakRequestQueueSize;
extern int32 PeakLoadQueueSize;
extern int32 PeakSaveQueueSize;

extern TMap<FTopLevelAssetPath, uint32> NumPackagesIncrementallyModifiedByClass;

void SendLogCookStats(ECookMode::Type CookMode);

}
#endif

#if PROFILE_NETWORK
double TimeTillRequestStarted = 0.0;
double TimeTillRequestForfilled = 0.0;
double TimeTillRequestForfilledError = 0.0;
double WaitForAsyncFilesWrites = 0.0;
FEvent* NetworkRequestEvent = nullptr;
#endif

DECLARE_STATS_GROUP(TEXT("Cooking"), STATGROUP_Cooking, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Precache Derived data for platform"), STAT_TickPrecacheCooking, STATGROUP_Cooking);
DECLARE_CYCLE_STAT(TEXT("Tick cooking"), STAT_TickCooker, STATGROUP_Cooking);

namespace UE::Cook
{

/**
 * Used for profiling memory after a garbage collect. Find how many instances of each class exist, and find the list
 * of referencers that are keeping instances of each class in memory. Dump this information to the log (GLog).
 * 
 * @param SessionStartupObjects: The list of objects that existed at the start of the cook session. Objects in this
 *                               list are removed from the results since we do not expect them to be garbage collected.
 */
void DumpObjClassList(TConstArrayView<FWeakObjectPtr> SessionStartupObjects);

/** Used for profiling after a soft garbage collect. Find the referencers keeping each package in memory. */
void DumpPackageReferencers(const TCHAR* DebugNameOfPackageList, TConstArrayView<FWeakObjectPtr> InitialObjects, 
	TConstArrayView<UPackage*> Packages = TConstArrayView<UPackage*>());


}