// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookRequestCluster.h"

#include "Algo/AllOf.h"
#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Algo/TopologicalSort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDiagnostics.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookRequests.h"
#include "Cooker/CookTypes.h"
#include "Cooker/PackageTracker.h"
#include "CookerSettings.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "Engine/AssetManager.h"
#include "Engine/Level.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Logging/StructuredLog.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/RedirectCollector.h"
#include "Misc/ReverseIterate.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "String/Find.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SavePackage.h"

namespace UE::Cook
{

FRequestCluster::FRequestCluster(UCookOnTheFlyServer& InCOTFS, EReachability ExploreReachability)
	: GraphSearch(*this)
	, COTFS(InCOTFS)
	, PackageDatas(*InCOTFS.PackageDatas)
	, AssetRegistry(*IAssetRegistry::Get())
	, PackageTracker(*InCOTFS.PackageTracker)
	, BuildDefinitions(*InCOTFS.BuildDefinitions)
{
	TConstArrayView<const ITargetPlatform*> SessionPlatforms = COTFS.PlatformManager->GetSessionPlatforms();
	check(SessionPlatforms.Num() > 0);
	NumFetchPlatforms = SessionPlatforms.Num() + 2;
	VertexAllocator.SetMinBlockSize(1024);
	VertexAllocator.SetMaxBlockSize(65536);

	// CookByTheBookOptions is always available; in other modes it is set to the default values
	UE::Cook::FCookByTheBookOptions& Options = *COTFS.CookByTheBookOptions;
	bAllowHardDependencies = !Options.bSkipHardReferences;
	bAllowSoftDependencies = !Options.bSkipSoftReferences;
	bErrorOnEngineContentUse = Options.bErrorOnEngineContentUse;
	if (COTFS.IsCookOnTheFlyMode())
	{
		// Do not queue soft-dependencies during CookOnTheFly; wait for them to be requested
		// TODO: Report soft dependencies separately, and mark them as normal priority,
		// and mark all hard dependencies as high priority in cook on the fly.
		bAllowSoftDependencies = false;
	}

	if (COTFS.IsCookWorkerMode())
	{
		if (ExploreReachability == EReachability::Build)
		{
			TraversalTier = ETraversalTier::MarkForBuildDependency;
		}
		else
		{
			check(ExploreReachability == EReachability::Runtime);
			TraversalTier = ETraversalTier::MarkForRuntime;
		}
	}
	else if (ExploreReachability == EReachability::Build)
	{
		TraversalTier = ETraversalTier::BuildDependencies;
	}
	else
	{
		check(ExploreReachability == EReachability::Runtime);
		TraversalTier = bAllowHardDependencies ? ETraversalTier::RuntimeFollowDependencies :
			ETraversalTier::RuntimeVisitVertices;
	}

	if (bErrorOnEngineContentUse)
	{
		DLCPath = FPaths::Combine(*COTFS.GetBaseDirectoryForDLC(), TEXT("Content"));
		FPaths::MakeStandardFilename(DLCPath);
	}
	if (TraversalMarkCookable())
	{
		GConfig->GetBool(TEXT("CookSettings"), TEXT("PreQueueBuildDefinitions"), 
			bPreQueueBuildDefinitions, GEditorIni);
	}
	else
	{
		bPreQueueBuildDefinitions = false;
	}

	bAllowIncrementalResults = true;
	bool bFirst = true;
	for (const ITargetPlatform* TargetPlatform : COTFS.PlatformManager->GetSessionPlatforms())
	{
		FPlatformData* PlatformData = COTFS.PlatformManager->GetPlatformData(TargetPlatform);
		if (bFirst)
		{
			bAllowIncrementalResults = PlatformData->bAllowIncrementalResults;
			bFirst = false;
		}
		else
		{
			if (PlatformData->bAllowIncrementalResults != bAllowIncrementalResults)
			{
				UE_LOGF(LogCook, Warning,
					"Full build is requested for some platforms but not others, but this is not supported. All platforms will be built full.");
				bAllowIncrementalResults = false;
			}
		}
	}
}

FRequestCluster::~FRequestCluster()
{
	EmptyClusterPackages();
}

void FRequestCluster::EmptyClusterPackages()
{
	// Call the FVertexData destructors, but do not bother calling DeleteElement or Free on the VertexAllocator
	// since we are destructing the VertexAllocator.
	for (TPair<FName, FVertexData*>& VertexPair : ClusterPackages)
	{
		check(VertexPair.Value);
		FVertexData& Vertex = *VertexPair.Value;
		Vertex.~FVertexData();
	}
	ClusterPackages.Empty();
	// Empty frees the struct memory for each FVertexData we allocated, but it does not call the destructor.
	VertexAllocator.Empty();
}

FRequestCluster::FRequestCluster(UCookOnTheFlyServer& InCOTFS, TArray<FFilePlatformRequest>&& InRequests)
	: FRequestCluster(InCOTFS, EReachability::Runtime)
{
	ReserveInitialRequests(InRequests.Num());
	FilePlatformRequests = MoveTemp(InRequests);
	InRequests.Empty();
}

FRequestCluster::FRequestCluster(UCookOnTheFlyServer& InCOTFS, TPackageDataMap<ESuppressCookReason>&& InRequests,
	EReachability InExploreReachability)
	: FRequestCluster(InCOTFS, InExploreReachability)
{
	ReserveInitialRequests(InRequests.Num());
	for (TPair<FPackageData*, ESuppressCookReason>& Pair : InRequests)
	{
		FPackageData* PackageData = Pair.Key;
		ESuppressCookReason SuppressCookReason = Pair.Value;
		check(PackageData);
		FVertexData& Vertex = FindOrAddVertex(*PackageData);
		// Setting bNeedsStateChange=false is important to avoid a crash: changing the state will try to remove it
		// from InRequests while we are iterating over it trigger an assertion/crash. Calling InRequests::Empty below will
		// accomplish what bNeedsStateChange=true would have tried to do more slowly.
		check(PackageData->GetState() == EPackageState::Request);
		SetOwnedByCluster(Vertex, true /* bOwnedByCluster */, false /* bNeedsStateChange */);

		// Some restartedrequests need reachability changes, and reachability changes can only be made by a
		// RequestCluster, so do them here.
		switch (SuppressCookReason)
		{
		case ESuppressCookReason::GeneratedPackageNeedsRequestUpdate:
		{
			// TODO_COOKGENERATIONHELPER: We don't currently support separate cooking for one platform but not
			// another for a generated package, see the class comment on FGenerationHelper. Therefore if any platform
			// was found to be reachable, then mark the other platforms as reachable.
			TConstArrayView<const ITargetPlatform*> SessionPlatforms = InCOTFS.PlatformManager->GetSessionPlatforms();
			if (PackageData->HasReachablePlatforms(InExploreReachability, SessionPlatforms))
			{
				PackageData->AddReachablePlatforms(*this, InExploreReachability, SessionPlatforms,
					FInstigator(EInstigator::GeneratedPackage, PackageData->GetPackageName()));
			}
			break;
		}
		default:
			break;
		}
	}
	InRequests.Empty();
}

FRequestCluster::FRequestCluster(UCookOnTheFlyServer& InCOTFS, TRingBuffer<FDiscoveryQueueElement>& DiscoveryQueue)
	: FRequestCluster(InCOTFS, EReachability::Runtime)
{
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> BufferPlatforms;
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> ImmediateAddPlatforms;
	if (!COTFS.bSkipOnlyEditorOnly)
	{
		BufferPlatforms = COTFS.PlatformManager->GetSessionPlatforms();
		BufferPlatforms.Add(CookerLoadingPlatformKey);
	}

	while (!DiscoveryQueue.IsEmpty())
	{
		FDiscoveryQueueElement* Discovery = &DiscoveryQueue.First();
		ON_SCOPE_EXIT
		{
			DiscoveryQueue.PopFrontNoCheck();
		};
		FPackageData& PackageData = *Discovery->PackageData;

		TConstArrayView<const ITargetPlatform*> NewReachablePlatforms;
		if (COTFS.bSkipOnlyEditorOnly)
		{
			NewReachablePlatforms = Discovery->ReachablePlatforms.GetPlatforms(COTFS, &Discovery->Instigator,
				TConstArrayView<const ITargetPlatform*>(), EReachability::Runtime, BufferPlatforms);
		}
		else
		{
			NewReachablePlatforms = BufferPlatforms;
		}

		FPackageData* Referencer = COTFS.PackageDatas->FindPackageDataByPackageName(Discovery->Instigator.Referencer);
		if (Referencer)
		{
			// The discovery may have come from a cookworker; add it again in case it was not already added.
			Referencer->AddDiscoveredDependency(Discovery->ReachablePlatforms, &PackageData, Discovery->Instigator.Category);
		}

		// Create a list of immediate add platforms: platforms for which the referencer is already reachable so we know
		// we can add the Discovery to the cook now. For Platforms on which the 
		// If its referencer is not reachable for a given platform, then take no further action for that
		// platform now; if the referencer becomes reachable later, we will then add the target to the cook
		// when we visit the referencer and traverse the DiscoveredDependency edge we just added.
		if (!Referencer ||
			Discovery->Urgency > PackageData.GetUrgency())
		{
			// In the no-referencer case, add all discovered reachability platforms immediately.
			// And for urgent requests, also add them immediately. Urgency greater than normal can only be set from
			// referencers that were already reachable; we currently rely on this so that we don't have to make
			// another message type for them and can process them here in the DiscoveryQueue. If the discovery
			// carries a raise in urgency, then add it to the cook even if the referencer is not yet reachable.
			ImmediateAddPlatforms = NewReachablePlatforms;
		}
		else
		{
			ImmediateAddPlatforms.Reset();
			for (const ITargetPlatform* TargetPlatform : NewReachablePlatforms)
			{
				FPackagePlatformData& PlatformData = Referencer->FindOrAddPlatformData(TargetPlatform);
				if (PlatformData.IsReachable(EReachability::Runtime) && PlatformData.IsExplorable())
				{
					ImmediateAddPlatforms.Add(TargetPlatform);
				}
			}
		}

		// Remove platforms that are already reachable and explorable from the ImmediateAddPlatforms.
		// Also handle the ForceExplorableSaveTimeSoftDependency flag to mark the platform explorable.
		for (TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>>::TIterator
			Iter(ImmediateAddPlatforms); Iter; ++Iter)
		{
			const ITargetPlatform* TargetPlatform = *Iter;
			FPackagePlatformData& PlatformData = PackageData.FindOrAddPlatformData(TargetPlatform);

			if (Discovery->Instigator.Category == EInstigator::ForceExplorableSaveTimeSoftDependency &&
				!PlatformData.IsExplorable())
			{
				PlatformData.MarkAsExplorable(); // Clears reachability so IsReachable below returns false.
			}
			if (PlatformData.IsReachable(EReachability::Runtime))
			{
				Iter.RemoveCurrentSwap();
			}
		}

		bool bAddToCluster = !ImmediateAddPlatforms.IsEmpty();
		// Handle the edge case that all of the Addable platforms are already reachable, and are not yet committed,
		// but the package is not in progress.
		// TODO: Is this edge case possible? How can it occur?
		if (!bAddToCluster && !PackageData.IsInProgress()
			&& (PackageData.GetPlatformsNeedingCommitNum(EReachability::Runtime) > 0
				|| !PackageData.AreAllReachablePlatformsVisitedByCluster(EReachability::Runtime)))
		{
			bAddToCluster = true;
		}

		if (bAddToCluster)
		{
			// Add the new reachable platforms
			PackageData.AddReachablePlatforms(*this, EReachability::Runtime, ImmediateAddPlatforms,
				MoveTemp(Discovery->Instigator));

			// Send it to the Request state if it's not already there, remove it from its old container
			// and add it to this cluster.
			FVertexData& Vertex = FindOrAddVertex(PackageData);
			if (!Vertex.IsOwnedByCluster())
			{
				// QueueRemove in SendToState does not know how to handle Packages assigned to a cluster
				// in construction, so we must pass in QueueRemove if and only if it's not in this cluster
				PackageData.SendToState(EPackageState::Request, ESendFlags::QueueRemove, EStateChangeReason::RequestCluster);
				SetOwnedByCluster(Vertex, true /* bOwnedByCluster */, false /* bNeedsStateChange */);
			}
			else
			{
				// If it is in this cluster, it should have already been put into the Request state.
				ensure(PackageData.GetState() == EPackageState::Request);
			}
		}

		// If urgency was specified and the package is now (or was already) in progress, raise the urgency
		if (Discovery->Urgency > PackageData.GetUrgency() && PackageData.IsInProgress())
		{
			PackageData.RaiseUrgency(Discovery->Urgency,
				// Raising urgency depending on state will need to remove and readd it, but don't allow that
				// if we added it to this cluster because RaiseUrgency doesn't know how to handle adding and
				// removing from the cluster.
				bAddToCluster ? ESendFlags::QueueNone : ESendFlags::QueueAddAndRemove);
		}
	}
}

FRequestCluster::FRequestCluster(UCookOnTheFlyServer& InCOTFS, EBuildDependencyQueueConstructorType,
	TRingBuffer<FPackageData*>& BuildDependencyDiscoveryQueue)
	: FRequestCluster(InCOTFS, EReachability::Build)
{
	while (!BuildDependencyDiscoveryQueue.IsEmpty())
	{
		FPackageData& PackageData = *BuildDependencyDiscoveryQueue.PopFrontValue();
		if (PackageData.IsInProgress() ||
			PackageData.GetPlatformsNeedingCommitNum(EReachability::Build) == 0)
		{
			// Already kicked or committed since being queued
			continue;
		}

		PackageData.SendToState(EPackageState::Request, ESendFlags::QueueRemove,
			EStateChangeReason::RequestCluster);
		FVertexData& Vertex = FindOrAddVertex(PackageData);
		SetOwnedByCluster(Vertex, true /* bOwnedByCluster */, false /* bNeedsStateChange */);
	}
}

bool FRequestCluster::TraversalExploreRuntimeDependencies()
{
	switch (TraversalTier)
	{
	case ETraversalTier::MarkForRuntime:
		return false;
	case ETraversalTier::MarkForBuildDependency:
		return false;
	case ETraversalTier::BuildDependencies:
		return false;
	case ETraversalTier::RuntimeVisitVertices:
		return false;
	case ETraversalTier::RuntimeFollowDependencies:
		return true;
	default:
		checkNoEntry();
		return false;
	}
}

bool FRequestCluster::TraversalExploreIncremental()
{
	switch (TraversalTier)
	{
	case ETraversalTier::MarkForRuntime:
		return false;
	case ETraversalTier::MarkForBuildDependency:
		return false;
	case ETraversalTier::BuildDependencies:
		return IsIncrementalCook();
	case ETraversalTier::RuntimeVisitVertices:
		return IsIncrementalCook();
	case ETraversalTier::RuntimeFollowDependencies:
		return IsIncrementalCook();
	default:
		checkNoEntry();
		return false;
	}
}

bool FRequestCluster::TraversalMarkCookable()
{
	switch (TraversalTier)
	{
	case ETraversalTier::MarkForRuntime:
		return true;
	case ETraversalTier::MarkForBuildDependency:
		return false;
	case ETraversalTier::BuildDependencies:
		return false;
	case ETraversalTier::RuntimeVisitVertices:
		return true;
	case ETraversalTier::RuntimeFollowDependencies:
		return true;
	default:
		checkNoEntry();
		return false;
	}
}

FName GInstigatorRequestCluster(TEXT("RequestCluster"));

void FRequestCluster::Process(const FCookerTimer& CookerTimer, bool& bOutComplete)
{
	bOutComplete = true;

	FetchPackageNames(CookerTimer, bOutComplete);
	if (!bOutComplete)
	{
		return;
	}
	PumpExploration(CookerTimer, bOutComplete);
	if (!bOutComplete)
	{
		return;
	}
	StartAsync(CookerTimer, bOutComplete);
}

void FRequestCluster::FetchPackageNames(const FCookerTimer& CookerTimer, bool& bOutComplete)
{
	if (bPackageNamesComplete)
	{
		return;
	}

	check(TraversalMarkCookable() || FilePlatformRequests.IsEmpty());
	constexpr int32 TimerCheckPeriod = 100; // Do not incur the cost of checking the timer on every package
	int32 NextRequest = 0;
	for (; NextRequest < FilePlatformRequests.Num(); ++NextRequest)
	{
		if ((NextRequest+1) % TimerCheckPeriod == 0 && CookerTimer.IsActionTimeUp())
		{
			break;
		}

		FFilePlatformRequest& Request = FilePlatformRequests[NextRequest];
		FName OriginalName = Request.GetFilename();

		// The input filenames are normalized, but might be missing their extension, so allow PackageDatas
		// to correct the filename if the package is found with a different filename
		bool bExactMatchRequired = false;
		FPackageData* PackageData = PackageDatas.TryAddPackageDataByStandardFileName(OriginalName,
			bExactMatchRequired);
		if (!PackageData)
		{
			LogCookerMessage(FString::Printf(TEXT("Could not find package at file %s!"),
				*OriginalName.ToString()), EMessageSeverity::Error);
			UE_LOGF(LogCook, Error, "Could not find package at file %ls!", *OriginalName.ToString());
			FCompletionCallback CompletionCallback(MoveTemp(Request.GetCompletionCallback()));
			if (CompletionCallback)
			{
				CompletionCallback(nullptr);
			}
			continue;
		}

		// If it has new reachable platforms we definitely need to explore it
		if (!PackageData->HasReachablePlatforms(EReachability::Runtime, Request.GetPlatforms()))
		{
			PackageData->AddReachablePlatforms(*this, EReachability::Runtime, Request.GetPlatforms(),
				MoveTemp(Request.GetInstigator()));
			FVertexData& Vertex = FindOrAddVertex(*PackageData);
			SetOwnedByCluster(Vertex, true);
			if (Request.IsUrgent())
			{
				PackageData->SetUrgency(EUrgency::Blocking, ESendFlags::QueueNone);
			}
		}
		else
		{
			if (PackageData->IsInProgress())
			{
				// If it's already in progress with no new platforms, we don't need to add it to the cluster, but add
				// add on our urgency setting
				if (Request.IsUrgent())
				{
					PackageData->SetUrgency(EUrgency::Blocking, ESendFlags::QueueAddAndRemove);
				}
			}
			else if (PackageData->GetPlatformsNeedingCommitNum(EReachability::Runtime) > 0 || !PackageData->AreAllReachablePlatformsVisitedByCluster(EReachability::Runtime))
			{
				// If it's missing cookable platforms and not in progress we need to add it to the cluster for cooking
				FVertexData& Vertex = FindOrAddVertex(*PackageData);
				SetOwnedByCluster(Vertex, true);
				if (Request.IsUrgent())
				{
					PackageData->SetUrgency(EUrgency::Blocking, ESendFlags::QueueNone);
				}
			}
		}
		// Add on our completion callback, or call it immediately if already done
		PackageData->AddCompletionCallback(Request.GetPlatforms(), MoveTemp(Request.GetCompletionCallback()));
	}
	if (NextRequest < FilePlatformRequests.Num())
	{
		FilePlatformRequests.RemoveAt(0, NextRequest);
		bOutComplete = false;
		return;
	}

	FilePlatformRequests.Empty();
	bPackageNamesComplete = true;
}

void FRequestCluster::ReserveInitialRequests(int32 RequestNum)
{
	ClusterPackages.Reserve(FMath::Max(RequestNum, 1024));
}

void FRequestCluster::AddVertexCounts(FVertexData& Vertex, int32 Delta)
{
	if (Vertex.IsOwnedByCluster())
	{
		NumOwned += Delta;
		if (Vertex.IsOwnedButNotInProgress())
		{
			NumOwnedButNotInProgress += Delta;
		}
	}
}

void FRequestCluster::SetOwnedByCluster(FVertexData& Vertex, bool bOwnedByCluster, bool bNeedsStateChange)
{
	if (bOwnedByCluster == Vertex.IsOwnedByCluster())
	{
		return;
	}
	AddVertexCounts(Vertex, -1);
	Vertex.SetOwnedByCluster(bOwnedByCluster);
	AddVertexCounts(Vertex, 1);

	if (bOwnedByCluster && bNeedsStateChange && Vertex.GetPackageData())
	{
		FPackageData& PackageData = *Vertex.GetPackageData();
		// Steal it from wherever it is and send it to Request State. It has already been added to this cluster
		if (PackageData.GetState() == EPackageState::Request)
		{
			COTFS.PackageDatas->GetRequestQueue().RemoveRequestExceptFromCluster(&PackageData, this);
		}
		else
		{
			PackageData.SendToState(EPackageState::Request, ESendFlags::QueueRemove,
				EStateChangeReason::RequestCluster);
		}
	}
}

void FRequestCluster::SetSuppressReason(FVertexData& Vertex, ESuppressCookReason Reason)
{
	check(Reason != ESuppressCookReason::Invalid);

	AddVertexCounts(Vertex, -1);
	Vertex.SetSuppressReason(Reason);
	AddVertexCounts(Vertex, 1);
}

void FRequestCluster::SetWasMarkedSkipped(FVertexData& Vertex, bool bValue)
{
	AddVertexCounts(Vertex, -1);
	Vertex.SetWasMarkedSkipped(bValue);
	AddVertexCounts(Vertex, 1);
}

void FRequestCluster::StartAsync(const FCookerTimer& CookerTimer, bool& bOutComplete)
{
	using namespace UE::DerivedData;
	using namespace UE::EditorDomain;

	if (bStartAsyncComplete)
	{
		return;
	}

	if (!TraversalMarkCookable())
	{
		return;
	}

	FEditorDomain* EditorDomain = FEditorDomain::Get();
	if (EditorDomain && EditorDomain->IsReadingPackages())
	{
		bool bBatchDownloadEnabled = true;
		GConfig->GetBool(TEXT("EditorDomain"), TEXT("BatchDownloadEnabled"), bBatchDownloadEnabled, GEditorIni);
		if (bBatchDownloadEnabled)
		{
			// If the EditorDomain is active, then batch-download all packages to cook from remote cache into local
			TArray<FName> BatchDownload;
			BatchDownload.Reserve(ClusterPackages.Num());
			for (TPair<FName, FVertexData*>& Pair : ClusterPackages)
			{
				FVertexData* Vertex = Pair.Value;
				if (Vertex->IsOwnedByCluster() && Vertex->GetSuppressReason() == ESuppressCookReason::NotSuppressed)
				{
					BatchDownload.Add(Pair.Key);
				}
			};
			EditorDomain->BatchDownload(BatchDownload);
		}
	}

	bStartAsyncComplete = true;
}

void FRequestCluster::RemovePackageData(FPackageData* PackageData)
{
	if (!PackageData)
	{
		return;
	}
	FVertexData** VertexPtr = ClusterPackages.Find(PackageData->GetPackageName());
	if (!VertexPtr)
	{
		return;
	}
	check(*VertexPtr);
	FVertexData& Vertex = **VertexPtr;
	SetOwnedByCluster(Vertex, false);
}

void FRequestCluster::OnNewReachablePlatforms(FPackageData* PackageData)
{
	if (GraphSearch.IsInitialized())
	{
		GraphSearch.OnNewReachablePlatforms(PackageData);
	}
}

void FRequestCluster::OnBeforePlatformAddedToSession(const ITargetPlatform* TargetPlatform)
{
	FCookerTimer CookerTimer(FCookerTimer::Forever);
	bool bComplete;
	while (Process(CookerTimer, bComplete), !bComplete)
	{
		UE_LOGF(LogCook, Display, "Waiting for RequestCluster to finish before adding platform to session.");
		FPlatformProcess::Sleep(.001f);
	}
}

void FRequestCluster::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform)
{
	FCookerTimer CookerTimer(FCookerTimer::Forever);
	bool bComplete;
	while (Process(CookerTimer, bComplete), !bComplete)
	{
		UE_LOGF(LogCook, Display,
			"Waiting for RequestCluster to finish before removing platform from session.");
		FPlatformProcess::Sleep(.001f);
	}
}

void FRequestCluster::RemapTargetPlatforms(TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	if (GraphSearch.IsStarted())
	{
		// The platforms have already been invalidated, which means we can't wait for GraphSearch to finish
		// Need to wait for all async operations to finish, then remap all the platforms
		checkNoEntry(); // Not yet implemented
	}
}

bool FRequestCluster::Contains(FPackageData* PackageData) const
{
	if (!PackageData)
	{
		return false;
	}
	FVertexData*const* VertexPtr = ClusterPackages.Find(PackageData->GetPackageName());
	if (!VertexPtr)
	{
		return false;
	}
	check(*VertexPtr);
	return (*VertexPtr)->IsOwnedByCluster();
}

void FRequestCluster::ClearAndDetachOwnedPackageDatas(TArray<FPackageData*>& OutRequestsToLoad,
	TArray<TPair<FPackageData*, ESuppressCookReason>>& OutRequestsToDemote,
	TMap<FPackageData*, TArray<FPackageData*>>& OutRequestGraph)
{
	if (bStartAsyncComplete)
	{
		check(!GraphSearch.IsStarted());
		OutRequestsToLoad.Reset();
		OutRequestsToDemote.Reset();
		for (TPair<FName, FVertexData*>& Pair : ClusterPackages)
		{
			check(Pair.Value);
			FVertexData& Vertex = *Pair.Value;
			if (Vertex.IsOwnedByCluster() && Vertex.GetPackageData())
			{
				if (Vertex.GetSuppressReason() == ESuppressCookReason::NotSuppressed)
				{
					OutRequestsToLoad.Add(Vertex.GetPackageData());
				}
				else
				{
					OutRequestsToDemote.Add({ Vertex.GetPackageData(), Vertex.GetSuppressReason()});
				}
			}
		}
		OutRequestGraph = MoveTemp(RequestGraph);
	}
	else
	{
		OutRequestsToLoad.Reset();
		for (TPair<FName, FVertexData*>& Pair : ClusterPackages)
		{
			check(Pair.Value);
			FVertexData& Vertex = *Pair.Value;
			if (Vertex.IsOwnedByCluster() && Vertex.GetPackageData())
			{
				OutRequestsToLoad.Add(Vertex.GetPackageData());
			}
		}
		OutRequestsToDemote.Reset();
		OutRequestGraph.Reset();
	}
	FilePlatformRequests.Empty();
	EmptyClusterPackages();
	NumOwned = 0;
	NumOwnedButNotInProgress = 0;
	GraphSearch.Reset();
	RequestGraph.Reset();
}

void FRequestCluster::PumpExploration(const FCookerTimer& CookerTimer, bool& bOutComplete)
{
	if (bDependenciesComplete)
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		COTFS.LogHandler->ConditionalPruneReplay();
	};
	if (!GraphSearch.IsStarted())
	{
		GraphSearch.Initialize();
		if (!TraversalExploreIncremental() && !TraversalExploreRuntimeDependencies())
		{
			GraphSearch.VisitWithoutFetching();
			bDependenciesComplete = true;
			return;
		}
		GraphSearch.StartSearch();
	}

	constexpr double WaitTime = 0.50;
	bool bDone;
	while (GraphSearch.TickExploration(bDone), !bDone)
	{
		GraphSearch.WaitForAsyncQueue(WaitTime);
		if (CookerTimer.IsActionTimeUp())
		{
			bOutComplete = false;
			return;
		}
	}

	TArray<FPackageData*> SortedPackages;
	SortedPackages.Reserve(ClusterPackages.Num());
	for (TPair<FName, FVertexData*>& Pair : ClusterPackages)
	{
		check(Pair.Value);
		FVertexData& Vertex = *Pair.Value;
		if (Vertex.IsOwnedByCluster() && Vertex.GetPackageData()
			&& Vertex.GetSuppressReason() == ESuppressCookReason::NotSuppressed)
		{
			SortedPackages.Add(Vertex.GetPackageData());
		}
	}

	// Sort the NewRequests in leaf to root order and replace the requests list with NewRequests
	TArray<FPackageData*> Empty;
	auto GetElementDependencies = [this, &Empty](FPackageData* PackageData) -> const TArray<FPackageData*>&
	{
		const TArray<FPackageData*>* VertexEdges = GraphSearch.GetGraphEdges().Find(PackageData);
		return VertexEdges ? *VertexEdges : Empty;
	};

	Algo::TopologicalSort(SortedPackages, GetElementDependencies, Algo::ETopologicalSort::AllowCycles);
	TMap<FPackageData*, int32> SortOrder;
	int32 Counter = 0;
	SortOrder.Reserve(SortedPackages.Num());
	for (FPackageData* PackageData : SortedPackages)
	{
		SortOrder.Add(PackageData, Counter++);
	}
	ClusterPackages.ValueSort([&SortOrder](const FVertexData& A, const FVertexData& B)
		{
			int32* CounterA = SortOrder.Find(A.GetPackageData());
			int32* CounterB = SortOrder.Find(B.GetPackageData());
			if ((CounterA != nullptr) != (CounterB != nullptr))
			{
				// Sort the missing packages, unowned vertices, or demotes to occur last
				return CounterB == nullptr;
			}
			else if (CounterA)
			{
				return *CounterA < *CounterB;
			}
			else
			{
				return false; // missing packages, unowned vertices and demotes are unsorted
			}
		});

	RequestGraph = MoveTemp(GraphSearch.GetGraphEdges());
	GraphSearch.Reset();
	bDependenciesComplete = true;
}

FRequestCluster::FGraphSearch::FGraphSearch(FRequestCluster& InCluster)
	: Cluster(InCluster)
	, ExploreEdgesContext(InCluster, *this)
	, AsyncResultsReadyEvent(EEventMode::ManualReset)
{
}

void FRequestCluster::FGraphSearch::Initialize()
{
	if (bInitialized)
	{
		return;
	}

	AsyncResultsReadyEvent->Trigger();
	LastActivityTime = FPlatformTime::Seconds();
	BatchAllocator.SetMinBlockSize(16);
	BatchAllocator.SetMaxBlockSize(16);

	TConstArrayView<const ITargetPlatform*> SessionPlatforms = Cluster.COTFS.PlatformManager->GetSessionPlatforms();
	check(SessionPlatforms.Num() > 0);
	check(SessionPlatforms.Num() == Cluster.GetNumSessionPlatforms())
	FetchPlatforms.SetNum(Cluster.GetNumFetchPlatforms());
	FetchPlatforms[PlatformAgnosticPlatformIndex].bIsPlatformAgnosticPlatform = true;
	FetchPlatforms[CookerLoadingPlatformIndex].Platform = CookerLoadingPlatformKey;
	FetchPlatforms[CookerLoadingPlatformIndex].bIsCookerLoadingPlatform = true;
	for (int32 SessionPlatformIndex = 0; SessionPlatformIndex < SessionPlatforms.Num(); ++SessionPlatformIndex)
	{
		FFetchPlatformData& FetchPlatform = FetchPlatforms[SessionPlatformIndex + 2];
		FetchPlatform.Platform = SessionPlatforms[SessionPlatformIndex];
		FetchPlatform.Writer = &Cluster.COTFS.FindOrCreatePackageWriter(FetchPlatform.Platform);
	}
	Algo::Sort(FetchPlatforms, [](const FFetchPlatformData& A, const FFetchPlatformData& B)
		{
			return A.Platform < B.Platform;
		});
	check(FetchPlatforms[PlatformAgnosticPlatformIndex].bIsPlatformAgnosticPlatform);
	check(FetchPlatforms[CookerLoadingPlatformIndex].bIsCookerLoadingPlatform);

	bInitialized = true;
}

void FRequestCluster::FGraphSearch::VisitWithoutFetching()
{
	// PumpExploration is responsible for marking all requests as explored and cookable/uncoookable.
	// If we're skipping the dependencies search, handle that responsibility for the initial requests and return.
	for (TPair<FName, FVertexData*>& Pair : Cluster.ClusterPackages)
	{
		check(Pair.Value);
		FVertexData& Vertex = *Pair.Value;
		if (!Vertex.GetPackageData())
		{
			continue;
		}
		check(Vertex.IsOwnedByCluster());
		VisitVertex(Vertex);
	}
}

void FRequestCluster::FGraphSearch::StartSearch()
{
	VisitVertexQueue.Reserve(Cluster.ClusterPackages.Num());
	for (TPair<FName, FVertexData*>& Pair : Cluster.ClusterPackages)
	{
		check(Pair.Value);
		FVertexData& Vertex = *Pair.Value;
		if (!Vertex.GetPackageData())
		{
			continue;
		}
		check(Vertex.IsOwnedByCluster());
		AddToVisitVertexQueue(Vertex);
	}
	bStarted = true;
}

FRequestCluster::FGraphSearch::~FGraphSearch()
{
	Reset();
}

void FRequestCluster::FGraphSearch::Reset()
{
	for (;;)
	{
		bool bHadActivity = false;
		bool bAsyncBatchesEmpty = false;
		{
			FScopeLock ScopeLock(&Lock);
			bAsyncBatchesEmpty = AsyncQueueBatches.IsEmpty();
			if (!bAsyncBatchesEmpty)
			{
				// It is safe to Reset AsyncResultsReadyEvent and wait on it later because we are inside the lock and
				// there is a remaining batch, so it will be triggered after the Reset when that batch completes.
				AsyncResultsReadyEvent->Reset();
			}
		}
		for (;;)
		{
			if (AsyncQueueResults.Dequeue())
			{
				bHadActivity = true;
			}
			else
			{
				break;
			}
		}
		if (bAsyncBatchesEmpty)
		{
			break;
		}
		if (bHadActivity)
		{
			LastActivityTime = FPlatformTime::Seconds();
		}
		else
		{
			UpdateDisplay();
		}
		constexpr double WaitTime = 1.0;
		WaitForAsyncQueue(WaitTime);
	}

	GraphEdges.Empty();
	VisitVertexQueue.Empty();
	PendingTransitiveBuildDependencyVertices.Empty();
	PreAsyncQueue.Empty();
	RunAwayTickLoopCount = 0;
	bStarted = false;

	BatchAllocator.Empty();
	check(AsyncQueueBatches.IsEmpty()); // Emptied by for loop above
	check(AsyncQueueResults.IsEmpty()); // Emptied by for loop above
	AsyncResultsReadyEvent->Trigger();
}

void FRequestCluster::FGraphSearch::OnNewReachablePlatforms(FPackageData* PackageData)
{
	if (!PackageData)
	{
		return;
	}
	FVertexData** VertexPtr = Cluster.ClusterPackages.Find(PackageData->GetPackageName());
	if (!VertexPtr)
	{
		return;
	}
	AddToVisitVertexQueue(**VertexPtr);
}

void FRequestCluster::FGraphSearch::QueueEdgesFetch(FVertexData& Vertex, TConstArrayView<int32> PlatformIndexes)
{
	check(Vertex.GetPackageData()); // Caller must not call without a PackageData; doing so serves no purpose

	bool bAnyRequestedNeedsPlatformAgnostic = false;
	bool bAnyRequested = false;
	bool bAllHaveAlreadyCompletedFetch = true;

	for (int32 PlatformIndex : PlatformIndexes)
	{
		// The platform data may have already been requested; request it only if current status is NotRequested
		FQueryPlatformData& QueryData = Vertex.GetPlatformData()[PlatformIndex];
		if (!QueryData.bSchedulerThreadFetchCompleted)
		{
			bAllHaveAlreadyCompletedFetch = false;
			EAsyncQueryStatus ExpectedStatus = EAsyncQueryStatus::NotRequested;
			if (QueryData.CompareExchangeAsyncQueryStatus(ExpectedStatus, EAsyncQueryStatus::SchedulerRequested))
			{
				bAnyRequested = true;
			}
		}
	}

	if (bAnyRequested)
	{
		PreAsyncQueue.Add(&Vertex);
		CreateAvailableBatches(false /* bAllowIncompleteBatch */);
	}

	if (bAllHaveAlreadyCompletedFetch)
	{
		// We are contractually obligated to kick the vertex. Normally we would put it into PreAsyncQueue and that
		// queue would take responsibility for kicking it. Also, it might still be in the AsyncQueueResults for one
		// of the platforms so it will be kicked by TickExplore pulling it out of the AsyncQueueResults. But if all
		// requested platforms already previously pulled it out of AsyncQueueResults, then we need to kick it again.
		KickVertex(&Vertex);
	}
}

void FRequestCluster::FGraphSearch::WaitForAsyncQueue(double WaitTimeSeconds)
{
	uint32 WaitTime = (WaitTimeSeconds > 0.0) ? static_cast<uint32>(FMath::Floor(WaitTimeSeconds * 1000)) : MAX_uint32;
	AsyncResultsReadyEvent->Wait(WaitTime);
}

void FRequestCluster::FGraphSearch::TickExploration(bool& bOutDone)
{
	bool bHadActivity = false;

	int64 RunawayLoopCount = 0;
	// Add a counter to detect bugs that lead to an infinite loop. Calculation of upper bound:
	// Each time through the loop we either process or delay one platform of one vertex. Once delayed, a vertex is not added to
	// the queue again until all of its UnreadyDependencies for that platform are cleared, therefore we only delay it once per platform.
	// An upper bound is therefore 2 * NumPlatforms * NumVertices.
	int64 RunawayLoopUpperBound = 2 * Cluster.ClusterPackages.Num() * (FetchPlatforms.Num() - 1);
	for (;;)
	{
		TOptional<FVertexData*> FrontVertex = AsyncQueueResults.Dequeue();
		if (!FrontVertex.IsSet())
		{
			break;
		}
		FVertexData* Vertex = *FrontVertex;
		for (FQueryPlatformData& PlatformData : GetPlatformDataArray(*Vertex))
		{
			if (!PlatformData.bSchedulerThreadFetchCompleted)
			{
				PlatformData.bSchedulerThreadFetchCompleted =
					PlatformData.GetAsyncQueryStatus() >= EAsyncQueryStatus::Complete;
				// Note that AsyncQueryStatus might change immediately after we read it, so we might have set
				// FetchCompleted=false but now AsyncQueryStatus is complete. In that case, whatever async thread
				// changed the AsyncQueryStatus will also kick the vertex again and we will detect the new value when
				// we reach the new value of the vertexdata later in AsyncQueueResults
			}
		}

		ExploreEdgesContext.Explore(*Vertex);
		bHadActivity = true;

		if (RunawayLoopCount++ > RunawayLoopUpperBound)
		{
			UE_LOGF(LogCook, Fatal, "Infinite loop detected in FRequestCluster::TickExploration's AsyncQueueResults.");
		}
	}

	RunawayLoopCount = 0;
	// Calculation of upper bound: we visit each vertex at most once per platform.
	RunawayLoopUpperBound = Cluster.ClusterPackages.Num() * (FetchPlatforms.Num() - 1);
	while (!VisitVertexQueue.IsEmpty())
	{
		bHadActivity = true;
		// VisitVertex might try to add other vertices onto VisitVertexQueue, so move it into a snapshot and process
		// the snapshot. After snapshot processing is done, add on anything that was added and then move it back.
		// We move it back even if it is empty so we can avoid reallocating when we add to it again later.
		TSet<FVertexData*> Snapshot = MoveTemp(VisitVertexQueue);
		VisitVertexQueue.Reset();
		for (FVertexData* Vertex : Snapshot)
		{
			VisitVertex(*Vertex);
		}
		Snapshot.Reset();
		Snapshot.Append(VisitVertexQueue);
		VisitVertexQueue = MoveTemp(Snapshot);

		if (RunawayLoopCount++ > RunawayLoopUpperBound)
		{
			UE_LOGF(LogCook, Fatal, "Infinite loop detected in FRequestCluster::TickExploration's VisitVertexQueue.");
		}
	}

	if (bHadActivity)
	{
		++RunAwayTickLoopCount;
		if (RunAwayTickLoopCount++ > 2 * Cluster.ClusterPackages.Num()*Cluster.GetNumFetchPlatforms())
		{
			UE_LOGF(LogCook, Fatal, "Infinite loop detected in reentrant calls to FRequestCluster::TickExploration.");
		}
		LastActivityTime = FPlatformTime::Seconds();
		bOutDone = false;
		return;
	}

	bool bAsyncQueueEmpty;
	{
		FScopeLock ScopeLock(&Lock);
		if (!AsyncQueueResults.IsEmpty())
		{
			bAsyncQueueEmpty = false;
		}
		else
		{
			bAsyncQueueEmpty = AsyncQueueBatches.IsEmpty();
			// AsyncResultsReadyEvent can only be Reset when either the AsyncQueue is empty or it is non-empty and we
			// know the AsyncResultsReadyEvent will be triggered again "later".
			// The guaranteed place where it will be Triggered is when a batch completes. To guarantee that
			// place will be called "later", the batch completion trigger and this reset have to both
			// be done inside the lock.
			AsyncResultsReadyEvent->Reset();
		}
	}
	if (!bAsyncQueueEmpty)
	{
		// Waiting on the AsyncQueue; give a warning if we have been waiting for long with no AsyncQueueResults.
		UpdateDisplay();
		bOutDone = false;
		return;
	}

	// No more work coming in the future from the AsyncQueue, and we are out of work to do
	// without it. If we have any queued vertices in the PreAsyncQueue, send them now and continue
	// waiting. Otherwise we are done.
	if (!PreAsyncQueue.IsEmpty())
	{
		CreateAvailableBatches(true /* bAllowInCompleteBatch */);
		bOutDone = false;
		return;
	}

	if (!VisitVertexQueue.IsEmpty() || !bAsyncQueueEmpty || !PreAsyncQueue.IsEmpty())
	{
		// A container ticked earlier was populated by the tick of a later container; restart tick from beginning
		bOutDone = false;
		return;
	}

	// We are out of direct dependency work to do, but there could be a cycle in the graph of
	// TransitiveBuildDependencies. If so, resolve the cycle and allow those vertices' edges to be explored.
	if (!PendingTransitiveBuildDependencyVertices.IsEmpty())
	{
		ResolveTransitiveBuildDependencyCycle();
		bOutDone = false;
		++RunAwayTickLoopCount;
		if (RunAwayTickLoopCount++ > 2 * Cluster.ClusterPackages.Num() * Cluster.GetNumFetchPlatforms())
		{
			UE_LOGF(LogCook, Fatal, "Infinite loop detected in FRequestCluster::PendingTransitiveBuildDependencyVertices.");
		}
		return;
	}

	bOutDone = true;
}

void FRequestCluster::FGraphSearch::ResolveTransitiveBuildDependencyCycle()
{
	// We interpret cycles in the transitive build dependency graph to mean that every vertex in the cycle is
	// invalidated if and only if any dependency from any vertex that points outside the cycle is invalidated (the
	// dependency pointing outside the cycle might be either a transitive build dependency on a package outside of the
	// cycle or a direct dependency).

	// Using this definition, we can resolve as not incrementally modified, with no further calculation needed, all
	// elements in the PendingTransitiveBuildDependencyVertices graph, when we run out of direct dependency work to do.
	// Proof:

	// Every package in the PendingTransitiveBuildDependencyVertices set is one that is not invalidated by any of its
	// direct dependencies, but it has transitive build dependencies that might be invalidated.
	// If we have run out of direct dependency work to do, then there are no transitive build dependencies on any
	// vertex not in the set.
	// No direct dependency invalidations and no transitive build dependency invalidations, by our interpretation of a
	// cycle above, mean that the package is not invalidated.

	// Mark all of the currently fetched platforms of all packages in the PendingTransitiveBuildDependencyVertices as
	// ignore transitive build dependencies and kick them.

	FVertexData* FirstVertex = nullptr;
	for (FVertexData* CycleVert : PendingTransitiveBuildDependencyVertices)
	{
		check(CycleVert != nullptr); // Required hint for static analyzers.
		if (!FirstVertex)
		{
			FirstVertex = CycleVert;
		}
		for (FQueryPlatformData& PlatformData : GetPlatformDataArray(*CycleVert))
		{
			if (PlatformData.bIncrementallyUnmodifiedRequested || PlatformData.bExploreRequested)
			{
				PlatformData.bTransitiveBuildDependenciesResolvedAsNotModified = true;
			}
		}
		// We can also empty the IncrementallyModifiedListeners since any remaining listeners must be in
		// PendingTransitiveBuildDependencyVertices. Emptying the list here avoids the expense of kicking
		// for a second time each of the listeners.
		CycleVert->GetIncrementallyModifiedListeners().Empty();
		KickVertex(CycleVert);
	}
	check(FirstVertex); // This function should not be called if PendingTransitiveBuildDependencyVertices is empty.
	PendingTransitiveBuildDependencyVertices.Empty();
	UE_LOGF(LogCook, Verbose,
		"Cycle detected in the graph of transitive build dependencies." " No vertices in the cycle are invalidated by their direct dependencies, so marking them all as incrementally skippable." "\n\tVertex in the cycle: %ls",
		*FirstVertex->GetPackageName().ToString());
}

void FRequestCluster::FGraphSearch::UpdateDisplay()
{
	constexpr double WarningTimeout = 60.0;
	if (FPlatformTime::Seconds() > LastActivityTime + WarningTimeout && Cluster.IsIncrementalCook())
	{
		FScopeLock ScopeLock(&Lock);
		int32 NumPendingRequestsInBatches = 0;
		int32 NumBatches = AsyncQueueBatches.Num();
		for (FQueryVertexBatch* Batch : AsyncQueueBatches)
		{
			NumPendingRequestsInBatches += Batch->NumPendingRequests;
		}

		UE_LOGF(LogCook, Warning,
			"FRequestCluster waited more than %.0lfs for previous build results from the oplog. " "NumPendingBatches == %d, NumPendingRequestsInBatches == %d. Continuing to wait...",
			WarningTimeout, NumBatches, NumPendingRequestsInBatches);
		LastActivityTime = FPlatformTime::Seconds();
	}
}

void FRequestCluster::FGraphSearch::VisitVertex(FVertexData& Vertex)
{
	// Only called from scheduler thread

	// The PackageData will not exist if the package does not exist on disk
	if (!Vertex.GetPackageData())
	{
		return;
	}
	FPackageData& PackageData = *Vertex.GetPackageData();

	const EReachability ClusterReachability = Cluster.TraversalMarkCookable()
		? EReachability::Runtime : EReachability::Build;
	int32 LocalNumFetchPlatforms = Cluster.GetNumFetchPlatforms();
	TBitArray<> ShouldFetchPlatforms(false, LocalNumFetchPlatforms);
	
	FPackagePlatformData* CookerLoadingPlatform = nullptr;
	const ITargetPlatform* FirstReachableSessionPlatform = nullptr;
	bool bAnyPlatformNeedsCook = false;
	ESuppressCookReason SuppressCookReason = ESuppressCookReason::Invalid;
	for (TPair<const ITargetPlatform*, FPackagePlatformData>& Pair :
		PackageData.GetPlatformDatasConstKeysMutableValues())
	{
		FPackagePlatformData& PlatformData = Pair.Value;
		const ITargetPlatform* TargetPlatform = Pair.Key;
		if (TargetPlatform == CookerLoadingPlatformKey)
		{
			CookerLoadingPlatform = &PlatformData;
		}
		else if (PlatformData.IsReachable(ClusterReachability))
		{
			int32 PlatformIndex = Algo::BinarySearchBy(FetchPlatforms, TargetPlatform, [](const FFetchPlatformData& D)
				{
					return D.Platform;
				});
			check(PlatformIndex != INDEX_NONE);

			if (!FirstReachableSessionPlatform)
			{
				FirstReachableSessionPlatform = TargetPlatform;
			}
			ESuppressCookReason SuppressCookReasonForPlatform;
			if (!PlatformData.IsVisitedByCluster(ClusterReachability))
			{
				VisitVertexForPlatform(Vertex, TargetPlatform, ClusterReachability, PlatformData,
					SuppressCookReasonForPlatform);
			}
			else
			{
				// Already visited. Use the existing values of IsCookable and IsExplorable.
				SuppressCookReasonForPlatform = PlatformData.IsCookable()
					? ESuppressCookReason::NotSuppressed : ESuppressCookReason::Invalid;
			}

			// We cook the package for all platforms on which it is reachable and not suppressed. We mark that this
			// unsuppressed reachability has occurred by setting SuppressCookReason=NotSuppressed for the package in
			// Cluster.SetSuppressReason below.
			if (SuppressCookReasonForPlatform == ESuppressCookReason::NotSuppressed ||
				SuppressCookReason == ESuppressCookReason::NotSuppressed)
			{
				SuppressCookReason = ESuppressCookReason::NotSuppressed;
			}
			else if (SuppressCookReason == ESuppressCookReason::Invalid)
			{
				SuppressCookReason = SuppressCookReasonForPlatform;
			}

			bool bPlatformNeedsCook = PlatformData.IsCookable() && !PlatformData.IsCookAttempted();
			bAnyPlatformNeedsCook |= bPlatformNeedsCook;

			// Other questions that the cluster might need to answer about the package is what runtime dependencies of
			// the package need to be marked reachable because this package is reachable, and whether we want to mark
			// the package as incrementally skippable (and therefore mark it already cooked without cooking it in this
			// session). Both of those questions require expensively and asynchronously fetching dependencies about
			// the package from the previous cook's oplog. Mark that we need to do that fetch based on whether this
			// cluster needs to answer those questions about the package.
			const bool bWasCookedThisSessionOrReevaluateRequested = PlatformData.GetWhereCooked() == EWhereCooked::ThisSession || EnumHasAnyFlags(Cluster.COTFS.CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::DlcReevaluateUncookedAssets);
			const bool bShouldExploreDependencies = (Cluster.TraversalExploreRuntimeDependencies()
				&& PlatformData.IsExplorable()
				&& bWasCookedThisSessionOrReevaluateRequested);

			if ((Cluster.TraversalExploreIncremental() && bPlatformNeedsCook)
				|| bShouldExploreDependencies)
			{
				ShouldFetchPlatforms[PlatformIndex] = true;
				Vertex.GetPlatformData()[PlatformIndex].bExploreRequested = true;
				// Exploration of any session platform also requires exploration of PlatformAgnosticPlatform
				Vertex.GetPlatformData()[PlatformAgnosticPlatformIndex].bExploreRequested = true;
			}
		}
	}

	// TODO_COOKGENERATIONHELPER: We don't currently support separate cooking for one platform but not
	// another for a generated package, see the class comment on FGenerationHelper. Therefore if any platform
	// was found to be reachable, then mark the other platforms as reachable.
	if (PackageData.GetGenerationHelper() && FirstReachableSessionPlatform != nullptr)
	{
		TConstArrayView<const ITargetPlatform*> SessionPlatforms = Cluster.COTFS.PlatformManager->GetSessionPlatforms();
		if (!PackageData.HasReachablePlatforms(ClusterReachability, SessionPlatforms))
		{
			PackageData.AddReachablePlatforms(Cluster, ClusterReachability, SessionPlatforms,
				FInstigator(EInstigator::GeneratedPackage, PackageData.GetPackageName()));
			// Restart the visit with a recursive call. We will not come in here again, because HasReachablePlatforms will
			// return true.
			return VisitVertex(Vertex);
		}
	}

	if (Cluster.TraversalMarkCookable())
	{
		// Set bMarkPackageAsCookable=true even if we don't know yet because it was unreachable and so we didn't run
		// IsCookable. This is important because we initially mark every package in the cluster is cookable until it
		// becomes reachable and we test cookability. This is important for build dependencies, which might be visited
		// for incrementally modified calculations on a referencer package before being visited again for
		// reachability.
		bool bMarkPackageAsCookable = FirstReachableSessionPlatform == nullptr
			|| SuppressCookReason == ESuppressCookReason::NotSuppressed;
		if (bMarkPackageAsCookable != Vertex.IsAnyCookable())
		{
			if (!bMarkPackageAsCookable)
			{
				if (SuppressCookReason == ESuppressCookReason::Invalid)
				{
					// We need the SuppressCookReason for reporting. If we didn't calculate it this Visit and
					// we don't have it stored in this->ClusterPackages, then we must have calculated it in
					// a previous cluster, but we don't store it anywhere. Recalculate it from the
					// FirstReachableSessionPlatform. FirstReachableSessionPlatform must be non-null, otherwise
					// bMarkPackageAsCookable would be true.
					check(FirstReachableSessionPlatform);
					bool bCookable;
					bool bExplorable;
					Cluster.IsRequestCookable(FirstReachableSessionPlatform, PackageData, SuppressCookReason, bCookable,
						bExplorable);
					if (bCookable && PackageData.FindOrAddPlatformData(FirstReachableSessionPlatform).IsCookAttempted())
					{
						SuppressCookReason = ESuppressCookReason::AlreadyCooked;
					}
					// We don't support bCookable changing for a given package and platform, so if it is suppressed now
					// it should have either been not cookable before or should be suppressed because it was already
					// cooked.
					check(SuppressCookReason != ESuppressCookReason::Invalid
						&& SuppressCookReason != ESuppressCookReason::NotSuppressed);
				}
			}
			else
			{
				check(SuppressCookReason == ESuppressCookReason::NotSuppressed);
			}
			Cluster.SetSuppressReason(Vertex, SuppressCookReason);
			Vertex.SetAnyCookable(bMarkPackageAsCookable);
		}

		// If any reachable target platform is cookable and not already cooked, then we need to mark the
		// CookerLoadingPlatform as reachable and explore its dependencies because we will need to load the package
		// to cook it.
		if (bAnyPlatformNeedsCook)
		{
			if (!CookerLoadingPlatform)
			{
				CookerLoadingPlatform = &PackageData.FindOrAddPlatformData(CookerLoadingPlatformKey);
			}
			CookerLoadingPlatform->AddReachability(EReachability::Runtime);
			if (!CookerLoadingPlatform->IsVisitedByCluster(EReachability::Runtime))
			{
				// Once reachable due to a SessionPlatform being cookable and reachable, the cooker loading platform
				// is defined as cookable and reachable.
				CookerLoadingPlatform->SetCookable(true);
				CookerLoadingPlatform->SetExplorable(true);
				CookerLoadingPlatform->AddVisitedByCluster(EReachability::Runtime);
			}
			// Fetch and explore the hard imports of the package to mark other packages as expected load, but skip
			// doing that if the cluster is running in a mode where it is just marking each package as cookable
			// individually rather than searching dependencies, because we don't need to read the reachability flags
			// packages in that case (e.g. because we are on a CookWorker being told a list of packages to cook by the
			// director).
			if (Cluster.TraversalExploreRuntimeDependencies())
			{
				ShouldFetchPlatforms[CookerLoadingPlatformIndex] = true;
				Vertex.GetPlatformData()[CookerLoadingPlatformIndex].bExploreRequested = true;
			}
		}
	}

	const bool bMightNeedToFetch = Cluster.TraversalExploreIncremental()
		|| Cluster.TraversalExploreRuntimeDependencies();
	if (bMightNeedToFetch)
	{
		for (int32 PlatformIndex = 0; PlatformIndex < LocalNumFetchPlatforms; ++PlatformIndex)
		{
			FQueryPlatformData& PlatformData = Vertex.GetPlatformData()[PlatformIndex];

			// Add on the fetch (but not the explore) of bIncrementallyUnmodifiedRequested platforms
			if (PlatformData.bIncrementallyUnmodifiedRequested)
			{
				ShouldFetchPlatforms[PlatformIndex] = true;
			}

			// Also add the fetch (but not necessarily the explore) of PlatformAgnosticPlatform if a
			// SessionPlatform is fetched.
			if (ShouldFetchPlatforms[PlatformIndex] &&
				PlatformIndex != CookerLoadingPlatformIndex && PlatformIndex != PlatformAgnosticPlatformIndex)
			{
				ShouldFetchPlatforms[PlatformAgnosticPlatformIndex] = true;
			}
		}

		// Convert Bit Array to an array of indexes and fetch them if non empty
		TArray<int32, TInlineAllocator<10>> FetchPlatformIndexes;
		for (int32 PlatformIndex = 0; PlatformIndex < LocalNumFetchPlatforms; ++PlatformIndex)
		{
			if (ShouldFetchPlatforms[PlatformIndex])
			{
				FetchPlatformIndexes.Add(PlatformIndex);
			}
		}
		if (!FetchPlatformIndexes.IsEmpty())
		{
			QueueEdgesFetch(Vertex, FetchPlatformIndexes);
		}
	}
}

void FRequestCluster::FGraphSearch::VisitVertexForPlatform(FVertexData& Vertex, const ITargetPlatform* Platform,
	EReachability ClusterReachability, FPackagePlatformData& PlatformData,
	ESuppressCookReason& OutSuppressCookReason)
{
	if (Cluster.TraversalMarkCookable())
	{
		FPackageData& PackageData = *Vertex.GetPackageData();
		bool bCookable;
		bool bExplorable;
		Cluster.IsRequestCookable(Platform, PackageData, OutSuppressCookReason,
			bCookable, bExplorable);
		PlatformData.SetCookable(bCookable);
		PlatformData.SetExplorable(bExplorable);
		if (bCookable)
		{
			OutSuppressCookReason = ESuppressCookReason::NotSuppressed;
		}
		else
		{
			check(OutSuppressCookReason != ESuppressCookReason::Invalid
				&& OutSuppressCookReason != ESuppressCookReason::NotSuppressed);
		}
	}
	else
	{
		OutSuppressCookReason = ESuppressCookReason::Invalid;
	}
	PlatformData.AddVisitedByCluster(ClusterReachability);
}

FRequestCluster::FGraphSearch::FExploreEdgesContext::FExploreEdgesContext(FRequestCluster& InCluster,
	FGraphSearch& InGraphSearch)
	: Cluster(InCluster)
	, GraphSearch(InGraphSearch)
{
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::Explore(FVertexData& InVertex)
{
	// Only called from scheduler thread

	Initialize(InVertex);
	CalculatePlatformsToProcess();
	if (PlatformsToProcess.IsEmpty())
	{
		return;
	}

	if (!TryCalculateIncrementallyUnmodified())
	{
		// The vertex was added as a listener to the pending data it needs. Exit from explore
		// for now and we will reenter it later when the data becomes available.
		return;
	}
	if (PlatformsToExplore.IsEmpty())
	{
		// We had platforms we needed to test for incrementally unmodified (for e.g. TransitiveBuildDependencies), but
		// nothing to explore. No more work to do until/unless they become marked for explore later.
		return;
	}

	CalculatePackageDataDependenciesPlatformAgnostic();
	CalculateDependenciesAndIncrementallySkippable();
	QueueVisitsOfDependencies();
	MarkExploreComplete();
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::Initialize(FVertexData& InVertex)
{
	Vertex = &InVertex;
	PackageData = Vertex->GetPackageData();
	PackageName = Vertex->GetPackageName();
	// Vertices without a package data are never queued for fetch
	check(PackageData);

	HardGameDependencies.Reset();
	HardEditorDependencies.Reset();
	SoftGameDependencies.Reset();
	CookerLoadingDependencies.Reset();
	PlatformsToProcess.Reset();
	PlatformsToExplore.Reset();
	PlatformDependencyMap.Reset();
	HardDependenciesSet.Reset();
	SkippedPackages.Reset();
	UnreadyTransitiveBuildVertices.Reset();

	LocalNumFetchPlatforms = Cluster.GetNumFetchPlatforms();
	bFetchAnyTargetPlatform = false;

	GraphSearch.PendingTransitiveBuildDependencyVertices.Remove(Vertex);
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::CalculatePlatformsToProcess()
{
	FQueryPlatformData& PlatformAgnosticQueryData = Vertex->GetPlatformData()[PlatformAgnosticPlatformIndex];
	for (int32 PlatformIndex = 0; PlatformIndex < LocalNumFetchPlatforms; ++PlatformIndex)
	{
		if (PlatformIndex == PlatformAgnosticPlatformIndex)
		{
			continue;
		}
		FQueryPlatformData& QueryPlatformData = Vertex->GetPlatformData()[PlatformIndex];
		if (!QueryPlatformData.bSchedulerThreadFetchCompleted)
		{
			continue;
		}
		bool bIncrementallyUnmodifiedNeeded = false;
		if (PlatformIndex != CookerLoadingPlatformIndex)
		{
			const ITargetPlatform* TargetPlatform = GraphSearch.FetchPlatforms[PlatformIndex].Platform;
			const FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
			bIncrementallyUnmodifiedNeeded = !PackagePlatformData.IsIncrementallyUnmodifiedSet();
		}
		bool bExploreNeeded = !QueryPlatformData.bExploreCompleted && QueryPlatformData.bExploreRequested;
		if (!bIncrementallyUnmodifiedNeeded && !bExploreNeeded)
		{
			continue;
		}
		if (bExploreNeeded && PlatformIndex != CookerLoadingPlatformIndex)
		{
			if (!PlatformAgnosticQueryData.bSchedulerThreadFetchCompleted)
			{
				continue;
			}
			// bExploreNeeded implies bExploreRequested, and wherever bExploreRequested is set to true we also set it
			// to true for PlatformAgnosticQueryData.
			check(PlatformAgnosticQueryData.bExploreRequested);
			bFetchAnyTargetPlatform = true;
		}
		PlatformsToProcess.Add(PlatformIndex);
		if (bExploreNeeded)
		{
			PlatformsToExplore.Add(PlatformIndex);
		}
	}
}

bool FRequestCluster::FGraphSearch::FExploreEdgesContext::TryCalculateIncrementallyUnmodified()
{
	using namespace UE::TargetDomain;

	if (!Cluster.IsIncrementalCook())
	{
		return true;
	}

	Vertex->GetUnreadyDependencies().Reset();
	Vertex->SetWaitingOnUnreadyDependencies(false);
	bool bAllPlatformsAreReady = true;

	TRefCountPtr<FGenerationHelper> GenerationHelper;
	FPackageData* ParentPackageData = nullptr;
	if (!PackageData->IsGenerated())
	{
		GenerationHelper = PackageData->GetGenerationHelper();
	}
	else
	{
		GenerationHelper = Vertex->IsOwnedByCluster() ? PackageData->GetOrFindParentGenerationHelper()
			: PackageData->GetOrFindParentGenerationHelperNoCache();
		if (GenerationHelper)
		{
			ParentPackageData = &GenerationHelper->GetOwner();
		}
	}

	for (int32 PlatformIndex : PlatformsToProcess)
	{
		if (PlatformIndex == CookerLoadingPlatformIndex)
		{
			continue;
		}

		FFetchPlatformData& FetchPlatformData = GraphSearch.FetchPlatforms[PlatformIndex];
		const ITargetPlatform* TargetPlatform = FetchPlatformData.Platform;
		FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
		if (PackagePlatformData.IsIncrementallyUnmodifiedSet())
		{
			continue;
		}
		FQueryPlatformData& QueryPlatformData = Vertex->GetPlatformData()[PlatformIndex];

		UE::Cook::FPackageArtifacts& Artifacts = QueryPlatformData.CookAttachments.Artifacts;
		bool bPreviouslyCooked =
			(QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Success)
			| (QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Error);
		FIncrementallyModifiedContext SetModifiedArgs;
		SetModifiedArgs.Artifacts = &Artifacts;
		SetModifiedArgs.GenerationHelper = GenerationHelper.GetReference();
		SetModifiedArgs.TargetPlatform = TargetPlatform;

		if (PackageData->IsGenerated())
		{
			if (!GenerationHelper)
			{
				// Generated packages for which we do not have the GenerationHelper available are not incrementally
				// skippable.
				SetIncrementallyModified(PlatformIndex, PackagePlatformData, bPreviouslyCooked,
					EIncrementallyModifiedReason::NoGenerator, SetModifiedArgs);
				continue;
			}
			check(ParentPackageData); // Set above in the IsGenerated && GenerationHelper case
			// If a generator is marked incrementally unmodified, then by contract we are not required to test its
			// generated packages; they are all marked incrementally unmodified as well
			const FPackagePlatformData* ParentPlatformData =
				ParentPackageData->GetPlatformDatas().Find(TargetPlatform);
			if (ParentPlatformData && ParentPlatformData->IsIncrementallyUnmodified())
			{
				SetIncrementallyUnmodified(PlatformIndex, PackagePlatformData);
				continue;
			}
		}
		if (!Artifacts.HasKeyMatch(TargetPlatform, GenerationHelper.GetReference()))
		{
			SetIncrementallyModified(PlatformIndex, PackagePlatformData, bPreviouslyCooked,
				Artifacts.IsValid() ? EIncrementallyModifiedReason::TargetDomainKey
				: EIncrementallyModifiedReason::NotPreviouslyCooked,
				 SetModifiedArgs);
			continue;
		}

		// Generated packages of a generator that is not incrementally enabled are also not incrementally enabled, even
		// if they would otherwise qualify for incremental on their own. e.g. if worlds are incrementally disallowed,
		// then streamingobject generated packages of the world are also disallowed.
		FName PackageNameForIncrementalTest = ParentPackageData ? ParentPackageData->GetPackageName() : PackageName;
		bool bIncrementalCookEnabled = IsIncrementalCookEnabled(PackageNameForIncrementalTest,
			Cluster.COTFS.bCookIncrementalAllowAllClasses);
		
		if (!bIncrementalCookEnabled)
		{
			SetIncrementallyModified(PlatformIndex, PackagePlatformData, bPreviouslyCooked,
				EIncrementallyModifiedReason::IncrementalCookDisabled, SetModifiedArgs);
			continue;
		}

		if (!QueryPlatformData.bTransitiveBuildDependenciesResolvedAsNotModified)
		{
			FName ModifiedTransitiveBuildDependency;
			UnreadyTransitiveBuildVertices.Reset();
			TArray<FName, TInlineAllocator<10>> TransitiveBuildDependencies;
			Artifacts.GetTransitiveBuildDependencies(TransitiveBuildDependencies);
			for (FName TransitiveBuildPackageName : TransitiveBuildDependencies)
			{
				FVertexData& TransitiveBuildVertex = GraphSearch.Cluster.FindOrAddVertex(TransitiveBuildPackageName);
				if (!TransitiveBuildVertex.GetPackageData())
				{
					// A build dependency on a non-existent package can occur e.g. if the package is in an
					// unmounted plugin. If the package does not exist we count the transitivebuilddependency
					// as not incrementally unmodified, the same as any package that is not cooked, so mark this
					// package as not incrementally unmodified.
					// This is an unexpected data layout however, so log it as a warning.
					UE_LOGF(LogCook, Warning,
						"TransitiveBuildDependency to non-existent package." " Package %ls has a transitive build dependency on package %ls, which does not exist or is not mounted." " Package %ls will be marked as not incrementally skippable and will be recooked.",
						*Vertex->GetPackageName().ToString(), *TransitiveBuildPackageName.ToString(),
						*Vertex->GetPackageName().ToString());
					ModifiedTransitiveBuildDependency = TransitiveBuildPackageName;
					break;
				}

				FPackagePlatformData& TransitivePlatformData = TransitiveBuildVertex.GetPackageData()->FindOrAddPlatformData(TargetPlatform);
				if (!TransitivePlatformData.IsIncrementallyUnmodifiedSet())
				{
					UnreadyTransitiveBuildVertices.Add(&TransitiveBuildVertex);
					continue;
				}
				if (!TransitivePlatformData.IsIncrementallyUnmodified())
				{
					ModifiedTransitiveBuildDependency = TransitiveBuildPackageName;
					break;
				}
			}

			if (!ModifiedTransitiveBuildDependency.IsNone())
			{
				SetModifiedArgs.TransitiveBuildDependencyName = ModifiedTransitiveBuildDependency;
				SetIncrementallyModified(PlatformIndex, PackagePlatformData, bPreviouslyCooked,
					EIncrementallyModifiedReason::TransitiveBuildDependency, SetModifiedArgs);
				continue;
			}
			if (!UnreadyTransitiveBuildVertices.IsEmpty())
			{
				// Add this vertex as a listener to the TransitiveBuildVertices' TryCalculateIncrementallyUnmodified
				for (FVertexData* TransitiveBuildVertex : UnreadyTransitiveBuildVertices)
				{
					FQueryPlatformData& TransitivePlatformData = TransitiveBuildVertex->GetPlatformData()[PlatformIndex];

					// Do not kick the vertex again if it has already been fetched; doing so will create busy work
					// in the case of a cycle and prevent us from detecting the cycle.
					if (!TransitivePlatformData.bSchedulerThreadFetchCompleted)
					{
						TransitivePlatformData.bIncrementallyUnmodifiedRequested = true;
						GraphSearch.AddToVisitVertexQueue(*TransitiveBuildVertex);
					}
					// It's okay to add duplicates to IncrementallyModifiedListeners; we remove them when broadcasting
					TransitiveBuildVertex->GetIncrementallyModifiedListeners().Add(this->Vertex);
					Vertex->GetUnreadyDependencies().Add(TransitiveBuildVertex);
				}

				bAllPlatformsAreReady = false;
				continue;
			}
		}

		SetIncrementallyUnmodified(PlatformIndex, PackagePlatformData);
	}

	if (!bAllPlatformsAreReady)
	{
		GraphSearch.PendingTransitiveBuildDependencyVertices.Add(Vertex);
		Vertex->SetWaitingOnUnreadyDependencies(true);
		return false;
	}

	if (GenerationHelper && !PackageData->IsGenerated())
	{
		// TODO_COOKGENERATIONHELPER: We don't currently support separate incrementally skippable results per platform
		// for a generator, see the class comment on FGenerationHelper. Therefore if any platform was found to be
		// incrementally modified, then set all platforms to modified.
		bool bAllUnmodified = true;
		for (int32 PlatformIndex : PlatformsToProcess)
		{
			if (PlatformIndex == CookerLoadingPlatformIndex)
			{
				continue;
			}
			const ITargetPlatform* TargetPlatform = GraphSearch.FetchPlatforms[PlatformIndex].Platform;
			FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
			check(PackagePlatformData.IsIncrementallyUnmodifiedSet()); // Otherwise we early exited in !bAllPlatformsAreReady
			bAllUnmodified &= PackagePlatformData.IsIncrementallyUnmodified();
		}
		if (!bAllUnmodified)
		{
			for (int32 PlatformIndex : PlatformsToProcess)
			{
				if (PlatformIndex == CookerLoadingPlatformIndex)
				{
					continue;
				}
				FFetchPlatformData& FetchPlatformData = GraphSearch.FetchPlatforms[PlatformIndex];
				const ITargetPlatform* TargetPlatform = FetchPlatformData.Platform;
				FQueryPlatformData& QueryPlatformData = Vertex->GetPlatformData()[PlatformIndex];
				bool bPreviouslyCooked =
					(QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Success)
					| (QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Error);

				FIncrementallyModifiedContext SetModifiedArgs;
				SetModifiedArgs.Artifacts = &QueryPlatformData.CookAttachments.Artifacts;
				SetModifiedArgs.GenerationHelper = GenerationHelper.GetReference();
				SetModifiedArgs.TargetPlatform = TargetPlatform;

				FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
				SetIncrementallyModified(PlatformIndex, PackagePlatformData, bPreviouslyCooked,
					EIncrementallyModifiedReason::GeneratorOtherPlatform, SetModifiedArgs);
				PackagePlatformData.SetIncrementallyUnmodified(false);
			}
		}
	}

	Vertex->GetUnreadyDependencies().Empty(); // Already reset above, call empty to free the memory
	TArray<FVertexData*>& Listeners = Vertex->GetIncrementallyModifiedListeners();
	if (!Listeners.IsEmpty())
	{
		bool bAnyPlatformIncrementallyModified = false;
		for (int32 PlatformIndex = 0; PlatformIndex < LocalNumFetchPlatforms; ++PlatformIndex)
		{
			if (PlatformIndex == PlatformAgnosticPlatformIndex)
			{
				continue;
			}
			const ITargetPlatform* TargetPlatform = GraphSearch.FetchPlatforms[PlatformIndex].Platform;
			FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
			if (PackagePlatformData.IsIncrementallyUnmodifiedSet() && !PackagePlatformData.IsIncrementallyUnmodified())
			{
				bAnyPlatformIncrementallyModified = true;
				break;
			}
		}
		Algo::Sort(Listeners);
		Listeners.SetNum(Algo::Unique(Listeners));
		for (FVertexData* ListenerVertex : Listeners)
		{
			if (!ListenerVertex->IsWaitingOnUnreadyDependencies())
			{
				continue;
			}

			ListenerVertex->GetUnreadyDependencies().Remove(Vertex);
			if (bAnyPlatformIncrementallyModified || ListenerVertex->GetUnreadyDependencies().IsEmpty())
			{
				ListenerVertex->SetWaitingOnUnreadyDependencies(false);
				GraphSearch.KickVertex(ListenerVertex);
			}
		}
		Listeners.Empty();
	}
	return true;
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::CalculatePackageDataDependenciesPlatformAgnostic()
{
	using namespace UE::AssetRegistry;

	if (!bFetchAnyTargetPlatform || !Cluster.TraversalMarkCookable())
	{
		return;
	}

	EDependencyQuery FlagsForHardDependencyQuery;
	if (Cluster.COTFS.bSkipOnlyEditorOnly)
	{
		Cluster.AssetRegistry.GetDependencies(PackageName, HardGameDependencies, EDependencyCategory::Package,
			EDependencyQuery::Game | EDependencyQuery::Hard);
		HardDependenciesSet.Append(HardGameDependencies);
	}
	else
	{
		// We're not allowed to skip editoronly imports, so include all hard dependencies
		FlagsForHardDependencyQuery = EDependencyQuery::Hard;
		Cluster.AssetRegistry.GetDependencies(PackageName, HardGameDependencies, EDependencyCategory::Package,
			EDependencyQuery::Game | EDependencyQuery::Hard);
		Cluster.AssetRegistry.GetDependencies(PackageName, HardEditorDependencies, EDependencyCategory::Package,
			EDependencyQuery::EditorOnly | EDependencyQuery::Hard);
		HardDependenciesSet.Append(HardGameDependencies);
		HardDependenciesSet.Append(HardEditorDependencies);
	}
	if (Cluster.bAllowSoftDependencies)
	{
		// bSkipOnlyEditorOnly is always true for soft dependencies; skip editoronly soft dependencies
		Cluster.AssetRegistry.GetDependencies(PackageName, SoftGameDependencies, EDependencyCategory::Package,
			EDependencyQuery::Game | EDependencyQuery::Soft);

		// Even if we're following soft references in general, we need to check with the SoftObjectPath registry
		// for any startup packages that marked their softobjectpaths as excluded, and not follow those
		if (GRedirectCollector.RemoveAndCopySoftObjectPathExclusions(PackageName, SkippedPackages))
		{
			SoftGameDependencies.RemoveAll([this](FName SoftDependency)
				{
					return SkippedPackages.Contains(SoftDependency);
				});
		}

		// LocalizationReferences are a source of SoftGameDependencies that are not present in the AssetRegistry
		SoftGameDependencies.Append(GetLocalizationReferences(PackageName, Cluster.COTFS));

		// The AssetManager can provide additional SoftGameDependencies
		SoftGameDependencies.Append(GetAssetManagerReferences(PackageName));
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::CalculateDependenciesAndIncrementallySkippable()
{
	using namespace UE::AssetRegistry;

	for (int32 PlatformIndex : PlatformsToExplore)
	{
		FQueryPlatformData& QueryPlatformData = Vertex->GetPlatformData()[PlatformIndex];
		FFetchPlatformData& FetchPlatformData = GraphSearch.FetchPlatforms[PlatformIndex];
		const ITargetPlatform* TargetPlatform = FetchPlatformData.Platform;
		FPackagePlatformData& PackagePlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
		if (!GraphSearch.Cluster.TraversalExploreRuntimeDependencies() || !PackagePlatformData.IsExplorable())
		{
			// ExploreVertexEdges is responsible for updating package modification status so we might
			// have been called for this platform even if not explorable. If not explorable, just update
			// package modification status for the given platform, except for CookerLoadingPlatformIndex which has
			// no status to update.
			if (PlatformIndex != CookerLoadingPlatformIndex)
			{
				ProcessPlatformAttachments(PlatformIndex, TargetPlatform, FetchPlatformData, PackagePlatformData,
					QueryPlatformData.CookAttachments, false /* bExploreRuntimeDependencies */);
			}
			continue;
		}

		if (PlatformIndex == CookerLoadingPlatformIndex)
		{
			Cluster.AssetRegistry.GetDependencies(PackageName, CookerLoadingDependencies, EDependencyCategory::Package,
				EDependencyQuery::Hard);

			// INCREMENTALCOOK_TODO: To improve cooker load performance, we should declare EDependencyQuery::Build
			// packages as packages that will be loaded during the cook, by adding them as edges for the
			// CookerLoadingPlatformIndex platform.
			// But we can't do that yet; in some important cases the build dependencies are declared by a class but not
			// always used - some build dependencies might be a conservative list but unused by the asset, or unused on
			// targetplatform.
			// Adding BuildDependencies also sets up many circular dependencies, because maps declare their external
			// actors as build dependencies and the external actors declare the map as a build or hard dependency.
			// Topological sort done at the end of the Cluster has poor performance when there are 100k+ circular
			// dependencies.
			constexpr bool bAddBuildDependenciesToGraph = false;
			if (bAddBuildDependenciesToGraph)
			{
				Cluster.AssetRegistry.GetDependencies(PackageName, CookerLoadingDependencies,
					EDependencyCategory::Package, EDependencyQuery::Build);
			}
			// CookerLoadingPlatform does not cause SetInstigator so it does not modify the platformdependency's
			// InstigatorType
			AddPlatformDependencyRange(CookerLoadingDependencies, PlatformIndex, EInstigator::InvalidCategory);
		}
		else
		{
			AddPlatformDependencyRange(HardGameDependencies, PlatformIndex, EInstigator::HardDependency);
			AddPlatformDependencyRange(HardEditorDependencies, PlatformIndex, EInstigator::HardEditorOnlyDependency);
			AddPlatformDependencyRange(SoftGameDependencies, PlatformIndex, EInstigator::SoftDependency);
			ProcessPlatformDiscoveredDependencies(PlatformIndex, TargetPlatform);
			ProcessPlatformAttachments(PlatformIndex, TargetPlatform, FetchPlatformData, PackagePlatformData,
				QueryPlatformData.CookAttachments, true /* bExploreRuntimeDependencies  */);
		}
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::ProcessPlatformDiscoveredDependencies(int32 PlatformIndex,
	const ITargetPlatform* TargetPlatform)
{
	// nullptr in PackageData->GetDiscoveredDependencies means PlatformAgnostic; this function is for
	// the platformspecific, so TargetPlatform must not be null.
	check(TargetPlatform != nullptr);
	const TMap<FPackageData*, EInstigator>* PlatformDependencies
		= PackageData->GetDiscoveredDependencies(TargetPlatform);
	const TMap<FPackageData*, EInstigator>* AgnosticDependencies
		= PackageData->GetDiscoveredDependencies(nullptr);
	for (const TMap<FPackageData*, EInstigator>* DependenciesMap : { AgnosticDependencies, PlatformDependencies })
	{
		if (DependenciesMap)
		{
			for (const TPair<FPackageData*, EInstigator>& PackagePair : *DependenciesMap)
			{
				// DiscoveredDependencies are always treated as Soft, but might also have the
				// ForceExplorableSaveTimeSoftDependency property, which sets the explorable property on the target
				// if it doesn't already have it.
				EInstigator EdgeType = PackagePair.Value == EInstigator::ForceExplorableSaveTimeSoftDependency ?
					EInstigator::ForceExplorableSaveTimeSoftDependency : EInstigator::SoftDependency;
				AddPlatformDependency(PackagePair.Key->GetPackageName(), PlatformIndex, EdgeType);
			}
		}
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::QueueVisitsOfDependencies()
{
	if (PlatformDependencyMap.IsEmpty())
	{
		return;
	}

	const EReachability ClusterReachability = Cluster.TraversalMarkCookable()
		? EReachability::Runtime : EReachability::Build;
	TArray<FPackageData*>* Edges = nullptr;
	TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
	for (TPair<FName, FScratchPlatformDependencyBits>& PlatformDependencyPair : PlatformDependencyMap)
	{
		FName DependencyName = PlatformDependencyPair.Key;
		TBitArray<>& HasRuntimePlatformByIndex = PlatformDependencyPair.Value.HasRuntimePlatformByIndex;
		TBitArray<>& HasBuildPlatformByIndex = PlatformDependencyPair.Value.HasBuildPlatformByIndex;
		TBitArray<>& ForceExplorableByIndex = PlatformDependencyPair.Value.ForceExplorableByIndex;
		EInstigator InstigatorType = PlatformDependencyPair.Value.InstigatorType;
		EInstigator BuildInstigatorType = PlatformDependencyPair.Value.BuildInstigatorType;

		// Process any CoreRedirects before checking whether the package exists
		FName Redirected = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package,
			FCoreRedirectObjectName(NAME_None, NAME_None, DependencyName)).PackageName;
		DependencyName = Redirected;

		FVertexData& DependencyVertex = GraphSearch.Cluster.FindOrAddVertex(DependencyName,
			GenerationHelper.GetReference());
		if (!DependencyVertex.GetPackageData())
		{
			TStringBuilder<256> DependencyNameStr(InPlace, DependencyName);
			if (!FPackageName::IsScriptPackage(DependencyNameStr))
			{
				UE_LOGFMT(LogCook, Display, "Package {PackageName} has a dependency on package {MissingDependency} which does not exist.",
					PackageName.ToString(), *DependencyNameStr);
			}
			continue;
		}
		FPackageData& DependencyPackageData(*DependencyVertex.GetPackageData());
		bool bAddToVisitVertexQueue = false;

		for (int32 PlatformIndex = 0; PlatformIndex < LocalNumFetchPlatforms; ++PlatformIndex)
		{
			if (!HasRuntimePlatformByIndex[PlatformIndex] &&
				!HasBuildPlatformByIndex[PlatformIndex])
			{
				continue;
			}

			FFetchPlatformData& FetchPlatformData = GraphSearch.FetchPlatforms[PlatformIndex];
			const ITargetPlatform* TargetPlatform = FetchPlatformData.Platform;
			FPackagePlatformData& PlatformData = DependencyPackageData.FindOrAddPlatformData(TargetPlatform);

			if (HasRuntimePlatformByIndex[PlatformIndex] && ClusterReachability == EReachability::Runtime)
			{
				// RuntimeDependencies only cause edges and reachability in EReachability::Runtime clusters.

				if (TargetPlatform != CookerLoadingPlatformKey && ForceExplorableByIndex[PlatformIndex])
				{
					// This package was possibly previously marked as not explorable, but now the referencer wants to
					// mark it as explorable. One example of this is externalactor packages - they are by default not
					// cookable and not explorable (see comment in FRequestCluster::IsRequestCookable). But once
					// WorldPartition loads them, we need to mark them as explored so that their imports are marked as
					// expected and all of their soft dependencies are included.
					if (!PlatformData.IsExplorableOverride())
					{
						// MarkAsExplorable calls ResetReachable, so we need to process this before testing IsReachable
						// below.
						PlatformData.MarkAsExplorable();
					}
				}

				if (PlatformIndex == CookerLoadingPlatformIndex)
				{
					if (!Edges)
					{
						Edges = &GraphSearch.GraphEdges.FindOrAdd(PackageData);
						Edges->Reset(PlatformDependencyMap.Num());
					}
					Edges->Add(&DependencyPackageData);
				}

				if (!PlatformData.IsReachable(EReachability::Runtime))
				{
					PlatformData.AddReachability(EReachability::Runtime);
					if (InstigatorType != EInstigator::InvalidCategory
						&& !DependencyPackageData.HasInstigator(EReachability::Runtime)
						&& TargetPlatform != CookerLoadingPlatformKey)
					{
						DependencyPackageData.SetInstigator(Cluster, EReachability::Runtime,
							FInstigator(InstigatorType, PackageName));
					}
				}
				if (!PlatformData.IsVisitedByCluster(EReachability::Runtime))
				{
					bAddToVisitVertexQueue = true;
				}
			}
			if (HasBuildPlatformByIndex[PlatformIndex] && PlatformIndex != CookerLoadingPlatformIndex)
			{
				// CookerLoadingPlatform does not cause BuildDependencies.

				// BuildDependencies from session platforms set the packages build reachability no matter what kind of
				// cluster we're in, but they only get added to cluster if we're in a EReachability::Build cluster,
				// Otherwise they will need to get picked up later by UCookOnTheFlyServer::KickBuildDependencies.
				if (!PlatformData.IsReachable(EReachability::Build))
				{
					PlatformData.AddReachability(EReachability::Build);
					if (BuildInstigatorType != EInstigator::InvalidCategory
						&& !DependencyPackageData.HasInstigator(EReachability::Build))
					{
						DependencyPackageData.SetInstigator(Cluster, EReachability::Build,
							FInstigator(BuildInstigatorType, PackageName));
					}
				}
				if (ClusterReachability == EReachability::Build)
				{
					// Being Reachable for EReachability::Build does not necessarily mean that it needs to be visited by
					// a cluster, as it does for EReachability::Runtime. We only need to visit BuildDependencies that 
					// were not committed
					if (!PlatformData.IsCommitted() && !PlatformData.IsVisitedByCluster(EReachability::Build))
					{
						bAddToVisitVertexQueue = true;
					}
				}
			}
		}
		if (bAddToVisitVertexQueue)
		{
			// Only pull the vertex into the cluster if it has not already been pulled into the cluster.
			// This prevents us from trying to readd a packagedata after COTFS called Cluster->RemovePackageData.
			if (!DependencyVertex.HasBeenPulledIntoCluster())
			{
				Cluster.SetOwnedByCluster(DependencyVertex, true);
			}
			GraphSearch.AddToVisitVertexQueue(DependencyVertex);
		}
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::MarkExploreComplete()
{
	for (int32 PlatformIndex : PlatformsToExplore)
	{
		Vertex->GetPlatformData()[PlatformIndex].bExploreCompleted = true;
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::AddPlatformDependency(FName DependencyName,
	int32 PlatformIndex, EInstigator InstigatorType)
{
	FScratchPlatformDependencyBits& PlatformDependencyBits = PlatformDependencyMap.FindOrAdd(DependencyName);
	if (PlatformDependencyBits.HasRuntimePlatformByIndex.Num() != LocalNumFetchPlatforms)
	{
		PlatformDependencyBits.HasRuntimePlatformByIndex.Init(false, LocalNumFetchPlatforms);
		PlatformDependencyBits.HasBuildPlatformByIndex.Init(false, LocalNumFetchPlatforms);
		PlatformDependencyBits.ForceExplorableByIndex.Init(false, LocalNumFetchPlatforms);
		PlatformDependencyBits.InstigatorType = EInstigator::InvalidCategory;
		PlatformDependencyBits.BuildInstigatorType = EInstigator::InvalidCategory;
	}

	if (InstigatorType != EInstigator::BuildDependency)
	{
		PlatformDependencyBits.HasRuntimePlatformByIndex[PlatformIndex] = true;

		// For runtime dependencies, calculate PlatformDependencyType.InstigatorType == 
		// Max(InstigatorType, PlatformDependencyType.InstigatorType)
		// based on the enum values, from least required to most: [ Soft, HardEditorOnly, Hard ]
		switch (InstigatorType)
		{
		case EInstigator::HardDependency:
			PlatformDependencyBits.InstigatorType = InstigatorType;
			break;
		case EInstigator::HardEditorOnlyDependency:
			if (PlatformDependencyBits.InstigatorType != EInstigator::HardDependency)
			{
				PlatformDependencyBits.InstigatorType = InstigatorType;
			}
			break;
		case EInstigator::SoftDependency: [[fallthrough]];
		case EInstigator::ForceExplorableSaveTimeSoftDependency:
			if (PlatformDependencyBits.InstigatorType != EInstigator::HardDependency
				&& PlatformDependencyBits.InstigatorType != EInstigator::HardEditorOnlyDependency)
			{
				PlatformDependencyBits.InstigatorType = InstigatorType;
			}
			if (InstigatorType == EInstigator::ForceExplorableSaveTimeSoftDependency)
			{
				PlatformDependencyBits.ForceExplorableByIndex[PlatformIndex] = true;
			}
			break;
		case EInstigator::InvalidCategory:
			// Caller indicated they do not want to set the InstigatorType
			break;
		default:
			checkNoEntry();
			break;
		}
	}
	else
	{
		PlatformDependencyBits.HasBuildPlatformByIndex[PlatformIndex] = true;

		// For build dependencies, there is only one instigator type so just set it.
		PlatformDependencyBits.BuildInstigatorType = InstigatorType;
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::AddPlatformDependencyRange(TConstArrayView<FName> Range,
	int32 PlatformIndex, EInstigator InstigatorType)
{
	for (FName DependencyName : Range)
	{
		AddPlatformDependency(DependencyName, PlatformIndex, InstigatorType);
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::ProcessPlatformAttachments(int32 PlatformIndex,
	const ITargetPlatform* TargetPlatform, FFetchPlatformData& FetchPlatformData,
	FPackagePlatformData& PackagePlatformData, FIncrementalCookAttachments& PlatformAttachments,
	bool bExploreRuntimeDependencies)
{
	bool bFoundBuildDefinitions = false;
	ICookedPackageWriter* PackageWriter = FetchPlatformData.Writer;
	FQueryPlatformData& QueryPlatformData = Vertex->GetPlatformData()[PlatformIndex];
	bool bReportedInstigator = false;

	if (Cluster.IsIncrementalCook())
	{
		check(PackagePlatformData.IsIncrementallyUnmodifiedSet());
		bool bIncrementallyUnmodified = PackagePlatformData.IsIncrementallyUnmodified();
		bool bShouldIncrementallySkip = bIncrementallyUnmodified;
		ECookResult CookResult =
			QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Success
			? ECookResult::Succeeded : ECookResult::Failed;
		bool bPreviouslyCooked =
			(QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Success)
			| (QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Error);
		if (QueryPlatformData.CookAttachments.CommitStatus == IPackageWriter::ECommitStatus::Error)
		{
			// Recook packages with errors even if they have not changed, so that the error is not hidden from
			// the output for the incremental cook.
			bShouldIncrementallySkip = false;

			// INCREMENTALCOOK_TODO: Need to mark a generator package as not incrementally skippable if any of its
			// generated packages had errors. This will require waiting on the generated package results to be fetched,
			// and will require care to avoid causing a softlock in the presence of a transitive build dependency
			// cycle.
		}
		if (!QueryPlatformData.CookAttachments.Artifacts.HasSaveResults() && Cluster.TraversalMarkCookable())
		{
			// The package was previously committed only as a build dependency, with no attempt made to save its data
			// because it was not marked reachable in the previous cook. But during this cook, it is marked reachable
			// so we need to know the results of its save attempt. Therefore we need to cook it and it is not
			// incrementally skippable.
			bShouldIncrementallySkip = false;
		}

		if (Cluster.TraversalMarkCookable())
		{
			ICookedPackageWriter::FUpdatePackageModifiedStatusContext Context;
			Context.PackageName = PackageName;
			check(!bIncrementallyUnmodified || !QueryPlatformData.bIncrementallyModifiedInstigated);
			Context.bIncrementallyUnmodified = bIncrementallyUnmodified;
			Context.bPreviouslyCooked = bPreviouslyCooked;
			Context.bInOutShouldIncrementallySkip = bShouldIncrementallySkip;
			PackageWriter->UpdatePackageModifiedStatus(Context);
			bShouldIncrementallySkip = Context.bInOutShouldIncrementallySkip;		
		}

		TRefCountPtr<FGenerationHelper> ParentGenerationHelper;
		if (PackageData->IsGenerated())
		{
			// If a GeneratorPackage is incrementally skipped, its generated packages must be incrementally skipped as well
			FPackageData* ParentPackage = Cluster.PackageDatas.FindPackageDataByPackageName(PackageData->GetParentGenerator());
			if (ParentPackage)
			{
				ParentGenerationHelper = ParentPackage->GetGenerationHelper();
				const FPackagePlatformData* ParentPlatformData = ParentPackage->GetPlatformDatas().Find(TargetPlatform);
				if (ParentPlatformData && ParentPlatformData->IsIncrementallySkipped())
				{
					bShouldIncrementallySkip = true;
				}
			}
		}

		if (bShouldIncrementallySkip)
		{
			// If we incrementally skip a package and we have its previously stored dependencies, try to queue those
			// dependencies.
			UE::Cook::FPackageArtifacts& Artifacts = PlatformAttachments.Artifacts;
			if (Artifacts.IsValid())
			{
				// Queue runtime dependencies if we are exploring runtime dependencies, build definitions if
				// we are queuing those, and always queue build dependencies
				if (bExploreRuntimeDependencies && PackagePlatformData.IsCookable() && Cluster.bAllowSoftDependencies)
				{
					TArray<FName, TInlineAllocator<16>> RuntimeContentDependencies;
					Artifacts.GetRuntimeContentDependencies(RuntimeContentDependencies);
					AddPlatformDependencyRange(RuntimeContentDependencies, PlatformIndex, EInstigator::SoftDependency);
				}

				if (Cluster.bPreQueueBuildDefinitions && PackagePlatformData.IsCookable())
				{
					bFoundBuildDefinitions = true;
					Cluster.BuildDefinitions.AddBuildDefinitionList(PackageName, TargetPlatform,
						PlatformAttachments.BuildDefinitions.Definitions);
				}

				TArray<FName, TInlineAllocator<10>> TransitiveBuildDependencies;
				Artifacts.GetTransitiveBuildDependencies(TransitiveBuildDependencies);
				for (FName TransitivePackageName : TransitiveBuildDependencies)
				{
					AddPlatformDependency(TransitivePackageName, PlatformIndex, EInstigator::BuildDependency);
				}
			}

			if (Cluster.TraversalMarkCookable())
			{
				// Call SetPlatformCooked instead of just PackagePlatformData.SetCookResults because we might also need
				// to set OnFirstCookedPlatformAdded
				PackageData->SetPlatformCooked(TargetPlatform, CookResult);
				PackagePlatformData.SetIncrementallySkipped(true);
				if (TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper())
				{
					GenerationHelper->MarkPackageIncrementallySkipped(*PackageData, TargetPlatform,
						true /* bIncrementallySkipped */);
				}
				if (ParentGenerationHelper)
				{
					ParentGenerationHelper->MarkPackageIncrementallySkipped(*PackageData, TargetPlatform,
						true /* bIncrementallySkipped */);
				}
				if (PlatformIndex == FirstSessionPlatformIndex && CookResult == ECookResult::Succeeded)
				{
					COOK_STAT(++DetailedCookStats::NumPackagesIncrementallySkipped);
				}
				if ((GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators) && !bReportedInstigator)
				{
					bReportedInstigator = true;
					UE_LOGF(LogCook, Display, "Incrementally Skipped %ls, Instigator: { %ls }",
						*WriteToString<256>(PackageData->GetPackageName()),
						*PackageData->GetInstigator(EReachability::Runtime).ToString());
				}

				// Replay the package's saved data into our collectors of packagedata
				FEDLCookCheckerThreadState::Get().Add(QueryPlatformData.CookAttachments.ImportsCheckerData,
					PackageData->GetPackageName());
				TConstArrayView<FReplicatedLogData> LogMessages = QueryPlatformData.CookAttachments.LogMessages;
				if (!LogMessages.IsEmpty() && !PackageData->HasReplayedLogMessages())
				{
					Cluster.COTFS.LogHandler->ReplayLogsFromIncrementallySkipped(PackageData->GetPackageName(),
						LogMessages);
					PackageData->SetHasReplayedLogMessages(true);
				}
			}
			else
			{
				// Mark the package as already committed.
				PackagePlatformData.SetCommitted(true);
			}

			Cluster.SetWasMarkedSkipped(*Vertex, true);
		}
		else
		{
			if (Cluster.TraversalMarkCookable())
			{
				if (TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper())
				{
					GenerationHelper->MarkPackageIncrementallySkipped(*PackageData, TargetPlatform,
						false /* bIncrementallySkipped */);
				}
				if (ParentGenerationHelper)
				{
					ParentGenerationHelper->MarkPackageIncrementallySkipped(*PackageData, TargetPlatform,
						false /* bIncrementallySkipped */);
				}
			}
		}
	}

	if (Cluster.bPreQueueBuildDefinitions && !bFoundBuildDefinitions)
	{
		FQueryPlatformData& PlatformAgnosticQueryData = Vertex->GetPlatformData()[PlatformAgnosticPlatformIndex];

		if (PlatformAgnosticQueryData.bSchedulerThreadFetchCompleted
			&& PlatformAgnosticQueryData.CookAttachments.Artifacts.IsValid())
		{
			const FAssetPackageData* OverrideAssetPackageData = nullptr;
			TRefCountPtr<FGenerationHelper> GenerationHelper;
			if (!PackageData->IsGenerated())
			{
				GenerationHelper = PackageData->GetGenerationHelper();
			}
			else
			{
				GenerationHelper = Vertex->IsOwnedByCluster() ? PackageData->GetOrFindParentGenerationHelper()
					: PackageData->GetOrFindParentGenerationHelperNoCache();
			}
			if (PlatformAgnosticQueryData.CookAttachments.Artifacts.HasKeyMatch(nullptr, GenerationHelper.GetReference()))
			{
				Cluster.BuildDefinitions.AddBuildDefinitionList(PackageName, TargetPlatform,
					PlatformAgnosticQueryData.CookAttachments.BuildDefinitions.Definitions);
			}
		}
	}
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::SetIncrementallyUnmodified(int32 PlatformIndex,
	FPackagePlatformData& PackagePlatformData)
{
	PackagePlatformData.SetIncrementallyUnmodified(true);
}

void FRequestCluster::FGraphSearch::FExploreEdgesContext::SetIncrementallyModified(int32 PlatformIndex,
	FPackagePlatformData& PackagePlatformData, bool bPreviouslyCooked, EIncrementallyModifiedReason Reason,
	FIncrementallyModifiedContext& Context)
{
	FQueryPlatformData& ClusterPlatformData = Vertex->GetPlatformData()[PlatformIndex];
	if (!PackagePlatformData.IsIncrementallyUnmodifiedSet())
	{
		ClusterPlatformData.bIncrementallyModifiedInstigated = true;
		PackagePlatformData.SetIncrementallyUnmodified(false);

#if ENABLE_COOK_STATS
		if (PlatformIndex == FirstSessionPlatformIndex && bPreviouslyCooked && Cluster.TraversalMarkCookable())
		{
			FTopLevelAssetPath AssetClassPath;
			FARFilter PackageNameFilter;
			PackageNameFilter.PackageNames.Add(PackageData->GetPackageName());
			Cluster.AssetRegistry.EnumerateAssets(PackageNameFilter, [&AssetClassPath](const FAssetData& AssetData)
				{
					AssetClassPath = AssetData.AssetClassPath;
					return false;
				});

			++DetailedCookStats::NumPackagesIncrementallyModifiedByClass.FindOrAdd(AssetClassPath, 0);

			if (Cluster.COTFS.IncrementallyModifiedDiagnostics)
			{
				Context.PackageData = PackageData;
				Context.Reason = Reason;
				Context.AssetClass = AssetClassPath;
				Cluster.COTFS.IncrementallyModifiedDiagnostics->AddPackage(Context);
			}
		}
#endif
	}
}

FRequestCluster::FVertexData* FRequestCluster::AllocateVertex(FName PackageName, FPackageData* PackageData)
{
	// TODO: Change TypeBlockedAllocator to have an optional Size and Align argument,
	// and use it to allocate the FVertexData's array of PlatformData, to reduce cpu time of allocating the array.
	return VertexAllocator.NewElement(PackageName, PackageData, GetNumFetchPlatforms());
}

FRequestCluster::FVertexData::FVertexData(FName InPackageName, UE::Cook::FPackageData* InPackageData,
	int32 NumFetchPlatforms)
	: PackageName(InPackageName)
	, PackageData(InPackageData)
{
	PlatformData.SetNum(NumFetchPlatforms);
}

FRequestCluster::FVertexData&
FRequestCluster::FindOrAddVertex(FName PackageName, FGenerationHelper* ParentGenerationHelper)
{
	// Only called from process thread
	FVertexData*& ExistingVertex = ClusterPackages.FindOrAdd(PackageName);
	if (ExistingVertex)
	{
		return *ExistingVertex;
	}

	FPackageData* PackageData = nullptr;
	TStringBuilder<256> NameBuffer;
	PackageName.ToString(NameBuffer);
	if (!FPackageName::IsScriptPackage(NameBuffer))
	{
		PackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(PackageName);
		if (!PackageData && ParentGenerationHelper && ICookPackageSplitter::IsUnderGeneratedPackageSubPath(NameBuffer))
		{
			// Look up the AssetPackageData for the generated package, from any previously recorded platform;
			// we just need to know whether it was a .map or .uasset, which should be the same per platform.
			const FAssetPackageData* PreviousPackageData = ParentGenerationHelper->GetAssetPackageDataAnyPlatform(PackageName);
			if (PreviousPackageData)
			{
				bool bIsMap = PreviousPackageData->Extension == EPackageExtension::Map;
				PackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(PackageName,
					false /* bRequireExists */, bIsMap);
				if (PackageData)
				{
					PackageData->SetGenerated(ParentGenerationHelper->GetOwner().GetPackageName());
				}
			}
		}
	}

	ExistingVertex = AllocateVertex(PackageName, PackageData);
	return *ExistingVertex;
}

FRequestCluster::FVertexData&
FRequestCluster::FindOrAddVertex(FPackageData& PackageData)
{
	// Only called from process thread
	FName PackageName = PackageData.GetPackageName();
	FVertexData*& ExistingVertex = ClusterPackages.FindOrAdd(PackageName);
	if (ExistingVertex)
	{
		check(!ExistingVertex->GetPackageData() || ExistingVertex->GetPackageData() == &PackageData);
		return *ExistingVertex;
	}

	ExistingVertex = AllocateVertex(PackageName, &PackageData);
	return *ExistingVertex;
}

void FRequestCluster::FGraphSearch::AddToVisitVertexQueue(FVertexData& Vertex)
{
	VisitVertexQueue.Add(&Vertex);
}

void FRequestCluster::FGraphSearch::CreateAvailableBatches(bool bAllowIncompleteBatch)
{
	constexpr int32 BatchSize = 1000;
	if (PreAsyncQueue.IsEmpty() || (!bAllowIncompleteBatch && PreAsyncQueue.Num() < BatchSize))
	{
		return;
	}

	TArray<FQueryVertexBatch*> NewBatches;
	NewBatches.Reserve((PreAsyncQueue.Num() + BatchSize - 1) / BatchSize);
	{
		FScopeLock ScopeLock(&Lock);
		while (PreAsyncQueue.Num() >= BatchSize)
		{
			NewBatches.Add(CreateBatchOfPoppedVertices(BatchSize));
		}
		if (PreAsyncQueue.Num() > 0 && bAllowIncompleteBatch)
		{
			NewBatches.Add(CreateBatchOfPoppedVertices(PreAsyncQueue.Num()));
		}
	}
	for (FQueryVertexBatch* NewBatch : NewBatches)
	{
		NewBatch->Send();
	}
}

FRequestCluster::FQueryVertexBatch* FRequestCluster::FGraphSearch::AllocateBatch()
{
	// Called from inside this->Lock
	// BatchAllocator uses DeferredDestruction, so this might be a resused Batch, but we don't need to Reset it during
	// allocation because Batches are Reset during Free.
	return BatchAllocator.NewElement(*this);
}

void FRequestCluster::FGraphSearch::FreeBatch(FQueryVertexBatch* Batch)
{
	// Called from inside this->Lock
	Batch->Reset();
	BatchAllocator.Free(Batch);
}

FRequestCluster::FQueryVertexBatch* FRequestCluster::FGraphSearch::CreateBatchOfPoppedVertices(int32 BatchSize)
{
	// Called from inside this->Lock
	check(BatchSize <= PreAsyncQueue.Num());
	FQueryVertexBatch* BatchData = AllocateBatch();
	BatchData->Vertices.Reserve(BatchSize);
	for (int32 BatchIndex = 0; BatchIndex < BatchSize; ++BatchIndex)
	{
		FVertexData* Vertex = PreAsyncQueue.PopFrontValue();
		FVertexData*& ExistingVert = BatchData->Vertices.FindOrAdd(Vertex->GetPackageName());
		// Each PackageName should be used by just a single vertex.
		check(!ExistingVert || ExistingVert == Vertex);
		// If the vertex was already previously added to the batch that's okay, just ignore the new add.
		// A batch size of 0 is a problem but that can't happen just because a vertex is in the batch twice.
		// A batch size smaller than the expected `BatchSize` parameter is a minor performance issue but not a problem.
		ExistingVert = Vertex;
	}
	AsyncQueueBatches.Add(BatchData);
	return BatchData;
}

void FRequestCluster::FGraphSearch::OnBatchCompleted(FQueryVertexBatch* Batch)
{
	FScopeLock ScopeLock(&Lock);
	AsyncQueueBatches.Remove(Batch);
	FreeBatch(Batch);
	AsyncResultsReadyEvent->Trigger();
}

void FRequestCluster::FGraphSearch::KickVertex(FVertexData* Vertex)
{
	// The trigger occurs outside of the lock, and might get clobbered and incorrectly ignored by a call from the
	// scheduler thread if the scheduler tried to pop the AsyncQueueResults and found it empty before KickVertex calls
	// Enqueue but then pauses and calls AsyncResultsReadyEvent->Reset after KicKVertex calls Trigger. This clobbering
	// will not cause a deadlock, because eventually DestroyBatch will be called which triggers it inside the lock. Doing
	// the per-vertex trigger outside the lock is good for performance.
	AsyncQueueResults.Enqueue(Vertex);
	AsyncResultsReadyEvent->Trigger();
}

FRequestCluster::FQueryVertexBatch::FQueryVertexBatch(FGraphSearch& InGraphSearch)
	: ThreadSafeOnlyVars(InGraphSearch)
{
	PlatformDatas.SetNum(InGraphSearch.FetchPlatforms.Num());
}

void FRequestCluster::FQueryVertexBatch::Reset()
{
	for (FPlatformData& PlatformData : PlatformDatas)
	{
		PlatformData.PackageIds.Reset();
	}
	Vertices.Reset();
}

void FRequestCluster::FQueryVertexBatch::Send()
{
	int32 NumAddedRequests = 0;
	for (const TPair<FName, FVertexData*>& Pair : Vertices)
	{
		FVertexData* Vertex = Pair.Value;
		bool bAnyRequested = false;
		bool bAllHaveAlreadyCompletedFetch = false;
		for (int32 PlatformIndex = 0; PlatformIndex < PlatformDatas.Num(); ++PlatformIndex)
		{
			// The platform data may have already been requested; request it only if current status is NotRequested
			FQueryPlatformData& PlatformData = Vertex->GetPlatformData()[PlatformIndex];
			if (!PlatformData.bSchedulerThreadFetchCompleted)
			{
				bAllHaveAlreadyCompletedFetch = false;
				EAsyncQueryStatus ExpectedStatus = EAsyncQueryStatus::SchedulerRequested;
				if (PlatformData.CompareExchangeAsyncQueryStatus(ExpectedStatus,
					EAsyncQueryStatus::AsyncRequested))
				{
					FFetchPlatformData& FetchPlatformData = ThreadSafeOnlyVars.FetchPlatforms[PlatformIndex];
					EWhereCooked WhereCooked = EWhereCooked::ThisSession;
					if (FetchPlatformData.Platform)
					{
						FPackagePlatformData& PackagePlatformData = Vertex->GetPackageData()->FindOrAddPlatformData(FetchPlatformData.Platform);
						WhereCooked = PackagePlatformData.GetWhereCooked();
					}

					PlatformDatas[PlatformIndex].PackageIds.Add(FPackageIncrementalCookId{ Pair.Key, WhereCooked });
					++NumAddedRequests;
				}
			}
		}
		if (bAllHaveAlreadyCompletedFetch)
		{
			// We are contractually obligated to kick the vertex. Normally we would call
			// FIncrementalCookAttachments::Fetch with it and would then kick the vertex in our callback. Also, it
			// might still be in the AsyncQueueResults for one of the platforms so it will be kicked by TickExplore
			// pulling it out of the AsyncQueueResults. But if all requested platforms already previously pulled it
			// out of AsyncQueueResults, then we need to kick it again.
			ThreadSafeOnlyVars.KickVertex(Vertex);
		}
	}
	if (NumAddedRequests == 0)
	{
		// We turned out not to need to send any from this batch. Report that the batch is complete.
		ThreadSafeOnlyVars.OnBatchCompleted(this);
		// *this is no longer accessible
		return;
	}

	NumPendingRequests.store(NumAddedRequests, std::memory_order_release);

	for (int32 PlatformIndex = 0; PlatformIndex < PlatformDatas.Num(); ++PlatformIndex)
	{
		FPlatformData& PlatformData = PlatformDatas[PlatformIndex];
		if (PlatformData.PackageIds.IsEmpty())
		{
			continue;
		}
		FFetchPlatformData& FetchPlatformData = ThreadSafeOnlyVars.FetchPlatforms[PlatformIndex];

		if (ThreadSafeOnlyVars.Cluster.IsIncrementalCook() // Only FetchCookAttachments if our cookmode supports it.
															// Otherwise keep them all empty
			&& !FetchPlatformData.bIsPlatformAgnosticPlatform // The PlatformAgnosticPlatform has no stored
															// CookAttachments; always use empty
			&& !FetchPlatformData.bIsCookerLoadingPlatform // The CookerLoadingPlatform has no stored CookAttachments;
															// always use empty
			)
		{
			TFunction<void(FName PackageName, FIncrementalCookAttachments&& Result)> Callback =
				[this, PlatformIndex](FName PackageName, FIncrementalCookAttachments&& Attachments)
			{
				RecordCacheResults(PackageName, PlatformIndex, MoveTemp(Attachments));
			};
			FIncrementalCookAttachments::Fetch(PlatformData.PackageIds, FetchPlatformData.Platform,
				FetchPlatformData.Writer, MoveTemp(Callback));
		}
		else
		{
			// When we do not need to asynchronously fetch, we record empty cache results to keep the edgefetch
			// flow similar to the FetchCookAttachments case

			// Don't use a ranged-for, as we are not allowed to access this or this->PackageNames after the
			// last index, and ranged-for != at the end of the final loop iteration can read from PackageNames
			int32 NumPackages = PlatformData.PackageIds.Num();
			FPackageIncrementalCookId* PackageIdsData = PlatformData.PackageIds.GetData();
			for (int32 PackageNameIndex = 0; PackageNameIndex < NumPackages; ++PackageNameIndex)
			{
				const FPackageIncrementalCookId& PackageId = PackageIdsData[PackageNameIndex];
				FIncrementalCookAttachments Attachments;
				RecordCacheResults(PackageId.PackageName, PlatformIndex, MoveTemp(Attachments));
			}
		}
	}
}

void FRequestCluster::FQueryVertexBatch::RecordCacheResults(FName PackageName, int32 PlatformIndex,
	FIncrementalCookAttachments&& CookAttachments)
{
	FVertexData* Vertex = Vertices.FindChecked(PackageName);
	FQueryPlatformData& PlatformData = Vertex->GetPlatformData()[PlatformIndex];
	PlatformData.CookAttachments = MoveTemp(CookAttachments);

	EAsyncQueryStatus Expected = EAsyncQueryStatus::AsyncRequested;
	if (PlatformData.CompareExchangeAsyncQueryStatus(Expected, EAsyncQueryStatus::Complete))
	{
		// Kick the vertex if it has no more platforms in pending. Otherwise keep waiting and the later
		// call to RecordCacheResults will kick the vertex. Note that the "later call" might be another
		// call to RecordCacheResults on a different thread executing at the same time, and we are racing.
		// The last one to set CompareExchangeAsyncQueryStatus(EAsyncQueryStatus::Complete) will definitely
		// see all other values as complete, because we are using std::memory_order_release. It is possible
		// that both calls to RecordCacheResults will see all values complete, and we will kick it twice.
		// Kicking twice is okay; it is supported and is a noop.
		bool bAllPlatformsComplete = true;
		int32 LocalNumFetchPlatforms = ThreadSafeOnlyVars.Cluster.GetNumFetchPlatforms();
		for (int32 OtherPlatformIndex = 0; OtherPlatformIndex < LocalNumFetchPlatforms; ++OtherPlatformIndex)
		{
			if (OtherPlatformIndex == PlatformIndex)
			{
				continue;
			}
			FQueryPlatformData& OtherPlatformData = Vertex->GetPlatformData()[OtherPlatformIndex];
			EAsyncQueryStatus OtherStatus = OtherPlatformData.GetAsyncQueryStatus();
			if (EAsyncQueryStatus::AsyncRequested <= OtherStatus && OtherStatus < EAsyncQueryStatus::Complete)
			{
				bAllPlatformsComplete = false;
				break;
			}
		}
		if (bAllPlatformsComplete)
		{
			ThreadSafeOnlyVars.KickVertex(Vertex);
		}
	}

	if (NumPendingRequests.fetch_sub(1, std::memory_order_relaxed) == 1)
	{
		ThreadSafeOnlyVars.OnBatchCompleted(this);
		// *this is no longer accessible
	}
}

TMap<FPackageData*, TArray<FPackageData*>>& FRequestCluster::FGraphSearch::GetGraphEdges()
{
	return GraphEdges;
}

bool FRequestCluster::IsIncrementalCook() const
{
	return bAllowIncrementalResults && !COTFS.bLegacyBuildDependencies;
}

void FRequestCluster::IsRequestCookable(const ITargetPlatform* Platform, const FPackageData& PackageData,
	UCookOnTheFlyServer& COTFS, ESuppressCookReason& OutReason, bool& bOutCookable, bool& bOutExplorable)
{
	FString LocalDLCPath;
	if (COTFS.CookByTheBookOptions->bErrorOnEngineContentUse)
	{
		LocalDLCPath = FPaths::Combine(*COTFS.GetBaseDirectoryForDLC(), TEXT("Content"));
		FPaths::MakeStandardFilename(LocalDLCPath);
	}

	IsRequestCookable(Platform, PackageData, COTFS, LocalDLCPath, OutReason, bOutCookable, bOutExplorable);
}

void FRequestCluster::IsRequestCookable(const ITargetPlatform* Platform, const FPackageData& PackageData,
	ESuppressCookReason& OutReason, bool& bOutCookable, bool& bOutExplorable)
{
	return IsRequestCookable(Platform, PackageData, COTFS,
		DLCPath, OutReason, bOutCookable, bOutExplorable);
}

void FRequestCluster::IsRequestCookable(const ITargetPlatform* Platform, const FPackageData& PackageData,
	UCookOnTheFlyServer& InCOTFS, FStringView InDLCPath, ESuppressCookReason& OutReason, bool& bOutCookable,
	bool& bOutExplorable)
{
	// IsRequestCookable should not be called for The CookerLoadingPlatform; it has different rules
	check(Platform != CookerLoadingPlatformKey);
	FName PackageName = PackageData.GetPackageName();

	TStringBuilder<256> NameBuffer;
	// We need to reject packagenames from adding themselves or their transitive dependencies using all the same rules
	// that UCookOnTheFlyServer::ProcessRequest uses. Packages that are rejected from cook do not add their 
	// dependencies to the cook.
	PackageName.ToString(NameBuffer);
	if (FPackageName::IsScriptPackage(NameBuffer))
	{
		OutReason = ESuppressCookReason::ScriptPackage;
		bOutCookable = false;
		bOutExplorable = false;
		return;
	}

	const FPackagePlatformData* PlatformData = PackageData.FindPlatformData(Platform);
	bool bExplorableOverride = PlatformData ? PlatformData->IsExplorableOverride() : false;
	ON_SCOPE_EXIT
	{
		bOutExplorable = bOutExplorable | bExplorableOverride;
	};

	FName FileName = PackageData.GetFileName();

	if (InCOTFS.CookByTheBookOptions->bDlcPluginOnly && !InDLCPath.IsEmpty())
	{
		FileName.ToString(NameBuffer);
		if (!FStringView(NameBuffer).StartsWith(InDLCPath))
		{
			OutReason = ESuppressCookReason::NotInCurrentPlugin;
			bOutCookable = false;
			bOutExplorable = false;
			return;
		}
	}

	if (InCOTFS.PackageTracker->NeverCookPackageList.Contains(PackageName))
	{
		if (INDEX_NONE != UE::String::FindFirst(NameBuffer, ULevel::GetExternalActorsFolderName(), 
			ESearchCase::IgnoreCase) ||
			INDEX_NONE != UE::String::FindFirst(NameBuffer, FPackagePath::GetExternalObjectsFolderName(),
			ESearchCase::IgnoreCase))
		{
			// EXTERNALACTOR_TODO: Add a separate category for ExternalActors rather than putting them in
			// NeverCookPackageList and checking naming convention here.
			OutReason = ESuppressCookReason::NeverCook;
			bOutCookable = false;

			// EXTERNALACTOR_TODO: We want to explore externalactors, because they add references to the cook that will
			// otherwise not be found until the map package loads them and adds them as unsolicited packages
			// But some externalactor packages will never be loaded by the generator, and we don't have a way to
			// discover which ones will not be loaded until we load the Map and WorldPartition object.
			// So set them to explorable = false until we implement an interface to determine which actors will be
			// loaded up front.
			bOutExplorable = false;
		}
		else
		{
			UE_LOGF(LogCook, Verbose,
				"Package %ls is referenced but is in the never cook package list, discarding request",
				*NameBuffer);
			OutReason = ESuppressCookReason::NeverCook;
			bOutCookable = false;
			bOutExplorable = false;
		}
		return;
	}

	if (InCOTFS.CookByTheBookOptions->bErrorOnEngineContentUse && !InDLCPath.IsEmpty())
	{
		FileName.ToString(NameBuffer);
		if (!FStringView(NameBuffer).StartsWith(InDLCPath))
		{
			// Editoronly content that was not cooked by the base game is allowed to be "cooked"; if it references
			// something not editoronly then we will exclude and give a warning on that followup asset. We need to
			// handle editoronly objects being referenced because the base game will not have marked them as cooked so
			// we will think we still need to "cook" them.
			// The only case where this comes up is in ObjectRedirectors, so we only test for those for performance.
			TArray<FAssetData> Assets;
			IAssetRegistry::GetChecked().GetAssetsByPackageName(PackageName, Assets,
				true /* bIncludeOnlyOnDiskAssets */);
			bool bEditorOnly = !Assets.IsEmpty() &&
				Algo::AllOf(Assets, [](const FAssetData& Asset)
					{
						return Asset.IsRedirector();
					});

			if (!bEditorOnly)
			{
				bool bIsReferenceAnError = true;
				const UPackage* UnrealPackage = FindPackage(NULL, *PackageData.GetPackageName().ToString());
				if (UnrealPackage && UnrealPackage->HasAnyPackageFlags(PKG_RuntimeGenerated))
				{
					bIsReferenceAnError = false;
				}
				if (bIsReferenceAnError)
				{
					bIsReferenceAnError = !PackageData.HasCookedPlatform(Platform, true /* bIncludeFailed */) && !InCOTFS.CookByTheBookOptions->bAllowUncookedAssetReferences;
				}
				if (bIsReferenceAnError)
				{
					UE_LOGF(LogCook, Error, "Uncooked Engine or Game content %ls is being referenced by DLC!", *NameBuffer);
				}
				OutReason = ESuppressCookReason::NotInCurrentPlugin;
				bOutCookable = false;
				bOutExplorable = false;
				return;
			}
		}
	}

	// The package is ordinarily cookable and explorable. In some cases we filter out for testing
	// packages that are ordinarily cookable; set bOutCookable to false if so.
	bOutExplorable = true;
	if (InCOTFS.bCookFilter)
	{
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		FName PackageNameToTest = PackageName;
		if (PackageData.IsGenerated())
		{
			FName ParentName = PackageData.GetParentGenerator();
			FPackageData* ParentData = InCOTFS.PackageDatas->FindPackageDataByPackageName(ParentName);
			if (ParentData)
			{
				PackageNameToTest = ParentName;
			}
		}

		if (!InCOTFS.CookFilterIncludedClasses.IsEmpty())
		{
			TOptional<FAssetPackageData> AssetData = AssetRegistry.GetAssetPackageDataCopy(PackageNameToTest);
			bool bIncluded = false;
			if (AssetData)
			{
				for (FName ClassName : AssetData->ImportedClasses)
				{
					if (InCOTFS.CookFilterIncludedClasses.Contains(ClassName))
					{
						bIncluded = true;
						break;
					}
				}
			}
			if (!bIncluded)
			{
				OutReason = ESuppressCookReason::CookFilter;
				bOutCookable = false;
				return;
			}
		}
		if (!InCOTFS.CookFilterIncludedAssetClasses.IsEmpty())
		{
			TArray<FAssetData> AssetDatas;
			AssetRegistry.GetAssetsByPackageName(PackageNameToTest, AssetDatas, true /* bIncludeOnlyDiskAssets */);
			bool bIncluded = false;
			for (FAssetData& AssetData : AssetDatas)
			{
				if (InCOTFS.CookFilterIncludedAssetClasses.Contains(FName(*AssetData.AssetClassPath.ToString())))
				{
					bIncluded = true;
					break;
				}
			}
			if (!bIncluded)
			{
				OutReason = ESuppressCookReason::CookFilter;
				bOutCookable = false;
				return;
			}
		}
	}

	OutReason = ESuppressCookReason::NotSuppressed;
	bOutCookable = true;
}

TConstArrayView<FName> FRequestCluster::GetLocalizationReferences(FName PackageName, UCookOnTheFlyServer& InCOTFS)
{
	if (!FPackageName::IsLocalizedPackage(WriteToString<256>(PackageName)))
	{
		TArray<FName>* Result = InCOTFS.CookByTheBookOptions->SourceToLocalizedPackageVariants.Find(PackageName);
		if (Result)
		{
			return TConstArrayView<FName>(*Result);
		}
	}
	return TConstArrayView<FName>();
}

TArray<FName> FRequestCluster::GetAssetManagerReferences(FName PackageName)
{
	TArray<FName> Results;
	UAssetManager::Get().ModifyCookReferences(PackageName, Results);
	return Results;
}

template <typename T>
static void ArrayShuffle(TArray<T>& Array)
{
	// iterate 0 to N-1, picking a random remaining vertex each loop
	int32 N = Array.Num();
	for (int32 I = 0; I < N; ++I)
	{
		Array.Swap(I, FMath::RandRange(I, N - 1));
	}
}

template <typename T>
static TArray<T> FindRootsFromLeafToRootOrderList(TConstArrayView<T> LeafToRootOrder, const TMap<T, TArray<T>>& Edges,
	const TSet<T>& ValidVertices)
{
	// Iteratively
	//    1) Add the leading rootward non-visited element to the root
	//    2) Visit all elements reachable from that root
	// This works because the input array is already sorted RootToLeaf, so we
	// know the leading element has no incoming edges from anything later.
	TArray<T> Roots;
	TSet<T> Visited;
	Visited.Reserve(LeafToRootOrder.Num());
	struct FVisitEntry
	{
		T Vertex;
		const TArray<T>* Edges;
		int32 NextEdge;
		void Set(T V, const TMap<T, TArray<T>>& AllEdges)
		{
			Vertex = V;
			Edges = AllEdges.Find(V);
			NextEdge = 0;
		}
	};
	TArray<FVisitEntry> DFSStack;
	int32 StackNum = 0;
	auto Push = [&DFSStack, &Edges, &StackNum](T Vertex)
	{
		while (DFSStack.Num() <= StackNum)
		{
			DFSStack.Emplace();
		}
		DFSStack[StackNum++].Set(Vertex, Edges);
	};
	auto Pop = [&StackNum]()
	{
		--StackNum;
	};

	for (T Root : ReverseIterate(LeafToRootOrder))
	{
		bool bAlreadyExists;
		Visited.Add(Root, &bAlreadyExists);
		if (bAlreadyExists)
		{
			continue;
		}
		Roots.Add(Root);

		Push(Root);
		check(StackNum == 1);
		while (StackNum > 0)
		{
			FVisitEntry& Entry = DFSStack[StackNum - 1];
			bool bPushed = false;
			while (Entry.Edges && Entry.NextEdge < Entry.Edges->Num())
			{
				T Target = (*Entry.Edges)[Entry.NextEdge++];
				Visited.Add(Target, &bAlreadyExists);
				if (!bAlreadyExists && ValidVertices.Contains(Target))
				{
					Push(Target);
					bPushed = true;
					break;
				}
			}
			if (!bPushed)
			{
				Pop();
			}
		}
	}
	return Roots;
}

} // namespace UE::Cook
