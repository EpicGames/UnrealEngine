// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Event.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/MpscQueue.h"
#include "Containers/RingBuffer.h"
#include "Cooker/CookPackageArtifacts.h"
#include "Cooker/CookTypes.h"
#include "Cooker/TypedBlockAllocator.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"
#include "UObject/ICookInfo.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FAssetPackageData;
class IAssetRegistry;
class ICookedPackageWriter;
class ITargetPlatform;
class UCookOnTheFlyServer;
namespace UE::Cook { class FPackageArtifacts; }
namespace UE::Cook { class FRequestQueue; }
namespace UE::Cook { enum class EIncrementallyModifiedReason : uint8; }
namespace UE::Cook { enum class EReachability : uint8; }
namespace UE::Cook { struct FDiscoveryQueueElement; }
namespace UE::Cook { struct FFilePlatformRequest; }
namespace UE::Cook { struct FIncrementallyModifiedContext; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FPackageDatas; }
namespace UE::Cook { struct FPackagePlatformData; }
namespace UE::Cook { struct FPackageTracker; }

namespace UE::Cook
{

/**
 * A group of external requests sent to CookOnTheFlyServer's tick loop. Transitive dependencies are found and all of the
 * requested or dependent packagenames are added as requests together to the cooking state machine.
 */
class FRequestCluster
{
public:
	~FRequestCluster();
	FRequestCluster(UCookOnTheFlyServer& COTFS, TArray<FFilePlatformRequest>&& InRequests);
	FRequestCluster(UCookOnTheFlyServer& COTFS, TPackageDataMap<ESuppressCookReason>&& InRequests, EReachability InExploreReachability);
	FRequestCluster(UCookOnTheFlyServer& COTFS, TRingBuffer<FDiscoveryQueueElement>& DiscoveryQueue);
	FRequestCluster(FRequestCluster&&) = delete; // Has internal pointers, would have to write manually
	FRequestCluster(const FRequestCluster&) = delete;
	enum EBuildDependencyQueueConstructorType
	{
		BuildDependencyQueue
	};
	FRequestCluster(UCookOnTheFlyServer& COTFS, EBuildDependencyQueueConstructorType,
		TRingBuffer<FPackageData*>& BuildDependencyDiscoveryQueue);

	/**
	 * Calculate the information needed to create a PackageData, and transitive search dependencies for all requests.
	 * Called repeatedly (due to timeslicing) until bOutComplete is set to true.
	 */
	void Process(const FCookerTimer& CookerTimer, bool& bOutComplete);

	/** Return whether the cluster found work to do after construction and needs to be processed. */
	bool NeedsProcessing() const;

	/** PackageData container interface: return the number of PackageDatas owned by this container. */
	int32 NumPackageDatas() const;
	/** PackageData container interface: remove the PackageData from this container. */
	void RemovePackageData(FPackageData* PackageData);
	void OnNewReachablePlatforms(FPackageData* PackageData);
	void OnBeforePlatformAddedToSession(const ITargetPlatform* TargetPlatform);
	void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform);
	void RemapTargetPlatforms(TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	/** PackageData container interface: whether the PackageData is owned by this container. */
	bool Contains(FPackageData* PackageData) const;
	/**
	 * Remove all PackageDatas owned by this container and return them.
	 * OutRequestsToLoad is the set of PackageDatas sorted by leaf to root load order.
	 * OutRequestToDemote is the set of Packages that are uncookable or have already been cooked.
	 * If called before Process sets bOutComplete=true, all packages are put in OutRequestToLoad and are unsorted.
	 */
	void ClearAndDetachOwnedPackageDatas(TArray<FPackageData*>& OutRequestsToLoad,
		TArray<TPair<FPackageData*, ESuppressCookReason>>& OutRequestsToDemote,
		TMap<FPackageData*, TArray<FPackageData*>>& OutRequestGraph);
	/**
	 * Report packages that are in request state and assigned to this Cluster, but that should not be counted as in
	 * progress for progress displays because this cluster has marked them as already cooked or as to be demoted.
	 */
	int32 GetPackagesToMarkNotInProgress() const;

	static TConstArrayView<FName> GetLocalizationReferences(FName PackageName, UCookOnTheFlyServer& InCOTFS);
	static TArray<FName> GetAssetManagerReferences(FName PackageName);
	static void IsRequestCookable(const ITargetPlatform* TargetPlatform, const FPackageData& PackageData,
		UCookOnTheFlyServer& COTFS, ESuppressCookReason& OutReason, bool& bOutCookable, bool& bOutExplorable);

private:
	struct FGraphSearch;

	/** GraphSearch cached data for a packagename that has already been visited. */
	struct FVisitStatus
	{
		FPackageData* PackageData = nullptr;
		bool bVisited = false;
	};

	/** Status for where a vertex is on the journey through having its CookDependency information fetched from DDC. */
	enum class EAsyncQueryStatus : uint8
	{
		NotRequested,
		SchedulerRequested,
		AsyncRequested,
		Complete,
	};

	/** Per-platform data in an active query for a vertex's dependencies/previous incremental results. */
	struct FQueryPlatformData
	{
		EAsyncQueryStatus GetAsyncQueryStatus();
		bool CompareExchangeAsyncQueryStatus(EAsyncQueryStatus& Expected, EAsyncQueryStatus Desired);

	public:
		// All fields other than CookAttachments and AsyncQueryStatus are read/write on Scheduler thread only
		/**
		 * Data looked up about the package's dependencies from the PackageWriter's previous cook of the package.
		 * Thread synchronization: this field is write-once from the async thread and is not readable until
		 * bSchedulerThreadFetchCompleted.
		 */
		FIncrementalCookAttachments CookAttachments;
		bool bSchedulerThreadFetchCompleted = false;
		bool bExploreRequested = false;
		bool bExploreCompleted = false;
		bool bIncrementallyUnmodifiedRequested = false;
		bool bTransitiveBuildDependenciesResolvedAsNotModified = false;
		bool bIncrementallyModifiedInstigated = false;
	private:
		std::atomic<EAsyncQueryStatus> AsyncQueryStatus;
	};

	/**
	 * Extra data about a package owned or referenced by the cluster that is needed for the lifetime of the cluster.
	 * FVertexDatas are never deallocated while async operations are active, they can only be deallocated after all
	 * async operations are complete, and all FVertexDatas are deallocated together.
	 */
	struct FVertexData
	{
	public:
		// Constructor and initialization that occurs before async work on the vertex is possible
		FVertexData(FName InPackageName, UE::Cook::FPackageData* InPackageData, int32 NumFetchPlatforms);


		// Interface that is readonly once async work on the vertex is possible and is therefore readable from any thread


		FName GetPackageName() const;


		// Interface that is callable only by the current owner thread, which switches from process thread to
		// async thread during fetch


		/** Settings and Results for each of the GraphSearch's FetchPlatforms. Element n corresponds to FetchPlatform n. */
		TArrayView<FQueryPlatformData> GetPlatformData();


		// Interface that is callable from the process thread only


		FPackageData* GetPackageData() const;
		TArray<FVertexData*>& GetIncrementallyModifiedListeners();
		TSet<FVertexData*> GetUnreadyDependencies();

		/**
		 * Whether the package is owned by this cluster and the cluster should decide its next state.
		 * BuildDependency packages are the example where this is not true; they are tracked by the cluster to decide
		 * skippability of other packages but are not in the cluster and might be idle or in another state.
		 */
		bool IsOwnedByCluster() const;
		void SetOwnedByCluster(bool bOwned);

		/**
		 * Whether the package has already been pulled into the cluster once. If there is access from multiple
		 * clusters this can be true even if GetOwnedByCluster is false.
		 */
		bool HasBeenPulledIntoCluster() const;

		/** The package's SuppressCookReason, either NotSuppressed or a reason it was suppressed. */
		ESuppressCookReason GetSuppressReason() const;
		void SetSuppressReason(ESuppressCookReason Value);

		/** Whether the package has been marked cookable by any platform. */
		bool IsAnyCookable() const;
		void SetAnyCookable(bool bInCookable);

		/**
		 * Whether the vertex has already checked its dependencies once for skippability, but found some dependencies
		 * that need to be evaluated before it can decide, and is now waiting for their evaluation to complete.
		 */
		bool IsWaitingOnUnreadyDependencies() const;
		void SetWaitingOnUnreadyDependencies(bool bWaiting);

		/** Whether the package was marked as committed for any platform by this cluster. */
		bool WasMarkedSkipped() const;
		void SetWasMarkedSkipped(bool bValue);

		/**
		 * Whether this package is owned by the cluster and therefore in progress, but should be subtracted from the
		 * inprogress count because it will be removed from inprogress when the cluster completes.
		 * Used by COTFS when displaying number of PackageDatas in each state.
		 */
		bool IsOwnedButNotInProgress() const;

	private:
		// Data that is readonly once async work on the vertex is possible and is therefore readable from any thread

		FName PackageName;

		// Data that is read/write only by the current owner thread, which switches from process thread to
		// async thread during fetch

		TArray<FQueryPlatformData> PlatformData;

		// Data that is read/write from the process thread only

		TArray<FVertexData*> IncrementallyModifiedListeners;
		TSet<FVertexData*> UnreadyDependencies;
		UE::Cook::FPackageData* PackageData = nullptr;
		ESuppressCookReason SuppressCookReason = ESuppressCookReason::NotSuppressed;
		bool bOwnedByCluster = false;
		bool bHasBeenPulledIntoCluster = false;
		bool bAnyCookable = true;
		bool bWaitingOnUnreadyDependencies = false;
		bool bWasMarkedSkipped = false;
	};

	/**
	 * Each FVertexData includes has-been-cooked existence and dependency information that is looked up
	 * from PackageWriter storage of previous cooks. The lookup can have significant latency and per-query
	 * costs. We therefore do the lookups for vertices asynchronously and in batches. An FQueryVertexBatch
	 * is a collection of FVertexData that are sent in a single lookup batch. The batch is destroyed
	 * once the results for all requested vertices are received.
	 */
	struct FQueryVertexBatch
	{
		FQueryVertexBatch(FGraphSearch& InGraphSearch);
		void Reset();
		void Send();

		void RecordCacheResults(FName PackageName, int32 PlatformIndex,
			FIncrementalCookAttachments&& CookAttachments);

		struct FPlatformData
		{
			TArray<FPackageIncrementalCookId> PackageIds;
		};

		TArray<FPlatformData> PlatformDatas;
		/**
		 * Map of the requested vertices by name. The map is created during Send and is
		 * read-only afterwards (so the map is multithread-readable). The Vertices pointed to have their own
		 * rules for what is accessible from the async work threads.
		 * */
		TMap<FName, FVertexData*> Vertices;
		/** Accessor for the GraphSearch; only thread-safe functions and variables should be accessed. */
		FGraphSearch& ThreadSafeOnlyVars;
		/** Number of vertex*platform requests that still await results. Batch is done when NumPendingRequests == 0. */
		std::atomic<uint32> NumPendingRequests;
	};

	/** Platform information that is constant (usually, some events can change it) during the cluster's lifetime. */
	struct FFetchPlatformData
	{
		const ITargetPlatform* Platform = nullptr;
		ICookedPackageWriter* Writer = nullptr;
		bool bIsPlatformAgnosticPlatform = false;
		bool bIsCookerLoadingPlatform = false;
	};
	// Platforms are listed in various arrays, always in the same order. Some special case entries exist and are added
	// at specified indices in the arrays.
	static constexpr int32 PlatformAgnosticPlatformIndex = 0;
	static constexpr int32 CookerLoadingPlatformIndex = 1;
	static constexpr int32 FirstSessionPlatformIndex = 2;

	/** How much traversal the GraphSearch should do based on settings for the entire cook. */
	enum class ETraversalTier
	{
		/**
		 * Do not fetch any edgedata, do not evaluate skippability. Mark each input vertex as should-be-cooked.
		 * Used on CookWorkers when saving runtime packages.
		 */
		MarkForRuntime,
		/**
		 * Do not fetch any edgedata, do not evaluate skippability. Mark each input vertex as should-be-committed.
		 * Used on CookWorkers when committing build dependencies without saving them.
		 */
		MarkForBuildDependency,
		/**
		 * Mark vertices as skippable if they have uptodate dependencies, even without a saveresult. 
		 * Explore dependencies necessary for evaluating modification status, otherwise do not explore dependencies.
		 */
		BuildDependencies,
		/**
		 * Mark vertices as skippable only if they have uptodate dependencies and a saveresult.
		 * Explore dependencies necessary for evaluating modification status, otherwise do not explore dependencies.
		 * Used when traversing runtime packages to save, with debug cooking flag such as -cooksinglepackagenorefs.
		 */
		RuntimeVisitVertices,
		/**
		 * Mark vertices as skippable only if they have uptodate dependencies and a saveresult. Explore runtime dependencies
		 * and add them to the cluster. Used when traversing runtime packages to save on the cookdirector.
		 */
		RuntimeFollowDependencies,
	};

	/**
	 * Variables and functions that are only used during PumpExploration. PumpExploration executes a graph search
	 * over the graph of packages (vertices) and their hard/soft dependencies upon other packages (edges). 
	 * Finding the dependencies for each package uses previous cook results and is executed asynchronously.
	 * After the graph is searched, packages are sorted topologically from leaf to root, so that packages are
	 * loaded/saved by the cook before the packages that need them to be in memory to load.
	 */
	struct FGraphSearch
	{
	public:
		FGraphSearch(FRequestCluster& InCluster);
		void Initialize();
		~FGraphSearch();
		void Reset();
		bool IsInitialized() const;

		// All public functions are callable only from the process thread
		/** Skip the entire GraphSearch and just visit the Cluster's current ClusterPackages. */
		void VisitWithoutFetching();
		/** Start a search from the Cluster's current ClusterPackages. */
		void StartSearch();
		bool IsStarted() const;
		void OnNewReachablePlatforms(FPackageData* PackageData);

		/**
		 * Visit newly reachable PackageDatas, queue a fetch of their dependencies, harvest new reachable PackageDatas
		 * from the results of the fetch.
		 */
		void TickExploration(bool& bOutDone);
		/** Sleep (with timeout) until work is available in TickExploration */
		void WaitForAsyncQueue(double WaitTimeSeconds);

		/**
		 * Edges in the dependency graph found during graph search.
		 * Only includes PackageDatas that are part of this cluster
		 */
		TMap<FPackageData*, TArray<FPackageData*>>& GetGraphEdges();

	private:
		// Scratch data structures used to avoid dynamic allocations; lifetime of each use is only on the stack
		struct FScratchPlatformDependencyBits
		{
			TBitArray<> HasRuntimePlatformByIndex;
			TBitArray<> HasBuildPlatformByIndex;
			TBitArray<> ForceExplorableByIndex;
			EInstigator InstigatorType = EInstigator::InvalidCategory;
			EInstigator BuildInstigatorType = EInstigator::InvalidCategory;
		};
		struct FExploreEdgesContext
		{
		public:
			FExploreEdgesContext(FRequestCluster& InCluster, FGraphSearch& InGraphSearch);

			/**
			 * Process the results from async edges fetch and queue the found dependencies-for-visiting. Only does
			 * portions of the work for each FQueryPlatformData that were requested by the flags on the PlatformData.
			 */
			void Explore(FVertexData& InVertex);

		private:
			void Initialize(FVertexData& InVertex);
			void CalculatePlatformsToProcess();
			bool TryCalculateIncrementallyUnmodified();
			void CalculatePackageDataDependenciesPlatformAgnostic();
			void CalculateDependenciesAndIncrementallySkippable();
			void QueueVisitsOfDependencies();
			void MarkExploreComplete();

			void AddPlatformDependency(FName DependencyName, int32 PlatformIndex, EInstigator InstigatorType);
			void AddPlatformDependencyRange(TConstArrayView<FName> Range, int32 PlatformIndex, EInstigator InstigatorType);
			void ProcessPlatformAttachments(int32 PlatformIndex, const ITargetPlatform* TargetPlatform,
				FFetchPlatformData& FetchPlatformData, FPackagePlatformData& PackagePlatformData,
				FIncrementalCookAttachments& PlatformAttachments, bool bExploreDependencies);
			void ProcessPlatformDiscoveredDependencies(int32 PlatformIndex, const ITargetPlatform* TargetPlatform);

			void SetIncrementallyUnmodified(int32 PlatformIndex, FPackagePlatformData& PackagePlatformData);
			void SetIncrementallyModified(int32 PlatformIndex, FPackagePlatformData& PackagePlatformData,
				bool bPreviouslyCooked, EIncrementallyModifiedReason Reason,
				FIncrementallyModifiedContext& Context);

		private:
			FRequestCluster& Cluster;
			FGraphSearch& GraphSearch;
			FVertexData* Vertex = nullptr;
			FPackageData* PackageData = nullptr;
			TArray<FName>* DiscoveredDependencies = nullptr;
			TArray<FName> HardGameDependencies;
			TArray<FName> HardEditorDependencies;
			TArray<FName> SoftGameDependencies;
			TArray<FName> CookerLoadingDependencies;
			TArray<int32, TInlineAllocator<10>> PlatformsToProcess;
			TArray<int32, TInlineAllocator<10>> PlatformsToExplore;
			TMap<FName, FScratchPlatformDependencyBits> PlatformDependencyMap;
			TSet<FName> HardDependenciesSet;
			TSet<FName> SkippedPackages;
			TArray<FVertexData*> UnreadyTransitiveBuildVertices;
			FName PackageName;
			int32 LocalNumFetchPlatforms = 0;
			bool bFetchAnyTargetPlatform = false;
		};
		friend struct FQueryVertexBatch;
		friend struct FVertexData;
		
		// Functions callable only from the Process thread
		/** Log diagnostic information about the search, e.g. timeout warnings. */
		void UpdateDisplay();

		/** Asynchronously fetch the dependencies and previous incremental results for a vertex */
		void QueueEdgesFetch(FVertexData& Vertex, TConstArrayView<int32> PlatformIndexes);
		/** Calculate and store the vertex's PackageData's cookability for each reachable platform. Kick off edges fetch. */
		void VisitVertex(FVertexData& Vertex);
		/** Calculate and store the vertex's PackageData's cookability for the platform. */
		void VisitVertexForPlatform(FVertexData& Vertex, const ITargetPlatform* Platform,
			EReachability ClusterReachability, FPackagePlatformData& PlatformData,
			ESuppressCookReason& OutSuppressCookReason);
		void ResolveTransitiveBuildDependencyCycle();

		/** Queue a vertex for visiting and dependency traversal */
		void AddToVisitVertexQueue(FVertexData& Vertex);

		// Functions that must be called only within the Lock
		/** Allocate memory for a new batch; returned batch is not yet constructed. */
		FQueryVertexBatch* AllocateBatch();
		/** Free an allocated batch. */ 
		void FreeBatch(FQueryVertexBatch* Batch);
		/** Pop vertices from VerticesToRead into batches, if there are enough of them. */
		void CreateAvailableBatches(bool bAllowIncompleteBatch);
		/** Pop a single batch vertices from VerticesToRead. */
		FQueryVertexBatch* CreateBatchOfPoppedVertices(int32 BatchSize);

		// Functions that are safe to call from any thread
		/** Notify process thread of batch completion and deallocate it. */
		void OnBatchCompleted(FQueryVertexBatch* Batch);
		/** Notify process thread of vertex completion. */
		void KickVertex(FVertexData* Vertex);

		TArrayView<FQueryPlatformData> GetPlatformDataArray(FVertexData& Vertex);

	private:
		// Variables that are read-only during multithreading
		TArray<FFetchPlatformData> FetchPlatforms;
		FRequestCluster& Cluster;

		// Variables that are accessible only from the Process thread
		/** A set of stack and scratch variables used when calculating and exploring the edges of a vertex. */
		FExploreEdgesContext ExploreEdgesContext;
		TMap<FPackageData*, TArray<FPackageData*>> GraphEdges;
		TSet<FVertexData*> VisitVertexQueue;
		TSet<FVertexData*> PendingTransitiveBuildDependencyVertices;
		/** Vertices queued for async processing that are not yet numerous enough to fill a batch. */
		TRingBuffer<FVertexData*> PreAsyncQueue;
		/** Time-tracker for timeout warnings in Poll */
		double LastActivityTime = 0.;
		int32 RunAwayTickLoopCount = 0;
		bool bInitialized = false;
		bool bStarted = false;

		// Variables that are accessible from multiple threads, guarded by Lock
		FCriticalSection Lock;
		TTypedBlockAllocatorResetList<FQueryVertexBatch> BatchAllocator;
		TSet<FQueryVertexBatch*> AsyncQueueBatches;

		// Variables that are accessible from multiple threads, internally threadsafe
		TMpscQueue<FVertexData*> AsyncQueueResults;
		FEventRef AsyncResultsReadyEvent;
	};

private:
	FRequestCluster(UCookOnTheFlyServer& COTFS, EReachability ExploreReachability);
	void EmptyClusterPackages();
	void ReserveInitialRequests(int32 RequestNum);
	/**
	 * Track the cluster's count of vertices that depend on the vertex's state. Delta indicates whether the
	 * vertex is being added or removed from the counts.
	 */
	void AddVertexCounts(FVertexData& Vertex, int32 Delta);
	void SetOwnedByCluster(FVertexData& Vertex, bool bOwnedByCluster, bool bNeedsStateChange = true);
	void SetSuppressReason(FVertexData& Vertex, ESuppressCookReason Reason);
	void SetWasMarkedSkipped(FVertexData& Vertex, bool bWasMarkedSkipped);
	void FetchPackageNames(const FCookerTimer& CookerTimer, bool& bOutComplete);
	void PumpExploration(const FCookerTimer& CookerTimer, bool& bOutComplete);
	void StartAsync(const FCookerTimer& CookerTimer, bool& bOutComplete);
	bool IsIncrementalCook() const;
	void IsRequestCookable(const ITargetPlatform* TargetPlatform, const FPackageData& PackageData,
		ESuppressCookReason& OutReason, bool& bOutCookable, bool& bOutExplorable);
	static void IsRequestCookable(const ITargetPlatform* TargetPlatform, const FPackageData& PackageData,
		UCookOnTheFlyServer& InCOTFS, FStringView InDLCPath, ESuppressCookReason& OutReason, bool& bOutCookable,
		bool& bOutExplorable);
	/**
	 * TraversalTier property: runtime dependencies of visited vertices should be explored and their targets
	 * added to the cluster.
	 */
	bool TraversalExploreRuntimeDependencies();
	/**
	 * TraversalTier property: True if we need to test packages for incrementally skippable, false if we don't.
	 */
	bool TraversalExploreIncremental();
	/**
	 * TraversalTier property: True if we are marking packages as should-be-saved for runtime, false if
	 * we are committing packages just to record CookDependencies instead of saving them for runtime.
	 */
	bool TraversalMarkCookable();

	/** Total number of platforms known to the cluster, including the special cases. */
	int32 GetNumFetchPlatforms() const;
	/** Total number of non-special-case platforms known to the cluster.Identical to COTFS's session platforms */
	int32 GetNumSessionPlatforms() const;

	/** Find or add a Vertex for PackageName. If PackageData is provided, use it, otherwise look it up. */
	FVertexData& FindOrAddVertex(FName PackageName, FGenerationHelper* ParentGenerationHelper = nullptr);
	FVertexData& FindOrAddVertex(FPackageData& PackageData);
	/** Batched allocation for vertices. */
	FVertexData* AllocateVertex(FName PackageName, FPackageData* PackageData);


	TArray<FFilePlatformRequest> FilePlatformRequests;
	TMap<FName, FVertexData*> ClusterPackages;
	TMap<FPackageData*, TArray<FPackageData*>> RequestGraph;
	TTypedBlockAllocatorFreeList<FVertexData> VertexAllocator;
	FString DLCPath;
	FGraphSearch GraphSearch;
	UCookOnTheFlyServer& COTFS;
	FPackageDatas& PackageDatas;
	IAssetRegistry& AssetRegistry;
	FPackageTracker& PackageTracker;
	FBuildDefinitions& BuildDefinitions;
	ETraversalTier TraversalTier = ETraversalTier::RuntimeFollowDependencies;
	int32 NumOwned = 0;
	int32 NumOwnedButNotInProgress = 0;
	int32 NumFetchPlatforms = 0;
	bool bAllowHardDependencies = true;
	bool bAllowSoftDependencies = true;
	bool bErrorOnEngineContentUse = false;
	bool bPackageNamesComplete = false;
	bool bDependenciesComplete = false;
	bool bStartAsyncComplete = false;
	bool bAllowIncrementalResults = false;
	bool bPreQueueBuildDefinitions = true;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline FName FRequestCluster::FVertexData::GetPackageName() const
{
	return PackageName;
}

inline TArrayView<FRequestCluster::FQueryPlatformData> FRequestCluster::FVertexData::GetPlatformData()
{
	return PlatformData;
}

inline FPackageData* FRequestCluster::FVertexData::GetPackageData() const
{
	return PackageData;
}

inline TArray<FRequestCluster::FVertexData*>& FRequestCluster::FVertexData::GetIncrementallyModifiedListeners()
{
	return IncrementallyModifiedListeners;
}

inline TSet<FRequestCluster::FVertexData*> FRequestCluster::FVertexData::GetUnreadyDependencies()
{
	return UnreadyDependencies;
}

inline bool FRequestCluster::FVertexData::IsOwnedByCluster() const
{
	return bOwnedByCluster;
}

inline void FRequestCluster::FVertexData::SetOwnedByCluster(bool bOwned)
{
	bOwnedByCluster = bOwned;
	bHasBeenPulledIntoCluster |= bOwned;
}

inline bool FRequestCluster::FVertexData::HasBeenPulledIntoCluster() const
{
	return bHasBeenPulledIntoCluster;
}

inline ESuppressCookReason FRequestCluster::FVertexData::GetSuppressReason() const
{
	return SuppressCookReason;
}

inline void FRequestCluster::FVertexData::SetSuppressReason(ESuppressCookReason Value)
{
	SuppressCookReason = Value;
}

inline bool FRequestCluster::FVertexData::IsAnyCookable() const
{
	return bAnyCookable;
}

inline void FRequestCluster::FVertexData::SetAnyCookable(bool bInCookable)
{
	bAnyCookable = bInCookable;
}

inline bool FRequestCluster::FVertexData::IsWaitingOnUnreadyDependencies() const
{
	return bWaitingOnUnreadyDependencies;
}

inline void FRequestCluster::FVertexData::SetWaitingOnUnreadyDependencies(bool bWaiting)
{
	bWaitingOnUnreadyDependencies = bWaiting;
}

inline bool FRequestCluster::FVertexData::WasMarkedSkipped() const
{
	return bWasMarkedSkipped;
}

inline void FRequestCluster::FVertexData::SetWasMarkedSkipped(bool bValue)
{
	bWasMarkedSkipped = bValue;
}

inline bool FRequestCluster::FVertexData::IsOwnedButNotInProgress() const
{
	return bOwnedByCluster &
		((SuppressCookReason != ESuppressCookReason::NotSuppressed) | bWasMarkedSkipped);
}

inline bool FRequestCluster::FGraphSearch::IsInitialized() const
{
	return bInitialized;
}

inline bool FRequestCluster::FGraphSearch::IsStarted() const
{
	return bStarted;
}

inline TArrayView<FRequestCluster::FQueryPlatformData> FRequestCluster::FGraphSearch::GetPlatformDataArray(
	FVertexData& Vertex)
{
	return Vertex.GetPlatformData();
}

inline FRequestCluster::EAsyncQueryStatus FRequestCluster::FQueryPlatformData::GetAsyncQueryStatus()
{
	return AsyncQueryStatus.load(std::memory_order_acquire);
}

inline bool FRequestCluster::FQueryPlatformData::CompareExchangeAsyncQueryStatus(EAsyncQueryStatus& Expected,
	EAsyncQueryStatus Desired)
{
	return AsyncQueryStatus.compare_exchange_strong(Expected, Desired,
		// For the read operation to see whether we should set it, we need only relaxed memory order;
		// we don't care about the values of other related variables that depend on it when deciding whether
		// it is our turn to set it.
		// For the write operation if we decide to set it, we need release memory order to guard reads of
		// the variables that depend on it (e.g. CookAttachments).
		std::memory_order_release /* success memory order */,
		std::memory_order_relaxed /* failure memory order */
	);
}

inline bool FRequestCluster::NeedsProcessing() const
{
	return !ClusterPackages.IsEmpty() || !FilePlatformRequests.IsEmpty();
}

inline int32 FRequestCluster::NumPackageDatas() const
{
	return NumOwned;
}

inline int32 FRequestCluster::GetPackagesToMarkNotInProgress() const
{
	return NumOwnedButNotInProgress;
}

inline int32 FRequestCluster::GetNumFetchPlatforms() const
{
	return NumFetchPlatforms;
}

inline int32 FRequestCluster::GetNumSessionPlatforms() const
{
	return NumFetchPlatforms - 2;
}

}
