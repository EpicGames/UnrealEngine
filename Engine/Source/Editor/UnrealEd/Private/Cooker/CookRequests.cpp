// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookRequests.h"

#include "Algo/Find.h"
#include "CookOnTheSide/CookLog.h"
#include "CookPackageData.h"
#include "CookPlatformManager.h"
#include "Cooker/CookTypes.h"
#include "HAL/Platform.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/UnrealNames.h"

namespace UE::Cook
{

//////////////////////////////////////////////////////////////////////////
// FFilePlatformRequest

FFilePlatformRequest::FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator)
	:FFilePlatformRequest(InFilename, MoveTemp(InInstigator),
		TArray<const ITargetPlatform*>(), FCompletionCallback())
{
}

FFilePlatformRequest::FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator, 
	const ITargetPlatform* InPlatform, FCompletionCallback&& InCompletionCallback)
	:FFilePlatformRequest(InFilename, MoveTemp(InInstigator),
		TArrayView<const ITargetPlatform* const>({ InPlatform }), MoveTemp(InCompletionCallback))
{
}

FFilePlatformRequest::FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator,
	const TArrayView<const ITargetPlatform* const>& InPlatforms, FCompletionCallback&& InCompletionCallback)
	:FFilePlatformRequest(InFilename, MoveTemp(InInstigator),
		TArray<const ITargetPlatform*>(InPlatforms.GetData(), InPlatforms.Num()), MoveTemp(InCompletionCallback))
{
}

FFilePlatformRequest::FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator,
	TArray<const ITargetPlatform*>&& InPlatforms, FCompletionCallback&& InCompletionCallback)
	: Platforms(MoveTemp(InPlatforms))
	, CompletionCallback(MoveTemp(InCompletionCallback))
	, Instigator(MoveTemp(InInstigator))
{
	SetFilename(InFilename.ToString());
}

FFilePlatformRequest::FFilePlatformRequest(const FFilePlatformRequest& InFilePlatformRequest)
	: Filename(InFilePlatformRequest.Filename)
	, Platforms(InFilePlatformRequest.Platforms)
	, Instigator(InFilePlatformRequest.Instigator)
	, bUrgent(InFilePlatformRequest.bUrgent)
{
	check(!InFilePlatformRequest.CompletionCallback); // CompletionCallbacks can not be copied, so the caller's intent is not clear in this constructor if the input has one
}
		
FFilePlatformRequest::FFilePlatformRequest(FFilePlatformRequest&& InFilePlatformRequest)
	: Filename(MoveTemp(InFilePlatformRequest.Filename))
	, Platforms(MoveTemp(InFilePlatformRequest.Platforms))
	, CompletionCallback(MoveTemp(InFilePlatformRequest.CompletionCallback))
	, Instigator(MoveTemp(InFilePlatformRequest.Instigator))
	, bUrgent(InFilePlatformRequest.bUrgent)
{
}

FFilePlatformRequest& FFilePlatformRequest::operator=(FFilePlatformRequest&& InFileRequest)
{
	Filename = MoveTemp(InFileRequest.Filename);
	Platforms = MoveTemp(InFileRequest.Platforms);
	check(!CompletionCallback); // We don't support holding multiple completion callbacks
	CompletionCallback = MoveTemp(InFileRequest.CompletionCallback);
	Instigator = MoveTemp(InFileRequest.Instigator);
	bUrgent = InFileRequest.bUrgent;
	return *this;
}

void FFilePlatformRequest::SetFilename(FString InFilename)
{
	Filename = FPackageDatas::GetStandardFileName(InFilename);
}

const FName& FFilePlatformRequest::GetFilename() const
{
	return Filename;
}

FInstigator& FFilePlatformRequest::GetInstigator()
{
	return Instigator;
}

const TArray<const ITargetPlatform*>& FFilePlatformRequest::GetPlatforms() const
{
	return Platforms;
}

TArray<const ITargetPlatform*>& FFilePlatformRequest::GetPlatforms()
{
	return Platforms;
}

void FFilePlatformRequest::RemovePlatform(const ITargetPlatform* Platform)
{
	Platforms.Remove(Platform);
}

void FFilePlatformRequest::AddPlatform(const ITargetPlatform* Platform)
{
	check(Platform != nullptr);
	Platforms.Add(Platform);
}

bool FFilePlatformRequest::HasPlatform(const ITargetPlatform* Platform) const
{
	return Platforms.Find(Platform) != INDEX_NONE;
}

FCompletionCallback& FFilePlatformRequest::GetCompletionCallback()
{
	return CompletionCallback;
}

bool FFilePlatformRequest::IsValid() const
{
	return Filename != NAME_None;
}

void FFilePlatformRequest::Clear()
{
	Filename = TEXT("");
	Platforms.Empty();
}

bool FFilePlatformRequest::operator==(const FFilePlatformRequest& InFileRequest) const
{
	if (InFileRequest.Filename == Filename)
	{
		if (InFileRequest.Platforms == Platforms)
		{
			return true;
		}
	}
	return false;
}

FString FFilePlatformRequest::ToString() const
{
	FString Result = FString::Printf(TEXT("%s;"), *Filename.ToString());

	for (const ITargetPlatform* Platform : Platforms)
	{
		Result += FString::Printf(TEXT("%s,"), *Platform->PlatformName());
	}
	return Result;
}

void FFilePlatformRequest::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	RemapArrayElements(Platforms, Remap);
}

//////////////////////////////////////////////////////////////////////////
// FExternalRequests

int32 FExternalRequests::GetNumRequests() const
{
	return RequestCount.load(std::memory_order_relaxed);
}

bool FExternalRequests::HasRequests() const
{
	return GetNumRequests() > 0;
}

void FExternalRequests::AddCallback(FSchedulerCallback&& Callback)
{
	FScopeLock ScopeLock(&RequestLock);

	Callbacks.Add(MoveTemp(Callback));
	RequestCount.fetch_add(1, std::memory_order_relaxed);
}

void FExternalRequests::EnqueueUnique(FFilePlatformRequest&& FileRequest, bool bForceFrontOfQueue)
{
	FScopeLock ScopeLock(&RequestLock);
	FName Filename = FileRequest.GetFilename();
	FFilePlatformRequest* ExistingRequest = RequestMap.Find(Filename);
	if (!ExistingRequest)
	{
		RequestMap.Add(Filename, MoveTemp(FileRequest));
		if (bForceFrontOfQueue)
		{
			Queue.AddFront(Filename);
		}
		else
		{
			Queue.Add(Filename);
		}

		RequestCount.fetch_add(1, std::memory_order_relaxed);
	}
	else
	{
		if (FileRequest.GetCompletionCallback())
		{
			check(!ExistingRequest->GetCompletionCallback()); // We don't support multiple callbacks
			ExistingRequest->GetCompletionCallback() = MoveTemp(FileRequest.GetCompletionCallback());
		}

		// add the requested platforms to the platform list
		for (const ITargetPlatform* Platform : FileRequest.GetPlatforms())
		{
			ExistingRequest->GetPlatforms().AddUnique(Platform);
		}

		if (bForceFrontOfQueue)
		{
			FName* ExistingName = Algo::Find(Queue, Filename);
			int32 Index = Queue.ConvertPointerToIndex(ExistingName);
			check(Index != INDEX_NONE);
			if (Index != 0)
			{
				Queue[Index] = Queue[0];
				Queue[0] = Filename;
			}
		}
	}
}

EExternalRequestType FExternalRequests::DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks, TArray<FFilePlatformRequest>& OutBuildRequests)
{
	FScopeLock ScopeLock(&RequestLock);

	if (ThreadUnsafeDequeueCallbacks(OutCallbacks))
	{
		return EExternalRequestType::Callback;
	}
	else if (Queue.Num())
	{
		OutBuildRequests.Reserve(Queue.Num() + OutBuildRequests.Num());
		int32 NumDequeued = Queue.Num();
		while (Queue.Num())
		{
			FName Filename = Queue.PopFrontValue();
			OutBuildRequests.Add(RequestMap.FindAndRemoveChecked(Filename));
		}
		RequestCount.fetch_add(-NumDequeued, std::memory_order_relaxed);
		return EExternalRequestType::Cook;
	}
	else
	{
		return EExternalRequestType::None;
	}
}

bool FExternalRequests::DequeueCallbacks(TArray<FSchedulerCallback>& OutCallbacks)
{
	FScopeLock ScopeLock(&RequestLock);
	return ThreadUnsafeDequeueCallbacks(OutCallbacks);
}

bool FExternalRequests::ThreadUnsafeDequeueCallbacks(TArray<FSchedulerCallback>& OutCallbacks)
{
	if (Callbacks.Num() > 0)
	{
		OutCallbacks = MoveTemp(Callbacks);
		Callbacks.Empty();
		RequestCount.fetch_add(-OutCallbacks.Num(), std::memory_order_relaxed);
		return true;
	}
	else
	{
		return false;
	}
}

void FExternalRequests::EmptyRequests()
{
	FScopeLock ScopeLock(&RequestLock);

	Queue.Empty();
	RequestMap.Empty();
	Callbacks.Empty();
	RequestCount.store(0, std::memory_order_relaxed);
}

void FExternalRequests::DequeueAll(TArray<FSchedulerCallback>& OutCallbacks, TArray<FFilePlatformRequest>& OutCookRequests)
{
	FScopeLock ScopeLock(&RequestLock);

	OutCallbacks = MoveTemp(Callbacks);
	Callbacks.Empty();

	for (TPair<FName, FFilePlatformRequest>& Request : RequestMap)
	{
		OutCookRequests.Add(MoveTemp(Request.Value));
	}
	RequestMap.Empty();
	Queue.Empty();

	RequestCount.store(0, std::memory_order_relaxed);
}

void FExternalRequests::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform)
{
	FScopeLock ScopeLock(&RequestLock);

	// The caller should not be removing platforms if we have an active request referencing that platform, but in case they did, remove the platform
	// from all pending requests
	for (TPair<FName, FFilePlatformRequest>& kvpair : RequestMap)
	{
		FFilePlatformRequest& Request = kvpair.Value;
		Request.GetPlatforms().Remove(TargetPlatform);
		if (Request.GetPlatforms().Num() == 0)
		{
			UE_LOGF(LogCook, Error, "RemovePlatform call has left an empty list of platforms requested in CookOnTheSide request.");
		}
	}
}

void FExternalRequests::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	for (TPair<FName, FFilePlatformRequest>& KVPair : RequestMap)
	{
		KVPair.Value.RemapTargetPlatforms(Remap);
	}
}

void FExternalRequests::LogAllRequestedFiles()
{
	UE_LOGF(LogCook, Display, "Listing initially requested set of files to be cooked (does not include list of transitive dependencies):");
	for (const TPair<FName, FFilePlatformRequest>& KVPair : RequestMap)
	{
		UE_LOGF(LogCook, Display, "%ls", *KVPair.Value.ToString());
	}
	UE_LOGF(LogCook, Display, "----------------------------------------------------------\n\n");
}
}
