// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "Containers/UnrealString.h"
#include "CookTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "Templates/UniquePtr.h"
#include "UObject/ICookInfo.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FEvent;
class ITargetPlatform;

namespace UE::Cook
{

struct FFreeFEvent { void operator()(FEvent* Ptr) const { if (Ptr) FPlatformProcess::ReturnSynchEventToPool(Ptr); } };

/**
 * Structure holding the data for a request for the CookOnTheFlyServer to cook a FileName. Includes platform which file is requested for.
 * These requests are external to the cooker's scheduler, and do not use the FPackageData the scheduler uses internally.
 */
struct FFilePlatformRequest
{
protected:
	FName Filename;
	TArray<const ITargetPlatform*> Platforms;
	FCompletionCallback CompletionCallback;
	FInstigator Instigator;
	bool bUrgent = false;

public:
	FFilePlatformRequest() = default;

	FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator);
	FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator, const ITargetPlatform* InPlatform,
		FCompletionCallback&& InCompletionCallback = FCompletionCallback());
	FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator,
		const TArrayView<const ITargetPlatform* const>& InPlatforms, FCompletionCallback&& InCompletionCallback = FCompletionCallback());
	FFilePlatformRequest(const FName& InFilename, FInstigator&& InInstigator,
		TArray<const ITargetPlatform*>&& InPlatforms, FCompletionCallback&& InCompletionCallback = FCompletionCallback());
	FFilePlatformRequest(const FFilePlatformRequest& InFilePlatformRequest);
	FFilePlatformRequest(FFilePlatformRequest&& InFilePlatformRequest);
	FFilePlatformRequest& operator=(FFilePlatformRequest&& InFileRequest);

	void SetFilename(FString InFilename);
	const FName& GetFilename() const;
	FInstigator& GetInstigator();

	void SetUrgent(bool bInUrgent) { bUrgent = bInUrgent; }
	bool IsUrgent() const { return bUrgent; }

	const TArray<const ITargetPlatform*>& GetPlatforms() const;
	TArray<const ITargetPlatform*>& GetPlatforms();
	void RemovePlatform(const ITargetPlatform* Platform);
	void AddPlatform(const ITargetPlatform* Platform);
	bool HasPlatform(const ITargetPlatform* Platform) const;

	/** A callback that the scheduler will call after the request is processed and is cooked, fails to cook, is canceled, or is skipped because it already exists. */
	FCompletionCallback& GetCompletionCallback();

	bool IsValid() const;
	void Clear();
	bool operator==(const FFilePlatformRequest& InFileRequest) const;
	FString ToString() const;

	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);
};


/**
 * A container class for External Requests made to the cooker.
 * External Requests are cook requests that are made outside of the scheduler's lock and hence need to be separately synchronized.
 * External Requests can be either a request to cook a given FileName (packages are identified by FileName in this container) on given platforms,
 * or a request to run an arbitrary callback inside the scheduler's lock.
 * This class is threadsafe; all methods are guarded by a CriticalSection. 
 */
class FExternalRequests
{
public:

	/**
	 * Lockless value for the number of External Requests in the container.
	 * May be out of date after calling; do not assume the number of actual requests is any one of equal, greater than, or less than the returned value.
	 * Intended usage is for the Scheduler to be the only consumer of requests, and to use this value for rough reporting of periodic progress.
	 */
	int32 GetNumRequests() const;

	/**
	 * Lockless value for the number of External Requests in the container.
	 * May be out of date after calling; do not assume a true return value means Requests are actually present or a false value means no Requests are present.
	 * Intended usage is for the Scheduler to be the only consumer of requests, and to use this value for periodic checking of whether there is any work that justifies the expense of taking the lock.
	 * In a single-consumer case, HasRequests will eventually correctly return true as long as the consumer is not consuming.
	 */
	bool HasRequests() const;

	/** Add a callback-type request. The scheduler will run all callbacks (in FIFO order) as soon as it completes its current task. */
	void AddCallback(FSchedulerCallback&& Callback);

	/** Add the given cook-type request, merging its list of platforms with any existing request if one already exists. */
	void EnqueueUnique(FFilePlatformRequest&& FileRequest, bool bForceFrontOfQueue = false);

	/**
	 * If this FExternalRequests has any callbacks, dequeue them all into OutCallbacks and return EExternalRequestType::Callback; Callbacks take priority over cook requests.
	 * Otherwise, if there are any cook requests, dequeue them all into OutBuildRequests and return EExternalRequestType::Cook.
	 * Otherwise, return EExternalRequestType::None.
	 */
	EExternalRequestType DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks, TArray<FFilePlatformRequest>& OutBuildRequests);
	/* Move any existing callbacks onto OutCallbacks, and return whether any were added. */
	bool DequeueCallbacks(TArray<FSchedulerCallback>& OutCallbacks);
	/** Eliminate all callbacks and cook requests and free memory */
	void EmptyRequests();
	/** Move all callbacks into OutCallbacks, and all cook requests into OutCookRequests. This is used when canceling a cook session. */
	void DequeueAll(TArray<FSchedulerCallback>& OutCallbacks, TArray<FFilePlatformRequest>& OutCookRequests);

	/** Remove references to the given platform from all cook requests. */
	void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform);

	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	/* Prints a list of all files in the RequestMap to the log */
	void LogAllRequestedFiles();

public:
	/** An FEvent the scheduler can sleep on when waiting for new cookonthefly requests. */
	TUniquePtr<FEvent, FFreeFEvent> CookRequestEvent;
private:
	/* Implementation for DequeueCallbacks that assumes the caller has entered the RequestLock. */
	bool ThreadUnsafeDequeueCallbacks(TArray<FSchedulerCallback>& OutCallbacks);

	/** Queue of the FileName for the cook-type requests in this instance. The FileName can be used to look up the rest of the data for the request. */
	TRingBuffer<FName> Queue;
	/** Map of the extended information for the cook-type requests in this instance, keyed by the FileName of the request. */
	TMap<FName, FFilePlatformRequest> RequestMap;
	TArray<FSchedulerCallback> Callbacks;
	FCriticalSection RequestLock;
	std::atomic<int32> RequestCount = 0;
};

}