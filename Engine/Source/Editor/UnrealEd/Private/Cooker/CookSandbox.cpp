// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookSandbox.h"

#include "CookTypes.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Misc/App.h"
#include "Misc/CString.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "String/Find.h"

namespace UE::Cook
{

FCookSandbox::FCookSandbox(FStringView OutputDirectory, TArray<TSharedRef<IPlugin>>& InPluginsToRemap)
{
	// Local sandbox file wrapper. This will be used to handle path conversions, but will not be used to actually
	// write/read files so we can safely use [Platform] token in the sandbox directory name and then replace it
	// with the actual platform name.
	SandboxFile = FSandboxPlatformFile::Create(false);
	SandboxFile->Initialize(&FPlatformFileManager::Get().GetPlatformFile(),
		*FString::Printf(TEXT("-sandbox=\"%.*s\""), OutputDirectory.Len(), OutputDirectory.GetData()));

	PluginsToRemap.Reserve(InPluginsToRemap.Num());
	for (TSharedRef<IPlugin>& Plugin : InPluginsToRemap)
	{
		FPluginData& Data = PluginsToRemap.Emplace_GetRef();
		Data.Plugin = Plugin;
		Data.NormalizedRootDir = Plugin->GetBaseDir();
		FPaths::MakeStandardFilename(Data.NormalizedRootDir);
	}
}

const FString& FCookSandbox::GetSandboxDirectory() const
{
	return SandboxFile->GetSandboxDirectory();
}

const FString& FCookSandbox::GetGameSandboxDirectoryName() const
{
	return SandboxFile->GetGameSandboxDirectoryName();
}

FString FCookSandbox::ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) const
{
	return SandboxFile->ConvertToAbsolutePathForExternalAppForWrite(Filename);
}

FString FCookSandbox::ConvertFromSandboxPath(const TCHAR* Filename) const
{
	return ConvertFromSandboxPathInPlatformRoot(Filename, GetSandboxDirectory());
}

FString FCookSandbox::GetSandboxDirectory(const FString& PlatformName) const
{
	FString Result = SandboxFile->GetSandboxDirectory();
	Result.ReplaceInline(TEXT("[Platform]"), *PlatformName);
	return Result;
}

FString FCookSandbox::ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename, const FString& PlatformName) const
{
	FString Result = SandboxFile->ConvertToAbsolutePathForExternalAppForWrite(Filename);
	Result.ReplaceInline(TEXT("[Platform]"), *PlatformName);
	return Result;
}

FString FCookSandbox::ConvertFromSandboxPathInPlatformRoot(const TCHAR* Filename, FStringView PlatformSandboxRootDir) const
{
	FCookSandboxConvertCookedPathToPackageNameContext Context;
	Context.SandboxRootDir = PlatformSandboxRootDir;
	FillContext(Context);
	return ConvertCookedPathToUncookedPath(Filename, Context);
}

bool FCookSandbox::TryConvertUncookedFilenameToCookedRemappedPluginFilename(FStringView FileName,
	FString& OutCookedFileName, FStringView PlatformSandboxRootDir) const
{
	// Ideally this would be in the Sandbox File but it can't access the project or plugin
	if (PluginsToRemap.IsEmpty())
	{
		return false;
	}
	if (PlatformSandboxRootDir.IsEmpty())
	{
		PlatformSandboxRootDir = GetSandboxDirectory();
	}
	FString NormalizedFileName(FileName);
	FPaths::MakeStandardFilename(NormalizedFileName);
	constexpr FStringView ContentFolderName(TEXTVIEW("Content"));

	for (const FPluginData& Data : PluginsToRemap)
	{
		// If these match, then this content is part of plugin that gets remapped when packaged/staged
		FStringView PluginRootRelPath;
		if (FPathViews::TryMakeChildPathRelativeTo(NormalizedFileName, Data.NormalizedRootDir, PluginRootRelPath))
		{
			const FString& PluginName = Data.Plugin->GetName();

			// Put this is in <sandbox path>/RemappedPlugins/<PluginName>/PluginRootRelPath
			constexpr FStringView RemappedPluginsDirName(REMAPPED_PLUGINS);
			OutCookedFileName.Reserve(PlatformSandboxRootDir.Len() + RemappedPluginsDirName.Len() +
				PluginName.Len() + PluginRootRelPath.Len() + 3);
			OutCookedFileName = PlatformSandboxRootDir;
			OutCookedFileName /= REMAPPED_PLUGINS;
			OutCookedFileName /= Data.Plugin->GetName();
			OutCookedFileName /= PluginRootRelPath;
			return true;
		}
	}
	return false;
}

FSandboxPlatformFile& FCookSandbox::GetSandboxPlatformFile()
{
	return *SandboxFile;
}

FString FCookSandbox::ConvertToFullSandboxPath(const FString& FileName, bool bForWrite) const
{
	return ConvertToFullSandboxPathInPlatformRoot(FileName, bForWrite, GetSandboxDirectory());
}

FString FCookSandbox::ConvertToFullPlatformSandboxPath(const FString& FileName, bool bForWrite,
	const FString& PlatformName) const
{
	FString Result = ConvertToFullSandboxPath(FileName, bForWrite);
	Result.ReplaceInline(TEXT("[Platform]"), *PlatformName);
	return Result;
}

FString FCookSandbox::ConvertToFullSandboxPathInPlatformRoot(const FString& FileName, bool bForWrite,
	FStringView PlatformSandboxRootDir) const
{
	FString Result;
	if (bForWrite)
	{
		if (TryConvertUncookedFilenameToCookedRemappedPluginFilename(FileName, Result, PlatformSandboxRootDir))
		{
			return Result;
		}
		Result = SandboxFile->ConvertToAbsolutePathForExternalAppForWrite(*FileName);
	}
	else
	{
		Result = SandboxFile->ConvertToAbsolutePathForExternalAppForRead(*FileName);
	}

	return Result;
}

void FCookSandbox::FillContext(FCookSandboxConvertCookedPathToPackageNameContext& Context) const
{
	if (Context.SandboxRootDir.IsEmpty())
	{
		Context.SandboxRootDir = GetSandboxDirectory();
	}
	if (Context.UncookedRelativeRootDir.IsEmpty())
	{
		Context.UncookedRelativeRootDir = FPaths::GetRelativePathToRoot();
	}
	if (Context.SandboxProjectDir.IsEmpty())
	{
		Context.ScratchSandboxProjectDir = FPaths::Combine(Context.SandboxRootDir, FApp::GetProjectName()) + TEXT("/");
		Context.SandboxProjectDir = Context.ScratchSandboxProjectDir;
	}
	if (Context.UncookedRelativeProjectDir.IsEmpty())
	{
		Context.ScratchUncookedRelativeProjectDir = FPaths::ProjectDir();
		Context.UncookedRelativeProjectDir = Context.ScratchUncookedRelativeProjectDir;
	}
	Context.ScratchFileName.Reserve(1024);
	Context.ScratchPackageName.Reserve(1024);
}

FName FCookSandbox::ConvertCookedPathToPackageName(FStringView CookedPath,
	FCookSandboxConvertCookedPathToPackageNameContext& Context) const
{
	FString& UncookedFileName = ConvertCookedPathToUncookedPath(CookedPath, Context);
	if (!FPackageName::TryConvertFilenameToLongPackageName(UncookedFileName, Context.ScratchPackageName))
	{
		return NAME_None;
	}

	return FName(*Context.ScratchPackageName);
}

FString& FCookSandbox::ConvertCookedPathToUncookedPath(FStringView CookedPath,
	FCookSandboxConvertCookedPathToPackageNameContext& Context) const
{
	FString& UncookedFileName = Context.ScratchFileName;
	UncookedFileName.Reset();
	constexpr FStringView RemappedPluginsFolder(REMAPPED_PLUGINS);
	constexpr FStringView ContentFolder(TEXTVIEW("Content/"));

	// Check for remapped plugins' cooked content
	if (PluginsToRemap.Num() > 0)
	{
		int32 RemappedIndex = UE::String::FindFirst(CookedPath, RemappedPluginsFolder, ESearchCase::IgnoreCase);
		if (RemappedIndex >= 0)
		{
			// Snip everything up through the RemappedPlugins/ off so we can find the plugin it corresponds to
			FStringView PluginPath = CookedPath.RightChop(RemappedIndex + RemappedPluginsFolder.Len() + 1);
			// Find the plugin that owns this content
			FString ExpectedRemainingRoot;
			for (const FPluginData& Data: PluginsToRemap)
			{
				ExpectedRemainingRoot = Data.Plugin->GetName();
				FStringView RelPathFromPluginRoot;
				if (FPathViews::TryMakeChildPathRelativeTo(PluginPath, ExpectedRemainingRoot, RelPathFromPluginRoot))
				{
					UncookedFileName = Data.NormalizedRootDir;
					UncookedFileName /= RelPathFromPluginRoot;
					break;
				}
			}
		}
		// If we did not find a containing plugin, fall through to sandbox handling to construct UncookedFileName
	}

	if (UncookedFileName.IsEmpty())
	{
		auto BuildUncookedPath =
			[&UncookedFileName](FStringView CookedPath, FStringView CookedRoot, FStringView UncookedRoot)
		{
			UncookedFileName.AppendChars(UncookedRoot.GetData(), UncookedRoot.Len());
			UncookedFileName.AppendChars(CookedPath.GetData() + CookedRoot.Len(), CookedPath.Len() - CookedRoot.Len());
		};

		if (CookedPath.StartsWith(Context.SandboxRootDir))
		{
			// Optimized CookedPath.StartsWith(SandboxProjectDir) that does not compare all of SandboxRootDir again
			if (CookedPath.Len() >= Context.SandboxProjectDir.Len() &&
				0 == FCString::Strnicmp(
					CookedPath.GetData() + Context.SandboxRootDir.Len(),
					Context.SandboxProjectDir.GetData() + Context.SandboxRootDir.Len(),
					Context.SandboxProjectDir.Len() - Context.SandboxRootDir.Len()))
			{
				BuildUncookedPath(CookedPath, Context.SandboxProjectDir, Context.UncookedRelativeProjectDir);
			}
			else
			{
				BuildUncookedPath(CookedPath, Context.SandboxRootDir, Context.UncookedRelativeRootDir);
			}
		}
		else
		{
			TStringBuilder<1024> FullCookedFilename;
			FPathViews::ToAbsolutePath(CookedPath, FullCookedFilename);
			BuildUncookedPath(FullCookedFilename, Context.SandboxRootDir, Context.UncookedRelativeRootDir);
		}
	}
	return Context.ScratchFileName;
}

FString FCookSandbox::ConvertPackageNameToCookedPath(FStringView PackageName,
	FCookSandboxConvertCookedPathToPackageNameContext& Context) const
{
	FString UncookedFileName;
	Context.ScratchPackageName = PackageName;
	if (!FPackageName::TryConvertLongPackageNameToFilename(Context.ScratchPackageName, UncookedFileName))
	{
		return TEXT("");
	}

	FString CookedFileName;
	if (TryConvertUncookedFilenameToCookedRemappedPluginFilename(UncookedFileName, CookedFileName,
		Context.SandboxRootDir))
	{
		return CookedFileName;
	}

	auto BuildCookedPath =
		[&CookedFileName](FStringView UncookedPath, FStringView CookedRoot, FStringView UncookedRoot)
	{
		CookedFileName.Reset();
		CookedFileName.AppendChars(CookedRoot.GetData(), CookedRoot.Len());
		CookedFileName.AppendChars(UncookedPath.GetData() + UncookedRoot.Len(), UncookedPath.Len() - UncookedRoot.Len());
	};

	if (UncookedFileName.StartsWith(Context.UncookedRelativeProjectDir))
	{
		BuildCookedPath(UncookedFileName, Context.SandboxProjectDir, Context.UncookedRelativeProjectDir);
	}
	else if (UncookedFileName.StartsWith(Context.UncookedRelativeRootDir))
	{
		BuildCookedPath(UncookedFileName, Context.SandboxRootDir, Context.UncookedRelativeRootDir);
	}
	else
	{
		CookedFileName.Empty();
	}
	return CookedFileName;
}


} // namespace UE::Cook
