// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Cooker/ICookSandbox.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class IPlugin;

namespace UE::Cook
{

/**
 * A wrapper around FSandboxPlatformFile that provides a similar interface but also handles cook-specific
 * functionality like REMAPPED_PLUGINS
 */
class FCookSandbox final : public UE::Cook::ICookSandbox
{
public:
	FCookSandbox(FStringView OutputDirectory, TArray<TSharedRef<IPlugin>>& InPluginsToRemap);

	// ISandboxPlatformFile
	virtual const FString& GetSandboxDirectory() const override;
	virtual const FString& GetGameSandboxDirectoryName() const override;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) const override;
	virtual FString ConvertFromSandboxPath(const TCHAR* Filename) const override;

	// Cooker API functions that handle the Platform mapping on top of ISandboxPlatformFile
	virtual FString GetSandboxDirectory(const FString& PlatformName) const override;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename, const FString& PlatformName) const override;
	virtual FString ConvertFromSandboxPathInPlatformRoot(const TCHAR* Filename, FStringView PlatformSandboxRootDir) const override;

	// Cooker API functions on top of ISandboxPlatformFile
	FSandboxPlatformFile& GetSandboxPlatformFile(); // Not in ICookSandbox to avoid pulling in FSandboxPlatformFile
	virtual FString ConvertToFullSandboxPath(const FString& FileName, bool bForWrite) const override;
	virtual FString ConvertToFullPlatformSandboxPath(const FString& FileName, bool bForWrite,
		const FString& PlatformName) const override;
	virtual FString ConvertToFullSandboxPathInPlatformRoot(const FString& FileName, bool bForWrite,
		FStringView PlatformSandboxRootdir) const override;

	virtual bool TryConvertUncookedFilenameToCookedRemappedPluginFilename(FStringView FileName, FString& OutCookedFileName,
		FStringView PlatformSandboxRootdir = FStringView()) const override;

	virtual void FillContext(FCookSandboxConvertCookedPathToPackageNameContext& Context) const override;
	virtual FString& ConvertCookedPathToUncookedPath(FStringView CookedPath,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const override;
	virtual FName ConvertCookedPathToPackageName(FStringView CookedPath,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const override;
	virtual FString ConvertPackageNameToCookedPath(FStringView PackageName,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const override;

private:
	struct FPluginData
	{
		TSharedPtr<IPlugin> Plugin;
		FString NormalizedRootDir;
	};

	TUniquePtr<FSandboxPlatformFile> SandboxFile;
	TArray<FPluginData> PluginsToRemap;
};

} // namespace UE::Cook
