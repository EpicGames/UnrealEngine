// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookSavePackage.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/InheritedGuardValue.h"
#include "Commandlets/AssetRegistryGenerator.h"
#include "Cooker/CookActivePackage.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDeterminismManager.h"
#include "Cooker/CookDiagnostics.h"
#include "Cooker/CookEvents.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookArtifact.h"
#include "Cooker/CookImportsChecker.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookOnTheFlyServerInterface.h"
#include "Cooker/CookPackageArtifacts.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/CookRequestCluster.h"
#include "Cooker/CookRequests.h"
#include "Cooker/CookTypes.h"
#include "Cooker/PackageTracker.h"
#include "CookerSettings.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Engine/AssetManager.h"
#include "Engine/World.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/Paths.h"
#include "Misc/RedirectCollector.h"
#include "Logging/LogMacros.h"
#include "Serialization/ArchiveCookData.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.h"
#include "UObject/CookEnums.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ObjectSaveOverride.h"
#include "UObject/SavePackage.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

COREUOBJECT_API extern bool GetOutputCookingWarnings();
COREUOBJECT_API extern void SetOutputCookingWarnings(bool bValue);

#if OUTPUT_COOKTIMING

UE_TRACE_EVENT_BEGIN(UE_CUSTOM_COOKTIMER_LOG, SaveCookedPackage, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

#endif //OUTPUT_COOKTIMING

LLM_DEFINE_TAG(Cooker_SavePackage);

void UCookOnTheFlyServer::SaveCookedPackage(UE::Cook::FSaveCookedPackageContext& Context)
{
	using namespace UE::Cook;

	UE_SCOPED_HIERARCHICAL_CUSTOM_COOKTIMER_AND_DURATION(SaveCookedPackage, DetailedCookStats::TickCookOnTheSideSaveCookedPackageTimeSec)
		UE_ADD_CUSTOM_COOKTIMER_META(SaveCookedPackage, PackageName, *WriteToString<256>(Context.PackageData.GetFileName()));
	double SaveStartTime = FPlatformTime::Seconds();

	UPackage* Package = Context.Package;
	FOnScopeExit ScopedPackageFlags([Package, OriginalPackageFlags = Package->GetPackageFlags()]()
		{
			Package->SetPackageFlagsTo(OriginalPackageFlags);
		});

	Context.SetupPackage();

	TInheritedGuardValueAccessors<GetOutputCookingWarnings, SetOutputCookingWarnings>
		ScopedOutputCookerWarnings(IsCookFlagSet(ECookInitializationFlags::OutputVerboseCookerWarnings));
	// SavePackage can CollectGarbage, so we need to store the currently-unqueued PackageData in a separate variable
	// that we register for garbage collection
	TGuardValue<UE::Cook::FPackageData*> ScopedSavingPackageData(SavingPackageData, &Context.PackageData);
	TGuardValue<bool> ScopedIsSavingPackage(bIsSavingPackage, true);
	// For legacy reasons we set GIsCookerLoadingPackage == true during save. Some classes use it to conditionally
	// execute cook operations in both save and load
	TGuardValue<bool> ScopedIsCookerLoadingPackage(GIsCookerLoadingPackage, true);

	for (int32 PlatformIndex = 0; PlatformIndex < Context.PlatformsForPackage.Num(); ++PlatformIndex)
	{
		const ITargetPlatform* TargetPlatform = Context.PlatformsForPackage[PlatformIndex];
		Context.SetupPlatform(TargetPlatform, PlatformIndex);
		if (Context.bPlatformSetupSuccessful)
		{
			UE_SCOPED_HIERARCHICAL_COOKTIMER(GEditorSavePackage);
			UE_TRACK_REFERENCING_PLATFORM_SCOPED(TargetPlatform);

			TMap<UObject*, FObjectSaveOverride> SaveOverrides;
			FArchiveCookData CookData(*TargetPlatform, *Context.ArchiveCookContext);
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = Context.FlagsToCook;
			SaveArgs.bForceByteSwapping = Context.bEndianSwap;
			SaveArgs.bWarnOfLongFilename = false;
			SaveArgs.SaveFlags = Context.SaveFlags;
			SaveArgs.ArchiveCookData = &CookData;
			SaveArgs.bSlowTask = false;
			SaveArgs.SavePackageContext = Context.SavePackageContext;
			SaveArgs.InOutSaveOverrides = &SaveOverrides;

			Context.PackageWriter->UpdateSaveArguments(SaveArgs);
			FSavePackageResultStruct AuthoritativeResult = ESavePackageResult::Error;
			bool IsFirstPass = true;
			for (;;)
			{
#if !UE_AUTORTFM
				try
#endif
				{
					LLM_SCOPE_BYTAG(Cooker_SavePackage);
					UE::Cook::FScopedCookActivePackage ScopedActivePackage(Context.PackageData.GetPackageName(), NAME_None);
					if (bSkipSave)
					{
						Context.SavePackageResult = ESavePackageResult::Success;
					}
					else
					{
						Context.SavePackageResult = GEditor->Save(Package, Context.World, *Context.PlatFilename, SaveArgs);
					}
				}
#if !UE_AUTORTFM
				catch (std::exception&)
				{
					UE_LOGF(LogCook, Warning, "Tried to save package %ls for target platform %ls but threw an exception",
						*Package->GetName(), *TargetPlatform->PlatformName());
					Context.SavePackageResult = ESavePackageResult::Error;
				}
#endif

				bool IsAnotherSaveNeeded = Context.PackageWriter->IsAnotherSaveNeeded(Context.SavePackageResult, SaveArgs);
				if (IsFirstPass)
				{
					AuthoritativeResult = MoveTemp(Context.SavePackageResult);
					IsFirstPass = false;
				}
				if (IsAnotherSaveNeeded)
				{
					// We must not try a second save of a package while the first save is still in flight.
					// The optimal solution is to wait for ONLY the package that needs a second save, but we don't
					// have the bookkeeping data to do that, so we have to wait for all async package writes to complete.
					UPackage::WaitForAsyncFileWrites();
				}
				else
				{
					break;
				}
			}
			Context.SavePackageResult = MoveTemp(AuthoritativeResult);

			// If package was actually saved check with asset manager to make sure it wasn't excluded for being a
			// development or never cook package. But skip sending the warnings from this check if it was editor-only.
			if (Context.SavePackageResult == ESavePackageResult::Success)
			{
				UE_SCOPED_HIERARCHICAL_COOKTIMER(VerifyCanCookPackage);
				if (!UAssetManager::Get().VerifyCanCookPackage(this, Package->GetFName()))
				{
					Context.SavePackageResult = ESavePackageResult::Error;
				}
			}

			++this->StatSavedPackageCount;
		}

		Context.FinishPlatform();
	}

	// Need to restore flags before calling FinishPackage because it might need to save again
	ScopedPackageFlags.ExitEarly();
	Context.FinishPackage();

	constexpr double SavePackageMinDurationLogTimeSeconds = 600.;
	float SaveDurationSeconds = static_cast<float>(FPlatformTime::Seconds() - SaveStartTime);
	UE_CLOGF(SaveDurationSeconds >= SavePackageMinDurationLogTimeSeconds,
		LogCook, Display, "SavePackagePerformance: Package %ls took %.0fs to save.",
		*WriteToString<256>(Package->GetFName()), SaveDurationSeconds);
}

void UCookOnTheFlyServer::CommitUncookedPackage(UE::Cook::FSaveCookedPackageContext& Context)
{
	using namespace UE::Cook;

	UPackage* Package = Context.Package;
	FOnScopeExit ScopedPackageFlags([Package, OriginalPackageFlags = Package->GetPackageFlags()]()
		{
			Package->SetPackageFlagsTo(OriginalPackageFlags);
		});
	Context.SetupPackage();

	for (int32 PlatformIndex = 0; PlatformIndex < Context.PlatformsForPackage.Num(); ++PlatformIndex)
	{
		const ITargetPlatform* TargetPlatform = Context.PlatformsForPackage[PlatformIndex];
		Context.SetupPlatform(TargetPlatform, PlatformIndex);
		Context.SavePackageResult = ESavePackageResult::Canceled;
		Context.FinishPlatform();
	}

	Context.FinishPackage();
}

namespace UE::Cook
{

FSaveCookedPackageContext::FSaveCookedPackageContext(UCookOnTheFlyServer& InCOTFS, UE::Cook::FPackageData& InPackageData,
	TArrayView<const ITargetPlatform*> InPlatformsForPackage, UE::Cook::FTickStackData& InStackData,
	EReachability InCommitType)
	: COTFS(InCOTFS)
	, PackageData(InPackageData)
	, PlatformsForPackage(InPlatformsForPackage)
	, StackData(InStackData)
	, Package(PackageData.GetPackage())
	, PackageName(Package ? Package->GetName() : FString())
	, Filename(PackageData.GetFileName().ToString())
	, CommitType(InCommitType)
{
	PlatformDependencies.SetNum(InPlatformsForPackage.Num());
	check(CommitType == EReachability::Runtime || CommitType == EReachability::Build);
}

void FSaveCookedPackageContext::SetupPackage()
{
	check(Package && Package->IsFullyLoaded()); // PackageData should not be in the save state if Package is not fully loaded
	check(Package->GetPathName().Equals(PackageName)); // We should only be saving outermost packages, so the path name should be the same as the package name
	check(!Filename.IsEmpty()); // PackageData guarantees FileName is non-empty; if not found it should never make it into save state
	// Use SandboxFile to do path conversion to properly handle sandbox paths (outside of standard paths in particular).
	Filename = COTFS.ConvertToFullSandboxPath(*Filename, true);

	if (CommitType == EReachability::Runtime)
	{
		if (Package->HasAnyPackageFlags(PKG_ReloadingForCooker))
		{
			UE_LOGF(LogCook, Warning, "Package %ls marked as reloading for cook was requested to save", *PackageName);
			UE_LOGF(LogCook, Fatal, "Package %ls marked as reloading for cook was requested to save", *PackageName);
		}

		SaveFlags = SAVE_Async
			| (COTFS.IsCookFlagSet(ECookInitializationFlags::Unversioned) ? SAVE_Unversioned : 0);
		SaveFlags |= COTFS.IsCookFlagSet(ECookInitializationFlags::CookEditorOptional) ? SAVE_Optional : SAVE_None;

		if (COTFS.CookByTheBookOptions->bCookSoftPackageReferences)
		{
			SaveFlags |= SAVE_CookSoftPackageReferences;
		}
	}
}

void FSaveCookedPackageContext::SetupPlatform(const ITargetPlatform* InTargetPlatform, int32 InPlatformIndex)
{
	PlatformIndex = InPlatformIndex;
	TargetPlatform = InTargetPlatform;
	PlatFilename = Filename.Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
	bPlatformSetupSuccessful = false;

	// It's safe to get this cook context even if we may fail to cook this package as the context is per-platform and not per-package.
	CookContext = &COTFS.FindOrCreateSaveContext(TargetPlatform);
	SavePackageContext = &CookContext->SaveContext;
	PackageWriter = CookContext->PackageWriter;

	ArchiveCookContext.Emplace(Package,
		COTFS.IsDirectorCookByTheBook() ? UE::Cook::ECookType::ByTheBook : UE::Cook::ECookType::OnTheFly,
		COTFS.GetCookingDLC(),
		TargetPlatform, &COTFS);

	if (CommitType == EReachability::Runtime)
	{
		// Skip saving the Package, and mark it as a benign savefail result, if it is a generator that the
		// CookPackageSplitter requested not to save.
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper();
		if (GenerationHelper && !GenerationHelper->IsSaveGenerator())
		{
			SavePackageResult = ESavePackageResult::ContainsEditorOnlyData;
			const TCHAR* RejectedReason = TEXT("!bCookSaveGeneratorPackage");
			if (GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators)
			{
				UE_LOGF(LogCook, Display, "Cooking %ls, Instigator: { %ls } -> Rejected %ls", *PackageName,
					*(PackageData.GetInstigator(EReachability::Runtime).ToString()), RejectedReason);
			}
			else
			{
				UE_LOGF(LogCook, Display, "Cooking %ls -> Rejected %ls", *PackageName, RejectedReason);
			}
			return;
		}
		// Don't save Editor resources from the Engine if the target doesn't have editoronly data.
		else if (COTFS.IsCookFlagSet(ECookInitializationFlags::SkipEditorContent) &&
			(PackageName.StartsWith(TEXT("/Engine/Editor")) || PackageName.StartsWith(TEXT("/Engine/VREditor"))) &&
			!TargetPlatform->HasEditorOnlyData())
		{
			SavePackageResult = ESavePackageResult::ContainsEditorOnlyData;
			const TCHAR* RejectedReason = TEXT("EngineEditorContent");
			if (GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators)
			{
				UE_LOGF(LogCook, Display, "Cooking %ls, Instigator: { %ls } -> Rejected %ls", *PackageName,
					*(PackageData.GetInstigator(EReachability::Runtime).ToString()), RejectedReason);
			}
			else
			{
				UE_LOGF(LogCook, Display, "Cooking %ls -> Rejected %ls", *PackageName, RejectedReason);
			}
			return;
		}
		// Check whether or not game-specific behaviour should prevent this package from being cooked for the target platform
		else if (!UAssetManager::Get().ShouldCookForPlatform(Package, TargetPlatform))
		{
			SavePackageResult = ESavePackageResult::ContainsEditorOnlyData;
			const TCHAR* RejectedReason = TEXT("NotAssetManagerShouldCookForPlatform");
			if (GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators)
			{
				UE_LOGF(LogCook, Display, "Cooking %ls, Instigator: { %ls } -> Rejected %ls", *PackageName,
					*(PackageData.GetInstigator(EReachability::Runtime).ToString()), RejectedReason);
			}
			else
			{
				UE_LOGF(LogCook, Display, "Cooking %ls -> Rejected %ls", *PackageName, RejectedReason);
			}
			return;
		}
		// check if this package is unsupported for the target platform (typically plugin content)
		else
		{
			if (TSet<FName>* NeverCookPackages = COTFS.PackageTracker->PlatformSpecificNeverCookPackages.Find(TargetPlatform))
			{
				FGenerationHelper* ParentGenerationHelper =
					PackageData.IsGenerated() ? PackageData.GetParentGenerationHelper().GetReference() : nullptr;

				if (NeverCookPackages->Find(Package->GetFName()) ||
					(ParentGenerationHelper && NeverCookPackages->Find(ParentGenerationHelper->GetOwner().GetPackageName())))
				{
					SavePackageResult = ESavePackageResult::ContainsEditorOnlyData;
					const TCHAR* RejectedReason = TEXT("PlatformSpecificNeverCook");
					if (GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators)
					{
						UE_LOGF(LogCook, Display, "Cooking %ls, Instigator: { %ls } -> Rejected %ls", *PackageName,
							*(PackageData.GetInstigator(EReachability::Runtime).ToString()), RejectedReason);
					}
					else
					{
						UE_LOGF(LogCook, Display, "Cooking %ls -> Rejected %ls", *PackageName, RejectedReason);
					}
					return;
				}
			}
		}

		if (!PackageWriter->GetCookCapabilities().bIgnorePathLengthLimits)
		{
			const FString FullFilename = FPaths::ConvertRelativePathToFull(PlatFilename);
			if (FullFilename.Len() >= FPlatformMisc::GetMaxPathLength())
			{
				LogCookerMessage(FString::Printf(TEXT("Couldn't save package, filename is too long (%d >= %d): %s"),
					FullFilename.Len(), FPlatformMisc::GetMaxPathLength(), *FullFilename), EMessageSeverity::Error);
				SavePackageResult = ESavePackageResult::Error;
				return;
			}
		}

		bEndianSwap = (!TargetPlatform->IsLittleEndian()) ^ (!PLATFORM_LITTLE_ENDIAN);

		if (!TargetPlatform->HasEditorOnlyData())
		{
			Package->SetPackageFlags(PKG_FilterEditorOnly);
		}
		else
		{
			Package->ClearPackageFlags(PKG_FilterEditorOnly);
		}

		// Set platform-specific save flags
		FPackagePlatformData& PlatformData = PackageData.FindOrAddPlatformData(TargetPlatform);
		uint32 PlatformSaveFlagsMask = SAVE_AllowTimeout;
		SaveFlags = (SaveFlags & ~PlatformSaveFlagsMask);
		if (!PlatformData.IsSaveTimedOut())
		{
			// If we timedout before, do not allow another timeout, otherwise do allow it
			SaveFlags |= SAVE_AllowTimeout;
		}
	}

	if (!bHasDelayLoaded)
	{
		// look for a world object in the package (if there is one, there's a map)
		World = UWorld::FindWorldInPackage(Package);
		if (World)
		{
			FlagsToCook = RF_NoFlags;
		}
		bContainsMap = Package->ContainsMap();
		bHasDelayLoaded = true;
	}

	if (CommitType == EReachability::Runtime)
	{
		UE_CLOGF((GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators) && PlatformIndex == 0,
			LogCook, Display, "Cooking %ls, Instigator: { %ls }",
			*PackageName, *(PackageData.GetInstigator(EReachability::Runtime).ToString()));
		UE_CLOGF(GCookProgressDisplay & (int32)ECookProgressDisplayMode::PackageNames, LogCook, Display,
			"Cooking %ls", *PackageName);
	}
	else
	{
		UE_CLOGF((GCookProgressDisplay & (int32)ECookProgressDisplayMode::Instigators) && PlatformIndex == 0,
			LogCook, Display, "Committing BuildDependencies for %ls, Instigator: { %ls }",
			*PackageName, *(PackageData.GetInstigator(EReachability::Build).ToString()));
		UE_CLOGF(GCookProgressDisplay & (int32)ECookProgressDisplayMode::PackageNames, LogCook, Display,
			"Committing BuildDependencies for %ls", *PackageName);
	}

	ICookedPackageWriter::FBeginPackageInfo Info;
	Info.PackageName = Package->GetFName();
	Info.LooseFilePath = PlatFilename;
	PackageWriter->BeginPackage(Info);
	if (CookContext->DeterminismManager)
	{
		CookContext->DeterminismManager->BeginPackage(Package, TargetPlatform, PackageWriter);
	}

	// Indicate Setup was successful
	bPlatformSetupSuccessful = true;
	SavePackageResult = ESavePackageResult::Success;
}

bool IsRetryErrorCode(ESavePackageResult Result)
{
	return Result == ESavePackageResult::Timeout;
}

void FSaveCookedPackageContext::FinishPlatform()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSaveCookedPackageContext::FinishPlatform);
	check(PlatformIndex >= 0 && PlatformDependencies.IsValidIndex(PlatformIndex));

	bool bSuccessful = HasSavePackageResult() ? SavePackageResult.IsSuccessful() : false;
	ECookResult CookResult;

	// Calculate up-to-date assetregistry data for Generator and Generated packages
	TOptional<TArray<FAssetDependency>> OverridePackageDependencies;
	TOptional<FBuildResultDependenciesMap> GeneratedPackageBuildResultDependencies;
	TOptional<FAssetPackageData> AssetPackageDataBuffer;
	TOptional<FAssetPackageData> OverrideAssetPackageData;
	const FAssetPackageData* AssetPackageData = nullptr;
	TRefCountPtr<FGenerationHelper> GenerationHelper;
	bool bGenerated = false;
	if (CommitType == EReachability::Runtime)
	{
		CookResult = bSuccessful ? ECookResult::Succeeded : ECookResult::Failed;

		if (GenerationHelper = PackageData.GetGenerationHelper(); GenerationHelper)
		{
			OverridePackageDependencies.Emplace();
			GenerationHelper->FinishGeneratorPlatformSave(PackageData, PlatformIndex == 0, *OverridePackageDependencies);
		}
		else if (GenerationHelper = PackageData.GetParentGenerationHelper(); GenerationHelper)
		{
			bGenerated = true;
			OverrideAssetPackageData.Emplace();
			OverridePackageDependencies.Emplace();
			GeneratedPackageBuildResultDependencies.Emplace();
			GenerationHelper->FinishGeneratedPlatformSave(PackageData, TargetPlatform, *OverrideAssetPackageData,
				 *OverridePackageDependencies, *GeneratedPackageBuildResultDependencies);
			AssetPackageData = OverrideAssetPackageData.GetPtrOrNull();
		}
		if (!AssetPackageData)
		{
			AssetPackageDataBuffer = COTFS.AssetRegistry->GetAssetPackageDataCopy(Package->GetFName());
			AssetPackageData = AssetPackageDataBuffer.GetPtrOrNull();
		}
	}
	else
	{
		CookResult = ECookResult::NotAttempted;
	}

	// Calculate dependencies of the save if successful, or if we otherwise will need them.
	if (bPlatformSetupSuccessful || (GenerationHelper && !GenerationHelper->IsSaveGenerator()))
	{
		// ProcessUnsolicitedPackages so that we record discovereddependencies for any hidden dependency
		// packages loaded during Save,BeginCacheForCookedPlatformData, or Generator functions. These dependencies
		// are added to the runtime dependencies stored in the oplog, so we need to know them now.
		COTFS.ProcessUnsolicitedPackages();
		// Flush any logs that were logged from other threads into the Package's RecordedCookLogs so we can store
		// them in the CommitAttachments.
		COTFS.LogHandler->FlushIncrementalCookLogs();

		// Collect dependencies from all sources and record them.
		CalculatePlatformAgnosticRuntimeDependencies();
		CalculatePlatformRuntimeDependencies();
		// Note CalculateCookDependencies mutates this->SaveResult; it moves BuildResultDependencies out of it
		CalculateCookDependencies(GenerationHelper.GetReference(), bGenerated,
			OverridePackageDependencies.GetPtrOrNull() /** const, not modified */,
			GeneratedPackageBuildResultDependencies.GetPtrOrNull() /** non-const, function sets it to empty */);
		RecordPlatformBuildDependencies();
		RecordCookImportsCheckerData();
	}
	// Commit the saved bytes and the incremental cook data to the PackageWriter.
	if (bPlatformSetupSuccessful)
	{
		ICookedPackageWriter::FCommitPackageInfo Info;
		Info.Attachments = GetCommitAttachments();
		Info.Status = HasSavePackageResult() ? PackageResultToCommitStatus(SavePackageResult.Result)
			: IPackageWriter::ECommitStatus::NothingToCook;
		Info.PackageName = Package->GetFName();
		Info.PackageHash = AssetPackageData ? AssetPackageData->GetPackageSavedHash() : FIoHash();
		Info.WriteOptions = GetCommitWriteOptions();

		PackageWriter->CommitPackage(MoveTemp(Info));
		if (CookContext->DeterminismManager)
		{
			CookContext->DeterminismManager->EndPackage();
		}
	}

	// Update asset registry
	if (COTFS.IsDirectorCookByTheBook())
	{
		FPlatformData* PlatformData = COTFS.PlatformManager->GetPlatformData(TargetPlatform);
		IAssetRegistryReporter& Reporter = *(PlatformData->RegistryReporter);
		TOptional<TArray<FAssetData>> AssetDatasFromSave;
		if (bSuccessful)
		{
			AssetDatasFromSave.Emplace(MoveTemp(SavePackageResult.SavedAssets));
			// Apply the cook's tag filtering settings to the tags from SavePackage, e.g. remove tags that are allowed
			// or denied from runtime AssetRegistry by the AssetRegistry's configuration settings.
			for (FAssetData& AssetData : *AssetDatasFromSave)
			{
				FAssetRegistryState::FilterTags(AssetData, PlatformData->AssetRegistryDevelopmentSaveOptions);
			}
		}
		Reporter.UpdateAssetRegistryData(Package->GetFName(), Package, CookResult, &SavePackageResult,
			MoveTemp(AssetDatasFromSave), MoveTemp(OverrideAssetPackageData), MoveTemp(OverridePackageDependencies),
			COTFS);
	}

	if (CommitType == EReachability::Runtime)
	{
		// If not retrying, mark the package as cooked, either successfully or with failure
		bool bIsRetryErrorCode = IsRetryErrorCode(SavePackageResult.Result);
		if (!bIsRetryErrorCode)
		{
			PackageData.SetPlatformCooked(TargetPlatform, CookResult);
		}

		// Update flags used to determine garbage collection.
		if (bSuccessful)
		{
			if (bContainsMap)
			{
				StackData.ResultFlags |= UCookOnTheFlyServer::COSR_CookedMap;
			}
			else
			{
				StackData.ResultFlags |= UCookOnTheFlyServer::COSR_CookedPackage;
			}
			++COTFS.CookedPackageCountSinceLastGC;
		}

		// Accumulate results for SaveCookedPackage_Finish
		if (SavePackageResult.Result == ESavePackageResult::Timeout)
		{
			PackageData.FindOrAddPlatformData(TargetPlatform).SetSaveTimedOut(true);
			bHasTimeOut = true;
		}
		bAnySaveSucceeded |= bSuccessful;

		bHasRetryErrorCode |= bIsRetryErrorCode;
	}
	else
	{
		PackageData.SetPlatformCommitted(TargetPlatform);
	}
	ArchiveCookContext.Reset();
	PlatformIndex = -1;
}

void FSaveCookedPackageContext::FinishPackage()
{
	// If any save succeeded, add all dependencies from all platforms to the cook for the platform
	TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper();
	bool bGenerationSkippedSave = GenerationHelper && !GenerationHelper->IsSaveGenerator();
	bool bAddBuildReferences = CommitType == EReachability::Build || bAnySaveSucceeded;
	bool bAddRuntimeReferences = CommitType == EReachability::Runtime && (bAnySaveSucceeded || bGenerationSkippedSave)
		&& !COTFS.CookByTheBookOptions->bSkipSoftReferences;;
	if (bAddBuildReferences || bAddRuntimeReferences)
	{
		FName PackageFName = Package->GetFName();
		if (PlatformsForPackage.Num() == 1)
		{
			TConstArrayView<const ITargetPlatform*> ReachablePlatforms(&TargetPlatform, 1);
			if (bAddRuntimeReferences)
			{
				for (const TPair<FPackageData*, EInstigator>& DependencyPair : PlatformDependencies[0].RuntimeDependencies)
				{
					COTFS.QueueDiscoveredPackage(*DependencyPair.Key,
						FInstigator(DependencyPair.Value, PackageFName), FDiscoveredPlatformSet(ReachablePlatforms));
				}
			}
			if (bAddBuildReferences)
			{
				for (FPackageData* BuildDependency : PlatformDependencies[0].BuildDependencies)
				{
					COTFS.QueueDiscoveredPackage(*BuildDependency,
						FInstigator(EInstigator::BuildDependency, PackageFName), FDiscoveredPlatformSet(ReachablePlatforms));
				}
			}
		}
		else
		{
			TMap<FPackageData*, TMap<EInstigator, TArray<const ITargetPlatform*>>> PackagePlatformsForInstigator;
			for (int32 LocalIndex = 0; LocalIndex < PlatformsForPackage.Num(); ++LocalIndex)
			{
				const FPlatformDiscoveryData& DData = PlatformDependencies[LocalIndex];
				const ITargetPlatform* CurrentPlatform = PlatformsForPackage[LocalIndex];

				if (bAddRuntimeReferences)
				{
					// Merge the runtime dependencies for each package into a single QueueDiscoveredPackage call, if possible
					// It will not be possible if the different platforms have different instigators, so track a list of
					// platforms for each package for each instigator type.
					for (const TPair<FPackageData*, EInstigator>& PackagePlatformPair : DData.RuntimeDependencies)
					{
						TMap<EInstigator, TArray<const ITargetPlatform*>>& TargetMap =
							PackagePlatformsForInstigator.FindOrAdd(PackagePlatformPair.Key);
						TargetMap.FindOrAdd(PackagePlatformPair.Value).Add(CurrentPlatform);
					}
				}

				if (bAddBuildReferences)
				{
					// Merge the build dependencies for each package into a single QueueDiscoveredPackage call.
					for (FPackageData* BuildDependency : DData.BuildDependencies)
					{
						TMap<EInstigator, TArray<const ITargetPlatform*>>& TargetMap =
							PackagePlatformsForInstigator.FindOrAdd(BuildDependency);
						TargetMap.FindOrAdd(EInstigator::BuildDependency).Add(CurrentPlatform);
					}
				}
			}
			for (const TPair<FPackageData*, TMap<EInstigator, TArray<const ITargetPlatform*>>>& PackagePair
				: PackagePlatformsForInstigator)
			{
				for (const TPair<EInstigator, TArray<const ITargetPlatform*>>& InstigatorPair : PackagePair.Value)
				{
					COTFS.QueueDiscoveredPackage(*PackagePair.Key,
						UE::Cook::FInstigator(InstigatorPair.Key, PackageFName),
						FDiscoveredPlatformSet(InstigatorPair.Value));
				}
			}
		}
	}

	if (CommitType == EReachability::Runtime)
	{
		if (COTFS.IsDebugRecordUnsolicited())
		{
			TMap<FPackageData*, EInstigator> AllPlatformDependenciesBuffer;
			TMap<FPackageData*, EInstigator>* AllPlatformDependencies = &PlatformDependencies[0].RuntimeDependencies;
			if (PlatformsForPackage.Num() > 1)
			{
				AllPlatformDependencies = &AllPlatformDependenciesBuffer;
				for (int32 LocalIndex = 0; LocalIndex < PlatformsForPackage.Num(); ++LocalIndex)
				{
					AllPlatformDependenciesBuffer.Append(PlatformDependencies[LocalIndex].RuntimeDependencies);
				}
			}

			FDiagnostics::AnalyzeHiddenDependencies(COTFS, PackageData,
				PackageData.GetDiscoveredDependencies(nullptr /* PlatformAgnostic TargetPlatform*/),
				*AllPlatformDependencies, PlatformsForPackage, COTFS.bOnlyEditorOnlyDebug, COTFS.bHiddenDependenciesDebug);
		}

		if (!bHasRetryErrorCode)
		{
			if (COTFS.IsCookOnTheFlyMode() && PackageData.GetUrgency() != EUrgency::Blocking &&
				(!COTFS.CookOnTheFlyRequestManager || COTFS.CookOnTheFlyRequestManager->ShouldUseLegacyScheduling()))
			{
				// this is an unsolicited package
				if (FPaths::FileExists(Filename))
				{
					COTFS.PackageTracker->UnsolicitedCookedPackages.AddCookedPackage(
						FFilePlatformRequest(PackageData.GetFileName(), EInstigator::Unspecified, PlatformsForPackage));

#if DEBUG_COOKONTHEFLY
					UE_LOGF(LogCook, Display, "UnsolicitedCookedPackages: %ls", *Filename);
#endif
				}
			}
		}
	}
}

void FSaveCookedPackageContext::CalculatePlatformAgnosticRuntimeDependencies()
{
	if (bPlatformAgnosticDependenciesCalculated)
	{
		return;
	}
	bPlatformAgnosticDependenciesCalculated = true;

	FName PackageFName = PackageData.GetPackageName();
	for (FName LocalizedPackageName : FRequestCluster::GetLocalizationReferences(PackageFName, COTFS))
	{
		UE::Cook::FPackageData* LocalizedPackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(LocalizedPackageName);
		if (LocalizedPackageData)
		{
			AddDependency(PlatformAgnosticDependencies, LocalizedPackageData, false /* bHard */);
		}
	}
	// Also add any references from the package that are required by the AssetManager
	for (FName AMPackageName : FRequestCluster::GetAssetManagerReferences(PackageFName))
	{
		UE::Cook::FPackageData* AMPackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(AMPackageName);
		if (AMPackageData)
		{
			AddDependency(PlatformAgnosticDependencies, AMPackageData, false /* bHard */);
		}
	}

	// When using legacy WhatGetsCookedRules, add all the SoftObjectPaths discovered during the package's load, plus any added on
	// during save, to the cook for all platforms
	TSet<FName> SoftObjectPackages;
	GRedirectCollector.ProcessSoftObjectPathPackageList(PackageFName, false /* bGetEditorOnly */, SoftObjectPackages);
	for (FName SoftObjectPackage : SoftObjectPackages)
	{
		TMap<FSoftObjectPath, FSoftObjectPath> RedirectedPaths;

		// If this is a redirector, extract destination from asset registry
		if (COTFS.ContainsRedirector(SoftObjectPackage, RedirectedPaths))
		{
			for (TPair<FSoftObjectPath, FSoftObjectPath>& RedirectedPath : RedirectedPaths)
			{
				GRedirectCollector.AddAssetPathRedirection(RedirectedPath.Key, RedirectedPath.Value);
			}
		}

		UE::Cook::FPackageData* SoftObjectPackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(SoftObjectPackage);
		if (SoftObjectPackageData)
		{
			if (!COTFS.bSkipOnlyEditorOnly)
			{
				AddDependency(PlatformAgnosticDependencies, SoftObjectPackageData, false /* bHard */);
			}

			if (COTFS.IsDebugRecordUnsolicited())
			{
				PackageData.AddDiscoveredDependency(EDiscoveredPlatformSet::CopyFromInstigator,	SoftObjectPackageData,
					EInstigator::Unspecified);
			}
		}
	}

	// Add discovered dependencies
	TMap<FPackageData*, EInstigator>* DiscoveredDependencies = PackageData.GetDiscoveredDependencies(nullptr /* PlatformAgnostic */);
	if (DiscoveredDependencies)
	{
		for (TPair<FPackageData*, EInstigator>& Pair : *DiscoveredDependencies)
		{
			AddDependency(PlatformAgnosticDependencies, Pair.Key, false /* bHard */);
		}
	}
}

void FSaveCookedPackageContext::CalculatePlatformRuntimeDependencies()
{
	check(PlatformDependencies.IsValidIndex(PlatformIndex));
	TMap<FPackageData*, EInstigator>& CurrentDependencies = PlatformDependencies[PlatformIndex].RuntimeDependencies;
	CurrentDependencies.Reset();
	// Add PlatformAgnostic dependencies to the PlatformDependencies
	CurrentDependencies.Append(PlatformAgnosticDependencies);

	// Add imports and softobjectpaths from the save to the SaveDependencies for the current Platform and for all Platforms
	FName PackageFName = Package->GetFName();
	if (HasSavePackageResult())
	{
		TConstArrayView<const ITargetPlatform*> ReachablePlatforms(&TargetPlatform, 1);
		for (const TArray<FName>* DependencyNames : { &SavePackageResult.ImportPackages, &SavePackageResult.SoftPackageReferences })
		{
			bool bHard = DependencyNames == &SavePackageResult.ImportPackages;
			for (FName DependencyName : *DependencyNames)
			{
				UE::Cook::FPackageData* DependencyData = COTFS.PackageDatas->TryAddPackageDataByPackageName(DependencyName);
				if (DependencyData)
				{
					AddDependency(CurrentDependencies, DependencyData, bHard);
				}
			}
		}
	}

	// Add discovered dependencies
	TMap<FPackageData*, EInstigator>* DiscoveredDependencies = PackageData.GetDiscoveredDependencies(TargetPlatform);
	if (DiscoveredDependencies)
	{
		for (TPair<FPackageData*, EInstigator>& Pair : *DiscoveredDependencies)
		{
			AddDependency(CurrentDependencies, Pair.Key, false /* bHard */);
		}
	}
}

TArray<FName> FSaveCookedPackageContext::GetPlatformRuntimeDependencies() const
{
	TArray<FName> PlatformDependencyNames;
	PlatformDependencyNames.Reserve(PlatformDependencies[PlatformIndex].RuntimeDependencies.Num());
	for (const TPair<FPackageData*, EInstigator>& DependencyPair : PlatformDependencies[PlatformIndex].RuntimeDependencies)
	{
		PlatformDependencyNames.Add(DependencyPair.Key->GetPackageName());
	}
	return PlatformDependencyNames;
}

/** FArchive used to collect BuildResultDependencies from structs on UObjects in a package being recorded as a BuildDependency. */
class FBuildDependencyHarvestArchive : public UE::SavePackageUtilities::Private::FArchiveSavePackageCollector
{
public:
	FBuildDependencyHarvestArchive(UPackage* InPackage, FArchiveCookContext& ArchiveCookContext,
		FObjectSaveContextData& InObjectSaveContextData)
		: ObjectSaveContextData(InObjectSaveContextData)
		, ObjectSavePackageSerializeContext(ObjectSaveContextData)
		, ArchiveSavePackageData(ObjectSavePackageSerializeContext,
			InObjectSaveContextData.TargetPlatform, &ArchiveCookContext)
	{
		SetArchiveFlags(ArchiveSavePackageData, InPackage->HasAnyPackageFlags(PKG_FilterEditorOnly),
			(ObjectSaveContextData.SaveFlags & SAVE_Unversioned) != 0, true /* bCooking */);
	}

	// We use the empty operator<< functions defined on the parent class. The function of this class is not to interpret
	// any serialized data, it is instead only to provide structs and UObjects access to the
	// FObjectSavePackageSerializeContext API, which they get by calling
	// Archive.GetSavePackageData()->SavePackageContext.

private:
	FObjectSaveContextData ObjectSaveContextData;
	FObjectSavePackageSerializeContext ObjectSavePackageSerializeContext;
	FArchiveSavePackageData ArchiveSavePackageData;
};

void FSaveCookedPackageContext::CalculateCookDependencies(FGenerationHelper* GenerationHelper, bool bGenerated,
	const TArray<FAssetDependency>* ExtraArDependencies, FBuildResultDependenciesMap* ExtraBuildResultDependencies)
{
	if (!COTFS.bDependencyTrackingEnabled)
	{
		return;
	}

	UE_SCOPED_HIERARCHICAL_COOKTIMER(TargetDomainDependencies);
	FBuildResultDependenciesMap BuildResultDependencies;
	TArray<FName> PlatformRuntimeDependencies;
	TConstArrayView<FName> UntrackedSoftPackageReferences;
	TConstArrayView<UObject*> Imports;
	TConstArrayView<UObject*> Exports;
	TConstArrayView<UE::SavePackageUtilities::FPreloadDependency> PreloadDependencies;

	PlatformRuntimeDependencies = GetPlatformRuntimeDependencies();
	if (ExtraArDependencies)
	{
		PlatformRuntimeDependencies.Reserve(PlatformRuntimeDependencies.Num() + ExtraArDependencies->Num());
		for (const FAssetDependency& Dependency : *ExtraArDependencies)
		{
			PlatformRuntimeDependencies.Add(Dependency.AssetId.PackageName);
		}
	}
	if (ExtraBuildResultDependencies)
	{
		BuildResultDependencies.Append(MoveTemp(*ExtraBuildResultDependencies));
	}
	bool bHasSavePackageResult = HasSavePackageResult();
	FSavePackageResultStruct* LocalSavePackageResult = bHasSavePackageResult ? &SavePackageResult : nullptr;

	if (!bHasSavePackageResult)
	{
		// During cook saves of a Package, SavePackage is responsible for collecting BuildResultDependencies from
		// objects, but if this SaveCookedPackage call is instead for the recording of a BuildDependency package,
		// we need to collect them here.
		// Call OnCookEvent(ECookEvent::LoadDependencies) on every UObject in the package, and to support UStructs,
		// also serialize each object into an FArchive that identifies as a SavePackage archive and contains an
		// FArchiveCookContext.
		// Pass the reported BuildResult dependencies into FIncrementalCookAttachments::Collect.
		FObjectSaveContextData ObjectSaveContextData;
		ObjectSaveContextData.Set(Package, TargetPlatform, FPackagePath(), SaveFlags);
		ObjectSaveContextData.CookType = COTFS.GetCookType();
		ObjectSaveContextData.CookingDLC = COTFS.GetCookingDLC();
		ObjectSaveContextData.CookInfo = &COTFS;
		ObjectSaveContextData.ObjectSaveContextPhase = EObjectSaveContextPhase::CookDependencyHarvest;

		UE::Cook::FCookEventContext CookEventContext(ObjectSaveContextData);
		FBuildDependencyHarvestArchive Harvester(Package, *ArchiveCookContext, ObjectSaveContextData);

		for (FCachedObjectInOuter& CachedObjectInOuter : PackageData.GetCachedObjectsInOuter())
		{
			UObject* Object = CachedObjectInOuter.Object.Get();;
			ObjectSaveContextData.Object = Object;
			if (Object)
			{
				Object->OnCookEvent(ECookEvent::PlatformCookDependencies, CookEventContext);
				Object->Serialize(Harvester);
			}
		}

		BuildResultDependencies = MoveTemp(ObjectSaveContextData.BuildResultDependencies);
		for (FSoftObjectPath& RuntimeDependency : ObjectSaveContextData.CookRuntimeDependencies)
		{
			FName PackageDependency = RuntimeDependency.GetLongPackageFName();
			if (!PackageDependency.IsNone())
			{
				PlatformRuntimeDependencies.Add(PackageDependency);
			}
		}
	}

	if (bHasSavePackageResult)
	{
		BuildResultDependencies.Append(MoveTemp(LocalSavePackageResult->BuildResultDependencies));
		PlatformRuntimeDependencies.Append(MoveTemp(LocalSavePackageResult->SoftPackageReferences));
		UntrackedSoftPackageReferences = LocalSavePackageResult->UntrackedSoftPackageReferences;
		Imports = LocalSavePackageResult->Imports;
		Exports = LocalSavePackageResult->Exports;
		PreloadDependencies = LocalSavePackageResult->PreloadDependencies;
	}
	const FBuildResultDependenciesMap* LoadDependencies = PackageData.GetLoadDependencies();
	checkf(LoadDependencies,
		TEXT("LoadDependencies not found during save of package %s. LoadDependencies are supposted to be created by LoadPackageInQueue before entering the Save state."),
		*PackageData.GetPackageName().ToString());
	BuildResultDependencies.Append(*LoadDependencies);

	PlatformCookAttachments = FIncrementalCookAttachments::Collect(Package, TargetPlatform,
		MoveTemp(BuildResultDependencies), bHasSavePackageResult, UntrackedSoftPackageReferences, GenerationHelper,
		bGenerated, MoveTemp(PlatformRuntimeDependencies), Imports, Exports, PreloadDependencies,
		PackageData.GetLogMessages());
}

void FSaveCookedPackageContext::RecordPlatformBuildDependencies()
{
	if (!COTFS.bDependencyTrackingEnabled)
	{
		return;
	}

	TSet<FPackageData*>& CurrentDependencies = PlatformDependencies[PlatformIndex].BuildDependencies;
	TArray<FName, TInlineAllocator<10>> TransitiveBuildDependencies;
	PlatformCookAttachments.Artifacts.GetTransitiveBuildDependencies(TransitiveBuildDependencies);
	for (FName TransitivePackageName : TransitiveBuildDependencies)
	{
		FPackageData* BuildPackageData = COTFS.PackageDatas->TryAddPackageDataByPackageName(TransitivePackageName);
		if (BuildPackageData)
		{
			CurrentDependencies.Add(BuildPackageData);
		}
	}
}

void FSaveCookedPackageContext::RecordCookImportsCheckerData()
{
	if (HasSavePackageResult())
	{
		FEDLCookCheckerThreadState& CookChecker = FEDLCookCheckerThreadState::Get();
		for (UObject* Import : SavePackageResult.Imports)
		{
			CookChecker.AddImport(Import, Package);
		}
		for (UObject* Export : SavePackageResult.Exports)
		{
			CookChecker.AddExport(Export);
		}
		for (UE::SavePackageUtilities::FPreloadDependency& PreloadDependency : SavePackageResult.PreloadDependencies)
		{
			CookChecker.AddArc(PreloadDependency.TargetObject, PreloadDependency.bTargetIsSerialize,
				PreloadDependency.SourceObject, PreloadDependency.bSourceIsSerialize);
		}
	}
}

TArray<IPackageWriter::FCommitAttachmentInfo> FSaveCookedPackageContext::GetCommitAttachments()
{
	using namespace UE::Cook::Artifact;
	FAppendPackageMetaDataContext Context;
	Context.Package = Package;

	for (const TRefCountPtr<UE::Cook::ICookArtifact>& Artifact : COTFS.CookArtifacts)
	{
		Artifact->AppendPackageMetadata(Context);
	}

	if (!COTFS.bLegacyBuildDependencies)
	{
		PlatformCookAttachments.AppendCommitAttachments(Context);
	}
	if (CookContext->DeterminismManager)
	{
		CookContext->DeterminismManager->AppendCommitAttachments(Context);
	}

	TArray<IPackageWriter::FCommitAttachmentInfo> Result;
	Result.Reserve(Context.Entries.Num());
	for (FAppendPackageMetaDataContext::FEntry& PackageMetaEntry : Context.Entries)
	{
		IPackageWriter::FCommitAttachmentInfo& Info = Result.Emplace_GetRef();
		Info.Key = PackageMetaEntry.Key;
		Info.Value = MoveTemp(PackageMetaEntry.Value);
		Info.FieldStorage = PackageMetaEntry.FieldStorage;
	}
	return Result;
}

IPackageWriter::EWriteOptions FSaveCookedPackageContext::GetCommitWriteOptions() const
{
	IPackageWriter::EWriteOptions Result = IPackageWriter::EWriteOptions::None;
	if (!COTFS.bSkipSave)
	{
		Result |= IPackageWriter::EWriteOptions::Write;

		if (COTFS.IsDirectorCookByTheBook())
		{
			Result |= IPackageWriter::EWriteOptions::ComputeHash;
		}
	}
	return Result;
}

void FSaveCookedPackageContext::AddDependency(TMap<FPackageData*, EInstigator>& InDependencies,
	FPackageData* PackageData, bool bHard)
{
	EInstigator& Existing = InDependencies.FindOrAdd(PackageData, EInstigator::Unspecified);
	if (bHard)
	{
		Existing = EInstigator::SaveTimeHardDependency;
	}
	else if (Existing == EInstigator::Unspecified)
	{
		Existing = EInstigator::SoftDependency;
	}
}

bool FSaveCookedPackageContext::HasSavePackageResult() const
{
	return CommitType == EReachability::Runtime;
}
} // namespace UE::Cook
