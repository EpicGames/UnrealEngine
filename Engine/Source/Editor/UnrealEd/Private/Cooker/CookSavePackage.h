// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookPackageArtifacts.h"
#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Serialization/PackageWriter.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/ICookInfo.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"

class FSavePackageContext;
class ITargetPlatform;
class UCookOnTheFlyServer;
class UWorld;
namespace UE::Cook { enum class EReachability : uint8; }
namespace UE::Cook { struct FCookSavePackageContext; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FTickStackData; }
struct FAssetDependency;
struct FSavePackageResultStruct;

namespace UE::Cook
{

/** Local parameters and helper functions used by SaveCookedPackage */
class FSaveCookedPackageContext
{
private: // Used only by UCookOnTheFlyServer, which has private access

	FSaveCookedPackageContext(UCookOnTheFlyServer& InCOTFS, UE::Cook::FPackageData& InPackageData,
		TArrayView<const ITargetPlatform*> InPlatformsForPackage, UE::Cook::FTickStackData& StackData,
		EReachability InCommitType);

	// Hooks used by friends
	void SetupPackage();
	void SetupPlatform(const ITargetPlatform* InTargetPlatform, int32 InPlatformIndex);
	void FinishPlatform();
	void FinishPackage();

	// private helper functions
	void CalculatePlatformAgnosticRuntimeDependencies();
	void CalculatePlatformRuntimeDependencies();
	TArray<FName> GetPlatformRuntimeDependencies() const;
	void CalculateCookDependencies(FGenerationHelper* GenerationHelper, bool bGenerated,
		const TArray<FAssetDependency>* ExtraARDependencies, FBuildResultDependenciesMap* ExtraBuildResultDependencies);
	void RecordPlatformBuildDependencies();
	void RecordCookImportsCheckerData();
	bool HasSavePackageResult() const;

	TArray<IPackageWriter::FCommitAttachmentInfo> GetCommitAttachments();
	IPackageWriter::EWriteOptions GetCommitWriteOptions() const;
	static void AddDependency(TMap<FPackageData*, EInstigator>& InDependencies, FPackageData* PackageData, bool bHard);

	// General Package Data
	UCookOnTheFlyServer& COTFS;
	FPackageData& PackageData;
	TArrayView<const ITargetPlatform*> PlatformsForPackage;
	TMap<FPackageData*, EInstigator> PlatformAgnosticDependencies;
	struct FPlatformDiscoveryData
	{
		TMap<FPackageData*, EInstigator> RuntimeDependencies;
		TSet<FPackageData*> BuildDependencies;
	};
	TArray<FPlatformDiscoveryData, TInlineAllocator<1>> PlatformDependencies;
	FIncrementalCookAttachments PlatformCookAttachments;
	FTickStackData& StackData;
	UPackage* Package;
	const FString PackageName;
	FString Filename;
	uint32 SaveFlags = 0;
	bool bHasTimeOut = false;
	bool bHasRetryErrorCode = false;
	bool bPlatformAgnosticDependenciesCalculated = false;
	bool bAnySaveSucceeded = false;

	// General Package Data that is delay-loaded the first time we save a platform
	UWorld* World = nullptr;
	EObjectFlags FlagsToCook = RF_Public;
	bool bHasDelayLoaded = false;
	bool bContainsMap = false;

	// Per-platform data, only valid in PerPlatform callbacks
	const ITargetPlatform* TargetPlatform = nullptr;
	FCookSavePackageContext* CookContext = nullptr;
	// This holds cook data generated by Serialize() that isn't saved in the package
	TOptional<FArchiveCookContext> ArchiveCookContext;
	FSavePackageContext* SavePackageContext = nullptr;
	ICookedPackageWriter* PackageWriter = nullptr;
	FString PlatFilename;
	FSavePackageResultStruct SavePackageResult;
	int32 PlatformIndex = -1;
	EReachability CommitType = static_cast<EReachability>(0);
	bool bPlatformSetupSuccessful = false;
	bool bEndianSwap = false;

	friend class ::UCookOnTheFlyServer;
};

} // namespace UE::Cook;
