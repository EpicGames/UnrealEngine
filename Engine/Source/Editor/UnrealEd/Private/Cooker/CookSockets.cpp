// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookSockets.h"

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeExit.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

namespace UE::Cook::Sockets
{

void CloseSocket(FSocket*& Socket)
{
	if (Socket)
	{
		Socket->Close();
		ISocketSubsystem::Get()->DestroySocket(Socket);
		Socket = nullptr;
	}
}

FSocket* CreateListenSocket(int32& InOutPort, TSharedPtr<FInternetAddr>& OutAddr, FString& OutConnectAuthority,
	const TCHAR* SocketDebugName, FString& OutErrorReason)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	OutAddr = SocketSubsystem->GetLocalBindAddr(*GLog);

	// REMOTEMPCOOKTODO: Calculate ConnectAuthorityAddr from the proper element of GetLocalAdapterAddresses
	// rather than hardcoding to 127.0.0.1
	TSharedPtr<FInternetAddr> ConnectAuthorityAddr = SocketSubsystem->GetAddressFromString(TEXT("127.0.0.1"));

	OutAddr->SetPort(InOutPort);
	FSocket* LocalSocket = SocketSubsystem->CreateSocket(NAME_Stream, SocketDebugName, OutAddr->GetProtocolType());
	if (!LocalSocket)
	{
		OutErrorReason = TEXT("Could not create listen socket");
		return nullptr;
	}

	ON_SCOPE_EXIT
	{
		CloseSocket(LocalSocket);
	};

	LocalSocket->SetReuseAddr(false);
	LocalSocket->SetNoDelay(true);
	LocalSocket->SetNonBlocking(true);

	if (!LocalSocket->Bind(*OutAddr))
	{
		OutErrorReason = FString::Printf(TEXT("Failed to bind listen socket %s"), *OutAddr->ToString(true /* bAppendPort */));
		return nullptr;
	}
	int32 MaxBacklog = 16;
	if (!LocalSocket->Listen(MaxBacklog))
	{
		OutErrorReason = FString::Printf(TEXT("Failed to listen to socket %s"), *OutAddr->ToString(true /* bAppendPort */));
		return nullptr;
	}

	InOutPort = LocalSocket->GetPortNo();
	OutAddr->SetPort(InOutPort);
	ConnectAuthorityAddr->SetPort(InOutPort);
	OutConnectAuthority = ConnectAuthorityAddr->ToString(true /* bAppendPort */);

	FSocket* OutSocket = LocalSocket;
	LocalSocket = nullptr;
	return OutSocket;
}

FSocket* ConnectToHost(FInternetAddr& HostAddr, const TCHAR* SocketDebugName)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	FSocket* Socket = SocketSubsystem->CreateSocket(NAME_Stream, SocketDebugName, HostAddr.GetProtocolType());
	if (Socket == nullptr)
	{
		return nullptr;
	}

	Socket->SetNonBlocking(true);
	if (!Socket->Connect(HostAddr))
	{
		CloseSocket(Socket);
		return nullptr;
	}
	return Socket;
}

TSharedPtr<FInternetAddr> GetAddressFromStringWithPort(FStringView Text)
{
	int32 PortValue = -1;
	int32 PortStartIndex;
	if (Text.FindLastChar(':', PortStartIndex))
	{
		if (PortStartIndex == 0)
		{
			UE_LOGF(LogSockets, Warning, "Could not serialize %.*ls, it is missing the <Addr>: prefix.",
				Text.Len(), Text.GetData());
			return TSharedPtr<FInternetAddr>();
		}
		if (!LexTryParseString(PortValue, *WriteToString<16>(Text.RightChop(PortStartIndex + 1))))
		{
			PortValue = -1;
		}
	}
	if (PortValue < 0)
	{
		UE_LOGF(LogSockets, Warning, "Could not serialize %.*ls, it is missing the :<Port> suffix.",
			Text.Len(), Text.GetData());
		return TSharedPtr<FInternetAddr>();
	}
	
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	TSharedPtr<FInternetAddr> Result = SocketSubsystem->GetAddressFromString(FString(Text.Mid(0, PortStartIndex)));
	if (Result)
	{
		Result->SetPort(PortValue);
	}
	return Result;
}

}