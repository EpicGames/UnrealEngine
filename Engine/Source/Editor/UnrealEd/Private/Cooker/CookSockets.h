// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HAL/Platform.h"
#include "IPAddress.h"
#include "Templates/SharedPointer.h"

class FSocket;

namespace UE::Cook::Sockets
{

inline constexpr int32 COOKDIRECTOR_DEFAULT_REQUEST_CONNECTION_PORT = 41897;

/** Destruct Socket as gracefully as possible. Sets Socket to nullptr. Early exits if Socket is already null. */
void CloseSocket(FSocket*& Socket);

/**
 * Create a socket to listen for incoming connections on the given port.
 * 
 * @param InOutPort: On input, the desired listening port.
 *     On output, the selected port if desired listening port was 0, otherwise unchanged.
 * @param OutAddr: The address being listened to at the OS socket level, e.g. 0.0.0.0:<port>. 
 * @param OutConnectAuthority: The URI authority other processes should connect to, e.g. 192.168.1.10:<port>.
 * @param SocketDebugName: String used in logs sent for the socket.
 * @param OutErrorReason: Reason why creation failed if return value is nullptr.
 * 
 * @return Valid socket or nullptr on failure.
 */
FSocket* CreateListenSocket(int32& InOutPort, TSharedPtr<FInternetAddr>& OutAddr, FString& OutConnectAuthority,
	const TCHAR* SocketDebugName, FString& OutErrorReason);

/** Create a socket to create to a Host using an InternetAddr created from e.g. OutConnectURI from CreateListenSocket. */
FSocket* ConnectToHost(FInternetAddr& Host, const TCHAR* SocketDebugName);

/** Convert "<addr>:<port>"" into InternetAddr, e.g. "192.168.1.10:48000" or "2001:0db8:85a3:0000:0000:8a2e:0370:7334:48000" */
TSharedPtr<FInternetAddr> GetAddressFromStringWithPort(FStringView Text);

}