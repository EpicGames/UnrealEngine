// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookTypes.h"

#include "Containers/StringView.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookArtifactReader.h"
#include "Cooker/CookDeterminismManager.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/PackageTracker.h"
#include "DerivedDataRequest.h"
#include "Editor.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/ReverseIterate.h"
#include "Misc/StringBuilder.h"
#include "Serialization/PackageWriterToSharedBuffer.h"

LLM_DEFINE_TAG(Cooker_CachedPlatformData);
DEFINE_LOG_CATEGORY(LogCookGenerationHelper);

namespace UE::Cook
{

const TCHAR* LexToString(EStateChangeReason Reason)
{
	switch (Reason)
	{
	case EStateChangeReason::Completed: return TEXT("Completed");
	case EStateChangeReason::DoneForNow: return TEXT("DoneForNow");
	case EStateChangeReason::SaveError: return TEXT("SaveError");
	case EStateChangeReason::RecreateObjectCache: return TEXT("RecreateObjectCache");
	case EStateChangeReason::CookerShutdown: return TEXT("CookerShutdown");
	case EStateChangeReason::ReassignAbortedPackages: return TEXT("ReassignAbortedPackages");
	case EStateChangeReason::Retraction: return TEXT("Retraction");
	case EStateChangeReason::Discovered: return TEXT("Discovered");
	case EStateChangeReason::Requested: return TEXT("Requested");
	case EStateChangeReason::RequestCluster: return TEXT("RequestCluster");
	case EStateChangeReason::DirectorRequest: return TEXT("DirectorRequest");
	case EStateChangeReason::Loaded: return TEXT("Loaded");
	case EStateChangeReason::Saved: return TEXT("Saved");
	case EStateChangeReason::CookSuppressed: return TEXT("CookSuppressed");
	case EStateChangeReason::GarbageCollected: return TEXT("GarbageCollected");
	case EStateChangeReason::GeneratorPreGarbageCollected: return TEXT("GeneratorPreGarbageCollected");
	case EStateChangeReason::ForceRecook: return TEXT("ForceRecook");
	case EStateChangeReason::UrgencyUpdated: return TEXT("UrgencyUpdated");
	default: return TEXT("Invalid");
	}
}

bool IsTerminalStateChange(EStateChangeReason Reason)
{
	switch (Reason)
	{
	case EStateChangeReason::Completed: return true;
	case EStateChangeReason::SaveError: return true;
	case EStateChangeReason::CookerShutdown: return true;
	case EStateChangeReason::CookSuppressed: return true;
	default: return false;
	}
}

const TCHAR* LexToString(ESuppressCookReason Reason)
{
	switch (Reason)
	{
	case ESuppressCookReason::Invalid: return TEXT("Invalid");
	case ESuppressCookReason::NotSuppressed: return TEXT("NotSuppressed");
	case ESuppressCookReason::AlreadyCooked: return TEXT("AlreadyCooked");
	case ESuppressCookReason::NeverCook: return TEXT("NeverCook");
	case ESuppressCookReason::DoesNotExistInWorkspaceDomain: return TEXT("DoesNotExistInWorkspaceDomain");
	case ESuppressCookReason::ScriptPackage: return TEXT("ScriptPackage");
	case ESuppressCookReason::NotInCurrentPlugin: return TEXT("NotInCurrentPlugin");
	case ESuppressCookReason::Redirected: return TEXT("Redirected");
	case ESuppressCookReason::OrphanedGenerated: return TEXT("OrphanedGenerated");
	case ESuppressCookReason::LoadError: return TEXT("LoadError");
	case ESuppressCookReason::ValidationError: return TEXT("ValidationError");
	case ESuppressCookReason::SaveError: return TEXT("SaveError");
	case ESuppressCookReason::OnlyEditorOnly: return TEXT("OnlyEditorOnly");
	case ESuppressCookReason::CookCanceled: return TEXT("CookCanceled");
	case ESuppressCookReason::MultiprocessAssignmentError: return TEXT("MultiprocessAssignmentError");
	case ESuppressCookReason::RetractedByCookDirector: return TEXT("RetractedByCookDirector");
	case ESuppressCookReason::CookFilter: return TEXT("CookFilter");
	case ESuppressCookReason::NotYetReadyForRequest: return TEXT("NotYetReadyForRequest");
	case ESuppressCookReason::GeneratedPackageNeedsRequestUpdate: return TEXT("GeneratedPackageNeedsRequestUpdate");
	default: return TEXT("Invalid");
	}
}

const TCHAR* LexToString(UE::Cook::EPackageState Reason)
{
	static_assert(static_cast<int>(EPackageState::Count) == 7);
	switch (Reason)
	{
	case EPackageState::Idle: return TEXT("Idle");
	case EPackageState::Request: return TEXT("Request");
	case EPackageState::AssignedToWorker: return TEXT("AssignedToWorker");
	case EPackageState::Load: return TEXT("Load");
	case EPackageState::SaveActive: return TEXT("SaveActive");
	case EPackageState::SaveStalledRetracted: return TEXT("SaveStalledRetracted");
	case EPackageState::SaveStalledAssignedToWorker: return TEXT("SaveStalledAssignedToWorker");
	default: return TEXT("Invalid");
	}
}

const TCHAR* LexToString(UE::Cook::EPreloaderState State)
{
	static_assert(static_cast<int>(EPreloaderState::Count) == 4);
	switch (State)
	{
	case EPreloaderState::Inactive: return TEXT("Inactive");
	case EPreloaderState::PendingKick: return TEXT("PendingKick");
	case EPreloaderState::ActivePreload: return TEXT("ActivePreload");
	case EPreloaderState::ReadyForLoad: return TEXT("ReadyForLoad");
	default: return TEXT("Invalid");
	}
}

const TCHAR* LexToString(UE::Cook::ESaveSubState State)
{
	static_assert(static_cast<int>(ESaveSubState::Count) == 17);
	switch (State)
	{
	case ESaveSubState::StartSave: return TEXT("StartSave");
	case ESaveSubState::FirstCookedPlatformData_CreateObjectCache: return TEXT("FirstCookedPlatformData_CreateObjectCache");
	case ESaveSubState::FirstCookedPlatformData_CallingBegin: return TEXT("FirstCookedPlatformData_CallingBegin");
	case ESaveSubState::FirstCookedPlatformData_CheckForGenerator: return TEXT("FirstCookedPlatformData_CheckForGenerator");
	case ESaveSubState::FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded: return TEXT("FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded");
	case ESaveSubState::Generation_TryReportGenerationManifest: return TEXT("Generation_TryReportGenerationManifest");
	case ESaveSubState::Generation_QueueGeneratedPackages: return TEXT("Generation_QueueGeneratedPackages");
	case ESaveSubState::CheckForIsGenerated: return TEXT("CheckForIsGenerated");
	case ESaveSubState::Generation_PreMoveCookedPlatformData_WaitingForIsLoaded: return TEXT("Generation_PreMoveCookedPlatformData_WaitingForIsLoaded");
	case ESaveSubState::Generation_CallObjectsToMove: return TEXT("Generation_CallObjectsToMove");
	case ESaveSubState::Generation_BeginCacheObjectsToMove: return TEXT("Generation_BeginCacheObjectsToMove");
	case ESaveSubState::Generation_FinishCacheObjectsToMove: return TEXT("Generation_FinishCacheObjectsToMove");
	case ESaveSubState::Generation_CallPreSave: return TEXT("Generation_CallPreSave");
	case ESaveSubState::Generation_CallGetPostMoveObjects: return TEXT("Generation_CallGetPostMoveObjects");
	case ESaveSubState::LastCookedPlatformData_CallingBegin: return TEXT("LastCookedPlatformData_CallingBegin");
	case ESaveSubState::LastCookedPlatformData_WaitingForIsLoaded: return TEXT("LastCookedPlatformData_WaitingForIsLoaded");
	case ESaveSubState::ReadyForSave: return TEXT("ReadyForSave");
	default: return TEXT("Invalid");
	}
}

const TCHAR* LexToString(UE::Cook::EUrgency Urgency)
{
	static_assert(static_cast<int>(EUrgency::Count) == 3);
	switch (Urgency)
	{
	case EUrgency::Normal: return TEXT("Normal");
	case EUrgency::High: return TEXT("High");
	case EUrgency::Blocking: return TEXT("Blocking");
	default: return TEXT("Invalid");
	}
}

const TCHAR* LexToString(UE::Cook::ECookPhase Phase)
{
	static_assert(static_cast<int>(ECookPhase::Count) == 2);
	switch (Phase)
	{
	case ECookPhase::Cook: return TEXT("Cook");
	case ECookPhase::BuildDependencies: return TEXT("BuildDependencies");
	default: return TEXT("Invalid");
	}
}


EStateChangeReason ConvertToStateChangeReason(ESuppressCookReason Reason)
{
	switch (Reason)
	{
	case ESuppressCookReason::OrphanedGenerated: return EStateChangeReason::SaveError;
	case ESuppressCookReason::LoadError: return EStateChangeReason::SaveError;
	case ESuppressCookReason::ValidationError: return EStateChangeReason::SaveError;
	case ESuppressCookReason::SaveError: return EStateChangeReason::SaveError;
	case ESuppressCookReason::CookCanceled: return EStateChangeReason::CookerShutdown;
	case ESuppressCookReason::MultiprocessAssignmentError: return EStateChangeReason::ReassignAbortedPackages;
	case ESuppressCookReason::RetractedByCookDirector: return EStateChangeReason::Retraction;
	case ESuppressCookReason::NotYetReadyForRequest: return EStateChangeReason::Retraction;
	case ESuppressCookReason::GeneratedPackageNeedsRequestUpdate: return EStateChangeReason::Retraction;
	default: return EStateChangeReason::CookSuppressed;
	}
}

FCookerTimer::FCookerTimer(float InTimeSlice)
	: TickStartTime(FPlatformTime::Seconds()), ActionStartTime(TickStartTime)
	, TickTimeSlice(InTimeSlice), ActionTimeSlice(InTimeSlice)
{
}

FCookerTimer::FCookerTimer(EForever)
	: FCookerTimer(MAX_flt)
{
}

FCookerTimer::FCookerTimer(ENoWait)
	: FCookerTimer(0.0f)
{
}

float FCookerTimer::GetTickTimeSlice() const
{
	return TickTimeSlice;
}

double FCookerTimer::GetTickEndTimeSeconds() const
{
	return FMath::Min(TickStartTime + TickTimeSlice, MAX_flt);
}

bool FCookerTimer::IsTickTimeUp() const
{
	return IsTickTimeUp(FPlatformTime::Seconds());
}

bool FCookerTimer::IsTickTimeUp(double CurrentTimeSeconds) const
{
	return CurrentTimeSeconds - TickStartTime > TickTimeSlice;
}

double FCookerTimer::GetTickTimeRemain() const
{
	return TickTimeSlice - (FPlatformTime::Seconds() - TickStartTime);
}

double FCookerTimer::GetTickTimeTillNow() const
{
	return FPlatformTime::Seconds() - TickStartTime;
}

float FCookerTimer::GetActionTimeSlice() const
{
	return ActionTimeSlice;
}

void FCookerTimer::SetActionTimeSlice(float InTimeSlice)
{
	double TickEndTime = GetTickEndTimeSeconds();
	ActionTimeSlice = FMath::Min(InTimeSlice, FMath::Max(0, TickEndTime - ActionStartTime));
}

void FCookerTimer::SetActionStartTime()
{
	SetActionStartTime(FPlatformTime::Seconds());
}

void FCookerTimer::SetActionStartTime(double CurrentTimeSeconds)
{
	ActionStartTime = CurrentTimeSeconds;
	double TickEndTime = GetTickEndTimeSeconds();
	ActionTimeSlice = FMath::Min(ActionTimeSlice, FMath::Max(0, TickEndTime - ActionStartTime));
}

double FCookerTimer::GetActionEndTimeSeconds() const
{
	return FMath::Min(ActionStartTime + ActionTimeSlice, MAX_flt);
}

bool FCookerTimer::IsActionTimeUp() const
{
	return IsActionTimeUp(FPlatformTime::Seconds());
}

bool FCookerTimer::IsActionTimeUp(double CurrentTimeSeconds) const
{
	return CurrentTimeSeconds - ActionStartTime > ActionTimeSlice;
}

double FCookerTimer::GetActionTimeRemain() const
{
	return ActionTimeSlice - (FPlatformTime::Seconds() - ActionStartTime);
}

double FCookerTimer::GetActionTimeTillNow() const
{
	return FPlatformTime::Seconds() - ActionStartTime;
}

static uint32 SchedulerThreadTlsSlot = FPlatformTLS::InvalidTlsSlot;
void InitializeTls()
{
	if (!FPlatformTLS::IsValidTlsSlot(SchedulerThreadTlsSlot))
	{
		SchedulerThreadTlsSlot = FPlatformTLS::AllocTlsSlot();
		SetIsSchedulerThread(true);
	}
}

bool IsSchedulerThread()
{
	return FPlatformTLS::GetTlsValue(SchedulerThreadTlsSlot) != 0;
}

void SetIsSchedulerThread(bool bValue)
{
	FPlatformTLS::SetTlsValue(SchedulerThreadTlsSlot, bValue ? (void*)0x1 : (void*)0x0);
}

FCookSavePackageContext::FCookSavePackageContext(const ITargetPlatform* InTargetPlatform, TSharedPtr<ICookArtifactReader> InCookArtifactReader,
	ICookedPackageWriter* InPackageWriter, FStringView InWriterDebugName, FSavePackageSettings InSettings,
	TUniquePtr<FDeterminismManager>&& InDeterminismManager)
	: SaveContext(InTargetPlatform, InPackageWriter, MoveTemp(InSettings))
	, WriterDebugName(InWriterDebugName)
	, ArtifactReader(InCookArtifactReader)
	, PackageWriter(InPackageWriter)
	, DeterminismManager(MoveTemp(InDeterminismManager))
{
	PackageWriterCapabilities = InPackageWriter->GetCookCapabilities();
}

FCookSavePackageContext::~FCookSavePackageContext()
{
	// SaveContext destructor deletes the PackageWriter, so if we passed our writer into SaveContext, we do not delete it
	if (!SaveContext.PackageWriter)
	{
		delete PackageWriter;
	}
}

FBuildDefinitions::FBuildDefinitions()
{
	bTestPendingBuilds = FParse::Param(FCommandLine::Get(), TEXT("CookTestPendingBuilds"));
}

FBuildDefinitions::~FBuildDefinitions()
{
	Cancel();
}

void FBuildDefinitions::AddBuildDefinitionList(FName PackageName, const ITargetPlatform* TargetPlatform, TConstArrayView<UE::DerivedData::FBuildDefinition> BuildDefinitionList)
{
	using namespace UE::DerivedData;

	// TODO_BuildDefinitionList: Trigger the builds
	if (!bTestPendingBuilds)
	{
		return;
	}

	FPendingBuildData& BuildData = PendingBuilds.FindOrAdd(PackageName);
	BuildData.bTryRemoved = false; // overwrite any previous value
}

bool FBuildDefinitions::TryRemovePendingBuilds(FName PackageName)
{
	FPendingBuildData* BuildData = PendingBuilds.Find(PackageName);
	if (BuildData)
	{
		if (!bTestPendingBuilds || BuildData->bTryRemoved)
		{
			PendingBuilds.Remove(PackageName);
			return true;
		}
		else
		{
			BuildData->bTryRemoved = true;
			return false;
		}
	}

	return true;
}

void FBuildDefinitions::Wait()
{
	PendingBuilds.Empty();
}

void FBuildDefinitions::Cancel()
{
	PendingBuilds.Empty();
}

bool IsCookIgnoreTimeouts()
{
	static bool bIsIgnoreCookTimeouts = FParse::Param(FCommandLine::Get(), TEXT("CookIgnoreTimeouts"));
	return bIsIgnoreCookTimeouts;
}

TConstArrayView<const TCHAR*> GetCommandLineDelimiterStrs()
{
	static const TCHAR* Delimiters[] = { TEXT(","), TEXT("+"), TEXT(";") };
	return TConstArrayView<const TCHAR*>(Delimiters, UE_ARRAY_COUNT(Delimiters));
}

TConstArrayView<const TCHAR> GetCommandLineDelimiterChars()
{
	static const TCHAR Delimiters[] = { ',', '+', ';' };
	return TConstArrayView<TCHAR>(Delimiters, UE_ARRAY_COUNT(Delimiters));
}

FDiscoveredPlatformSet::FDiscoveredPlatformSet(EDiscoveredPlatformSet InSource)
	: Source(InSource)
{
	ConstructUnion();
}

FDiscoveredPlatformSet::FDiscoveredPlatformSet(TConstArrayView<const ITargetPlatform*> InPlatforms)
	: FDiscoveredPlatformSet(EDiscoveredPlatformSet::EmbeddedList)
{
	Platforms.Append(InPlatforms);
}

FDiscoveredPlatformSet::FDiscoveredPlatformSet(const TBitArray<>& InOrderedPlatformBits)
	: FDiscoveredPlatformSet(EDiscoveredPlatformSet::EmbeddedBitField)
{
	OrderedPlatformBits = InOrderedPlatformBits;
}

FDiscoveredPlatformSet::~FDiscoveredPlatformSet()
{
	DestructUnion();
}

FDiscoveredPlatformSet::FDiscoveredPlatformSet(const FDiscoveredPlatformSet& Other)
	: FDiscoveredPlatformSet(Other.Source)
{
	*this = Other;
}

FDiscoveredPlatformSet::FDiscoveredPlatformSet(FDiscoveredPlatformSet&& Other)
	: FDiscoveredPlatformSet(Other.Source)
{
	*this = MoveTemp(Other);
}

FDiscoveredPlatformSet& FDiscoveredPlatformSet::operator=(const FDiscoveredPlatformSet& Other)
{
	if (Source != Other.Source)
	{
		DestructUnion();
		Source = Other.Source;
		ConstructUnion();
	}
	switch (Source)
	{
	case EDiscoveredPlatformSet::EmbeddedBitField:
		OrderedPlatformBits = Other.OrderedPlatformBits;
		break;
	default:
		Platforms = Other.Platforms;
		break;
	}
	return *this;
}

FDiscoveredPlatformSet& FDiscoveredPlatformSet::operator=(FDiscoveredPlatformSet&& Other)
{
	if (Source != Other.Source)
	{
		DestructUnion();
		Source = Other.Source;
		ConstructUnion();
	}
	switch (Other.Source)
	{
	case EDiscoveredPlatformSet::EmbeddedBitField:
		OrderedPlatformBits = MoveTemp(Other.OrderedPlatformBits);
		break;
	default:
		Platforms = MoveTemp(Other.Platforms);
		break;
	}
	return *this;
}

void FDiscoveredPlatformSet::ConstructUnion()
{
	switch (Source)
	{
	case EDiscoveredPlatformSet::EmbeddedBitField:
		new (&OrderedPlatformBits) TBitArray<>();
		break;
	default:
		new (&Platforms) TArray<const ITargetPlatform*>();
		break;
	}
}

void FDiscoveredPlatformSet::DestructUnion()
{
	switch (Source)
	{
	case EDiscoveredPlatformSet::EmbeddedBitField:
		OrderedPlatformBits.~TBitArray<>();
		break;
	default:
		Platforms.~TArray<const ITargetPlatform*>();
		break;
	}
}

void FDiscoveredPlatformSet::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	if (Source != EDiscoveredPlatformSet::EmbeddedBitField)
	{
		for (const ITargetPlatform*& Existing : Platforms)
		{
			Existing = Remap[Existing];
		}
	}
}

void FDiscoveredPlatformSet::OnRemoveSessionPlatform(const ITargetPlatform* Platform, int32 RemovedIndex)
{
	if (Source == EDiscoveredPlatformSet::EmbeddedBitField)
	{
		int32 OldNumPlatforms = OrderedPlatformBits.Num();
		check(OldNumPlatforms > RemovedIndex);
		TBitArray<> NewBits(false, OldNumPlatforms - 1);
		int32 ReadIndex;
		for (ReadIndex = 0; ReadIndex < RemovedIndex; ++ReadIndex)
		{
			NewBits[ReadIndex] = OrderedPlatformBits[ReadIndex];
		}
		for (++ReadIndex; ReadIndex < OldNumPlatforms; ++ReadIndex)
		{
			NewBits[ReadIndex - 1] = OrderedPlatformBits[ReadIndex];
		}
		OrderedPlatformBits = MoveTemp(NewBits);
	}
	else
	{
		Platforms.Remove(Platform);
	}
}

void FDiscoveredPlatformSet::OnPlatformAddedToSession(const ITargetPlatform* Platform)
{
	if (Source == EDiscoveredPlatformSet::EmbeddedBitField)
	{
		OrderedPlatformBits.Add(false);
	}
}

void FDiscoveredPlatformSet::ConvertFromBitfield(TConstArrayView<const ITargetPlatform*> OrderedPlatforms)
{
	if (Source == EDiscoveredPlatformSet::EmbeddedBitField)
	{
		TArray<const ITargetPlatform*> LocalPlatforms;
		int32 NumPlatforms = OrderedPlatformBits.Num();
		check(NumPlatforms == OrderedPlatforms.Num());
		for (int32 Index = 0; Index < NumPlatforms; ++Index)
		{
			if (OrderedPlatformBits[Index])
			{
				LocalPlatforms.Add(OrderedPlatforms[Index]);
			}
		}
		DestructUnion();
		Source = EDiscoveredPlatformSet::EmbeddedList;
		ConstructUnion();
		Platforms = MoveTemp(LocalPlatforms);
	}
}

void FDiscoveredPlatformSet::ConvertToBitfield(TConstArrayView<const ITargetPlatform*> OrderedPlatforms)
{
	if (Source == EDiscoveredPlatformSet::EmbeddedBitField)
	{
		check(OrderedPlatformBits.Num() == OrderedPlatforms.Num());
	}
	else if (Source == EDiscoveredPlatformSet::EmbeddedList)
	{
		TArray<const ITargetPlatform*> LocalPlatforms = MoveTemp(Platforms);
		DestructUnion();
		Source = EDiscoveredPlatformSet::EmbeddedBitField;
		ConstructUnion();
		int32 NumPlatforms = OrderedPlatforms.Num();
		OrderedPlatformBits.Init(false, NumPlatforms);
		for (int32 Index = 0; Index < NumPlatforms; ++Index)
		{
			OrderedPlatformBits[Index] = LocalPlatforms.Contains(OrderedPlatforms[Index]);
		}
	}
}

TConstArrayView<const ITargetPlatform*> FDiscoveredPlatformSet::GetPlatforms(UCookOnTheFlyServer& COTFS,
	FInstigator* Instigator, TConstArrayView<const ITargetPlatform*> OrderedPlatforms, EReachability Reachability,
	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>>& OutBuffer) const
{
	switch (Source)
	{
	case EDiscoveredPlatformSet::EmbeddedList:
		return Platforms;
	case EDiscoveredPlatformSet::EmbeddedBitField:
	{
		OutBuffer.Reset();
		int32 NumPlatforms = OrderedPlatformBits.Num();
		check(NumPlatforms == OrderedPlatforms.Num());
		for (int32 Index = 0; Index < NumPlatforms; ++Index)
		{
			if (OrderedPlatformBits[Index])
			{
				OutBuffer.Add(OrderedPlatforms[Index]);
			}
		}
		return OutBuffer;
	}
	case EDiscoveredPlatformSet::CopyFromInstigator:
		OutBuffer.Reset();
		check(Instigator);
		FPackageData::GetReachablePlatformsForInstigator(Reachability, COTFS, Instigator->Referencer, OutBuffer);
		return OutBuffer;
	default:
		checkNoEntry();
		OutBuffer.Reset();
		return OutBuffer;
	}
}

void WriteToCompactBinary(FCbWriter& Writer, const FDiscoveredPlatformSet& Value,
	TConstArrayView<const ITargetPlatform*> OrderedReplicationPlatforms)
{
	Writer.BeginArray();
	Writer << static_cast<uint8>(Value.Source);
	switch (Value.Source)
	{
	case EDiscoveredPlatformSet::EmbeddedList:
	{
		TArray<uint8, TInlineAllocator<ExpectedMaxNumPlatforms>> PlatformIntegers;
		for (const ITargetPlatform* Platform : Value.Platforms)
		{
			if (Platform == CookerLoadingPlatformKey)
			{
				PlatformIntegers.Add(MAX_uint8);
			}
			else
			{
				int32 PlatformIndex = OrderedReplicationPlatforms.IndexOfByKey(Platform);
				check(0 <= PlatformIndex && PlatformIndex < MAX_uint8);
				PlatformIntegers.Add(static_cast<uint8>(PlatformIndex));
			}
		}
		Writer << PlatformIntegers;
		break;
	}
	case EDiscoveredPlatformSet::EmbeddedBitField:
	{
		int32 NumPlatforms = Value.OrderedPlatformBits.Num();
		check(OrderedReplicationPlatforms.Num() == NumPlatforms);
		Writer.BeginArray();
		for (int32 Index = 0; Index < NumPlatforms; ++Index)
		{
			Writer.AddBool(Value.OrderedPlatformBits[Index]);
		}
		Writer.EndArray();
		break;
	}
	case EDiscoveredPlatformSet::CopyFromInstigator:
		break;
	default:
		checkNoEntry();
		break;
	}
	Writer.EndArray();
}

bool LoadFromCompactBinary(FCbFieldView Field, FDiscoveredPlatformSet& OutValue,
	TConstArrayView<const ITargetPlatform*> OrderedReplicationPlatforms)
{
	FCbArrayView FieldAsArray = Field.AsArrayView();
	if (Field.HasError())
	{
		return false;
	}
	FCbFieldViewIterator It = FieldAsArray.CreateViewIterator();
	uint8 SourceAsInt;
	if (!LoadFromCompactBinary(*It++, SourceAsInt) || SourceAsInt >= static_cast<uint8>(EDiscoveredPlatformSet::Count))
	{
		return false;
	}
	OutValue = FDiscoveredPlatformSet(static_cast<EDiscoveredPlatformSet>(SourceAsInt));

	bool bOk = true;
	switch (OutValue.Source)
	{
	case EDiscoveredPlatformSet::EmbeddedList:
	{
		TArray<uint8, TInlineAllocator<ExpectedMaxNumPlatforms>> PlatformIntegers;
		if (!LoadFromCompactBinary(*It++, PlatformIntegers))
		{
			bOk = false;
		}
		OutValue.Platforms.Reserve(PlatformIntegers.Num());
		for (uint8 PlatformInteger : PlatformIntegers)
		{
			if (PlatformInteger == MAX_uint8)
			{
				OutValue.Platforms.Add(CookerLoadingPlatformKey);
			}
			else
			{
				if (PlatformInteger < OrderedReplicationPlatforms.Num())
				{
					OutValue.Platforms.Add(OrderedReplicationPlatforms[PlatformInteger]);
				}
				else
				{
					bOk = false;
				}
			}
		}
		break;
	}
	case EDiscoveredPlatformSet::EmbeddedBitField:
	{
		FCbArrayView BitArrayField = (*It++).AsArrayView();
		int32 NumPlatforms = BitArrayField.Num();
		if (NumPlatforms != OrderedReplicationPlatforms.Num())
		{
			bOk = false;
			OutValue = FDiscoveredPlatformSet(EDiscoveredPlatformSet::EmbeddedList);
		}
		else
		{
			OutValue.OrderedPlatformBits.Init(false, NumPlatforms);
			int32 Index = 0;
			for (FCbFieldView BoolField : BitArrayField)
			{
				OutValue.OrderedPlatformBits[Index++] = BoolField.AsBool(false);
			}
		}
		break;
	}
	case EDiscoveredPlatformSet::CopyFromInstigator:
		break;
	default:
		checkNoEntry();
		break;
	}
	return bOk;
}

FString InstigatorChainToString(TConstArrayView<FInstigator> Chain, FStringView FinalTargetToAppend)
{
	TStringBuilder<256> ChainText;
	if (Chain.Num() == 0)
	{
		ChainText << TEXT("<NoInstigator>");
	}
	else
	{
		bool bFirst = true;
		for (const FInstigator& Instigator : ReverseIterate(Chain))
		{
			if (!bFirst) ChainText << TEXT(" -> ");
			ChainText << TEXT("{ ") << Instigator.ToString() << TEXT(" }");
			bFirst = false;
		}
	}
	if (!FinalTargetToAppend.IsEmpty())
	{
		ChainText << TEXT(" -> ") << FinalTargetToAppend;
	}
	return ChainText.ToString();
}

}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FInitializeConfigSettings& Value)
{
	Writer.BeginObject();
	Writer << "OutputDirectoryOverride" << Value.OutputDirectoryOverride;
	Writer << "MaxPrecacheShaderJobs" << Value.MaxPrecacheShaderJobs;
	Writer << "MaxConcurrentShaderJobs" << Value.MaxConcurrentShaderJobs;
	Writer << "PackagesPerGC" << Value.PackagesPerGC;
	Writer << "MemoryExpectedFreedToSpreadRatio" << Value.MemoryExpectedFreedToSpreadRatio;
	Writer << "IdleTimeToGC" << Value.IdleTimeToGC;
	Writer << "MemoryMaxUsedVirtual" << Value.MemoryMaxUsedVirtual;
	Writer << "MemoryMaxUsedPhysical" << Value.MemoryMaxUsedPhysical;
	Writer << "MemoryMinFreeVirtual" << Value.MemoryMinFreeVirtual;
	Writer << "MemoryMinFreePhysical" << Value.MemoryMinFreePhysical;
	Writer << "MemoryTriggerGCAtPressureLevel" << static_cast<uint8>(Value.MemoryTriggerGCAtPressureLevel);
	Writer << "bUseSoftGC" << Value.bUseSoftGC;
	Writer << "SoftGCStartNumerator" << Value.SoftGCStartNumerator;
	Writer << "SoftGCDenominator" << Value.SoftGCDenominator;
	Writer << "SoftGCTimeFractionBudget" << Value.SoftGCTimeFractionBudget;
	Writer << "SoftGCMinimumPeriodSeconds" << Value.SoftGCMinimumPeriodSeconds;
	Writer << "MinFreeUObjectIndicesBeforeGC" << Value.MinFreeUObjectIndicesBeforeGC;
	Writer << "MaxNumPackagesBeforePartialGC" << Value.MaxNumPackagesBeforePartialGC;
	Writer << "ConfigSettingDenyList" << Value.ConfigSettingDenyList;
	Writer << "MaxAsyncCacheForType" << Value.MaxAsyncCacheForType;
	Writer << "bRandomizeCookOrder" << Value.bRandomizeCookOrder;
	// Make sure new values are added to LoadFromCompactBinary and MoveOrCopy
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FInitializeConfigSettings& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["OutputDirectoryOverride"], OutValue.OutputDirectoryOverride) & bOk;
	bOk = LoadFromCompactBinary(Field["MaxPrecacheShaderJobs"], OutValue.MaxPrecacheShaderJobs) & bOk;
	bOk = LoadFromCompactBinary(Field["MaxConcurrentShaderJobs"], OutValue.MaxConcurrentShaderJobs) & bOk;
	bOk = LoadFromCompactBinary(Field["PackagesPerGC"], OutValue.PackagesPerGC) & bOk;
	bOk = LoadFromCompactBinary(Field["MemoryExpectedFreedToSpreadRatio"], OutValue.MemoryExpectedFreedToSpreadRatio) & bOk;
	bOk = LoadFromCompactBinary(Field["IdleTimeToGC"], OutValue.IdleTimeToGC) & bOk;
	bOk = LoadFromCompactBinary(Field["MemoryMaxUsedVirtual"], OutValue.MemoryMaxUsedVirtual) & bOk;
	bOk = LoadFromCompactBinary(Field["MemoryMaxUsedPhysical"], OutValue.MemoryMaxUsedPhysical) & bOk;
	bOk = LoadFromCompactBinary(Field["MemoryMinFreeVirtual"], OutValue.MemoryMinFreeVirtual) & bOk;
	bOk = LoadFromCompactBinary(Field["MemoryMinFreePhysical"], OutValue.MemoryMinFreePhysical) & bOk;
	uint8 PressureLevelAsInt;
	if (LoadFromCompactBinary(Field["MemoryTriggerGCAtPressureLevel"], PressureLevelAsInt))
	{
		OutValue.MemoryTriggerGCAtPressureLevel = static_cast<FGenericPlatformMemoryStats::EMemoryPressureStatus>(PressureLevelAsInt);
	}
	else
	{
		OutValue.MemoryTriggerGCAtPressureLevel = FGenericPlatformMemoryStats::EMemoryPressureStatus::Unknown;
		bOk = false;
	}
	bOk = LoadFromCompactBinary(Field["bUseSoftGC"], OutValue.bUseSoftGC) & bOk;
	bOk = LoadFromCompactBinary(Field["SoftGCStartNumerator"], OutValue.SoftGCStartNumerator) & bOk;
	bOk = LoadFromCompactBinary(Field["SoftGCDenominator"], OutValue.SoftGCDenominator) & bOk;
	bOk = LoadFromCompactBinary(Field["SoftGCTimeFractionBudget"], OutValue.SoftGCTimeFractionBudget) & bOk;
	bOk = LoadFromCompactBinary(Field["SoftGCMinimumPeriodSeconds"], OutValue.SoftGCMinimumPeriodSeconds) & bOk;
	bOk = LoadFromCompactBinary(Field["MinFreeUObjectIndicesBeforeGC"], OutValue.MinFreeUObjectIndicesBeforeGC) & bOk;
	bOk = LoadFromCompactBinary(Field["MaxNumPackagesBeforePartialGC"], OutValue.MaxNumPackagesBeforePartialGC) & bOk;
	bOk = LoadFromCompactBinary(Field["ConfigSettingDenyList"], OutValue.ConfigSettingDenyList) & bOk;
	bOk = LoadFromCompactBinary(Field["MaxAsyncCacheForType"], OutValue.MaxAsyncCacheForType) & bOk;
	bOk = LoadFromCompactBinary(Field["bRandomizeCookOrder"], OutValue.bRandomizeCookOrder) & bOk;
	// Make sure new values are added to MoveOrCopy and operator<<
	return bOk;
}

namespace UE::Cook
{

template <typename SourceType, typename TargetType>
void FInitializeConfigSettings::MoveOrCopy(SourceType&& Source, TargetType&& Target)
{
	Target.OutputDirectoryOverride = MoveTempIfPossible(Source.OutputDirectoryOverride);
	Target.MaxPrecacheShaderJobs = Source.MaxPrecacheShaderJobs;
	Target.MaxConcurrentShaderJobs = Source.MaxConcurrentShaderJobs;
	Target.PackagesPerGC = Source.PackagesPerGC;
	Target.MemoryExpectedFreedToSpreadRatio = Source.MemoryExpectedFreedToSpreadRatio;
	Target.IdleTimeToGC = Source.IdleTimeToGC;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Target.MemoryMaxUsedVirtual = Source.MemoryMaxUsedVirtual;
	Target.MemoryMaxUsedPhysical = Source.MemoryMaxUsedPhysical;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Target.MemoryMinFreeVirtual = Source.MemoryMinFreeVirtual;
	Target.MemoryMinFreePhysical = Source.MemoryMinFreePhysical;
	Target.MemoryTriggerGCAtPressureLevel = Source.MemoryTriggerGCAtPressureLevel;
	Target.bUseSoftGC = Source.bUseSoftGC;
	Target.SoftGCStartNumerator = Source.SoftGCStartNumerator;
	Target.SoftGCDenominator = Source.SoftGCDenominator;
	Target.SoftGCTimeFractionBudget = Source.SoftGCTimeFractionBudget;
	Target.SoftGCMinimumPeriodSeconds = Source.SoftGCMinimumPeriodSeconds;
	Target.MinFreeUObjectIndicesBeforeGC = Source.MinFreeUObjectIndicesBeforeGC;
	Target.MaxNumPackagesBeforePartialGC = Source.MaxNumPackagesBeforePartialGC;
	Target.ConfigSettingDenyList = MoveTempIfPossible(Source.ConfigSettingDenyList);
	Target.MaxAsyncCacheForType = MoveTempIfPossible(Source.MaxAsyncCacheForType);
	Target.bRandomizeCookOrder = Source.bRandomizeCookOrder;
	// Make sure new values are added to operator<< and LoadFromCompactBinary
}

void FInitializeConfigSettings::CopyFromLocal(const UCookOnTheFlyServer& COTFS)
{
	MoveOrCopy(COTFS, *this);
}

void FInitializeConfigSettings::MoveToLocal(UCookOnTheFlyServer& COTFS)
{
	MoveOrCopy(MoveTemp(*this), COTFS);
}

void FBeginCookConfigSettings::CopyFromLocal(const UCookOnTheFlyServer& COTFS)
{
	bLegacyBuildDependencies = COTFS.bLegacyBuildDependencies;
	bLegacyBuildDependenciesDueToNoZenStore = COTFS.bLegacyBuildDependenciesDueToNoZenStore;
	bDependencyTrackingEnabled = COTFS.bDependencyTrackingEnabled;
	bCookIncrementalAllowAllClasses = COTFS.bCookIncrementalAllowAllClasses;
	FParse::Value(FCommandLine::Get(), TEXT("-CookShowInstigator="), CookShowInstigator); // We don't store this on COTFS, so reparse it from commandLine
	TSet<FName> COTFSNeverCookPackageList;
	COTFS.PackageTracker->NeverCookPackageList.GetValues(COTFSNeverCookPackageList);
	NeverCookPackageList = COTFSNeverCookPackageList.Array();
	PlatformSpecificNeverCookPackages = COTFS.PackageTracker->PlatformSpecificNeverCookPackages;
	// Make sure new values are added to SetBeginCookConfigSettings, operator<<, and LoadFromCompactBinary
}

}

bool LexTryParseString(FPlatformMemoryStats::EMemoryPressureStatus& OutValue, FStringView Text)
{
	if (Text == TEXTVIEW("None")) { OutValue = FPlatformMemoryStats::EMemoryPressureStatus::Unknown; return true; }
	if (Text == TEXTVIEW("Unknown")) { OutValue = FPlatformMemoryStats::EMemoryPressureStatus::Unknown; return true; }
	if (Text == TEXTVIEW("Nominal")) { OutValue = FPlatformMemoryStats::EMemoryPressureStatus::Nominal; return true; }
	if (Text == TEXTVIEW("Critical")) { OutValue = FPlatformMemoryStats::EMemoryPressureStatus::Critical; return true; }
	OutValue = FPlatformMemoryStats::EMemoryPressureStatus::Unknown;
	return false;
}

FString LexToString(FPlatformMemoryStats::EMemoryPressureStatus Value)
{
	switch (Value)
	{
	case FPlatformMemoryStats::EMemoryPressureStatus::Unknown: return FString(TEXTVIEW("None"));
	case FPlatformMemoryStats::EMemoryPressureStatus::Nominal: return FString(TEXTVIEW("Nominal"));
	case  FPlatformMemoryStats::EMemoryPressureStatus::Critical: return FString(TEXTVIEW("Critical"));
	default: return FString(TEXTVIEW("None"));
	}
}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FBeginCookConfigSettings& Value)
{
	Writer.BeginObject();
	Writer << "LegacyBuildDependencies" << Value.bLegacyBuildDependencies;
	Writer << "LegacyBuildDependenciesDueToNoZenStore" << Value.bLegacyBuildDependenciesDueToNoZenStore;
	Writer << "DependencyTrackingEnabled" << Value.bDependencyTrackingEnabled;
	Writer << "CookIncrementalAllowAllClasses" << Value.bCookIncrementalAllowAllClasses;
	Writer << "CookShowInstigator" << Value.CookShowInstigator;
	Writer << "NeverCookPackageList" << Value.NeverCookPackageList;
	
	Writer.BeginArray("PlatformSpecificNeverCookPackages");
	for (const TPair<const ITargetPlatform*, TSet<FName>>& Pair : Value.PlatformSpecificNeverCookPackages)
	{
		Writer.BeginObject();
		Writer << "K" << Pair.Key->PlatformName();
		Writer << "V" << Pair.Value;
		Writer.EndObject();
	}
	Writer.EndArray();
	Writer.EndObject();
	// Make sure new values are added to SetBeginCookConfigSettings, LoadFromCompactBinary, and CopyFromLocal
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FBeginCookConfigSettings& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["LegacyBuildDependencies"], OutValue.bLegacyBuildDependencies) & bOk;
	bOk = LoadFromCompactBinary(Field["LegacyBuildDependenciesDueToNoZenStore"], OutValue.bLegacyBuildDependenciesDueToNoZenStore) & bOk;
	bOk = LoadFromCompactBinary(Field["DependencyTrackingEnabled"], OutValue.bDependencyTrackingEnabled) & bOk;
	bOk = LoadFromCompactBinary(Field["CookIncrementalAllowAllClasses"], OutValue.bCookIncrementalAllowAllClasses) & bOk;
	bOk = LoadFromCompactBinary(Field["CookShowInstigator"], OutValue.CookShowInstigator) & bOk;
	bOk = LoadFromCompactBinary(Field["NeverCookPackageList"], OutValue.NeverCookPackageList) & bOk;

	ITargetPlatformManagerModule& TPM(GetTargetPlatformManagerRef());
	FCbFieldView PlatformNeverCookField = Field["PlatformSpecificNeverCookPackages"];
	{
		bOk = PlatformNeverCookField.IsArray() & bOk;
		OutValue.PlatformSpecificNeverCookPackages.Reset();
		OutValue.PlatformSpecificNeverCookPackages.Reserve(PlatformNeverCookField.AsArrayView().Num());
		for (FCbFieldView PairField : PlatformNeverCookField)
		{
			bOk &= PairField.IsObject();
			TStringBuilder<128> KeyName;
			if (LoadFromCompactBinary(PairField["K"], KeyName))
			{
				const ITargetPlatform* TargetPlatform = TPM.FindTargetPlatform(KeyName.ToView());
				if (TargetPlatform)
				{
					TSet<FName>& Value = OutValue.PlatformSpecificNeverCookPackages.FindOrAdd(TargetPlatform);
					bOk = LoadFromCompactBinary(PairField["V"], Value) & bOk;
				}
				else
				{
					UE_LOGF(LogCook, Error, "Could not find TargetPlatform \"%.*ls\" received from CookDirector.",
						KeyName.Len(), KeyName.GetData());
					bOk = false;
				}
			}
			else
			{
				bOk = false;
			}
		}
	}
	// Make sure new values are added to SetBeginCookConfigSettings, CopyFromLocal, and operator<<
	return bOk;
}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FCookByTheBookOptions& Value)
{
	Writer.BeginObject();
	// StartupPackages and SessionStartupObjects are process-specific

	Writer << "DlcName" << Value.DlcName;
	Writer << "BasedOnReleaseVersion" << Value.BasedOnReleaseVersion;
	Writer << "CreateReleaseVersion" << Value.CreateReleaseVersion;
	Writer << "SourceToLocalizedPackageVariants" << Value.SourceToLocalizedPackageVariants;
	Writer << "AllCulturesToCook" << Value.AllCulturesToCook;

	// CookTime is process-specific
	// CookStartTime is process-specific

	Writer << "StartupOptions" << static_cast<int32>(Value.StartupOptions);
	Writer << "GenerateStreamingInstallManifests" << Value.bGenerateStreamingInstallManifests;
	Writer << "ErrorOnEngineContentUse" << Value.bErrorOnEngineContentUse;
	Writer << "AllowUncookedAssetReferences" << Value.bAllowUncookedAssetReferences;
	Writer << "SkipHardReferences" << Value.bSkipHardReferences;
	Writer << "SkipSoftReferences" << Value.bSkipSoftReferences;
	Writer << "CookAgainstFixedBase" << Value.bCookAgainstFixedBase;
	Writer << "DlcLoadMainAssetRegistry" << Value.bDlcLoadMainAssetRegistry;

	// CookList is process-specific

	Writer << "CookSoftPackageReferences" << Value.bCookSoftPackageReferences;
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookByTheBookOptions& OutValue)
{
	bool bOk = Field.IsObject();
	OutValue.StartupPackages.Empty();
	OutValue.SessionStartupObjects.Empty();

	bOk = LoadFromCompactBinary(Field["DlcName"], OutValue.DlcName) & bOk;
	bOk = LoadFromCompactBinary(Field["BasedOnReleaseVersion"], OutValue.BasedOnReleaseVersion) & bOk;
	bOk = LoadFromCompactBinary(Field["CreateReleaseVersion"], OutValue.CreateReleaseVersion) & bOk;
	bOk = LoadFromCompactBinary(Field["SourceToLocalizedPackageVariants"], OutValue.SourceToLocalizedPackageVariants) & bOk;
	bOk = LoadFromCompactBinary(Field["AllCulturesToCook"], OutValue.AllCulturesToCook) & bOk;

	OutValue.CookTime = 0.;
	OutValue.CookStartTime = 0.;

	int32 LocalStartupOptions;
	bOk = LoadFromCompactBinary(Field["StartupOptions"], LocalStartupOptions) & bOk;
	OutValue.StartupOptions = static_cast<ECookByTheBookOptions>(LocalStartupOptions);
	bOk = LoadFromCompactBinary(Field["GenerateStreamingInstallManifests"], OutValue.bGenerateStreamingInstallManifests) & bOk;
	bOk = LoadFromCompactBinary(Field["ErrorOnEngineContentUse"], OutValue.bErrorOnEngineContentUse) & bOk;
	bOk = LoadFromCompactBinary(Field["AllowUncookedAssetReferences"], OutValue.bAllowUncookedAssetReferences) & bOk;
	bOk = LoadFromCompactBinary(Field["SkipHardReferences"], OutValue.bSkipHardReferences) & bOk;
	bOk = LoadFromCompactBinary(Field["SkipSoftReferences"], OutValue.bSkipSoftReferences) & bOk;
	bOk = LoadFromCompactBinary(Field["CookAgainstFixedBase"], OutValue.bCookAgainstFixedBase) & bOk;
	bOk = LoadFromCompactBinary(Field["DlcLoadMainAssetRegistry"], OutValue.bDlcLoadMainAssetRegistry) & bOk;
	bOk = LoadFromCompactBinary(Field["CookSoftPackageReferences"], OutValue.bCookSoftPackageReferences) & bOk;

	return bOk;
}

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FCookOnTheFlyOptions& Value)
{
	Writer.BeginObject();
	Writer << "Port" << Value.Port;
	Writer << "PlatformProtocol" << Value.bPlatformProtocol;
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookOnTheFlyOptions& OutValue)
{
	bool bOk = Field.IsObject();
	bOk = LoadFromCompactBinary(Field["Port"], OutValue.Port) & bOk;
	bOk = LoadFromCompactBinary(Field["PlatformProtocol"], OutValue.bPlatformProtocol) & bOk;
	return bOk;
}

void FBeginCookContextForWorkerPlatform::Set(const FBeginCookContextPlatform& InContext)
{
	bFullBuild = InContext.bFullBuild;
	TargetPlatform = InContext.TargetPlatform;
}

FCbWriter& operator<<(FCbWriter& Writer, const FBeginCookContextForWorkerPlatform& Value)
{
	Writer.BeginObject();
	Writer << "Platform" << (Value.TargetPlatform ? Value.TargetPlatform->PlatformName() : FString());
	Writer << "FullBuild" << Value.bFullBuild;
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FBeginCookContextForWorkerPlatform& OutValue)
{
	bool bOk = true;
	FString PlatformName;
	LoadFromCompactBinary(Field["Platform"], PlatformName);
	OutValue.TargetPlatform = nullptr;
	if (!PlatformName.IsEmpty())
	{
		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
		OutValue.TargetPlatform = TPM.FindTargetPlatform(*PlatformName);
		bOk = (OutValue.TargetPlatform != nullptr) & bOk;
	}
	bOk = LoadFromCompactBinary(Field["FullBuild"], OutValue.bFullBuild) & bOk;
	return bOk;
}

void FBeginCookContextForWorker::Set(const FBeginCookContext& InContext)
{
	PlatformContexts.SetNum(InContext.PlatformContexts.Num());
	for (int32 Index = 0; Index < InContext.PlatformContexts.Num(); ++Index)
	{
		PlatformContexts[Index].Set(InContext.PlatformContexts[Index]);
	}
}

FCbWriter& operator<<(FCbWriter& Writer, const FBeginCookContextForWorker& Value)
{
	Writer << Value.PlatformContexts;
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FBeginCookContextForWorker& OutValue)
{
	return LoadFromCompactBinary(Field, OutValue.PlatformContexts);
}

FMultiPackageReaderResults GetSaveExportsAndImports(UPackage* Package, UObject* Asset, FSavePackageArgs SaveArgs)
{
	uint32 OriginalPackageFlags = Package->GetPackageFlags();
	ON_SCOPE_EXIT{ Package->SetPackageFlagsTo(OriginalPackageFlags); };

	FPackageWriterToRecord* PackageWriter = new FPackageWriterToRecord(); // SavePackageContext deletes it
	const ITargetPlatform* TargetPlatform = SaveArgs.SavePackageContext ? SaveArgs.SavePackageContext->TargetPlatform : nullptr;
	FSavePackageContext SavePackageContext(TargetPlatform, PackageWriter);
	SaveArgs.SavePackageContext = &SavePackageContext;

	IPackageWriter::FBeginPackageInfo BeginInfo;
	BeginInfo.PackageName = Package->GetFName();
	PackageWriter->BeginPackage(BeginInfo);

	FString FileName = Package->GetName();
	FSavePackageResultStruct SaveResult = GEditor->Save(Package, Asset, *FileName, SaveArgs);
	FMultiPackageReaderResults Result;
	Result.Result = SaveResult.Result;
	if (Result.Result != ESavePackageResult::Success)
	{
		return Result;
	}

	ICookedPackageWriter::FCommitPackageInfo Info;
	Info.Status = IPackageWriter::ECommitStatus::Success;
	Info.PackageName = Package->GetFName();
	Info.WriteOptions = IPackageWriter::EWriteOptions::Write;
	PackageWriter->CommitPackage(MoveTemp(Info));

	for (int32 MultiOutputIndex = 0; MultiOutputIndex < PackageWriter->SavedRecord.Packages.Num(); ++MultiOutputIndex)
	{
		if (MultiOutputIndex >= UE_ARRAY_COUNT(Result.Realms))
		{
			break;
		}
		FPackageReaderResults& Realm = Result.Realms[MultiOutputIndex];

		FMemoryReaderView HeaderArchive(PackageWriter->SavedRecord.Packages[MultiOutputIndex].Buffer.GetView());
		FPackageReader PackageReader;

		Realm.bValid = PackageReader.OpenPackageFile(&HeaderArchive) &&
			PackageReader.ReadLinkerObjects(Realm.Exports, Realm.Imports, Realm.SoftPackageReferences);
	}

	return Result;
}

IPackageWriter::ECommitStatus PackageResultToCommitStatus(ESavePackageResult Result)
{
	if (IsSuccessful(Result))
	{
		return IPackageWriter::ECommitStatus::Success;
	}
	else if (Result == ESavePackageResult::Timeout)
	{
		return IPackageWriter::ECommitStatus::Canceled;
	}
	else if (Result == ESavePackageResult::ContainsEditorOnlyData)
	{
		return IPackageWriter::ECommitStatus::NothingToCook;
	}
	else
	{
		return IPackageWriter::ECommitStatus::Error;
	}
}

const TCHAR* LexToString(UE::Cook::ECookResult CookResult)
{
	using namespace UE::Cook;
	switch (CookResult)
	{
	case ECookResult::NotAttempted:
		return TEXT("NotAttempted");
	case ECookResult::Succeeded:
		return TEXT("Succeeded");
	case ECookResult::Failed:
		return TEXT("Failed");
	case ECookResult::NeverCookPlaceholder:
		return TEXT("NeverCookPlaceholder");
	case ECookResult::Invalid:
		[[fallthrough]];
	default:
		return TEXT("Invalid");
	}
}

void ParseBoolParamOrValue(const TCHAR* CommandLine, const TCHAR* ParamNameNoDashOrEquals, bool& bInOutValue)
{
	FString ParamText;
	TStringBuilder<256> ParamStr(InPlace, TEXT("-"), ParamNameNoDashOrEquals, TEXT("="));
	if (FParse::Value(CommandLine, *ParamStr, ParamText))
	{
		LexFromString(bInOutValue, *ParamText);
		return;
	}
	bool bParamWithNoValuePresent = FParse::Param(CommandLine, ParamNameNoDashOrEquals);
	if (bParamWithNoValuePresent)
	{
		bInOutValue = true;
		return;
	}
}
