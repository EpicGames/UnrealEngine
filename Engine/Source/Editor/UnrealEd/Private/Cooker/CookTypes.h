// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/PackageReader.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookProfilingPrivate.h"
#include "Cooker/MPCollector.h"
#include "CookOnTheSide/CookOnTheFlyServer.h" // ECookTickFlags
#include "DerivedDataRequestOwner.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMath.h"
#include "HAL/Platform.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/LogMacros.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/PackageWriter.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/CookEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/SavePackage.h"

class FCbFieldView;
class FCbWriter;
class ITargetPlatform;
struct FWeakObjectPtr;
namespace UE::DerivedData { class FBuildDefinition; }
namespace UE::Cook { class ICookArtifactReader; }
namespace UE::Cook { enum class EReachability : uint8; }
namespace UE::Cook { struct FBeginCookConfigSettings; }
namespace UE::Cook { struct FCookByTheBookOptions; }
namespace UE::Cook { struct FCookOnTheFlyOptions; }
namespace UE::Cook { struct FInitializeConfigSettings; }
namespace UE::Cook { struct FInstigator; }
struct FGuid;

FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FBeginCookConfigSettings& Value);
bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FBeginCookConfigSettings& OutValue);
FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FCookByTheBookOptions& Value);
bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookByTheBookOptions& OutValue);
FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FCookOnTheFlyOptions& Value);
bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookOnTheFlyOptions& OutValue);
FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FInitializeConfigSettings& Value);
bool LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FInitializeConfigSettings& OutValue);

#define COOK_CHECKSLOW_PACKAGEDATA 0
#define DEBUG_COOKONTHEFLY 0

LLM_DECLARE_TAG(Cooker_CachedPlatformData);
DECLARE_LOG_CATEGORY_EXTERN(LogCookGenerationHelper, Log, All);

inline constexpr uint32 ExpectedMaxNumPlatforms = 32;

namespace UE::Cook
{
	class FDeterminismManager;
	struct FPackageData;
	struct FPlatformData;

	/** A function that is called when a requested package finishes cooking (when successful, failed, or skipped) */
	typedef TUniqueFunction<void(FPackageData*)> FCompletionCallback;

	class FPackageDataSet : public TSet<FPackageData*>
	{
		using TSet<FPackageData*>::TSet;
	};
	template <typename ValueType>
	class TPackageDataMap : public TMap<FPackageData*, ValueType>
	{
		using TMap<FPackageData*, ValueType>::TMap;
	};

	/**
	 * External Requests to the cooker can either by cook requests for a specific file, or arbitrary callbacks that
	 * need to execute within the Scheduler's lock.
	 */
	enum class EExternalRequestType
	{
		None,
		Callback,
		Cook
	};

	/** Return type for functions called reentrantly that can succeed,fail,or be incomplete */
	enum class EPollStatus : uint8
	{
		Success,
		Error,
		Incomplete,
	};

	/** The reasons that a FPackageData can change its state, used for diagnostics and some control flow. */
	enum class EStateChangeReason : uint8
	{
		Completed,
		DoneForNow,
		SaveError,
		RecreateObjectCache,
		CookerShutdown,
		ReassignAbortedPackages,
		Retraction,
		Discovered,
		Requested,
		RequestCluster,
		DirectorRequest,
		Loaded,
		Saved,
		CookSuppressed,
		GarbageCollected,
		GeneratorPreGarbageCollected,
		ForceRecook,
		UrgencyUpdated,
	};
	bool IsTerminalStateChange(EStateChangeReason Reason);
	const TCHAR* LexToString(UE::Cook::EStateChangeReason Reason);

	enum class ESuppressCookReason : uint8
	{
		Invalid, // Used by containers for values not in container, not used in cases for passing between containers
		NotSuppressed,
		AlreadyCooked,
		NeverCook,
		DoesNotExistInWorkspaceDomain,
		ScriptPackage,
		NotInCurrentPlugin,
		Redirected,
		OrphanedGenerated,
		LoadError,
		ValidationError,
		SaveError,
		OnlyEditorOnly,
		CookCanceled,
		MultiprocessAssignmentError,
		RetractedByCookDirector,
		CookFilter,
		NotYetReadyForRequest,
		GeneratedPackageNeedsRequestUpdate,
		Count,
		BitCount = FPlatformMath::ConstExprCeilLogTwo(Count),
	};
	const TCHAR* LexToString(UE::Cook::ESuppressCookReason Reason);
	EStateChangeReason ConvertToStateChangeReason(ESuppressCookReason Reason);

	/** The type of callback for External Requests that needs to be executed within the Scheduler's lock. */
	typedef TUniqueFunction<void()> FSchedulerCallback;

	/** Which phase of cooking a Package is in.  */
	enum class EPackageState : uint8
	{
		/**
		 * The Package is not being operated on by the cooker, and is not in any queues. This is the state both for
		 * packages that have never been requested and for packages that have finished cooking.
		 */
		Idle = 0,
		/**
		 * The Package is in the RequestQueue; it is requested for cooking but has not had any operations performed
		 * on it.
		*/
		Request,
		/**
		 * The Package is in the AssignedToWorkerSet; it has been sent a remote CookWorker for cooking and has not
		 * had any operations performed on it locally.
		 */
		AssignedToWorker,
		/** The Package is in the LoadQueue, in one of multiple substates that handle loading and preloading. */
		Load,
		/** The Package is in the SaveQueue; it has been fully loaded and some target data may have been calculated. */
		SaveActive,
		/**
		 * The Package is in the SaveStalled Set. It might have Saving data, but it has been retracted by the CookDirector
		 * and has not yet completed save elsewhere. It will stay in this stalled state until the CookDirector reassigns
		 * it back to this worker or reports that its save was completed elsewhere.
		 */
		SaveStalledRetracted,
		/**
		 * The Package is in the SaveStalled Set. We are on the CookDirector and the package was previously assigned
		 * locally for saving on the Director, but we retracted it from saving locally and assigned it to a remote
		 * CookWorker. It will stay in this stalled state until COTFS.Director reassigns it back for local saving or a
		 * worker reports that it finished saving.
		 */
		SaveStalledAssignedToWorker,

		Min = Idle,
		Max = SaveStalledAssignedToWorker,
		/** Number of values in this enum, not a valid value for any EPackageState variable. */
		Count = Max + 1,
		/** Number of bits required to store a valid EPackageState */
		BitCount = FPlatformMath::ConstExprCeilLogTwo(Count),
	};
	const TCHAR* LexToString(UE::Cook::EPackageState State);

	enum class EPackageStateProperty // Bitfield
	{
		None		= 0,
		/** The package is being worked on by the cooker. */
		InProgress	= 0x1,
		/** The package is in one of the loading states and has preload data. */
		Loading		= 0x2,
		/**
		 * The package is in one of the saving states and has access to saving-only data. The UPackage pointer on
		 * the FPackageData is non-null.
		 */
		Saving		= 0x4,
		/** The package is assigned to a remote worker, and here on the director it is in a stalled state. */
		AssignedToWorkerProperty = 0x8,

		Min = InProgress,
		Max = AssignedToWorkerProperty,
	};
	ENUM_CLASS_FLAGS(EPackageStateProperty);

	/**
	 * A substate of EPackageState::Load; it describes the state of the PackagePreloader in PumpLoads.
	 * This state is on the PackagePreloader and not the PackageData, and might be active even while the package
	 * is not in the load state.
	 */
	enum class EPreloaderState : uint8
	{
		Inactive,
		PendingKick,
		ActivePreload,
		ReadyForLoad,
		Count,
	};
	const TCHAR* LexToString(UE::Cook::EPreloaderState State);

	/** SubState when in a Saving state. */
	enum class ESaveSubState : uint8
	{
		StartSave = 0,
		FirstCookedPlatformData_CreateObjectCache,
		FirstCookedPlatformData_CallingBegin,
		FirstCookedPlatformData_CheckForGenerator,
		FirstCookedPlatformData_CheckForGeneratorAfterWaitingForIsLoaded,
		Generation_TryReportGenerationManifest,
		Generation_QueueGeneratedPackages,

		CheckForIsGenerated,

		Generation_PreMoveCookedPlatformData_WaitingForIsLoaded,
		Generation_CallObjectsToMove,
		Generation_BeginCacheObjectsToMove,
		Generation_FinishCacheObjectsToMove,
		Generation_CallPreSave,
		Generation_CallGetPostMoveObjects,

		LastCookedPlatformData_CallingBegin,
		LastCookedPlatformData_WaitingForIsLoaded,

		ReadyForSave,
		Last = ReadyForSave,
		Count = Last + 1,
		/** Number of bits required to store a valid ESaveSubState */
		BitCount = FPlatformMath::ConstExprCeilLogTwo(Count),
	};
	const TCHAR* LexToString(UE::Cook::ESaveSubState State);

	/** How quickly we should push a PackageData through the cook, compared to other PackageDatas. */
	enum class EUrgency : uint8
	{
		Normal = 0,
		High,
		Blocking,

		Min = Normal,
		Max = Blocking,
		Count = Max + 1,
		BitCount = FPlatformMath::ConstExprCeilLogTwo(Count),
	};
	const TCHAR* LexToString(UE::Cook::EUrgency Urgency);

	/**
	 * Which phase the cook is in. CookPhases change the rules for how the Cooker follows dependencies and what steps
	 * it takes to commit packages. CookPhase is different from CookMode: CookMode is constant for the entire process,
	 * while CookPhase can change throughout a single CookSession.
	 */
	enum class ECookPhase : uint8
	{
		/**
		 * Packages are Saved into the TargetPlatform, and the runtime and build dependencies they report are
		 * transitively followed, respectively, to add other packages to the cook or to add CookDependencies to the
		 * oplog. The Saved packages and their Cook Metadata is Committed into the Oplog. Only Cooked (aka
		 * runtime-referenced) packages are committed during this phase.
		 */
		Cook,
		/**
		 * Packages are Loaded but not Saved. The build dependencies they report are transitively followed to add
		 * CookDependencies to the oplog. The Cook Metadata of each package is Committed into the Oplog. Only
		 * non-Cooked packages that are needed as transitive build dependencies of Cooked packages are Committed during
		 * this phase.
		 */
		BuildDependencies,
		Count,
	};
	const TCHAR* LexToString(UE::Cook::ECookPhase Phase);

	/** Used as a helper to timeslice cooker functions. */
	struct FCookerTimer
	{
	public:
		enum EForever
		{
			Forever
		};
		enum ENoWait
		{
			NoWait
		};

		FCookerTimer(float InTimeSlice);
		FCookerTimer(EForever);
		FCookerTimer(ENoWait);

		float GetTickTimeSlice() const;
		double GetTickEndTimeSeconds() const;
		bool IsTickTimeUp() const;
		bool IsTickTimeUp(double CurrentTimeSeconds) const;
		double GetTickTimeRemain() const;
		double GetTickTimeTillNow() const;

		float GetActionTimeSlice() const;
		void SetActionTimeSlice(float InTimeSlice);
		void SetActionStartTime();
		void SetActionStartTime(double CurrentTimeSeconds);
		double GetActionEndTimeSeconds() const;
		bool IsActionTimeUp() const;
		bool IsActionTimeUp(double CurrentTimeSeconds) const;
		double GetActionTimeRemain() const;
		double GetActionTimeTillNow() const;

	public:
		const double TickStartTime;
		double ActionStartTime;
		const float TickTimeSlice;
		float ActionTimeSlice;
	};

	/** Temporary-lifetime data about the current tick of the cooker. */
	struct FTickStackData
	{
		/** Time at which the current iteration of the DecideCookAction loop started. */
		double LoopStartTime = 0.;
		/** A bitmask of flags of type enum ECookOnTheSideResult that were set during the tick. */
		uint32 ResultFlags = 0;
		/**
		 * The CookerTimer for the current tick. Used by slow reentrant operations that need to check whether they
		 * have timed out.
		 */
		FCookerTimer Timer;
		/** CookFlags describing details of the caller's desired behavior for the current tick. */
		ECookTickFlags TickFlags;

		bool bCookComplete = false;
		bool bCookCancelled = false;

		explicit FTickStackData(float TimeSlice, ECookTickFlags InTickFlags)
			:Timer(TimeSlice), TickFlags(InTickFlags)
		{
		}
	};

	/** 
	* Context data passed into SavePackage for a given TargetPlatform. Constant across packages, and internal
	* to the cooker.
	*/
	struct FCookSavePackageContext
	{
		FCookSavePackageContext(const ITargetPlatform* InTargetPlatform, TSharedPtr<ICookArtifactReader> InCookArtifactReader,
			ICookedPackageWriter* InPackageWriter, FStringView InWriterDebugName, FSavePackageSettings InSettings,
			TUniquePtr<FDeterminismManager>&& InDeterminismManager);
		~FCookSavePackageContext();

		FSavePackageContext SaveContext;
		FString WriterDebugName;
		TSharedPtr<ICookArtifactReader> ArtifactReader;
		ICookedPackageWriter* PackageWriter;
		ICookedPackageWriter::FCookCapabilities PackageWriterCapabilities;
		TUniquePtr<FDeterminismManager> DeterminismManager;
		TSet<IPlugin*> EnabledPlugins;
		/**
		 * RefCount Pointers to the EnabledPlugins. We pass them to external API as a TSet of raw pointers,
		 * but need to keep them referenced.
		 */
		TArray<TSharedRef<IPlugin>> EnabledPluginRefPtrs;
	};

	/**
	 * Category for in which session a package was cooked. When cooking a project normally, every cooked package will be
	 * cooked (or incrementally skipped) in the current session. But for DLC packages some packages may have been
	 * previously cooked in the basegame or other DLCs. Whether a package was incrementally skipped does not affect this
	 * category. If not cooked for a different basegame or plugin, incrementally skipped packages are still marked as
	 * cooked in the current session.
	 */
	enum class EWhereCooked : uint8
	{
		/** Cooked in the current session, the default. */
		ThisSession,
		/** Current session is a DLC cook based on a basegame cook, and the package was cooked in the basegame. */
		BaseGame,
		/**
		 * We were instructed by commandline argument to treat these packages as already cooked, without being told
		 * in which other session they were cooked.
		 */
		ExtraReleaseVersionAssets,

		Count,
		NumBits = FPlatformMath::ConstExprCeilLogTwo(EWhereCooked::Count),
	};

	/**
	 * Data identifiying a package for fetching Metadata about the package recorded in an earlier cook
	 * (e.g. FIncrementalCookAttachments).
	 */
	struct FPackageIncrementalCookId
	{
		FName PackageName;
		EWhereCooked WhereCooked;
	};

	/* Thread Local Storage access to identify which thread is the SchedulerThread for cooking. */
	void InitializeTls();
	bool IsSchedulerThread();
	void SetIsSchedulerThread(bool bValue);


	/** Placeholder to handle executing BuildDefinitions for requested but not-yet-loaded packages. */
	class FBuildDefinitions
	{
	public:
		FBuildDefinitions();
		~FBuildDefinitions();

		void AddBuildDefinitionList(FName PackageName, const ITargetPlatform* TargetPlatform,
			TConstArrayView<UE::DerivedData::FBuildDefinition> BuildDefinitionList);
		bool TryRemovePendingBuilds(FName PackageName);
		void Wait();
		void Cancel();

	private:
		bool bTestPendingBuilds = false;
		struct FPendingBuildData
		{
			bool bTryRemoved = false;
		};
		TMap<FName, FPendingBuildData> PendingBuilds;
	};

	struct FInitializeConfigSettings
	{
	public:
		void LoadLocal(const FString& InOutputDirectoryOverride);
		void CopyFromLocal(const UCookOnTheFlyServer& COTFS);
		void MoveToLocal(UCookOnTheFlyServer& COTFS);
	private:
		template <typename SourceType, typename TargetType>
		void MoveOrCopy(SourceType&& Source, TargetType&& Target);

	public:
		FString OutputDirectoryOverride;
		int32 MaxPrecacheShaderJobs = 0;
		int32 MaxConcurrentShaderJobs = 0;
		uint32 PackagesPerGC = 0;
		float MemoryExpectedFreedToSpreadRatio = 0.f;
		double IdleTimeToGC = 0.;
		uint64 MemoryMaxUsedVirtual = 0;
		uint64 MemoryMaxUsedPhysical = 0;
		uint64 MemoryMinFreeVirtual = 0;
		uint64 MemoryMinFreePhysical = 0;
		FGenericPlatformMemoryStats::EMemoryPressureStatus MemoryTriggerGCAtPressureLevel;
		int32 MinFreeUObjectIndicesBeforeGC = 0;
		int32 MaxNumPackagesBeforePartialGC = 0;
		int32 SoftGCStartNumerator = 0;
		int32 SoftGCDenominator = 1;
		float SoftGCTimeFractionBudget = 0.f;
		float SoftGCMinimumPeriodSeconds = 0.f;
		TArray<FString> ConfigSettingDenyList;
		/** max number of objects of a specific type which are allowed to async cache at once */
		TMap<FName, int32> MaxAsyncCacheForType;
		bool bUseSoftGC = false;
		bool bRandomizeCookOrder = false;

		friend FCbWriter& ::operator<<(FCbWriter& Writer, const UE::Cook::FInitializeConfigSettings& Value);
		friend bool ::LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FInitializeConfigSettings& Value);
	};

	struct FBeginCookConfigSettings
	{
		/** Initialize NeverCookPackageList from packaging settings and platform-specific sources. */
		void LoadLocal(FBeginCookContext& BeginContext);
		void LoadNeverCookLocal(FBeginCookContext& BeginContext);
		void CopyFromLocal(const UCookOnTheFlyServer& COTFS);

		FString CookShowInstigator;
		bool bLegacyBuildDependencies = false;
		bool bLegacyBuildDependenciesDueToNoZenStore = false;
		bool bDependencyTrackingEnabled = true;
		bool bCookIncrementalAllowAllClasses = false;
		TArray<FName> NeverCookPackageList;
		TMap<const ITargetPlatform*, TSet<FName>> PlatformSpecificNeverCookPackages;

		friend FCbWriter& ::operator<<(FCbWriter& Writer, const UE::Cook::FBeginCookConfigSettings& Value);
		friend bool ::LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FBeginCookConfigSettings& Value);
	};

	/** Report whether commandline/config has disabled use of timeouts throughout the cooker, useful for debugging. */
	bool IsCookIgnoreTimeouts();

	TConstArrayView<const TCHAR*> GetCommandLineDelimiterStrs();
	TConstArrayView<TCHAR> GetCommandLineDelimiterChars();

	extern FGuid CookIncrementalVersion;
	extern FGuid ProjectCookIncrementalVersion;

	constexpr FStringView MPCookTransferRootRelPath(TEXTVIEW("MPCookTransfer"));
} // namespace UE::Cook

bool LexTryParseString(FPlatformMemoryStats::EMemoryPressureStatus& OutValue, FStringView Text);
FString LexToString(FPlatformMemoryStats::EMemoryPressureStatus Value);

//////////////////////////////////////////////////////////////////////////
// Cook by the book options

namespace UE::Cook
{

struct FCookByTheBookOptions
{
public:
	// Process-lifetime variables
	TSet<FName>						StartupPackages;

	// Session-lifetime variables
	/**
	 * The list of UObjects that existed at the start of the cook. This is used to tell which UObjects
	 * were created during the cook.
	 */
	TArray<FWeakObjectPtr>			SessionStartupObjects;

	/** DlcName setup if we are cooking dlc will be used as the directory to save cooked files to */
	FString							DlcName;

	/** If cooking DLC or a patch, the release version of the basegame it is based on. */
	FString							BasedOnReleaseVersion;

	/** Create a release from this manifest and store it in the releases directory for this game. */
	FString							CreateReleaseVersion;

	/**
	 * Mapping from source packages to their localized variants (based on the culture list in
	 * FCookByTheBookStartupOptions)
	 */
	TMap<FName, TArray<FName>>		SourceToLocalizedPackageVariants;
	/** List of all the cultures (e.g. "en") that need to be cooked */
	TArray<FString>					AllCulturesToCook;

	/** Timing information about cook by the book */
	double							CookTime = 0.0;
	double							CookStartTime = 0.0;

	ECookByTheBookOptions			StartupOptions = ECookByTheBookOptions::None;

	/** Should we generate streaming install manifests (only valid option in cook by the book) */
	bool							bGenerateStreamingInstallManifests = false;

	/** Should we generate a separate manifest for map dependencies */
	bool							bGenerateDependenciesForMaps = false;

	/** error when detecting engine content being used in this cook */
	bool							bErrorOnEngineContentUse = false;
	/** this is a flag for dlc, will allow DLC to be cook when the fixed base might be missing references. */
	bool							bAllowUncookedAssetReferences = false;
	bool							bSkipHardReferences = false;
	bool							bSkipSoftReferences = false;
	bool							bCookSoftPackageReferences = false;
	bool							bCookAgainstFixedBase = false;
	bool							bDlcLoadMainAssetRegistry = false;
	/** True when performing a DLC plugin only cook, always ignores references outside the DLC path. */
	bool							bDlcPluginOnly = false;
	/** True if CookByTheBook is being run in cooklist mode and will not be loading/saving packages. */
	bool							bCookList = false;

	void ClearSessionData()
	{
		FCookByTheBookOptions EmptyOptions;
		// Preserve Process-lifetime variables
		EmptyOptions.StartupPackages = MoveTemp(StartupPackages);
		*this = MoveTemp(EmptyOptions);
	}

	friend FCbWriter& ::operator<<(FCbWriter& Writer, const UE::Cook::FCookByTheBookOptions& Value);
	friend bool ::LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookByTheBookOptions& Value);
};

//////////////////////////////////////////////////////////////////////////
// Cook on the fly startup options
struct FCookOnTheFlyOptions
{
	/** What port the network file server or the I/O store connection server should bind to */
	int32 Port = -1;
	/**
	 * Whether the network file server should use a platform-specific communication protocol instead of TCP (used when
	 * bZenStore == false)
	 */
	bool bPlatformProtocol = false;

	friend FCbWriter& ::operator<<(FCbWriter& Writer, const UE::Cook::FCookOnTheFlyOptions& Value);
	friend bool ::LoadFromCompactBinary(FCbFieldView Field, UE::Cook::FCookOnTheFlyOptions& Value);
};

/** Enum used by FDiscoveredPlatformSet to specify what source it will use for the set of platforms. */
enum class EDiscoveredPlatformSet
{
	EmbeddedList,
	EmbeddedBitField,
	CopyFromInstigator,
	Count,
};

/**
 * A provider of a set of platforms to mark reachable for a discovered package. It might be an embedded list or
 * it might hold instructions for where to get the platforms from other context data.
 */
struct FDiscoveredPlatformSet
{
	FDiscoveredPlatformSet(EDiscoveredPlatformSet InSource = EDiscoveredPlatformSet::EmbeddedList);
	explicit FDiscoveredPlatformSet(TConstArrayView<const ITargetPlatform*> InPlatforms);
	explicit FDiscoveredPlatformSet(const TBitArray<>& InOrderedPlatformBits);
	~FDiscoveredPlatformSet();
	FDiscoveredPlatformSet(const FDiscoveredPlatformSet& Other);
	FDiscoveredPlatformSet(FDiscoveredPlatformSet&& Other);
	FDiscoveredPlatformSet& operator=(const FDiscoveredPlatformSet& Other);
	FDiscoveredPlatformSet& operator=(FDiscoveredPlatformSet&& Other);

	EDiscoveredPlatformSet GetSource() const { return Source; }

	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);
	void OnRemoveSessionPlatform(const ITargetPlatform* Platform, int32 RemovedIndex);
	void OnPlatformAddedToSession(const ITargetPlatform* Platform);
	TConstArrayView<const ITargetPlatform*> GetPlatforms(UCookOnTheFlyServer& COTFS,
		FInstigator* Instigator, TConstArrayView<const ITargetPlatform*> OrderedPlatforms, EReachability Reachability,
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>>& OutBuffer) const;
	/** If the current type is EmbeddedBitField, change it to EmbeddedList. */
	void ConvertFromBitfield(TConstArrayView<const ITargetPlatform*> OrderedPlatforms);
	/**
	 * If the current type is EmbeddedList, change it to EmbeddedBitfield. Asserts if the type is already
	 * EmbeddedBitfield.
	 */
	void ConvertToBitfield(TConstArrayView<const ITargetPlatform*> OrderedPlatforms);

private:
	void DestructUnion();
	void ConstructUnion();
	friend void WriteToCompactBinary(FCbWriter& Writer, const FDiscoveredPlatformSet& Value,
		TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms);
	friend bool LoadFromCompactBinary(FCbFieldView Field, FDiscoveredPlatformSet& OutValue,
		TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms);

	union 
	{
		TArray<const ITargetPlatform*> Platforms;
		TBitArray<> OrderedPlatformBits;
	};
	EDiscoveredPlatformSet Source;
};

/** Platform-specific data about each ICookArtifact during cook start. */
struct FCookArtifactsCurrentSettings
{
	FConfigFile Settings;
	bool bRequestInvalidate = false;
};

FString InstigatorChainToString(TConstArrayView<FInstigator> Chain, FStringView FinalTargetToAppend = FStringView());

} // namespace UE::Cook


/** Helper struct for FBeginCookContext; holds the context data for each platform being cooked */
struct FBeginCookContextPlatform
{
	ITargetPlatform* TargetPlatform = nullptr;
	UE::Cook::FPlatformData* PlatformData = nullptr;
	TMap<UE::Cook::ICookArtifact*, UE::Cook::FCookArtifactsCurrentSettings> CookArtifactsCurrentSettings;

	/** If true, we are deleting all old results from the previous cook. If false, we are keeping the old results. */
	bool bFullBuild = false;
	/**
	 * If true, we will use results from the previous cook for the new cook, if present. If false, we will recook.
	 * -diffonly is the expected case where bFullBuild=false but bAllowIncrementalResults=false.
	 */
	bool bAllowIncrementalResults = true;
	/** If true, a cook has already been run in the current process and we still have results from it. */
	bool bHasMemoryResults = false;
	/** If true, we should delete the in-memory results from an earlier cook in the same process, if we have any. */
	bool bClearMemoryResults = false;
	/**
	 * If true, we should load results that previous cooks left on disk into the current cook's results; this is 
	 * required for incremental cooks and is one way to cook with LegacyIterative cooks.
	 */
	bool bPopulateMemoryResultsFromDiskResults = false;
	/**
	 * If true we are cooking with legacyiterative, from results in a shared build (e.g. from buildfarm) rather than
	 * from our previous cook.
	 */
	bool bLegacyIterativeSharedBuild = false;
	/**
	 * If true we are a CookWorker, and we are working on a Sandbox directory that has already been populated by a
	 * remote Director process.
	 */
	bool bWorkerOnSharedSandbox = false;
};
FCbWriter& operator<<(FCbWriter& Writer, const FBeginCookContextPlatform& Value);
bool LoadFromCompactBinary(FCbFieldView Field, FBeginCookContextPlatform& Value);

/**
 * Data held on the stack and shared with multiple subfunctions when running StartCookByTheBook or 
 * StartCookOnTheFly
 */
struct FBeginCookContext
{
	FBeginCookContext(UCookOnTheFlyServer& InCOTFS)
		: COTFS(InCOTFS)
	{
	}

	const UCookOnTheFlyServer::FCookByTheBookStartupOptions* StartupOptions = nullptr;
	/** List of the platforms we are building, with startup context data about each one */
	TArray<FBeginCookContextPlatform> PlatformContexts;
	/** The list of platforms by themselves, for passing to functions that need just a list of platforms */
	TArray<ITargetPlatform*> TargetPlatforms;
	UCookOnTheFlyServer& COTFS;
};

/** Helper struct for FBeginCookContextForWorker; holds the context data for each platform being cooked */
struct FBeginCookContextForWorkerPlatform
{
	void Set(const FBeginCookContextPlatform& InContext);

	const ITargetPlatform* TargetPlatform = nullptr;
	/**
	 * If true, we are deleting all old results from disk and rebuilding every package. If false, we are building
	 * incrementally.
	 */
	bool bFullBuild = false;
};
FCbWriter& operator<<(FCbWriter& Writer, const FBeginCookContextForWorkerPlatform& Value);
bool LoadFromCompactBinary(FCbFieldView Field, FBeginCookContextForWorkerPlatform& Value);

/** Data from the director's FBeginCookContext that needs to be copied to workers. */
struct FBeginCookContextForWorker
{
	void Set(const FBeginCookContext& InContext);

	/** List of the platforms we are building, with startup context data about each one */
	TArray<FBeginCookContextForWorkerPlatform> PlatformContexts;
};
FCbWriter& operator<<(FCbWriter& Writer, const FBeginCookContextForWorker& Value);
bool LoadFromCompactBinary(FCbFieldView Field, FBeginCookContextForWorker& Value);

LLM_DECLARE_TAG(Cooker);

#define REMAPPED_PLUGINS TEXTVIEW("RemappedPlugins")
extern float GCookProgressWarnBusyTime;

inline constexpr float TickCookableObjectsFrameTime = .100f;

/**
 * Scoped struct to run a function when leaving the scope. The same purpose as ON_SCOPE_EXIT, 
 * but it can also be triggered early.
 */
struct FOnScopeExit
{
public:
	explicit FOnScopeExit(TUniqueFunction<void()>&& InExitFunction)
		:ExitFunction(MoveTemp(InExitFunction))
	{
	}
	~FOnScopeExit()
	{
		ExitEarly();
	}
	void ExitEarly()
	{
		if (ExitFunction)
		{
			ExitFunction();
			ExitFunction.Reset();
		}
	}
	void Abandon()
	{
		ExitFunction.Reset();
	}
private:
	TUniqueFunction<void()> ExitFunction;
};
/**
 * The linker results for a single realm of a package save
 * (e.g. the main package or the optional package that extends the main package for optionally packaged data)
 */
struct FPackageReaderResults
{
	TMap<FSoftObjectPath, FPackageReader::FObjectData> Exports;
	TMap<FSoftObjectPath, FPackageReader::FObjectData> Imports;
	TMap<FName, bool> SoftPackageReferences;
	bool bValid = false;
};

/** The linker results of saving a package. A SavePackage can have multiple outputs (for e.g. optional realm). */
struct FMultiPackageReaderResults
{
	FPackageReaderResults Realms[2];
	ESavePackageResult Result;
};

/** Save the package and read the LinkerTables of its saved data. */
FMultiPackageReaderResults GetSaveExportsAndImports(UPackage* Package, UObject* Asset, FSavePackageArgs SaveArgs);

IPackageWriter::ECommitStatus PackageResultToCommitStatus(ESavePackageResult Result);

extern int32 GCookProgressDisplay;

const TCHAR* LexToString(UE::Cook::ECookResult CookResult);

/**
 * Uses the FMessageLog to log a message
 *
 * @param Message to log
 * @param Severity of the message
 */
void LogCookerMessage(const FString& MessageText, EMessageSeverity::Type Severity);

/**
 * Parses commandline for -Param or -Param=<1|0|true|false|on|off>.
 * If -Param=<CoercableToBoolValue> is found, bInOutValue is set to the value.
 * Else If -Param is found, bInOutValue is set to true.
 * If multiple occurrences of -Param= are found, the first value is used.
 * If neither -Param nor -Param= is found, bInOutValue is unchanged.
 */
void ParseBoolParamOrValue(const TCHAR* CommandLine, const TCHAR* ParamNameNoDashOrEquals, bool& bInOutValue);
