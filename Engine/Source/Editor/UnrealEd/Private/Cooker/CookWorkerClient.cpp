// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookWorkerClient.h"

#include "AssetRegistry/AssetData.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookTypes.h"
#include "Cooker/CookWorkerServer.h"
#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "PackageResultsMessage.h"
#include "PackageTracker.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "WorkerRequestsRemote.h"

namespace UE::Cook
{

namespace CookWorkerClient
{
constexpr float WaitForConnectReplyTimeout = 60.f;
}

FCookWorkerClient::FCookWorkerClient(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{
	LogMessageHandler = new FLogMessagesMessageHandler(*COTFS.LogHandler);
	Register(LogMessageHandler);
	Register(new TMPCollectorClientMessageCallback<FRetractionRequestMessage>([this]
	(FMPCollectorClientMessageContext& Context, bool bReadSuccessful, FRetractionRequestMessage&& Message)
		{
			HandleRetractionMessage(Context, bReadSuccessful, MoveTemp(Message));
		}));
	Register(new TMPCollectorClientMessageCallback<FAbortPackagesMessage>([this]
	(FMPCollectorClientMessageContext& Context, bool bReadSuccessful, FAbortPackagesMessage&& Message)
		{
			HandleAbortPackagesMessage(Context, bReadSuccessful, MoveTemp(Message));
		}));
	Register(new TMPCollectorClientMessageCallback<FHeartbeatMessage>([this]
	(FMPCollectorClientMessageContext& Context, bool bReadSuccessful, FHeartbeatMessage&& Message)
		{
			HandleHeartbeatMessage(Context, bReadSuccessful, MoveTemp(Message));
		}));
	Register(new FAssetRegistryMPCollector(COTFS));
	Register(new FPackageWriterMPCollector(COTFS));
}

FCookWorkerClient::~FCookWorkerClient()
{
	if (ConnectStatus == EConnectStatus::Connected ||
		(EConnectStatus::FlushAndAbortFirst <= ConnectStatus && ConnectStatus <= EConnectStatus::FlushAndAbortLast))
	{
		UE_LOGF(LogCook, Warning,
			"CookWorker was destroyed before it finished Disconnect. The CookDirector may be missing some information.");
	}
	Sockets::CloseSocket(ServerSocket);

	// Before destructing, wait on all of the Futures that could have async access to *this from a TaskThread
	TArray<FPendingResultNeedingAsyncWork> LocalPendingResultsNeedingAsyncWork;
	{
		FScopeLock PendingResultsScopeLock(&PendingResultsLock);
		for (TPair<FPackageRemoteResult*, FPendingResultNeedingAsyncWork>& Pair : PendingResultsNeedingAsyncWork)
		{
			LocalPendingResultsNeedingAsyncWork.Add(MoveTemp(Pair.Value));
		}
		PendingResultsNeedingAsyncWork.Empty();
	}
	for (FPendingResultNeedingAsyncWork& PendingResult : LocalPendingResultsNeedingAsyncWork)
	{
		PendingResult.CompletionFuture.Get();
	}
}

bool FCookWorkerClient::TryConnect(FDirectorConnectionInfo&& ConnectInfo)
{
	EPollStatus Status;
	for (;;)
	{
		Status = PollTryConnect(ConnectInfo);
		if (Status != EPollStatus::Incomplete)
		{
			break;
		}
		constexpr float SleepTime = 0.01f; // 10 ms
		FPlatformProcess::Sleep(SleepTime);
	}
	return Status == EPollStatus::Success;
}

void FCookWorkerClient::TickFromSchedulerThread(FTickStackData& StackData)
{
	if (ConnectStatus == EConnectStatus::Connected)
	{
		PumpReceiveMessages();
		if (ConnectStatus == EConnectStatus::Connected)
		{
			SendPendingMessages();
			PumpSendBuffer();
			TickCollectors(StackData, false /* bFlush */);
		}
	}
	else
	{
		PumpDisconnect(StackData);
	}
}

bool FCookWorkerClient::IsDisconnecting() const
{
	return ConnectStatus == EConnectStatus::LostConnection ||
		(EConnectStatus::FlushAndAbortFirst <= ConnectStatus && ConnectStatus <= EConnectStatus::FlushAndAbortLast);
}

bool FCookWorkerClient::IsDisconnectComplete() const
{
	return ConnectStatus == EConnectStatus::LostConnection;
}

ECookInitializationFlags FCookWorkerClient::GetCookInitializationFlags()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->GetCookInitializationFlags();
}

bool FCookWorkerClient::GetInitializationIsZenStore()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->IsZenStore();
}

FInitializeConfigSettings&& FCookWorkerClient::ConsumeInitializeConfigSettings()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->ConsumeInitializeConfigSettings();
}
FBeginCookConfigSettings&& FCookWorkerClient::ConsumeBeginCookConfigSettings()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->ConsumeBeginCookConfigSettings();
}
FCookByTheBookOptions&& FCookWorkerClient::ConsumeCookByTheBookOptions()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->ConsumeCookByTheBookOptions();
}
const FBeginCookContextForWorker& FCookWorkerClient::GetBeginCookContext()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->GetBeginCookContext();
}
FCookOnTheFlyOptions&& FCookWorkerClient::ConsumeCookOnTheFlyOptions()
{
	check(InitialConfigMessage); // Should only be called after TryConnect and before DoneWithInitialSettings
	return InitialConfigMessage->ConsumeCookOnTheFlyOptions();
}
const TArray<ITargetPlatform*>& FCookWorkerClient::GetTargetPlatforms() const
{
	return OrderedSessionPlatforms;
}
void FCookWorkerClient::DoneWithInitialSettings()
{
	InitialConfigMessage.Reset();
	// Process remaining deferred initialization messages and discard them
	HandleReceiveMessages(MoveTemp(DeferredInitializationMessages));
}

void FCookWorkerClient::ReportDemotion(const FPackageData& PackageData, ESuppressCookReason Reason)
{
	if (Reason == ESuppressCookReason::RetractedByCookDirector)
	{
		return;
	}
	TUniquePtr<FPackageRemoteResult> ResultOwner(new FPackageRemoteResult());
	FName PackageName = PackageData.GetPackageName();
	ResultOwner->SetPackageName(PackageName);
	ResultOwner->SetSuppressCookReason(Reason);
	// Set the platforms, use the default values for each platform (e.g. bSuccessful=false)
	ResultOwner->SetPlatforms(OrderedSessionPlatforms);

	ReportPackageMessage(PackageName, MoveTemp(ResultOwner));
}

void FCookWorkerClient::ReportPromoteToSaveComplete(FPackageData& PackageData)
{
	TUniquePtr<FPackageRemoteResult> ResultOwner(new FPackageRemoteResult());
	FPackageRemoteResult* Result = ResultOwner.Get();

	FName PackageName = PackageData.GetPackageName();
	Result->SetPackageName(PackageName);
	Result->SetSuppressCookReason(ESuppressCookReason::NotSuppressed);
	Result->SetPlatforms(OrderedSessionPlatforms);
	if (TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.GetGenerationHelper(); GenerationHelper)
	{
		Result->SetExternalActorDependencies(GenerationHelper->ReleaseExternalActorDependencies());
	}

	int32 NumPlatforms = OrderedSessionPlatforms.Num();
	for (int32 PlatformIndex = 0; PlatformIndex < NumPlatforms; ++PlatformIndex)
	{
		ITargetPlatform* TargetPlatform = OrderedSessionPlatforms[PlatformIndex];
		FPackageRemoteResult::FPlatformResult& PlatformResults = Result->GetPlatforms()[PlatformIndex];
		FPackagePlatformData& PackagePlatformData = PackageData.FindOrAddPlatformData(TargetPlatform);
		if (!PackagePlatformData.IsCommitted() || PackagePlatformData.IsReportedToDirector())
		{
			// We didn't attempt to commit this platform for this package, or we committed it previously and already
			// sent the information about it
			PlatformResults.SetWasCommitted(false);
			PlatformResults.SetCookResults(ECookResult::Invalid);
		}
		else
		{
			PlatformResults.SetWasCommitted(true);
			PlatformResults.SetCookResults(PackagePlatformData.GetCookResults());
			PackagePlatformData.SetReportedToDirector(true);
		}
	}

	ReportPackageMessage(PackageName, MoveTemp(ResultOwner));
}

void FCookWorkerClient::ReportPackageMessage(FName PackageName, TUniquePtr<FPackageRemoteResult>&& ResultOwner)
{
	FPackageRemoteResult* Result = ResultOwner.Get();

	TArray<FMPCollectorClientTickPackageContext::FPlatformData, TInlineAllocator<1>> ContextPlatformDatas;
	ContextPlatformDatas.Reserve(Result->GetPlatforms().Num());
	for (FPackageRemoteResult::FPlatformResult& PlatformResult : Result->GetPlatforms())
	{
		ContextPlatformDatas.Add(FMPCollectorClientTickPackageContext::FPlatformData
			{ PlatformResult.GetPlatform(), PlatformResult.GetCookResults() });
	}
	FMPCollectorClientTickPackageContext Context;
	Context.PackageName = PackageName;
	Context.Platforms = OrderedSessionPlatforms;
	Context.PlatformDatas = ContextPlatformDatas;

	for (const TPair<FGuid, TRefCountPtr<IMPCollector>>& CollectorPair : Collectors)
	{
		IMPCollector* Collector = CollectorPair.Value.GetReference();
		Collector->ClientTickPackage(Context);
		const FGuid& MessageType = CollectorPair.Key;
		for (TPair<const ITargetPlatform*, FCbObject>& MessagePair : Context.Messages)
		{
			const ITargetPlatform* TargetPlatform = MessagePair.Key;
			FCbObject Object = MoveTemp(MessagePair.Value);
			if (!TargetPlatform)
			{
				Result->AddPackageMessage(MessageType, MoveTemp(Object));
			}
			else
			{
				Result->AddPlatformMessage(TargetPlatform, MessageType, MoveTemp(Object));
			}
		}
		Context.Messages.Reset();
		for (TPair<const ITargetPlatform*, TFuture<FCbObject>>& MessagePair : Context.AsyncMessages)
		{
			const ITargetPlatform* TargetPlatform = MessagePair.Key;
			TFuture<FCbObject> ObjectFuture = MoveTemp(MessagePair.Value);
			if (!TargetPlatform)
			{
				Result->AddAsyncPackageMessage(MessageType, MoveTemp(ObjectFuture));
			}
			else
			{
				Result->AddAsyncPlatformMessage(TargetPlatform, MessageType, MoveTemp(ObjectFuture));
			}
		}
		Context.AsyncMessages.Reset();
	}

	++(Result->GetUserRefCount()); // Used to test whether the async Future still needs to access *this
	TFuture<void> CompletionFuture = Result->GetCompletionFuture().Then(
	[this, Result](TFuture<int>&& OldFuture)
	{
		FScopeLock PendingResultsScopeLock(&PendingResultsLock);
		FPendingResultNeedingAsyncWork PendingResult;
		PendingResultsNeedingAsyncWork.RemoveAndCopyValue(Result, PendingResult);

		// Result might have not been added into PendingResultsNeedingAsyncWork yet, and also could have
		// been removed by cancellation from e.g. CookWorkerClient destructor.
		if (PendingResult.PendingResult)
		{
			PendingResults.Add(MoveTemp(PendingResult.PendingResult));
		}
		--(Result->GetUserRefCount());
	});

	{
		FScopeLock PendingResultsScopeLock(&PendingResultsLock);
		if (Result->GetUserRefCount() == 0)
		{
			// Result->GetCompletionFuture() has already been called
			check(Result->IsComplete());
			PendingResults.Add(MoveTemp(ResultOwner));
		}
		else
		{
			FPendingResultNeedingAsyncWork Work;
			Work.PendingResult = MoveTemp(ResultOwner);
			Work.CompletionFuture = MoveTemp(CompletionFuture);
			PendingResultsNeedingAsyncWork.Add(Result, MoveTemp(Work));
		}
	}
}

void FCookWorkerClient::ReportDiscoveredPackage(const FPackageData& PackageData, const FInstigator& Instigator,
	FDiscoveredPlatformSet&& ReachablePlatforms, FGenerationHelper* ParentGenerationHelper, EUrgency Urgency)
{
	FDiscoveredPackageReplication& Discovered = PendingDiscoveredPackages.Emplace_GetRef();
	Discovered.PackageName = PackageData.GetPackageName();
	Discovered.NormalizedFileName = PackageData.GetFileName();
	Discovered.ParentGenerator = PackageData.GetParentGenerator();
	Discovered.Instigator = Instigator;
	Discovered.Platforms = MoveTemp(ReachablePlatforms);
	Discovered.Platforms.ConvertToBitfield(OrderedSessionAndSpecialPlatforms);
	Discovered.DoesGeneratedRequireGenerator = PackageData.DoesGeneratedRequireGenerator();
	Discovered.Urgency = Urgency;
	if (ParentGenerationHelper)
	{
		if (FCookGenerationInfo* Info = ParentGenerationHelper->FindInfo(PackageData))
		{
			Discovered.GeneratedPackageHash = Info->PackageHash;
		}
	}
}

void FCookWorkerClient::ReportLogMessage(const FReplicatedLogData& LogData)
{
	LogMessageHandler->ClientReportLogMessage(LogData);
}

void FCookWorkerClient::ReportFileTransfer(FStringView TempFileName, FStringView TargetFileName)
{
	FScopeLock FileTransferScopeLock(&FileTransferLock);
	FFileTransferMessage& Message = PendingFileTransfers.Emplace_GetRef();
	Message.TempFileName = TempFileName;
	Message.TargetFileName = TargetFileName;
}

void FCookWorkerClient::ReportGeneratorQueuedGeneratedPackages(FGenerationHelper& GenerationHelper)
{
	PendingGeneratorEvents.Add(FGeneratorEventMessage(EGeneratorEvent::QueuedGeneratedPackages,
		GenerationHelper.GetOwner().GetPackageName()));
}

void FCookWorkerClient::HandleDirectorMessage(FDirectorEventMessage&& DirectorMessage)
{
	switch (DirectorMessage.Event)
	{
	case EDirectorEvent::KickBuildDependencies:
		COTFS.bKickedBuildDependencies = true;
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FCookWorkerClient::HandleGeneratorMessage(FGeneratorEventMessage&& GeneratorMessage)
{
	FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(GeneratorMessage.PackageName);
	if (PackageData)
	{
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
		if (GenerationHelper)
		{
			switch (GeneratorMessage.Event)
			{
			case EGeneratorEvent::SkipSaveExistingGenerator:
				GenerationHelper->OnSkipSaveExistingGenerator();
				break;
			case EGeneratorEvent::QueuedGeneratedPackagesFencePassed:
				GenerationHelper->OnQueuedGeneratedPackagesFencePassed(COTFS);
				break;
			case EGeneratorEvent::AllSavesCompleted:
				GenerationHelper->OnAllSavesCompleted(COTFS);
				break;
			default:
				// We do not handle the remaining GeneratorEvents on clients
				break;
			}
		}
	}
}

EPollStatus FCookWorkerClient::PollTryConnect(const FDirectorConnectionInfo& ConnectInfo)
{
	for (;;)
	{
		switch (ConnectStatus)
		{
		case EConnectStatus::Connected:
			return EPollStatus::Success;
		case EConnectStatus::Uninitialized:
			CreateServerSocket(ConnectInfo);
			break;
		case EConnectStatus::PollWriteConnectMessage:
			PollWriteConnectMessage();
			if (ConnectStatus == EConnectStatus::PollWriteConnectMessage)
			{
				return EPollStatus::Incomplete;
			}
			break;
		case EConnectStatus::PollReceiveConfigMessage:
			PollReceiveConfigMessage();
			if (ConnectStatus == EConnectStatus::PollReceiveConfigMessage)
			{
				return EPollStatus::Incomplete;
			}
			break;
		case EConnectStatus::LostConnection:
			return EPollStatus::Error;
		default:
			return EPollStatus::Error;
		}
	}
}

void FCookWorkerClient::CreateServerSocket(const FDirectorConnectionInfo& ConnectInfo)
{
	using namespace CompactBinaryTCP;

	ConnectStartTimeSeconds = FPlatformTime::Seconds();
	DirectorURI = ConnectInfo.HostURI;

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (!SocketSubsystem)
	{
		UE_LOGF(LogCook, Error,
			"CookWorker initialization failure: platform does not support network sockets, cannot connect to CookDirector.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	DirectorAddr = Sockets::GetAddressFromStringWithPort(DirectorURI);
	if (!DirectorAddr)
	{
		UE_LOGF(LogCook, Error,
			"CookWorker initialization failure: could not convert -CookDirectorHost=%ls into an address, cannot connect to CookDirector.",
			*DirectorURI);
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	UE_LOGF(LogCook, Display, "Connecting to CookDirector at %ls...", *DirectorURI);

	ServerSocket = Sockets::ConnectToHost(*DirectorAddr, TEXT("FCookWorkerClient-WorkerConnect"));
	if (!ServerSocket)
	{
		UE_LOGF(LogCook, Error, "CookWorker initialization failure: Could not connect to CookDirector.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	constexpr float WaitForConnectTimeout = 60.f * 10;
	float ConditionalTimeoutSeconds = IsCookIgnoreTimeouts() ? MAX_flt : WaitForConnectTimeout;
	bool bServerSocketReady = ServerSocket->Wait(ESocketWaitConditions::WaitForWrite,
		FTimespan::FromSeconds(ConditionalTimeoutSeconds));
	if (!bServerSocketReady)
	{
		UE_LOGF(LogCook, Error,
			"CookWorker initialization failure: Timed out after %.0f seconds trying to connect to CookDirector.",
			ConditionalTimeoutSeconds);
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	FWorkerConnectMessage ConnectMessage;
	ConnectMessage.RemoteIndex = ConnectInfo.RemoteIndex;
	EConnectionStatus Status = TryWritePacket(ServerSocket, SendBuffer, MarshalToCompactBinaryTCP(ConnectMessage));
	UpdateSocketSendDiagnostics(Status);
	if (Status == EConnectionStatus::Incomplete)
	{
		SendToState(EConnectStatus::PollWriteConnectMessage);
		return;
	}
	else if (Status != EConnectionStatus::Okay)
	{
		UE_LOGF(LogCook, Error, "CookWorker initialization failure: could not send ConnectMessage.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}
	LogConnected();

	SendToState(EConnectStatus::PollReceiveConfigMessage);
}

void FCookWorkerClient::PollWriteConnectMessage()
{
	using namespace CompactBinaryTCP;

	EConnectionStatus Status = TryFlushBuffer(ServerSocket, SendBuffer);
	UpdateSocketSendDiagnostics(Status);
	if (Status == EConnectionStatus::Incomplete)
	{
		if (FPlatformTime::Seconds() - ConnectStartTimeSeconds > CookWorkerClient::WaitForConnectReplyTimeout &&
			!IsCookIgnoreTimeouts())
		{
			UE_LOGF(LogCook, Error,
				"CookWorker initialization failure: timed out waiting for %fs to send ConnectMessage.",
				CookWorkerClient::WaitForConnectReplyTimeout);
			SendToState(EConnectStatus::LostConnection);
		}
		return;
	}
	else if (Status != EConnectionStatus::Okay)
	{
		UE_LOGF(LogCook, Error, "CookWorker initialization failure: could not send ConnectMessage.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}
	LogConnected();
	SendToState(EConnectStatus::PollReceiveConfigMessage);
}

void FCookWorkerClient::PollReceiveConfigMessage()
{
	using namespace UE::CompactBinaryTCP;
	TArray<FMarshalledMessage> Messages;
	EConnectionStatus SocketStatus = TryReadPacket(ServerSocket, ReceiveBuffer, Messages);
	UpdateSocketSendDiagnostics(SocketStatus);
	if (SocketStatus != EConnectionStatus::Okay && SocketStatus != EConnectionStatus::Incomplete)
	{
		UE_LOGF(LogCook, Error, "CookWorker initialization failure: failed to read from socket.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}
	if (Messages.Num() == 0)
	{
		if (FPlatformTime::Seconds() - ConnectStartTimeSeconds > CookWorkerClient::WaitForConnectReplyTimeout &&
			!IsCookIgnoreTimeouts())
		{
			UE_LOGF(LogCook, Error,
				"CookWorker initialization failure: timed out waiting for %fs to receive InitialConfigMessage.",
				CookWorkerClient::WaitForConnectReplyTimeout);
			SendToState(EConnectStatus::LostConnection);
		}
		return;
	}
	
	if (Messages[0].MessageType != FInitialConfigMessage::MessageType)
	{
		UE_LOGF(LogCook, Warning,
			"CookWorker initialization failure: Director sent a different message before sending an InitialConfigMessage. MessageType: %ls.",
			*Messages[0].MessageType.ToString());
		SendToState(EConnectStatus::LostConnection);
		return;
	}
	check(!InitialConfigMessage);
	InitialConfigMessage = MakeUnique<FInitialConfigMessage>();
	if (!InitialConfigMessage->TryRead(Messages[0].Object))
	{
		UE_LOGF(LogCook, Warning,
			"CookWorker initialization failure: Director sent an invalid InitialConfigMessage.");
		SendToState(EConnectStatus::LostConnection);
		return;
	}
	DirectorCookMode = InitialConfigMessage->GetDirectorCookMode();
	OrderedSessionPlatforms = InitialConfigMessage->GetOrderedSessionPlatforms();
	OrderedSessionAndSpecialPlatforms.Reset(OrderedSessionPlatforms.Num() + 1);
	OrderedSessionAndSpecialPlatforms.Append(OrderedSessionPlatforms);
	OrderedSessionAndSpecialPlatforms.Add(CookerLoadingPlatformKey);
	const TArray<ITargetPlatform*>& ActiveTargetPlatforms = GetTargetPlatformManagerRef().GetActiveTargetPlatforms();

	auto GetPlatformDetails = [this, &ActiveTargetPlatforms](TStringBuilder<512>& StringBuilder)
	{
		StringBuilder << TEXT("ActiveTargetPlatforms(") << ActiveTargetPlatforms.Num() << TEXT("): ");
		for (int32 PlatformIndex = 0; PlatformIndex < ActiveTargetPlatforms.Num(); ++PlatformIndex)
		{
			ITargetPlatform* Platform = ActiveTargetPlatforms[PlatformIndex];
			StringBuilder << Platform->PlatformName();
			if (PlatformIndex < (ActiveTargetPlatforms.Num() - 1))
			{
				StringBuilder << TEXT(", ");
			}
		}
		StringBuilder << TEXT("\n");

		StringBuilder << TEXT("OrderedSessionPlatforms(") << OrderedSessionPlatforms.Num() << TEXT("): ");
		for (int32 PlatformIndex = 0; PlatformIndex < OrderedSessionPlatforms.Num(); ++PlatformIndex)
		{
			ITargetPlatform* Platform = OrderedSessionPlatforms[PlatformIndex];
			StringBuilder << Platform->PlatformName();
			if (PlatformIndex < (OrderedSessionPlatforms.Num() - 1))
			{
				StringBuilder << TEXT(", ");
			}
		}
	};

	if (OrderedSessionPlatforms.Num() != ActiveTargetPlatforms.Num())
	{
		TStringBuilder<512> StringBuilder;
		GetPlatformDetails(StringBuilder);
		UE_LOGF(LogCook, Error,
			"CookWorker initialization failure: Director sent a mismatch in session platform quantity.\n%ls", *StringBuilder);
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	bool bPlatformMismatch = false;
	for (ITargetPlatform* Platform : ActiveTargetPlatforms)
	{
		if (!OrderedSessionPlatforms.Contains(Platform))
		{
			bPlatformMismatch = true;
			break;
		}
	}

	if (bPlatformMismatch)
	{
		TStringBuilder<512> StringBuilder;
		GetPlatformDetails(StringBuilder);
		UE_LOGF(LogCook, Error,
			"CookWorker initialization failure: Director sent a mismatch in session platform contents.\n%ls", *StringBuilder);
		SendToState(EConnectStatus::LostConnection);
		return;
	}

	HandleReceiveMessages(InitialConfigMessage->ConsumeCollectorMessages());

	UE_LOGF(LogCook, Display, "Initialization from CookDirector complete.");
	SendToState(EConnectStatus::Connected);
	Messages.RemoveAt(0);
	HandleReceiveMessages(MoveTemp(Messages));
}

void FCookWorkerClient::LogConnected()
{
	UE_LOGF(LogCook, Display, "Connection to CookDirector successful.");
}

void FCookWorkerClient::PumpSendBuffer()
{
	UE::CompactBinaryTCP::EConnectionStatus Status = UE::CompactBinaryTCP::TryFlushBuffer(ServerSocket, SendBuffer);
	UpdateSocketSendDiagnostics(Status);
	if (Status == UE::CompactBinaryTCP::EConnectionStatus::Failed)
	{
		UE_LOGF(LogCook, Error,
			"CookWorkerClient failed to write message to Director. We will abort the CookAsCookWorker commandlet.");
		SendToState(EConnectStatus::LostConnection);
	}
}

void FCookWorkerClient::SendPendingMessages()
{
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> LocalCopyQueuedMessages;
	FPackageResultsMessage Message;
	{
		FScopeLock PendingResultsScopeLock(&PendingResultsLock);
		if (!PendingResults.IsEmpty())
		{
			Message.Results.Reserve(PendingResults.Num());
			for (TUniquePtr<FPackageRemoteResult>& Result : PendingResults)
			{
				Message.Results.Add(MoveTemp(*Result));
			}
			PendingResults.Reset();
		}
		LocalCopyQueuedMessages = MoveTemp(QueuedMessages);
		QueuedMessages.Empty();
	}
	if (!Message.Results.IsEmpty())
	{
		SendMessage(Message);
	}

	if (!PendingDiscoveredPackages.IsEmpty())
	{
		FDiscoveredPackagesMessage DiscoveredMessage;
		DiscoveredMessage.OrderedSessionAndSpecialPlatforms = OrderedSessionAndSpecialPlatforms;
		DiscoveredMessage.Packages = MoveTemp(PendingDiscoveredPackages);
		SendMessage(DiscoveredMessage);
		PendingDiscoveredPackages.Reset();
	}

	if (!PendingGeneratorEvents.IsEmpty())
	{
		for (FGeneratorEventMessage& GeneratorMessage : PendingGeneratorEvents)
		{
			SendMessage(GeneratorMessage);
		}
		PendingGeneratorEvents.Reset();
	}

	TArray<FFileTransferMessage> LocalPendingFileTransfers;
	{
		FScopeLock FileTransferScopeLock(&FileTransferLock);
		Swap(LocalPendingFileTransfers, PendingFileTransfers);
	}
	for (FFileTransferMessage& FileTransferMessage : LocalPendingFileTransfers)
	{
		SendMessage(FileTransferMessage);
	}

	for (UE::CompactBinaryTCP::FMarshalledMessage& QueuedMessage: LocalCopyQueuedMessages)
	{
		SendMessage(MoveTemp(QueuedMessage));
	}
}

void FCookWorkerClient::PumpReceiveMessages()
{
	using namespace UE::CompactBinaryTCP;
	TArray<FMarshalledMessage> Messages;

	// Read a packet at a time (with 1 or more messages per packet) until we fail to read any messages
	for (;;)
	{
		if (ServerSocket == nullptr)
		{
			// HandleReceiveMessage might change our connectionstatus to LostConnection and kill the ServerSocket,
			// so we need to check for null after each time we handle messages.
			break;
		}
		Messages.Reset();
		EConnectionStatus SocketStatus = TryReadPacket(ServerSocket, ReceiveBuffer, Messages);
		UpdateSocketSendDiagnostics(SocketStatus);
		if (SocketStatus != EConnectionStatus::Okay && SocketStatus != EConnectionStatus::Incomplete)
		{
			UE_LOGF(LogCook, Error,
				"CookWorkerClient failed to read from Director. We will abort the CookAsCookWorker commandlet.");
			SendToState(EConnectStatus::LostConnection);
			return;
		}
		if (Messages.IsEmpty())
		{
			break;
		}
		HandleReceiveMessages(MoveTemp(Messages));
	}
}

void FCookWorkerClient::HandleReceiveMessages(TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& Messages, FName OptionalPackageName /*= NAME_None*/)
{
	for (UE::CompactBinaryTCP::FMarshalledMessage& Message : Messages)
	{
		if (EConnectStatus::FlushAndAbortFirst <= ConnectStatus && ConnectStatus <= EConnectStatus::FlushAndAbortLast)
		{
			if (Message.MessageType == FAbortWorkerMessage::MessageType)
			{
				UE_LOGF(LogCook, Display,
					"CookWorkerClient received AbortWorker message from Director. Terminating flush and shutting down.");
				SendToState(EConnectStatus::LostConnection);
				break;
			}
			UE_LOGF(LogCook, Error,
				"CookWorkerClient received message %ls from Director after receiving Abort message. Message will be ignored.",
			*Message.MessageType.ToString());
		}
		else
		{
			if (Message.MessageType == FAbortWorkerMessage::MessageType)
			{
				FAbortWorkerMessage AbortMessage;
				AbortMessage.TryRead(Message.Object);
				if (AbortMessage.Type == FAbortWorkerMessage::EType::CookComplete)
				{
					UE_LOGF(LogCook, Display,
						"CookWorkerClient received CookComplete message from Director. Flushing messages and shutting down.");
					SendToState(EConnectStatus::FlushAndAbortFirst);
				}
				else
				{
					UE_LOGF(LogCook, Display,
						"CookWorkerClient received AbortWorker message from Director. Shutting down.");
					SendToState(EConnectStatus::LostConnection);
					break;
				}
			}
			else if (Message.MessageType == FInitialConfigMessage::MessageType)
			{
				UE_LOGF(LogCook, Warning,
					"CookWorkerClient received unexpected repeat of InitialConfigMessage. Ignoring it.");
			}
			else if (Message.MessageType == FAssignPackagesMessage::MessageType)
			{
				FAssignPackagesMessage AssignPackagesMessage;
				AssignPackagesMessage.OrderedSessionPlatforms = OrderedSessionPlatforms;
				if (!AssignPackagesMessage.TryRead(Message.Object))
				{
					LogInvalidMessage(TEXT("FAssignPackagesMessage"));
				}
				else
				{
					AssignPackages(AssignPackagesMessage);
				}
			}
			else if (Message.MessageType == FDirectorEventMessage::MessageType)
			{
				FDirectorEventMessage DirectorMessage;
				if (!DirectorMessage.TryRead(Message.Object))
				{
					LogInvalidMessage(TEXT("FDirectorEventMessage"));
				}
				else
				{
					HandleDirectorMessage(MoveTemp(DirectorMessage));
				}
			}
			else if (Message.MessageType == FGeneratorEventMessage::MessageType)
			{
				FGeneratorEventMessage GeneratorMessage;
				if (!GeneratorMessage.TryRead(Message.Object))
				{
					LogInvalidMessage(TEXT("FGeneratorEventMessage"));
				}
				else
				{
					HandleGeneratorMessage(MoveTemp(GeneratorMessage));
				}
			}
			else
			{
				TRefCountPtr<IMPCollector>* Collector = Collectors.Find(Message.MessageType);
				if (Collector)
				{
					check(*Collector);
					FMPCollectorClientMessageContext Context;
					Context.Platforms = OrderedSessionPlatforms;
					Context.PackageName = OptionalPackageName;
					(*Collector)->ClientReceiveMessage(Context, Message.Object);
				}
				else if (InitialConfigMessage.IsValid())
				{
					ensureMsgf(Messages.GetData() != DeferredInitializationMessages.GetData(), 
						TEXT("HandleReceiveMessages may not be called with the deferred initialization message array until after calling DoneWithInitialSettings()"));

					// If we are still running our initialization, then we may not have the relevant collectors registered yet
					// Defer the message and try again at the end of initialization
					DeferredInitializationMessages.Add(Message);
				}
				else
				{
					UE_LOGF(LogCook, Error,
						"CookWorkerClient received message of unknown type %ls from CookDirector. Ignoring it.",
						*Message.MessageType.ToString());
				}
			}
		}
	}
}

void FCookWorkerClient::PumpDisconnect(FTickStackData& StackData)
{
	for (;;)
	{
		switch (ConnectStatus)
		{
		case EConnectStatus::FlushAndAbortFirst:
		{
			TickCollectors(StackData, true /* bFlush */);
			// Add code here for any waiting we need to do for the local CookOnTheFlyServer to gracefully shutdown
			COTFS.CookAsCookWorkerFinished();
			SendMessage(FAbortWorkerMessage(FAbortWorkerMessage::EType::Abort));
			SendToState(EConnectStatus::WaitForAbortAcknowledge);
			break;
		}
		case EConnectStatus::WaitForAbortAcknowledge:
		{
			using namespace UE::CompactBinaryTCP;
			PumpReceiveMessages();
			if (ConnectStatus == EConnectStatus::WaitForAbortAcknowledge)
			{
				PumpSendBuffer();

				constexpr float WaitForDisconnectTimeout = 60.f;
				if (FPlatformTime::Seconds() - ConnectStartTimeSeconds > WaitForDisconnectTimeout
					&& !IsCookIgnoreTimeouts())
				{
					UE_LOGF(LogCook, Warning,
						"Timedout after %.0fs waiting to send disconnect message to CookDirector.",
						WaitForDisconnectTimeout);
					SendToState(EConnectStatus::LostConnection);
					check(ConnectStatus == EConnectStatus::LostConnection);
					// Fall through to LostConnection
					break;
				}
				else
				{
					return; // Exit the Pump loop for now and keep waiting
				}
			}
			else
			{
				check(ConnectStatus == EConnectStatus::LostConnection);
				// Fall through to LostConnection
				break;
			}
		}
		case EConnectStatus::LostConnection:
		{
			StackData.bCookCancelled = true;
			StackData.ResultFlags |= UCookOnTheFlyServer::COSR_YieldTick;
			return;
		}
		default:
			return;
		}
	}
}

void FCookWorkerClient::SendMessage(const IMPCollectorMessage& Message)
{
	SendMessage(MarshalToCompactBinaryTCP(Message));
}

void FCookWorkerClient::SendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message)
{
	UE::CompactBinaryTCP::EConnectionStatus Status = TryWritePacket(ServerSocket, SendBuffer, MoveTemp(Message));
	UpdateSocketSendDiagnostics(Status);
}

void FCookWorkerClient::QueueMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message)
{
	FScopeLock PendingResultsScopeLock(&PendingResultsLock);
	QueuedMessages.Add(MoveTemp(Message));
}

void FCookWorkerClient::SendToState(EConnectStatus TargetStatus)
{
	switch (TargetStatus)
	{
	case EConnectStatus::FlushAndAbortFirst:
		ConnectStartTimeSeconds = FPlatformTime::Seconds();
		break;
	case EConnectStatus::LostConnection:
		Sockets::CloseSocket(ServerSocket);
		break;
	}
	ConnectStatus = TargetStatus;
}

void FCookWorkerClient::LogInvalidMessage(const TCHAR* MessageTypeName)
{
	UE_LOGF(LogCook, Error,
		"CookWorkerClient received invalidly formatted message for type %ls from CookDirector. Ignoring it.",
		MessageTypeName);
}

void FCookWorkerClient::UpdateSocketSendDiagnostics(UE::CompactBinaryTCP::EConnectionStatus Status)
{
	using namespace UE::CompactBinaryTCP;
	if (Status != EConnectionStatus::Incomplete)
	{
		LastTimeOfCompleteSocketStatusSeconds = 0.0;
		LastTimeOfWarningOfSocketStatusSeconds = 0.0;
		return;
	}

	if (LastTimeOfCompleteSocketStatusSeconds <= 0.0)
	{
		LastTimeOfCompleteSocketStatusSeconds = FPlatformTime::Seconds();
		LastTimeOfWarningOfSocketStatusSeconds = LastTimeOfCompleteSocketStatusSeconds;
	}
	else
	{
		constexpr double WarningTimePeriod = 60.;
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - LastTimeOfWarningOfSocketStatusSeconds >= WarningTimePeriod)
		{
			UE_LOGF(LogCook, Display,
				"CookWorkerClient has been unable to send messages to the CookDirector for the past %.1f seconds. Continuing cooking locally and attempting to send...",
				(float)(CurrentTime - LastTimeOfCompleteSocketStatusSeconds));
			LastTimeOfWarningOfSocketStatusSeconds = CurrentTime;
		}
	}
}

void FCookWorkerClient::AssignPackages(FAssignPackagesMessage& Message)
{
	if (!Message.ExistenceInfos.IsEmpty())
	{
		for (FPackageDataExistenceInfo& ExistenceInfo : Message.ExistenceInfos)
		{
			FPackageData& PackageData = COTFS.PackageDatas->FindOrAddPackageData(ExistenceInfo.ConstructData.PackageName,
				ExistenceInfo.ConstructData.NormalizedFileName);
			if (!ExistenceInfo.ParentGenerator.IsNone())
			{
				PackageData.SetGenerated(ExistenceInfo.ParentGenerator);
			}
		}
	}
	if (!Message.PackageDatas.IsEmpty())
	{
		TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> NeedCommitPlatformsBuffer;
		for (FAssignPackageData& AssignData : Message.PackageDatas)
		{
			FPackageData& PackageData = COTFS.PackageDatas->FindOrAddPackageData(AssignData.ConstructData.PackageName,
				AssignData.ConstructData.NormalizedFileName);
			if (!AssignData.ParentGenerator.IsNone())
			{
				PackageData.SetGenerated(AssignData.ParentGenerator);
				PackageData.SetDoesGeneratedRequireGenerator(AssignData.DoesGeneratedRequireGenerator);
			}
			if (!AssignData.GeneratorPerPlatformPreviousGeneratedPackages.IsEmpty() ||
				AssignData.bSkipSaveExistingGenerator || AssignData.bQueuedGeneratedPackagesFencePassed)
			{
				TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData.CreateUninitializedGenerationHelper();
				for (TPair<uint8, TMap<FName, FAssetPackageData>>& PlatformPair
					: AssignData.GeneratorPerPlatformPreviousGeneratedPackages)
				{
					const ITargetPlatform* TargetPlatform = OrderedSessionPlatforms[PlatformPair.Key];
					GenerationHelper->SetPreviousGeneratedPackages(TargetPlatform, MoveTemp(PlatformPair.Value));
				}
				if (AssignData.bSkipSaveExistingGenerator)
				{
					// Should be called before OnQueuedGeneratedPackagesFencePassed because Generators wait for
					// OnQueuedGeneratedPackagesFencePassed and then execute code that uses this value if set.
					GenerationHelper->OnSkipSaveExistingGenerator();
				}
				if (AssignData.bQueuedGeneratedPackagesFencePassed)
				{
					GenerationHelper->OnQueuedGeneratedPackagesFencePassed(COTFS);
				}
			}
			if (!AssignData.PerPackageCollectorMessages.IsEmpty())
			{
				HandleReceiveMessages(MoveTemp(AssignData.PerPackageCollectorMessages), AssignData.ConstructData.PackageName);
			}
			EReachability Reachability = AssignData.Reachability;
			TConstArrayView<const ITargetPlatform*> NeedCommitPlatforms =
				AssignData.NeedCommitPlatforms.GetPlatforms(COTFS, nullptr, OrderedSessionPlatforms,
					Reachability, NeedCommitPlatformsBuffer);
			if (PackageData.IsInProgress())
			{
				// If already in progress but there are new platforms requested, demote the package back to Load
				for (const ITargetPlatform* TargetPlatform : NeedCommitPlatforms)
				{
					check(TargetPlatform != CookerLoadingPlatformKey);
					if (!PackageData.FindOrAddPlatformData(TargetPlatform).IsReachable(Reachability))
					{
						if (PackageData.IsInStateProperty(EPackageStateProperty::Saving))
						{
							UE_LOGF(LogCook, Display,
								"Package %ls is in the save state, but the CookDirector updated the requested platforms to include the new platform %ls. Restarting the package's save.",
								*PackageData.GetPackageName().ToString(), *TargetPlatform->PlatformName());
							PackageData.SendToState(EPackageState::Load, ESendFlags::QueueAddAndRemove, EStateChangeReason::DirectorRequest);
						}
					}
				}
				// Allow the package to continue in its progress. If it was in a stalled-by-retraction state, return it to active.
				PackageData.UnStall(ESendFlags::QueueAddAndRemove);
				PackageData.RaiseUrgency(AssignData.Urgency, ESendFlags::QueueAddAndRemove);
				continue;
			}

			// We do not want CookWorkers to explore dependencies in CookRequestCluster because the Director
			// did it already. Mark the PackageDatas we get from the Director as already explored.
			for (const ITargetPlatform* TargetPlatform : NeedCommitPlatforms)
			{
				PackageData.FindOrAddPlatformData(TargetPlatform).MarkCommittableForWorker(Reachability, *this);
			}
			if (Reachability == EReachability::Runtime)
			{
				checkf(COTFS.GetCookPhase() == ECookPhase::Cook,
					TEXT("CookDirector has assigned a package for EReachability::Runtime cooking after the CookWorker has entered ECookPhase::BuildDependencies.")
					TEXT(" This would soft-lock the cook so we assert instead.")
					TEXT(" Package: %s"), *PackageData.GetPackageName().ToString());
				// Also mark that CookerLoadingPlatformKey is reachable, since we do expect to need to load the package
				PackageData.FindOrAddPlatformData(CookerLoadingPlatformKey).MarkCommittableForWorker(EReachability::Runtime, *this);
			}
			else
			{
				checkf(COTFS.GetCookPhase() == ECookPhase::BuildDependencies,
					TEXT("CookDirector has assigned a package for EReachability::Build committing before the CookWorker has entered ECookPhase::BuildDependencies.")
					TEXT(" This would soft-lock the cook so we assert instead.")
					TEXT(" Package: %s"), *PackageData.GetPackageName().ToString());
				// BuildDependency phase does not use the CookerLoadingPlatformKey, so we do not need to mark it reachable.
			}
			PackageData.SetInstigator(*this, Reachability, FInstigator(AssignData.Instigator));
			PackageData.RaiseUrgency(AssignData.Urgency, ESendFlags::QueueAddAndRemove, true /* bAllowUrgencyInIdle */);
			PackageData.SendToState(EPackageState::Request, ESendFlags::QueueAddAndRemove,
				EStateChangeReason::DirectorRequest);
		}

		// Clear the SoftGC diagnostic ExpectedNeverLoadPackages because we have new assigned packages
		// that we didn't consider during SoftGC
		COTFS.PackageTracker->ClearExpectedNeverLoadPackages();
	}
}

void FCookWorkerClient::Register(IMPCollector* Collector)
{
	TRefCountPtr<IMPCollector>& Existing = Collectors.FindOrAdd(Collector->GetMessageType());
	if (Existing)
	{
		UE_LOGF(LogCook, Error,
			"Duplicate IMPCollectors registered. Guid: %ls, Existing: %ls, Registering: %ls. Keeping the Existing.",
			*Collector->GetMessageType().ToString(), Existing->GetDebugName(), Collector->GetDebugName());
		return;
	}
	Existing = Collector;
}

void FCookWorkerClient::Unregister(IMPCollector* Collector)
{
	TRefCountPtr<IMPCollector> Existing;
	Collectors.RemoveAndCopyValue(Collector->GetMessageType(), Existing);
	if (Existing && Existing.GetReference() != Collector)
	{
		UE_LOGF(LogCook, Error,
			"Duplicate IMPCollector during Unregister. Guid: %ls, Existing: %ls, Unregistering: %ls. Ignoring the Unregister.",
			*Collector->GetMessageType().ToString(), Existing->GetDebugName(), Collector->GetDebugName());
		Collectors.Add(Collector->GetMessageType(), MoveTemp(Existing));
	}
}

void FCookWorkerClient::FlushLogs()
{
	FTickStackData TickData(MAX_flt, ECookTickFlags::None);
	TickCollectors(TickData, true, LogMessageHandler);
}

void FCookWorkerClient::TickCollectors(FTickStackData& StackData, bool bFlush, IMPCollector* SingleCollector)
{
	if (StackData.LoopStartTime < NextTickCollectorsTimeSeconds && !bFlush)
	{
		return;
	}

	if (!Collectors.IsEmpty())
	{
		FMPCollectorClientTickContext Context;
		Context.Platforms = OrderedSessionPlatforms;
		Context.bFlush = bFlush;
		TArray<UE::CompactBinaryTCP::FMarshalledMessage> MarshalledMessages;

		auto TickCollector = [&MarshalledMessages, &Context](IMPCollector* Collector)
		{
			Collector->ClientTick(Context);
			if (!Context.Messages.IsEmpty())
			{
				FGuid MessageType = Collector->GetMessageType();
				for (FCbObject& Object : Context.Messages)
				{
					MarshalledMessages.Add({ MessageType, MoveTemp(Object) });
				}
				Context.Messages.Reset();
			}
		};

		if (SingleCollector)
		{
			TickCollector(SingleCollector);
		}
		else
		{
			for (const TPair<FGuid, TRefCountPtr<IMPCollector>>& Pair : Collectors)
			{
				TickCollector(Pair.Value.GetReference());
			}
		}

		if (!MarshalledMessages.IsEmpty())
		{
			UE::CompactBinaryTCP::EConnectionStatus Status
				= UE::CompactBinaryTCP::TryWritePacket(ServerSocket, SendBuffer, MoveTemp(MarshalledMessages));
			UpdateSocketSendDiagnostics(Status);
		}
	}

	constexpr float TickCollectorsPeriodSeconds = 10.f;
	NextTickCollectorsTimeSeconds = FPlatformTime::Seconds() + TickCollectorsPeriodSeconds;
}

void FCookWorkerClient::HandleAbortPackagesMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
	FAbortPackagesMessage&& Message)
{
	if (!bReadSuccessful)
	{
		LogInvalidMessage(TEXT("AbortPackagesMessage"));
		return;
	}

	for (FName PackageName : Message.PackageNames)
	{
		FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(PackageName);
		if (PackageData)
		{
			COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove,
				ESuppressCookReason::RetractedByCookDirector);
		}
	}
}

void FCookWorkerClient::HandleRetractionMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
	FRetractionRequestMessage&& Message)
{
	if (!bReadSuccessful)
	{
		LogInvalidMessage(TEXT("RetractionRequestMessage"));
		return;
	}

	TArray<FName> PackageNames;
	COTFS.GetPackagesToRetract(Message.RequestedCount, PackageNames);
	for (FName PackageName : PackageNames)
	{
		FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(PackageName);
		check(PackageData);
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
		if (!GenerationHelper)
		{
			GenerationHelper = PackageData->GetParentGenerationHelper();
		}
		bool bShouldStall = false;
		if (GenerationHelper)
		{
			bShouldStall = GenerationHelper->ShouldRetractionStallRatherThanDemote(*PackageData);
		}
		if (bShouldStall)
		{
			UE_LOGF(LogCook, Display, "Retracting generated package %ls; it will remain in memory on this worker until the generator finishes saving.",
				*WriteToString<256>(PackageData->GetPackageName()));
			PackageData->Stall(EPackageState::SaveStalledRetracted, ESendFlags::QueueAddAndRemove);
		}
		else
		{
			COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove,
				ESuppressCookReason::RetractedByCookDirector);
			PackageData->ResetReachable(EReachability::All);
		}
	}

	UE_LOGF(LogCook, Display, "Retraction message received from director. %d packages retracted.",
		PackageNames.Num());

	// Queue the message for sending rather than sending it immediately. We need it to be processed on the director
	// AFTER any PendingResults or GeneratorEvent messages. Most notably, if we have sent
	// EGeneratorEvent::QueuedGeneratedPackages for a retracted generator package, we need the director to process
	// that before it processes the retraction message, otherwise it might assign the generator to itself, finish the
	// save of the generator package, and then receive our QueuedGeneratedPackages and call
	// EndQueueGeneratedPackagesOnDirector, which is invalid to receive after a generator package has finished saving.
	FRetractionResultsMessage ResultsMessage;
	ResultsMessage.ReturnedPackages = MoveTemp(PackageNames);
	QueueMessage(MarshalToCompactBinaryTCP(ResultsMessage));
}

void FCookWorkerClient::HandleHeartbeatMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
	FHeartbeatMessage&& Message)
{
	if (!bReadSuccessful)
	{
		LogInvalidMessage(TEXT("HeartbeatMessage"));
		return;
	}

	UE_LOGF(LogCook, Display, "%.*ls %d",
		HeartbeatCategoryText.Len(), HeartbeatCategoryText.GetData(), Message.HeartbeatNumber);
	SendMessage(FHeartbeatMessage(Message.HeartbeatNumber));
}

} // namespace UE::Cook