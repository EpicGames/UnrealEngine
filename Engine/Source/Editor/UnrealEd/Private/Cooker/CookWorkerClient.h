// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CompactBinaryTCP.h"
#include "Containers/Array.h"
#include "Cooker/CookTypes.h"
#include "Cooker/MPCollector.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "HAL/CriticalSection.h"
#include "IPAddress.h"
#include "Misc/Guid.h"
#include "Templates/UniquePtr.h"

namespace UE::Cook { class FLogMessagesMessageHandler; }
namespace UE::Cook { class FMPCollectorClientMessageContext; }
namespace UE::Cook { class IMPCollector; }
namespace UE::Cook { struct FAbortPackagesMessage; }
namespace UE::Cook { struct FAssignPackagesMessage; }
namespace UE::Cook { struct FDirectorConnectionInfo; }
namespace UE::Cook { struct FDirectorEventMessage; }
namespace UE::Cook { struct FDiscoveredPackageReplication; }
namespace UE::Cook { struct FFileTransferMessage; }
namespace UE::Cook { struct FHeartbeatMessage; }
namespace UE::Cook { struct FInitialConfigMessage; }
namespace UE::Cook { struct FGenerationHelper; }
namespace UE::Cook { struct FGeneratorEventMessage; }
namespace UE::Cook { struct FPackageRemoteResult; }
namespace UE::Cook { struct FReplicatedLogData; }
namespace UE::Cook { struct FRetractionRequestMessage; }

namespace UE::Cook
{

/** Class in a CookWorker process that communicates over a Socket with FCookWorkerServer in a Director process. */
class FCookWorkerClient
{
public:
	FCookWorkerClient(UCookOnTheFlyServer& COTFS);
	~FCookWorkerClient();

	/** Blocking operation: open the socket to the Director, send the Connected message, receive the setup message. */
	bool TryConnect(FDirectorConnectionInfo&& ConnectInfo);
	/** Periodic Tick function to send and receive messages to the Server. */
	void TickFromSchedulerThread(FTickStackData& StackData);
	/** Is this either shutting down or completed shutdown of communications with the Director? */
	bool IsDisconnecting() const;
	/** Is this not yet or no longer connected to the Director? */
	bool IsDisconnectComplete() const;

	/** Reads the cookmode that was sent from the Director. */
	ECookMode::Type GetDirectorCookMode() const { return DirectorCookMode; }
	/** Reads the OrderedSessionPlatforms received from the Director. */
	const TArray<ITargetPlatform*>& GetTargetPlatforms() const;
	/** Consumes the initialization settings from the Director. Only available during initialization. */
	ECookInitializationFlags GetCookInitializationFlags();
	bool GetInitializationIsZenStore();
	FInitializeConfigSettings&& ConsumeInitializeConfigSettings();
	FBeginCookConfigSettings&& ConsumeBeginCookConfigSettings();
	FCookByTheBookOptions&& ConsumeCookByTheBookOptions();
	const FBeginCookContextForWorker& GetBeginCookContext();
	FCookOnTheFlyOptions&& ConsumeCookOnTheFlyOptions();
	/** Mark that initialization is complete and we can free the memory for initialization settings. */
	void DoneWithInitialSettings();
	bool HasRunFinished() const { return bHasRunFinished; }
	void SetHasRunFinished(bool Value) { bHasRunFinished = Value; }

	/** Queue a message to the server that the Package was cook-suppressed. Will be sent during Tick. */
	void ReportDemotion(const FPackageData& PackageData, ESuppressCookReason Reason);
	/** Queue a message to the server that the Package was saved. Will be sent during Tick. */
	void ReportPromoteToSaveComplete(FPackageData& PackageData);
	/** Queue a message to the server that a package was discovered as needed in the cook. Will be sent during Tick. */
	void ReportDiscoveredPackage(const FPackageData& PackageData, const FInstigator& Instigator,
		FDiscoveredPlatformSet&& ReachablePlatforms, FGenerationHelper* ParentGenerationHelper, EUrgency Urgency);
	void ReportLogMessage(const FReplicatedLogData& LogData);
	void ReportFileTransfer(FStringView TempFileName, FStringView TargetFileName);

	/**
	 * Queue a message to the server that a PackageGenerator queued its generated packages for cooking, and will
	 * keep itself in memory until it receives an EGeneratorEvent::QueuedGeneratedPackagesFencePassed.
	 */
	void ReportGeneratorQueuedGeneratedPackages(FGenerationHelper& GenerationHelper);

	/** Register a Collector for periodic ticking that sends messages to the Director. */
	void Register(IMPCollector* Collector);
	/** Unegister a Collector that was registered. */
	void Unregister(IMPCollector* Collector);

	/** Called on worker cook process shutdown to flush any remaining log messages. */
	void FlushLogs();

private:
	enum class EConnectStatus
	{
		Uninitialized,
		PollWriteConnectMessage,
		PollReceiveConfigMessage,
		Connected,
		FlushAndAbortFirst,
		WaitForAbortAcknowledge,
		FlushAndAbortLast = WaitForAbortAcknowledge,
		LostConnection,
	};

private:
	/** Reentrant helper for TryConnect which early exits if currently blocked. */
	EPollStatus PollTryConnect(const FDirectorConnectionInfo& ConnectInfo);
	/** Helper for PollTryConnect: create the ServerSocket */
	void CreateServerSocket(const FDirectorConnectionInfo& ConnectInfo);
	/** Try to send the Connect Message, switch state when it succeeds or fails. */
	void PollWriteConnectMessage();
	/** Wait for the Config Message, switch state when it succeeds or fails. */
	void PollReceiveConfigMessage();
	void LogConnected();
	/**
	 * Helper for Tick. Gather messages from pending package results, queued messages, and any other sources, and add
	 * them to the SendBuffer.
	 */
	void SendPendingMessages();
	/** Helper for Tick, pump the messages that have been added to the buffer to send to the Server. */
	void PumpSendBuffer();
	/** Helper for Tick, pump receive messages from the Server. */
	void PumpReceiveMessages();
	/** Helper for PumpReceiveMessages: dispatch the messages received from the socket. */
	void HandleReceiveMessages(TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& Messages, FName OptionalPackageName = NAME_None);
	/** Helper for Tick, pump Send/Receive and check for whether we are done shutting down. */
	void PumpDisconnect(FTickStackData& StackData);
	/** Send the message immediately to the Socket. If cannot complete immediately, it will be finished during Tick. */
	void SendMessage(const IMPCollectorMessage& Message);
	void SendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message);
	/**
	 * Queue a message for sending during the next SendPendingMessages. @see QueuedMessages for how it is ordered
	 * with other messages.
	 */
	void QueueMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message);
	/** Send this into the given state. Update any state-dependent variables. */
	void SendToState(EConnectStatus TargetStatus);
	void LogInvalidMessage(const TCHAR* MessageTypeName);
	void UpdateSocketSendDiagnostics(UE::CompactBinaryTCP::EConnectionStatus Status);
	/** Send packages assigned from the server into the request state. */
	void AssignPackages(FAssignPackagesMessage& Message);
	/** Tick the registered collectors, or the single given collector if non-null. */
	void TickCollectors(FTickStackData& StackData, bool bFlush, IMPCollector* SingleCollector = nullptr);
	/** Helper for ReportDemote/ReportPromote: Collect IMPCollectors and asynchronously add the message to pending. */
	void ReportPackageMessage(FName PackageName, TUniquePtr<FPackageRemoteResult>&& ResultOwner);

	void HandleAbortPackagesMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		FAbortPackagesMessage&& Message);
	void HandleRetractionMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		FRetractionRequestMessage&& Message);
	void HandleHeartbeatMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		FHeartbeatMessage&& Message);
	void HandleDirectorMessage(FDirectorEventMessage&& GeneratorMessage);
	void HandleGeneratorMessage(FGeneratorEventMessage&& GeneratorMessage);

private:
	/**
	 * A PendingResult constructed during ReportPromoteToSaveComplete that is not yet ready to 
	 * send because it has some asynchronous messages still pending.
	 */
	struct FPendingResultNeedingAsyncWork
	{
		FPendingResultNeedingAsyncWork() = default;
		FPendingResultNeedingAsyncWork(FPendingResultNeedingAsyncWork&&) = default;
		FPendingResultNeedingAsyncWork& operator=(FPendingResultNeedingAsyncWork&&) = default;

		TUniquePtr<FPackageRemoteResult> PendingResult;
		TFuture<void> CompletionFuture;
	};
private:
	// Variables Read/Write only from the Scheduler thread
	TSharedPtr<FInternetAddr> DirectorAddr;
	TUniquePtr<FInitialConfigMessage> InitialConfigMessage;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> DeferredInitializationMessages;
	TRefCountPtr<FLogMessagesMessageHandler> LogMessageHandler;
	TArray<ITargetPlatform*> OrderedSessionPlatforms;
	TArray<ITargetPlatform*> OrderedSessionAndSpecialPlatforms;
	TArray<FDiscoveredPackageReplication> PendingDiscoveredPackages;
	TMap<FGuid, TRefCountPtr<IMPCollector>> Collectors;
	TArray<FGeneratorEventMessage> PendingGeneratorEvents;
	UE::CompactBinaryTCP::FSendBuffer SendBuffer;
	UE::CompactBinaryTCP::FReceiveBuffer ReceiveBuffer;
	FString DirectorURI;
	UCookOnTheFlyServer& COTFS;
	FSocket* ServerSocket = nullptr;
	double ConnectStartTimeSeconds = 0.;
	double NextTickCollectorsTimeSeconds = 0.;
	double LastTimeOfCompleteSocketStatusSeconds = 0.;
	double LastTimeOfWarningOfSocketStatusSeconds = 0.;
	EConnectStatus ConnectStatus = EConnectStatus::Uninitialized;
	ECookMode::Type DirectorCookMode = ECookMode::CookByTheBook;
	bool bHasRunFinished = false;

	// Variables Read/Write only within PendingResultsLock
	FCriticalSection PendingResultsLock;
	TArray<TUniquePtr<FPackageRemoteResult>> PendingResults;
	TMap<FPackageRemoteResult*, FPendingResultNeedingAsyncWork> PendingResultsNeedingAsyncWork;
	/**
	 * QueuedMessages are added to the SendBuffer during Tick, after PendingResults, GeneratorEvents, and
	 * FileTransfers. QueuedMessages are sent in the order in which they are queued, but they are unordered
	 * with respect to PendingResults/GeneratorEvents/FileTransfers. A QueuedMessage can be sent after a
	 * PendingResult/GeneratorEvent/FileTransfer even if it was queued before it.
	 */
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> QueuedMessages;

	// Variables Read/Write only within FileTransferLock
	FCriticalSection FileTransferLock;
	TArray<FFileTransferMessage> PendingFileTransfers;
};

}