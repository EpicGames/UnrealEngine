// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookWorkerServer.h"

#include "Algo/Find.h"
#include "Commandlets/AssetRegistryGenerator.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookLogPrivate.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Misc/ScopeLock.h"
#include "PackageResultsMessage.h"
#include "PackageTracker.h"
#include "UnrealEdMisc.h"

namespace UE::Cook
{

FCookWorkerServer::FCookWorkerServer(FCookDirector& InDirector, int32 InProfileId, FWorkerId InWorkerId)
	: Director(InDirector)
	, COTFS(InDirector.COTFS)
	, ProfileId(InProfileId)
	, WorkerId(InWorkerId)
{
}

FCookWorkerServer::~FCookWorkerServer()
{
	FCommunicationScopeLock ScopeLock(this, ECookDirectorThread::CommunicateThread, ETickAction::Queue);

	checkf(PendingPackages.IsEmpty() && PackagesToAssign.IsEmpty(),
		TEXT("CookWorkerServer still has assigned packages when it is being destroyed; we will leak them and block the cook."));

	if (ConnectStatus == EConnectStatus::Connected || ConnectStatus == EConnectStatus::PumpingCookComplete
		|| ConnectStatus == EConnectStatus::WaitForDisconnect)
	{
		UE_LOGF(LogCook, Error,
			"CookWorkerServer %d was destroyed before it finished Disconnect. The remote process may linger and may interfere with writes of future packages.",
			ProfileId);
	}
	DetachFromRemoteProcess(EWorkerDetachType::StillRunning);
}

void FCookWorkerServer::DetachFromRemoteProcess(EWorkerDetachType DetachType)
{
	if (Socket != nullptr)
	{
		FCoreDelegates::OnMultiprocessWorkerDetached.Broadcast({WorkerId.GetMultiprocessId(), DetachType != EWorkerDetachType::Dismissed});
	}
	Sockets::CloseSocket(Socket);
	CookWorkerHandle = FProcHandle();
	CookWorkerProcessId = 0;
	bTerminateImmediately = false;
	SendBuffer.Reset();
	ReceiveBuffer.Reset();

	if (bNeedCrashDiagnostics)
	{
		SendCrashDiagnostics();
	}
}

bool TryParseLogCategoryVerbosityMessage(FStringView Line, FName& OutCategory, ELogVerbosity::Type& OutVerbosity,
	FStringView& OutMessage)
{
	TPair<FStringView, ELogVerbosity::Type> VerbosityMarkers[]{
		{ TEXTVIEW(": Fatal:"), ELogVerbosity::Fatal },
		{ TEXTVIEW(": Error:"), ELogVerbosity::Error },
		{ TEXTVIEW(": Warning:"), ELogVerbosity::Warning},
		{ TEXTVIEW(": Display:"), ELogVerbosity::Display },
		{ TEXTVIEW(":"), ELogVerbosity::Log },
	};


	// Find the first colon not in brackets and look for ": <Verbosity>:". This is complicated by Log verbosity not
	// printing out the Verbosity:
	// [2023.03.20-16.32.48:878][  0]LogCook: MessageText
	// [2023.03.20-16.32.48:878][  0]LogCook: Display: MessageText

	int32 FirstColon = INDEX_NONE;
	int32 SubExpressionLevel = 0;
	for (int32 Index = 0; Index < Line.Len(); ++Index)
	{
		switch (Line[Index])
		{
		case '[':
			++SubExpressionLevel;
			break;
		case ']':
			if (SubExpressionLevel > 0)
			{
				--SubExpressionLevel;
			}
			break;
		case ':':
			if (SubExpressionLevel == 0)
			{
				FirstColon = Index;
			}
			break;
		default:
			break;
		}
		if (FirstColon != INDEX_NONE)
		{
			break;
		}
	}
	if (FirstColon == INDEX_NONE)
	{
		return false;
	}

	FStringView RestOfLine = FStringView(Line).RightChop(FirstColon);
	for (TPair<FStringView, ELogVerbosity::Type>& VerbosityPair : VerbosityMarkers)
	{
		if (RestOfLine.StartsWith(VerbosityPair.Key, ESearchCase::IgnoreCase))
		{
			int32 CategoryEndIndex = FirstColon;
			while (CategoryEndIndex > 0 && FChar::IsWhitespace(Line[CategoryEndIndex - 1])) --CategoryEndIndex;
			int32 CategoryStartIndex = CategoryEndIndex > 0 ? CategoryEndIndex - 1 : CategoryEndIndex;
			while (CategoryStartIndex > 0 && FChar::IsAlnum(Line[CategoryStartIndex - 1])) --CategoryStartIndex;
			int32 MessageStartIndex = CategoryEndIndex + VerbosityPair.Key.Len();
			while (MessageStartIndex < Line.Len() && FChar::IsWhitespace(Line[MessageStartIndex])) ++MessageStartIndex;

			OutCategory = FName(FStringView(Line).SubStr(CategoryStartIndex, CategoryEndIndex - CategoryStartIndex));
			OutVerbosity = VerbosityPair.Value;
			OutMessage = FStringView(Line).SubStr(MessageStartIndex, Line.Len() - MessageStartIndex);
			return true;
		}
	}
	return false;
}

void FCookWorkerServer::SendCrashDiagnostics()
{
	FString LogFileName = Director.GetWorkerLogFileName(ProfileId);
	UE_LOGF(LogCook, Display,
		"LostConnection to CookWorker %d. Log messages written after communication loss:", ProfileId);
	FString LogText;
	// To be able to open a file for read that might be open for write from another process,
	// we have to specify FILEREAD_AllowWrite
	int32 ReadFlags = FILEREAD_AllowWrite;
	bool bLoggedErrorMessage = false;
	if (!FFileHelper::LoadFileToString(LogText, *LogFileName, FFileHelper::EHashOptions::None, ReadFlags))
	{
		UE_LOGF(LogCook, Warning, "No log file found for CookWorker %d.", ProfileId);
	}
	else
	{
		FString LastSentHeartbeat = FString::Printf(TEXT("%.*s %d"), HeartbeatCategoryText.Len(),
			HeartbeatCategoryText.GetData(), LastReceivedHeartbeatNumber);
		int32 StartIndex = INDEX_NONE;
		for (FStringView MarkerText : { FStringView(LastSentHeartbeat),
			HeartbeatCategoryText, TEXTVIEW("Connection to CookDirector successful") })
		{
			StartIndex = UE::String::FindLast(LogText, MarkerText);
			if (StartIndex >= 0)
			{
				break;
			}
		}
		const TCHAR* StartText = *LogText;
		FString Line;
		if (StartIndex != INDEX_NONE)
		{
			// Skip the MarkerLine
			StartText = *LogText + StartIndex;
			FParse::Line(&StartText, Line);
			if (*StartText == '\0')
			{
				// If there was no line after the MarkerLine, write out the MarkerLine
				StartText = *LogText + StartIndex;
			}
		}

		while (FParse::Line(&StartText, Line))
		{
			// Get the Category,Severity,Message out of each line and log it with that Category and Severity
			// TODO: Change the CookWorkers to write out structured logs rather than interpreting their text logs
			FName Category;
			ELogVerbosity::Type Verbosity;
			FStringView Message;
			if (!TryParseLogCategoryVerbosityMessage(Line, Category, Verbosity, Message))
			{
				Category = LogCook.GetCategoryName();
				Verbosity = ELogVerbosity::Display;
				Message = Line;
			}
			// Downgrade Fatals in our local verbosity from Fatal to Error to avoid crashing the CookDirector
			if (Verbosity == ELogVerbosity::Fatal)
			{
				Verbosity = ELogVerbosity::Error;
			}
			bLoggedErrorMessage |= Verbosity == ELogVerbosity::Error;
			// NOTE:  If you change the format of [CookWorker %d] you must also change
			// the corresponding hash aononymization in HashedIssueHandler.cs, and generally
			// search for [CookWorker as it's cut and pasted around a few spots. 
			FMsg::Logf(__FILE__, __LINE__, UE_FNAME_TO_LOG_CATEGORY_NAME(Category), Verbosity, TEXT("[CookWorker %d]: %.*s"),
				ProfileId, Message.Len(), Message.GetData());
		}
	}
	if (!CrashDiagnosticsError.IsEmpty())
	{
		if (!bLoggedErrorMessage)
		{
			UE_LOGF(LogCook, Error, "%ls", *CrashDiagnosticsError);
		}
		else
		{
			// When we already logged an error from the crashed worker, log the what-went-wrong as a warning rather
			// than an error, to avoid making it seem like a separate issue.
			UE_LOGF(LogCook, Warning, "%ls", *CrashDiagnosticsError);
		}
	}

	bNeedCrashDiagnostics = false;
	CrashDiagnosticsError.Empty();
}

void FCookWorkerServer::ShutdownRemoteProcess()
{
	EWorkerDetachType DetachType = EWorkerDetachType::Dismissed;
	if (CookWorkerHandle.IsValid())
	{
		FPlatformProcess::TerminateProc(CookWorkerHandle, /* bKillTree */true);
		DetachType = EWorkerDetachType::ForceTerminated;
	}
	DetachFromRemoteProcess(DetachType);
}

void FCookWorkerServer::AppendAssignments(TArrayView<FPackageData*> Assignments,
	TMap<FPackageData*, FAssignPackageExtraData>&& ExtraDatas, TArrayView<FPackageData*> InfoPackages,
	ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Queue);
	++PackagesAssignedFenceMarker;
	PackagesToAssign.Append(Assignments);
	PackagesToAssignExtraDatas.Append(MoveTemp(ExtraDatas));
	PackagesToAssignInfoPackages.Append(InfoPackages);
}

void FCookWorkerServer::AbortAllAssignments(TSet<FPackageData*>& OutPendingPackages, ECookDirectorThread TickThread,
	int32 CurrentHeartbeat)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Queue);
	AbortAllAssignmentsInLock(OutPendingPackages, CurrentHeartbeat);
}

void FCookWorkerServer::AbortAllAssignmentsInLock(TSet<FPackageData*>& OutPendingPackages, int32 CurrentHeartbeat)
{
	if (PendingPackages.Num())
	{
		if (ConnectStatus == EConnectStatus::Connected)
		{
			TArray<FName> PackageNames;
			PackageNames.Reserve(PendingPackages.Num());
			for (FPackageData* PackageData : PendingPackages)
			{
				PackageNames.Add(PackageData->GetPackageName());
			}
			SendMessageInLock(FAbortPackagesMessage(MoveTemp(PackageNames)));
		}
		OutPendingPackages.Append(MoveTemp(PendingPackages));
		PendingPackages.Empty();
	}
	OutPendingPackages.Append(PackagesToAssign);
	PackagesToAssign.Empty();
	PackagesToAssignExtraDatas.Empty();
	PackagesToAssignInfoPackages.Empty();
	++PackagesRetiredFenceMarker;
}

void FCookWorkerServer::AbortAssignment(FPackageData& PackageData, ECookDirectorThread TickThread,
	int32 CurrentHeartbeat, ENotifyRemote NotifyRemote)
{
	FPackageData* PackageDataPtr = &PackageData;
	AbortAssignments(TConstArrayView<FPackageData*>(&PackageDataPtr, 1), TickThread, CurrentHeartbeat, NotifyRemote);
}

void FCookWorkerServer::AbortAssignments(TConstArrayView<FPackageData*> PackageDatas, ECookDirectorThread TickThread,
	int32 CurrentHeartbeat, ENotifyRemote NotifyRemote)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Queue);

	TArray<FName> PackageNamesToMessage;
	bool bSignalRemote = ConnectStatus == EConnectStatus::Connected && NotifyRemote == ENotifyRemote::NotifyRemote;
	for (FPackageData* PackageData : PackageDatas)
	{
		if (PendingPackages.Remove(PackageData))
		{
			if (bSignalRemote)
			{
				PackageNamesToMessage.Add(PackageData->GetPackageName());
			}
		}

		PackagesToAssign.Remove(PackageData);
		PackagesToAssignExtraDatas.Remove(PackageData);
		// We don't remove InfoPackages from PackagesToAssignInfoPackages because it would be too hard to calculate,
		// and it's not a problem to send extra InfoPackages.
	}
	++PackagesRetiredFenceMarker;
	if (!PackageNamesToMessage.IsEmpty())
	{
		SendMessageInLock(FAbortPackagesMessage(MoveTemp(PackageNamesToMessage)));
	}
	LastAbortHeartbeatNumber = CurrentHeartbeat;
}

void FCookWorkerServer::AbortWorker(TSet<FPackageData*>& OutPendingPackages, ECookDirectorThread TickThread,
	int32 CurrentHeartbeat)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);

	AbortAllAssignmentsInLock(OutPendingPackages, CurrentHeartbeat);
	switch (ConnectStatus)
	{
	case EConnectStatus::Uninitialized: // Fall through
	case EConnectStatus::WaitForConnect:
		SendToState(EConnectStatus::LostConnection);
		break;
	case EConnectStatus::Connected: // Fall through
	case EConnectStatus::PumpingCookComplete:
	{
		SendMessageInLock(FAbortWorkerMessage(FAbortWorkerMessage::EType::Abort));
		SendToState(EConnectStatus::WaitForDisconnect);
		break;
	}
	default:
		break;
	}
}

void FCookWorkerServer::SendToState(EConnectStatus TargetStatus)
{
	switch (TargetStatus)
	{
	case EConnectStatus::WaitForConnect:
		ConnectStartTimeSeconds = FPlatformTime::Seconds();
		ConnectTestStartTimeSeconds = ConnectStartTimeSeconds;
		break;
	case EConnectStatus::WaitForDisconnect:
		ConnectStartTimeSeconds = FPlatformTime::Seconds();
		ConnectTestStartTimeSeconds = ConnectStartTimeSeconds;
		break;
	case EConnectStatus::PumpingCookComplete:
		ConnectStartTimeSeconds = FPlatformTime::Seconds();
		ConnectTestStartTimeSeconds = ConnectStartTimeSeconds;
		break;
	case EConnectStatus::LostConnection:
		DetachFromRemoteProcess(bNeedCrashDiagnostics ? EWorkerDetachType::Crashed : EWorkerDetachType::Dismissed);
		break;
	default:
		break;
	}
	ConnectStatus = TargetStatus;
}

bool FCookWorkerServer::IsConnected() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return ConnectStatus == EConnectStatus::Connected;
}

bool FCookWorkerServer::IsShuttingDown() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return ConnectStatus == EConnectStatus::PumpingCookComplete || ConnectStatus == EConnectStatus::WaitForDisconnect
		|| ConnectStatus == EConnectStatus::LostConnection;
}

bool FCookWorkerServer::IsFlushingBeforeShutdown() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return ConnectStatus == EConnectStatus::PumpingCookComplete;
}

bool FCookWorkerServer::IsShutdownComplete() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return ConnectStatus == EConnectStatus::LostConnection;
}

int32 FCookWorkerServer::NumAssignments() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return PackagesToAssign.Num() + PendingPackages.Num();
}

bool FCookWorkerServer::HasMessages() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return !ReceiveMessages.IsEmpty();
}

int32 FCookWorkerServer::GetLastReceivedHeartbeatNumber() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return LastReceivedHeartbeatNumber;

}
void FCookWorkerServer::SetLastReceivedHeartbeatNumberInLock(int32 InHeartbeatNumber)
{
	LastReceivedHeartbeatNumber = InHeartbeatNumber;
}

int32 FCookWorkerServer::GetPackagesAssignedFenceMarker() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return PackagesAssignedFenceMarker;
}

int32 FCookWorkerServer::GetPackagesRetiredFenceMarker() const
{
	FScopeLock CommunicationScopeLock(&CommunicationLock);
	return PackagesRetiredFenceMarker;
}

bool FCookWorkerServer::TryHandleConnectMessage(FWorkerConnectMessage& Message, FSocket* InSocket,
	TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& OtherPacketMessages, ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);

	if (ConnectStatus != EConnectStatus::WaitForConnect)
	{
		return false;
	}
	check(!Socket);
	Socket = InSocket;

	SendToState(EConnectStatus::Connected);
	UE_LOGF(LogCook, Display, "CookWorker %d connected after %.3fs.", ProfileId,
		static_cast<float>(FPlatformTime::Seconds() - ConnectStartTimeSeconds));
	for (UE::CompactBinaryTCP::FMarshalledMessage& OtherMessage : OtherPacketMessages)
	{
		ReceiveMessages.Add(MoveTemp(OtherMessage));
	}
	HandleReceiveMessagesInternal();
	const FInitialConfigMessage& InitialConfigMessage = Director.GetInitialConfigMessage();
	OrderedSessionPlatforms = InitialConfigMessage.GetOrderedSessionPlatforms();
	OrderedSessionAndSpecialPlatforms.Reset(OrderedSessionPlatforms.Num() + 1);
	OrderedSessionAndSpecialPlatforms.Append(OrderedSessionPlatforms);
	OrderedSessionAndSpecialPlatforms.Add(CookerLoadingPlatformKey);
	SendMessageInLock(InitialConfigMessage);
	return true;
}

void FCookWorkerServer::TickCommunication(ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);

	for (;;)
	{
		switch (ConnectStatus)
		{
		case EConnectStatus::Uninitialized:
			LaunchProcess();
			break;
		case EConnectStatus::WaitForConnect:
			TickWaitForConnect();
			if (ConnectStatus == EConnectStatus::WaitForConnect)
			{
				return; // Try again later
			}
			break;
		case EConnectStatus::Connected:
			PumpReceiveMessages();
			if (ConnectStatus == EConnectStatus::Connected)
			{
				SendPendingMessages();
				PumpSendMessages();
				return; // Tick duties complete; yield the tick
			}
			break;
		case EConnectStatus::PumpingCookComplete:
		{
			PumpReceiveMessages();
			if (ConnectStatus == EConnectStatus::PumpingCookComplete)
			{
				PumpSendMessages();
				constexpr float WaitForPumpCompleteTimeout = 10.f * 60;
				if (FPlatformTime::Seconds() - ConnectStartTimeSeconds <= WaitForPumpCompleteTimeout 
					|| IsCookIgnoreTimeouts())
				{
					return; // Try again later
				}
				UE_LOGF(LogCook, Error,
					"CookWorker process of CookWorkerServer %d failed to finalize its cook within %.0f seconds; we will tell it to shutdown.",
					ProfileId, WaitForPumpCompleteTimeout);
				SendMessageInLock(FAbortWorkerMessage(FAbortWorkerMessage::EType::Abort));
				SendToState(EConnectStatus::WaitForDisconnect);
			}
			break;
		}
		case EConnectStatus::WaitForDisconnect:
			TickWaitForDisconnect();
			if (ConnectStatus == EConnectStatus::WaitForDisconnect)
			{
				return; // Try again later
			}
			break;
		case EConnectStatus::LostConnection:
			return; // Nothing further to do
		default:
			checkNoEntry();
			return;
		}
	}
}

void FCookWorkerServer::SignalHeartbeat(ECookDirectorThread TickThread, int32 HeartbeatNumber)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);

	switch (ConnectStatus)
	{
	case EConnectStatus::Connected:
		SendMessageInLock(FHeartbeatMessage(HeartbeatNumber));
		break;
	default:
		break;
	}
}

void FCookWorkerServer::SignalCookComplete(ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);

	switch (ConnectStatus)
	{
	case EConnectStatus::Uninitialized: // Fall through
	case EConnectStatus::WaitForConnect:
		SendToState(EConnectStatus::LostConnection);
		break;
	case EConnectStatus::Connected:
		SendMessageInLock(FAbortWorkerMessage(FAbortWorkerMessage::EType::CookComplete));
		SendToState(EConnectStatus::PumpingCookComplete);
		break;
	default:
		break; // Already in a disconnecting state
	}
}

void FCookWorkerServer::LaunchProcess()
{
	FCookDirector::FLaunchInfo LaunchInfo = Director.GetLaunchInfo(WorkerId, ProfileId);
	bool bShowCookWorkers = LaunchInfo.ShowWorkerOption == FCookDirector::EShowWorker::SeparateWindows;

	CookWorkerHandle = FPlatformProcess::CreateProc(*LaunchInfo.CommandletExecutable, *LaunchInfo.WorkerCommandLine,
		true /* bLaunchDetached */, !bShowCookWorkers /* bLaunchHidden */, !bShowCookWorkers /* bLaunchReallyHidden */,
		&CookWorkerProcessId, 0 /* PriorityModifier */, *FPaths::GetPath(LaunchInfo.CommandletExecutable),
		nullptr /* PipeWriteChild */);
	if (CookWorkerHandle.IsValid())
	{
		UE_LOGF(LogCook, Display,
			"CookWorkerServer %d launched CookWorker as WorkerId %d and PID %u with commandline \"%ls\".",
			ProfileId, WorkerId.GetRemoteIndex(), CookWorkerProcessId, *LaunchInfo.WorkerCommandLine);
		FCoreDelegates::OnMultiprocessWorkerCreated.Broadcast({WorkerId.GetMultiprocessId()});
		SendToState(EConnectStatus::WaitForConnect);
	}
	else
	{
		// GetLastError information was logged by CreateProc
		CrashDiagnosticsError = FString::Printf(
			TEXT("CookWorkerCrash: Failed to create process for CookWorker %d. Assigned packages will be returned to the director."),
			ProfileId);
		bNeedCrashDiagnostics = true;
		SendToState(EConnectStatus::LostConnection);
	}
}

void FCookWorkerServer::TickWaitForConnect()
{
	constexpr float TestProcessExistencePeriod = 1.f;
	constexpr float WaitForConnectTimeout = 60.f * 20;

	// When the Socket is assigned we leave the WaitForConnect state, and we set it to null before entering
	check(!Socket);

	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - ConnectTestStartTimeSeconds > TestProcessExistencePeriod)
	{
		if (!FPlatformProcess::IsProcRunning(CookWorkerHandle))
		{
			CrashDiagnosticsError = FString::Printf(
				TEXT("CookWorkerCrash: CookWorker %d process terminated before connecting. Assigned packages will be returned to the director."),
				ProfileId);
			bNeedCrashDiagnostics = true;
			SendToState(EConnectStatus::LostConnection);
			return;
		}
		ConnectTestStartTimeSeconds = FPlatformTime::Seconds();
	}

	if (CurrentTime - ConnectStartTimeSeconds > WaitForConnectTimeout && !IsCookIgnoreTimeouts())
	{
		CrashDiagnosticsError = FString::Printf(
			TEXT("CookWorkerCrash: CookWorker %d process failed to connect within %.0f seconds. Assigned packages will be returned to the director."),
			ProfileId, WaitForConnectTimeout);
		bNeedCrashDiagnostics = true;
		ShutdownRemoteProcess();
		SendToState(EConnectStatus::LostConnection);
		return;
	}
}

void FCookWorkerServer::TickWaitForDisconnect()
{
	constexpr float TestProcessExistencePeriod = 1.f;
	constexpr float WaitForDisconnectTimeout = 60.f * 10;

	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - ConnectTestStartTimeSeconds > TestProcessExistencePeriod)
	{
		if (!FPlatformProcess::IsProcRunning(CookWorkerHandle))
		{
			SendToState(EConnectStatus::LostConnection);
			return;
		}
		ConnectTestStartTimeSeconds = FPlatformTime::Seconds();
	}

	// We might have been blocked from sending the disconnect, so keep trying to flush the buffer
	UE::CompactBinaryTCP::TryFlushBuffer(Socket, SendBuffer);
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> Messages;
	TryReadPacket(Socket, ReceiveBuffer, Messages);

	if (bTerminateImmediately ||
		(CurrentTime - ConnectStartTimeSeconds > WaitForDisconnectTimeout && !IsCookIgnoreTimeouts()))
	{
		UE_CLOGF(!bTerminateImmediately, LogCook, Warning,
			"CookWorker process of CookWorkerServer %d failed to disconnect within %.0f seconds; we will terminate it.",
			ProfileId, WaitForDisconnectTimeout);
		bNeedCrashDiagnostics = true;
		ShutdownRemoteProcess();
		SendToState(EConnectStatus::LostConnection);
	}
}

void FCookWorkerServer::PumpSendMessages()
{
	UE::CompactBinaryTCP::EConnectionStatus Status = UE::CompactBinaryTCP::TryFlushBuffer(Socket, SendBuffer);
	if (Status == UE::CompactBinaryTCP::EConnectionStatus::Failed)
	{
		UE_LOGF(LogCook, Error,
			"CookWorkerCrash: CookWorker %d failed to write to socket, we will shutdown the remote process. Assigned packages will be returned to the director.",
			ProfileId);
		bNeedCrashDiagnostics = true;
		SendToState(EConnectStatus::WaitForDisconnect);
		bTerminateImmediately = true;
	}
}

void FCookWorkerServer::SendPendingMessages()
{
	// If we aborted any packages, do not allow any new assignment messages to be sent until we receive an acknowledge of
	// the abort. This prevents us from incorrectly assuming a package results message that was sent before the abort is
	// the package results message after reassignment of that package, with new requested platforms, that we sent after
	// the abort.
	// Because we contractually are not allowed to send QueuedMessagesToSendAfterPackagesToAssign until after we have sent
	// the assignment message, do not allow those to be sent out either.
	if (LastReceivedHeartbeatNumber <= LastAbortHeartbeatNumber)
	{
		return;
	}

	SendPendingPackages();
	for (UE::CompactBinaryTCP::FMarshalledMessage& MarshalledMessage : QueuedMessagesToSendAfterPackagesToAssign)
	{
		UE::CompactBinaryTCP::QueueMessage(SendBuffer, MoveTemp(MarshalledMessage));
	}
	QueuedMessagesToSendAfterPackagesToAssign.Empty();
}

void FCookWorkerServer::SendPendingPackages()
{
	if (PackagesToAssign.IsEmpty())
	{
		PackagesToAssignExtraDatas.Empty();
		PackagesToAssignInfoPackages.Empty();
		return;
	}
	LLM_SCOPE_BYTAG(Cooker_MPCook);

	TArray<FAssignPackageData> AssignDatas;
	AssignDatas.Reserve(PackagesToAssign.Num());
	TBitArray<> SessionPlatformNeedsCommit;
	TArray<FPackageDataExistenceInfo> ExistenceInfos;
	ExistenceInfos.Reserve(PackagesToAssignInfoPackages.Num());

	ECookPhase CookPhase = Director.COTFS.GetCookPhase();
	EReachability Reachability = CookPhase == ECookPhase::Cook ? EReachability::Runtime : EReachability::Build;

	for (FPackageData* PackageData : PackagesToAssign)
	{
		SessionPlatformNeedsCommit.Init(false, OrderedSessionPlatforms.Num());
		int32 PlatformIndex = 0;
		int32 NumNeedCommitPlatforms = 0;
		for (const ITargetPlatform* SessionPlatform : OrderedSessionPlatforms)
		{
			FPackagePlatformData* PlatformData = PackageData->FindPlatformData(SessionPlatform);
			if (PlatformData && PlatformData->NeedsCommit(SessionPlatform, CookPhase))
			{
				SessionPlatformNeedsCommit[PlatformIndex] = true;
				++NumNeedCommitPlatforms;
			}
			++PlatformIndex;
		}
		// It should not have been added to PackagesToAssign if there are no platforms to commit
		if (NumNeedCommitPlatforms == 0)
		{
			TStringBuilder<256> PlatformDataText;
			for (const TPair<const ITargetPlatform*, FPackagePlatformData>& PlatformPair : PackageData->GetPlatformDatas())
			{
				PlatformDataText.Appendf(TEXT("{ %s: Reachable=%s, Committed=%s }, "),
					PlatformPair.Key == CookerLoadingPlatformKey ? TEXT("CookerLoadingPlatform") : *PlatformPair.Key->PlatformName(),
					PlatformPair.Value.IsReachable(CookPhase == ECookPhase::Cook
						? EReachability::Runtime : EReachability::Build) ? TEXT("true") : TEXT("false"),
					PlatformPair.Value.IsCommitted() ? TEXT("true") : TEXT("false"));
			}
			UE_LOGF(LogCook, Warning, "Package %ls was assigned to worker, but at sendmessage time it has no platforms needing commit. State = %ls. CookPhase = %ls. [ %ls ]",
				*PackageData->GetPackageName().ToString(), LexToString(PackageData->GetState()),
					LexToString(CookPhase), *PlatformDataText);
			// SendPendingMessages is called from the communication thread, so we cannot demote this package to idle
			// immediately; that has to be done on the scheduler thread. To avoid adding special case code to handle
			// this edge case, instead add a received message for it.
			FPackageResultsMessage DemoteMessage;
			FPackageRemoteResult& DemoteResult = DemoteMessage.Results.Emplace_GetRef();
			DemoteResult.SetPackageName(PackageData->GetPackageName());
			DemoteResult.SetSuppressCookReason(ESuppressCookReason::AlreadyCooked);
			DemoteResult.SetPlatforms(OrderedSessionPlatforms);
			ReceiveMessages.Add(MarshalToCompactBinaryTCP(DemoteMessage));
			continue;
		}

		FAssignPackageData& AssignData = AssignDatas.Emplace_GetRef();
		AssignData.ConstructData = PackageData->CreateConstructData();
		AssignData.ParentGenerator = PackageData->GetParentGenerator();
		AssignData.DoesGeneratedRequireGenerator = PackageData->DoesGeneratedRequireGenerator();
		AssignData.Reachability = Reachability;
		AssignData.Instigator = PackageData->GetInstigator(Reachability);
		AssignData.Urgency = PackageData->GetUrgency();

		AssignData.NeedCommitPlatforms = FDiscoveredPlatformSet(SessionPlatformNeedsCommit);
		FAssignPackageExtraData* ExtraData = PackagesToAssignExtraDatas.Find(PackageData);
		if (ExtraData)
		{
			AssignData.bSkipSaveExistingGenerator = ExtraData->bSkipSaveExistingGenerator;
			AssignData.bQueuedGeneratedPackagesFencePassed = ExtraData->bQueuedGeneratedPackagesFencePassed;
			AssignData.GeneratorPerPlatformPreviousGeneratedPackages.Reserve(
				ExtraData->GeneratorPerPlatformPreviousGeneratedPackages.Num());
			for (TPair<const ITargetPlatform*, TMap<FName, FAssetPackageData>>& PlatformPair
				: ExtraData->GeneratorPerPlatformPreviousGeneratedPackages)
			{
				int32 PlatformIdInt = OrderedSessionPlatforms.IndexOfByKey(PlatformPair.Key);
				check(0 <= PlatformIdInt && PlatformIdInt <= MAX_uint8);
				uint8 PlatformId = static_cast<uint8>(PlatformIdInt);
				TMap<FName, FAssetPackageData>& DestMap
					= AssignData.GeneratorPerPlatformPreviousGeneratedPackages.Add(PlatformId);
				DestMap = MoveTemp(PlatformPair.Value);
			}
			AssignData.PerPackageCollectorMessages = MoveTemp(ExtraData->PerPackageCollectorMessages);
		}
	}
	for (FPackageData* PackageData : PackagesToAssignInfoPackages)
	{
		FPackageDataExistenceInfo& ExistenceInfo = ExistenceInfos.Emplace_GetRef();
		ExistenceInfo.ConstructData = PackageData->CreateConstructData();
		ExistenceInfo.ParentGenerator = PackageData->GetParentGenerator();
	}
	PendingPackages.Append(PackagesToAssign);
	PackagesToAssign.Empty();
	PackagesToAssignExtraDatas.Empty();
	PackagesToAssignInfoPackages.Empty();
	FAssignPackagesMessage AssignPackagesMessage(MoveTemp(AssignDatas), MoveTemp(ExistenceInfos));
	AssignPackagesMessage.OrderedSessionPlatforms = OrderedSessionPlatforms;
	SendMessageInLock(MoveTemp(AssignPackagesMessage));
}

void FCookWorkerServer::PumpReceiveMessages()
{
	using namespace UE::CompactBinaryTCP;
	LLM_SCOPE_BYTAG(Cooker_MPCook);
	TArray<FMarshalledMessage> Messages;
	EConnectionStatus SocketStatus = TryReadPacket(Socket, ReceiveBuffer, Messages);
	if (SocketStatus != EConnectionStatus::Okay && SocketStatus != EConnectionStatus::Incomplete)
	{
		CrashDiagnosticsError = FString::Printf(
			TEXT("CookWorkerCrash: CookWorker %d failed to read from socket with description: %s. we will shutdown the remote process. Assigned packages will be returned to the director."),
			ProfileId,
			DescribeStatus(SocketStatus));
		bNeedCrashDiagnostics = true;
		SendToState(EConnectStatus::WaitForDisconnect);
		bTerminateImmediately = true;
		return;
	}
	for (FMarshalledMessage& Message : Messages)
	{
		ReceiveMessages.Add(MoveTemp(Message));
	}
	HandleReceiveMessagesInternal();
}

void FCookWorkerServer::HandleReceiveMessages(ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Queue);
	HandleReceiveMessagesInternal();
}

void FCookWorkerServer::HandleReceiveMessagesInternal()
{
	while (!ReceiveMessages.IsEmpty())
	{
		UE::CompactBinaryTCP::FMarshalledMessage& PeekMessage = ReceiveMessages[0];

		if (PeekMessage.MessageType == FAbortWorkerMessage::MessageType)
		{
			UE::CompactBinaryTCP::FMarshalledMessage Message = ReceiveMessages.PopFrontValue();
			if (ConnectStatus != EConnectStatus::PumpingCookComplete
				&& ConnectStatus != EConnectStatus::WaitForDisconnect)
			{
				CrashDiagnosticsError = FString::Printf(
					TEXT("CookWorkerCrash: CookWorker %d remote process shut down unexpectedly. Assigned packages will be returned to the director."),
					ProfileId);
				bNeedCrashDiagnostics = true;
			}
			SendMessageInLock(FAbortWorkerMessage(FAbortWorkerMessage::AbortAcknowledge));
			SendToState(EConnectStatus::WaitForDisconnect);
			ReceiveMessages.Reset();
			break;
		}

		if (TickState.TickThread != ECookDirectorThread::SchedulerThread)
		{
			break;
		}

		UE::CompactBinaryTCP::FMarshalledMessage Message = ReceiveMessages.PopFrontValue();
		if (Message.MessageType == FPackageResultsMessage::MessageType)
		{
			FPackageResultsMessage ResultsMessage;
			if (!ResultsMessage.TryRead(Message.Object))
			{
				LogInvalidMessage(TEXT("FPackageResultsMessage"));
			}
			else
			{
				RecordResults(ResultsMessage);
			}
		}
		else if (Message.MessageType == FDiscoveredPackagesMessage::MessageType)
		{
			FDiscoveredPackagesMessage DiscoveredMessage;
			DiscoveredMessage.OrderedSessionAndSpecialPlatforms = OrderedSessionAndSpecialPlatforms;
			if (!DiscoveredMessage.TryRead(Message.Object))
			{
				LogInvalidMessage(TEXT("FDiscoveredPackagesMessage"));
			}
			else
			{
				for (FDiscoveredPackageReplication& DiscoveredPackage : DiscoveredMessage.Packages)
				{
					QueueDiscoveredPackage(MoveTemp(DiscoveredPackage));
				}
			}
		}
		else if (Message.MessageType == FGeneratorEventMessage::MessageType)
		{
			FGeneratorEventMessage GeneratorMessage;
			if (!GeneratorMessage.TryRead(Message.Object))
			{
				LogInvalidMessage(TEXT("FGeneratorEventMessage"));
			}
			else
			{
				HandleGeneratorMessage(GeneratorMessage);
			}
		}
		else if (Message.MessageType == FFileTransferMessage::MessageType)
		{
			FFileTransferMessage FileTransferMessage;
			if (!FileTransferMessage.TryRead(Message.Object))
			{
				LogInvalidMessage(TEXT("FFileTransferMessage"));
			}
			else
			{
				Director.HandleFileTransferMessage(*this, MoveTemp(FileTransferMessage));
			}
		}
		else
		{
			TRefCountPtr<IMPCollector>* Collector = Director.Collectors.Find(Message.MessageType);
			if (Collector)
			{
				check(*Collector);
				FMPCollectorServerMessageContext Context;
				Context.Server = this;
				Context.Platforms = OrderedSessionPlatforms;
				Context.WorkerId = WorkerId;
				Context.ProfileId = ProfileId;
				(*Collector)->ServerReceiveMessage(Context, Message.Object);
			}
			else
			{
				UE_LOGF(LogCook, Error,
					"CookWorkerServer received message of unknown type %ls from CookWorker. Ignoring it.",
					*Message.MessageType.ToString());
			}
		}
	}
}

void FCookWorkerServer::HandleReceivedPackagePlatformMessages(FPackageData& PackageData,
	const ITargetPlatform* TargetPlatform, TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& Messages)
{
	check(TickState.TickThread == ECookDirectorThread::SchedulerThread);
	if (Messages.IsEmpty())
	{
		return;
	}

	FMPCollectorServerMessageContext Context;
	Context.Platforms = OrderedSessionPlatforms;
	Context.PackageName = PackageData.GetPackageName();
	Context.TargetPlatform = TargetPlatform;
	Context.Server = this;
	Context.ProfileId = ProfileId;
	Context.WorkerId = WorkerId;

	for (UE::CompactBinaryTCP::FMarshalledMessage& Message : Messages)
	{
		TRefCountPtr<IMPCollector>* Collector = Director.Collectors.Find(Message.MessageType);
		if (Collector)
		{
			check(*Collector);
			(*Collector)->ServerReceiveMessage(Context, Message.Object);
		}
		else
		{
			UE_LOGF(LogCook, Error,
				"CookWorkerServer received PackageMessage of unknown type %ls from CookWorker. Ignoring it.",
				*Message.MessageType.ToString());
		}
	}
}

void FCookWorkerServer::SendMessage(const IMPCollectorMessage& Message, ECookDirectorThread TickThread)
{
	SendMessage(MarshalToCompactBinaryTCP(Message), TickThread);
}

void FCookWorkerServer::SendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message,
	ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Tick);
	SendMessageInLock(MoveTemp(Message));
}

void FCookWorkerServer::AppendMessage(const IMPCollectorMessage& Message, ECookDirectorThread TickThread)
{
	AppendMessage(MarshalToCompactBinaryTCP(Message), TickThread);
}

void FCookWorkerServer::AppendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message,
	ECookDirectorThread TickThread)
{
	FCommunicationScopeLock ScopeLock(this, TickThread, ETickAction::Queue);
	QueuedMessagesToSendAfterPackagesToAssign.Add(MoveTemp(Message));
}

void FCookWorkerServer::SendMessageInLock(const IMPCollectorMessage& Message)
{
	SendMessageInLock(MarshalToCompactBinaryTCP(Message));
}

void FCookWorkerServer::SendMessageInLock(UE::CompactBinaryTCP::FMarshalledMessage&& Message)
{
	if (TickState.TickAction == ETickAction::Tick)
	{
		UE::CompactBinaryTCP::TryWritePacket(Socket, SendBuffer, MoveTemp(Message));
	}
	else
	{
		check(TickState.TickAction == ETickAction::Queue);
		UE::CompactBinaryTCP::QueueMessage(SendBuffer, MoveTemp(Message));
	}
}

FString WriteCookStatus(FPackageData& PackageData, TConstArrayView<const ITargetPlatform*> SessionPlatforms)
{
	FString Result;
	Result.Reserve(256);
	auto BoolToString = [](bool bValue)
		{
			return bValue ? TEXT("true") : TEXT("false");
		};
	for (const ITargetPlatform* TargetPlatform : SessionPlatforms)
	{
		FPackagePlatformData* PlatformData = PackageData.FindPlatformData(TargetPlatform);
		Result.Appendf(TEXT("[ %s: { Reachable: %s, Cookable: %s, CookResult: %s }, "),
			*TargetPlatform->PlatformName(),
			BoolToString(PlatformData && PlatformData->IsReachable(EReachability::Runtime)),
			BoolToString(PlatformData && PlatformData->IsCookable()),
			(PlatformData ? ::LexToString(PlatformData->GetCookResults()) : TEXT("<NotCooked>")));
	}
	if (Result.Len() >= 2)
	{
		Result.LeftChopInline(2); // Remove the trailing ", "
		Result.Append(TEXT(" ]"));
	}

	return Result;
}

TArray<const ITargetPlatform*> GetCommittedPlatformListFromPlatformResults(
	TConstArrayView<const ITargetPlatform*> OrderedPlatforms,
	TConstArrayView<FPackageRemoteResult::FPlatformResult> PlatformResults)
{
	TArray<const ITargetPlatform*> PlatformList;
	int32 NumPlatforms = OrderedPlatforms.Num();
	if (NumPlatforms != PlatformResults.Num())
	{
		return PlatformList;
	}
	for (int32 PlatformIndex = 0; PlatformIndex < NumPlatforms; ++PlatformIndex)
	{
		const FPackageRemoteResult::FPlatformResult& PlatformResult = PlatformResults[PlatformIndex];
		if (PlatformResult.WasCommitted())
		{
			PlatformList.Add(OrderedPlatforms[PlatformIndex]);
		}
	}
	return PlatformList;
}

FString PlatformListToString(TConstArrayView<const ITargetPlatform*> Platforms)
{
	TStringBuilder<256> Result;
	for (const ITargetPlatform* TargetPlatform : Platforms)
	{
		Result << TargetPlatform->PlatformName() << TEXT(", ");
	}
	if (Result.Len() > 0)
	{
		Result.RemoveSuffix(2); // ", "
	}
	return FString(Result);
}

void FCookWorkerServer::RecordResults(FPackageResultsMessage& Message)
{
	check(TickState.TickThread == ECookDirectorThread::SchedulerThread);

	bool bRetiredAnyPackages = false;
	for (FPackageRemoteResult& Result : Message.Results)
	{
		FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(Result.GetPackageName());
		if (!PackageData)
		{
			UE_LOGF(LogCook, Warning,
				"CookWorkerServer %d received FPackageResultsMessage for invalid package %ls. Ignoring it.",
				ProfileId, *Result.GetPackageName().ToString());
			continue;
		}
		if (PendingPackages.Remove(PackageData) != 1)
		{
			UE_LOGF(LogCook, Display,
				"CookWorkerServer %d received FPackageResultsMessage for package %ls which is not a pending package. Ignoring it." "\n\tState = %ls, WorkerId = %ls, CookResults = { %ls }, Result.GetSuppressCookReason = %ls",
				ProfileId, *Result.GetPackageName().ToString(),
				LexToString(PackageData->GetState()), *Director.GetDisplayName(PackageData->GetWorkerAssignment()),
				*WriteCookStatus(*PackageData, COTFS.GetSessionPlatforms()), LexToString(Result.GetSuppressCookReason()));
			continue;
		}
		int32 NumPlatforms = OrderedSessionPlatforms.Num();
		if (Result.GetPlatforms().Num() != NumPlatforms)
		{
			UE_LOGF(LogCook, Error,
				"CookWorkerServer %d received FPackageResultsMessage for package %ls with an invalid number of platform results: expected %d, actual %d. Ignoring it.",
				ProfileId, *Result.GetPackageName().ToString(), NumPlatforms, Result.GetPlatforms().Num());
			continue;
		}
		bRetiredAnyPackages = true;

		bool bResultIsSaveResult = Result.GetSuppressCookReason() == ESuppressCookReason::NotSuppressed;
		EStateChangeReason StateChangeReason;
		bool bTerminalStateChange;
		if (bResultIsSaveResult)
		{
			StateChangeReason = EStateChangeReason::Saved;
			bTerminalStateChange = true;
		}
		else
		{
			StateChangeReason = ConvertToStateChangeReason(Result.GetSuppressCookReason());
			bTerminalStateChange = IsTerminalStateChange(StateChangeReason);
		}

		// MPCOOKTODO: Refactor FSaveCookedPackageContext::FinishPlatform and ::FinishPackage so we can call them from
		// here to reduce duplication
		if (bResultIsSaveResult)
		{
			HandleReceivedPackagePlatformMessages(*PackageData, nullptr, Result.ReleaseMessages());
			for (int32 PlatformIndex = 0; PlatformIndex < NumPlatforms; ++PlatformIndex)
			{
				ITargetPlatform* TargetPlatform = OrderedSessionPlatforms[PlatformIndex];
				FPackageRemoteResult::FPlatformResult& PlatformResult = Result.GetPlatforms()[PlatformIndex];
				FPackagePlatformData& ExistingData = PackageData->FindOrAddPlatformData(TargetPlatform);
				if (ExistingData.IsCommitted())
				{
					if (PlatformResult.WasCommitted())
					{
						UE_LOGF(LogCook, Display,
							"CookWorkerServer %d received FPackageResultsMessage for package %ls, platform %ls, but that platform has already been committed. Ignoring the results for that platform.",
							ProfileId, *Result.GetPackageName().ToString(), *TargetPlatform->PlatformName());
					}
				}
				else
				{
					bool bWasCooked = PlatformResult.GetCookResults() != ECookResult::Invalid
						&& PlatformResult.GetCookResults() != ECookResult::NotAttempted;
					if (!ExistingData.NeedsCooking(TargetPlatform) && bWasCooked)
					{
						UE_LOGF(LogCook, Display,
							"CookWorkerServer %d received FPackageResultsMessage for package %ls, platform %ls, but that platform has already been cooked. Ignoring the results for that platform.",
							ProfileId, *Result.GetPackageName().ToString(), *TargetPlatform->PlatformName());
					}
					else
					{
						if (bWasCooked)
						{
							PackageData->SetPlatformCooked(TargetPlatform, PlatformResult.GetCookResults());
						}
						else if (PlatformResult.WasCommitted())
						{
							PackageData->SetPlatformCommitted(TargetPlatform);
						}
						HandleReceivedPackagePlatformMessages(*PackageData, TargetPlatform,
							PlatformResult.ReleaseMessages());
					}
				}
			}
			COTFS.RecordExternalActorDependencies(Result.GetExternalActorDependencies());
		}

		// For generator and generated packages, after we handle all of their save recording above, execute their state
		// changes in the required order:
		// *) Defer the GenerationHelper's events so that we don't yet complete it if this was the last save.
		// *) Mark saved on the generator, so that the generator has full context for the save.
		// *) Transition the PackageData state to complete. The code to automatically mark generated as saved with the
		//    generator will be skipped since we already did it in the step above.
		// *) Unfreeze the GenerationHelper's events and call OnAllSavesCompleted if necessary.
		TOptional<FGenerationHelper::FScopeDeferEvents> DeferGenerationHelperEvents;

		// If generated or generator, and this is a save or other end-of-cook signal, defer events and mark saved
		if (PackageData->IsGenerated() && bTerminalStateChange)
		{
			TRefCountPtr<FGenerationHelper> ParentGenerationHelper = PackageData->GetOrFindParentGenerationHelper();
			if (!ParentGenerationHelper)
			{
				UE_LOGF(LogCook, Warning,
					"RecordResults received for generated package %ls, but its ParentGenerationHelper has already been destructed so we can not update the save flag. Leaving the save flag unupdated; this might cause workers to run out of memory due to keeping the Generator referenced.",
					*PackageData->GetPackageName().ToString());
			}
			else
			{
				DeferGenerationHelperEvents.Emplace(ParentGenerationHelper);
				for (int32 PlatformIndex = 0; PlatformIndex < NumPlatforms; ++PlatformIndex)
				{
					ITargetPlatform* TargetPlatform = OrderedSessionPlatforms[PlatformIndex];
					FPackageRemoteResult::FPlatformResult& PlatformResult = Result.GetPlatforms()[PlatformIndex];
					if (!bResultIsSaveResult || PlatformResult.WasCommitted())
					{
						ParentGenerationHelper->MarkPackageSavedRemotely(COTFS, *PackageData, TargetPlatform, GetWorkerId());
					}
				}
				// ReleaseCookedPlatformData will called when we leave the SaveState during PromoteToSaveComplete or
				// SendToState below, but we need to call it now before we SetParentGenerationHelper(nullptr), so that
				// it can call ParentGenerationHelper->ResetSaveState. It is safe to call twice, so call it now and it
				// will be a noop during SaveState. Call with EStateChangeReason::DoneForNow and
				// EPackageState::AssignedToWorker so that it does the minimum shutdown for leaving the SaveState
				// and lets us handle the work to return to idle ourselves in our required order.
				COTFS.ReleaseCookedPlatformData(*PackageData, EStateChangeReason::DoneForNow,
					EPackageState::AssignedToWorker);
				PackageData->SetParentGenerationHelper(nullptr, StateChangeReason);
			}
		}
		TRefCountPtr<FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
		if (GenerationHelper)
		{
			DeferGenerationHelperEvents.Emplace(GenerationHelper);
			if (bTerminalStateChange)
			{
				for (int32 PlatformIndex = 0; PlatformIndex < NumPlatforms; ++PlatformIndex)
				{
					ITargetPlatform* TargetPlatform = OrderedSessionPlatforms[PlatformIndex];
					FPackageRemoteResult::FPlatformResult& PlatformResult = Result.GetPlatforms()[PlatformIndex];
					if (!bResultIsSaveResult || PlatformResult.WasCommitted())
					{
						GenerationHelper->MarkPackageSavedRemotely(COTFS, *PackageData, TargetPlatform, GetWorkerId());
					}
				}
			}
			GenerationHelper.SafeRelease();
		}

		// For all packages, transition them to their next state
		PackageData->SetWorkerAssignment(FWorkerId::Invalid(), ESendFlags::QueueNone);
		if (bResultIsSaveResult)
		{
			ECookPhase CookPhase = COTFS.GetCookPhase();
			if (PackageData->GetPlatformsNeedingCommitNum(CookPhase) > 0)
			{
				TArray<const ITargetPlatform*> Remaining;
				PackageData->GetPlatformsNeedingCommit(Remaining, CookPhase);
				UE_LOGF(LogCook, Display,
					"Package %ls was completed by CookWorker %d for platforms { %ls }, but it still needs to commit platforms { %ls }."
					" Sending it back to the request state.",
					*PackageData->GetPackageName().ToString(), ProfileId,
					*PlatformListToString(GetCommittedPlatformListFromPlatformResults(OrderedSessionPlatforms, Result.GetPlatforms())),
					*PlatformListToString(Remaining));
				PackageData->SendToState(EPackageState::Request, ESendFlags::QueueAddAndRemove, EStateChangeReason::Discovered);
			}
			else
			{
				COTFS.PromoteToSaveComplete(*PackageData, ESendFlags::QueueAddAndRemove);
			}
		}
		else if (Result.GetSuppressCookReason() == ESuppressCookReason::RetractedByCookDirector)
		{
			UE_LOGF(LogCook, Error,
				"Package %ls was retracted by CookWorker %d, but it still sent a RecordResults message for the package which is supposed to be omitted for RetractedByCookDirector suppressions.",
				*PackageData->GetPackageName().ToString(), ProfileId);
			if (PackageData->GetWorkerAssignment() == GetWorkerId())
			{
				PackageData->SendToState(EPackageState::Request, ESendFlags::QueueAddAndRemove, EStateChangeReason::Retraction);
			}
		}
		else if (!bTerminalStateChange)
		{
			// Non-terminal SuppressCookReasons send it back to request via COTFS.DemoteToRequest. 
			// DemoteToRequest will also handle any request data changes indicated by the SuppressCookReason
			COTFS.DemoteToRequest(*PackageData, ESendFlags::QueueAddAndRemove, Result.GetSuppressCookReason());
		}
		else
		{
			// Terminal SuppressCookReasons send it to idle via DemoteToIdle.
			// DemoteToIdle will also handle any required logging.
			COTFS.DemoteToIdle(*PackageData, ESendFlags::QueueAddAndRemove, Result.GetSuppressCookReason());
		}

		// For generated packages, undefer events and process AllSavesCompleted if necessary.
		DeferGenerationHelperEvents.Reset();
	}

	Director.ResetFinalIdleHeartbeatFence();
	if (bRetiredAnyPackages)
	{
		++PackagesRetiredFenceMarker;
	}
}

void FCookWorkerServer::LogInvalidMessage(const TCHAR* MessageTypeName)
{
	UE_LOGF(LogCook, Error,
		"CookWorkerServer received invalidly formatted message for type %ls from CookWorker. Ignoring it.",
		MessageTypeName);
}

void FCookWorkerServer::QueueDiscoveredPackage(FDiscoveredPackageReplication&& DiscoveredPackage)
{
	check(TickState.TickThread == ECookDirectorThread::SchedulerThread);

	FPackageDatas& PackageDatas = *COTFS.PackageDatas;
	FInstigator& Instigator = DiscoveredPackage.Instigator;
	FDiscoveredPlatformSet& Platforms = DiscoveredPackage.Platforms;
	FPackageData& PackageData = PackageDatas.FindOrAddPackageData(DiscoveredPackage.PackageName,
		DiscoveredPackage.NormalizedFileName);

	TArray<const ITargetPlatform*, TInlineAllocator<ExpectedMaxNumPlatforms>> BufferPlatforms;
	TConstArrayView<const ITargetPlatform*> DiscoveredPlatforms;
	EReachability DiscoveredReachability =
		Instigator.Category == EInstigator::BuildDependency ? EReachability::Build : EReachability::Runtime;
	if (!COTFS.bSkipOnlyEditorOnly)
	{
		DiscoveredPlatforms = OrderedSessionAndSpecialPlatforms;
	}
	else
	{
		DiscoveredPlatforms = Platforms.GetPlatforms(COTFS, &Instigator, OrderedSessionAndSpecialPlatforms,
			DiscoveredReachability, BufferPlatforms);
	}

	if (Instigator.Category != EInstigator::ForceExplorableSaveTimeSoftDependency &&
		PackageData.HasReachablePlatforms(DiscoveredReachability, DiscoveredPlatforms))
	{
		// The CookWorker thought there were some new reachable platforms, but the Director already knows about
		// all of them; ignore the report
		return;
	}

	if (COTFS.bSkipOnlyEditorOnly &&
		Instigator.Category == EInstigator::Unsolicited &&
		Platforms.GetSource() == EDiscoveredPlatformSet::CopyFromInstigator &&
		PackageData.FindOrAddPlatformData(CookerLoadingPlatformKey).IsReachable(EReachability::Runtime))
	{
		// The CookWorker thought this package was new (previously unreachable even by editoronly references),
		// and it is not marked as a known used-in-game or editor-only issue, so it fell back to reporting it
		// as used-in-game-because-its-not-a-known-issue (see UCookOnTheFlyServer::ProcessUnsolicitedPackages's
		// use of PackageData->FindOrAddPlatformData(CookerLoadingPlatformKey).IsReachable(EReachability::Runtime)).
		// But we only do that fall back for unexpected packages not found by the search of editor-only AssetRegistry
		// dependencies. And this package was found by that search; the director has already marked it as reachable by
		// editoronly references. Correct the heuristic: ignore the unmarked load because the load is expected as an
		// editor-only reference.
		return;
	}

	if (!DiscoveredPackage.ParentGenerator.IsNone())
	{
		// Registration of the discovered Generated package with its generator needs to come after we early-exit
		// for already discovered packages, because when one generated package can refer to another from the same
		// generator, the message that a CookWorker has discovered the referred-to generated package can show up
		// on the director AFTER all save messages have already been processed and the GenerationHelper has shut
		// down and destroyed its information about the list of generated packages.
		PackageData.SetGenerated(DiscoveredPackage.ParentGenerator);
		PackageData.SetDoesGeneratedRequireGenerator(DiscoveredPackage.DoesGeneratedRequireGenerator);
		FPackageData* GeneratorPackageData = PackageDatas.FindPackageDataByPackageName(
			DiscoveredPackage.ParentGenerator);
		if (GeneratorPackageData)
		{
			TRefCountPtr<FGenerationHelper> GenerationHelper =
				GeneratorPackageData->CreateUninitializedGenerationHelper();
			GenerationHelper->NotifyStartQueueGeneratedPackages(COTFS, WorkerId);
			GenerationHelper->TrackGeneratedPackageListedRemotely(COTFS, PackageData, DiscoveredPackage.GeneratedPackageHash);
		}
	}

	if (PackageData.IsGenerated()
		&& (PackageData.DoesGeneratedRequireGenerator() >= ICookPackageSplitter::EGeneratedRequiresGenerator::Save
				|| COTFS.MPCookGeneratorSplit == EMPCookGeneratorSplit::AllOnSameWorker))
	{
		PackageData.SetWorkerAssignmentConstraint(GetWorkerId());
	}
	Director.ResetFinalIdleHeartbeatFence();
	Platforms.ConvertFromBitfield(OrderedSessionAndSpecialPlatforms);
	COTFS.QueueDiscoveredPackageOnDirector(PackageData, MoveTemp(Instigator), MoveTemp(Platforms),
		DiscoveredPackage.Urgency);
}

void FCookWorkerServer::HandleGeneratorMessage(FGeneratorEventMessage& GeneratorMessage)
{
	FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(GeneratorMessage.PackageName);
	if (!PackageData)
	{
		// This error should be impossible because GeneratorMessages are only sent in response to assignment from the server.
		UE_LOGF(LogCook, Error,
			"CookWorkerServer received unexpected GeneratorMessage for package %ls. The PackageData %ls does not exist on the CookDirector. " "\n\tCook of this generator package and its generated packages will be invalid.",
			*GeneratorMessage.PackageName.ToString(),
			(!PackageData ? TEXT("does not exist") : TEXT("is not a valid generator")));
		return;
	}

	TRefCountPtr<FGenerationHelper> GenerationHelper;
	GenerationHelper = PackageData->CreateUninitializedGenerationHelper();
	check(GenerationHelper);

	switch (GeneratorMessage.Event)
	{
	case EGeneratorEvent::QueuedGeneratedPackages:
		GenerationHelper->EndQueueGeneratedPackagesOnDirector(COTFS, GetWorkerId());
		break;
	default:
		// We do not handle the remaining GeneratorEvents on the server
		break;
	}
}

FCookWorkerServer::FTickState::FTickState()
{
	TickThread = ECookDirectorThread::Invalid;
	TickAction = ETickAction::Invalid;
}

FCookWorkerServer::FCommunicationScopeLock::FCommunicationScopeLock(FCookWorkerServer* InServer,
	ECookDirectorThread TickThread, ETickAction TickAction)
	: ScopeLock(&InServer->CommunicationLock)
	, Server(*InServer)
{
	check(TickThread != ECookDirectorThread::Invalid);
	check(TickAction != ETickAction::Invalid);
	check(Server.TickState.TickThread == ECookDirectorThread::Invalid);
	Server.TickState.TickThread = TickThread;
	Server.TickState.TickAction = TickAction;
}

FCookWorkerServer::FCommunicationScopeLock::~FCommunicationScopeLock()
{
	check(Server.TickState.TickThread != ECookDirectorThread::Invalid);
	Server.TickState.TickThread = ECookDirectorThread::Invalid;
	Server.TickState.TickAction = ETickAction::Invalid;
}

UE::CompactBinaryTCP::FMarshalledMessage MarshalToCompactBinaryTCP(const IMPCollectorMessage& Message)
{
	UE::CompactBinaryTCP::FMarshalledMessage Marshalled;
	Marshalled.MessageType = Message.GetMessageType();
	FCbWriter Writer;
	Writer.BeginObject();
	Message.Write(Writer);
	Writer.EndObject();
	Marshalled.Object = Writer.Save().AsObject();
	return Marshalled;
}

FAssignPackagesMessage::FAssignPackagesMessage(TArray<FAssignPackageData>&& InPackageDatas,
	TArray<FPackageDataExistenceInfo>&& InExistenceInfos)
	: PackageDatas(MoveTemp(InPackageDatas))
	, ExistenceInfos(MoveTemp(InExistenceInfos))
{
}

void FAssignPackagesMessage::Write(FCbWriter& Writer) const
{
	Writer.BeginArray("P");
	for (const FAssignPackageData& PackageData : PackageDatas)
	{
		WriteToCompactBinary(Writer, PackageData, OrderedSessionPlatforms);
	}
	Writer.EndArray();
	Writer.BeginArray("I");
	for (const FPackageDataExistenceInfo& ExistenceInfo : ExistenceInfos)
	{
		Writer << ExistenceInfo;
	}
	Writer.EndArray();
}

bool FAssignPackagesMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	PackageDatas.Reset();
	for (FCbFieldView PackageField : Object["P"])
	{
		FAssignPackageData& PackageData = PackageDatas.Emplace_GetRef();
		if (!LoadFromCompactBinary(PackageField, PackageData, OrderedSessionPlatforms))
		{
			PackageDatas.Pop();
			bOk = false;
		}
	}
	ExistenceInfos.Reset();
	for (FCbFieldView PackageField : Object["I"])
	{
		FPackageDataExistenceInfo& ExistenceInfo = ExistenceInfos.Emplace_GetRef();
		if (!LoadFromCompactBinary(PackageField, ExistenceInfo))
		{
			ExistenceInfos.Pop();
			bOk = false;
		}
	}
	return bOk;
}

FGuid FAssignPackagesMessage::MessageType(TEXT("B7B1542B73254B679319D73F753DB6F8"));

void FAssignPackageData::Write(FCbWriter& Writer,
	TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms) const
{
	Writer.BeginArray();
	Writer << ConstructData;
	Writer << ParentGenerator;
	Writer << Instigator;
	Writer << static_cast<uint8>(Urgency);
	static_assert(sizeof(EUrgency) <= sizeof(uint8), "We are storing it in a uint8");
	Writer << static_cast<uint8>(Reachability);
	static_assert(sizeof(EReachability) <= sizeof(uint8), "We are storing it in a uint8");
	Writer << bSkipSaveExistingGenerator;
	Writer << bQueuedGeneratedPackagesFencePassed;
	WriteToCompactBinary(Writer, NeedCommitPlatforms, OrderedSessionPlatforms);
	{
		Writer.BeginArray();
		for (const TPair<uint8, TMap<FName, FAssetPackageData>>& PlatformPair
			: GeneratorPerPlatformPreviousGeneratedPackages)
		{
			Writer.BeginArray();
			Writer << PlatformPair.Key;
			Writer.BeginArray();
			for (const TPair<FName, FAssetPackageData>& PackagePair : PlatformPair.Value)
			{
				Writer.BeginArray();
				Writer << PackagePair.Key;
				PackagePair.Value.NetworkWrite(Writer);
				Writer.EndArray();
			}
			Writer.EndArray();
			Writer.EndArray();
		}
		Writer.EndArray();
	}
	static_assert(sizeof(ICookPackageSplitter::EGeneratedRequiresGenerator) <= sizeof(uint8), "We are storing it in a uint8");
	Writer << static_cast<uint8>(DoesGeneratedRequireGenerator);
	Writer << PerPackageCollectorMessages;
	Writer.EndArray();
}

bool FAssignPackageData::TryRead(FCbFieldView Field, TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms)
{
	FCbFieldViewIterator It = Field.CreateViewIterator();
	bool bOk = true;
	bOk = LoadFromCompactBinary(*It++, ConstructData) & bOk;
	bOk = LoadFromCompactBinary(*It++, ParentGenerator) & bOk;
	bOk = LoadFromCompactBinary(*It++, Instigator) & bOk;
	uint8 UrgencyInt = It->AsUInt8();
	if (!(It++)->HasError() && UrgencyInt < static_cast<uint8>(EUrgency::Count))
	{
		Urgency = static_cast<EUrgency>(UrgencyInt);
	}
	else
	{
		bOk = false;
	}
	uint8 ReachabilityInt = It->AsUInt8();
	if (!(It++)->HasError() && ReachabilityInt < (static_cast<uint8>(EReachability::MaxBit) << 1))
	{
		Reachability = static_cast<EReachability>(ReachabilityInt);
	}
	else
	{
		bOk = false;
	}
	bOk = LoadFromCompactBinary(*It++, bSkipSaveExistingGenerator) & bOk;
	bOk = LoadFromCompactBinary(*It++, bQueuedGeneratedPackagesFencePassed) & bOk;
	bOk = LoadFromCompactBinary(*It++, NeedCommitPlatforms, OrderedSessionPlatforms) & bOk;
	{
		FCbFieldView PlatformArrayFieldView = *It++;
		bool bGeneratorPreviousGeneratedPackagesOk = false;
		const uint64 PlatformLength = PlatformArrayFieldView.AsArrayView().Num();
		if (PlatformLength <= MAX_int32)
		{
			GeneratorPerPlatformPreviousGeneratedPackages.Empty((int32)PlatformLength);
			bGeneratorPreviousGeneratedPackagesOk = !PlatformArrayFieldView.HasError();
			for (const FCbFieldView& PlatformIt : PlatformArrayFieldView)
			{
				bool bPlatformPairOk = false;
				FCbFieldViewIterator PlatformPairIt = PlatformIt.CreateViewIterator();
				uint8 PlatformIndex;
				TMap<FName, FAssetPackageData> PackagesMap;
				if (LoadFromCompactBinary(*PlatformPairIt++, PlatformIndex))
				{
					FCbFieldView PackagesArrayFieldView = *PlatformPairIt++;
					const uint64 PackagesLength = PackagesArrayFieldView.AsArrayView().Num();
					if (PackagesLength <= MAX_int32)
					{
						PackagesMap.Empty((int32)PackagesLength);
						bPlatformPairOk = !PackagesArrayFieldView.HasError();
						for (const FCbFieldView& PackagesElementField : PackagesArrayFieldView)
						{
							FCbFieldViewIterator PairIt = PackagesElementField.CreateViewIterator();
							bool bElementOk = false;
							FName Key;
							FAssetPackageData Value;
							if (LoadFromCompactBinary(*PairIt++, Key))
							{
								if (Value.TryNetworkRead(*PairIt++))
								{
									PackagesMap.Add(Key, MoveTemp(Value));
									bElementOk = true;
								}
							}
							bPlatformPairOk &= bElementOk;
						}
					}
				}
				if (bPlatformPairOk)
				{
					GeneratorPerPlatformPreviousGeneratedPackages.Add(PlatformIndex, MoveTemp(PackagesMap));
				}
				bGeneratorPreviousGeneratedPackagesOk &= bPlatformPairOk;
			}
		}
		else
		{
			GeneratorPerPlatformPreviousGeneratedPackages.Empty();
		}
		bOk &= bGeneratorPreviousGeneratedPackagesOk;
	}
	uint8 DoesGeneratedRequireGeneratorInt = It->AsUInt8();
	if (!(It++)->HasError() && DoesGeneratedRequireGeneratorInt
		< static_cast<uint8>(ICookPackageSplitter::EGeneratedRequiresGenerator::Count))
	{
		DoesGeneratedRequireGenerator =
			static_cast<ICookPackageSplitter::EGeneratedRequiresGenerator>(DoesGeneratedRequireGeneratorInt);
	}
	else
	{
		bOk = false;
	}
	bOk = LoadFromCompactBinary(*It++, PerPackageCollectorMessages) & bOk;
	return bOk;
}

void FPackageDataExistenceInfo::Write(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << ConstructData;
	Writer << ParentGenerator;
	Writer.EndArray();
}

bool FPackageDataExistenceInfo::TryRead(FCbFieldView Field)
{
	FCbFieldViewIterator It = Field.CreateViewIterator();
	bool bOk = true;
	bOk = LoadFromCompactBinary(*It++, ConstructData) & bOk;
	bOk = LoadFromCompactBinary(*It++, ParentGenerator) & bOk;
	return bOk;
}

FCbWriter& operator<<(FCbWriter& Writer, const FInstigator& Instigator)
{
	Writer.BeginObject();
	Writer << "C" << static_cast<uint8>(Instigator.Category);
	Writer << "R" << Instigator.Referencer;
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FInstigator& Instigator)
{
	uint8 CategoryInt;
	bool bOk = true;
	if (LoadFromCompactBinary(Field["C"], CategoryInt) &&
		CategoryInt < static_cast<uint8>(EInstigator::Count))
	{
		Instigator.Category = static_cast<EInstigator>(CategoryInt);
	}
	else
	{
		Instigator.Category = EInstigator::InvalidCategory;
		bOk = false;
	}
	bOk = LoadFromCompactBinary(Field["R"], Instigator.Referencer) & bOk;
	return bOk;
}

FAbortPackagesMessage::FAbortPackagesMessage(TArray<FName>&& InPackageNames)
	: PackageNames(MoveTemp(InPackageNames))
{
}

void FAbortPackagesMessage::Write(FCbWriter& Writer) const
{
	Writer << "PackageNames" <<  PackageNames;
}

bool FAbortPackagesMessage::TryRead(FCbObjectView Object)
{
	return LoadFromCompactBinary(Object["PackageNames"], PackageNames);
}

FGuid FAbortPackagesMessage::MessageType(TEXT("D769F1BFF2F34978868D70E3CAEE94E7"));

FAbortWorkerMessage::FAbortWorkerMessage(EType InType)
	: Type(InType)
{
}

void FAbortWorkerMessage::Write(FCbWriter& Writer) const
{
	Writer << "Type" << (uint8)Type;
}

bool FAbortWorkerMessage::TryRead(FCbObjectView Object)
{
	Type = static_cast<EType>(Object["Type"].AsUInt8((uint8)EType::Abort));
	return true;
}

FGuid FAbortWorkerMessage::MessageType(TEXT("83FD99DFE8DB4A9A8E71684C121BE6F3"));

void FInitialConfigMessage::ReadFromLocal(const UCookOnTheFlyServer& COTFS,
	const TConstArrayView<const ITargetPlatform*>& InOrderedSessionPlatforms, const FCookByTheBookOptions& InCookByTheBookOptions,
	const FCookOnTheFlyOptions& InCookOnTheFlyOptions, const FBeginCookContextForWorker& InBeginContext)
{
	InitialSettings.CopyFromLocal(COTFS);
	BeginCookSettings.CopyFromLocal(COTFS);
	BeginCookContext = InBeginContext;
	OrderedSessionPlatforms.Reset(InOrderedSessionPlatforms.Num());
	for (const ITargetPlatform* Platform : InOrderedSessionPlatforms)
	{
		OrderedSessionPlatforms.Add(const_cast<ITargetPlatform*>(Platform));
	}
	DirectorCookMode = COTFS.GetCookMode();
	CookInitializationFlags = COTFS.GetCookFlags();
	CookByTheBookOptions = InCookByTheBookOptions;
	CookOnTheFlyOptions = InCookOnTheFlyOptions;
	bZenStore = COTFS.IsUsingZenStore();
}

void FInitialConfigMessage::Write(FCbWriter& Writer) const
{
	int32 LocalCookMode = static_cast<int32>(DirectorCookMode);
	Writer << "DirectorCookMode" << LocalCookMode;
	int32 LocalCookFlags = static_cast<int32>(CookInitializationFlags);
	Writer << "CookInitializationFlags" << LocalCookFlags;
	Writer << "ZenStore" << bZenStore;

	Writer.BeginArray("TargetPlatforms");
	for (const ITargetPlatform* TargetPlatform : OrderedSessionPlatforms)
	{
		Writer << TargetPlatform->PlatformName();
	}
	Writer.EndArray();
	Writer << "InitialSettings" << InitialSettings;
	Writer << "BeginCookSettings" << BeginCookSettings;
	Writer << "BeginCookContext" << BeginCookContext;
	Writer << "CookByTheBookOptions" << CookByTheBookOptions;
	Writer << "CookOnTheFlyOptions" << CookOnTheFlyOptions;
	Writer << "MPCollectorMessages" << MPCollectorMessages;
}

bool FInitialConfigMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	int32 LocalCookMode;
	bOk = LoadFromCompactBinary(Object["DirectorCookMode"], LocalCookMode) & bOk;
	DirectorCookMode = static_cast<ECookMode::Type>(LocalCookMode);
	int32 LocalCookFlags;
	bOk = LoadFromCompactBinary(Object["CookInitializationFlags"], LocalCookFlags) & bOk;
	CookInitializationFlags = static_cast<ECookInitializationFlags>(LocalCookFlags);
	bOk = LoadFromCompactBinary(Object["ZenStore"], bZenStore) & bOk;

	ITargetPlatformManagerModule& TPM(GetTargetPlatformManagerRef());
	FCbFieldView TargetPlatformsField = Object["TargetPlatforms"];
	{
		bOk = TargetPlatformsField.IsArray() & bOk;
		OrderedSessionPlatforms.Reset(TargetPlatformsField.AsArrayView().Num());
		for (FCbFieldView ElementField : TargetPlatformsField)
		{
			TStringBuilder<128> KeyName;
			if (LoadFromCompactBinary(ElementField, KeyName))
			{
				ITargetPlatform* TargetPlatform = TPM.FindTargetPlatform(KeyName.ToView());
				if (TargetPlatform)
				{
					OrderedSessionPlatforms.Add(TargetPlatform);
				}
				else
				{
					UE_LOGF(LogCook, Error, "Could not find TargetPlatform \"%.*ls\" received from CookDirector.",
						KeyName.Len(), KeyName.GetData());
					bOk = false;
				}

			}
			else
			{
				bOk = false;
			}
		}
	}

	bOk = LoadFromCompactBinary(Object["InitialSettings"], InitialSettings) & bOk;
	bOk = LoadFromCompactBinary(Object["BeginCookSettings"], BeginCookSettings) & bOk;
	bOk = LoadFromCompactBinary(Object["BeginCookContext"], BeginCookContext) & bOk;
	bOk = LoadFromCompactBinary(Object["CookByTheBookOptions"], CookByTheBookOptions) & bOk;
	bOk = LoadFromCompactBinary(Object["CookOnTheFlyOptions"], CookOnTheFlyOptions) & bOk;
	bOk = LoadFromCompactBinary(Object["MPCollectorMessages"], MPCollectorMessages) & bOk;

	return bOk;
}

FGuid FInitialConfigMessage::MessageType(TEXT("340CDCB927304CEB9C0A66B5F707FC2B"));

void FDiscoveredPackageReplication::Write(FCbWriter& Writer,
	TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms) const
{
	Writer.BeginArray();
	Writer << PackageName;
	Writer << NormalizedFileName;
	Writer << ParentGenerator;
	Writer << static_cast<uint8>(Instigator.Category);
	Writer << Instigator.Referencer;
	Writer << static_cast<uint8>(DoesGeneratedRequireGenerator);
	static_assert(sizeof(ICookPackageSplitter::EGeneratedRequiresGenerator) <= sizeof(uint8), "We are storing it in a uint8");
	Writer << static_cast<uint8>(Urgency);
	static_assert(sizeof(EUrgency) <= sizeof(uint8), "We are storing it in a uint8");
	bool bGeneratedPackageHash = !GeneratedPackageHash.IsZero();
	Writer << bGeneratedPackageHash;
	if (bGeneratedPackageHash)
	{
		Writer << GeneratedPackageHash;
	}
	WriteToCompactBinary(Writer, Platforms, OrderedSessionAndSpecialPlatforms);
	Writer.EndArray();
}

bool FDiscoveredPackageReplication::TryRead(FCbFieldView Field,
	TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms)
{
	FCbArrayView FieldList = Field.AsArrayView();
	if (Field.HasError())
	{
		*this = FDiscoveredPackageReplication();
		return false;
	}
	FCbFieldViewIterator Iter = FieldList.CreateViewIterator();

	bool bOk = LoadFromCompactBinary(Iter++, PackageName);
	bOk = LoadFromCompactBinary(Iter++, NormalizedFileName) & bOk;
	bOk = LoadFromCompactBinary(Iter++, ParentGenerator) & bOk;
	uint8 CategoryInt;
	if (LoadFromCompactBinary(Iter++, CategoryInt) &&
		CategoryInt < static_cast<uint8>(EInstigator::Count))
	{
		Instigator.Category = static_cast<EInstigator>(CategoryInt);
	}
	else
	{
		bOk = false;
	}
	bOk = LoadFromCompactBinary(Iter++, Instigator.Referencer) & bOk;
	uint8 DoesGeneratedRequireGeneratorInt = Iter->AsUInt8();
	if (!(Iter++)->HasError() && DoesGeneratedRequireGeneratorInt
		< static_cast<uint8>(ICookPackageSplitter::EGeneratedRequiresGenerator::Count))
	{
		DoesGeneratedRequireGenerator = static_cast<ICookPackageSplitter::EGeneratedRequiresGenerator>(
			DoesGeneratedRequireGeneratorInt);
	}
	else
	{
		bOk = false;
	}
	uint8 UrgencyInt = Iter->AsUInt8();
	if (!(Iter++)->HasError() && UrgencyInt < static_cast<uint8>(EUrgency::Count))
	{
		Urgency = static_cast<EUrgency>(UrgencyInt);
	}
	else
	{
		bOk = false;
	}
	bool bGeneratedPackageHash = false;
	bOk = LoadFromCompactBinary(Iter++, bGeneratedPackageHash) & bOk;
	if (bGeneratedPackageHash)
	{
		bOk = LoadFromCompactBinary(Iter++, GeneratedPackageHash) & bOk;
	}
	else
	{
		GeneratedPackageHash = FIoHash::Zero;
	}
	bOk = LoadFromCompactBinary(Iter++, Platforms, OrderedSessionAndSpecialPlatforms) & bOk;
	if (!bOk)
	{
		*this = FDiscoveredPackageReplication();
	}
	return bOk;
}

void FDiscoveredPackagesMessage::Write(FCbWriter& Writer) const
{
	Writer.BeginArray("Packages");
	for (const FDiscoveredPackageReplication& Package : Packages)
	{
		WriteToCompactBinary(Writer, Package, OrderedSessionAndSpecialPlatforms);
	}
	Writer.EndArray();
}

bool FDiscoveredPackagesMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	Packages.Reset();
	for (FCbFieldView PackageField : Object["Packages"])
	{
		FDiscoveredPackageReplication& Package = Packages.Emplace_GetRef();
		if (!LoadFromCompactBinary(PackageField, Package, OrderedSessionAndSpecialPlatforms))
		{
			Packages.Pop();
			bOk = false;
		}
	}
	return bOk;
}

FGuid FDiscoveredPackagesMessage::MessageType(TEXT("C9F5BC5C11484B06B346B411F1ED3090"));

FGeneratorEventMessage::FGeneratorEventMessage(EGeneratorEvent InEvent, FName InPackageName)
	: PackageName(InPackageName)
	, Event(InEvent)
{
}

void FGeneratorEventMessage::Write(FCbWriter& Writer) const
{
	Writer << "E" << static_cast<uint8>(Event);
	Writer << "P" << PackageName;
}

bool FGeneratorEventMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	FCbFieldView EventField = Object["E"];
	uint8 EventInt = EventField.AsUInt8();
	if (!EventField.HasError() && EventInt < static_cast<uint8>(EGeneratorEvent::Num))
	{
		Event = static_cast<EGeneratorEvent>(EventInt);
	}
	else
	{
		Event = EGeneratorEvent::Invalid;
		bOk = false;
	}
	bOk = LoadFromCompactBinary(Object["P"], PackageName) & bOk;
	return bOk;
}

FGuid FGeneratorEventMessage::MessageType(TEXT("B6EE94CA70EC4F40B0D2214EDC11ED03"));

FGuid FLogMessagesMessageHandler::MessageType(TEXT("DB024D28203D4FBAAAF6AAD7080CF277"));

FLogMessagesMessageHandler::FLogMessagesMessageHandler(ILogHandler& InCOTFSLogHandler)
	: COTFSLogHandler(InCOTFSLogHandler)
{
}

void FLogMessagesMessageHandler::ClientReportLogMessage(const FReplicatedLogData& LogData)
{
	FScopeLock QueueScopeLock(&QueueLock);
	QueuedLogs.Add(LogData);
}

void FLogMessagesMessageHandler::ClientTick(FMPCollectorClientTickContext& Context)
{
	{
		FScopeLock QueueScopeLock(&QueueLock);
		Swap(QueuedLogs, QueuedLogsBackBuffer);
	}
	if (!QueuedLogsBackBuffer.IsEmpty())
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << "Messages" << QueuedLogsBackBuffer;
		Writer.EndObject();
		Context.AddMessage(Writer.Save().AsObject());
		QueuedLogsBackBuffer.Reset();
	}
}

void FLogMessagesMessageHandler::ServerReceiveMessage(FMPCollectorServerMessageContext& Context,
	FCbObjectView InMessage)
{
	TArray<FReplicatedLogData> Messages;

	if (!LoadFromCompactBinary(InMessage["Messages"], Messages))
	{
		UE_LOGF(LogCook, Error, "FLogMessagesMessageHandler received corrupted message from CookWorker");
		return;
	}

	int32 CookWorkerProfileId = Context.GetProfileId();
	for (FReplicatedLogData& LogData : Messages)
	{
		COTFSLogHandler.ReplayLogFromCookWorker(MoveTemp(LogData), CookWorkerProfileId);
	}
	COTFSLogHandler.ConditionalPruneReplay();
}

FGuid FHeartbeatMessage::MessageType(TEXT("C08FFAF07BF34DD3A2FFB8A287CDDE83"));

FHeartbeatMessage::FHeartbeatMessage(int32 InHeartbeatNumber)
	: HeartbeatNumber(InHeartbeatNumber)
{
}

void FHeartbeatMessage::Write(FCbWriter& Writer) const
{
	Writer << "H" << HeartbeatNumber;
}

bool FHeartbeatMessage::TryRead(FCbObjectView Object)
{
	return LoadFromCompactBinary(Object["H"], HeartbeatNumber);
}

FPackageWriterMPCollector::FPackageWriterMPCollector(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{

}

void FPackageWriterMPCollector::ClientTickPackage(FMPCollectorClientTickPackageContext& Context)
{
	for (const FMPCollectorClientTickPackageContext::FPlatformData& PlatformData : Context.GetPlatformDatas())
	{
		if (PlatformData.CookResults == ECookResult::Invalid)
		{
			continue;
		}
		ICookedPackageWriter& PackageWriter = COTFS.FindOrCreatePackageWriter(PlatformData.TargetPlatform);
		TFuture<FCbObject> ObjectFuture = PackageWriter.WriteMPCookMessageForPackage(Context.GetPackageName());
		Context.AddAsyncPlatformMessage(PlatformData.TargetPlatform, MoveTemp(ObjectFuture));
	}
}

void FPackageWriterMPCollector::ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	FName PackageName = Context.GetPackageName();
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	check(PackageName.IsValid() && TargetPlatform);

	ICookedPackageWriter& PackageWriter = COTFS.FindOrCreatePackageWriter(TargetPlatform);
	if (!PackageWriter.TryReadMPCookMessageForPackage(PackageName, Message))
	{
		UE_LOGF(LogCook, Error,
			"CookWorkerServer received invalidly formatted PackageWriter message from CookWorker %d. Ignoring it.",
			Context.GetProfileId());
	}
}

FGuid FPackageWriterMPCollector::MessageType(TEXT("D2B1CE3FD26644AF9EC28FBADB1BD331"));

void FFileTransferMessage::Write(FCbWriter& Writer) const
{
	Writer << "S" << TempFileName;
	Writer << "T" << TargetFileName;
}

bool FFileTransferMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	bOk = LoadFromCompactBinary(Object["S"], TempFileName) & bOk;
	bOk = LoadFromCompactBinary(Object["T"], TargetFileName) & bOk;
	return bOk;
}

FGuid FFileTransferMessage::MessageType(0x49205105, 0x9D62427F, 0xAA3F2F7C, 0x118152E0);

} // namespace UE::Cook
