// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "Containers/Set.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookSockets.h"
#include "Cooker/CookTypes.h"
#include "Cooker/MPCollector.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CookPackageSplitter.h"
#include "HAL/CriticalSection.h"
#include "IO/IoHash.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Guid.h"
#include "Templates/RefCounting.h"

class FSocket;
class ITargetPlatform;
struct FProcHandle;
namespace UE { class FLogTemplate; }
namespace UE::CompactBinaryTCP { struct FMarshalledMessage; }
namespace UE::Cook { class FCookDirector; }
namespace UE::Cook { class ILogHandler; }
namespace UE::Cook { struct FDiscoveredPackageReplication; }
namespace UE::Cook { enum class ECookDirectorThread : uint8; };
namespace UE::Cook { struct FGeneratorEventMessage; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FPackageResultsMessage; }
namespace UE::Cook { struct FReplicatedLogData; }
namespace UE::Cook { struct FWorkerConnectMessage; }

namespace UE::Cook
{

enum class ENotifyRemote
{
	NotifyRemote,
	LocalOnly,
};

/**
 * Extra non-threadsafe data about an FPackageData that we save from the scheduler thread for reading on the
 * communication thread when we send the assignment of the PackageData out to the CookWorker.
 */
struct FAssignPackageExtraData
{
	TMap<const ITargetPlatform*, TMap<FName, FAssetPackageData>> GeneratorPerPlatformPreviousGeneratedPackages;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> PerPackageCollectorMessages;
	bool bSkipSaveExistingGenerator = false;
	bool bQueuedGeneratedPackagesFencePassed = false;
};

/** Class in a Director process that communicates over a Socket with FCookWorkerClient in a CookWorker process. */
class FCookWorkerServer : public FRefCountedObject
{
public:
	FCookWorkerServer(FCookDirector& InDirector, int32 InProfileId, FWorkerId InWorkerId);
	~FCookWorkerServer();

	int32 GetProfileId() const { return ProfileId; }
	FWorkerId GetWorkerId() const { return WorkerId; }

	/** Add the given assignments for the CookWorker. They will be sent during Tick */
	void AppendAssignments(TArrayView<FPackageData*> Assignments,
		TMap<FPackageData*, FAssignPackageExtraData>&& ExtraDatas, TArrayView<FPackageData*> InfoPackages,
		ECookDirectorThread TickThread);
	/** Remove assignment of the package from local state and optionally from the connected Client. */
	void AbortAssignment(FPackageData& PackageData, ECookDirectorThread TickThread,
		int32 CurrentHeartbeat, ENotifyRemote NotifyRemote = ENotifyRemote::NotifyRemote);
	void AbortAssignments(TConstArrayView<FPackageData*> PackageData, ECookDirectorThread TickThread,
		int32 CurrentHeartbeat, ENotifyRemote NotifyRemote = ENotifyRemote::NotifyRemote);

	/**
	 * Remove assignment of all assigned packages from local state and from the connected Client.
	 * Report all packages that were removed.
	 */
	void AbortAllAssignments(TSet<FPackageData*>& OutPendingPackages, ECookDirectorThread TickThread,
		int32 CurrentHeartbeat);
	/** AbortAllAssignments and tell the Client to gracefully terminate. Report all packages that were unassigned. */
	void AbortWorker(TSet<FPackageData*>& OutPendingPackages, ECookDirectorThread TickThread, int32 CurrentHeartbeat);
	/** Take over the Socket for a CookWorker that has just connected. */
	bool TryHandleConnectMessage(FWorkerConnectMessage& Message, FSocket* InSocket,
		TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& OtherPacketMessages, ECookDirectorThread TickThread);

	/** Send the message immediately to the Socket. If cannot complete immediately, it will be finished during Tick. */
	void SendMessage(const IMPCollectorMessage& Message, ECookDirectorThread TickThread);
	void SendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message, ECookDirectorThread TickThread);
	/**
	 * Queue the given message for sending; it will be sent during tick. Only needed for messages that have to be sent
	 * after queued package assignments or other appended messages.
	 */
	void AppendMessage(const IMPCollectorMessage& Message, ECookDirectorThread TickThread);
	void AppendMessage(UE::CompactBinaryTCP::FMarshalledMessage&& Message, ECookDirectorThread TickThread);

	/** Periodic Tick function to send and receive messages to the Client. */
	void TickCommunication(ECookDirectorThread TickThread);
	/** Called when the COTFS wants to send a heartbeat message to the Client. */
	void SignalHeartbeat(ECookDirectorThread TickThread, int32 HeartbeatNumber);
	/** Called when the Server detects all packages are complete. Tell the CookWorker to flush messages and exit. */
	void SignalCookComplete(ECookDirectorThread TickThread);
	/**
	 * Execute the respond for all messages that have been received and that can be executed from the given thread. 
	 * This is the entry point for the scheduler thread to pick up messages that the CommunicationThread has queued.
	 */
	void HandleReceiveMessages(ECookDirectorThread TickThread);

	/** Is this done connecting and not yet shutting down? */
	bool IsConnected() const;
	/** Is this either shutting down or completed shutdown of its remote Client? */
	bool IsShuttingDown() const;
	/** Is this executing graceful shutdown and is waiting for the CookWorker to transfer remaining messages? */
	bool IsFlushingBeforeShutdown() const;
	/** Is this not yet or no longer connected to a remote Client? */
	bool IsShutdownComplete() const;
	/** How many package assignments is the remote CookWorker supposed to save but hasn't yet? */
	int32 NumAssignments() const;
	/** Does this Server have any ReceivedMessages that need to be processed by the Scheduler thread? */
	bool HasMessages() const;

	/** Get the LastReceivedHeartbeatNumber. */
	int32 GetLastReceivedHeartbeatNumber() const;
	/**
	 * Set the LastReceivedHeartbeatNumber. Assumes lock is already entered; can only be called from with a
	 * HandleReceivedMessages callback.
	 */
	void SetLastReceivedHeartbeatNumberInLock(int32 InHeartbeatNumber);

	int32 GetPackagesAssignedFenceMarker() const;
	int32 GetPackagesRetiredFenceMarker() const;

private:
	enum class EConnectStatus
	{
		Uninitialized,
		WaitForConnect,
		Connected,
		PumpingCookComplete,
		WaitForDisconnect,
		LostConnection,
	};
	enum class EWorkerDetachType
	{
		Dismissed,
		StillRunning,
		ForceTerminated,
		Crashed,
	};
	enum class ETickAction
	{
		Tick,
		Queue,
		Invalid,
	};
	/**
	 * Stores from which thread the public function on *this was called, and whether that public function is a pumping
	 * function that should send/receive network messages or merely an accessor function that should send-to-queue or
	 * read-from-queued network messages.
	 */
	struct FTickState
	{
		FTickState();
		ECookDirectorThread TickThread;
		ETickAction TickAction;
	};
	/** An RAII structure that enters the lock and sets the TickState information required by many functions. */
	struct FCommunicationScopeLock
	{
		FScopeLock ScopeLock;
		FCookWorkerServer& Server;
		FCommunicationScopeLock(FCookWorkerServer* InServer, ECookDirectorThread TickThread, ETickAction TickAction);
		~FCommunicationScopeLock();
	};

private:
	/** Helper for PumpConnect, launch the remote Client process. */
	void LaunchProcess();
	/** Helper for PumpConnect, wait for connect message from Client, set state to LostConnection if we timeout. */
	void TickWaitForConnect();
	/** Helper for PumpConnect, wait for disconnect message from Client, set state to LostConnection if we timeout. */
	void TickWaitForDisconnect();
	/** Helper for Tick, pump send messages to a connected Client. */
	void PumpSendMessages();
	/** Helper for PumpSendMessages; send messages that were queued up for sending during tick. */
	void SendPendingMessages();
	/** Helper for SendPendingMessages; send a message for any PackagesToAssign we have. */
	void SendPendingPackages();
	/** Helper for Tick, pump receive messages from a connected Client. */
	void PumpReceiveMessages();
	/** The main implementation of AbortAllAssignments, only callable from inside the lock. */
	void AbortAllAssignmentsInLock(TSet<FPackageData*>& OutPendingPackages, int32 CurrentHeartbeat);
	/** Send the message immediately to the Socket. If cannot complete immediately, it will be finished during Tick. */
	void SendMessageInLock(const IMPCollectorMessage& Message);
	void SendMessageInLock(UE::CompactBinaryTCP::FMarshalledMessage&& Message);
	/** Send this into the given state. Update any state-dependent variables. */
	void SendToState(EConnectStatus TargetStatus);
	/** Close the connection and connection resources to the remote process. Does not kill the process. */
	void DetachFromRemoteProcess(EWorkerDetachType DetachType);
	/** Report from the log of the crashed CookWorker. */
	void SendCrashDiagnostics();
	/** Kill the Client process (non-graceful termination), and close the connection resources. */
	void ShutdownRemoteProcess();
	/** The main implementation of HandleReceiveMessages, only callable from inside the lock. */
	void HandleReceiveMessagesInternal();
	/** Helper for PumpReceiveMessages: dispatch the messages received from the socket. */
	void HandleReceivedPackagePlatformMessages(FPackageData& PackageData, const ITargetPlatform* TargetPlatform,
		TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& Messages);
	/** Add results from the client to the local CookOnTheFlyServer. */
	void RecordResults(FPackageResultsMessage& Message);
	void LogInvalidMessage(const TCHAR* MessageTypeName);
	void QueueDiscoveredPackage(FDiscoveredPackageReplication&& DiscoveredPackage);
	void HandleGeneratorMessage(FGeneratorEventMessage& GeneratorMessage);

	// Lock guarding access to all data on *this
	mutable FCriticalSection CommunicationLock;

	// All data can only be read or written while the CommunicationLock is entered.
	TArray<FPackageData*> PackagesToAssign;
	TSet<FPackageData*> PackagesToAssignInfoPackages;
	TMap<FPackageData*, FAssignPackageExtraData> PackagesToAssignExtraDatas;
	TSet<FPackageData*> PendingPackages;
	TArray<ITargetPlatform*> OrderedSessionPlatforms;
	TArray<ITargetPlatform*> OrderedSessionAndSpecialPlatforms;
	UE::CompactBinaryTCP::FSendBuffer SendBuffer;
	UE::CompactBinaryTCP::FReceiveBuffer ReceiveBuffer;
	TRingBuffer<UE::CompactBinaryTCP::FMarshalledMessage> ReceiveMessages;
	TRingBuffer<UE::CompactBinaryTCP::FMarshalledMessage> QueuedMessagesToSendAfterPackagesToAssign;
	FString CrashDiagnosticsError;
	FCookDirector& Director;
	UCookOnTheFlyServer& COTFS;
	FSocket* Socket = nullptr;
	FProcHandle CookWorkerHandle;
	FTickState TickState;
	uint32 CookWorkerProcessId = 0;
	int32 ProfileId = 0;
	int32 LastReceivedHeartbeatNumber = -1;
	int32 PackagesAssignedFenceMarker = 0;
	int32 PackagesRetiredFenceMarker = 0;
	int32 LastAbortHeartbeatNumber = -1;
	double ConnectStartTimeSeconds = 0.;
	double ConnectTestStartTimeSeconds = 0.;
	FWorkerId WorkerId = FWorkerId::Invalid();
	EConnectStatus ConnectStatus = EConnectStatus::Uninitialized;
	bool bTerminateImmediately = false;
	bool bNeedCrashDiagnostics = false;
};

UE::CompactBinaryTCP::FMarshalledMessage MarshalToCompactBinaryTCP(const IMPCollectorMessage& Message);

FCbWriter& operator<<(FCbWriter& Writer, const FInstigator& Instigator);
bool LoadFromCompactBinary(FCbFieldView Field, FInstigator& Instigator);

/** Information about a PackageData assigned to a CookWorker from the director. */
struct FAssignPackageData
{
	FConstructPackageData ConstructData;
	FName ParentGenerator;
	FInstigator Instigator;
	EUrgency Urgency;
	EReachability Reachability;
	FDiscoveredPlatformSet NeedCommitPlatforms;
	TMap<uint8, TMap<FName, FAssetPackageData>> GeneratorPerPlatformPreviousGeneratedPackages;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> PerPackageCollectorMessages;
	ICookPackageSplitter::EGeneratedRequiresGenerator DoesGeneratedRequireGenerator
		= ICookPackageSplitter::EGeneratedRequiresGenerator::None;
	bool bSkipSaveExistingGenerator = false;
	bool bQueuedGeneratedPackagesFencePassed = false;

private:
	void Write(FCbWriter& Writer, TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms) const;
	bool TryRead(FCbFieldView Field, TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms);

	friend FCbWriter& WriteToCompactBinary(FCbWriter& Writer, const FAssignPackageData& AssignData,
		TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms)
	{
		AssignData.Write(Writer, OrderedSessionPlatforms);
		return Writer;
	}
	friend bool LoadFromCompactBinary(FCbFieldView Field, FAssignPackageData& AssignData,
		TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms)
	{
		return AssignData.TryRead(Field, OrderedSessionPlatforms);
	}
};

/**
 * Information about a PackageData sent from the director that needs to be queryable on a CookWorker to support other
 * assigned packages but is not being assigned to the CookWorker.
 */
struct FPackageDataExistenceInfo
{
	FConstructPackageData ConstructData;
	FName ParentGenerator;
private:
	void Write(FCbWriter& Writer) const;
	bool TryRead(FCbFieldView Field);

	friend FCbWriter& operator<<(FCbWriter& Writer, const FPackageDataExistenceInfo& ExistenceInfo)
	{
		ExistenceInfo.Write(Writer);
		return Writer;
	}
	friend bool LoadFromCompactBinary(FCbFieldView Field, FPackageDataExistenceInfo& ExistenceInfo)
	{
		return ExistenceInfo.TryRead(Field);
	}
};

/** Message from Server to Client to cook the given packages. */
struct FAssignPackagesMessage : public IMPCollectorMessage
{
public:
	FAssignPackagesMessage() = default;
	FAssignPackagesMessage(TArray<FAssignPackageData>&& InPackageDatas,
		TArray<FPackageDataExistenceInfo>&& InExistenceInfos);

	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("AssignPackagesMessage"); }

public:
	TArray<FAssignPackageData> PackageDatas;
	TArray<FPackageDataExistenceInfo> ExistenceInfos;
	TConstArrayView<const ITargetPlatform*> OrderedSessionPlatforms;
	static FGuid MessageType;
};

/** Message from Server to Client to cancel the cook of the given packages. */
struct FAbortPackagesMessage : public IMPCollectorMessage
{
public:
	FAbortPackagesMessage() = default;
	explicit FAbortPackagesMessage(TArray<FName>&& InPackageNames);

	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("AbortPackagesMessage"); }

public:
	TArray<FName> PackageNames;
	static FGuid MessageType;
};

/**
 * Message from either Server to Client.
 * If from Server, request that Client shutdown.
 * If from Client, notify Server it is shutting down.
 */
struct FAbortWorkerMessage : public IMPCollectorMessage
{
public:
	enum EType
	{
		CookComplete,
		Abort,
		AbortAcknowledge,
	};
	FAbortWorkerMessage(EType InType = EType::Abort);
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("AbortWorkerMessage"); }

public:
	EType Type;
	static FGuid MessageType;
};

/** Message From Server to Client giving all of the COTFS settings the client needs. */
struct FInitialConfigMessage : public IMPCollectorMessage
{
public:
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("InitialConfigMessage"); }

	void ReadFromLocal(const UCookOnTheFlyServer& COTFS, const TConstArrayView<const ITargetPlatform*>& InOrderedSessionPlatforms,
		const FCookByTheBookOptions& InCookByTheBookOptions, const FCookOnTheFlyOptions& InCookOnTheFlyOptions,
		const FBeginCookContextForWorker& InBeginContext);
	void AddMessage(UE::CompactBinaryTCP::FMarshalledMessage&& InMarshalledMessage) { MPCollectorMessages.Add(MoveTemp(InMarshalledMessage)); }

	ECookMode::Type GetDirectorCookMode() const { return DirectorCookMode; }
	ECookInitializationFlags GetCookInitializationFlags() const { return CookInitializationFlags; }
	FInitializeConfigSettings&& ConsumeInitializeConfigSettings() { return MoveTemp(InitialSettings); }
	FBeginCookConfigSettings&& ConsumeBeginCookConfigSettings() { return MoveTemp(BeginCookSettings); }
	FCookByTheBookOptions&& ConsumeCookByTheBookOptions() { return MoveTemp(CookByTheBookOptions); }
	FCookOnTheFlyOptions&& ConsumeCookOnTheFlyOptions() { return MoveTemp(CookOnTheFlyOptions); }
	TArray<UE::CompactBinaryTCP::FMarshalledMessage>&& ConsumeCollectorMessages() { return MoveTemp(MPCollectorMessages); }

	const FBeginCookContextForWorker& GetBeginCookContext() const { return BeginCookContext; }
	const TArray<ITargetPlatform*>& GetOrderedSessionPlatforms() const { return OrderedSessionPlatforms; }
	bool IsZenStore() const { return bZenStore; }

public:
	static FGuid MessageType;
private:
	FInitializeConfigSettings InitialSettings;
	FBeginCookConfigSettings BeginCookSettings;
	FBeginCookContextForWorker BeginCookContext;
	FCookByTheBookOptions CookByTheBookOptions;
	FCookOnTheFlyOptions CookOnTheFlyOptions;
	TArray<ITargetPlatform*> OrderedSessionPlatforms;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> MPCollectorMessages;
	ECookMode::Type DirectorCookMode = ECookMode::CookByTheBook;
	ECookInitializationFlags CookInitializationFlags = ECookInitializationFlags::None;
	bool bZenStore = false;
};

/** Information about a discovered package sent from a CookWorker to the Director. */
struct FDiscoveredPackageReplication
{
	FName PackageName;
	FName NormalizedFileName;
	FName ParentGenerator;
	FIoHash GeneratedPackageHash;
	FInstigator Instigator;
	FDiscoveredPlatformSet Platforms;
	ICookPackageSplitter::EGeneratedRequiresGenerator DoesGeneratedRequireGenerator =
		ICookPackageSplitter::EGeneratedRequiresGenerator::None;
	EUrgency Urgency = EUrgency::Normal;

private:
	void Write(FCbWriter& Writer, TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms) const;
	bool TryRead(FCbFieldView Field, TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms);

	friend void WriteToCompactBinary(FCbWriter& Writer, const FDiscoveredPackageReplication& Package,
		TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms)
	{
		Package.Write(Writer, OrderedSessionAndSpecialPlatforms);
	}
	friend bool LoadFromCompactBinary(FCbFieldView Field, FDiscoveredPackageReplication& OutPackage,
		TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms)
	{
		return OutPackage.TryRead(Field, OrderedSessionAndSpecialPlatforms);
	}
};

/**
 * Message from CookWorker to Director that reports dependency packages discovered during load/save of
 * a package that were not found in the earlier traversal of the packages dependencies.
 */
struct FDiscoveredPackagesMessage : public IMPCollectorMessage
{
public:
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("DiscoveredPackagesMessage"); }

public:
	TArray<FDiscoveredPackageReplication> Packages;
	TConstArrayView<const ITargetPlatform*> OrderedSessionAndSpecialPlatforms;
	static FGuid MessageType;
};

/**
 * Events on a PackageGenerator that occur on the Director or a CookWorker and need to be replicated to the other
 * process(es).
 */
enum class EGeneratorEvent
{
	Invalid,
	/** Sent from Worker to Director to let the Director know to start waiting for AllSavesCompleted. */
	QueuedGeneratedPackages,
	/** Broadcast to Workers to set their value of bSkipSaveExistingGenerator when it is first set. */
	SkipSaveExistingGenerator,
	/**
	 * Broadcast to Workers so any waiting to save or waiting on the fence to pass before clearing their data can save
	 * or clear their data.
	 */
	QueuedGeneratedPackagesFencePassed,
	/** Broadcast to Workers to unconditionally clear their data. */
	AllSavesCompleted,
	Num,
};

/** A message reporting an EGeneratorEvent that occurred on the Director or a CookWorker. */
struct FGeneratorEventMessage : public IMPCollectorMessage
{
	FGeneratorEventMessage() = default;
	FGeneratorEventMessage(EGeneratorEvent InEvent, FName InPackageName);

	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("GeneratorEventMessage"); }

public:
	FName PackageName;
	EGeneratorEvent Event = EGeneratorEvent::Invalid;
	static FGuid MessageType;
};

/**
 * Send log messages from CookWorkers to the CookDirector, which marks them up with the CookWorkerId and
 * then prints them to its own log.
 */
class FLogMessagesMessageHandler : public IMPCollector
{
public:
	FLogMessagesMessageHandler(ILogHandler& InCOTFSLogHandler);
	void ClientReportLogMessage(const FReplicatedLogData& LogData);

	// IMPCollector
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("FLogMessagesMessageHandler"); }
	virtual void ClientTick(FMPCollectorClientTickContext& Context) override;
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

private:
	ILogHandler& COTFSLogHandler;

	FCriticalSection QueueLock;
	TArray<FReplicatedLogData> QueuedLogs;
	TArray<FReplicatedLogData> QueuedLogsBackBuffer;

	static FGuid MessageType;
};

/**
 * Message from Director to CookWorker or CookWorker to Director that reports a heartbeat number, in addition to
 * reporting the machine is still alive just by the presence of the message.
 * The Director intiates a heartbeat message; the CookWorker always responds to a heartbeat message with its own
 * heartbeat message in reply, with the same number.
 */
struct FHeartbeatMessage : public IMPCollectorMessage
{
public:
	FHeartbeatMessage(int32 InHeartbeatNumber=-1);
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("HeartbeatMessage"); }

public:
	int32 HeartbeatNumber;
	static FGuid MessageType;
};

class FPackageWriterMPCollector : public UE::Cook::IMPCollector
{
public:
	FPackageWriterMPCollector(UCookOnTheFlyServer& InCOTFS);
	virtual FGuid GetMessageType() const { return MessageType; }
	virtual const TCHAR* GetDebugName() const { return TEXT("PackageWriter"); }

	virtual void ClientTickPackage(FMPCollectorClientTickPackageContext& Context) override;
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

private:
	UCookOnTheFlyServer& COTFS;
	static FGuid MessageType;
};

/**
 * Message from CookWorker requesting that a file written from the CookWorker be moved into its target path
 * in the cook output directory on the CookDirector. Used for extra files specified by
 * UObject::CookAdditionalFilesOverride. The same file might be specified by multiple packages, so even though
 * CookWorkers and CookDirector are on the same machine, we need to have them moved to final location by the
 * CookDirector to avoid the possibility of multiple CookWorkers trying to write the same file at once.
 */
struct FFileTransferMessage : public IMPCollectorMessage
{
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("FileTransferMessage"); }

public:
	FString TempFileName;
	FString TargetFileName;
	static FGuid MessageType;
};

}