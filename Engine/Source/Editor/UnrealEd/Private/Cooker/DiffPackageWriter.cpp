// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/DiffPackageWriter.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Cooker/CookTypes.h"
#include "CookOnTheSide/CookLog.h"
#include "CoreGlobals.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "Engine/Engine.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CString.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Parse.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/App.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Templates/GuardValueAccessors.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/LinkerSave.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogDiff, Log, All);


static const TCHAR* const CallstackCutoffString = TEXT("UEditorEngine::Save()");

namespace UE::DiffWriter
{
extern const TCHAR* const IndentToken;
class FJsonDetailRecorder : public IDetailRecorder
{
public:
	FJsonDetailRecorder( const FString& InFilename )
	{
		FString JsonFilename = InFilename;
		if (UE::GetMultiprocessId() > 0)
		{
			FString JsonUniqueFilename = FString::Printf(TEXT("%s-%d.%s"), *FPaths::GetBaseFilename(InFilename), 
				UE::GetMultiprocessId(), *FPaths::GetExtension(InFilename));
			JsonFilename = FPaths::Combine( FPaths::GetPath(InFilename), JsonUniqueFilename);
		}
		UE_LOGF(LogDiff, Display, "Using Json details file %ls", *JsonFilename);

		JsonFile.Reset(IFileManager::Get().CreateFileWriter(*JsonFilename));
		JsonWriter = TJsonWriterFactory<ANSICHAR>::Create(JsonFile.Get());

		JsonWriter->WriteObjectStart(); // global
		JsonWriter->WriteValue("ProjectName", FApp::GetProjectName());
		JsonWriter->WriteValue("BuildVersion", FApp::GetBuildVersion());

		JsonWriter->WriteArrayStart("Packages");
	}

	virtual ~FJsonDetailRecorder()
	{
		if (JsonFile.IsValid())
		{
			JsonWriter->WriteArrayEnd(); // "Packages"
			JsonWriter->WriteObjectEnd(); // global
			JsonWriter->Close();
			JsonWriter.Reset();

			JsonFile.Reset();
		}
	}

	virtual void BeginPackage() override
	{
		check(State == EState::Idle);
		State = EState::InPackage;
	}

	virtual void BeginSection(const TCHAR* SectionFilename, EDiffWriterSectionType SectionType, 
		const FPackageData& SourceSection, const FPackageData& DestSection) override
	{
		check(State == EState::InPackage);

		CurrentPackage.CurrentSectionIndex = (int32)SectionType;
		State = EState::InSection;

		FSection& Section = GetSection();
		Section.SectionFilename = SectionFilename;
		Section.SourceSize = SourceSection.Size;
		Section.DestSize = DestSection.Size;

		GetSection().SectionFilename = SectionFilename;
	}
		 
	virtual void RecordDiff( int64 LocalOffset, 
		const FCallstacks::FCallstackData* DifferenceCallstackData ) override
	{
		check(State == EState::InSection);
		FDiff& Diff = GetSection().Diffs.AddDefaulted_GetRef();

		Diff.LocalOffset = LocalOffset;
		Diff.Size = 1;
		if (DifferenceCallstackData)
		{
			Diff.ObjectName = DifferenceCallstackData->GetObjectName();
			Diff.PropertyName = DifferenceCallstackData->GetPropertyName();
			Diff.Callstack = DifferenceCallstackData->ToString(CallstackCutoffString);
		}
		CurrentPackage.bHasDiffs = true;
	}

	virtual void ExtendPreviousDiff( int64 LocalOffset ) override
	{
		check(State == EState::InSection);
		check(CurrentPackage.bHasDiffs);
		FDiff& Diff = GetSection().Diffs.Last();

		check(Diff.LocalOffset <= LocalOffset);
		Diff.Size = (LocalOffset - Diff.LocalOffset) + 1;
	}

	virtual void IncrementPreviousDiff() override
	{
		check(State == EState::InSection);
		check(CurrentPackage.bHasDiffs);
		FDiff& Diff = GetSection().Diffs.Last();
		Diff.Count++;
	}

	virtual void RecordUndiagnosedDiff() override
	{
		check(State == EState::InSection);

		CurrentPackage.bHasDiffs = true;
		GetSection().bUndiagnosedDiff = true;
	}

	virtual void RecordUnreportedDiffs(int32 NumUnreportedDiffs) override
	{
		check(State == EState::InSection);

		CurrentPackage.bHasDiffs = true;
		GetSection().NumUnreportedDiffs = NumUnreportedDiffs;
	}

	virtual void RecordDetermismDiagnostics(const TCHAR* DeterminismLines) override
	{
		check(State == EState::InPackage);			
		CurrentPackage.DeterminismDiagnostics = DeterminismLines;
	}

	virtual void RecordTableDifferences(const TCHAR* ItemName, const TCHAR* HumanReadableString) override
	{
		check(State == EState::InSection);
		FTableDiff& TableDiff = GetSection().TableDiffs.AddDefaulted_GetRef();
		TableDiff.ItemName = ItemName;
		TableDiff.HumanReadableString = HumanReadableString;
		CurrentPackage.bHasDiffs = true;
	}


	virtual void EndSection() override
	{
		check(State == EState::InSection);
		CurrentPackage.CurrentSectionIndex = -1;
		State = EState::InPackage;
	}

	virtual void EndPackage(const TCHAR* Filename, const FPackageData& SourcePackage, 
		const FPackageData& DestPackage, const TCHAR* ClassName) override
	{
		static const TCHAR* SectionNames[(int32)EDiffWriterSectionType::MAX] =
		{
			TEXT("Header"),
			TEXT("Exports"),
		};

		check(State == EState::InPackage);

		if (CurrentPackage.bHasDiffs)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue("Filename",  GetSanitizedPaths(Filename));
			JsonWriter->WriteValue("ClassName", ClassName);

			for (int32 SectionIndex = 0; SectionIndex < (int32)EDiffWriterSectionType::MAX; SectionIndex++)
			{
				FSection& Section = CurrentPackage.Sections[SectionIndex];
				if (Section.Diffs.Num() > 0 || Section.TableDiffs.Num() > 0 || Section.bUndiagnosedDiff || Section.NumUnreportedDiffs)
				{
					JsonWriter->WriteObjectStart( SectionNames[SectionIndex] );
					if (SectionIndex != (int32)EDiffWriterSectionType::Header || Section.SectionFilename != Filename)
					{
						JsonWriter->WriteValue("Filename", GetSanitizedPaths(Section.SectionFilename));
					}
					JsonWriter->WriteValue("SourceSize",      Section.SourceSize);
					JsonWriter->WriteValue("DestSize",        Section.DestSize);

					if (Section.bUndiagnosedDiff)
					{
						JsonWriter->WriteValue("UndiagnosedDiff", true);
					}
					if (Section.NumUnreportedDiffs > 0)
					{
						JsonWriter->WriteValue("UnreportedDiffs", Section.NumUnreportedDiffs);
					}
					if (Section.Diffs.Num() > 0)
					{
						JsonWriter->WriteArrayStart("Diffs");
						for (const FDiff& Diff : Section.Diffs)
						{
							JsonWriter->WriteObjectStart();
							JsonWriter->WriteValue("LocalOffset", Diff.LocalOffset);
							JsonWriter->WriteValue("Size",        Diff.Size);
							if (Diff.Count > 1)
							{
								JsonWriter->WriteValue("Count", Diff.Count);
							}
							if (!Diff.Callstack.IsEmpty())
							{
								JsonWriter->WriteValue("ObjectName",   Diff.ObjectName);
								JsonWriter->WriteValue("PropertyName", Diff.PropertyName);
								JsonWriter->WriteValue("Callstack",    GetSanitizedCallstack(Diff));
							}
							JsonWriter->WriteObjectEnd();
						}
						JsonWriter->WriteArrayEnd();
					}
					if (Section.TableDiffs.Num() > 0)
					{
						JsonWriter->WriteArrayStart("TableDiffs");
						for (const FTableDiff& TableDiff : Section.TableDiffs)
						{
							JsonWriter->WriteObjectStart();
							JsonWriter->WriteValue("ItemName", TEXT("Map") + TableDiff.ItemName);
							JsonWriter->WriteValue("Detail", 
								TableDiff.HumanReadableString.Replace(IndentToken,TEXT("")));
							JsonWriter->WriteObjectEnd();
						}
						JsonWriter->WriteArrayEnd();

					}

					JsonWriter->WriteObjectEnd();
				}
			}

			if (!CurrentPackage.DeterminismDiagnostics.IsEmpty())
			{
				JsonWriter->WriteValue("Diagnostics", CurrentPackage.DeterminismDiagnostics);
			}
			JsonWriter->WriteObjectEnd();
		}

		CurrentPackage = FCurrentPackage();
		State = EState::Idle;
	}

protected:
	TUniquePtr<FArchive> JsonFile;
	TSharedPtr<TJsonWriter<ANSICHAR>> JsonWriter;

	enum class EState : uint8
	{
		Idle,
		InPackage,
		InSection,
	};
	EState State = EState::Idle;

	struct FDiff
	{
		int64 LocalOffset = 0;
		int64 Size = 0;
		FString ObjectName;
		FString PropertyName;
		FString Callstack;
		int32 Count = 1;
	};

	struct FTableDiff
	{
		FString ItemName;
		FString HumanReadableString;
	};

	struct FSection
	{
		FString SectionFilename;
		uint64 SourceSize = 0;
		uint64 DestSize = 0;
		bool bUndiagnosedDiff = false;
		int32 NumUnreportedDiffs = 0;
		TArray<FDiff> Diffs;
		TArray<FTableDiff> TableDiffs;
	};

	struct FCurrentPackage
	{
		FString DeterminismDiagnostics;
		FSection Sections[(int32)EDiffWriterSectionType::MAX];

		bool bHasDiffs = false;
		int32 CurrentSectionIndex = -1;
	};
	FCurrentPackage CurrentPackage;

	FSection& GetSection()
	{
		check(CurrentPackage.CurrentSectionIndex != -1);
		check(State == EState::InSection);
		return CurrentPackage.Sections[CurrentPackage.CurrentSectionIndex];
	}

	FString GetSanitizedPaths( const FString& Path ) const
	{
		FStringView RelativePath;
		if (FPathViews::TryMakeChildPathRelativeTo(Path, FPaths::RootDir(), RelativePath))
		{
			return FString(RelativePath);
		}
		else
		{
			// this is used for the absolute file paths embedded in the callstack strings
			FString Result = Path;

			Result.ReplaceCharInline('\\', '/');
			Result.ReplaceInline(*FPaths::RootDir(), TEXT(""));		
			return Result;
		}
	}


	FString GetSanitizedCallstack(const FDiff& Diff) const
	{
		FString Result = Diff.Callstack.Replace(IndentToken, TEXT("")).Replace(TEXT("%DWA%"), TEXT(""));
		return GetSanitizedPaths(Result);
	}
};

// Responsible for replicating diff stats collected on workers to the director
class FDiffPackageWriterMPCollector : public UE::Cook::IMPCollector
{
public:
	FDiffPackageWriterMPCollector(FDiffPackageWriter* InOwner) : Owner(InOwner) {}

	virtual FGuid GetMessageType() const { return MessageType; }
	virtual const TCHAR* GetDebugName() const { return TEXT("DiffPackageWriterMPCollector"); }

	virtual void ClientTickPackage(UE::Cook::FMPCollectorClientTickPackageContext& Context) override;
	virtual void ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

public:
	static FGuid MessageType;

private:
	FDiffPackageWriter* Owner;
};

FGuid FDiffPackageWriterMPCollector::MessageType(0x0E2BA390, 0x2D574257, 0xBBD107D7, 0xB5930C79);

// sends the diff stats for the current package to the director, if we have any 
void FDiffPackageWriterMPCollector::ClientTickPackage(UE::Cook::FMPCollectorClientTickPackageContext& Context)
{
	const FDiffPackageWriter::FPackageDiffStats* FoundStats = Owner->PackageDiffStats.Find(Context.GetPackageName());
	if (FoundStats != nullptr)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << "Package" << Context.GetPackageName();
		Writer << "AssetClass" << FoundStats->AssetClass;
		Writer << "TotalDiffCount" << FoundStats->TotalDifferences;
		Writer << "TotalDiffBytes" << FoundStats->TotalSizeDifferenceBytes;
		Writer.EndObject();
		Context.AddMessage(Writer.Save().AsObject());
	}
}

// receives diff stats from a client
void FDiffPackageWriterMPCollector::ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	FDiffPackageWriter::FPackageDiffStats ReceivedStats;
	FName ReceivedPackageName;
	bool bReadOK = LoadFromCompactBinary(Message["Package"], ReceivedPackageName);
	if (bReadOK)
	{
		bReadOK = LoadFromCompactBinary(Message["AssetClass"], ReceivedStats.AssetClass);
	}
	if (bReadOK)
	{
		bReadOK = LoadFromCompactBinary(Message["TotalDiffCount"], ReceivedStats.TotalDifferences);
	}
	if (bReadOK)
	{
		bReadOK = LoadFromCompactBinary(Message["TotalDiffBytes"], ReceivedStats.TotalSizeDifferenceBytes);
	}
	if (bReadOK)
	{
		Owner->PackageDiffStats.FindOrAdd(ReceivedPackageName, ReceivedStats);		// Duplicates can be received here, we ignore them
	}
	else
	{
		UE_LOGF(LogCook, Error, "Failed to read diff status from MP collector");
	}
}

} // namespace UE::DiffWriter


FDiffPackageWriter::FDiffPackageWriter(UCookOnTheFlyServer& InCOTFS, TUniquePtr<ICookedPackageWriter>&& InInner,
	UE::Cook::FDeterminismManager* InDeterminismManager, EReportingMode ReportMode)
	: Inner(MoveTemp(InInner))
	, COTFS(InCOTFS)
{
	AccumulatorGlobals.Reset(new UE::DiffWriter::FAccumulatorGlobals(Inner.Get()));

	GConfig->GetInt(TEXT("CookSettings"), TEXT("MaxDiffsToLog"), MaxDiffsToLog, GEditorIni);
	// Command line override for MaxDiffsToLog
	FParse::Value(FCommandLine::Get(), TEXT("MaxDiffstoLog="), MaxDiffsToLog);

	bSaveForDiff = FParse::Param(FCommandLine::Get(), TEXT("SaveForDiff"));
	bDiffOptional = FParse::Param(FCommandLine::Get(), TEXT("DiffOptional"));

	GConfig->GetBool(TEXT("CookSettings"), TEXT("IgnoreHeaderDiffs"), bIgnoreHeaderDiffs, GEditorIni);
	// Command line override for IgnoreHeaderDiffs
	if (bIgnoreHeaderDiffs)
	{
		bIgnoreHeaderDiffs = !FParse::Param(FCommandLine::Get(), TEXT("HeaderDiffs"));
	}
	else
	{
		bIgnoreHeaderDiffs = FParse::Param(FCommandLine::Get(), TEXT("IgnoreHeaderDiffs"));
	}

	FString JsonFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("CookDiffJson="), JsonFile))
	{
		AccumulatorGlobals->DetailRecorder = MakeUnique<UE::DiffWriter::FJsonDetailRecorder>(JsonFile);
	}

	ParseCmds();

	Indent = FCString::Spc(FOutputDeviceHelper::FormatLogLine(ELogVerbosity::Warning,
		LogDiff.GetCategoryName(), TEXT(""), GPrintLogTimes).Len());
	NewLine = TEXT("\n"); // OutputDevices are responsible for remapping to LINE_TERMINATOR if desired

	check(InDeterminismManager);
	DeterminismManager = InDeterminismManager;

	if (ReportMode == EReportingMode::OutputPackageSummary)
	{
		DiffStatCollector = new UE::DiffWriter::FDiffPackageWriterMPCollector(this);
		COTFS.RegisterCollector(DiffStatCollector);
		bCollectDiffStats = true;
	}
}

FDiffPackageWriter::~FDiffPackageWriter()
{
	if (DiffStatCollector != nullptr)
	{
		COTFS.UnregisterCollector(DiffStatCollector);
		DiffStatCollector.SafeRelease();
	}
}

void FDiffPackageWriter::ParseCmds()
{
	const TCHAR* DumpObjListParam = TEXT("dumpobjlist");
	const TCHAR* DumpObjectsParam = TEXT("dumpobjects");

	FString CmdsText;
	const TCHAR* const CommandLine = FCommandLine::Get();

	if (FParse::Value(CommandLine, TEXT("-diffcmds="), CmdsText, false))
	{
		CmdsText = CmdsText.TrimQuotes();
		TArray<FString> CmdsList;
		CmdsText.ParseIntoArray(CmdsList, TEXT(","));
		for (const FString& Cmd : CmdsList)
		{
			if (Cmd.StartsWith(DumpObjListParam))
			{
				bDumpObjList = true;
				ParseDumpObjList(*Cmd + FCString::Strlen(DumpObjListParam));
			}
			else if (Cmd.StartsWith(DumpObjectsParam))
			{
				bDumpObjects = true;
				ParseDumpObjects(*Cmd + FCString::Strlen(DumpObjectsParam));
			}
		}
	}

	if (FParse::Param(CommandLine, TEXT("DiffDenyList")))
	{
		for (FTopLevelAssetPath& DenyBaseClassPath : UE::EditorDomain::ConstructTargetIncrementalClassBlockList())
		{
			UClass* DenyBaseClass = FindObject<UClass>(DenyBaseClassPath);
			if (DenyBaseClass)
			{
				CompareDenyListClasses.Add(DenyBaseClassPath);
				TArray<UClass*> DerivedClasses;
				GetDerivedClasses(DenyBaseClass, DerivedClasses);
				for (UClass* DerivedClass : DerivedClasses)
				{
					CompareDenyListClasses.Add(DerivedClass->GetClassPathName());
				}
			}
		}
	}
}

void FDiffPackageWriter::ParseDumpObjList(FString InParams)
{
	const TCHAR* PackageFilterParam = TEXT("-packagefilter=");
	FParse::Value(*InParams, PackageFilterParam, PackageFilter);
	RemoveParam(InParams, PackageFilterParam);

	// Add support for more parameters here
	// After all parameters have been parsed and removed, pass the remaining string as objlist params
	DumpObjListParams = InParams;
}

void FDiffPackageWriter::ParseDumpObjects(FString InParams)
{
	const TCHAR* PackageFilterParam = TEXT("-packagefilter=");
	FParse::Value(*InParams, PackageFilterParam, PackageFilter);
	RemoveParam(InParams, PackageFilterParam);

	const TCHAR* SortParam = TEXT("sort");
	bDumpObjectsSorted = FParse::Param(*InParams, SortParam);
	RemoveParam(InParams, SortParam);
}

void FDiffPackageWriter::RemoveParam(FString& InOutParams, const TCHAR* InParamToRemove)
{
	int32 ParamIndex = InOutParams.Find(InParamToRemove);
	if (ParamIndex >= 0)
	{
		int32 NextParamIndex = InOutParams.Find(TEXT(" -"),
			ESearchCase::CaseSensitive, ESearchDir::FromStart, ParamIndex + 1);
		if (NextParamIndex < ParamIndex)
		{
			NextParamIndex = InOutParams.Len();
		}
		InOutParams = InOutParams.Mid(0, ParamIndex) + InOutParams.Mid(NextParamIndex);
	}
}

void FDiffPackageWriter::BeginPackage(const FBeginPackageInfo& Info)
{
	BeginPackageDiff(Info);

	ConditionallyDumpObjList();
	ConditionallyDumpObjects();
	Inner->BeginPackage(Info);
}

void FDiffPackageWriter::CommitPackage(FCommitPackageInfo&& Info)
{
	if (bHasStartedSecondSave && bSaveForDiff)
	{
		// Write the package to _ForDiff, but do not write any sidecars
		EnumRemoveFlags(Info.WriteOptions, EWriteOptions::WriteSidecars);
		EnumAddFlags(Info.WriteOptions, EWriteOptions::SaveForDiff);
	}
	else
	{
		EnumRemoveFlags(Info.WriteOptions, EWriteOptions::Write);
	}
	Inner->CommitPackage(MoveTemp(Info));
	Package = nullptr;
}

void FDiffPackageWriter::WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
	const TArray<FFileRegion>& FileRegions)
{
	check(Info.MultiOutputIndex < 2);
	check(Accumulators[Info.MultiOutputIndex].IsValid()); // Should have been constructed by CreateLinkerArchive
	UE::DiffWriter::FAccumulator& Accumulator = *Accumulators[Info.MultiOutputIndex];
	UE::DiffWriter::FDiffArchive& ExportsArchiveInternal =static_cast<UE::DiffWriter::FDiffArchive&>(ExportsArchive);
	check(&ExportsArchiveInternal.GetAccumulator() == &Accumulator);

	FPackageInfo LocalInfo(Info);
	Inner->CompleteExportsArchiveForDiff(LocalInfo, ExportsArchive);

	if (!bHasStartedSecondSave)
	{
		ICookedPackageWriter::FPreviousCookedBytesData PreviousInnerData;
		if (!Inner->GetPreviousCookedBytes(LocalInfo, PreviousInnerData))
		{
			PreviousInnerData.Data.Reset();
			PreviousInnerData.HeaderSize = 0;
			PreviousInnerData.Size = 0;
		}
		check(PreviousInnerData.Data.Get() != nullptr || (PreviousInnerData.Size == 0 && PreviousInnerData.HeaderSize == 0));

		bNewPackage = PreviousInnerData.Size == 0;
		Accumulator.OnFirstSaveComplete(LocalInfo.LooseFilePath, LocalInfo.HeaderSize, Info.HeaderSize,
			MoveTemp(PreviousInnerData));
		bIsDifferent = Accumulator.HasDifferences();
		if (bIsDifferent && !IsPackageDiffAllowed())
		{
			bIsDifferent = false;
		}

		if (bCollectDiffStats)
		{
			FPackageDiffStats& DiffStats = PackageDiffStats.Add(Info.PackageName);
			DiffStats = {
				.AssetClass = Accumulator.GetAssetClass(),
				.TotalSizeDifferenceBytes = 0,
				.TotalDifferences = 0
			};
		}
	}
	else
	{
		// Avoid an assert when calling StaticFindObject during save, which we do to list the "exports" from a package.
		// We are not writing the discovered objects into the saved package, so the call to StaticFindObject is legal.
		TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);

		Accumulator.OnSecondSaveComplete(LocalInfo.HeaderSize);

		TMap<FName, FArchiveDiffStats> DiffStats;
		Accumulator.SetDeterminismManager(*DeterminismManager);
		Accumulator.CompareWithPrevious(CallstackCutoffString, DiffStats);

		if (bCollectDiffStats)
		{
			FPackageDiffStats* Stats = PackageDiffStats.Find(Info.PackageName);
			if (Stats)
			{
				for (const TPair<FName, FArchiveDiffStats>& DiffStat : DiffStats)
				{
					Stats->TotalSizeDifferenceBytes += FMath::Abs(DiffStat.Value.DiffSize);
					Stats->TotalDifferences += DiffStat.Value.NumDiffs;
				}
			}
		}
		

		//COOK_STAT(FSavePackageStats::NumberOfDifferentPackages++);
		//COOK_STAT(FSavePackageStats::MergeStats(PackageDiffStats));
	}

	Inner->WritePackageData(LocalInfo, ExportsArchive, FileRegions);
}

TMap<FName, FDiffPackageWriter::FPackageDiffAssetClassSummary> FDiffPackageWriter::GetAssetClassDifferenceSummaries() const
{
	TMap<FName, FPackageDiffAssetClassSummary> AllSummaries;
	for (const TPair<FName, FPackageDiffStats>& PackageDiffStat : PackageDiffStats)
	{
		if (PackageDiffStat.Value.TotalDifferences != 0 || PackageDiffStat.Value.TotalSizeDifferenceBytes != 0)
		{
			FPackageDiffAssetClassSummary& Summary = AllSummaries.FindOrAdd(PackageDiffStat.Value.AssetClass);
			Summary.TotalPackageCount++;
			Summary.TotalPackagesWithDifferences += PackageDiffStat.Value.TotalDifferences > 0 ? 1 : 0;
			Summary.TotalDifferenceCount += PackageDiffStat.Value.TotalDifferences;
		}
	}

	// Sort by package differences + class name
	AllSummaries.KeySort([](const FName& A, const FName& B) {
		return A.LexicalLess(B);
	});
	AllSummaries.ValueStableSort([](const FPackageDiffAssetClassSummary& A, const FPackageDiffAssetClassSummary& B) {
		return A.TotalPackagesWithDifferences > B.TotalPackagesWithDifferences;
	});

	return AllSummaries;
}

void FDiffPackageWriter::LogAssetClassDiffSummary() const
{
	const TMap<FName, FDiffPackageWriter::FPackageDiffAssetClassSummary> AllSummaries = GetAssetClassDifferenceSummaries();
	if (AllSummaries.Num() > 0)
	{
		UE_LOGF(LogDiff, Display, "Differences detected per asset class:");
		UE_LOGF(LogDiff, Display, "\tAsset Class | Changed Packages | Avg. Differences per Package");
	}
	for (const TPair<FName, FPackageDiffAssetClassSummary>& AssetClassDifferences : AllSummaries)
	{
		const FPackageDiffAssetClassSummary& Stats = AssetClassDifferences.Value;
		if (Stats.TotalPackagesWithDifferences > 0)
		{
			const float AverageDifferenceCount = Stats.TotalDifferenceCount / static_cast<float>(Stats.TotalPackagesWithDifferences);
			UE_LOGF(LogDiff, Display, "\t%ls | %lld of %lld | %.1f",
				*AssetClassDifferences.Key.ToString(), Stats.TotalPackagesWithDifferences, Stats.TotalPackageCount, AverageDifferenceCount);
		}
	}
}

void FDiffPackageWriter::EndCook(const FCookInfo& Info)
{
	Inner->EndCook(Info);

	const bool bShouldOutputClassDummary = COTFS.GetCookMode() != ECookMode::CookWorker;	// only the director or single process cook should output summary here
	if (bShouldOutputClassDummary)
	{
		LogAssetClassDiffSummary();
	}
}

bool FDiffPackageWriter::IsPackageDiffAllowed() const
{
	if (!CompareDenyListClasses.IsEmpty() && Package != nullptr)
	{
		bool bHasDenyClass = false;
		ForEachObjectWithPackage(Package, [this, &bHasDenyClass](UObject* Object)
			{
				FTopLevelAssetPath ClassPath = Object->GetClass()->GetClassPathName();
				if (CompareDenyListClasses.Contains(ClassPath))
				{
					bHasDenyClass = true;
					return false; // Stop iterating
				}
				return true; // Keep iterating
			});
		if (bHasDenyClass)
		{
			return false;
		}
	}
	return true;
}

void FDiffPackageWriter::BeginPackageDiff(const FBeginPackageInfo& Info)
{
	bIsDifferent = false;
	bNewPackage = false;
	bHasStartedSecondSave = false;
	Accumulators[0].SafeRelease();
	Accumulators[1].SafeRelease();

	BeginInfo = Info;
	Package = FindObjectFast<UPackage>(nullptr, BeginInfo.PackageName);
}

UE::DiffWriter::FMessageCallback FDiffPackageWriter::GetDiffWriterMessageCallback()
{
	return UE::DiffWriter::FMessageCallback([this](ELogVerbosity::Type Verbosity, FStringView Message)
		{
			this->OnDiffWriterMessage(Verbosity, Message);
		});
}

void FDiffPackageWriter::OnDiffWriterMessage(ELogVerbosity::Type Verbosity, FStringView Message)
{
	FMsg::Logf(__FILE__, __LINE__, LogDiff.GetCategoryName(), Verbosity, TEXT("%s"), *ResolveText(Message));
}

FString FDiffPackageWriter::ResolveText(FStringView Message)
{
	FString ResolvedText(Message);
	check(this->Indent && this->NewLine);
	ResolvedText.ReplaceInline(UE::DiffWriter::IndentToken, this->Indent);
	ResolvedText.ReplaceInline(UE::DiffWriter::NewLineToken, this->NewLine);
	return ResolvedText;
}

UE::DiffWriter::FAccumulator& FDiffPackageWriter::ConstructAccumulator(FName PackageName, UObject* Asset,
	uint16 MultiOutputIndex)
{
	check(MultiOutputIndex < 2);
	TRefCountPtr<UE::DiffWriter::FAccumulator>& Accumulator = Accumulators[MultiOutputIndex];
	if (!Accumulator.IsValid())
	{
		TSharedPtr<UE::DiffWriter::FDiffOutputRecorder> DiffOutputRecorder = 
			MakeShared<UE::DiffWriter::FDiffOutputRecorder>(GetDiffWriterMessageCallback(), 
				AccumulatorGlobals->DetailRecorder.Get());

		check(!bHasStartedSecondSave); // Accumulator should already exist from CreateLinkerArchive in the first save
		Accumulator = new UE::DiffWriter::FAccumulator(*AccumulatorGlobals, Asset, *PackageName.ToString(), MaxDiffsToLog,
			bIgnoreHeaderDiffs, DiffOutputRecorder, Inner->GetCookCapabilities().HeaderFormat);
	}
	return *Accumulator;
}

TUniquePtr<FLargeMemoryWriter> FDiffPackageWriter::CreateLinkerArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	UE::DiffWriter::FAccumulator& Accumulator = ConstructAccumulator(PackageName, Asset, MultiOutputIndex);
	return TUniquePtr<FLargeMemoryWriter>(new UE::DiffWriter::FDiffArchiveForLinker(Accumulator));
}

TUniquePtr<FLargeMemoryWriter> FDiffPackageWriter::CreateLinkerExportsArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	UE::DiffWriter::FAccumulator& Accumulator = ConstructAccumulator(PackageName, Asset, MultiOutputIndex);
	return TUniquePtr<FLargeMemoryWriter>(new UE::DiffWriter::FDiffArchiveForExports(Accumulator));
}

void FDiffPackageWriter::UpdateSaveArguments(FSavePackageArgs& SaveArgs)
{
	// if we are diffing optional data, add it to the save args, otherwise strip it
	if (bDiffOptional)
	{
		SaveArgs.SaveFlags |= SAVE_Optional;
	}

	Inner->UpdateSaveArguments(SaveArgs);
}


bool FDiffPackageWriter::IsAnotherSaveNeeded(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs)
{
	checkf(!Inner->IsAnotherSaveNeeded(PreviousResult, SaveArgs),
		TEXT("DiffPackageWriter does not support an Inner that needs multiple saves."));
	if (PreviousResult == ESavePackageResult::Timeout)
	{
		return false;
	}

	// When looking for deterministic cook issues, first serialize the package to memory and do a simple diff with the
	// existing package. If the simple memory diff was not identical, collect callstacks for all Serialize calls and
	// dump differences to log
	if (!bHasStartedSecondSave)
	{
		bHasStartedSecondSave = true;
		if (PreviousResult.Result == ESavePackageResult::Success && bIsDifferent && !bNewPackage)
		{
			// The contract with the Inner is that Begin is paired with a single commit;
			// send the old commit and the new begin
			FCommitPackageInfo CommitInfo;
			CommitInfo.Status = IPackageWriter::ECommitStatus::Success;
			CommitInfo.PackageName = BeginInfo.PackageName;
			CommitInfo.WriteOptions = EWriteOptions::None;
			Inner->CommitPackage(MoveTemp(CommitInfo));
			Inner->BeginPackage(BeginInfo);

			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		return false;
	}
}

void FDiffPackageWriter::RegisterDeterminismHelper(UObject* SourceObject,
	const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	DeterminismManager->RegisterDeterminismHelper(SourceObject, DeterminismHelper);
}

bool FDiffPackageWriter::FilterPackageName(const FString& InWildcard)
{
	bool bInclude = false;
	FString PackageName = BeginInfo.PackageName.ToString();
	if (PackageName.MatchesWildcard(InWildcard))
	{
		bInclude = true;
	}
	else if (FPackageName::GetShortName(PackageName).MatchesWildcard(InWildcard))
	{
		bInclude = true;
	}
	else
	{
		const FString& Filename = BeginInfo.LooseFilePath;
		bInclude = Filename.MatchesWildcard(InWildcard);
	}
	return bInclude;
}

void FDiffPackageWriter::ConditionallyDumpObjList()
{
	if (bDumpObjList)
	{
		if (FilterPackageName(PackageFilter))
		{
			FString ObjListExec = TEXT("OBJ LIST ");
			ObjListExec += DumpObjListParams;

			TGuardValue GuardLogTimes(GPrintLogTimes, ELogTimes::None);
			TGuardValue GuardLogCategory(GPrintLogCategory, false);
			TGuardValue GuardPrintLogVerbosity(GPrintLogVerbosity, false);

			GEngine->Exec(nullptr, *ObjListExec);
		}
	}
}

void FDiffPackageWriter::ConditionallyDumpObjects()
{
	if (bDumpObjects)
	{
		if (FilterPackageName(PackageFilter))
		{
			TArray<FString> AllObjects;
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				AllObjects.Add(*It->GetFullName());
			}
			if (bDumpObjectsSorted)
			{
				AllObjects.Sort();
			}

			TGuardValue GuardLogTimes(GPrintLogTimes, ELogTimes::None);
			TGuardValue GuardLogCategory(GPrintLogCategory, false);
			TGuardValue GuardPrintLogVerbosity(GPrintLogVerbosity, false);

			for (const FString& Obj : AllObjects)
			{
				UE_LOGF(LogCook, Display, "%ls", *Obj);
			}
		}
	}
}