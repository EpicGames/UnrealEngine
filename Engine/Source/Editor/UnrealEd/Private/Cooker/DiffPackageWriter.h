// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetRegistryState.h"
#include "Cooker/CookDeterminismManager.h"
#include "Cooker/DiffWriterArchive.h"
#include "Serialization/PackageWriter.h"

class UCookOnTheFlyServer;
namespace UE::DiffWriter { struct FAccumulatorGlobals; class FDiffPackageWriterMPCollector; }

/** A CookedPackageWriter that diffs output from the current cook with the file that was saved in the previous cook. */
class FDiffPackageWriter : public ICookedPackageWriter
{
public:
	enum class EReportingMode {
		OutputPackageSummary,
		NoPackageSummary
	};

	FDiffPackageWriter(UCookOnTheFlyServer& InCOTFS, TUniquePtr<ICookedPackageWriter>&& InInner, 
		UE::Cook::FDeterminismManager* InDeterminismManager, EReportingMode ReportMode);
	virtual ~FDiffPackageWriter();

	virtual void SetCooker(UE::PackageWriter::Private::ICookerInterface* CookerInterface) override
	{
		// Nothing to do, our Inner has the Cooker interface, we don't use it directly
	}

	// IPackageWriter
	virtual FCapabilities GetCapabilities() const override
	{
		FCapabilities Result = Inner->GetCapabilities();
		Result.bIgnoreHeaderDiffs = bIgnoreHeaderDiffs;
		Result.bDeterminismDebug = true;
		return Result;
	}
	virtual void BeginPackage(const FBeginPackageInfo& Info) override;
	virtual void CommitPackage(FCommitPackageInfo&& Info) override;
	virtual void WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
		const TArray<FFileRegion>& FileRegions) override;
	virtual void WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData,
		const TArray<FFileRegion>& FileRegions) override
	{
		Inner->WriteBulkData(Info, BulkData, FileRegions);
	}
	virtual void WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData) override
	{
		Inner->WriteAdditionalFile(Info, FileData);
	}
	virtual void WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info, const FIoBuffer& Data,
		const TArray<FFileRegion>& FileRegions) override
	{
		Inner->WriteLinkerAdditionalData(Info, Data, FileRegions);
	}
	virtual void WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data) override
	{
		Inner->WritePackageTrailer(Info, Data);
	}
	virtual int64 GetExportsFooterSize() override
	{
		return Inner->GetExportsFooterSize();
	}
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) override;
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerExportsArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) override;
	virtual bool IsPreSaveCompleted() const override
	{
		return bHasStartedSecondSave;
	}
	virtual void RegisterDeterminismHelper(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) override;

	// ICookedPackageWriter
	virtual FCookCapabilities GetCookCapabilities() const override
	{
		FCookCapabilities Result = Inner->GetCookCapabilities();
		Result.bDiffModeSupported = false; // DiffPackageWriter can not be an inner of another DiffPackageWriter
		Result.bReadOnly = true;
		return Result;
	}
	virtual FDateTime GetPreviousCookTime() const
	{
		return Inner->GetPreviousCookTime();
	}
	virtual void Initialize(const FCookInfo& Info) override
	{
		Inner->Initialize(Info);
	}
	virtual void BeginCook(const FCookInfo& Info) override
	{
		Inner->BeginCook(Info);
	}
	virtual void EndCook(const FCookInfo& Info) override;
	virtual void PopulateOplog(const FAssetRegistryState& PreviousState, int32& OutNumPackagesInOplog) override
	{
		Inner->PopulateOplog(PreviousState, OutNumPackagesInOplog);
	}
	virtual void UpdateLastReferenceDateAndPruneStaleOps(
		UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext) override
	{
		Inner->UpdateLastReferenceDateAndPruneStaleOps(OplogContext);
	}
	virtual TArray<FName> GetOplogPackageNames() override
	{
		return Inner->GetOplogPackageNames();
	}
	virtual FCbObject GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey) override
	{
		return Inner->GetOplogAttachment(PackageName, AttachmentKey);
	}
	virtual void GetOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override
	{
		return Inner->GetOplogAttachments(PackageNames, AttachmentKeys, MoveTemp(Callback));
	}
	virtual void GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override
	{
		return Inner->GetBaseGameOplogAttachments(PackageNames, AttachmentKeys, MoveTemp(Callback));
	}

	virtual ECommitStatus GetCommitStatus(FName PackageName) override
	{
		return Inner->GetCommitStatus(PackageName);
	}
	virtual void RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove) override
	{
		Inner->RemoveCookedPackages(PackageNamesToRemove);
	}
	virtual void RemoveCookedPackages() override
	{
		Inner->RemoveCookedPackages();
	}
	virtual void UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context) override
	{
		Inner->UpdatePackageModifiedStatus(Context);
	}
	virtual EPackageWriterResult BeginCacheForCookedPlatformData(FBeginCacheForCookedPlatformDataInfo& Info) override
	{
		return Inner->BeginCacheForCookedPlatformData(Info);
	}
	virtual void UpdateSaveArguments(FSavePackageArgs& SaveArgs) override;
	virtual bool IsAnotherSaveNeeded(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs) override;
	virtual TFuture<FCbObject> WriteMPCookMessageForPackage(FName PackageName) override
	{
		return Inner->WriteMPCookMessageForPackage(PackageName);
	}
	virtual bool TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message) override
	{
		return Inner->TryReadMPCookMessageForPackage(PackageName, Message);
	}
	virtual TMap<FName, TRefCountPtr<FPackageHashes>>& GetPackageHashes() override
	{
		return Inner->GetPackageHashes();
	}

protected:
	// Stores differences found for a particular package
	struct FPackageDiffStats
	{
		FName AssetClass;						// Top-level asset class for this package
		int64 TotalSizeDifferenceBytes = 0;		// Total size difference between old + new asset
		int64 TotalDifferences = 0;				// Total count of data differences in this asset 
	};

	// Accumulated diff stats for an entire asset class
	struct FPackageDiffAssetClassSummary
	{
		int64 TotalPackageCount = 0;				// Total packages saved of this asset class
		int64 TotalPackagesWithDifferences = 0;		// Total packages with detected differences
		int64 TotalDifferenceCount = 0;				// Total detected differences across all packages
	};

	void ParseCmds();
	void ParseDumpObjList(FString InParams);
	void ParseDumpObjects(FString InParams);
	void RemoveParam(FString& InOutParams, const TCHAR* InParamToRemove);
	bool FilterPackageName(const FString& InWildcard);
	void ConditionallyDumpObjList();
	void ConditionallyDumpObjects();
	UE::DiffWriter::FMessageCallback GetDiffWriterMessageCallback();
	virtual void OnDiffWriterMessage(ELogVerbosity::Type Verbosity, FStringView Message);
	FString ResolveText(FStringView Message);
	UE::DiffWriter::FAccumulator& ConstructAccumulator(FName PackageName, UObject* Asset, uint16 MultiOutputIndex);
	bool IsPackageDiffAllowed() const;
	void LogAssetClassDiffSummary() const;
	TMap<FName, FPackageDiffAssetClassSummary> GetAssetClassDifferenceSummaries() const;

	/** 
	 * When BeginPackage is called, CommitPackage must be called as well. But sometimes we don't want or don't need to do all this and we just want to
	 * initialize the internal variables as if BeginPackage was called. That's what BeginPackageDiff does.
	 * For example FIncrementalValidatePackageWriter can use an internal writer so FDiffPackageWriter::BeginPackage/CommitPackage will not be called, but to
	 * make sure the internal variables are correct and the object behaves as expected, BeginPackageDiff is called instead.
	*/
	void BeginPackageDiff(const FBeginPackageInfo& Info);

	TMap<FName, FPackageDiffStats> PackageDiffStats;
	TRefCountPtr<UE::DiffWriter::FDiffPackageWriterMPCollector> DiffStatCollector;
	TRefCountPtr<UE::DiffWriter::FAccumulator> Accumulators[2];
	FBeginPackageInfo BeginInfo;
	TSet<FTopLevelAssetPath> CompareDenyListClasses;
	TUniquePtr<ICookedPackageWriter> Inner;
	TUniquePtr<UE::DiffWriter::FAccumulatorGlobals> AccumulatorGlobals;
	UE::Cook::FDeterminismManager* DeterminismManager = nullptr;
	UCookOnTheFlyServer& COTFS;

	/** Only non-null between BeginPackage and CommitPackage. */
	UPackage* Package = nullptr;
	const TCHAR* Indent = nullptr;
	const TCHAR* NewLine = nullptr;
	FString DumpObjListParams;
	FString PackageFilter;
	int32 MaxDiffsToLog = 5;
	bool bSaveForDiff = false;
	bool bDiffOptional = false;
	bool bIgnoreHeaderDiffs = false;
	bool bIsDifferent = false;
	bool bNewPackage = false;
	bool bHasStartedSecondSave = false;
	bool bDumpObjList = false;
	bool bDumpObjects = false;
	bool bDumpObjectsSorted = false;
	bool bCollectDiffStats = false;

	friend class UE::DiffWriter::FDiffPackageWriterMPCollector;
};
