// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/DiffWriterArchive.h"

#include "Compression/CompressionUtil.h"
#include "Cooker/CookDeterminismManager.h"
#include "Cooker/DiffWriterLinkerLoadHeader.h"
#include "Cooker/DiffWriterZenHeader.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "PackageStoreOptimizer.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/StaticMemoryReader.h"
#include "Templates/GuardValueAccessors.h"
#include "Templates/Function.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyTempVal.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectThreadContext.h"

namespace UE::DiffWriter
{

static const ANSICHAR* DebugDataStackMarker = "\r\nDebugDataStack:\r\n";
const TCHAR* const IndentToken = TEXT("%DWA%    ");
const TCHAR* const NewLineToken = TEXT("%DWA%\n");

/**
 * Data parsed from the header in the package. Might be stored either as a ZenPackageHeader or
 * LinkerLoad PackageHeader, depending on the EPackageHeaderFormat of the package.
 * Provides an interface for the data needed to interpret the bytes of the exports of the package, such as the NameMap.
 */
class FPackageHeaderData
{
public:
	FPackageHeaderData(const TCHAR* InName, bool bInReadFromPackageStore,
		const FString& InAssetFilename, const FPackageData& InPackageData, EPackageHeaderFormat InFormat,
		FAccumulatorGlobals& InGlobals, TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder);
	~FPackageHeaderData();
	void Initialize();

	const TCHAR* GetName() const;
	const FString& GetAssetFilename() const;
	const FPackageData& GetPackageData() const;
	EPackageHeaderFormat GetFormat() const;
	FAccumulatorGlobals& GetGlobals() const;
	const TSharedRef<FDiffOutputRecorder>& GetDiffOutputRecorder() const;

	FDiffWriterZenHeader& GetZenHeader() const;
	FLinkerLoad* GetLinker() const;

	bool IsValid() const;
	bool TryGetMappedName(int32 Index, int32 Number, FName& OutName) const;

private:
	const TCHAR* Name = nullptr;
	const FString& AssetFilename;
	const FPackageData& PackageData;
	FAccumulatorGlobals& Globals;
	TSharedRef<FDiffOutputRecorder> DiffOutputRecorder;
	EPackageHeaderFormat Format = EPackageHeaderFormat::PackageFileSummary;
	bool bReadFromPackageStore = false;
	bool bInitialized = false;

	TUniquePtr<FDiffWriterZenHeader> ZenHeader;
	FLinkerLoad* Linker = nullptr;
};

/**
 * Interprets the read of an FName in the manner that it is serialized by FLinkerSave,
 * and implements that read on inner archive which e.g. is a FMemoryArchive to the
 * bytes of the package with the given header.
 */
class FPackageHeaderDataProxyArchive : public FArchiveProxy
{
public:
	FPackageHeaderDataProxyArchive(FPackageHeaderData& InHeader, FArchive& InInner);
	virtual FArchive& operator<<(FName& Value) override;

private:
	FPackageHeaderData& Header;
};

/** Logs any mismatching header data. */
void DumpPackageHeaderDiffs(FPackageHeaderData& SourceHeaderData, FPackageHeaderData& DestHeaderData,
	const int32 MaxDiffsToLog);

/** Returns a new linker for loading the specified package. */
FLinkerLoad* CreateLinkerForPackage(
	FUObjectSerializeContext* LoadContext,
	const FString& InPackageName,
	const FString& InFilename,
	const FPackageData& PackageData);


FCallstacks::FCallstackData::FCallstackData(TUniquePtr<ANSICHAR[]>&& InCallstack, uint32 InCppCallstackHash, UObject* InSerializedObject, FProperty* InSerializedProperty)
	: Callstack(MoveTemp(InCallstack))
	, SerializedObject(InSerializedObject)
	, SerializedProp(InSerializedProperty)
	, CppCallstackHash(InCppCallstackHash)
{
}

FString FCallstacks::FCallstackData::ToString(const TCHAR* CallstackCutoffText) const
{
	FString HumanReadableString;

	FString StackTraceText = Callstack.Get();
	if (CallstackCutoffText != nullptr)
	{
		// If the cutoff string is provided, remove all functions starting with the one specifiec in the cutoff string
		int32 CutoffIndex = StackTraceText.Find(CallstackCutoffText, ESearchCase::CaseSensitive);
		if (CutoffIndex > 0)
		{
			CutoffIndex = StackTraceText.Find(TEXT("\n"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, CutoffIndex - 1);
			if (CutoffIndex > 0)
			{
				StackTraceText = StackTraceText.Left(CutoffIndex + 1);
			}
		}
	}

	TArray<FString> StackLines;
	StackTraceText.ParseIntoArrayLines(StackLines);
	for (FString& StackLine : StackLines)
	{
		if (StackLine.StartsWith(TEXT("0x")))
		{
			int32 CutoffIndex = StackLine.Find(TEXT(" "), ESearchCase::CaseSensitive);
			if (CutoffIndex >= -1 && CutoffIndex < StackLine.Len() - 2)
			{
				StackLine.MidInline(CutoffIndex + 1, MAX_int32, EAllowShrinking::No);
			}
		}
		HumanReadableString += IndentToken;
		HumanReadableString += StackLine;
		HumanReadableString += NewLineToken;
	}

	FString ObjectName = GetObjectName();
	if (!ObjectName.IsEmpty())
	{
		HumanReadableString += NewLineToken;
		HumanReadableString += IndentToken;
		HumanReadableString += TEXT("Serialized Object: ");
		HumanReadableString += ObjectName;
		HumanReadableString += NewLineToken;
	}
	FString PropertyName = GetPropertyName();
	if (!PropertyName.IsEmpty())
	{
		if (ObjectName.IsEmpty())
		{
			HumanReadableString += NewLineToken;
		}
		HumanReadableString += IndentToken;
		HumanReadableString += TEXT("Serialized Property: ");
		HumanReadableString += PropertyName;
		HumanReadableString += NewLineToken;
	}
	return HumanReadableString;
}

TRefCountPtr<FCallstacks::FCallstackData> FCallstacks::FCallstackData::Clone() const
{
	TUniquePtr<ANSICHAR[]> CallstackCopy;
	if (Callstack)
	{
		if (const int32 Len = FCStringAnsi::Strlen(Callstack.Get()); Len > 0)
		{
			CallstackCopy = MakeUnique<ANSICHAR[]>(Len + 1);
			FMemory::Memcpy(CallstackCopy.Get(), Callstack.Get(), Len + 1);
		}
	}

	TRefCountPtr<FCallstackData> Clone = new FCallstackData(MoveTemp(CallstackCopy),
		CppCallstackHash, SerializedObject, SerializedProp);

	return Clone;
}

FCallstackKey FCallstacks::FCallstackData::GetKey() const
{
	return FCallstackKey{ SerializedObject, SerializedProp, CppCallstackHash };
}

FString FCallstacks::FCallstackData::GetObjectName() const
{
	return SerializedObject ? SerializedObject->GetFullName() : FString();
}

FString FCallstacks::FCallstackData::GetPropertyName() const
{
	return SerializedProp ? SerializedProp->GetFullName() : FString();
}

FCallstacks::FCallstacks()
	: StackTraceSize(65535)
	, PreviousCppCallstackHash(MAX_uint32)
	, bCppCallstackDirty(true)
	, EndOffset(0)
{
	StackTrace = MakeUnique<ANSICHAR[]>(StackTraceSize);
	StackTrace[0] = 0;
	// We create one entry in CallstackAtOffsetMap for every Serialize call,
	// and one entry in UniqueCallstacks for every unique callstack key in those Serialize calls.
	// This adds up to hundreds or thousands of entries for most packages. Reserve a large constant
	// initial size to avoid the reallocation costs on the first 1000 entries.
	CallstackAtOffsetMap.Reserve(1000);
	UniqueCallstacks.Reserve(1000);
}

void FCallstacks::Reset()
{
	CallstackAtOffsetMap.Reset();
	UniqueCallstacks.Reset();
	bCppCallstackDirty = true;
	PreviousCppCallstackHash = MAX_uint32;
	StackTrace[0] = 0;
	EndOffset = 0;
}

TRefCountPtr<FCallstacks::FCallstackData> FCallstacks::AddUniqueCallstack(const FCallstackKey& Key)
{
	ANSICHAR* Callstack = nullptr;
	TRefCountPtr<FCallstackData>& Existing = UniqueCallstacks.FindOrAdd(Key);
	if (!Existing)
	{
		TUniquePtr<ANSICHAR[]> NewCallstack;
		if (StackTrace.Get()[0] != '\0')
		{
			const int32 Len = FCStringAnsi::Strlen(StackTrace.Get()) + 1;
			NewCallstack = MakeUnique<ANSICHAR[]>(Len);
			FCStringAnsi::Strncpy(NewCallstack.Get(), StackTrace.Get(), Len);
		}
		Existing = new FCallstackData(MoveTemp(NewCallstack), Key.CppCallstackHash,
			Key.SerializedObject, Key.SerializedProperty);
	}
	return Existing;
}

void FCallstacks::Add(
	int64 Offset,
	int64 Length,
	UObject* SerializedObject,
	FProperty* SerializedProperty,
	TArrayView<const FName> DebugDataStack,
	bool bIsCollectingCallstacks,
	bool bCollectCurrentCallstack,
	int32 StackIgnoreCount)
{
	if (UE::ArchiveStackTrace::ShouldBypassDiff())
	{
		return;
	}
	++StackIgnoreCount;

	const int64 CurrentOffset = Offset;
	EndOffset = FMath::Max(EndOffset, CurrentOffset + Length); 

	const bool bShouldCollectCallstack = bIsCollectingCallstacks && bCollectCurrentCallstack && !UE::ArchiveStackTrace::ShouldIgnoreDiff();
	if (bShouldCollectCallstack)
	{
		StackTrace[0] = '\0';
		FPlatformStackWalk::StackWalkAndDump(StackTrace.Get(), StackTraceSize, StackIgnoreCount);
		//if we have a debug name stack, plaster it onto the end of the current stack buffer so that it's a part of the unique stack entry.
		if (DebugDataStack.Num() > 0)
		{
			FCStringAnsi::StrncatTruncateDest(StackTrace.Get(), StackTraceSize, DebugDataStackMarker);

			const FString SubIndent = FString(IndentToken) + FString(TEXT("    "));

			bool bIsIndenting = true;
			for (const auto& DebugData : DebugDataStack)
			{
				if (bIsIndenting)
				{
					FCStringAnsi::StrncatTruncateDest(StackTrace.Get(), StackTraceSize, TCHAR_TO_ANSI(*SubIndent));
				}

				ANSICHAR DebugName[NAME_SIZE];
				DebugData.GetPlainANSIString(DebugName);
				FCStringAnsi::StrncatTruncateDest(StackTrace.Get(), StackTraceSize, DebugName);

				//these are special-cased, as we assume they'll be followed by object/property names and want the names on the same line for readability's sake.
				const bool bIsPropertyLabel = (DebugData == TEXT("SerializeScriptProperties") || DebugData == TEXT("PropertySerialize") || DebugData == TEXT("SerializeTaggedProperty"));
				const ANSICHAR* const LineEnd = bIsPropertyLabel ? ": " : "\r\n";
				FCStringAnsi::StrncatTruncateDest(StackTrace.Get(), StackTraceSize, LineEnd);
				bIsIndenting = !bIsPropertyLabel;
			}
		}
		bCppCallstackDirty = true;
	}
	else
	{
		bCppCallstackDirty = StackTrace[0] != '\0';
		StackTrace[0] = '\0';
	}

	FCallstackKey CallstackKey;
	CallstackKey.SerializedObject = SerializedObject;
	CallstackKey.SerializedProperty = SerializedProperty;
	if (bCppCallstackDirty)
	{
		CallstackKey.CppCallstackHash = FCrc::StrCrc32(StackTrace.Get());
		bCppCallstackDirty = false;
		PreviousCppCallstackHash = CallstackKey.CppCallstackHash;
	}
	else
	{
		CallstackKey.CppCallstackHash = PreviousCppCallstackHash;
	}
	TRefCountPtr<FCallstackData> CallstackData = AddUniqueCallstack(CallstackKey);

	bool bSuppressLogging = UE::ArchiveStackTrace::ShouldIgnoreDiff();
	FCallstackAtOffset NewBlock{ CurrentOffset, Length, CurrentOffset, Length, CallstackData, bSuppressLogging };
	FCallstackAtOffset* LastBlock = CallstackAtOffsetMap.Num() ? &CallstackAtOffsetMap.Last() : nullptr;
	if (!LastBlock || CurrentOffset >= LastBlock->Offset + LastBlock->Length)
	{
		// New block serialized at the end of archive buffer
		CallstackAtOffsetMap.Add(NewBlock);
	}
	else
	{
		// This happens after a Seek(). We need to modify or replace the old block that covered the range written
		// by this new Serialize call.
		const int32 OldBlockIndex = GetCallstackIndexAtOffset(CurrentOffset);
		check(OldBlockIndex != -1);

		FCallstackAtOffset* OldBlock = &CallstackAtOffsetMap[OldBlockIndex];

		int64 OldEnd = OldBlock->Offset + OldBlock->Length;
		int64 NewEnd = NewBlock.Offset + NewBlock.Length;
		if (OldEnd <= NewEnd)
		{
			// The new block overwrites the end of the old block, and possibly overwrites blocks after it
			check(OldBlock->Offset <= NewBlock.Offset); // GetCallstackIndexAtOffset guarantees this
			bool bNewEntirelyContainsOld = OldBlock->Offset == NewBlock.Offset;
			int32 StartRemoveIndex;
			if (bNewEntirelyContainsOld)
			{
				// The new block completely overwrites the old block; delete the old block and replace it with the
				// the new block. Still need to check whether new also overwrites old blocks after the first.
				*OldBlock = NewBlock;
				StartRemoveIndex = OldBlockIndex + 1;
			}
			else
			{
				// The new block does not overwrite the old block, so keep the old block, clamp it to end at the
				// new block, and add the new block after it. Still need to check whether new also overwrites old
				// blocks after the first.
				// There might be a gap in between the end of old block and the beginning of the new block. This
				// can occur when we are not recording every serialize call. Leave the old block unmodified in
				// that case.
				if (OldEnd > NewBlock.Offset)
				{
					OldBlock->Length = NewBlock.Offset - OldBlock->Offset;
				}
				CallstackAtOffsetMap.Insert(NewBlock, OldBlockIndex + 1);
				OldBlock = nullptr; // Our pointer for OldBlock is now possibly invalidated by reallocation.
				StartRemoveIndex = OldBlockIndex + 2;
			}
			int32 EndRemoveIndex = StartRemoveIndex;
			while (EndRemoveIndex < CallstackAtOffsetMap.Num())
			{
				OldBlock = &CallstackAtOffsetMap[EndRemoveIndex];
				if (OldBlock->Offset >= NewEnd)
				{
					break;
				}
				OldEnd = OldBlock->Offset + OldBlock->Length;
				if (OldEnd > NewEnd)
				{
					// The beginning of this followup block is overwritten by the new block; shorten it
					OldBlock->Length = OldEnd - NewEnd;
					OldBlock->Offset = NewEnd;
					break;
				}
				else
				{
					// This followup block is completely inside the new block; mark it for delete and move to next.
					++EndRemoveIndex;
				}
			}
			if (EndRemoveIndex > StartRemoveIndex)
			{
				CallstackAtOffsetMap.RemoveAt(StartRemoveIndex, EndRemoveIndex - StartRemoveIndex,
					EAllowShrinking::No);
			}
		}
		else // OldEnd > NewEnd
		{
			// The new block is completely inside the old block, which extends after it. Shorten the beginning of
			// the old block, and add an additional new block containing the portion of the old block that extends
			// after the new block.
			FCallstackAtOffset SegmentAfterNewBlock = *OldBlock;
			SegmentAfterNewBlock.Offset = NewEnd;
			SegmentAfterNewBlock.Length = OldEnd - NewEnd;
			OldBlock->Length = NewBlock.Offset - OldBlock->Offset;
			CallstackAtOffsetMap.Insert(NewBlock, OldBlockIndex + 1);
			CallstackAtOffsetMap.Insert(SegmentAfterNewBlock, OldBlockIndex + 2);
			OldBlock = nullptr; // Our pointer for OldBlock is now possibly invalidated by reallocation.
		}
	}
}

int32 FCallstacks::GetCallstackIndexAtOffset(int64 Offset, int32 MinOffsetIndex, int64* OutOffsetEnd) const
{
	if (Offset < 0 || Offset >= EndOffset || MinOffsetIndex >= CallstackAtOffsetMap.Num())
	{
		if (OutOffsetEnd)
		{
			(*OutOffsetEnd) = -1;
		}
		return -1;
	}

	// Find the index of the offset the InOffset maps to
	int32 OffsetForCallstackIndex = -1;
	MinOffsetIndex = FMath::Max(MinOffsetIndex, 0);
	int32 MaxOffsetIndex = CallstackAtOffsetMap.Num() - 1;

	// Binary search
	for (; MinOffsetIndex <= MaxOffsetIndex; )
	{
		int32 SearchIndex = (MinOffsetIndex + MaxOffsetIndex) / 2;
		if (CallstackAtOffsetMap[SearchIndex].Offset < Offset)
		{
			MinOffsetIndex = SearchIndex + 1;
		}
		else if (CallstackAtOffsetMap[SearchIndex].Offset > Offset)
		{
			MaxOffsetIndex = SearchIndex - 1;
		}
		else
		{
			OffsetForCallstackIndex = SearchIndex;
			break;
		}
	}
	
	if (OffsetForCallstackIndex == -1)
	{
		// We didn't find the exact offset value so let's try to find the first one that is lower than the requested one
		MinOffsetIndex = FMath::Min(MinOffsetIndex, CallstackAtOffsetMap.Num() - 1);
		for (int32 FirstLowerOffsetIndex = MinOffsetIndex; FirstLowerOffsetIndex >= 0; --FirstLowerOffsetIndex)
		{
			if (CallstackAtOffsetMap[FirstLowerOffsetIndex].Offset < Offset)
			{
				OffsetForCallstackIndex = FirstLowerOffsetIndex;
				break;
			}
		}
		if (OffsetForCallstackIndex != -1)
		{
			check(CallstackAtOffsetMap[OffsetForCallstackIndex].Offset < Offset);
			check(OffsetForCallstackIndex == (CallstackAtOffsetMap.Num() - 1) || CallstackAtOffsetMap[OffsetForCallstackIndex + 1].Offset > Offset);
		}
	}

	if (OutOffsetEnd)
	{
		if (OffsetForCallstackIndex != -1)
		{
			(*OutOffsetEnd) = CallstackAtOffsetMap[OffsetForCallstackIndex].Offset + CallstackAtOffsetMap[OffsetForCallstackIndex].Length - 1;
		}
		else
		{
			(*OutOffsetEnd) = -1;
		}
	}

	return OffsetForCallstackIndex;
}

void FCallstacks::RemoveRange(int64 StartOffset, int64 Length)
{
	CallstackAtOffsetMap.RemoveAll([StartOffset, Length](const FCallstackAtOffset& Entry)
		{
			return StartOffset <= Entry.Offset && Entry.Offset < StartOffset + Length;
		});
}

void FCallstacks::Append(const FCallstacks& Other, int64 OtherStartOffset)
{
	for (const FCallstackAtOffset& OtherOffset : Other.CallstackAtOffsetMap)
	{
		FCallstackAtOffset& New = CallstackAtOffsetMap.Add_GetRef(OtherOffset);
		New.Offset += OtherStartOffset;
		New.SerializeCallOffset += OtherStartOffset;
		if (New.Callstack)
		{
			FCallstackKey Key = New.Callstack->GetKey();
			TRefCountPtr<FCallstackData>& Existing = UniqueCallstacks.FindOrAdd(Key);
			if (!Existing)
			{
				Existing = New.Callstack->Clone();
			}
			New.Callstack = Existing;
		}
	}

	CallstackAtOffsetMap.Sort([](const FCallstackAtOffset& LHS,const FCallstackAtOffset& RHS)
	{
		return LHS.Offset < RHS.Offset;
	});

	EndOffset = FMath::Max(EndOffset, Other.EndOffset + OtherStartOffset);
}

struct FBreakAtOffsetSettings
{
	FString PackageToBreakOn;
	int64 OffsetToBreakOn = -1;
	bool bInitialized = false;

	void Initialize()
	{
		if (bInitialized)
		{
			return;
		}
		bInitialized = true;
		OffsetToBreakOn = -1;
		PackageToBreakOn.Empty();

		if (!FParse::Param(FCommandLine::Get(), TEXT("cooksinglepackage")) &&
			!FParse::Param(FCommandLine::Get(), TEXT("cooksinglepackagenorefs")))
		{
			return;
		}

		FString Package;
		if (!FParse::Value(FCommandLine::Get(), TEXT("map="), Package) &&
			!FParse::Value(FCommandLine::Get(), TEXT("package="), Package))
		{
			return;
		}

		int64 Offset;
		// DiffOnlyBreakOffset should be the Combined/DiffBreak Offset from the warning message
		if (!FParse::Value(FCommandLine::Get(), TEXT("diffonlybreakoffset="), Offset) || Offset <= 0)
		{
			return;
		}

		OffsetToBreakOn = Offset;
		PackageToBreakOn = TEXT("/") + FPackageName::GetShortName(Package);
	}

	bool MatchesFilename(const FString& Filename) const
	{
		int32 SubnameIndex = Filename.Find(PackageToBreakOn, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (SubnameIndex < 0)
		{
			return false;
		}
		int32 SubnameEndIndex = SubnameIndex + PackageToBreakOn.Len();
		return SubnameEndIndex == Filename.Len() || Filename[SubnameEndIndex] == TEXT('.');
	}

} GBreakAtOffsetSettings;


void FCallstacks::RecordSerialize(EOffsetFrame OffsetFrame, int64 CurrentOffset, int64 Length,
	const FAccumulator& Accumulator, FDiffArchive& Ar, int32 StackIgnoreCount)
{
	int64 LinkerOffset = -1;

	// If the writer is using postsave transforms, then we need to know what segment we are in before deciding whether
	// to allow use of the LinkerOffset for diffonlybreakoffset or for recording the LinkerOffset for each callstack.
	// The segment information is only known after the first save.
	if (!Accumulator.IsWriterUsingPostSaveTransforms() || Accumulator.bFirstSaveComplete)
	{
		switch (OffsetFrame)
		{
		case EOffsetFrame::Linker:
			LinkerOffset = CurrentOffset;
			break;
		case EOffsetFrame::Exports:
			if (Accumulator.bFirstSaveComplete)
			{
				LinkerOffset = CurrentOffset + Accumulator.HeaderSize;
			}
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	++StackIgnoreCount;

	if (LinkerOffset >= 0)
	{
		if (GBreakAtOffsetSettings.OffsetToBreakOn >= 0 &&
			LinkerOffset <= GBreakAtOffsetSettings.OffsetToBreakOn && GBreakAtOffsetSettings.OffsetToBreakOn < LinkerOffset + Length)
		{
			if (GBreakAtOffsetSettings.MatchesFilename(Accumulator.Filename))
			{
				if (Accumulator.IsWriterUsingPostSaveTransforms() && OffsetFrame == EOffsetFrame::Linker &&
					LinkerOffset < Accumulator.PreTransformHeaderSize)
				{
					// Do not break at this serialize call; it's in the pre-transformed header. If the requested
					// breakoffset is not within the post-transformed header, then this offset is in the wrong segment
					// and is not a match. If the breakoffset is within the post-transformed header we break and give
					// instructions during OnFirstSaveComplete.
				}
				else
				{
					if (!UE::ArchiveStackTrace::ShouldBypassDiff() && !UE::ArchiveStackTrace::ShouldIgnoreDiff())
					{
						UE_DEBUG_BREAK();
					}
				}
			}
		}
	}

	if (Length > 0)
	{
		UObject* SerializedObject = FUObjectThreadContext::Get().GetSerializeContext()->SerializedObject;
		TArrayView<const FName> DebugStack = Ar.GetDebugDataStack();

		const bool bCollectingCallstacks = Accumulator.bFirstSaveComplete;
		const bool bCollectCurrentCallstack = bCollectingCallstacks && LinkerOffset >= 0 &&
			(!Accumulator.IsWriterUsingPostSaveTransforms() || LinkerOffset >= Accumulator.HeaderSize) &&
			Accumulator.DiffMap.ContainsOffset(LinkerOffset);

		Add(CurrentOffset, Length, SerializedObject, Ar.GetSerializedProperty(), DebugStack, bCollectingCallstacks,
			bCollectCurrentCallstack, StackIgnoreCount);
	}
}

FAccumulatorGlobals::FAccumulatorGlobals(ICookedPackageWriter* InnerPackageWriter)
	: PackageWriter(InnerPackageWriter)
{
}

void FAccumulatorGlobals::Initialize(EPackageHeaderFormat InFormat)
{
	if (bInitialized)
	{
		return;
	}
	bInitialized = true;
	Format = InFormat;
	switch (Format)
	{
	case EPackageHeaderFormat::PackageFileSummary:
		break;
	case EPackageHeaderFormat::ZenPackageSummary:
		FPackageStoreOptimizer::FindScriptObjects(ScriptObjectsMap);
		break;
	default:
		checkNoEntry();
		break;
	}
}

FDiffOutputRecorder::FDiffOutputRecorder( FMessageCallback&& InMessageCallback, IDetailRecorder* InDetailRecorder )
	: MessageCallback(MoveTemp(InMessageCallback))
	, DetailRecorder(InDetailRecorder)
{
}

void FDiffOutputRecorder::RecordHeaderSizeMismatch( const TCHAR* Filename, int64 FirstHeaderSize, int64 SecondHeaderSize )
{
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: Indeterministic header size. When saving the package twice into memory, first header size %" INT64_FMT " != second header size %" INT64_FMT ". Callstacks for indeterminism in the exports will be incorrect.")
		TEXT("\n\tDumping differences from first and second memory saves."),
		Filename, FirstHeaderSize, SecondHeaderSize));
}

void FDiffOutputRecorder::RecordUndiagnosedHeaderDifference( const TCHAR* Filename )
{
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: headers are different, but DumpPackageHeaderDiffs does not yet implement describing the difference."),
		Filename
		));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordUndiagnosedDiff();
	}
}

void FDiffOutputRecorder::RecordTableDifferences(const TCHAR* Filename, const TCHAR* ItemName, int32 SourceTableNum, 
	int32 DestTableNum, const TCHAR* HumanReadableString)
{
	//SourceContext.LogMessage(ELogVerbosity::Warning, FString::Printf(
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: %s is different (%d %ss in source package vs %d %ss in dest package):%s%s"),
		Filename,
		ItemName,
		SourceTableNum,
		ItemName,
		DestTableNum,
		ItemName,
		NewLineToken,
		HumanReadableString));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordTableDifferences(ItemName, HumanReadableString);
	}
}

void FDiffOutputRecorder::RecordTableDifferences(const TCHAR* Filename, const TCHAR* ItemName, 
	const TCHAR* HumanReadableString)
{
	//LogMessage(ELogVerbosity::Warning, FString::Printf(
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: %s is different:%s%s"),
		Filename,
		ItemName,
		NewLineToken,
		HumanReadableString));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordTableDifferences(ItemName, HumanReadableString);
	}
}

void FDiffOutputRecorder::RecordSectionSizeMismatch(const TCHAR* SectionFilename, int64 SourceSize, int64 DestSize)
{
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: Size mismatch: on disk: %lld vs memory: %lld"), SectionFilename, SourceSize, DestSize));
}

void FDiffOutputRecorder::RecordDiff(const TCHAR* SectionFilename, int64 LocalOffset, int64 DestAbsoluteOffset, 
	uint8 SourceByte, uint8 DestByte, bool bHasOptimizedHeader)
{
	if (bHasOptimizedHeader)
	{
		MessageCallback(ELogVerbosity::Warning, FString::Printf(
			TEXT("%s: Difference at offset %lld (Combined/DiffBreak Offset: %" INT64_FMT "): OnDisk %d != %d InMemory.%s")
			TEXT("Callstack is unknown because the offset is in the header and the header has been optimized. See the output of DumpPackageHeaderDiffs to debug this difference."),
			SectionFilename, LocalOffset, DestAbsoluteOffset, SourceByte, DestByte, NewLineToken
		));
	}
	else
	{
		MessageCallback(ELogVerbosity::Warning, FString::Printf(
			TEXT("%s: Difference at offset %lld (Combined/DiffBreak Offset: %" INT64_FMT "): OnDisk %d != %d InMemory.%s")
			TEXT("Callstack is unknown."),
			SectionFilename, LocalOffset, DestAbsoluteOffset, SourceByte, DestByte, NewLineToken
		));
	}

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordDiff(LocalOffset, nullptr);
	}
}

void FDiffOutputRecorder::RecordDiff(const TCHAR* SectionFilename, int64 LocalOffset, int64 DestAbsoluteOffset, 
	uint8 SourceByte, uint8 DestByte, int64 DifferenceOffset, const TCHAR* LastDifferenceCallstackDataText, 
	const FString& BeforePropertyVal, const FString& AfterPropertyVal, 
	const FCallstacks::FCallstackData& DifferenceCallstackData)
{
	FString DiffValues;
	if (BeforePropertyVal != AfterPropertyVal)
	{
		DiffValues = FString::Printf(TEXT("\r\n%sBefore: %s\r\n%sAfter:  %s"),
			IndentToken, *BeforePropertyVal,
			IndentToken, *AfterPropertyVal);
	}


	FString DebugDataStackText;
	//check for a debug data stack as part of the unique stack entry, and log it out if we find it.
	FString FullStackText = DifferenceCallstackData.Callstack.Get();
	int32 DebugDataIndex = FullStackText.Find(ANSI_TO_TCHAR(DebugDataStackMarker), ESearchCase::CaseSensitive);
	if (DebugDataIndex > 0)
	{
		DebugDataStackText = FString::Printf(TEXT("\r\n%s"), IndentToken)
			+ FullStackText.RightChop(DebugDataIndex + 2);
	}

	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: Difference at offset %lld (Combined/DiffBreak Offset: %" INT64_FMT "): OnDisk %d != %d InMemory.%s")
			TEXT("Difference occurs at index %lld within Serialize call at callstack:%s%s%s%s"),
		SectionFilename, LocalOffset, DestAbsoluteOffset, SourceByte, DestByte, NewLineToken,
		DifferenceOffset, NewLineToken,
		LastDifferenceCallstackDataText, *DiffValues, *DebugDataStackText
	));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordDiff(LocalOffset, &DifferenceCallstackData);
	}
}

void FDiffOutputRecorder::ExtendPreviousDiff(int64 LocalOffset)
{
	if (DetailRecorder != nullptr)
	{
		DetailRecorder->ExtendPreviousDiff(LocalOffset);
	}
}

void FDiffOutputRecorder::IncrementPreviousDiff()
{
	if (DetailRecorder != nullptr)
	{
		DetailRecorder->IncrementPreviousDiff();
	}
}

void FDiffOutputRecorder::RecordDiffBytes(const TCHAR* SectionFilename, int32 BytesToLog, int64 LocalOffset, 
	const FPackageData& SourcePackage, const FPackageData& DestPackage)
{
	MessageCallback(ELogVerbosity::Display, FString::Printf(
		TEXT("%s: Logging %d bytes around offset: %" INT64_FMT " (%016" INT64_X_FMT ") in the OnDisk package:"),
		SectionFilename,
		BytesToLog, LocalOffset, LocalOffset
	));
	TArray<FString> HexDumpLines = FCompressionUtil::HexDumpLines(SourcePackage.Data + SourcePackage.StartOffset,
		SourcePackage.Size - SourcePackage.StartOffset,
		LocalOffset - BytesToLog / 2, LocalOffset + BytesToLog / 2);
	for (FString& Line : HexDumpLines)
	{
		MessageCallback(ELogVerbosity::Display, Line);
	}

	MessageCallback(ELogVerbosity::Display, FString::Printf(
		TEXT("%s: Logging %d bytes around offset: %" INT64_FMT " (%016" INT64_X_FMT ") in the InMemory package:"),
		SectionFilename, BytesToLog, LocalOffset, LocalOffset
	));
	HexDumpLines = FCompressionUtil::HexDumpLines(DestPackage.Data + DestPackage.StartOffset,
		DestPackage.Size - DestPackage.StartOffset,
		LocalOffset - BytesToLog / 2, LocalOffset + BytesToLog / 2);
	for (FString& Line : HexDumpLines)
	{
		MessageCallback(ELogVerbosity::Display, Line);
	}

}

void FDiffOutputRecorder::RecordUnreportedDiffs(const TCHAR* SectionFilename, int32 NumUnreportedDiffs, 
	int32 FirstUnreportedDiffIndex)
{
	MessageCallback(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: %d difference(s) not logged (first at offset: %d)."),
		SectionFilename,NumUnreportedDiffs, FirstUnreportedDiffIndex));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->RecordUnreportedDiffs(NumUnreportedDiffs);
	}
}

void FDiffOutputRecorder::RecordDeterminismDiagnostics(const FString& DeterminismLines)
{
	if (!DeterminismLines.IsEmpty())
	{
		if (DetailRecorder != nullptr)
		{
			DetailRecorder->RecordDetermismDiagnostics(*DeterminismLines);
		}

		MessageCallback(ELogVerbosity::Display, FString(TEXT("DeterminismHelper Diagnostics:\n") + DeterminismLines));
	}
}

void FDiffOutputRecorder::RecordNewPackage(const TCHAR* Filename)
{
	MessageCallback(ELogVerbosity::Warning, FString::Printf(TEXT("New package: %s"), Filename));
}

void FDiffOutputRecorder::BeginPackage(const TCHAR* Filename, const TCHAR* Classname)
{
	MessageCallback(ELogVerbosity::Display, FString::Printf(TEXT("Comparing: %s"), Filename));
	MessageCallback(ELogVerbosity::Warning, FString::Printf(TEXT("Asset class: %s"), Classname));

	if (DetailRecorder != nullptr)
	{
		DetailRecorder->BeginPackage();
	}
}
void FDiffOutputRecorder::BeginSection(const TCHAR* SectionFilename, EDiffWriterSectionType SectionType, 
	const FPackageData& SourceSection, const FPackageData& DestSection)
{
	if (DetailRecorder != nullptr)
	{
		DetailRecorder->BeginSection(SectionFilename, SectionType, SourceSection, DestSection);
	}
}
void FDiffOutputRecorder::EndSection()
{
	if (DetailRecorder != nullptr)
	{
		DetailRecorder->EndSection();
	}
}
void FDiffOutputRecorder::EndPackage(const TCHAR* Filename, const FPackageData& SourcePackage, const FPackageData& DestPackage, 
	const TCHAR* ClassName)
{
	if (DetailRecorder != nullptr)
	{
		DetailRecorder->EndPackage(Filename, SourcePackage, DestPackage, ClassName);
	}
}



const FMessageCallback& FDiffOutputRecorder::GetMessageCallback() const
{
	return MessageCallback;
}

IDetailRecorder* FDiffOutputRecorder::GetDetailRecorder() const
{
	return DetailRecorder;
}




FAccumulator::FAccumulator(FAccumulatorGlobals& InGlobals, UObject* InAsset, FName InPackageName,
	int32 InMaxDiffsToLog, bool bInIgnoreHeaderDiffs, TSharedPtr<FDiffOutputRecorder> InDiffOutputRecorder,
	EPackageHeaderFormat InPackageHeaderFormat)
	: LinkerCallstacks()
	, ExportsCallstacks()
	, Globals(InGlobals)
	, DiffOutputRecorder(InDiffOutputRecorder)
	, PackageName(InPackageName)
	, Asset(InAsset)
	, MaxDiffsToLog(InMaxDiffsToLog)
	, PackageHeaderFormat(InPackageHeaderFormat)
	, bIgnoreHeaderDiffs(bInIgnoreHeaderDiffs)
{
	GBreakAtOffsetSettings.Initialize();
}

FAccumulator::~FAccumulator()
{
}

void FAccumulator::SetHeaderSize(int64 InHeaderSize)
{
	HeaderSize = InHeaderSize;
}

void FAccumulator::SetDeterminismManager(UE::Cook::FDeterminismManager& InDeterminismManager)
{
	DeterminismManager = &InDeterminismManager;
}

FName FAccumulator::GetAssetClass() const
{
	return Asset != nullptr ? Asset->GetClass()->GetFName() : NAME_None;
}

bool FAccumulator::IsWriterUsingPostSaveTransforms() const
{
	return PackageHeaderFormat != EPackageHeaderFormat::PackageFileSummary;
}

void FAccumulator::OnFirstSaveComplete(FStringView LooseFilePath, int64 InHeaderSize, int64 InPreTransformHeaderSize,
	ICookedPackageWriter::FPreviousCookedBytesData&& InPreviousPackageData)
{
	Filename = LooseFilePath;
	HeaderSize = InHeaderSize;
	PreTransformHeaderSize = InPreTransformHeaderSize;
	PreviousPackageData = MoveTemp(InPreviousPackageData);

	if (IsWriterUsingPostSaveTransforms())
	{
		// The header has been transformed, so all callstacks in it are invalid; remove them
		LinkerCallstacks.RemoveRange(0, PreTransformHeaderSize);
	}
	LinkerCallstacks.Append(ExportsCallstacks, HeaderSize);
	ExportsCallstacks.Reset();

	GenerateDiffMap();
	if (HasDifferences())
	{
		// Make a copy of the LinkerArchive for comparison in case it differs even in the second memory save
		check(LinkerArchive);
		FirstSaveLinkerData.Empty(LinkerArchive->TotalSize());
		FirstSaveLinkerData.Append(LinkerArchive->GetData(), LinkerArchive->TotalSize());
	}

	LinkerCallstacks.Reset();
	bFirstSaveComplete = true;


	if (IsWriterUsingPostSaveTransforms() &&
		GBreakAtOffsetSettings.MatchesFilename(Filename) &&
		GBreakAtOffsetSettings.OffsetToBreakOn < HeaderSize)
	{
		// The package writer used for this cook transforms the header before saving it to disk, for e.g. compression.
		// This means that we don't in general know which offsets in the pre-transform header correspond to
		// the offsets in the header on disk, and we only know the callstack for the offsets in the pre-transform
		// header. So we don't know where to break during serialization of the header.
		// If you specified settings to break at an offset in the pre-transform header, you need instead to debug
		// using the information reported by DumpPackageHeaderDiffs.
		UE_DEBUG_BREAK();
	}
}

void FAccumulator::OnSecondSaveComplete(int64 InHeaderSize)
{
	Globals.Initialize(PackageHeaderFormat);

	check(bFirstSaveComplete); // Should have been set in OnFirstSaveComplete
	if (HeaderSize != InHeaderSize)
	{
		check(bFirstSaveComplete);
		check(FirstSaveLinkerData.Num() >= HeaderSize);
		check(LinkerArchive && LinkerArchive->TotalSize() >= InHeaderSize);

		FPackageData FirstSaveHeaderSegment{ FirstSaveLinkerData.GetData(), HeaderSize};
		FPackageData SecondSaveHeaderSegment{ LinkerArchive->GetData(), InHeaderSize };
		int32 NumHeaderDiffMessages = 0;
		FMessageCallback HeaderMessageCallback = [&NumHeaderDiffMessages, this](ELogVerbosity::Type Verbosity, FStringView Message)
			{
				DiffOutputRecorder->GetMessageCallback()(Verbosity, Message);
				++NumHeaderDiffMessages;
			};
        // Note: we do not record to the DetailRecorder when reporting the header differences between the first and second save.
        // The DetailRecorder is reporting the differences between the first save and the previous cook's save.
		TSharedRef<FDiffOutputRecorder> HeaderDiffOutputRecorder = MakeShared<FDiffOutputRecorder>(
			MoveTemp(HeaderMessageCallback), nullptr);

		HeaderDiffOutputRecorder->RecordHeaderSizeMismatch(*this->Filename, HeaderSize, InHeaderSize);

		FPackageHeaderData FirstSaveHeader(TEXT("source"), false /* bReadFromPackageStore */, Filename,
			FirstSaveHeaderSegment, PackageHeaderFormat, Globals, HeaderDiffOutputRecorder);
		FPackageHeaderData SecondSaveHeader(TEXT("dest"), false /* bReadFromPackageStore */, Filename,
			SecondSaveHeaderSegment, PackageHeaderFormat, Globals, HeaderDiffOutputRecorder);

		DumpPackageHeaderDiffs(FirstSaveHeader, SecondSaveHeader, MaxDiffsToLog);
		if (NumHeaderDiffMessages == 0)
		{
			HeaderDiffOutputRecorder->RecordUndiagnosedHeaderDifference(*Filename);
		}
	}

	if (IsWriterUsingPostSaveTransforms())
	{
		// The header has been transformed, so all callstacks in it are invalid; remove them
		LinkerCallstacks.RemoveRange(0, PreTransformHeaderSize);
	}
	LinkerCallstacks.Append(ExportsCallstacks, HeaderSize);
	ExportsCallstacks.Reset();
}

bool FAccumulator::HasDifferences() const
{
	return bHasDifferences;
}

FDiffArchive::FDiffArchive(FAccumulator& InAccumulator)
	: Accumulator(&InAccumulator)
{
	SetIsPersistent(true /* bIsPersistent */);
}

FString FDiffArchive::GetArchiveName() const
{
	return Accumulator->Filename;
}

void FDiffArchive::PushDebugDataString(const FName& DebugData)
{
	DebugDataStack.Push(DebugData);
}

void FDiffArchive::PopDebugDataString()
{
	DebugDataStack.Pop();
}

TArray<FName>& FDiffArchive::GetDebugDataStack()
{
	return DebugDataStack;
}

FDiffArchiveForLinker::FDiffArchiveForLinker(FAccumulator& InAccumulator)
	: FDiffArchive(InAccumulator)
{
	// SavePackage is supposed to destroy the archive before returning, we rely on that so that the linkerarchive
	// in the second save can use the same data that was used in the first save
	check(!Accumulator->LinkerArchive);
	Accumulator->LinkerArchive = this;
}

FDiffArchiveForLinker::~FDiffArchiveForLinker()
{
	check(Accumulator->LinkerArchive == this)
	Accumulator->LinkerArchive = nullptr;
}

void FDiffArchiveForLinker::Serialize(void* InData, int64 Num)
{
	int32 StackIgnoreCount = 1;
	int64 CurrentOffset = Tell();
	Accumulator->LinkerCallstacks.RecordSerialize(EOffsetFrame::Linker, CurrentOffset, Num, *Accumulator,
		*this, StackIgnoreCount);
	FDiffArchive::Serialize(InData, Num);
}

FDiffArchiveForExports::FDiffArchiveForExports(FAccumulator& InAccumulator)
	: FDiffArchive(InAccumulator)
{
	// SavePackage is supposed to destroy the archive before returning, we rely on that so that the exportsarchive
	// in the second save can use the same data that was used in the first save
	check(!Accumulator->ExportsArchive);
	Accumulator->ExportsArchive = this;
}

FDiffArchiveForExports::~FDiffArchiveForExports()
{
	check(Accumulator->ExportsArchive == this);
	Accumulator->ExportsArchive = nullptr;
}

void FDiffArchiveForExports::Serialize(void* InData, int64 Num)
{
	int32 StackIgnoreCount = 1;
	int64 CurrentOffset = Tell();
	Accumulator->ExportsCallstacks.RecordSerialize(EOffsetFrame::Exports, CurrentOffset, Num, *Accumulator,
		*this, StackIgnoreCount);
	FDiffArchive::Serialize(InData, Num);
}

bool ShouldDumpPropertyValueState(FProperty* Prop)
{
	if (Prop->IsA<FNumericProperty>()
		|| Prop->IsA<FStrProperty>()
		|| Prop->IsA<FBoolProperty>()
		|| Prop->IsA<FNameProperty>())
	{
		return true;
	}

	if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
	{
		return ShouldDumpPropertyValueState(ArrayProp->Inner);
	}

	if (FMapProperty* MapProp = CastField<FMapProperty>(Prop))
	{
		return ShouldDumpPropertyValueState(MapProp->KeyProp) && ShouldDumpPropertyValueState(MapProp->ValueProp);
	}

	if (FSetProperty* SetProp = CastField<FSetProperty>(Prop))
	{
		return ShouldDumpPropertyValueState(SetProp->ElementProp);
	}

	if (FStructProperty* StructProp = CastField<FStructProperty>(Prop))
	{
		if (StructProp->Struct == TBaseStructure<FVector>::Get()
			|| StructProp->Struct == TBaseStructure<FGuid>::Get())
		{
			return true;
		}
	}

	if (FOptionalProperty* OptionalProp = CastField<FOptionalProperty>(Prop))
	{
		return ShouldDumpPropertyValueState(OptionalProp->GetValueProperty());
	}

	return false;
}

void FAccumulator::CompareWithPreviousForSection(const FPackageData& SourcePackage, const FPackageData& DestPackage,
	FPackageHeaderData& SourceHeader, FPackageHeaderData& DestHeader,
	const TCHAR* CallstackCutoffText, int32& InOutDiffsLogged, TMap<FName, FArchiveDiffStats>& OutStats,
	const FString& SectionFilename)
{
	FCallstacks& Callstacks = this->LinkerCallstacks;
	const int64 SourceSize = SourcePackage.Size - SourcePackage.StartOffset;
	const int64 DestSize = DestPackage.Size - DestPackage.StartOffset;
	const int64 SizeToCompare = FMath::Min(SourceSize, DestSize);
	const FName AssetClass = GetAssetClass();
	
	FString LastDifferenceCallstackDataText;
	int32 LastDifferenceCallstackOffsetIndex = -1;
	int64 NumDiffsLocal = 0;
	int64 NumDiffsForLogStatLocal = 0;
	int64 NumDiffsLoggedLocal = 0;
	int64 FirstUnreportedDiffIndex = -1;

    // When we discover the first byte difference within a serialize call, we record the offset of the
    // end of the serialize call so we can record what range of bytes within the serialize call was
    // different.
    // We record some variables about the offset in the Local reference frame (from the beginning of the
    // current Section), and some in the Absolute reference frame (from the beginning of the Package that
    // contains all of the Sections).

    /**
     * If non-negative, we have found a diff and this is the offset to the end of the current serialize call. */
	int64 DestAbsoluteEndOffsetForActiveDiff = -1;

   /**
     * If non-negative, we have found a diff. The diff spans one or more bytes. This is the offset to the last byte
     * we have reached that was different, so we can record the range of bytes that differed. We reset this and start
     * a new diff report when we reach the end of the Serialize call in which we found the diff.
     */
	int64 ActiveDiffLocalOffset = -1;

    /**
     * If non-negative, we have found a diff and the diff has the same callstack repeated from the previous diff,
     * but the callstack was called repeatededly and we recorded a diff in the repeated call. This is the offset
     * to the end of the current serialize call for the repeated callstack.
     */
	int64 DestAbsoluteEndOffsetForDuplicateDiff = -1;

	for (int64 LocalOffset = 0; LocalOffset < SizeToCompare; ++LocalOffset)
	{
		const int64 SourceAbsoluteOffset = LocalOffset + SourcePackage.StartOffset;
		const int64 DestAbsoluteOffset = LocalOffset + DestPackage.StartOffset;
		if (DestAbsoluteEndOffsetForActiveDiff != -1 && DestAbsoluteOffset > DestAbsoluteEndOffsetForActiveDiff)
		{
			DiffOutputRecorder->ExtendPreviousDiff(ActiveDiffLocalOffset);
			DestAbsoluteEndOffsetForActiveDiff = -1;
			ActiveDiffLocalOffset = -1;
		}
		if (DestAbsoluteEndOffsetForDuplicateDiff != -1 && DestAbsoluteOffset > DestAbsoluteEndOffsetForDuplicateDiff)
		{
			DiffOutputRecorder->IncrementPreviousDiff();
			DestAbsoluteEndOffsetForDuplicateDiff = -1;
		}

		const uint8 SourceByte = SourcePackage.Data[SourceAbsoluteOffset];
		const uint8 DestByte   = DestPackage  .Data[DestAbsoluteOffset];
		if (SourceByte == DestByte)
		{
			continue;
		}
		ActiveDiffLocalOffset = LocalOffset;

		constexpr int BytesToLog = 128;
		int64 DestAbsoluteEndOffset = -1;
		int32 DifferenceCallstackOffsetIndex = Callstacks.GetCallstackIndexAtOffset(DestAbsoluteOffset,
			LastDifferenceCallstackOffsetIndex < 0 ? 0 : LastDifferenceCallstackOffsetIndex, &DestAbsoluteEndOffset);

		// Skip reporting the difference if we are still within the same Serialize call, or for any bytes after
		// the first different byte if we don't know the callstack.
		if (NumDiffsLocal > 0 &&
			DifferenceCallstackOffsetIndex == LastDifferenceCallstackOffsetIndex)
		{
			continue;
		}

		// Also skip reporting the difference if it is another occurrence of the last reported callstack
		const FCallstacks::FCallstackAtOffset* CallstackAtOffsetPtr = nullptr;
		const FCallstacks::FCallstackData* DifferenceCallstackDataPtr = nullptr;
		FString DifferenceCallstackDataText;
		bool bCallstackSuppressLogging = false;
		if (DifferenceCallstackOffsetIndex >= 0)
		{
			CallstackAtOffsetPtr = &Callstacks.GetCallstack(DifferenceCallstackOffsetIndex);
			bCallstackSuppressLogging = CallstackAtOffsetPtr->bSuppressLogging;

			DifferenceCallstackDataPtr = Callstacks.GetCallstackData(*CallstackAtOffsetPtr).GetReference();
			DifferenceCallstackDataText = DifferenceCallstackDataPtr->ToString(CallstackCutoffText);
			if (NumDiffsLocal > 0 && 
				LastDifferenceCallstackDataText.Compare(DifferenceCallstackDataText, ESearchCase::CaseSensitive) == 0)
			{
				DestAbsoluteEndOffsetForDuplicateDiff = DestAbsoluteEndOffset;
				continue;
			}
		}

		// Update counter for number of existing diffs
		OutStats.FindOrAdd(AssetClass).NumDiffs++;
		NumDiffsLocal++;
		// Update LastReported fields
		LastDifferenceCallstackOffsetIndex = DifferenceCallstackOffsetIndex;
		LastDifferenceCallstackDataText = MoveTemp(DifferenceCallstackDataText);

		// Skip logging of the difference if the callstack has a suppresslogging scope 
		if (bCallstackSuppressLogging)
		{
			return;
		}
		// Skip logging of header differences if requested
		bool bIsHeaderDiff = DestAbsoluteOffset < HeaderSize;
		if (bIgnoreHeaderDiffs && bIsHeaderDiff)
		{
			continue;
		}

		// Update counter for number of diffs that should be reported as existing when over the limit
		// Ignored header diffs and suppressed callstacks do not contribute to this count
		NumDiffsForLogStatLocal++;

		// Skip logging of the difference if we are over the limit
		if (bCallstackSuppressLogging || (MaxDiffsToLog >= 0 && InOutDiffsLogged >= MaxDiffsToLog))
		{
			if (FirstUnreportedDiffIndex == -1)
			{
				FirstUnreportedDiffIndex = LocalOffset;
			}
			continue;
		}

		// Update counter for number of logged diffs
		InOutDiffsLogged++;
		NumDiffsLoggedLocal++;

		if (DifferenceCallstackOffsetIndex < 0)
		{
			const bool bHasOptimizedHeader = (IsWriterUsingPostSaveTransforms() && DestAbsoluteOffset < HeaderSize);
			DiffOutputRecorder->RecordDiff(*SectionFilename, LocalOffset, DestAbsoluteOffset, SourceByte, DestByte, bHasOptimizedHeader);
		}
		else
		{
			check(CallstackAtOffsetPtr && DifferenceCallstackDataPtr); // These were set up above.
			const FCallstacks::FCallstackAtOffset& CallstackAtOffset = *CallstackAtOffsetPtr;
			const FCallstacks::FCallstackData& DifferenceCallstackData = *DifferenceCallstackDataPtr;
			if (DeterminismManager)
			{
				DeterminismManager->RecordExportModified(DifferenceCallstackData.GetObjectName());
			}

			FString BeforePropertyVal;
			FString AfterPropertyVal;
			FProperty* SerProp = DifferenceCallstackData.SerializedProp;
			if (SerProp && !bIsHeaderDiff)
			{
				// We don't attempt to serialize properties when we have already encountered at least one diff in the asset.
				// That is because we don't handle length differences in the source and destination data caused by the previous
				// diffs.  Those previous length differences could mean the current diff is at a position that is offset and
				// invalid to serialize the property from.  That can result in things like serializing an array or string which has
				// a negative number of elements, or other invalid situations that lead to an assert or crash.  If we must
				// sesrialize these properties, we would have to ensure that we keep an appropriate offset separate for the source
				// archive and the dest archive.
				if ((SourceSize == DestSize) && (InOutDiffsLogged < 2) && ShouldDumpPropertyValueState(SerProp))
				{
					// Walk backwards until we find a callstack which wasn't from the given property
					int64 OffsetX = DestAbsoluteOffset;
					for (;;)
					{
						if (OffsetX == 0)
						{
							break;
						}

						const int32 CallstackIndex = Callstacks.GetCallstackIndexAtOffset(OffsetX - 1, 0);
						if (CallstackIndex < 0)
						{
							break;
						}
						const FCallstacks::FCallstackAtOffset& PreviousCallstack = Callstacks.GetCallstack(CallstackIndex);
						const FCallstacks::FCallstackData& PreviousCallstackData = *Callstacks.GetCallstackData(PreviousCallstack);
						if (PreviousCallstackData.SerializedProp != SerProp)
						{
							break;
						}

						--OffsetX;
					}

					FPropertyTempVal SourceVal(SerProp);
					FPropertyTempVal DestVal(SerProp);

					FStaticMemoryReader SourceReader(&SourcePackage.Data[SourceAbsoluteOffset - (DestAbsoluteOffset - OffsetX)], SourcePackage.Size - SourceAbsoluteOffset);
					FStaticMemoryReader DestReader(&DestPackage.Data[OffsetX], DestPackage.Size - DestAbsoluteOffset);
					SourceHeader.Initialize();
					DestHeader.Initialize();
					FPackageHeaderDataProxyArchive SourceAr(SourceHeader, SourceReader);
					FPackageHeaderDataProxyArchive DestAr(DestHeader, DestReader);

					SourceVal.Serialize(SourceAr);
					DestVal.Serialize(DestAr);

					if (!SourceReader.IsError() && !DestReader.IsError())
					{
						SourceVal.ExportText(BeforePropertyVal);
						DestVal.ExportText(AfterPropertyVal);
					}
				}
			}

			DiffOutputRecorder->RecordDiff(*SectionFilename, LocalOffset, DestAbsoluteOffset, SourceByte, DestByte,
				DestAbsoluteOffset - CallstackAtOffset.SerializeCallOffset,
				*LastDifferenceCallstackDataText, BeforePropertyVal, AfterPropertyVal, DifferenceCallstackData);
			DestAbsoluteEndOffsetForActiveDiff = DestAbsoluteEndOffset;
		}

		DiffOutputRecorder->RecordDiffBytes(*SectionFilename, BytesToLog, LocalOffset, SourcePackage, DestPackage);
	}

	if (DestAbsoluteEndOffsetForActiveDiff != -1)
	{
		DiffOutputRecorder->ExtendPreviousDiff(ActiveDiffLocalOffset);
	}
	if (DestAbsoluteEndOffsetForDuplicateDiff != -1)
	{
		DiffOutputRecorder->IncrementPreviousDiff();
	}

	if (SourceSize != DestSize)
	{
		DiffOutputRecorder->RecordSectionSizeMismatch(*SectionFilename, SourceSize, DestSize);
		int64 SizeDiff = DestPackage.Size - SourcePackage.Size;
		OutStats.FindOrAdd(AssetClass).DiffSize += SizeDiff;
		OutStats.FindOrAdd(AssetClass).NumDiffs++;
		InOutDiffsLogged++;
	}

	if (MaxDiffsToLog >= 0 && NumDiffsForLogStatLocal > NumDiffsLoggedLocal)
	{
		DiffOutputRecorder->RecordUnreportedDiffs(*SectionFilename, NumDiffsForLogStatLocal - NumDiffsLoggedLocal, FirstUnreportedDiffIndex);
	}
}

void FAccumulator::CompareWithPrevious(const TCHAR* CallstackCutoffText, TMap<FName, FArchiveDiffStats>&OutStats)
{
	// An FDiffArchiveForLinker should have been constructed by SavePackage and should still be in memory
	check(LinkerArchive);

	Globals.Initialize(PackageHeaderFormat);

	const FName AssetClass = GetAssetClass();
	OutStats.FindOrAdd(AssetClass).NewFileTotalSize = LinkerArchive->TotalSize();
	if (PreviousPackageData.Size == 0)
	{
		DiffOutputRecorder->RecordNewPackage(*Filename);
		OutStats.FindOrAdd(AssetClass).DiffSize = OutStats.FindOrAdd(AssetClass).NewFileTotalSize;
		return;
	}

	FPackageData SourcePackage;
	SourcePackage.Data = PreviousPackageData.Data.Get();
	SourcePackage.Size = PreviousPackageData.Size;
	SourcePackage.HeaderSize = PreviousPackageData.HeaderSize;
	SourcePackage.StartOffset = PreviousPackageData.StartOffset;

	FPackageData DestPackage;
	DestPackage.Data = LinkerArchive->GetData();
	DestPackage.Size = LinkerArchive->TotalSize();
	DestPackage.HeaderSize = HeaderSize;
	DestPackage.StartOffset = 0;

	DiffOutputRecorder->BeginPackage(*Filename, *AssetClass.ToString());
	if (DeterminismManager)
	{
		DeterminismManager->RecordPackageModified(Asset);
	}

	int32 NumLoggedDiffs = 0;
	
	FPackageData SourceHeaderSegment = SourcePackage;
	SourceHeaderSegment.Size = SourcePackage.HeaderSize;
	SourceHeaderSegment.HeaderSize = 0;
	SourceHeaderSegment.StartOffset = 0;

	FPackageData DestHeaderSegment = DestPackage;
	DestHeaderSegment.Size = HeaderSize;
	DestHeaderSegment.HeaderSize = 0;
	DestHeaderSegment.StartOffset = 0;

	int32 NumHeaderDiffMessages = 0;
	FMessageCallback HeaderMessageCallback = [&NumHeaderDiffMessages, this](ELogVerbosity::Type Verbosity, FStringView Message)
		{
			DiffOutputRecorder->GetMessageCallback()(Verbosity, Message);
			++NumHeaderDiffMessages;
		};
	TSharedRef<FDiffOutputRecorder> HeaderDiffOutputRecorder = MakeShared<FDiffOutputRecorder>(
		MoveTemp(HeaderMessageCallback), DiffOutputRecorder->GetDetailRecorder());

	FPackageHeaderData SourceHeader(TEXT("source"), true /* bReadFromPackageStore */, Filename, SourcePackage,
		PackageHeaderFormat, Globals, HeaderDiffOutputRecorder);
	FPackageHeaderData DestHeader(TEXT("dest"), false /* bReadFromPackageStore */, Filename, DestPackage,
		PackageHeaderFormat, Globals, HeaderDiffOutputRecorder);

	DiffOutputRecorder->BeginSection(*Filename, EDiffWriterSectionType::Header, SourceHeaderSegment, DestHeaderSegment);
	CompareWithPreviousForSection(SourceHeaderSegment, DestHeaderSegment, SourceHeader, DestHeader,
		CallstackCutoffText, NumLoggedDiffs, OutStats, Filename);
	if (HeaderSize > 0 && OutStats.FindOrAdd(AssetClass).NumDiffs > 0)
	{
		DumpPackageHeaderDiffs(SourceHeader, DestHeader, MaxDiffsToLog);
		if (NumHeaderDiffMessages == 0)
		{
			DiffOutputRecorder->RecordUndiagnosedHeaderDifference(*Filename);
		}
	}
	DiffOutputRecorder->EndSection();


	FPackageData SourcePackageExports = SourcePackage;
	SourcePackageExports.HeaderSize = 0;
	SourcePackageExports.StartOffset = PreviousPackageData.HeaderSize;

	FPackageData DestPackageExports = DestPackage;
	DestPackageExports.HeaderSize = 0;
	DestPackageExports.StartOffset = HeaderSize;

	FString ExportsFilename;
	if (DestPackage.HeaderSize > 0)
	{
		ExportsFilename = FPaths::ChangeExtension(Filename, TEXT("uexp"));
	}
	else
	{
		ExportsFilename = Filename;
	}

	
	DiffOutputRecorder->BeginSection(*ExportsFilename, EDiffWriterSectionType::Exports, SourcePackageExports, DestPackageExports);
	CompareWithPreviousForSection(SourcePackageExports, DestPackageExports, SourceHeader, DestHeader,
		CallstackCutoffText, NumLoggedDiffs, OutStats, ExportsFilename);
	DiffOutputRecorder->EndSection();


	// Log comparison of the DeterminismHelper diagnostics, if we have any
	if (DeterminismManager)
	{
		FString DeterminismLines = DeterminismManager->GetCurrentPackageDiagnosticsAsText();
		DiffOutputRecorder->RecordDeterminismDiagnostics(DeterminismLines);
	}

	// Optionally save out any differences we detected.
	const FArchiveDiffStats& Stats = OutStats.FindOrAdd(AssetClass);
	if (Stats.NumDiffs > 0)
	{
		static struct FDiffOutputSettings
		{
			FString DiffOutputDir;

			FDiffOutputSettings()
			{
				FString Dir;
				if (!FParse::Value(FCommandLine::Get(), TEXT("diffoutputdir="), Dir))
				{
					return;
				}

				FPaths::NormalizeDirectoryName(Dir);
				DiffOutputDir = MoveTemp(Dir) + TEXT("/");
			}
		} DiffOutputSettings;

		// Only save out the differences if we have a -diffoutputdir set.
		if (!DiffOutputSettings.DiffOutputDir.IsEmpty())
		{
			FString OutputFilename = FPaths::ConvertRelativePathToFull(Filename);
			FString SavedDir       = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir());
			if (OutputFilename.StartsWith(SavedDir))
			{
				OutputFilename.ReplaceInline(*SavedDir, *DiffOutputSettings.DiffOutputDir);

				IFileManager& FileManager = IFileManager::Get();

				// Copy the original asset as '.before.uasset'.
				{
					TUniquePtr<FArchive> DiffUAssetArchive(FileManager.CreateFileWriter(
						*FPaths::SetExtension(OutputFilename, TEXT(".before.") + FPaths::GetExtension(Filename))));
					DiffUAssetArchive->Serialize(SourceHeaderSegment.Data + SourceHeaderSegment.StartOffset,
						SourceHeaderSegment.Size - SourceHeaderSegment.StartOffset);
				}
				{
					TUniquePtr<FArchive> DiffUExpArchive(FileManager.CreateFileWriter(
						*FPaths::SetExtension(OutputFilename, TEXT(".before.uexp"))));
					DiffUExpArchive->Serialize(SourcePackageExports.Data + SourcePackageExports.StartOffset,
						SourcePackageExports.Size - SourcePackageExports.StartOffset);
				}

				// Save out the in-memory data as '.after.uasset'.
				{
					TUniquePtr<FArchive> DiffUAssetArchive(FileManager.CreateFileWriter(
						*FPaths::SetExtension(OutputFilename, TEXT(".after.") + FPaths::GetExtension(Filename))));
					DiffUAssetArchive->Serialize(DestHeaderSegment.Data + DestHeaderSegment.StartOffset,
						DestHeaderSegment.Size - DestHeaderSegment.StartOffset);
				}
				{
					TUniquePtr<FArchive> DiffUExpArchive(FileManager.CreateFileWriter(
						*FPaths::SetExtension(OutputFilename, TEXT(".after.uexp"))));
					DiffUExpArchive->Serialize(DestPackageExports.Data + DestPackageExports.StartOffset,
						DestPackageExports.Size - DestPackageExports.StartOffset);
				}
			}
			else
			{
				DiffOutputRecorder->GetMessageCallback()(ELogVerbosity::Warning,
					FString::Printf(TEXT("Package '%s' doesn't seem to be writing to the Saved directory - skipping writing diff"), *OutputFilename));
			}
		}
	}
	DiffOutputRecorder->EndPackage(*Filename, SourcePackage, DestPackage, *AssetClass.ToString());
}

void FAccumulator::GenerateDiffMapForSection(const FPackageData& SourcePackage, const FPackageData& DestPackage,
	bool& bOutSectionIdentical)
{
	FCallstacks& Callstacks = this->LinkerCallstacks;
	bool bIdentical = true;
	int32 LastDifferenceCallstackOffsetIndex = -1;
	FCallstacks::FCallstackData* DifferenceCallstackData = nullptr;

	const int64 SourceSize = SourcePackage.Size - SourcePackage.StartOffset;
	const int64 DestSize = DestPackage.Size - DestPackage.StartOffset;
	const int64 SizeToCompare = FMath::Min(SourceSize, DestSize);
	
	for (int64 LocalOffset = 0; LocalOffset < SizeToCompare; ++LocalOffset)
	{
		const int64 SourceAbsoluteOffset = LocalOffset + SourcePackage.StartOffset;
		const int64 DestAbsoluteOffset = LocalOffset + DestPackage.StartOffset;
		if (SourcePackage.Data[SourceAbsoluteOffset] != DestPackage.Data[DestAbsoluteOffset])
		{
			bIdentical = false;
			if (DiffMap.Num() < MaxDiffsToLog)
			{
				const int32 DifferenceCallstackOffsetIndex = Callstacks.GetCallstackIndexAtOffset(DestAbsoluteOffset, FMath::Max<int32>(LastDifferenceCallstackOffsetIndex, 0));
				if (DifferenceCallstackOffsetIndex >= 0 && DifferenceCallstackOffsetIndex != LastDifferenceCallstackOffsetIndex)
				{
					const FCallstacks::FCallstackAtOffset& CallstackAtOffset = Callstacks.GetCallstack(DifferenceCallstackOffsetIndex);
					if (!CallstackAtOffset.bSuppressLogging)
					{
						DiffMap.Add(FDiffInfo(CallstackAtOffset.SerializeCallOffset,
							CallstackAtOffset.SerializeCallLength));
					}
				}
				LastDifferenceCallstackOffsetIndex = DifferenceCallstackOffsetIndex;
			}
		}
	}

	if (SourceSize < DestSize)
	{
		bIdentical = false;

		// Add all the remaining callstacks to the diff map
		for (int32 OffsetIndex = LastDifferenceCallstackOffsetIndex + 1; OffsetIndex < Callstacks.Num() && DiffMap.Num() < MaxDiffsToLog; ++OffsetIndex)
		{
			const FCallstacks::FCallstackAtOffset& CallstackAtOffset = Callstacks.GetCallstack(OffsetIndex);
			// Compare against the size without start offset as all callstack offsets are absolute (from the merged header + exports file)
			if (CallstackAtOffset.Offset < DestPackage.Size)
			{
				if (!CallstackAtOffset.bSuppressLogging)
				{
					DiffMap.Add(FDiffInfo(CallstackAtOffset.SerializeCallOffset,
						CallstackAtOffset.SerializeCallLength));
				}
			}
			else
			{
				break;
			}
		}
	}
	else if (SourceSize > DestSize)
	{
		bIdentical = false;
	}
	bOutSectionIdentical = bIdentical;
}

void FAccumulator::GenerateDiffMap()
{
	check(MaxDiffsToLog > 0);
	// An FDiffArchiveForLinker should have been constructed by SavePackage and should still be in memory
	check(LinkerArchive != nullptr);

	bHasDifferences = true;
	DiffMap.Reset();

	FPackageData SourcePackage;
	SourcePackage.Data = PreviousPackageData.Data.Get();
	SourcePackage.Size = PreviousPackageData.Size;
	SourcePackage.HeaderSize = PreviousPackageData.HeaderSize;
	SourcePackage.StartOffset = PreviousPackageData.StartOffset;

	bool bIdentical = true;
	bool bHeaderIdentical = true;
	bool bExportsIdentical = true;

	FPackageData DestPackage;
	DestPackage.Data = LinkerArchive->GetData();
	DestPackage.Size = LinkerArchive->TotalSize();
	DestPackage.HeaderSize = HeaderSize;
	DestPackage.StartOffset = 0;

	{
		FPackageData SourcePackageHeader = SourcePackage;
		SourcePackageHeader.Size = SourcePackage.HeaderSize;
		SourcePackageHeader.HeaderSize = 0;
		SourcePackageHeader.StartOffset = 0;

		FPackageData DestPackageHeader = DestPackage;
		DestPackageHeader.Size = HeaderSize;
		DestPackageHeader.HeaderSize = 0;
		DestPackageHeader.StartOffset = 0;

		GenerateDiffMapForSection(SourcePackageHeader, DestPackageHeader, bHeaderIdentical);
	}

	{
		FPackageData SourcePackageExports = SourcePackage;
		SourcePackageExports.HeaderSize = 0;
		SourcePackageExports.StartOffset = SourcePackage.HeaderSize;

		FPackageData DestPackageExports = DestPackage;
		DestPackageExports.HeaderSize = 0;
		DestPackageExports.StartOffset = HeaderSize;

		GenerateDiffMapForSection(SourcePackageExports, DestPackageExports, bExportsIdentical);
	}

	bIdentical = bHeaderIdentical && bExportsIdentical;
	bHasDifferences = !bIdentical;
	static bool bForceDiff = []()
		{
			return FParse::Param(FCommandLine::Get(), TEXT("cookforcediff"));
		}();
	if (bForceDiff)
	{
		bHasDifferences = true;
	}
}

FLinkerLoad* CreateLinkerForPackage(FUObjectSerializeContext* LoadContext, const FString& InPackageName, const FString& InFilename, const FPackageData& PackageData)
{
	// First create a temp package to associate the linker with
	UPackage* Package = FindObjectFast<UPackage>(nullptr, *InPackageName);
	if (!Package)
	{
		Package = CreatePackage(*InPackageName);
	}
	// Create an archive for the linker. The linker will take ownership of it.
	FLargeMemoryReader* PackageReader = new FLargeMemoryReader(PackageData.Data, PackageData.Size, ELargeMemoryReaderFlags::None, *InPackageName);	
	FLinkerLoad* Linker = FLinkerLoad::CreateLinker(LoadContext, Package, FPackagePath::FromLocalPath(InFilename), LOAD_NoVerify, PackageReader);

	if (Linker && Package)
	{
		Package->SetPackageFlags(PKG_ForDiffing);
	}

	return Linker;
}

/** Structure that holds an item from the NameMap/ImportMap/ExportMap in a TSet for diffing */
template <typename T>
struct TTableItem
{
	/** The key generated for this item */
	FString Key;
	/** Pointer to the original item */
	const T* Item;
	/** Index in the original *Map (table). Only for information purposes. */
	int32 Index;

	TTableItem(FString&& InKey, const T* InItem, int32 InIndex)
		: Key(MoveTemp(InKey))
		, Item(InItem)
		, Index(InIndex)
	{
	}

	FORCENOINLINE friend uint32 GetTypeHash(const TTableItem& TableItem)
	{
		return GetTypeHash(TableItem.Key);
	}

	FORCENOINLINE friend bool operator==(const TTableItem& Lhs, const TTableItem& Rhs)
	{
		return Lhs.Key == Rhs.Key;
	}
};

static bool Equal(const TOptional<FZenPackageVersioningInfo>& SourceInfo, const TOptional<FZenPackageVersioningInfo>& DestInfo)
{
	TStringBuilder<256> Message;
	if (SourceInfo.IsSet() != DestInfo.IsSet())
	{
		return false;
	}
	else if (!SourceInfo.IsSet())
	{
		return true;
	}
	else
	{
		if (SourceInfo->ZenVersion != DestInfo->ZenVersion)
		{
			return false;
		}
		if (SourceInfo->PackageVersion != DestInfo->PackageVersion)
		{
			return false;
		}
		if (SourceInfo->LicenseeVersion != DestInfo->LicenseeVersion)
		{
			return false;
		}
		for (const FCustomVersion& SrcVersion : SourceInfo->CustomVersions.GetAllVersions())
		{
			const FCustomVersion* DestVersion = DestInfo->CustomVersions.GetVersion(SrcVersion.Key);
			if (!DestVersion)
			{
				return false;
			}
			else if (SrcVersion.Version != DestVersion->Version)
			{
				return false;
			}
		}
		for (const FCustomVersion& DestVersion : DestInfo->CustomVersions.GetAllVersions())
		{
			const FCustomVersion* SrcVersion = SourceInfo->CustomVersions.GetVersion(DestVersion.Key);
			if (!SrcVersion)
			{
				return false;
			}
		}
		return true;
	}
}

template <typename ContextProvider>
static void DumpDifferences(ContextProvider& SourceContext, const TCHAR* AssetFilename,
	const TOptional<FZenPackageVersioningInfo>& SourceInfo, const TOptional<FZenPackageVersioningInfo>& DestInfo)
{
	TStringBuilder<256> Message;
	if (SourceInfo.IsSet() != DestInfo.IsSet())
	{
		Message << NewLineToken << TEXT("VersioningInfo present in source package vs not present in dest package.");
	}
	else if (SourceInfo.IsSet())
	{
		if (SourceInfo->ZenVersion != DestInfo->ZenVersion)
		{
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("-ZenVersion: %u"), (uint32)SourceInfo->ZenVersion);
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("+ZenVersion: %u"), (uint32)DestInfo->ZenVersion);
		}
		if (SourceInfo->PackageVersion != DestInfo->PackageVersion)
		{
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("-PackageVersion: %d"), (uint32)SourceInfo->PackageVersion.ToValue());
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("+PackageVersion: %d"), (uint32)DestInfo->PackageVersion.ToValue());
		}
		if (SourceInfo->LicenseeVersion != DestInfo->LicenseeVersion)
		{
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("-LicenseeVersion: %d"), (uint32)SourceInfo->LicenseeVersion);
			Message << NewLineToken << IndentToken;
			Message.Appendf(TEXT("+LicenseeVersion: %d"), (uint32)DestInfo->LicenseeVersion);
		}
		for (const FCustomVersion& SrcVersion : SourceInfo->CustomVersions.GetAllVersions())
		{
			const FCustomVersion* DestVersion = DestInfo->CustomVersions.GetVersion(SrcVersion.Key);
			if (!DestVersion)
			{
				Message << NewLineToken << IndentToken;
				Message.Appendf(TEXT("-CustomVersion: %s == %d"), *SrcVersion.Key.ToString(), SrcVersion.Version);
			}
			else if (SrcVersion.Version != DestVersion->Version)
			{
				Message << NewLineToken << IndentToken;
				Message.Appendf(TEXT("-CustomVersion: %s == %d"), *SrcVersion.Key.ToString(), SrcVersion.Version);
				Message << NewLineToken << IndentToken;
				Message.Appendf(TEXT("+CustomVersion: %s == %d"), *DestVersion->Key.ToString(), DestVersion->Version);
			}
		}
		for (const FCustomVersion& DestVersion : DestInfo->CustomVersions.GetAllVersions())
		{
			const FCustomVersion* SrcVersion = SourceInfo->CustomVersions.GetVersion(DestVersion.Key);
			if (!SrcVersion)
			{
				Message << NewLineToken << IndentToken;
				Message.Appendf(TEXT("+CustomVersion: %s == %d"), *DestVersion.Key.ToString(), DestVersion.Version);
			}
		}
	}

	if (Message.Len() == 0)
	{
		Message << NewLineToken << IndentToken << TEXT("<Unknown Difference>");
	}
	SourceContext.LogMessage(ELogVerbosity::Warning, FString::Printf(
		TEXT("%s: VersioningInfo is different:%s"),
		AssetFilename, *Message));
}

/** Dumps differences between Linker tables */
template <typename T, typename ContextProvider>
static void DumpTableDifferences(
	ContextProvider& SourceContext,
	ContextProvider& DestContext,
	TConstArrayView<T> SourceTable, 
	TConstArrayView<T> DestTable,
	const TCHAR* AssetFilename,
	const TCHAR* ItemName,
	const int32 MaxDiffsToLog)
{
	FString HumanReadableString;
	int32 LoggedDiffs = 0;
	int32 NumDiffs = 0;

	TSet<TTableItem<T>> SourceSet;
	TSet<TTableItem<T>> DestSet;

	SourceSet.Reserve(SourceTable.Num());
	DestSet.Reserve(DestTable.Num());

	for (int32 Index = 0; Index < SourceTable.Num(); ++Index)
	{
		const T& Item = SourceTable[Index];
		SourceSet.Add(TTableItem<T>(SourceContext.GetTableKey(Item), &Item, Index));
	}
	for (int32 Index = 0; Index < DestTable.Num(); ++Index)
	{
		const T& Item = DestTable[Index];
		DestSet.Add(TTableItem<T>(DestContext.GetTableKey(Item), &Item, Index));
	}

	// Determine the list of items removed from the source package and added to the dest package
	TSet<TTableItem<T>> RemovedItems = SourceSet.Difference(DestSet);
	TSet<TTableItem<T>> AddedItems   = DestSet.Difference(SourceSet);

	// Add changed items as added-and-removed
	for (const TTableItem<T>& ChangedSourceItem : SourceSet)
	{
		if (const TTableItem<T>* ChangedDestItem = DestSet.Find(ChangedSourceItem))
		{
			if (!SourceContext.CompareTableItem(DestContext, *ChangedSourceItem.Item,
				*ChangedDestItem->Item))
			{
				RemovedItems.Add(ChangedSourceItem);
				AddedItems  .Add(*ChangedDestItem);
			}
		}
	}

	// Sort all additions and removals by index
	RemovedItems.Sort([](const TTableItem<T>& Lhs, const TTableItem<T>& Rhs){ return Lhs.Index < Rhs.Index; });
	AddedItems.  Sort([](const TTableItem<T>& Lhs, const TTableItem<T>& Rhs){ return Lhs.Index < Rhs.Index; });

	// Dump all changes
	for (const TTableItem<T>& RemovedItem : RemovedItems)
	{
		HumanReadableString += IndentToken;
		HumanReadableString += FString::Printf(TEXT("-[%d] %s"), RemovedItem.Index, *SourceContext.ConvertItemToText(*RemovedItem.Item));
		HumanReadableString += NewLineToken;
	}
	for (const TTableItem<T>& AddedItem : AddedItems)
	{
		HumanReadableString += IndentToken;
		HumanReadableString += FString::Printf(TEXT("+[%d] %s"), AddedItem.Index, *DestContext.ConvertItemToText(*AddedItem.Item));
		HumanReadableString += NewLineToken;
	}

	// For now just log everything out. When this becomes too spammy, respect the MaxDiffsToLog parameter
	NumDiffs = RemovedItems.Num() + AddedItems.Num();
	LoggedDiffs = NumDiffs;

	if (NumDiffs > LoggedDiffs)
	{
		HumanReadableString += IndentToken;
		HumanReadableString += FString::Printf(TEXT("+ %d differences not logged."), (NumDiffs - LoggedDiffs));
		HumanReadableString += NewLineToken;
	}

	SourceContext.GetDiffOutputRecorder()->RecordTableDifferences(AssetFilename, ItemName,
		SourceTable.Num(), DestTable.Num(), *HumanReadableString);
}

static void DumpOrderedArrayDifferences(
	int32 SourceNum,
	int32 DestNum,
	TFunctionRef<bool(int32 Index)> IsElementsAtIndexEqual,
	TFunctionRef<FString(int32 Index)> ConvertSourceIndexToText,
	TFunctionRef<FString(int32 Index)> ConvertDestIndexToText,
	TFunctionRef<void(ELogVerbosity::Type, FString)> LogMessage,
	const TCHAR* AssetFilename,
	const TCHAR* ItemName,
	const int32 MaxDiffsToLog,
	UE::DiffWriter::FDiffOutputRecorder& DiffOutputRecorder
)
{
	FString HumanReadableString;
	int32 LoggedDiffs = 0;
	int32 NumDiffs = 0;

	int32 MaxIndex = FMath::Max(SourceNum, DestNum);
	for (int32 Index = 0; Index < MaxIndex; ++Index)
	{
		if (Index >= DestNum)
		{
			if (MaxDiffsToLog < 0 || NumDiffs < MaxDiffsToLog)
			{
				HumanReadableString += IndentToken;
				HumanReadableString += FString::Printf(TEXT("-[%d] %s"), Index, *ConvertSourceIndexToText(Index));
				HumanReadableString += NewLineToken;
				++LoggedDiffs;
			}
			++NumDiffs;
		}
		else if (Index >= SourceNum)
		{
			if (MaxDiffsToLog < 0 || NumDiffs < MaxDiffsToLog)
			{
				HumanReadableString += IndentToken;
				HumanReadableString += FString::Printf(TEXT("+[%d] %s"), Index, *ConvertDestIndexToText(Index));
				HumanReadableString += NewLineToken;
				++LoggedDiffs;
			}
			++NumDiffs;
		}
		else if (!IsElementsAtIndexEqual(Index))
		{
			if (MaxDiffsToLog < 0 || NumDiffs < MaxDiffsToLog)
			{
				HumanReadableString += IndentToken;
				HumanReadableString += FString::Printf(TEXT("-[%d] %s"), Index, *ConvertSourceIndexToText(Index));
				HumanReadableString += NewLineToken;
				HumanReadableString += IndentToken;
				HumanReadableString += FString::Printf(TEXT("+[%d] %s"), Index, *ConvertDestIndexToText(Index));
				HumanReadableString += NewLineToken;
				++LoggedDiffs;
			}
			++NumDiffs;
		}
	}

	if (NumDiffs > LoggedDiffs)
	{
		HumanReadableString += IndentToken;
		HumanReadableString += FString::Printf(TEXT("+ %d differences not logged."), (NumDiffs - LoggedDiffs));
		HumanReadableString += NewLineToken;
	}

	DiffOutputRecorder.RecordTableDifferences(AssetFilename, ItemName, *HumanReadableString);
}

void DumpPackageHeaderDiffs_LinkerLoad(FPackageHeaderData& SourceHeaderData, FPackageHeaderData& DestHeaderData,
	const int32 MaxDiffsToLog)
{
	SourceHeaderData.Initialize();
	DestHeaderData.Initialize();
	const FString& AssetFilename = SourceHeaderData.GetAssetFilename();
	FLinkerLoad* SourceLinker = SourceHeaderData.GetLinker();
	FLinkerLoad* DestLinker = DestHeaderData.GetLinker();
	if (!SourceLinker || !DestLinker)
	{
		return;
	}

	FDiffWriterLinkerLoadHeader SourceContext(SourceLinker, SourceHeaderData.GetDiffOutputRecorder());
	FDiffWriterLinkerLoadHeader DestContext(DestLinker, SourceHeaderData.GetDiffOutputRecorder());

	if (SourceLinker->NameMap != DestLinker->NameMap)
	{
		DumpTableDifferences<FNameEntryId>(SourceContext, DestContext, SourceLinker->NameMap, DestLinker->NameMap,
			*AssetFilename, TEXT("NameMap"), MaxDiffsToLog);
	}

	if (!SourceContext.IsImportMapIdentical(DestContext))
	{
		DumpTableDifferences<FObjectImport>(SourceContext, DestContext, SourceLinker->ImportMap, DestLinker->ImportMap,
			*AssetFilename, TEXT("ImportMap"), MaxDiffsToLog);
	}

	if (!SourceContext.IsExportMapIdentical(DestContext))
	{
		DumpTableDifferences<FObjectExport>(SourceContext, DestContext, SourceLinker->ExportMap, DestLinker->ExportMap,
			*AssetFilename, TEXT("ExportMap"), MaxDiffsToLog);
	}
}

void DumpPackageHeaderDiffs_ZenPackage(FPackageHeaderData& SourceHeaderData,
	FPackageHeaderData& DestHeaderData, const int32 MaxDiffsToLog)
{
	SourceHeaderData.Initialize();
	DestHeaderData.Initialize();

	const FString& AssetFilename = SourceHeaderData.GetAssetFilename();
	FDiffWriterZenHeader& SourceHeader = SourceHeaderData.GetZenHeader();
	FDiffWriterZenHeader& DestHeader = DestHeaderData.GetZenHeader();
	if (!SourceHeaderData.IsValid() || !DestHeaderData.IsValid())
	{
		// Explanation was logged by TryConstructSummary
		return;
	}
	const FZenPackageHeader& SourceZenHeader = SourceHeader.GetPackageHeader();
	const FZenPackageHeader& DestZenHeader = DestHeader.GetPackageHeader();

	TArray<FName> SourceNames;
	TArray<FName> DestNames;
	TArray<FString> SourceNameStrings;
	TArray<FString> DestNameStrings;
	SourceNames.Reset(SourceZenHeader.NameMap.Num());
	for (FDisplayNameEntryId Id : SourceZenHeader.NameMap)
	{
		SourceNames.Add(Id.ToName(0));
	}
	DestNames.Reset(DestZenHeader.NameMap.Num());
	for (FDisplayNameEntryId Id : DestZenHeader.NameMap)
	{
		DestNames.Add(Id.ToName(0));
	}

	if (!Equal(SourceZenHeader.VersioningInfo, DestZenHeader.VersioningInfo))
	{
		DumpDifferences(SourceHeader, *AssetFilename, SourceZenHeader.VersioningInfo,
			DestZenHeader.VersioningInfo);
	}
	bool bFoundDifferenceInLinkerTableData = false;
	if (!SourceHeader.IsNameMapIdentical(SourceNames, DestNames, SourceNameStrings, DestNameStrings))
	{
		bFoundDifferenceInLinkerTableData = true;
		DumpTableDifferences<FString>(SourceHeader, DestHeader, SourceNameStrings, DestNameStrings,
			*AssetFilename, TEXT("NameMap"), MaxDiffsToLog);
	}

	// ImportedPackageNames have their own FName storage and are not in the NameMap, so we need to look for diffs in
	// them separately.
	if (!SourceHeader.IsNameMapIdentical(SourceZenHeader.ImportedPackageNames, DestZenHeader.ImportedPackageNames,
		SourceNameStrings, DestNameStrings))
	{
		bFoundDifferenceInLinkerTableData = true;
		DumpTableDifferences<FString>(SourceHeader, DestHeader, SourceNameStrings, DestNameStrings,
			*AssetFilename, TEXT("ImportedPackageNames"), MaxDiffsToLog);
	}

	if (!SourceHeader.IsImportMapIdentical(DestHeader))
	{
		bFoundDifferenceInLinkerTableData = true;
		DumpTableDifferences<FPackageObjectIndex>(SourceHeader, DestHeader, SourceZenHeader.ImportMap,
			DestZenHeader.ImportMap, *AssetFilename, TEXT("ImportMap"), MaxDiffsToLog);
	}

	if (!SourceHeader.IsExportMapIdentical(DestHeader))
	{
		bFoundDifferenceInLinkerTableData = true;
		int32 SourceNum = SourceZenHeader.ExportMap.Num();
		int32 DestNum = DestZenHeader.ExportMap.Num();
		TArray<FZenHeaderIndexIntoExportMap> SourceIndices;
		SourceIndices.Reserve(SourceNum);
		for (int N = 0; N < SourceNum; ++N)
		{
			SourceIndices.Add(FZenHeaderIndexIntoExportMap{ N });
		}
		TArray<FZenHeaderIndexIntoExportMap> DestIndices;
		DestIndices.Reserve(DestNum);
		for (int N = 0; N < DestNum; ++N)
		{
			DestIndices.Add(FZenHeaderIndexIntoExportMap{ N });
		}
		DumpTableDifferences<FZenHeaderIndexIntoExportMap>(SourceHeader, DestHeader, SourceIndices, DestIndices,
			*AssetFilename, TEXT("ExportMap"), MaxDiffsToLog);
	}

	// Don't show differences in the dependency data if we found differences in the names/imports/exports data,
	// because the dependency data depends on that basic data and will usually have differences if any of them
	// changed, and it just creates noise for the developer.
	if (!bFoundDifferenceInLinkerTableData)
	{
		if (!SourceHeader.IsExportBundlesIdentical(DestHeader))
		{
			DumpOrderedArrayDifferences(SourceZenHeader.ExportBundleEntries.Num(),
				DestZenHeader.ExportBundleEntries.Num(),
				[&SourceHeader, &DestHeader](int32 Index)
				{
					return SourceHeader.IsExportBundleIdentical(DestHeader, Index);
				},
				[&SourceHeader](int32 Index)
				{
					return SourceHeader.ConvertExportBundleToText(Index);
				},
				[&DestHeader](int32 Index)
				{
					return DestHeader.ConvertExportBundleToText(Index);
				},
				[&SourceHeader](ELogVerbosity::Type Verbosity, FString Message)
				{
					SourceHeader.LogMessage(Verbosity, Message);
				},
				*AssetFilename, TEXT("ExportBundles"), MaxDiffsToLog, *SourceHeader.GetDiffOutputRecorder());
		}
		if (!SourceHeader.IsDependencyBundlesIdentical(DestHeader))
		{
			DumpOrderedArrayDifferences(SourceZenHeader.DependencyBundleHeaders.Num(),
				DestZenHeader.DependencyBundleHeaders.Num(),
				[&SourceHeader, &DestHeader](int32 Index)
				{
					return SourceHeader.IsDependencyBundleIdentical(DestHeader, Index);
				},
				[&SourceHeader](int32 Index)
				{
					return SourceHeader.ConvertDependencyBundleToText(Index);
				},
				[&DestHeader](int32 Index)
				{
					return DestHeader.ConvertDependencyBundleToText(Index);
				},
				[&SourceHeader](ELogVerbosity::Type Verbosity, FString Message)
				{
					SourceHeader.LogMessage(Verbosity, Message);
				},
				*AssetFilename, TEXT("DependencyBundles"), MaxDiffsToLog, *SourceHeader.GetDiffOutputRecorder());
		}
	}
}

void DumpPackageHeaderDiffs(FPackageHeaderData& SourceHeaderData, FPackageHeaderData& DestHeaderData,
	const int32 MaxDiffsToLog)
{
	switch (SourceHeaderData.GetFormat())
	{
	case EPackageHeaderFormat::PackageFileSummary:
		DumpPackageHeaderDiffs_LinkerLoad(SourceHeaderData, DestHeaderData, MaxDiffsToLog);
		break;
	case EPackageHeaderFormat::ZenPackageSummary:
		DumpPackageHeaderDiffs_ZenPackage(SourceHeaderData, DestHeaderData, MaxDiffsToLog);
		break;
	default:
		unimplemented();
	}
}

FPackageHeaderData::FPackageHeaderData(const TCHAR* InName, bool bInReadFromPackageStore,
	const FString& InAssetFilename, const FPackageData& InPackageData, EPackageHeaderFormat InFormat,
	FAccumulatorGlobals& InGlobals, TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder)
	: Name(InName)
	, AssetFilename(InAssetFilename)
	, PackageData(InPackageData)
	, Globals(InGlobals)
	, DiffOutputRecorder(MoveTemp(InDiffOutputRecorder))
	, Format(InFormat)
	, bReadFromPackageStore(bInReadFromPackageStore)
{
}

FPackageHeaderData::~FPackageHeaderData()
{
	if (Linker)
	{
		UE::ArchiveStackTrace::ForceKillPackageAndLinker(Linker);
	}
}

void FPackageHeaderData::Initialize()
{
	if (bInitialized)
	{
		return;
	}
	bInitialized = true;

	switch (Format)
	{
	case EPackageHeaderFormat::PackageFileSummary:
	{
		FString AssetPathName = FPaths::Combine(
			*FPaths::GetPath(AssetFilename.Mid(AssetFilename.Find(TEXT(":"), ESearchCase::CaseSensitive) + 1)),
			*FPaths::GetBaseFilename(AssetFilename));
		// The root directory could have a period in it (d:/Release5.0/EngineTest/Saved/Cooked),
		// which is not a valid character for a LongPackageName. Remove it.
		for (TCHAR c : FStringView(INVALID_LONGPACKAGE_CHARACTERS))
		{
			AssetPathName.ReplaceCharInline(c, TEXT('_'), ESearchCase::CaseSensitive);
		}
		FString AssetPackageName = FPaths::Combine(TEXT("/Memory"),
			WriteToString<32>(TEXT("/%sForDiff"), Name),
			*AssetPathName);
		check(FPackageName::IsValidLongPackageName(AssetPackageName, true /* bIncludeReadOnlyRoots */));

		TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);
		TGuardValue<int32> GuardAllowUnversionedContentInEditor(GAllowUnversionedContentInEditor, 1);
		TGuardValue<int32> GuardAllowCookedDataInEditorBuilds(GAllowCookedDataInEditorBuilds, 1);

		// Create linkers. Note there's no need to clean them up here since they will be removed by the package associated with them
		{
			TRefCountPtr<FUObjectSerializeContext> LinkerLoadContext(FUObjectThreadContext::Get().GetSerializeContext());
			BeginLoad(LinkerLoadContext);
			Linker = CreateLinkerForPackage(LinkerLoadContext, AssetPackageName, AssetFilename, PackageData);
			EndLoad(LinkerLoadContext);
		}
		break;
	}
	case EPackageHeaderFormat::ZenPackageSummary:
		ZenHeader = MakeUnique<FDiffWriterZenHeader>(Globals, DiffOutputRecorder, bReadFromPackageStore, PackageData,
			AssetFilename, Name);
		break;
	default:
		unimplemented();
		break;
	}
}

const TCHAR* FPackageHeaderData::GetName() const
{
	return Name;
}

const FString& FPackageHeaderData::GetAssetFilename() const
{
	return AssetFilename;
}

const FPackageData& FPackageHeaderData::GetPackageData() const
{
	return PackageData;
}

EPackageHeaderFormat FPackageHeaderData::GetFormat() const
{
	return Format;
}

FAccumulatorGlobals& FPackageHeaderData::GetGlobals() const
{
	return Globals;
}

const TSharedRef<FDiffOutputRecorder>& FPackageHeaderData::GetDiffOutputRecorder() const
{
	return DiffOutputRecorder;
}

FDiffWriterZenHeader& FPackageHeaderData::GetZenHeader() const
{
	check(Format == EPackageHeaderFormat::ZenPackageSummary && bInitialized);
	check(ZenHeader.IsValid());
	return *ZenHeader;
}

FLinkerLoad* FPackageHeaderData::GetLinker() const
{
	check(Format == EPackageHeaderFormat::PackageFileSummary && bInitialized);
	return Linker;
}

bool FPackageHeaderData::IsValid() const
{
	if (!bInitialized)
	{
		return false;
	}
	switch (Format)
	{
	case EPackageHeaderFormat::PackageFileSummary:
		return Linker != nullptr;
	case EPackageHeaderFormat::ZenPackageSummary:
		check(ZenHeader);
		return ZenHeader->IsValid();
	default:
		unimplemented();
		return false;
	}
}

bool FPackageHeaderData::TryGetMappedName(int32 Index, int32 Number, FName& OutName) const
{
	if (!bInitialized || !IsValid())
	{
		return false;
	}

	switch (Format)
	{
	case EPackageHeaderFormat::PackageFileSummary:
		check(Linker);
		if (!Linker->NameMap.IsValidIndex(Index))
		{
			return false;
		}
		OutName = FName::CreateFromDisplayId(Linker->NameMap[Index], Number);
		return true;
	case EPackageHeaderFormat::ZenPackageSummary:
		check(ZenHeader);
		return ZenHeader->GetPackageHeader().NameMap.TryGetName(
			FMappedName::Create((uint32)Index, (uint32)Number, FMappedName::EType::Package),
			OutName);
	default:
		unimplemented();
		return false;
	}
}

FPackageHeaderDataProxyArchive::FPackageHeaderDataProxyArchive(FPackageHeaderData& InHeader, FArchive& InInner)
	: FArchiveProxy(InInner)
	, Header(InHeader)
{
}

FArchive& FPackageHeaderDataProxyArchive::operator<<(FName& Value)
{
	int32 Index;
	int32 Number;

	InnerArchive << Index << Number;
	if (!Header.TryGetMappedName(Index, Number, Value))
	{
		Value = NAME_None;
	}
	return *this;
}

} // namespace UE::DiffWriter