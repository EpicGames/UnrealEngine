// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Logging/LogVerbosity.h"
#include "PackageStoreOptimizer.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/ArchiveStackTrace.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/PackageWriter.h"
#include "Templates/RefCounting.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class FDiffWriterArchiveTestsCallstacks;
class FLinkerLoad;
class FProperty;
class FUObjectThreadContext;
class UObject;
namespace UE::Cook { class FDeterminismManager; }
struct FUObjectSerializeContext;


namespace UE::DiffWriter
{

class FAccumulator;
class FDiffArchive;
class FPackageHeaderData;

typedef TUniqueFunction<void(ELogVerbosity::Type, FStringView)> FMessageCallback;
using EPackageHeaderFormat = ICookedPackageWriter::EPackageHeaderFormat;
using FPackageData = UE::ArchiveStackTrace::FPackageData;

extern const TCHAR* const IndentToken;
extern const TCHAR* const NewLineToken;

enum class EOffsetFrame
{
	Linker,
	Exports,
};

struct FDiffInfo
{
	int64 Offset;
	int64 Size;

	FDiffInfo()
		: Offset(0)
		, Size(0)
	{
	}
	FDiffInfo(int64 InOffset, int64 InSize)
		: Offset(InOffset)
		, Size(InSize)
	{
	}
	bool operator==(const FDiffInfo& InOther) const
	{
		return Offset == InOther.Offset;
	}
	bool operator<(const FDiffInfo& InOther) const
	{
		return Offset < InOther.Offset;
	}
	friend FArchive& operator << (FArchive& Ar, FDiffInfo& InDiffInfo)
	{
		Ar << InDiffInfo.Offset;
		Ar << InDiffInfo.Size;
		return Ar;
	}
};

class FDiffMap : public TArray<FDiffInfo>
{
public:
	bool ContainsOffset(int64 Offset) const
	{
		for (const FDiffInfo& Diff : *this)
		{
			if (Diff.Offset <= Offset && Offset < (Diff.Offset + Diff.Size))
			{
				return true;
			}
		}

		return false;
	}
};

/**
 * Key for the data that is stored per unique callstack by FCallstacks.
 * Each FCallstackAtOffset constructs the key based on c++ callstack and other data from the serialize
 * call. If the key does not already exist, more information is collected and stored under the key.
 */
struct FCallstackKey
{
	UObject* SerializedObject = nullptr;
	FProperty* SerializedProperty = nullptr;
	uint32 CppCallstackHash = MAX_uint32;
	bool operator==(const FCallstackKey& Other) const
	{
		return Other.SerializedObject == SerializedObject &&
			Other.SerializedProperty == SerializedProperty &&
			Other.CppCallstackHash == CppCallstackHash;
	}
	friend uint32 GetTypeHash(const FCallstackKey& Key)
	{
		constexpr uint32 Prime = 101;
		const uint32* WordPtr = reinterpret_cast<const uint32*>(&Key);
		const uint32* WordPtrEnd = WordPtr + sizeof(Key) / sizeof(uint32);
		uint32 Hash = 0;
		while (WordPtr < WordPtrEnd)
		{
			Hash = Hash * Prime + *WordPtr;
			++WordPtr;
		}
		return Hash;
	}
};

/** Holds offsets to captured callstacks. */
class FCallstacks
{
public:
	/** Struct to hold the actual Serialize call callstack and any associated data */
	struct FCallstackData : public FRefCountedObject
	{
		/** Full callstack */
		TUniquePtr<ANSICHAR[]> Callstack;
		/**
		 * The export being serialized. Garbage collections do not occur during the savepackage call,
		 * so it is safe for us to hold a raw pointer.
		 */
		UObject* SerializedObject = nullptr;
		/** The currently serialized property */
		FProperty* SerializedProp = nullptr;
		/** Hash of this->Callstack. */
		uint32 CppCallstackHash = MAX_uint32;

		FCallstackData() = default;
		FCallstackData(TUniquePtr<ANSICHAR[]>&& InCallstack, uint32 InCppCallstackHash, UObject* InSerializedObject, FProperty* InSerializedProperty);
		FCallstackData(FCallstackData&&) = delete; // FRefCountedObject doesn't allow it
		FCallstackData(const FCallstackData&) = delete;

		FCallstackData& operator=(FCallstackData&&) = delete; // FRefCountedObject doesn't allow it
		FCallstackData& operator=(const FCallstackData&) = delete;

		/** Converts the callstack and associated data to human readable string */
		FString ToString(const TCHAR* CallstackCutoffText) const;

		/** Clone the callstack data */
		TRefCountPtr<FCallstackData> Clone() const;
		/** Get the key under which this FCallstackData is stored. */
		FCallstackKey GetKey() const;
		FString GetObjectName() const;
		FString GetPropertyName() const;
	};

	/** Offset and callstack pair */
	struct FCallstackAtOffset
	{
		/**
		 * Offset of a block written by Serialize call. Equal to SerializeCallOffset unless the block was split by a
		 * separate Serialize call.
		 */
		int64 Offset = -1;
		/**
		 * Length of a block written by Serialize call. Equal to SerializeCallLength unless the block was split by a
		 * separate Serialize call.
		 */
		int64 Length = -1;
		/** The offset written to by the Serialize call. */
		int64 SerializeCallOffset = -1;
		/** The length written by the Serialize call. */
		int64 SerializeCallLength = -1;
		/** Callstack CRC for the Serialize call */
		TRefCountPtr<FCallstackData> Callstack;
		/** Collected inside of a scope that indicates diff should be recorded but logging should be suppressed */
		bool bSuppressLogging = false;
	};

	FCallstacks();

	/** Returns the total number of callstacks. */
	int32 Num() const
	{
		return CallstackAtOffsetMap.Num();
	}
	void Reset();

	FORCENOINLINE void RecordSerialize(EOffsetFrame OffsetFrame, int64 CurrentOffset, int64 Length,
		const FAccumulator& Accumulator, FDiffArchive& Ar, int32 StackIgnoreCount);

	/** Capture and append the current callstack. */
	void Add(
		int64 Offset,
		int64 Length,
		UObject* SerializedObject,
		FProperty* SerializedProperty,
		TArrayView<const FName> DebugDataStack,
		bool bIsCollectingCallstacks,
		bool bCollectCurrentCallstack,
		int32 StackIgnoreCount);

	/**
	 * Remove offset->callstack entries reported for a range of offsets. Only removes entries that start within the
	 * range, does not remove entries that start before the range but end in or after it.
	 */
	void RemoveRange(int64 StartOffset, int64 Length);

	/** Append other offset->callstacks entries and callstacks they refer to. */
	void Append(const FCallstacks& Other, int64 OtherStartOffset);

	/** Finds a callstack associated with data at the specified offset */
	int32 GetCallstackIndexAtOffset(int64 Offset, int32 MinOffsetIndex = 0, int64* OutOffsetEnd = nullptr) const;

	/** Finds a callstack associated with data at the specified offset */
	const FCallstackAtOffset& GetCallstack(int32 CallstackIndex) const
	{
		return CallstackAtOffsetMap[CallstackIndex];
	}
	
	const TRefCountPtr<FCallstackData>& GetCallstackData(const FCallstackAtOffset& CallstackOffset) const
	{
		return CallstackOffset.Callstack;
	}

	int64 GetEndOffset() const
	{
		return EndOffset;
	}

private:
	/** Adds a unique callstack to UniqueCallstacks map */
	TRefCountPtr<FCallstackData> AddUniqueCallstack(const FCallstackKey& Key);

	/** List of offsets and their respective callstacks */
	TArray<FCallstackAtOffset> CallstackAtOffsetMap;
	/** Contains all unique callstacks for all Serialize calls */
	TMap<FCallstackKey, TRefCountPtr<FCallstackData>> UniqueCallstacks;
	/** Maximum size of the stack trace */
	const SIZE_T StackTraceSize;
	/** Buffer for getting the current stack trace */
	TUniquePtr<ANSICHAR[]> StackTrace;
	/** Callstack associated with the previous Serialize call */
	uint32 PreviousCppCallstackHash;
	/**
	 * Optimizes callstack comparison. If false the Cpp callstack has not changed and does not need to be compared when
	 * checking whether the current UniqueCallstackHash has changed.
	 */
	bool bCppCallstackDirty;

	/** Total serialized bytes */
	int64 EndOffset;
};


enum class EDiffWriterSectionType : uint8
{
	Header,
	Exports,

	MAX
};



/** Diff detail serialization interface. Used for generating additional report artifacts */
class IDetailRecorder
{
public:
	virtual void BeginPackage() = 0;
	virtual void BeginSection(const TCHAR* SectionFilename, EDiffWriterSectionType SectionType, const FPackageData& SourceSection, 
		const FPackageData& DestSection) = 0;
	virtual void RecordDiff(int64 LocalOffset, const FCallstacks::FCallstackData* DifferenceCallstackData) = 0;
	virtual void ExtendPreviousDiff(int64 LocalOffset) = 0;
	virtual void IncrementPreviousDiff() = 0;
	virtual void RecordUndiagnosedDiff() = 0;
	virtual void RecordUnreportedDiffs(int32 NumUnreportedDiffs) = 0;
	virtual void RecordTableDifferences(const TCHAR* ItemName, const TCHAR* HumanReadableString) = 0;
	virtual void RecordDetermismDiagnostics(const TCHAR* DeterminismLines) = 0;
	virtual void EndSection() = 0;
	virtual void EndPackage(const TCHAR* Filename, const FPackageData& SourcePackage, const FPackageData& DestPackage, 
		const TCHAR* ClassName) = 0;

	virtual ~IDetailRecorder() = default;
};

/** Manages all output from the diff process */
class FDiffOutputRecorder
{
public:
	FDiffOutputRecorder(FMessageCallback&& InMessageCallback, IDetailRecorder* InDetailRecorder);

	void RecordNewPackage(const TCHAR* Filename);

	void BeginPackage(const TCHAR* Filename, const TCHAR* ClassName);
	void BeginSection(const TCHAR* SectionFilename, EDiffWriterSectionType SectionType, const FPackageData& SourceSection, 
		const FPackageData& DestSection);
	void EndSection();
	void EndPackage(const TCHAR* Filename, const FPackageData& SourcePackage, const FPackageData& DestPackage, const TCHAR* ClassName);

	void RecordHeaderSizeMismatch(const TCHAR* Filename, int64 FirstHeaderSize, int64 SecondHeaderSize);
	void RecordUndiagnosedHeaderDifference(const TCHAR* Filename);
	void RecordTableDifferences(const TCHAR* Filename, const TCHAR* ItemName, int32 SourceTableNum, int32 DestTableNum, 
		const TCHAR* HumanReadableString);
	void RecordTableDifferences(const TCHAR* Filename, const TCHAR* ItemName, const TCHAR* HumanReadableString);
	void RecordSectionSizeMismatch(const TCHAR* SectionFilename, int64 SourceSize, int64 DestSize);
	void RecordDiff(const TCHAR* SectionFilename, int64 LocalOffset, int64 DestAbsoluteOffset, uint8 SourceByte, uint8 DestByte, 
		bool bHasOptimizedHeader);
	void RecordDiff(const TCHAR* SectionFilename, int64 LocalOffset, int64 DestAbsoluteOffset, uint8 SourceByte, uint8 DestByte, 
		int64 DifferenceOffset, const TCHAR* LastDifferenceCallstackDataText, const FString& BeforePropertyVal, const FString& AfterPropertyVal, 
		const FCallstacks::FCallstackData& DifferenceCallstackData);
	void ExtendPreviousDiff(int64 LocalOffset);
	void IncrementPreviousDiff();

	void RecordDiffBytes(const TCHAR* SectionFilename, int32 BytesToLog, int64 LocalOffset, const FPackageData& SourcePackage, 
		const FPackageData& DestPackage);
	void RecordUnreportedDiffs(const TCHAR* SectionFilename, int32 NumUnreportedDiffs, int32 FirstUnreportedDiffIndex);

	void RecordDeterminismDiagnostics(const FString& DeterminismLines);

	const FMessageCallback& GetMessageCallback() const;
	IDetailRecorder* GetDetailRecorder() const;

protected:
	FMessageCallback MessageCallback;
	IDetailRecorder* DetailRecorder;
};





/** Global data (e.g. the FPackageId of every object in /Script) used during diffing */
struct FAccumulatorGlobals
{
public:
	// Zen variables
	TMap<FPackageObjectIndex, FPackageStoreOptimizer::FScriptObjectData> ScriptObjectsMap;

	// Shared variables
	ICookedPackageWriter* PackageWriter = nullptr;
	EPackageHeaderFormat Format = EPackageHeaderFormat::PackageFileSummary;
	bool bInitialized = false;

	TUniquePtr<IDetailRecorder> DetailRecorder;

public:
	FAccumulatorGlobals(ICookedPackageWriter* InnerPackageWriter = nullptr);
	void Initialize(EPackageHeaderFormat Format);
};

/**
 * Collects the memory version of a saved package, compares it with an existing package on disk, and reports callstack
 * for the Serialize call at each offset where they differ.
 * 
 * It works by saving a package twice. The first pass collects the serialization offsets without the stack traces and
 * creates a FDiffMap, and records sizes necessary to remap offsets during Serialize to the final offset in the package
 * on disk. In the second pass, the diff map is read during each call to Serialize to decide whether we need to collect
 * the stack trace for that call.
 */
class FAccumulator : public FRefCountedObject
{
public:
	FAccumulator(FAccumulatorGlobals& InGlobals, UObject* InAsset, FName InPackageName, int32 InMaxDiffsToLog,
		bool bInIgnoreHeaderDiffs, TSharedPtr<FDiffOutputRecorder> InDiffOutputRecorder, EPackageHeaderFormat InPackageHeaderFormat);
	virtual ~FAccumulator();

	void OnFirstSaveComplete(FStringView InLooseFilePath, int64 InHeaderSize, int64 InPreTransformHeaderSize,
		ICookedPackageWriter::FPreviousCookedBytesData&& InPreviousPackageData);
	void OnSecondSaveComplete(int64 InHeaderSize);
	bool HasDifferences() const;

	/** Compares results from the second save with the previous cook results in PreviousPackagedata.  */
	void CompareWithPrevious(const TCHAR* CallstackCutoffText, TMap<FName,FArchiveDiffStats>& OutStats);

	void SetHeaderSize(int64 InHeaderSize);
	void SetDeterminismManager(UE::Cook::FDeterminismManager& InDeterminismManager);
	void SetCollectingCallstacks(bool bInCollectingCallstacks);
	FName GetAssetClass() const;
	bool IsWriterUsingPostSaveTransforms() const;

private:
	void GenerateDiffMapForSection(const FPackageData& SourcePackage, const FPackageData& DestPackage, bool& bOutSectionIdentical);
	void GenerateDiffMap();
	/** Compares two packages and logs the differences and calltacks. */
	void CompareWithPreviousForSection(const FPackageData& SourcePackage, const FPackageData& DestPackage,
		FPackageHeaderData& SourceHeader, FPackageHeaderData& DestHeader,
		const TCHAR* CallstackCutoffText, int32& InOutLoggedDiffs,TMap<FName, FArchiveDiffStats>& OutStats,
		const FString& SectionFilename);

private:
	FCallstacks LinkerCallstacks;
	FCallstacks ExportsCallstacks;
	ICookedPackageWriter::FPreviousCookedBytesData PreviousPackageData;
	FDiffArchive* LinkerArchive = nullptr;
	FDiffArchive* ExportsArchive = nullptr;
	TArray<uint8> FirstSaveLinkerData;
	int64 FirstSaveLinkerSize = 0;
	FAccumulatorGlobals& Globals;
	UE::Cook::FDeterminismManager* DeterminismManager = nullptr;

	FDiffMap DiffMap;
	TSharedPtr<FDiffOutputRecorder> DiffOutputRecorder;
	FName PackageName;
	FString Filename;
	UObject* Asset = nullptr;
	int64 HeaderSize = 0;
	int64 PreTransformHeaderSize = 0;
	int32 MaxDiffsToLog = 5;
	EPackageHeaderFormat PackageHeaderFormat = EPackageHeaderFormat::PackageFileSummary;
	bool bFirstSaveComplete = false;
	bool bHasDifferences = false;
	bool bIgnoreHeaderDiffs = false;

	friend class FCallstacks;
	friend class FDiffArchive;
	friend class FDiffArchiveForLinker;
	friend class FDiffArchiveForExports;
	friend class ::FDiffWriterArchiveTestsCallstacks;
};

class FDiffArchive : public FLargeMemoryWriter
{
public:
	FDiffArchive(FAccumulator& InAccumulator);

	// FLargeMemoryWriterBase interface
	virtual FString GetArchiveName() const override;
	virtual void PushDebugDataString(const FName& DebugData) override;
	virtual void PopDebugDataString() override;

	// FDiffArchive interface
	FAccumulator& GetAccumulator() { return *Accumulator; }
	TArray<FName>& GetDebugDataStack();


protected:
	TArray<FName> DebugDataStack;
	TRefCountPtr<FAccumulator> Accumulator;
};

/**
 * The archive written to by SavePackage, includes the header and exports.
 */
class FDiffArchiveForLinker : public FDiffArchive
{
public:
	FDiffArchiveForLinker(FAccumulator& InAccumulator);
	~FDiffArchiveForLinker();

	// FLargeMemoryWriter interface
	FORCENOINLINE virtual void Serialize(void* InData, int64 Num) override; // FORCENOINLINE so it can be counted during StackTrace
};

/**
 * The archive written to when SavePackage is writing the Serialize blobs for exports.
 * When cooking, exports are serialized into a separate archive. We collect the serialization
 * callstack offsets and stack traces into a separate callstack collection and append it
 * at the proper offset to the overall callstacks for the entire linker archive.
 */
class FDiffArchiveForExports : public FDiffArchive
{
public:
	FDiffArchiveForExports(FAccumulator& InAccumulator);
	~FDiffArchiveForExports();

	// FLargeMemoryWriter interface
	FORCENOINLINE virtual void Serialize(void* InData, int64 Num) override; // FORCENOINLINE so it can be counted during StackTrace
};

} // namespace UE::DiffWriter