// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/DiffWriterLinkerLoadHeader.h"

#include "UObject/LinkerLoad.h"

namespace UE::DiffWriter
{

FDiffWriterLinkerLoadHeader::FDiffWriterLinkerLoadHeader(FLinkerLoad* InLinker, TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder)
	: Linker(InLinker)
	, DiffOutputRecorder(MoveTemp(InDiffOutputRecorder))
{
	Linker = InLinker;
}
FString FDiffWriterLinkerLoadHeader::GetTableKey(const FObjectExport& Export)
{
	FName ClassName = Export.ClassIndex.IsNull() ? FName(NAME_Class) : Linker->ImpExp(Export.ClassIndex).ObjectName;
	return FString::Printf(TEXT("%s %s.%s"),
		*ClassName.ToString(),
		!Export.OuterIndex.IsNull() ? *Linker->ImpExp(Export.OuterIndex).ObjectName.ToString() : *FPackageName::GetShortName(Linker->LinkerRoot),
		*Export.ObjectName.ToString());
}
FString FDiffWriterLinkerLoadHeader::GetTableKey(const FObjectImport& Import)
{
	return FString::Printf(TEXT("%s %s.%s"),
		*Import.ClassName.ToString(),
		!Import.OuterIndex.IsNull() ? *Linker->ImpExp(Import.OuterIndex).ObjectName.ToString() : TEXT("NULL"),
		*Import.ObjectName.ToString());
}
FString FDiffWriterLinkerLoadHeader::GetTableKey(const FName& Name)
{
	return *Name.ToString();
}

FString FDiffWriterLinkerLoadHeader::GetTableKey(FNameEntryId Id)
{
	return FName::GetEntry(Id)->GetPlainNameString();
}

FString FDiffWriterLinkerLoadHeader::GetTableKeyForIndex(FPackageIndex Index)
{
	if (Index.IsNull())
	{
		return TEXT("NULL");
	}
	else if (Index.IsExport())
	{
		return GetTableKey(Linker->Exp(Index));
	}
	else
	{
		return GetTableKey(Linker->Imp(Index));
	}
}

bool FDiffWriterLinkerLoadHeader::CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext,
	const FName& SourceName, const FName& DestName)
{
	return SourceName == DestName;
}

bool FDiffWriterLinkerLoadHeader::CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext,
	FNameEntryId SourceName, FNameEntryId DestName)
{
	return SourceName == DestName;
}

FString FDiffWriterLinkerLoadHeader::ConvertItemToText(const FName& Name)
{
	return Name.ToString();
}

FString FDiffWriterLinkerLoadHeader::ConvertItemToText(FNameEntryId Id)
{
	return FName::GetEntry(Id)->GetPlainNameString();
}

bool FDiffWriterLinkerLoadHeader::CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, const FObjectImport& SourceImport,
	const FObjectImport& DestImport)
{
	if (SourceImport.ObjectName != DestImport.ObjectName ||
		SourceImport.ClassName != DestImport.ClassName ||
		SourceImport.ClassPackage != DestImport.ClassPackage ||
		!ComparePackageIndices(DestContext, SourceImport.OuterIndex,
			DestImport.OuterIndex))
	{
		return false;
	}
	else
	{
		return true;
	}
}

FString FDiffWriterLinkerLoadHeader::ConvertItemToText(const FObjectImport& Import)
{
	return FString::Printf(
		TEXT("%s ClassPackage: %s"),
		*GetTableKey(Import),
		*Import.ClassPackage.ToString()
	);
}

bool FDiffWriterLinkerLoadHeader::CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, const FObjectExport& SourceExport,
	const FObjectExport& DestExport)
{
	if (SourceExport.ObjectName != DestExport.ObjectName ||
		SourceExport.PackageFlags != DestExport.PackageFlags ||
		SourceExport.ObjectFlags != DestExport.ObjectFlags ||
		SourceExport.SerialSize != DestExport.SerialSize ||
		SourceExport.bForcedExport != DestExport.bForcedExport ||
		SourceExport.bNotForClient != DestExport.bNotForClient ||
		SourceExport.bNotForServer != DestExport.bNotForServer ||
		SourceExport.bNotAlwaysLoadedForEditorGame != DestExport.bNotAlwaysLoadedForEditorGame ||
		SourceExport.bIsAsset != DestExport.bIsAsset ||
		SourceExport.bIsInheritedInstance != DestExport.bIsInheritedInstance ||
		SourceExport.bGeneratePublicHash != DestExport.bGeneratePublicHash ||
		!ComparePackageIndices(DestContext, SourceExport.TemplateIndex,
			DestExport.TemplateIndex) ||
		!ComparePackageIndices(DestContext, SourceExport.OuterIndex,
			DestExport.OuterIndex) ||
		!ComparePackageIndices(DestContext, SourceExport.ClassIndex,
			DestExport.ClassIndex) ||
		!ComparePackageIndices(DestContext, SourceExport.SuperIndex,
			DestExport.SuperIndex))
	{
		return false;
	}
	else
	{
		return true;
	}
}

bool FDiffWriterLinkerLoadHeader::IsImportMapIdentical(FDiffWriterLinkerLoadHeader& DestContext)
{
	FLinkerLoad* SourceLinker = this->Linker;
	FLinkerLoad* DestLinker = DestContext.Linker;
	bool bIdentical = (SourceLinker->ImportMap.Num() == DestLinker->ImportMap.Num());
	if (bIdentical)
	{
		for (int32 ImportIndex = 0; ImportIndex < SourceLinker->ImportMap.Num(); ++ImportIndex)
		{
			if (!CompareTableItem(DestContext, SourceLinker->ImportMap[ImportIndex],
				DestLinker->ImportMap[ImportIndex]))
			{
				bIdentical = false;
				break;
			}
		}
	}
	return bIdentical;
}

bool FDiffWriterLinkerLoadHeader::ComparePackageIndices(FDiffWriterLinkerLoadHeader& DestContext, const FPackageIndex& SourceIndex,
	const FPackageIndex& DestIndex)
{
	if (SourceIndex.IsNull() && DestIndex.IsNull())
	{
		return true;
	}
	FDiffWriterLinkerLoadHeader& SourceContext = *this;
	FLinkerLoad* SourceLinker = SourceContext.Linker;
	FLinkerLoad* DestLinker = DestContext.Linker;

	if (SourceIndex.IsExport() && DestIndex.IsExport())
	{
		int32 SourceArrayIndex = SourceIndex.ToExport();
		int32 DestArrayIndex = DestIndex.ToExport();

		if (!SourceLinker->ExportMap.IsValidIndex(SourceArrayIndex) || !DestLinker->ExportMap.IsValidIndex(DestArrayIndex))
		{
			LogMessage(ELogVerbosity::Warning, FString::Printf(
				TEXT("Invalid export indices found, source: %d (of %d), dest: %d (of %d)"),
				SourceArrayIndex, SourceLinker->ExportMap.Num(), DestArrayIndex, DestLinker->ExportMap.Num()));
			return false;
		}

		const FObjectExport& SourceOuterExport = SourceLinker->Exp(SourceIndex);
		const FObjectExport& DestOuterExport = DestLinker->Exp(DestIndex);

		FString SourceOuterExportKey = SourceContext.GetTableKey(SourceOuterExport);
		FString DestOuterExportKey = DestContext.GetTableKey(DestOuterExport);

		return SourceOuterExportKey == DestOuterExportKey;
	}

	if (SourceIndex.IsImport() && DestIndex.IsImport())
	{
		int32 SourceArrayIndex = SourceIndex.ToImport();
		int32 DestArrayIndex = DestIndex.ToImport();

		if (!SourceLinker->ImportMap.IsValidIndex(SourceArrayIndex) || !DestLinker->ImportMap.IsValidIndex(DestArrayIndex))
		{
			LogMessage(ELogVerbosity::Warning, FString::Printf(
				TEXT("Invalid import indices found, source: %d (of %d), dest: %d (of %d)"),
				SourceArrayIndex, SourceLinker->ImportMap.Num(), DestArrayIndex, DestLinker->ImportMap.Num()));
			return false;
		}

		const FObjectImport& SourceOuterImport = SourceLinker->Imp(SourceIndex);
		const FObjectImport& DestOuterImport = DestLinker->Imp(DestIndex);

		FString SourceOuterImportKey = SourceContext.GetTableKey(SourceOuterImport);
		FString DestOuterImportKey = DestContext.GetTableKey(DestOuterImport);

		return SourceOuterImportKey == DestOuterImportKey;
	}

	return false;
}

FString FDiffWriterLinkerLoadHeader::ConvertItemToText(const FObjectExport& Export)
{
	return FString::Printf(TEXT("%s Super: %s, Template: %s, Flags: %d, Size: %lld, PackageFlags: %d, ForcedExport: %d, NotForClient: %d, NotForServer: %d, NotAlwaysLoadedForEditorGame: %d, IsAsset: %d, IsInheritedInstance: %d, GeneratePublicHash: %d"),
		*GetTableKey(Export),
		*GetTableKeyForIndex(Export.SuperIndex),
		*GetTableKeyForIndex(Export.TemplateIndex),
		(int32)Export.ObjectFlags,
		Export.SerialSize,
		Export.PackageFlags,
		Export.bForcedExport,
		Export.bNotForClient,
		Export.bNotForServer,
		Export.bNotAlwaysLoadedForEditorGame,
		Export.bIsAsset,
		Export.bIsInheritedInstance,
		Export.bGeneratePublicHash);
}

bool FDiffWriterLinkerLoadHeader::IsExportMapIdentical(FDiffWriterLinkerLoadHeader& DestContext)
{
	FLinkerLoad* SourceLinker = this->Linker;
	FLinkerLoad* DestLinker = DestContext.Linker;
	bool bIdentical = (SourceLinker->ExportMap.Num() == DestLinker->ExportMap.Num());
	if (bIdentical)
	{
		for (int32 ExportIndex = 0; ExportIndex < SourceLinker->ExportMap.Num(); ++ExportIndex)
		{
			if (!CompareTableItem(DestContext, SourceLinker->ExportMap[ExportIndex],
				DestLinker->ExportMap[ExportIndex]))
			{
				bIdentical = false;
				break;
			}
		}
	}
	return bIdentical;
}

void FDiffWriterLinkerLoadHeader::LogMessage(ELogVerbosity::Type Verbosity, FStringView Message)
{
	DiffOutputRecorder->GetMessageCallback()(Verbosity, Message);
}

}