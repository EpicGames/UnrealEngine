// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiffWriterArchive.h"
#include "Templates/SharedPointer.h"

namespace UE::DiffWriter
{

/**
	* Context object for DumpTableDifferences when we are using EPackageHeaderFormat::PackageFileSummary and so the
	* header information comes from FLinkerLoad.
*/
struct FDiffWriterLinkerLoadHeader
{
public:
	FLinkerLoad* Linker = nullptr;
	TSharedRef<FDiffOutputRecorder> DiffOutputRecorder;

public:
	// Constructor
	FDiffWriterLinkerLoadHeader(FLinkerLoad* InLinker, TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder);

	// API for DumpTableDifferences
	FString GetTableKey(const FObjectExport& Export);
	FString GetTableKey(const FObjectImport& Import);
	FString GetTableKey(const FName& Name);
	FString GetTableKey(FNameEntryId Id);
	FString GetTableKeyForIndex(FPackageIndex Index);
	FString ConvertItemToText(const FObjectExport& Export);
	FString ConvertItemToText(const FName& Name);
	FString ConvertItemToText(FNameEntryId Id);
	FString ConvertItemToText(const FObjectImport& Import);
	bool CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, const FName& SourceName, const FName& DestName);
	bool CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, FNameEntryId SourceName, FNameEntryId DestName);
	bool CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, const FObjectImport& SourceImport,
		const FObjectImport& DestImport);
	bool CompareTableItem(FDiffWriterLinkerLoadHeader& DestContext, const FObjectExport& SourceExport,
		const FObjectExport& DestExport);
	bool ComparePackageIndices(FDiffWriterLinkerLoadHeader& DestContext, const FPackageIndex& SourceIndex,
		const FPackageIndex& DestIndex);
	bool IsImportMapIdentical(FDiffWriterLinkerLoadHeader& DestContext);
	bool IsExportMapIdentical(FDiffWriterLinkerLoadHeader& DestContext);
	void LogMessage(ELogVerbosity::Type Verbosity, FStringView Message);
	const TSharedRef<FDiffOutputRecorder>& GetDiffOutputRecorder() const
	{
		return DiffOutputRecorder;
	}
};

}