// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/DiffWriterZenHeader.h"

#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"

namespace UE::DiffWriter
{

FDiffWriterZenHeader::FDiffWriterZenHeader(FAccumulatorGlobals& InGlobals,
	TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder, bool bInUseZenStoreForImports,
	const FPackageData& Package, const FString& AssetFilename, const TCHAR* WhichComparisonPackage)
	: Globals(InGlobals)
	, DiffOutputRecorder(MoveTemp(InDiffOutputRecorder))
	, bUseZenStoreForImports(bInUseZenStoreForImports)
{
	FString ErrorMessage;
	PackageHeader = FZenPackageHeader::MakeView(FMemoryView(Package.Data, Package.Size), ErrorMessage);
	if (!ErrorMessage.IsEmpty())
	{
		DiffOutputRecorder->GetMessageCallback()(ELogVerbosity::Warning, FString::Printf(
			TEXT("%s: header is different, but cannot display differences. Package data for %s is invalid: %s"),
			*AssetFilename, WhichComparisonPackage, *ErrorMessage));
		PackageHeader = FZenPackageHeader();
		bValid = false;
	}
}

bool FDiffWriterZenHeader::IsValid() const
{
	return bValid;
}

const FZenPackageHeader& FDiffWriterZenHeader::GetPackageHeader() const
{
	return PackageHeader;
}

FStringView FDiffWriterZenHeader::GetObjectIndexPathName(FPackageObjectIndex PackageObjectIndex)
{
	FString* PathName = &ObjectIndexPathNames.FindOrAdd(PackageObjectIndex);
	if (!PathName->IsEmpty())
	{
		return *PathName;
	}

	if (PackageObjectIndex.IsExport())
	{
		uint32 ExportIndex = PackageObjectIndex.ToExport();
		if (!PackageHeader.ExportMap.IsValidIndex(ExportIndex))
		{
			*PathName = TEXT("InvalidExport");
			return *PathName;
		}
		const FExportMapEntry& Export = PackageHeader.ExportMap[ExportIndex];
		if (Export.OuterIndex.IsNull())
		{
			PackageHeader.NameMap.GetName(Export.ObjectName).ToString(*PathName);
		}
		else
		{
			*PathName = TEXT("<Cycle>"); // Set to non-empty to prevent cycles
			FStringView ParentText = GetObjectIndexPathName(Export.OuterIndex);
			// PathName is no longer usable because we potentially modified ObjectIndexPathNames when we called GetObjectIndexPathName
			PathName = ObjectIndexPathNames.Find(PackageObjectIndex);
			check(PathName);
			*PathName = ParentText;
			PathName->AppendChar('/');
			PackageHeader.NameMap.GetName(Export.ObjectName).AppendString(*PathName);
		}
	}
	else if (PackageObjectIndex.IsScriptImport())
	{
		FPackageStoreOptimizer::FScriptObjectData* ObjectData = Globals.ScriptObjectsMap.Find(PackageObjectIndex);
		if (ObjectData)
		{
			*PathName = ObjectData->FullName;
		}
		else
		{
			*PathName = FString::Printf(TEXT("<UnknownScriptImport:%" UINT64_FMT ">"), PackageObjectIndex.Value());
		}
	}
	else if (PackageObjectIndex.IsPackageImport())
	{
		FPackageImportReference Ref = PackageObjectIndex.ToPackageImportRef();
		if (PackageHeader.ImportedPackageNames.IsValidIndex(Ref.GetImportedPackageIndex()))
		{
			FName PackageName = PackageHeader.ImportedPackageNames[Ref.GetImportedPackageIndex()];
			FZenPackageExportsForDiff& PackageExports = ImportedPackageExports.FindOrAdd(PackageName);
			PackageExports.Initialize(Globals, PackageName, bUseZenStoreForImports);
			uint64 ExportHash = PackageHeader.ImportedPublicExportHashes[Ref.GetImportedPublicExportHashIndex()];
			FStringView ExportName = PackageExports.GetExportPackageRelativePath(ExportHash);
			if (ExportName.StartsWith('/'))
			{
				ExportName.RightChopInline(1);
			}
			*PathName = FString::Printf(TEXT("%s.%.*s"),
				*WriteToString<256>(PackageName), ExportName.Len(), ExportName.GetData());
		}
		else
		{
			*PathName = TEXT("<UnknownPackageImport>");
		}
	}
	else
	{
		check(PackageObjectIndex.IsNull());
		*PathName = TEXT("NULL");
	}
	return *PathName;
}

bool FDiffWriterZenHeader::IsNameMapIdentical(const TArray<FName>& SourceNames, const TArray<FName>& DestNames,
	TArray<FString>& OutSourceStrings, TArray<FString>& OutDestStrings)
{
	// Check for case-sensitive equality in the input order first. That is cheap to test, and if it is already
	// equal then we do not need to proceed further.
	int32 NumSourceNames = SourceNames.Num();
	if (NumSourceNames == DestNames.Num())
	{
		bool bPermutationEqual = true;
		for (int32 Index = 0; Index < NumSourceNames; ++Index)
		{
			if (SourceNames[Index].GetDisplayIndex() != DestNames[Index].GetDisplayIndex())
			{
				bPermutationEqual = false;
				break;
			}
		}

		if (bPermutationEqual)
		{
			return true;
		}
	}

	// Convert to strings and sort for the caller's use in displaying the differences.
	// But we might still return true and not use the strings if the lists are the same after sorting.
	OutSourceStrings.Reset(NumSourceNames);
	OutDestStrings.Reset(DestNames.Num());
	for (FName Name : SourceNames)
	{
		OutSourceStrings.Add(Name.ToString());
	}
	for (FName Name : DestNames)
	{
		OutDestStrings.Add(Name.ToString());
	}

	auto StringSortByNoCaseThenCase = [](const FString& A, const FString& B)
		{
			int32 NoCase = A.Compare(B, ESearchCase::IgnoreCase);
			if (NoCase != 0)
			{
				return NoCase < 0;
			}
			int32 Case = A.Compare(B, ESearchCase::CaseSensitive);
			return Case < 0;
		};
	Algo::Sort(OutSourceStrings, StringSortByNoCaseThenCase);
	Algo::Sort(OutDestStrings, StringSortByNoCaseThenCase);

	if (NumSourceNames != DestNames.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < NumSourceNames; ++Index)
	{
		if (OutSourceStrings[Index].Compare(OutDestStrings[Index], ESearchCase::CaseSensitive) != 0)
		{
			return false;
		}
	}
	return true;
}

FString FDiffWriterZenHeader::GetTableKey(const FString& Id)
{
	return Id;
}

bool FDiffWriterZenHeader::CompareTableItem(FDiffWriterZenHeader& DestContext,
	const FString& SourceName, const FString& DestName)
{
	return SourceName.Compare(DestName, ESearchCase::CaseSensitive) == 0;
}

FString FDiffWriterZenHeader::ConvertItemToText(const FString& Id)
{
	return Id;
}

bool FDiffWriterZenHeader::IsImportMapIdentical(FDiffWriterZenHeader& DestContext)
{
	int32 NumImports = PackageHeader.ImportMap.Num();
	if (NumImports != DestContext.PackageHeader.ImportMap.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < NumImports; ++Index)
	{
		if (!CompareTableItem(DestContext, PackageHeader.ImportMap[Index],
			DestContext.PackageHeader.ImportMap[Index]))
		{
			return false;
		}
	}
	return true;
}

FString FDiffWriterZenHeader::GetTableKey(FPackageObjectIndex Id)
{
	// TODO: When Id is an import, we could calculate the key more efficiently by changing it to the PackageId + ExportHash instead
	// of looking up the name of the ExportHash
	return FString(GetObjectIndexPathName(Id));
}

bool FDiffWriterZenHeader::CompareTableItem(FDiffWriterZenHeader& DestContext, const FPackageObjectIndex& SourceIndex,
	const FPackageObjectIndex& DestIndex)
{
	if (SourceIndex.IsImport())
	{
		if (!DestIndex.IsImport())
		{
			return false;
		}
		// TODO: we could compare more efficiently by comparing PackageId + ExportHash instead
		// of looking up the name of the ExportHash
		return GetObjectIndexPathName(SourceIndex) == DestContext.GetObjectIndexPathName(DestIndex);
	}
	else if (SourceIndex.IsExport())
	{
		if (!DestIndex.IsExport())
		{
			return false;
		}
		return GetObjectIndexPathName(SourceIndex) == DestContext.GetObjectIndexPathName(DestIndex);
	}
	else
	{
		check(SourceIndex.IsNull());
		return DestIndex.IsNull();
	}
}

FString FDiffWriterZenHeader::ConvertItemToText(FPackageObjectIndex Id)
{
	return FString(GetObjectIndexPathName(Id));
}

bool FDiffWriterZenHeader::IsExportMapIdentical(FDiffWriterZenHeader& DestContext)
{
	int32 NumExports = PackageHeader.ExportMap.Num();
	if (NumExports != DestContext.PackageHeader.ExportMap.Num())
	{
		return false;
	}

	for (int32 ExportIndex = 0; ExportIndex < NumExports; ++ExportIndex)
	{
		FZenHeaderIndexIntoExportMap ExportIndexStruct{ ExportIndex };
		if (!CompareTableItem(DestContext, ExportIndexStruct, ExportIndexStruct))
		{
			return false;
		}
	}
	return true;
}

FString FDiffWriterZenHeader::GetTableKey(const FZenHeaderIndexIntoExportMap& Index)
{
	return FString(GetObjectIndexPathName(FPackageObjectIndex::FromExportIndex(Index.Index)));
}

bool FDiffWriterZenHeader::CompareTableItem(FDiffWriterZenHeader& DestContext,
	const FZenHeaderIndexIntoExportMap& SourceExportIndex, const FZenHeaderIndexIntoExportMap& DestExportIndex)
{
	if (!PackageHeader.ExportMap.IsValidIndex(SourceExportIndex.Index) ||
		!DestContext.PackageHeader.ExportMap.IsValidIndex(DestExportIndex.Index))
	{
		return false;
	}
	const FExportMapEntry& SourceExport = PackageHeader.ExportMap[SourceExportIndex.Index];
	const FExportMapEntry& DestExport = DestContext.PackageHeader.ExportMap[DestExportIndex.Index];

	// Ignore CookedSerialOffset; it is irrelevant to the comparison of the export
	if (SourceExport.CookedSerialSize != DestExport.CookedSerialSize) return false;
	if (PackageHeader.NameMap.GetName(SourceExport.ObjectName) !=
		DestContext.PackageHeader.NameMap.GetName(DestExport.ObjectName)) return false;
	if (!CompareTableItem(DestContext, SourceExport.OuterIndex, DestExport.OuterIndex)) return false;
	if (!CompareTableItem(DestContext, SourceExport.ClassIndex, DestExport.ClassIndex)) return false;
	if (!CompareTableItem(DestContext, SourceExport.SuperIndex, DestExport.SuperIndex)) return false;
	if (!CompareTableItem(DestContext, SourceExport.TemplateIndex, DestExport.TemplateIndex)) return false;
	if (SourceExport.PublicExportHash != DestExport.PublicExportHash) return false;
	if (SourceExport.ObjectFlags != DestExport.ObjectFlags) return false;
	if (SourceExport.FilterFlags != DestExport.FilterFlags) return false;

	return true;
}

FString FDiffWriterZenHeader::ConvertItemToText(const FZenHeaderIndexIntoExportMap& Index)
{
	if (!PackageHeader.ExportMap.IsValidIndex(Index.Index))
	{
		return TEXT("<InvalidExport>");
	}
	const FExportMapEntry& Export = PackageHeader.ExportMap[Index.Index];
	FString ClassName = FString(GetObjectIndexPathName(Export.ClassIndex));
	FString PathName = FString(GetObjectIndexPathName(FPackageObjectIndex::FromExportIndex(Index.Index)));
	FString SuperName = FString(GetObjectIndexPathName(Export.SuperIndex));
	FString TemplateName = FString(GetObjectIndexPathName(Export.TemplateIndex));

	return FString::Printf(TEXT("%s %s Super: %s, Template: %s, Flags: %d, Size: %" INT64_FMT ", FilterFlags: %d"),
		*ClassName,
		*PathName,
		*SuperName,
		*TemplateName,
		(int32)Export.ObjectFlags,
		Export.CookedSerialSize,
		(int32)Export.FilterFlags);
}

bool FDiffWriterZenHeader::IsExportBundlesIdentical(FDiffWriterZenHeader& DestContext)
{
	int32 NumBundles = PackageHeader.ExportBundleEntries.Num();
	if (NumBundles != DestContext.PackageHeader.ExportBundleEntries.Num())
	{
		return false;
	}

	for (int32 BundleIndex = 0; BundleIndex < NumBundles; ++BundleIndex)
	{
		if (!IsExportBundleIdentical(DestContext, BundleIndex))
		{
			return false;
		}
	}
	return true;
}

bool FDiffWriterZenHeader::IsExportBundleIdentical(FDiffWriterZenHeader& DestContext, int32 Index)
{
	if (!PackageHeader.ExportBundleEntries.IsValidIndex(Index) ||
		!DestContext.PackageHeader.ExportBundleEntries.IsValidIndex(Index))
	{
		return false;
	}
	const FExportBundleEntry& Src = PackageHeader.ExportBundleEntries[Index];
	const FExportBundleEntry& Dest = DestContext.PackageHeader.ExportBundleEntries[Index];
	return Src.LocalExportIndex == Dest.LocalExportIndex
		&& Src.CommandType == Dest.CommandType;
}

FString FDiffWriterZenHeader::ConvertExportBundleToText(int32 Index)
{
	if (!PackageHeader.ExportBundleEntries.IsValidIndex(Index))
	{
		return TEXT("<InvalidIndex>");
	}
	const FExportBundleEntry& Entry = PackageHeader.ExportBundleEntries[Index];
	return FString::Printf(TEXT("[%d,%d]"), Entry.LocalExportIndex, Entry.CommandType);
}

bool FDiffWriterZenHeader::IsDependencyBundlesIdentical(FDiffWriterZenHeader& DestContext)
{
	int32 NumBundles = PackageHeader.DependencyBundleHeaders.Num();
	if (NumBundles != DestContext.PackageHeader.DependencyBundleHeaders.Num())
	{
		return false;
	}

	for (int32 BundleIndex = 0; BundleIndex < NumBundles; ++BundleIndex)
	{
		if (!IsDependencyBundleIdentical(DestContext, BundleIndex))
		{
			return false;
		}
	}
	return true;
}

bool FDiffWriterZenHeader::IsDependencyBundleIdentical(FDiffWriterZenHeader& DestContext, int32 Index)
{
	if (!PackageHeader.DependencyBundleHeaders.IsValidIndex(Index) ||
		!DestContext.PackageHeader.DependencyBundleHeaders.IsValidIndex(Index))
	{
		return false;
	}
	const FDependencyBundleHeader& SrcHeader = PackageHeader.DependencyBundleHeaders[Index];
	const FDependencyBundleHeader& DestHeader = DestContext.PackageHeader.DependencyBundleHeaders[Index];
	if (SrcHeader.FirstEntryIndex == -1)
	{
		return DestHeader.FirstEntryIndex == -1;
	}
	if (DestHeader.FirstEntryIndex == -1)
	{
		return false;
	}
	TArrayView<const FDependencyBundleEntry> SrcEntries = PackageHeader.DependencyBundleEntries;
	TArrayView<const FDependencyBundleEntry> DestEntries = DestContext.PackageHeader.DependencyBundleEntries;
	int32 SrcIndex = SrcHeader.FirstEntryIndex;
	int32 DestIndex = DestHeader.FirstEntryIndex;
	for (uint32 FromType = 0; FromType < (uint32)FExportBundleEntry::ExportCommandType_Count; ++FromType)
	{
		for (uint32 ToType = 0; ToType < (uint32)FExportBundleEntry::ExportCommandType_Count; ++ToType)
		{
			int32 EntryCount = SrcHeader.EntryCount[FromType][ToType];
			if (EntryCount != DestHeader.EntryCount[FromType][ToType])
			{
				return false;
			}
			for (int32 IndexWithinEntry = 0; IndexWithinEntry < EntryCount; ++IndexWithinEntry)
			{
				if (SrcIndex >= SrcEntries.Num() || DestIndex >= DestEntries.Num())
				{
					return false;
				}
				if (SrcEntries[SrcIndex].LocalImportOrExportIndex != DestEntries[DestIndex].LocalImportOrExportIndex)
				{
					return false;
				}
				++SrcIndex;
				++DestIndex;
			}
		}
	}
	return true;
}

FString FDiffWriterZenHeader::ConvertDependencyBundleToText(int32 Index)
{
	if (!PackageHeader.DependencyBundleEntries.IsValidIndex(Index))
	{
		return TEXT("<InvalidIndex>");
	}
	TStringBuilder<256> Result;

	const FDependencyBundleHeader& BundleHeader = PackageHeader.DependencyBundleHeaders[Index];
	TArrayView<const FDependencyBundleEntry> Entries = PackageHeader.DependencyBundleEntries;
	int32 EntryIndex = BundleHeader.FirstEntryIndex;
	for (uint32 FromType = 0; FromType < (uint32)FExportBundleEntry::ExportCommandType_Count; ++FromType)
	{
		for (uint32 ToType = 0; ToType < (uint32)FExportBundleEntry::ExportCommandType_Count; ++ToType)
		{
			Result << TEXT("[");
			int32 CountWithinEntry = BundleHeader.EntryCount[FromType][ToType];
			if (Entries.IsValidIndex(EntryIndex) && CountWithinEntry > 0)
			{
				for (int32 IndexWithinEntry = 0; IndexWithinEntry < CountWithinEntry; ++IndexWithinEntry)
				{
					if (Entries.IsValidIndex(EntryIndex))
					{
						FPackageIndex PackageIndex = Entries[EntryIndex].LocalImportOrExportIndex;
						if (PackageIndex.IsExport())
						{
							Result << TEXT("Export[") << PackageIndex.ToExport() << TEXT("]");
						}
						else
						{
							Result << TEXT("Import[") << PackageIndex.ToImport() << TEXT("]");
						}
						Result << ",";
						++EntryIndex;
					}
				}
				Result.RemoveSuffix(1);
			}
			Result << TEXT("],");
		}
	}
	Result.RemoveSuffix(1);
	return FString(Result);
}

void FDiffWriterZenHeader::LogMessage(ELogVerbosity::Type Verbosity, FStringView Message)
{
	DiffOutputRecorder->GetMessageCallback()(Verbosity, Message);
}

void FZenPackageExportsForDiff::Initialize(FAccumulatorGlobals& Globals, FName InPackageName, bool bUseZenStore)
{
	if (!PackageName.IsNone())
	{
		return;
	}
	PackageName = InPackageName;
	ExportPaths.Reset();
	if (bUseZenStore)
	{
		if (TryInitializeFromZenStore(Globals))
		{
			return;
		}
	}

	// Silently ignore if TryInitializeFromMemory fails; GetExportName will report <UnknownExport.%u>
	(void) TryInitializeFromMemory(Globals);
}

bool FZenPackageExportsForDiff::TryInitializeFromZenStore(FAccumulatorGlobals& Globals)
{
	if (!Globals.PackageWriter)
	{
		return false;
	}

	FPackageId PackageId = FPackageId::FromName(PackageName);
	IPackageWriter::FPackageInfo PackageInfo;
	uint16 MultiOutputIndex = 0; // We're reading exports from the non-optional version of the package
	PackageInfo.ChunkId = CreateIoChunkId(PackageId.Value(), MultiOutputIndex, EIoChunkType::ExportBundleData);
	ICookedPackageWriter::FPreviousCookedBytesData PackageBytes;
	if (!Globals.PackageWriter->GetPreviousCookedBytes(PackageInfo, PackageBytes))
	{
		return false;
	}
	FString Error;
	FZenPackageHeader PackageHeader = FZenPackageHeader::MakeView(
		FMemoryView(PackageBytes.Data.Get(), PackageBytes.Size), Error);
	if (!Error.IsEmpty())
	{
		return false;
	}

	TArray<FString> LocalPackageRelativePaths;
	FString EmptyPath;
	FString UnknownImportPath(TEXT("<UnknownImportInOtherPackage>"));
	FString InvalidExportIndex(TEXT("<InvalidExportIndex>"));
	FString InvalidCycleText(TEXT("<Cycle>"));
	TFunction<const FString& (FPackageObjectIndex)> FindOrConstructPackageRelativePath;
	FindOrConstructPackageRelativePath =
		[&FindOrConstructPackageRelativePath /** Recursive calls */, &LocalPackageRelativePaths, &PackageHeader, &EmptyPath,
		&UnknownImportPath, &InvalidExportIndex, &InvalidCycleText]
	(FPackageObjectIndex Index) -> const FString&
	{
		if (Index.IsNull())
		{
			return EmptyPath;
		}
		if (Index.IsExport())
		{
			uint32 ExportIndex = Index.ToExport();
			if (!LocalPackageRelativePaths.IsValidIndex(ExportIndex))
			{
				return InvalidExportIndex;
			}
			FString& PackageRelativePath = LocalPackageRelativePaths[ExportIndex];
			if (!PackageRelativePath.IsEmpty())
			{
				return PackageRelativePath;
			}
			// Avoid cycles by setting ObjectName before the recursive call to FindOrConstructPackageRelativePath
			PackageRelativePath = InvalidCycleText;

			check(PackageHeader.ExportMap.IsValidIndex(ExportIndex)); // LocalExportNames is the same length and was checked above
			const FExportMapEntry& Export = PackageHeader.ExportMap[ExportIndex];
			FName LeafName;
			if (!PackageHeader.NameMap.TryGetName(Export.ObjectName, LeafName) || LeafName.IsNone())
			{
				PackageRelativePath = InvalidExportIndex;
				return PackageRelativePath;
			}
			const FString& ParentPackageRelativePath = FindOrConstructPackageRelativePath(Export.OuterIndex);

			// Every path name is ParentPath (even if empty) + / + LeafName. PackageRelativePaths when parent is empty start with / 
			PackageRelativePath = ParentPackageRelativePath;
			PackageRelativePath.Append(TEXT("/"));
			LeafName.AppendString(PackageRelativePath);

			// Down-case the path names to match the behavior for the path names constructed from UObjects in memory
			// by FPackageStoreOptimizer::AppendPathForPublicExportHash
			PackageRelativePath.ToLowerInline();

			return PackageRelativePath;
		}
		else
		{
			return UnknownImportPath;
		}
	};

	uint32 NumExports = static_cast<uint32>(PackageHeader.ExportMap.Num());
	LocalPackageRelativePaths.SetNum(NumExports);
	for (uint32 ExportIndex = 0; ExportIndex < NumExports; ++ExportIndex)
	{
		FindOrConstructPackageRelativePath(FPackageObjectIndex::FromExportIndex(ExportIndex));
	}

	ExportPaths.Reserve(NumExports);
	for (uint32 ExportIndex = 0; ExportIndex < NumExports; ++ExportIndex)
	{
		const FExportMapEntry& Export = PackageHeader.ExportMap[ExportIndex];
		ExportPaths.Add(Export.PublicExportHash, LocalPackageRelativePaths[ExportIndex]);
	}

	return true;
}

bool FZenPackageExportsForDiff::TryInitializeFromMemory(FAccumulatorGlobals& Globals)
{
	UPackage* Package = FindPackage(nullptr, *WriteToString<256>(PackageName));
	if (!Package)
	{
		return false;
	}

	// We do not add the UPackage object itself, because UPackage object is not stored in the list
	// of exports in the disk version of the package
	ForEachObjectWithOuter(Package, [this](UObject* Object)
		{
			TStringBuilder<256> PackageRelativePath;
			FPackageStoreOptimizer::AppendPathForPublicExportHash(Object, PackageRelativePath);
			uint64 PublicExportHash;
			if (FPackageStoreOptimizer::TryGetPublicExportHash(PackageRelativePath, PublicExportHash))
			{
				ExportPaths.Add(PublicExportHash, FString(PackageRelativePath));
			}
		}, EGetObjectsFlags::IncludeNestedObjects);

	return true;
}

FStringView FZenPackageExportsForDiff::GetExportPackageRelativePath(uint64 PublicExportHash)
{
	FString& ExportPath = ExportPaths.FindOrAdd(PublicExportHash);
	if (ExportPath.IsEmpty())
	{
		ExportPath = FString::Printf(TEXT("<UnknownExport.%" UINT64_FMT ">"), PublicExportHash);
	}
	return ExportPath;
}

}