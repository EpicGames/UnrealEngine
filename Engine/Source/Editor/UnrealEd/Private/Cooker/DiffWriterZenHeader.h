// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiffWriterArchive.h"
#include "Serialization/ZenPackageHeader.h"
#include "Templates/SharedPointer.h"

namespace UE::DiffWriter
{

/**
 * Helper object for FDiffWriterZenHeader: Stores the map from PublicExportHash to package-relative
 * objectpath for all of the objects in a package, used to convert a ZenPackageHeader's import's
 * FPackageObjectIndex into the full path to the imported object
 */
class FZenPackageExportsForDiff
{
public:
	void Initialize(FAccumulatorGlobals& Globals, FName InPackageName, bool bUseZenStore);
	FStringView GetExportPackageRelativePath(uint64 PublicExportHash);

private:
	bool TryInitializeFromZenStore(FAccumulatorGlobals& Globals);
	bool TryInitializeFromMemory(FAccumulatorGlobals& Globals);

private:
	FName PackageName = NAME_None;
	TMap<uint64, FString> ExportPaths;
};

/** Adapter type for FDiffWriterZenHeader to represent its exports in DumpTableDifferences. */
struct FZenHeaderIndexIntoExportMap
{
	/** Index of the given export in FZenPackageHeader::ExportMap. */
	int32 Index;
};

/**
 * Context object for DumpTableDifferences when we are using EPackageHeaderFormat::ZenPackageSummary and so the
 * header information comes from ZenStore's optimized package format parsed into FZenPackageHeader.
*/
class FDiffWriterZenHeader
{
public:
	// Constructor and simple data readers
	FDiffWriterZenHeader(FAccumulatorGlobals& InGlobals, TSharedRef<FDiffOutputRecorder> InDiffOutputRecorder,
		bool bInUseZenStoreForImports, const FPackageData& Package, const FString& AssetFilename,
		const TCHAR* WhichComparisonPackage);
	bool IsValid() const;
	const FZenPackageHeader& GetPackageHeader() const;

	// Optional data loaded on demand
	FStringView GetObjectIndexPathName(FPackageObjectIndex PackageObjectIndex);

	// API for DumpTableDifferences
	bool IsNameMapIdentical(const TArray<FName>& SourceNames, const TArray<FName>& DestNames,
		TArray<FString>& OutSourceStrings, TArray<FString>& OutDestStrings);
	FString GetTableKey(const FString& Id);
	bool CompareTableItem(FDiffWriterZenHeader& DestContext, const FString& SourceText, const FString& DestText);
	FString ConvertItemToText(const FString& Id);

	bool IsImportMapIdentical(FDiffWriterZenHeader& DestContext);
	FString GetTableKey(FPackageObjectIndex Id);
	bool CompareTableItem(FDiffWriterZenHeader& DestContext, const FPackageObjectIndex& SourceIndex,
		const FPackageObjectIndex& DestIndex);
	FString ConvertItemToText(FPackageObjectIndex Id);

	bool IsExportMapIdentical(FDiffWriterZenHeader& DestContext);
	FString GetTableKey(const FZenHeaderIndexIntoExportMap& Id);
	bool CompareTableItem(FDiffWriterZenHeader& DestContext, const FZenHeaderIndexIntoExportMap& SourceExport,
		const FZenHeaderIndexIntoExportMap& DestExport);
	FString ConvertItemToText(const FZenHeaderIndexIntoExportMap& Id);

	bool IsExportBundlesIdentical(FDiffWriterZenHeader& DestContext);
	bool IsExportBundleIdentical(FDiffWriterZenHeader& DestContext, int32 Index);
	FString ConvertExportBundleToText(int32 Index);

	bool IsDependencyBundlesIdentical(FDiffWriterZenHeader& DestContext);
	bool IsDependencyBundleIdentical(FDiffWriterZenHeader& DestContext, int32 Index);
	FString ConvertDependencyBundleToText(int32 Index);

	void LogMessage(ELogVerbosity::Type Verbosity, FStringView Message);

	const TSharedRef<FDiffOutputRecorder>& GetDiffOutputRecorder() const
	{
		return DiffOutputRecorder;
	}

private:
	FZenPackageHeader PackageHeader;
	TMap<FName, FZenPackageExportsForDiff> ImportedPackageExports;
	TMap<FPackageObjectIndex, FString> ObjectIndexPathNames;
	FAccumulatorGlobals& Globals;
	TSharedRef<FDiffOutputRecorder> DiffOutputRecorder;
	bool bUseZenStoreForImports = false;
	bool bValid = true;
};

}