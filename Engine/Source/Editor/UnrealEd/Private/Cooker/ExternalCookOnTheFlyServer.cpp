// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalCookOnTheFlyServer.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "CookOnTheFly.h"
#include "CookOnTheFlyMessages.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformProcess.h"
#include "IMessageContext.h"
#include "IO/PackageId.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExternalCookOnTheFlyServer)

FExternalCookOnTheFlyServer::FExternalCookOnTheFlyServer()
	: CookOnTheFlyModule(FModuleManager::LoadModuleChecked<UE::Cook::ICookOnTheFlyModule>(TEXT("CookOnTheFly")))
	, AssetRegistry(FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get())
	, ServiceId(GenerateServiceId())
{
	MessageEndpoint = FMessageEndpoint::Builder("FCookOnTheFly")
		.Handling<FZenCookOnTheFlyRegisterServiceMessage>(this, &FExternalCookOnTheFlyServer::HandleRegisterServiceMessage)
		.ReceivingOnThread(ENamedThreads::GameThread);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe<FZenCookOnTheFlyRegisterServiceMessage>();
	}
}

FExternalCookOnTheFlyServer::~FExternalCookOnTheFlyServer()
{
	AssetRegistry.OnAssetUpdatedOnDisk().RemoveAll(this);
}

void FExternalCookOnTheFlyServer::HandleRegisterServiceMessage(const FZenCookOnTheFlyRegisterServiceMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.ServiceId != ServiceId)
	{
		UE_LOGF(LogCookOnTheFly, Verbose, "Rejecting service from %ls", *Context->GetSender().ToString());
		return;
	}
	UE_LOGF(LogCookOnTheFly, Verbose, "Accepting service from %ls", *Context->GetSender().ToString());
	UE::Cook::FCookOnTheFlyHostOptions HostOptions;
	HostOptions.Hosts.Add(FString::Printf(TEXT("127.0.0.1:%d"), Message.Port));
	
	CookOnTheFlyServerConnection = CookOnTheFlyModule.ConnectToServer(HostOptions);
	if (CookOnTheFlyServerConnection)
	{
		UE_LOGF(LogCookOnTheFly, Display, "Connected to server");
		AssetRegistry.OnAssetUpdatedOnDisk().AddRaw(this, &FExternalCookOnTheFlyServer::AssetUpdatedOnDisk);
	}
	else
	{
		UE_LOGF(LogCookOnTheFly, Warning, "Failed connecting to server");
	}
}

void FExternalCookOnTheFlyServer::Tick(float DeltaSeconds)
{
	{
		FScopeLock _(&AllPackagesToRecookCritical);
		if (!AllPackagesToRecook.IsEmpty())
		{
			TArray<FPackageId> PackageIdsToRecook;
			PackageIdsToRecook.Reserve(AllPackagesToRecook.Num());
			for (FName PackageName : AllPackagesToRecook)
			{
				PackageIdsToRecook.Add(FPackageId::FromName(PackageName));
			}
			AllPackagesToRecook.Empty();

			UE::Cook::FCookOnTheFlyRequest Request(UE::Cook::ECookOnTheFlyMessage::RecookPackages);
			Request.SetBodyTo(UE::ZenCookOnTheFly::Messaging::FRecookPackagesRequest{ MoveTemp(PackageIdsToRecook) });
			UE::Cook::FCookOnTheFlyResponse Response = CookOnTheFlyServerConnection->SendRequest(Request).Get();
			if (!Response.IsOk())
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Failed to send RecookPackages request");
			}
		}
	}
	if (CookOnTheFlyServerConnection && !CookOnTheFlyServerConnection->IsConnected())
	{
		AssetRegistry.OnAssetUpdatedOnDisk().RemoveAll(this);
		CookOnTheFlyServerConnection.Reset();
		UE_LOGF(LogCookOnTheFly, Display, "Disconnected from server");
	}
}

bool FExternalCookOnTheFlyServer::IsTickable() const
{
	return CookOnTheFlyServerConnection.IsValid();
}

void FExternalCookOnTheFlyServer::AssetUpdatedOnDisk(const FAssetData& AssetData)
{
	FScopeLock _(&AllPackagesToRecookCritical);
	if (AllPackagesToRecook.Contains(AssetData.PackageName))
	{
		return;
	}

	TArray<FName> ModifiedPackagesToRecurse;
	ModifiedPackagesToRecurse.Push(AssetData.PackageName);
	AllPackagesToRecook.Add(AssetData.PackageName);
	while (!ModifiedPackagesToRecurse.IsEmpty())
	{
		FName ModifiedPackage = ModifiedPackagesToRecurse.Pop(EAllowShrinking::No);
		TArray<FName> Referencers;
		AssetRegistry.GetReferencers(ModifiedPackage, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
		for (FName Referencer : Referencers)
		{
			if (!AllPackagesToRecook.Contains(Referencer))
			{
				ModifiedPackagesToRecurse.Push(Referencer);
				AllPackagesToRecook.Add(Referencer);
			}
		}
	}
}

FString FExternalCookOnTheFlyServer::GenerateServiceId()
{
	TStringBuilder<256> ServiceIdString;
	ServiceIdString.Append(FPlatformProcess::UserName());
	ServiceIdString.Append("@");
	ServiceIdString.Append(FPlatformProcess::ComputerName());
	ServiceIdString.Append(":");
	FPathViews::ToAbsolutePath(FPaths::GetProjectFilePath(), ServiceIdString);
	return FMD5::HashBytes((unsigned char*)*ServiceIdString, ServiceIdString.Len() * sizeof(TCHAR));
}
