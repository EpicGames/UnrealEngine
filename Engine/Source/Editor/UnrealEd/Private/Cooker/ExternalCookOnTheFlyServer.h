// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "TickableEditorObject.h"
#include "UObject/NameTypes.h"

#include "ExternalCookOnTheFlyServer.generated.h"

class FMessageEndpoint;
class IAssetRegistry;
class IMessageContext;
namespace UE::Cook { class ICookOnTheFlyModule; }
namespace UE::Cook { class ICookOnTheFlyServerConnection; }
struct FAssetData;
struct FZenCookOnTheFlyRegisterServiceMessage;

class FExternalCookOnTheFlyServer
	: public FTickableEditorObject
{
public:
	FExternalCookOnTheFlyServer();
	~FExternalCookOnTheFlyServer();
	void HandleRegisterServiceMessage(const FZenCookOnTheFlyRegisterServiceMessage& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	static FString GenerateServiceId();

private:
	virtual void Tick(float DeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override { return TStatId(); }

	void AssetUpdatedOnDisk(const FAssetData& AssetData);

	UE::Cook::ICookOnTheFlyModule& CookOnTheFlyModule;
	IAssetRegistry& AssetRegistry;
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	const FString ServiceId;
	TUniquePtr<UE::Cook::ICookOnTheFlyServerConnection> CookOnTheFlyServerConnection;
	FCriticalSection AllPackagesToRecookCritical;
	TSet<FName> AllPackagesToRecook;
};

USTRUCT()
struct FZenCookOnTheFlyRegisterServiceMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Message")
	FString ServiceId;

	UPROPERTY(EditAnywhere, Category = "Message")
	int32 Port = -1;
};