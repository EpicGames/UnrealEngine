// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Pointer.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NotNull.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Cook
{

/**
 * A container with API similar to TSet<T*> that supports iteration order == insertion order, with extra costs during
 * Add and Pop to handle the ordering; these costs are small when amortized.
 * This container only supports pointers, and nullptr is used as a special value and is not permitted in the container.
 */
template <UE::CPointer T>
class TFifoSet
{
public:
	/**
	 * Add Key to the container if it does not already exist.
	 * nullptr is a special value and is not permitted in the container.
	 */
	void Add(TNotNull<T> Key);
	/** Remove Key if it exists in the container and return 1 if it does, else 0. */
	int32 Remove(const T& Key);
	bool Contains(const T& Key) const;
	/** Remove and return the oldest element in the container. Returns null if empty. */
	T Pop();
	void Empty();
	int32 Num() const;
	bool IsEmpty() const;

private:
	struct FIterator
	{
	public:
		explicit FIterator(const TFifoSet<T>& InSet, int32 InIndex);

		/** Used only in range for. Assumes that Other is an iterator from the same set. */
		bool operator!=(const FIterator& Other) const;
		const T& operator*() const;
		FIterator& operator++();
	private:
		const TFifoSet<T>& Set;
		int32 Index;
	};

	// Begin and end are implemented as hidden friends to allow them in ranged for but not direct calls.
	friend FIterator begin(const TFifoSet<T>& Set)
	{
		return TFifoSet<T>::FIterator(Set, -1);
	}
	friend FIterator end(const TFifoSet<T>& Set)
	{
		return TFifoSet<T>::FIterator(Set, Set.PopOrder.Num());
	}
	void ReIndex();

	TMap<T, int32> KeyToIndex;
	TArray<T> PopOrder;
	int32 NextPop = 0;
};

template <UE::CPointer T>
void TFifoSet<T>::Add(TNotNull<T> Key)
{
	int32* StoredIndex = &KeyToIndex.FindOrAdd(Key, PopOrder.Num());
	if (*StoredIndex < PopOrder.Num())
	{
		// Already exists
		return;
	}

	if (PopOrder.Num() == PopOrder.Max() && PopOrder.Max() > 0)
	{
		// We are about to increase the size of PopOrder. Check whether we should instead strip the nulls out of it
		// and rewrite the stored indices because we have removed enough elements to make that worthwhile.
		constexpr int32 MinimumRatioToBeWorthReIndex = 2;
		if (KeyToIndex.Num() * MinimumRatioToBeWorthReIndex < PopOrder.Max())
		{
			ReIndex();

			// ReIndex might have modified KeyToIndex, so refetch our pointer to its internal memory.
			StoredIndex = KeyToIndex.Find(Key);
			check(StoredIndex != nullptr);

			// Assign the new index to end of PopOrder into the StoredIndex. ReIndex did not already do this
			// because Key was not yet present in PopOrder.
			*StoredIndex = PopOrder.Num();
		}
	}

	// We already set StoredIndex above, just assign Key into that index (the end of PopOrder).
	PopOrder.Add(Key);
}

template <UE::CPointer T>
int32 TFifoSet<T>::Remove(const T& Key)
{
	int32 Index;
	if (KeyToIndex.RemoveAndCopyValue(Key, Index))
	{
		check(PopOrder[Index] == Key);
		PopOrder[Index] = nullptr;
		return 1;
	}
	return 0;
}

template <UE::CPointer T>
bool TFifoSet<T>::Contains(const T& Key) const
{
	return KeyToIndex.Contains(Key);
}

template <UE::CPointer T>
T TFifoSet<T>::Pop()
{
	while (NextPop < PopOrder.Num())
	{
		if (!PopOrder[NextPop])
		{
			++NextPop;
			continue;
		}
		T Result = MoveTemp(PopOrder[NextPop]);
		PopOrder[NextPop] = nullptr;
		int32 StoredIndex = 0;
		bool bRemoved = KeyToIndex.RemoveAndCopyValue(Result, StoredIndex);
		check(bRemoved && StoredIndex == NextPop);
		++NextPop;
		return Result;
	}
	return nullptr;
}

template <UE::CPointer T>
void TFifoSet<T>::Empty()
{
	KeyToIndex.Empty();
	PopOrder.Empty();
	NextPop = 0;
}

template <UE::CPointer T>
int32 TFifoSet<T>::Num() const
{
	return KeyToIndex.Num();
}

template <UE::CPointer T>
bool TFifoSet<T>::IsEmpty() const
{
	return KeyToIndex.IsEmpty();
}

template <UE::CPointer T>
void TFifoSet<T>::ReIndex()
{
	// Note that ReIndex is called during Add and KeyToIndex has one element in it that is not yet
	// present in PopOrder.
	TArray<T> NewPopOrder;
	NewPopOrder.Reserve(KeyToIndex.Num());
	for (const T& Key : PopOrder)
	{
		if (Key)
		{
			int32* Index = KeyToIndex.Find(Key);
			check(Index != nullptr);
			*Index = NewPopOrder.Num();
			NewPopOrder.Add(Key);
		}
	}
	PopOrder = MoveTemp(NewPopOrder);
	NextPop = 0;
}

template <UE::CPointer T>
TFifoSet<T>::FIterator::FIterator(const TFifoSet<T>& InSet, int32 InIndex)
	: Set(InSet)
	, Index(InIndex)
{
	if (Index < 0)
	{
		this->operator++();
	}
}

template <UE::CPointer T>
bool TFifoSet<T>::FIterator::operator!=(const TFifoSet<T>::FIterator& Other) const
{
	return Index != Other.Index;
}

template <UE::CPointer T>
const T& TFifoSet<T>::FIterator::operator*() const
{
	return Set.PopOrder[Index];
}

template <UE::CPointer T>
TFifoSet<T>::FIterator& TFifoSet<T>::FIterator::operator++()
{
	check(Index < Set.PopOrder.Num());
	++Index;
	while (Index < Set.PopOrder.Num() && Set.PopOrder[Index] == nullptr)
	{
		++Index;
	}
	return *this;
}

} // namespace UE::Cook