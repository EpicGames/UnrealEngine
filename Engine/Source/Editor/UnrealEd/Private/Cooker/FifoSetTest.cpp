// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "Cooker/FifoSet.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFifoSetTest, "System.Core.Cooker.FifoSet",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FFifoSetTest::RunTest(const FString& Parameters)
{
	using namespace UE::Cook;

	int32 NumKeySet = 1000; 

	// KeyBuffer num must be 1 more than NumKeySet. We skip the middle number and use the N/2 values on to each side of it.
	int32 NumKeyBuffer = NumKeySet + 1;
	TArray<int32> KeyBuffer;
	KeyBuffer.SetNum(NumKeyBuffer);
	for (int32 Index = 0; Index < NumKeyBuffer; ++Index)
	{
		KeyBuffer[Index] = Index;
	}

	// Key pointers are in this sequence:
	// KeyBuffer.GetData() + [501, 499, 502, 498, ... 1000, 0]
	// This is an attempt to avoid the possibility that the TMap inside of TFifoSet just happens to sort
	// the keys in insertion order because insertion order is numerically increasing.
	TArray<int32*> Keys;
	Keys.SetNum(NumKeySet);
	for (int32 Index = 0; Index < NumKeySet; ++Index)
	{
		int32 BufferIndex = (NumKeySet / 2) + ((Index % 2) ? -1 : 1) * ((Index + 2) / 2);
		check(0 <= BufferIndex && BufferIndex < NumKeyBuffer);
		Keys[Index] = &KeyBuffer[BufferIndex];
	}

	TFifoSet<int32*> Set;

	// A string of Adds followed by string of pops, make sure we get them out in the expected order
	int32 NumKeys = 100;
	for (int32 Index = 0; Index < NumKeys; ++Index)
	{
		Set.Add(Keys[Index]);
	}
	int32 Index = 0;
	for (int32* Iter : Set)
	{
		TestEqual(TEXT("Adds followed by Pops iteration is in order"), Iter, Keys[Index]);
		++Index;
	}
	for (Index = 0; Index < NumKeys; ++Index)
	{
		int32* Key = Set.Pop();
		TestEqual(TEXT("Adds followed by Pops keep order"), Key, Keys[Index]);
	}
	TestTrue(TEXT("Adds followed by Pops return the set to empty"), Set.IsEmpty());

	// Interleave pops with adds, make sure we get them out in the expected order
	int32 Batch;
	for (Batch = 0; Batch < NumKeys; ++Batch)
	{
		Set.Add(Keys[2 * Batch]);
		Set.Add(Keys[2 * Batch + 1]);
		int32* Popped = Set.Pop();
		TestEqual(TEXT("Interleaved Adds and Pops keep order"), Popped, Keys[Batch]);
	}
	Index = NumKeys;
	for (int32* Iter : Set)
	{
		TestEqual(TEXT("Interleaved Adds and Pops iteration is in order"), Iter, Keys[Index]);
		++Index;
	}
	for (Index = NumKeys; Index < 2*NumKeys; ++Index)
	{
		int32* Key = Set.Pop();
		TestEqual(TEXT("Interleaved Adds and Pops keep order"), Key, Keys[Index]);
	}
	TestTrue(TEXT("Interleaved Adds and Pops return the set to empty"), Set.IsEmpty());

	// Interleave removes with adds, make sure we get them out in the expected order
	for (Batch = 0; Batch < NumKeys; ++Batch)
	{
		Set.Add(Keys[2 * Batch]);
		Set.Add(Keys[2 * Batch + 1]);
		TestEqual(TEXT("Interleaved Adds and Removes remove successfully"), Set.Remove(Keys[Batch]), 1);
	}
	Index = NumKeys;
	for (int32* Iter : Set)
	{
		TestEqual(TEXT("Interleaved Adds and Removes iteration is in order"), Iter, Keys[Index]);
		++Index;
	}
	Index = NumKeys;
	for (Batch = 0; Batch < NumKeys; ++Batch)
	{
		int32* Key = Set.Pop();
		TestEqual(TEXT("Interleaved Adds and Removes keep order"), Key, Keys[Index]);
		++Index;
	}
	TestTrue(TEXT("Interleaved Adds and Removes return the set to empty"), Set.IsEmpty());

	// Adding an element that already exists does not modify its position
	int32 NumKeysSmall = 5;
	for (Index = 0; Index < NumKeysSmall; ++Index)
	{
		Set.Add(Keys[Index]);
	}
	Set.Add(Keys[0]);
	TestEqual(TEXT("Add of an existing item does not change num"), Set.Num(), NumKeysSmall);
	Index = 0;
	for (int32* Iter : Set)
	{
		TestEqual(TEXT("Add of an existing item does not change order"), Iter, Keys[Index]);
		++Index;
	}
	// Also use this test to test emptying works.
	Set.Empty();
	TestEqual(TEXT("Empty clears Num -> 0"), Set.Num(), 0);
	TestEqual(TEXT("Empty clears IsEmpty -> true"), Set.IsEmpty(), true);
	Index = 0;
	for (int32* Iter : Set)
	{
		++Index;
	}
	TestEqual(TEXT("Empty clears iteration -> empty"), Index == 0, true);

	return true;
}

#endif // WITH_AUTOMATION_TESTS
