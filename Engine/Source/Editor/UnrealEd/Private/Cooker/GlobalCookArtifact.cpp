// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/GlobalCookArtifact.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Engine/DataAsset.h"
#include "Containers/Set.h"
#include "Cooker/CookArtifactReader.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookTypes.h"
#include "Serialization/CompactBinaryWriter.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/xxhash.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/ConfigAccessTracking.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Templates/UniquePtr.h"
#include "UObject/ICookInfo.h"
#include "UObject/NameTypes.h"

namespace UE::Cook
{

FGlobalCookArtifact::FGlobalCookArtifact(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{
}

FString FGlobalCookArtifact::GetArtifactName() const
{
	return FString(TEXT("global"));
}

static const TCHAR* TEXT_CookSettings(TEXT("CookSettings"));
static const TCHAR* TEXT_CookInProgress(TEXT("CookInProgress"));
static FName ExecutableHashName(TEXT("ExecutableHash"));
static FName ExecutableHashInvalidModuleName(TEXT("ExecutableHashInvalidModule"));

// TODO: Move into GlobalCookArtifact.cpp after review
FConfigFile FGlobalCookArtifact::CalculateCurrentSettings(ICookInfo& CookInfo, const ITargetPlatform* TargetPlatform)
{
	TMap<FName, FString> CookSettingStrings;
	const FName NAME_CookMode(TEXT("CookMode"));

	TArray<FModuleStatus> Modules;
	FModuleManager::Get().QueryModules(Modules);
	Modules.Sort([](const FModuleStatus& A, const FModuleStatus& B)
		{
			return A.FilePath.Compare(B.FilePath, ESearchCase::IgnoreCase) < 0;
		});
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	CookSettingStrings.Add(FName(TEXT("Version")), TEXT("21F52B9EDD4D456AB1AF381CA172BD28"));
	CookSettingStrings.Add(FName(TEXT("LegacyBuildDependencies")), ::LexToString(COTFS.bLegacyBuildDependencies));
	CookSettingStrings.Add(FName(TEXT("DependencyTrackingEnabled")), ::LexToString(COTFS.bDependencyTrackingEnabled));

	if (!COTFS.bLegacyBuildDependencies)
	{
		// Store the CookIncrementalVersion in the global settings, so that it will cause deletion of all artifacts
		// when it changes, in addition to invalidating all cooked packages.
		CookSettingStrings.Add(FName(TEXT("CookIncrementalVersion")), CookIncrementalVersion.ToString());
		CookSettingStrings.Add(FName(TEXT("ProjectCookIncrementalVersion")), ProjectCookIncrementalVersion.ToString());
	}
	else
	{
		// Calculate the executable hash by combining the module file hash of every loaded module
		// TODO: Write the module file hash from UnrealBuildTool into the .modules file and read it
		// here from the .modules file instead of calculating it on every cook.
		if (COTFS.bLegacyIterativeCalculateExe)
		{
			FString InvalidModule;
			bool bValid = true;
			FXxHash64Builder Hasher;
			TArray<uint8> Buffer;
			for (FModuleStatus& ModuleStatus : Modules)
			{
				TUniquePtr<IFileHandle> FileHandle(PlatformFile.OpenRead(*ModuleStatus.FilePath));
				if (!FileHandle)
				{
					InvalidModule = ModuleStatus.FilePath;
					break;
				}
				int64 FileSize = FileHandle->Size();
				Buffer.SetNumUninitialized(FileSize, EAllowShrinking::No);
				if (!FileHandle->Read(Buffer.GetData(), FileSize))
				{
					InvalidModule = ModuleStatus.FilePath;
					break;
				}
				Hasher.Update(Buffer.GetData(), FileSize);
			}
			if (InvalidModule.IsEmpty())
			{
				CookSettingStrings.Add(ExecutableHashName, FString(*WriteToString<64>(Hasher.Finalize())));
			}
			else
			{
				CookSettingStrings.Add(ExecutableHashInvalidModuleName, InvalidModule);
			}
		}
	}

	if (CookInfo.GetCookType() == ECookType::ByTheBook)
	{
		CookSettingStrings.Add(NAME_CookMode, TEXT("CookByTheBook"));
		CookSettingStrings.Add(FName(TEXT("DLCName")), CookInfo.GetDLCName());
		if (!CookInfo.GetDLCName().IsEmpty() || !CookInfo.GetCreateReleaseVersion().IsEmpty())
		{
			CookSettingStrings.Add(FName(TEXT("BasedOnReleaseVersion")), CookInfo.GetBasedOnReleaseVersion());
		}
	}
	else
	{
		check(CookInfo.GetCookType() == ECookType::OnTheFly);
		CookSettingStrings.Add(NAME_CookMode, TEXT("CookOnTheFly"));
	}

	CookSettingStrings.Add(FName(TEXT_CookInProgress), TEXT("true"));

	FConfigSection ConfigSection;
	FConfigFile ConfigFile;
	for (TPair<FName, FString>& CurrentSetting : CookSettingStrings)
	{
		ConfigSection.Add(CurrentSetting.Key, FConfigValue(MoveTemp(CurrentSetting.Value)));
	}
	ConfigFile.Add(TEXT_CookSettings, MoveTemp(ConfigSection));
	return ConfigFile;
}

void FGlobalCookArtifact::CompareSettings(UE::Cook::Artifact::FCompareSettingsContext& Context)
{
	// For the global settings, we only use RequestFullRecook; we have no files to invalidate.
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();

	const FConfigSection* PreviousSettings = Context.GetPrevious().FindSection(TEXT_CookSettings);
	if (PreviousSettings == nullptr)
	{
		UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because CookSettings file %ls is invalid. Clearing previously cooked packages.",
			*TargetPlatform->PlatformName(), *Context.GetPreviousFileName());
		Context.RequestFullRecook(true);
		return;
	}

	TSet<FName> IgnoreKeys;
	IgnoreKeys.Add(FName(TEXT_CookInProgress));
	IgnoreKeys.Add(ExecutableHashName);
	IgnoreKeys.Add(ExecutableHashInvalidModuleName);

	const FConfigSection* CurrentSettings = Context.GetCurrent().FindSection(TEXT_CookSettings);
	check(CurrentSettings);
	for (const TPair<FName, FConfigValue>& CurrentSetting : *CurrentSettings)
	{
		if (IgnoreKeys.Contains(CurrentSetting.Key))
		{
			continue;
		}
		const FConfigValue* PreviousSetting = PreviousSettings->Find(CurrentSetting.Key);
		if (!PreviousSetting || PreviousSetting->GetValue() != CurrentSetting.Value.GetValue())
		{
			UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because %ls has changed. Old: %ls, New: %ls. Clearing previously cooked packages.",
				*TargetPlatform->PlatformName(), *CurrentSetting.Key.ToString(),
				PreviousSetting ? *PreviousSetting->GetValue() : TEXT(""),
				*CurrentSetting.Value.GetValue());
			Context.RequestFullRecook(true);
			return;
		}
	}

	if (GIsBuildMachine)
	{
		bool bCookInProgress;
		if (Context.GetPrevious().GetBool(TEXT_CookSettings, TEXT_CookInProgress, bCookInProgress) && bCookInProgress)
		{
			UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because the previous cook crashed (or otherwise did not report completion)." " CookSettings file %ls still has [%ls]:%ls=true. Clearing previously cooked packages.",
				*TargetPlatform->PlatformName(), *Context.GetPreviousFileName(), TEXT_CookSettings, TEXT_CookInProgress);
			Context.RequestFullRecook(true);
			return;
		}
	}

	if (!COTFS.bLegacyIterativeIgnoreIni && COTFS.bLegacyBuildDependencies && COTFS.IniSettingsOutOfDate(TargetPlatform))
	{
		UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because ini settings have changed. Clearing previously cooked packages.",
			*TargetPlatform->PlatformName());
		Context.RequestFullRecook(true);
		return;
	}

	if (!COTFS.bLegacyIterativeIgnoreExe && COTFS.bLegacyBuildDependencies)
	{
		const FConfigValue* CurrentHash = CurrentSettings->Find(ExecutableHashName);
		if (!CurrentHash)
		{
			UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because current executable hash is invalid. Invalid module=%ls. Clearing previously cooked packages.",
				*TargetPlatform->PlatformName(), *CurrentSettings->FindRef(ExecutableHashInvalidModuleName).GetValue());
			Context.RequestFullRecook(true);
			return;
		}
		const FConfigValue* PreviousHash = PreviousSettings->Find(ExecutableHashName);
		if (!PreviousHash)
		{
			const FConfigValue* InvalidModuleName = PreviousSettings->Find(ExecutableHashInvalidModuleName);
			UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because old executable hash is invalid. Invalid module=%ls. Clearing previously cooked packages.",
				*TargetPlatform->PlatformName(), InvalidModuleName ? *InvalidModuleName->GetValue() : TEXT(""));
			Context.RequestFullRecook(true);
			return;
		}
		if (!CurrentHash->GetValue().Equals(*PreviousHash->GetValue(), ESearchCase::CaseSensitive))
		{
			UE_LOGF(LogCook, Display, "Cook invalidated for platform %ls because executable hash has changed. Old: %ls, New: %ls. Clearing previously cooked packages.",
				*TargetPlatform->PlatformName(), *PreviousHash->GetValue(), *CurrentHash->GetValue());
			Context.RequestFullRecook(true);
			return;
		}
	}

	if (!TryLoadPreviousAssetRegistry(Context))
	{
		// Log and RequestFullRecook was issued by TryLoadPreviousAssetRegistry.
		return;
	}
}

bool FGlobalCookArtifact::TryLoadPreviousAssetRegistry(UE::Cook::Artifact::FCompareSettingsContext& Context)
{
	// Load the previous asset registry; if we cannot load it then we are missing required data for all packages
	// and will need to do a full recook. After loading the PreviousAssetRegistry we hand it over during
	// PopulateCookedPackages to the AssetRegistryGenerator for update during the cook and writing at end of cook.
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	FPlatformData* PlatformData = COTFS.PlatformManager->GetPlatformData(TargetPlatform);
	check(PlatformData);
	FString PreviousAssetRegistryFile;
	if (!PlatformData->bLegacyIterativeSharedBuild)
	{
		PreviousAssetRegistryFile = FPaths::Combine(Context.GetCookInfo().GetCookMetadataOutputFolder(TargetPlatform),
			GetDevelopmentAssetRegistryFilename());
	}
	else
	{
		PreviousAssetRegistryFile = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("SharedIterativeBuild"),
			*TargetPlatform->PlatformName(), TEXT("Metadata"), GetDevelopmentAssetRegistryFilename());
	}

	TUniquePtr<FArchive> Reader(Context.GetArtifactReader().CreateFileReader(*PreviousAssetRegistryFile));

	if (!Reader)
	{
		UE_LOGFMT(LogCook, Display, "Cook invalidated for platform {Platform} because the previous AssetRegistry is not available. "
			"Unable to load '{AssetRegistryFile}'. Clearing previously cooked packages.",
			*TargetPlatform->PlatformName(), *PreviousAssetRegistryFile);
		Context.RequestFullRecook(true);
		return false;
	}

	TUniquePtr<FAssetRegistryState> PreviousState = MakeUnique<FAssetRegistryState>();
	if (!PreviousState->Load(*Reader))
	{
		UE_LOGFMT(LogCook, Display, "Cook invalidated for platform {Platform} because the previous AssetRegistry is not available. "
			"Error during reading '{AssetRegistryFile}'. Clearing previously cooked packages.",
			*TargetPlatform->PlatformName(), *PreviousAssetRegistryFile);
		Context.RequestFullRecook(true);
		return false;
	}

	PreviousAssetRegistries.Add(TargetPlatform, MoveTemp(PreviousState));
	return true;
}

TUniquePtr<FAssetRegistryState> FGlobalCookArtifact::DetachAssetRegistry(const ITargetPlatform* TargetPlatform)
{
	TUniquePtr<FAssetRegistryState> Result;
	PreviousAssetRegistries.RemoveAndCopyValue(TargetPlatform, Result);
	return Result;
}

void FGlobalCookArtifact::UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
{
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	FPlatformData* PlatformData = COTFS.PlatformManager->GetPlatformData(TargetPlatform);
	check(PlatformData && PlatformData->RegistryGenerator);
	FAssetRegistryGenerator& Generator = *PlatformData->RegistryGenerator;

	Generator.RemoveStalePackages(Context.GetOplogPackages());
	Generator.AddGlobalsToCookedAssetRegistry(COTFS);
}

void FGlobalCookArtifact::StoreDataInOplog(UE::Cook::Artifact::FStoreDataInOplogContext& Context)
{
	if (!COTFS.CookByTheBookOptions)
	{
		return;
	}
	const TSet<FName>& StartupPackages = COTFS.CookByTheBookOptions->StartupPackages;
	if (StartupPackages.IsEmpty())
	{
		return;
	}

	FCbWriter Writer;
	Writer.BeginObject();
	Writer.BeginArray("StartupPackages");
	for (const FName& PackageName : StartupPackages)
	{
		Writer << PackageName;
	}
	Writer.EndArray();
	Writer.EndObject();
	Context.AppendOp(TEXT("Cook.StartupPackages"), Writer.Save().AsObject());
}

void FGlobalCookArtifact::AppendPackageMetadata(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context)
{
	if (!Context.Package)
	{
		return;
	}

	static const FName MetadataKey = TEXT("meta.cook.package");

	if (UObject* Asset = Context.Package->FindAssetInPackage())
	{
		FString ClassPath;
		if (UClass* AssetAsClass = Cast<UClass>(Asset))
		{
			// Blueprint/cooked case: the asset is the generated class itself
			ClassPath = AssetAsClass->GetClassPathName().ToString();   // -> /Game/Foo.Foo_C
		}
		else if (Asset->GetClass())
		{
			ClassPath = Asset->GetClass()->GetClassPathName().ToString();
		}

		FCbWriter Writer;
		Writer.BeginObject();

		if (!ClassPath.IsEmpty())
		{
			Writer << "class" << ClassPath;
		}
		if (Asset->IsA(UPrimaryDataAsset::StaticClass()))
		{
			Writer << "isprimaryasset" << true;
		}

		Writer.EndObject();
		Context.SetAttachment(MetadataKey, Writer.Save().AsObject(), UE::Cook::Artifact::Inline);
	}
}

} // namespace UE::Cook

void UCookOnTheFlyServer::ClearCookInProgressFlagFromGlobalCookSettings(const ITargetPlatform* TargetPlatform) const
{
	using namespace UE::Cook;

	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;
	FConfigFile ConfigFile;
	check(GlobalArtifact);
	FString ArtifactName = GlobalArtifact->GetArtifactName();
	FString Filename = GetCookSettingsFileName(TargetPlatform, ArtifactName);
	ConfigFile.Read(Filename);
	ConfigFile.RemoveKeyFromSection(TEXT_CookSettings, TEXT_CookInProgress);
	SaveCookSettings(ConfigFile, TargetPlatform, ArtifactName);
}
