// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Map.h"
#include "Cooker/CookArtifact.h"
#include "Templates/UniquePtr.h"

class ITargetPlatform;
class UCookOnTheFlyServer;

namespace UE::Cook
{

/**
 * CookArtifact that triggers a recook of all packages if any of the global settings that affect all
 * packages change. This artifact is also responsible for incremental cook's load of the AssetRegistry artifact.
 */
class FGlobalCookArtifact : public ICookArtifact
{
public:
	FGlobalCookArtifact(UCookOnTheFlyServer& InCOTFS);

	FString GetArtifactName() const override;
	bool RegisterOnCookWorker() const override { return true; };
	FConfigFile CalculateCurrentSettings(ICookInfo& CookInfo, const ITargetPlatform* TargetPlatform) override;
	void CompareSettings(UE::Cook::Artifact::FCompareSettingsContext& Context) override;
	void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context) override;
	void StoreDataInOplog(UE::Cook::Artifact::FStoreDataInOplogContext& Context) override;
	void AppendPackageMetadata(UE::Cook::Artifact::FAppendPackageMetaDataContext& Context) override;

	TUniquePtr<FAssetRegistryState> DetachAssetRegistry(const ITargetPlatform* TargetPlatform);

private:
	bool TryLoadPreviousAssetRegistry(UE::Cook::Artifact::FCompareSettingsContext& Context);

	UCookOnTheFlyServer& COTFS;
	/**
	 * This artifact holds the PreviousAssetRegistries loaded from disk for a short time during StartCookByTheBook
	 * or StartCookOnTheFly. It relinquishes them to the AssetRegistryGenerator during PopulateCookedPackages.
	 */
	TMap<const ITargetPlatform*, TUniquePtr<FAssetRegistryState>> PreviousAssetRegistries;
};


} // namespace UE::Cook