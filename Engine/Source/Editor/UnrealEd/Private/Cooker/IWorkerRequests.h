// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CookRequests.h"
#include "CookTypes.h"

class FConfigFile;
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FGenerationHelper; }

namespace UE::Cook
{

/**
 * Interface for Requests sent to this processes' Cooker, and for information it needs to send back to its Director.
 * In SingleProcess cooks the functions are passed through to the local CookOnTheFlyServer.
 * In MultiProcess cooks the functions are implemented as interprocess messages to/from the CookDirector.
 **/
class IWorkerRequests
{
public:
	virtual ~IWorkerRequests() = default;

	// Reading Callbacks And Requests
	/**
	 * Lockless value for whether the Worker has ExternalRequests it has not yet reported to the Cooker.
	 * May be out of date after calling; do not assume a true return value means Requests are actually present or a
	 * false value means no Requests are present.
	 * Intended usage is for the Scheduler to be the only consumer of requests, and to use this value for periodic
	 * checking of whether there is any work that justifies the expense of taking the lock.
	 * In a single-consumer case, HasRequests will eventually correctly return true as long as the consumer is not
	 * consuming.
	 */
	virtual bool HasExternalRequests() const = 0;
	/**
	 * Lockless value for the number of ExternalRequests the Worker has which it has not yet reported to the Cooker.
	 * May be out of date after calling; do not assume the number of actual requests is any one of equal, greater than,
	 * or less than the returned value.
	 * Intended usage is for the Scheduler to be the only consumer of requests, and to use this value for rough
	 * reporting of periodic progress.
	 */
	virtual int32 GetNumExternalRequests() const = 0;
	/**
	 * If this Worker has any callbacks, dequeue them all into OutCallbacks and return EExternalRequestType::Callback;
	 * Callbacks take priority over cook requests.
	 * Otherwise, if there are any cook requests, dequeue them all into OutBuildRequests and return
	 * EExternalRequestType::Cook.
	 * Otherwise, return EExternalRequestType::None.
	 */
	virtual EExternalRequestType DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks, TArray<FFilePlatformRequest>& OutBuildRequests) = 0;
	/* Move any existing callbacks onto OutCallbacks, and return whether any were added. */
	virtual bool DequeueSchedulerCallbacks(TArray<FSchedulerCallback>& OutCallbacks) = 0;
	/**
	 * Move all callbacks into OutCallbacks, and all cook requests into OutCookRequests. This is used when canceling
	 * a cook session.
	 */
	virtual void DequeueAllExternal(TArray<FSchedulerCallback>& OutCallbacks,
		TArray<FFilePlatformRequest>& OutCookRequests) = 0;

	// Writing Packages during Cook
	virtual void QueueDiscoveredPackage(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
		FInstigator&& Instigator, FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency Urgency,
		FGenerationHelper* ParentGenerationHelper) = 0;
	virtual void EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS, FGenerationHelper& GenerationHelper) = 0;

	// Writing Packages from CookByTheBook
	virtual void AddStartCookByTheBookRequest(FFilePlatformRequest&& Request) = 0;

	// Writing Packages from CookOnTheFly
	virtual void InitializeCookOnTheFly() = 0;
	virtual void AddCookOnTheFlyRequest(FFilePlatformRequest&& Request) = 0;
	virtual void AddCookOnTheFlyCallback(FSchedulerCallback&& Callback) = 0;
	virtual void WaitForCookOnTheFlyEvents(int TimeoutMs) = 0;

	// Writing Packages from Editor interface
	virtual void AddEditorActionCallback(FSchedulerCallback&& Callback) = 0;
	virtual void AddPublicInterfaceRequest(FFilePlatformRequest&& Request, bool bForceFrontOfQueue) = 0;

	// Hooks into cooker events
	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	virtual void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap) = 0;
	/** Remove references to the given platform from all cook requests. */
	virtual void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform) = 0;

	// Functions that are written locally for Directors but are sent to Director for Workers
	virtual void ReportDemotion(UE::Cook::FPackageData& PackageData, ESuppressCookReason Reason) = 0;
	virtual void ReportPromoteToSaveComplete(UE::Cook::FPackageData& PackageData) = 0;

	// Functions that read from local for Directors but read from Director for Workers
	virtual void GetInitializeConfigSettings(UCookOnTheFlyServer& COTFS, const FString& OutputDirectoryOverride,
		UE::Cook::FInitializeConfigSettings& Settings) = 0;
	virtual void GetBeginCookConfigSettings(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext,
		UE::Cook::FBeginCookConfigSettings& Settings) = 0;
	virtual void GetBeginCookIncrementalFlags(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext) = 0;
	virtual ECookMode::Type GetDirectorCookMode(UCookOnTheFlyServer& COTFS) = 0;

	/* Prints a list of all files in the RequestMap to the log */
	virtual void LogAllRequestedFiles() = 0;
};

}
