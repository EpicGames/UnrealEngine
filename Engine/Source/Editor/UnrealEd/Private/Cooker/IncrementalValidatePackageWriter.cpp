// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/IncrementalValidatePackageWriter.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/MPCollector.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "Templates/UniquePtr.h"

DEFINE_LOG_CATEGORY_STATIC(LogIncrementalValidate, Log, All);

constexpr FStringView IncrementalValidateFilename(TEXTVIEW("IncrementalValidate.bin"));

class FIncrementalValidateMPCollector : public UE::Cook::IMPCollector
{
public:
	FIncrementalValidateMPCollector(FIncrementalValidatePackageWriter* InOwner) : Owner(InOwner) {}

	virtual FGuid GetMessageType() const { return MessageType; }
	virtual const TCHAR* GetDebugName() const { return TEXT("IncrementalValidateMPCollector"); }

	virtual void ServerTick(UE::Cook::FMPCollectorServerTickContext& Context) override;
	virtual void ClientTickPackage(UE::Cook::FMPCollectorClientTickPackageContext& Context) override;
	virtual void ServerTickPackage(UE::Cook::FMPCollectorServerTickPackageContext& Context);

	virtual void ClientReceiveMessage(UE::Cook::FMPCollectorClientMessageContext& Context, FCbObjectView Message) override;
	virtual void ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

public:
	static FGuid MessageType;

private:
	enum class EMessageSubtype : uint8
	{
		ServerToClient_WorkerStartup,
		ClientToServer_ReplIsAnotherSaveNeeded,
		ServerToClient_ReplUpdatePackageModificationStatus,
		Invalid
	};

	bool TryWritePackageStatus(FCbWriter& Writer, FName PackageName);
	void ReadAndSyncPackageStatus(FCbObjectView Message, FName PackageName);
	FIncrementalValidatePackageWriter* Owner;
};

FCbWriter& operator<<(FCbWriter& Writer, const FIncrementalValidatePackageWriter::FMessage& Message)
{
	Writer.BeginObject();
	Writer << "Text" << Message.Text;

	static_assert(sizeof(Message.Verbosity) == sizeof(uint8));
	Writer << "Verbosity" << (uint8)Message.Verbosity;

	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FIncrementalValidatePackageWriter::FMessage& Message)
{
	bool bOk = !Field.HasError();
	if (bOk && LoadFromCompactBinary(Field["Text"], Message.Text))
	{
		uint8 Verbosity = ELogVerbosity::NumVerbosity;
		bOk = LoadFromCompactBinary(Field["Verbosity"], Verbosity) && Verbosity < ELogVerbosity::NumVerbosity;
		if (bOk)
		{
			Message.Verbosity = (ELogVerbosity::Type)Verbosity;
		}
	}
	if (!bOk)
	{
		Message = FIncrementalValidatePackageWriter::FMessage();
	}
	return bOk;
}

FCbWriter& operator<<(FCbWriter& Writer, FIncrementalValidatePackageWriter::EPackageStatus Status)
{
	Writer << (uint8)Status;
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FIncrementalValidatePackageWriter::EPackageStatus& Status)
{
	uint8 StatusInteger = (uint8)FIncrementalValidatePackageWriter::EPackageStatus::Count;
	if (!LoadFromCompactBinary(Field, StatusInteger))
	{
		UE_LOGF(LogIncrementalValidate, Error, "Failed to deserialize package status.");
	}
	else if (StatusInteger >= (uint8)FIncrementalValidatePackageWriter::EPackageStatus::Count)
	{
		UE_LOGF(LogIncrementalValidate, Error, "Unexpected package status deserialized: %d", StatusInteger);
	}
	else
	{
		Status = (FIncrementalValidatePackageWriter::EPackageStatus)StatusInteger;
		return true;
	}
	return false;
}

FArchive& operator<<(FArchive& Ar, FIncrementalValidatePackageWriter::FPackageStatusInfo& Info)
{
	Ar << Info.Status;
	bool bHasAssetClass = Info.AssetClass.IsValid();
	Ar << bHasAssetClass;
	if (bHasAssetClass)
	{
		Ar << Info.AssetClass;
	}
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FIncrementalValidatePackageWriter::FPackageStatusInfo& Info)
{
	Writer.BeginArray();
	Writer << Info.Status;
	if (Info.AssetClass.IsValid())
	{
		Writer << Info.AssetClass;
	}
	Writer.EndArray();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FIncrementalValidatePackageWriter::FPackageStatusInfo& Info)
{
	FCbArrayView ArrayField = Field.AsArrayView();
	if (ArrayField.Num() < 1)
	{
		return false;
	}
	FCbFieldViewIterator Iter = ArrayField.CreateViewIterator();
	bool bOk = true;
	bOk = LoadFromCompactBinary(*(Iter++), Info.Status) & bOk;
	if (ArrayField.Num() >= 2)
	{
		bOk = LoadFromCompactBinary(*(Iter++), Info.AssetClass) & bOk;
	}
	return bOk;
}

void FIncrementalValidateMPCollector::ServerTick(UE::Cook::FMPCollectorServerTickContext& Context)
{
	static_assert(sizeof(EMessageSubtype) == sizeof(uint8));

	if (Context.GetEventType() == UE::Cook::FMPCollectorServerTickContext::EServerEventType::WorkerStartup)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer << "MessageSubtype" << (uint8)EMessageSubtype::ServerToClient_WorkerStartup;
		Writer << "PackageStatusMap" << Owner->PackageStatusMap;
		Writer << "PackageMessageMap" << Owner->PackageMessageMap;
		Writer.EndObject();
		Context.AddMessage(Writer.Save().AsObject());
	}
}

void FIncrementalValidateMPCollector::ClientTickPackage(UE::Cook::FMPCollectorClientTickPackageContext& Context)
{
	static_assert(sizeof(EMessageSubtype) == sizeof(uint8));

	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "MessageSubtype" << (uint8)EMessageSubtype::ClientToServer_ReplIsAnotherSaveNeeded;

	const FName PackageName = Context.GetPackageName();
	if (PackageName.IsNone())
	{
		UE_LOGF(LogCook, Error, "Context does not contain a valid package name.")
		// It's safe to continue because TryWritePackageStatus will return false if the name is none. 
		// The error is logged here to make the call site clear
	}

	if (TryWritePackageStatus(Writer, PackageName))
	{
		Writer.EndObject();
		Context.AddMessage(Writer.Save().AsObject());
	}
}

void FIncrementalValidateMPCollector::ServerTickPackage(UE::Cook::FMPCollectorServerTickPackageContext& Context)
{
	static_assert(sizeof(EMessageSubtype) == sizeof(uint8));

	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "MessageSubtype" << (uint8)EMessageSubtype::ServerToClient_ReplUpdatePackageModificationStatus;

	const FName PackageName = Context.GetPackageName();
	if (PackageName.IsNone())
	{
		UE_LOGF(LogCook, Error, "Context does not contain a valid package name.")
		// It's safe to continue because TryWritePackageStatus will return false if the name is none. 
		// The error is logged here to make the call site clear
	}

	if (TryWritePackageStatus(Writer, PackageName))
	{
		Writer.EndObject();
		Context.AddMessage(Writer.Save().AsObject());
	}
}

void FIncrementalValidateMPCollector::ClientReceiveMessage(UE::Cook::FMPCollectorClientMessageContext& Context, FCbObjectView Message)
{
	uint8 MessageSubtypeAsInteger = (uint8)EMessageSubtype::Invalid;
	if (LoadFromCompactBinary(Message["MessageSubtype"], MessageSubtypeAsInteger))
	{
		switch ((EMessageSubtype)MessageSubtypeAsInteger)
		{
			case EMessageSubtype::ServerToClient_WorkerStartup:
			{
				bool bOk = LoadFromCompactBinary(Message["PackageStatusMap"], Owner->PackageStatusMap);
				bOk = bOk && LoadFromCompactBinary(Message["PackageMessageMap"], Owner->PackageMessageMap);
				check(bOk); // If we fail this, we will fail to get anything right during the rest of the validation. Better to terminate quickly.
				break;
			}
			case EMessageSubtype::ServerToClient_ReplUpdatePackageModificationStatus:
			{
				FName PackageName = Context.GetPackageName();
				if (!PackageName.IsNone())
				{
					ReadAndSyncPackageStatus(Message, PackageName);
				}
				else
				{
					UE_LOGF(LogCook, Error, "Cannot process ServerToClient_ReplUpdatePackageModificationStatus without a valid package name in the current context.");
				}
				break;
			}
			default:
			{
				UE_LOGF(LogCook, Error, "Unexpected message type: %d", MessageSubtypeAsInteger);
				break;
			}
		};
	}
}

void FIncrementalValidateMPCollector::ServerReceiveMessage(UE::Cook::FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	uint8 MessageSubtypeAsInteger = (uint8)EMessageSubtype::Invalid;
	if (LoadFromCompactBinary(Message["MessageSubtype"], MessageSubtypeAsInteger))
	{
		FName PackageName = Context.GetPackageName();
		if (PackageName.IsNone())
		{
			UE_LOGF(LogCook, Error, "Cannot process messages on server without a valid package name in the current context.");
		}
		else if (MessageSubtypeAsInteger == (uint8)EMessageSubtype::ClientToServer_ReplIsAnotherSaveNeeded)
		{
			ReadAndSyncPackageStatus(Message, PackageName);
			Owner->MarkPackageCompletedOnDirector(PackageName, Context.GetWorkerId());
		}
		else
		{
			UE_LOGF(LogCook, Error, "Unexpected message received. MessageSubtype == %d", MessageSubtypeAsInteger);
		}
	}
	else
	{
		UE_LOGF(LogCook, Error, "Invalid message received. No MessageSubtype field available.");
	}
}

FGuid FIncrementalValidateMPCollector::MessageType(TEXT("5E56C5D96F3B455E9452C15ADA601A71"));

bool FIncrementalValidateMPCollector::TryWritePackageStatus(FCbWriter& Writer, FName PackageName)
{
	const FIncrementalValidatePackageWriter::FPackageStatusInfo* PackageStatus
		= Owner->PackageStatusMap.Find(PackageName);

	if (PackageStatus && PackageStatus->Status != FIncrementalValidatePackageWriter::EPackageStatus::NotYetProcessed)
	{
		Writer << "Status" << *PackageStatus;

		const TArray<FIncrementalValidatePackageWriter::FMessage>* Messages = Owner->PackageMessageMap.Find(PackageName);
		if (Messages != nullptr)
		{
			Writer << "MessageArray" << *Messages;
		}
		return true;
	}
	return false;
}

void FIncrementalValidateMPCollector::ReadAndSyncPackageStatus(FCbObjectView Message, FName PackageName)
{
	FIncrementalValidatePackageWriter::FPackageStatusInfo Info;
	bool bOk = LoadFromCompactBinary(Message["Status"], Info);

	if (bOk)
	{
		Owner->PackageStatusMap.FindOrAdd(PackageName) = MoveTemp(Info);
		if (Message.FindView("MessageArray").HasValue())
		{
			TArray<FIncrementalValidatePackageWriter::FMessage>& MessageArray = Owner->PackageMessageMap.FindOrAdd(PackageName);
			bOk = LoadFromCompactBinary(Message["MessageArray"], MessageArray);
		}
	}
	
	if (!bOk)
	{
		UE_LOGF(LogCook, Error, "Invalid message received in ReadAndSyncPackageStatus. Failed to load Info from Message[\"Status\"] for package \"%ls\"",
			*PackageName.ToString());
	}
}

FIncrementalValidatePackageWriter::FIncrementalValidatePackageWriter(UCookOnTheFlyServer& InCOTFS,
	TUniquePtr<ICookedPackageWriter>&& InInner, EPhase InPhase, const FString& ResolvedMetadataPath,
	UE::Cook::FDeterminismManager* InDeterminismManager)
	: FDiffPackageWriter(InCOTFS, MoveTemp(InInner), InDeterminismManager, FDiffPackageWriter::EReportingMode::NoPackageSummary)
	, MetadataPath(ResolvedMetadataPath)
	, Phase(InPhase)
{
	COTFS.RegisterCollector(new FIncrementalValidateMPCollector(this));

	Indent = FCString::Spc(FOutputDeviceHelper::FormatLogLine(ELogVerbosity::Warning,
		LogIncrementalValidate.GetCategoryName(), TEXT(""), GPrintLogTimes).Len());

	TArray<FString> IncrementalValidatePackageIgnoreList;
	GConfig->GetArray(TEXT("IncrementalValidate"), TEXT("PackageIgnoreList"),
		IncrementalValidatePackageIgnoreList, GEditorIni);
	PackageIgnoreList.Reserve(IncrementalValidatePackageIgnoreList.Num());
	for (const FString& PackageName : IncrementalValidatePackageIgnoreList)
	{
		PackageIgnoreList.Add(FName(FStringView(PackageName)));
	}

	LoggingSoftMaximum = -1;
	GConfig->GetValue(TEXT("IncrementalValidate"), TEXT("LoggingSoftMaximum"), LoggingSoftMaximum, GEditorIni);

	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		bReadOnly = !FParse::Param(FCommandLine::Get(), TEXT("IncrementalValidateAllowWrite"));
		break;
	case EPhase::Phase1:
		bReadOnly = false;
		break;
	case EPhase::Phase2:
		bReadOnly = false;
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::BeginPackage(const FBeginPackageInfo& Info)
{
	bPackageFirstPass = true;
	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		if (GetPackageStatus(Info.PackageName) == EPackageStatus::DeclaredUnmodified_NotYetProcessed)
		{
			// Save to memory and look for diffs before saving it out to disk
			SaveAction = ESaveAction::CheckForDiffs;
			Super::BeginPackage(Info);
		}
		else
		{
			BeginPackageDiff(Info);

			// Not incrementally skippable, so we expect it to change. No need to look for diffs, just save to disk.
			if (bReadOnly)
			{
				SaveAction = ESaveAction::IgnoreResults;
			}
			else
			{
				SaveAction = ESaveAction::SaveToInner;
				Inner->BeginPackage(Info);
			}
		}
		break;
	case EPhase::Phase1:
		SaveAction = ESaveAction::CheckForDiffs;
		Super::BeginPackage(Info);
		break;
	case EPhase::Phase2:
		{
			EPackageStatus PackageStatus = GetPackageStatus(Info.PackageName);
			if (PackageStatus == EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified ||
				PackageStatus == EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList)
			{
				// Already saved in Phase 1; no need to diff it or save it now
				SaveAction = ESaveAction::IgnoreResults;
				BeginPackageDiff(Info);
			}
			else if (PackageStatus == EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive)
			{
				SaveAction = ESaveAction::CheckForDiffs;
				Super::BeginPackage(Info);
			}
			else
			{
				// This is an IncrementallyModified package. It was found during Phase1 to be modified and would in a
				// normal incremental cook be resaved rather than incrementally skipped. Resave it as normal.
				SaveAction = ESaveAction::SaveToInner;
				BeginPackageDiff(Info);
				Inner->BeginPackage(Info);
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::CommitPackage(FCommitPackageInfo&& Info)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::CommitPackage(MoveTemp(Info));
		break;
	case ESaveAction::SaveToInner:
		Inner->CommitPackage(MoveTemp(Info));
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
	const TArray<FFileRegion>& FileRegions)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::WritePackageData(Info, ExportsArchive, FileRegions);
		break;
	case ESaveAction::SaveToInner:
		Inner->WritePackageData(Info, ExportsArchive, FileRegions);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData,
	const TArray<FFileRegion>& FileRegions)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::WriteBulkData(Info, BulkData, FileRegions);
		break;
	case ESaveAction::SaveToInner:
		Inner->WriteBulkData(Info, BulkData, FileRegions);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::WriteAdditionalFile(Info, FileData);
		break;
	case ESaveAction::SaveToInner:
		Inner->WriteAdditionalFile(Info, FileData);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}
void FIncrementalValidatePackageWriter::WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info,
	 const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::WriteLinkerAdditionalData(Info, Data, FileRegions);
		break;
	case ESaveAction::SaveToInner:
		Inner->WriteLinkerAdditionalData(Info, Data, FileRegions);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::WritePackageTrailer(Info, Data);
		break;
	case ESaveAction::SaveToInner:
		Inner->WritePackageTrailer(Info, Data);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

int64 FIncrementalValidatePackageWriter::GetExportsFooterSize()
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		return Super::GetExportsFooterSize();
	case ESaveAction::SaveToInner:
		return Inner->GetExportsFooterSize();
	case ESaveAction::IgnoreResults:
		return 0;
	default:
		checkNoEntry();
		return 0;
	}
}

TUniquePtr<FLargeMemoryWriter> FIncrementalValidatePackageWriter::CreateLinkerArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex)
{
	if (Asset)
	{
		PackageStatusMap.FindOrAdd(PackageName).AssetClass = Asset->GetClass()->GetClassPathName();
	}

	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		return Super::CreateLinkerArchive(PackageName, Asset, MultiOutputIndex);
	case ESaveAction::SaveToInner:
		return Inner->CreateLinkerArchive(PackageName, Asset, MultiOutputIndex);
	case ESaveAction::IgnoreResults:
		return MakeUnique<FLargeMemoryWriter>();
	default:
		checkNoEntry();
		return TUniquePtr<FLargeMemoryWriter>();
	}
}

TUniquePtr<FLargeMemoryWriter> FIncrementalValidatePackageWriter::CreateLinkerExportsArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		return Super::CreateLinkerExportsArchive(PackageName, Asset, MultiOutputIndex);
	case ESaveAction::SaveToInner:
		return Inner->CreateLinkerExportsArchive(PackageName, Asset, MultiOutputIndex);
	case ESaveAction::IgnoreResults:
		return MakeUnique<FLargeMemoryWriter>();
	default:
		checkNoEntry();
		return TUniquePtr<FLargeMemoryWriter>();
	}
}

bool FIncrementalValidatePackageWriter::IsPreSaveCompleted() const
{
	return !bPackageFirstPass;
}

ICookedPackageWriter::FCookCapabilities FIncrementalValidatePackageWriter::GetCookCapabilities() const
{
	FCookCapabilities Result = Super::GetCookCapabilities();
	Result.bReadOnly = bReadOnly;
	Result.bOverridesPackageModificationStatus = true;
	return Result;
}

void FIncrementalValidatePackageWriter::Initialize(const FCookInfo& CookInfo)
{
	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		if (CookInfo.bFullBuild)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"The cook is running non-incrementally. All packages are reported \"modified\" and will be resaved as in a normal cook.");
			check(!bReadOnly); // We should have set this due to incrementalvalidateallowwrite, or aborted the cook earlier
		}
		break;
	case EPhase::Phase1:
		if (CookInfo.bFullBuild)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"The cook is running non-incrementally. All packages are reported \"modified\" and will be resaved during the final IncrementalValidate phase.");
		}
		break;
	case EPhase::Phase2:
		if (CookInfo.bFullBuild)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker, 
				LogIncrementalValidate, Display,
				"The cook is running non-incrementally. Packages that were incrementally skipped and found valid will be resaved anyway.");
		}
		break;
	default:
		checkNoEntry();
		break;
	}
	Super::Initialize(CookInfo);
}

void FIncrementalValidatePackageWriter::UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context)
{
	// We need to not skip previously cooked generator packages, if they were modified and we're read only we still
	// need to cook them so we can investigate their generated packages. Look up whether they were a generator in
	// the previous cook results.
	bool bKnownGenerator = false;
	if (Context.bIncrementallyUnmodified)
	{
		// GenerationHelpers were created for previously cooked generators by UCOTFS::PopulateCookedPackages.
		UE::Cook::FPackageData* PackageData = COTFS.PackageDatas->FindPackageDataByPackageName(Context.PackageName);
		if (PackageData)
		{
			TRefCountPtr<UE::Cook::FGenerationHelper> GenerationHelper = PackageData->GetGenerationHelper();
			if (GenerationHelper)
			{
				bKnownGenerator = true;
			}
		}
	}

	switch (Phase)
	{
	case EPhase::AllInOnePhase:		
		// Save the input value for bIncrementallyUnmodified, and report skippable for the modified if possible
		if (Context.bIncrementallyUnmodified)
		{
			SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredUnmodified_NotYetProcessed);
			Context.bInOutShouldIncrementallySkip = false;
		}
		else
		{
			if (Context.bPreviouslyCooked)
			{
				SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_Cooked);
			}
			else
			{
				SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_NotCooked);
			}
			if (!bKnownGenerator && bReadOnly)
			{
				Context.bInOutShouldIncrementallySkip = true;
			}
		}
		break;
	case EPhase::Phase1:
		// Save the incrementally unmodified packages to verify their diffs. Skip the non-generator incrementally
		// modified packages. Do not skip generator packages; we need to save them to test their generated packages.
		Context.bInOutShouldIncrementallySkip = !Context.bIncrementallyUnmodified && !bKnownGenerator;
		if (!Context.bIncrementallyUnmodified)
		{
			if (Context.bPreviouslyCooked)
			{
				SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_Cooked);
			}
			else
			{
				SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_NotCooked);
			}
		}
		else
		{
			SetPackageStatus(Context.PackageName, EPackageStatus::DeclaredUnmodified_NotYetProcessed);
		}
		break;
	case EPhase::Phase2:
	{
		// Ignore the Unmodified flag from this cook phase. Skip the packages that were found to 
		// be IncrementallyValidated from Phase1. Save the packages that phase1 found modified; this phase
		// is responsible for getting those resaved. Reexecute save for the packages that were IncrementalFailed
		// from phase1, so we can test whether they are indeterministic. Always save generators so we can
		// test their generated packages.
		EPackageStatus PackageStatus = GetPackageStatus(Context.PackageName);
		Context.bInOutShouldIncrementallySkip =
			(PackageStatus == EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified
				|| PackageStatus == EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList)
			&& !bKnownGenerator;
		break;
	}
	default:
		checkNoEntry();
		break;
	}

	FUpdatePackageModifiedStatusContext InnerContext = Context;
	InnerContext.bIncrementallyUnmodified = Context.bInOutShouldIncrementallySkip;
	InnerContext.bInOutShouldIncrementallySkip = Context.bInOutShouldIncrementallySkip;
	Inner->UpdatePackageModifiedStatus(InnerContext);
	checkf(InnerContext.bInOutShouldIncrementallySkip == InnerContext.bIncrementallyUnmodified,
		TEXT("FIncrementalValidatePackageWriter is not supported with an Inner that modifies bInOutShouldIncrementallySkip."));
}

void FIncrementalValidatePackageWriter::BeginCook(const FCookInfo& Info)
{
	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		if (bReadOnly)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"-IncrementalValidateAllowWrite not present, read-only mode. Running -diffonly on all packages that were found to be incrementally unmodified.");
		}
		else
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"-IncrementalValidateAllowWrite is present, writable mode. Resaving packages as in a normal cook, but also running -diffonly on all packages that were found to be incrementally unmodified.");
		}
		if (Info.bFullBuild)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Error,
				"IncrementalValidate was bypassed on this run; it is a full cook and all packages are marked incrementally modified.");
		}
		break;
	case EPhase::Phase1:
		UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
			LogIncrementalValidate, Display,
			"Phase1: running -diffonly and a resave on all packages discovered to be incrementally unmodified.");
		if (Info.bFullBuild)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Error,
				"IncrementalValidate was bypassed on this run; it is a full cook and all packages are marked incrementally modified.");
		}
		break;
	case EPhase::Phase2:
		{
			Load();
			FStatusCounts StatusCounts = CountPackagesByStatus();
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"Phase2: %d packages were found during Phase1 to be incrementally unmodified but had differences. "
					"Running -diffonly on them again to check whether the differences are due to indeterminism or to FalsePositiveIncrementalSkips.", 
					StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive]);
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display,
				"%d packages were found during Phase1 to be modified or new and will be resaved.",
				StatusCounts[EPackageStatus::DeclaredModified_WillNotVerify_Cooked]);
			break;
		}
	default:
		checkNoEntry();
		break;
	}
	Super::BeginCook(Info);
}

void FIncrementalValidatePackageWriter::EndCook(const FCookInfo& Info)
{
	Super::EndCook(Info);
	FStatusCounts StatusCounts = CountPackagesByStatus();

	switch (Phase)
	{
	case EPhase::AllInOnePhase:
	{
		int32 DetectedUnmodified = StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList];
		UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
			LogIncrementalValidate, Display,
			"Modified: %d. DetectedUnmodified: %d. ValidatedUnmodified: %d. IncrementalSkipFalsePositive: %d.",
			StatusCounts[EPackageStatus::DeclaredModified_WillNotVerify_Cooked],
			DetectedUnmodified,
			StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList],
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]);
		FString Message = FString::Printf(TEXT("Packages Incrementally Skipped: %d: IncrementalSkipFalsePositive: %d."),
			DetectedUnmodified,
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]);
		if (StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive] > 0)
		{
			TStringBuilder<1024> MessageWithDiagnostics;
			MessageWithDiagnostics << Message;
			TMap<FTopLevelAssetPath, TArray<FName>> ClassFalsePositiveCounts = GetClassStatusSummary(
				EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive);
			int32 NumClassesPrinted = 0;
			constexpr int32 MaxNumClassesPrinted = 25;
			for (const TPair<FTopLevelAssetPath, TArray<FName>>& Pair : ClassFalsePositiveCounts)
			{
				MessageWithDiagnostics << TEXT("\n\t") << WriteToString<256>(Pair.Key) << TEXT(": ") << Pair.Value.Num();

				int32 NumPackagesPrinted = 0;
				constexpr int32 MaxNumPackagesPrinted = 10;
				for (FName PackageName : Pair.Value)
				{
					MessageWithDiagnostics << TEXT("\n\t\t") << PackageName;
					if (++NumPackagesPrinted >= MaxNumPackagesPrinted)
					{
						break;
					}
				}
				if (Pair.Value.Num() > NumPackagesPrinted)
				{
					MessageWithDiagnostics << TEXT("\n\t\t...");
				}

				if (++NumClassesPrinted >= MaxNumClassesPrinted)
				{
					break;
				}
			}
			if (ClassFalsePositiveCounts.Num() > NumClassesPrinted)
			{
				MessageWithDiagnostics << TEXT("\n\t...");
			}
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Error, "%ls", *MessageWithDiagnostics);
		}
		else if (StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList] > 0)
		{
			Message = FString::Printf(TEXT("Packages Incrementally Skipped: %d: IncrementalSkipFalsePositive (Ignored): %d."),
				DetectedUnmodified,
				StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList]);
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Warning, "%ls", *Message);
		}
		else
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display, "%ls", *Message);
		}
		break;
	}
	case EPhase::Phase1:
	{
		int32 DetectedUnmodified = StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList];
		UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
			LogIncrementalValidate, Display,
			"Modified: %d. DetectedUnmodified: %d. ValidatedUnmodified: %d. IncrementalSkipFalsePositiveOrIndeterminism: %d.",
			StatusCounts[EPackageStatus::DeclaredModified_WillNotVerify_Cooked],
			DetectedUnmodified,
			StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList],
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive]);
		Save();
		break;
	}
	case EPhase::Phase2:
	{
		int32 DetectedUnmodified = StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_Indeterminism]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList];
		UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
			LogIncrementalValidate, Display,
			"Modified: %d. DetectedUnmodified: %d. ValidatedUnmodified: %d. Indeterminism: %d. IncrementalSkipFalsePositive: %d.",
			StatusCounts[EPackageStatus::DeclaredModified_WillNotVerify_Cooked],
			DetectedUnmodified,
			StatusCounts[EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified]
			+ StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList],
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_Indeterminism],
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]
			);
		FString Message = FString::Printf(TEXT("Packages Incrementally Skipped: %d: IncrementalSkipFalsePositive: %d."),
			DetectedUnmodified,
			StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]);
		if (StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive] > 0)
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Error, "%ls", *Message);
		}
		else if (StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList] > 0)
		{
			Message = FString::Printf(TEXT("Packages Incrementally Skipped: %d: IncrementalSkipFalsePositive (Ignored): %d."),
				DetectedUnmodified,
				StatusCounts[EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive]);
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Warning, "%ls", *Message);
		}
		else
		{
			UE_CLOGF(COTFS.GetCookMode() != ECookMode::CookWorker,
				LogIncrementalValidate, Display, "%ls", *Message);
		}
		break;
	}
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::UpdateSaveArguments(FSavePackageArgs& SaveArgs)
{
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		Super::UpdateSaveArguments(SaveArgs);
		break;
	case ESaveAction::SaveToInner:
		Inner->UpdateSaveArguments(SaveArgs);
		break;
	case ESaveAction::IgnoreResults:
		break;
	default:
		checkNoEntry();
		break;
	}
}

bool FIncrementalValidatePackageWriter::IsAnotherSaveNeeded(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs)
{
	bool bResult = IsAnotherSaveNeededInternal(PreviousResult, SaveArgs);
	if (!bResult && COTFS.GetCookMode() != ECookMode::CookWorker)
	{
		MarkPackageCompletedOnDirector(BeginInfo.PackageName, UE::Cook::FWorkerId::Local());
	}
	return bResult;

}

bool FIncrementalValidatePackageWriter::IsAnotherSaveNeededInternal(FSavePackageResultStruct& PreviousResult,
	FSavePackageArgs& SaveArgs)
{
	bPackageFirstPass = false;
	checkf(!Inner->IsAnotherSaveNeeded(PreviousResult, SaveArgs),
		TEXT("FIncrementalValidatePackageWriter does not support an Inner that needs multiple saves."));
	if (PreviousResult == ESavePackageResult::Timeout)
	{
		return false;
	}
	switch (SaveAction)
	{
	case ESaveAction::CheckForDiffs:
		break;
	case ESaveAction::SaveToInner:
		// The SaveToInner pass, if present, is the last pass in a phase
		return false;
	case ESaveAction::IgnoreResults:
		// The IgnoreResults pass, if present, is the last pass in a phase
		return false;
	default:
		checkNoEntry();
		break;
	}

	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		check(GetPackageStatus(BeginInfo.PackageName) == EPackageStatus::DeclaredUnmodified_NotYetProcessed); // Otherwise we would have set SaveAction=SaveToInner or IgnoreResults and early exited above
		if (Super::IsAnotherSaveNeeded(PreviousResult, SaveArgs))
		{
			return true;
		}
		else
		{
			// Once our superclass has finished looking for differences, finish it off and start a SaveToInner pass
			FCommitPackageInfo CommitInfo;
			CommitInfo.Status = IPackageWriter::ECommitStatus::Success;
			CommitInfo.PackageName = BeginInfo.PackageName;
			CommitInfo.WriteOptions = EWriteOptions::None;
			Super::CommitPackage(MoveTemp(CommitInfo));

			if (bIsDifferent && !bNewPackage)
			{
				if (!PackageIgnoreList.Contains(BeginInfo.PackageName))
				{
					SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive);
				}
				else
				{
					SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList);
				}
			}
			else if (!bNewPackage)
			{
				SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified);
			}
			else
			{
				SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_Cooked);
			}

			if (bReadOnly)
			{
				SaveAction = ESaveAction::IgnoreResults;
				return false;
			}
			else
			{
				Inner->BeginPackage(BeginInfo);
				SaveAction = ESaveAction::SaveToInner;
				return true;
			}
		}
	case EPhase::Phase1:
		if (Super::IsAnotherSaveNeeded(PreviousResult, SaveArgs))
		{
			return true;
		}
		else if (bIsDifferent && !bNewPackage)
		{
			// If our superclass FDiffPackageWriter found differences, when it finishes the saves it wants to do,
			// Finish it off and start a SaveToInner pass
			FCommitPackageInfo CommitInfo;
			CommitInfo.Status = IPackageWriter::ECommitStatus::Success;
			CommitInfo.PackageName = BeginInfo.PackageName;
			CommitInfo.WriteOptions = EWriteOptions::None;
			Super::CommitPackage(MoveTemp(CommitInfo));
			Inner->BeginPackage(BeginInfo);
			SaveAction = ESaveAction::SaveToInner;

			if (!PackageIgnoreList.Contains(BeginInfo.PackageName))
			{
				// Mark that the incremental validation failed if it was not already marked by log or warning messages.
				// We need to record it for an indeterminism test
				SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive);
			}
			else
			{
				SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_OnIgnoreList);
			}

			return true;
		}
		else if (!bNewPackage)
		{
			// No differences found, so finish off the superclass's save during CommitPackage, without doing a
			// SaveToInner pass
			// Mark that the incremental validation passed
			SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified);

			TArray<FMessage> Messages;
			PackageMessageMap.RemoveAndCopyValue(BeginInfo.PackageName, Messages);
			for (FMessage& Message : Messages)
			{
				// If no differences were detected, we should not have logged any warning or error messages
				check(Message.Verbosity > ELogVerbosity::Warning);
			}
			return false;
		}
		else
		{
			// New packages need to be resaved in Phase2; for our purposes they are equivalent
			// to a package that the IncrementalCook detected as modified.
			// Do not add an entry for it in our results for incremental packages, and do not resave it in this pass
			SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredModified_WillNotVerify_Cooked);
			return false;
		}
	case EPhase::Phase2:
		LogIncrementalDifferences();
		// No need to do the Super's second diff pass to find callstacks - just knowing whether differences exist is enough.
		// No need to save package to disk; for these packages (packages found to be incrementally unmodified during
		// Phase1) they were already resaved during Phase1
		return false;
	default:
		checkNoEntry();
		return false;
	}
}

bool FIncrementalValidatePackageWriter::IsReadOnly() const
{
	return bReadOnly;
}

void FIncrementalValidatePackageWriter::OnDiffWriterMessage(ELogVerbosity::Type Verbosity, FStringView Message)
{
	PackageMessageMap.FindOrAdd(BeginInfo.PackageName).Add(FMessage{ FString(Message), Verbosity });
}

void FIncrementalValidatePackageWriter::MarkPackageCompletedOnDirector(FName PackageName,
	UE::Cook::FWorkerId WorkerId)
{
	FPackageStatusInfo* Status = PackageStatusMap.Find(PackageName);
	TArray<FMessage>* Messages = PackageMessageMap.Find(PackageName);

	if (!Status || !Messages)
	{
		return;
	}

	int32& TotalCount = TotalStatusCounts.FindOrAdd(Status->Status, 0);
	TArray<FName>& ClassStatusArray = ClassStatusSummary.FindOrAdd(Status->AssetClass).FindOrAdd(Status->Status);
	++TotalCount;
	ClassStatusArray.Add(PackageName);
	if (LoggingSoftMaximum >= 0 && TotalCount > LoggingSoftMaximum && ClassStatusArray.Num() > 1)
	{
		// Suppress reported logs for this package
		return;
	}

	switch (Phase)
	{
	case EPhase::AllInOnePhase:
		for (FMessage& Message : *Messages)
		{
			FMsg::Logf(__FILE__, __LINE__, LogIncrementalValidate.GetCategoryName(), Message.Verbosity,
				TEXT("%s"), *ResolveText(Message.Text));
		}
		break;
	case EPhase::Phase1:
		break;
	case EPhase::Phase2:
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FIncrementalValidatePackageWriter::LogIncrementalDifferences()
{
	// This function is called during Phase2 for packages that had differences during Phase1.
	// It is called immediately after first-pass package save that is run by our super class FDiffPackageWriter, which
	// compared it against the version that was written to disk during Phase1. If there are differences
	// now from Phase1, then this package has a determinism issue. We log that information at display rather
	// than Warning because this cookmode only logs warnings for IncrementalSkipFalsePositives.
	bool bHasDeterminismIssue = bIsDifferent;
	if (bHasDeterminismIssue)
	{
		UE_LOGF(LogIncrementalValidate, Display, "Could not validate %ls because it has a non-deterministic save.",
			*BeginInfo.PackageName.ToString());
		SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_Indeterminism);
		return;
	}

	// Otherwise, no determinism issues, so the differences indicate a bug in Diff Package
	SetPackageStatus(BeginInfo.PackageName, EPackageStatus::DeclaredUnmodified_FoundModified_FalsePositive);
	FMsg::Logf(__FILE__, __LINE__, LogIncrementalValidate.GetCategoryName(), ELogVerbosity::Warning,
		TEXT("IncrementalSkipFalsePositive package %s."), *BeginInfo.PackageName.ToString());
	TArray<FMessage>& Messages = PackageMessageMap.FindOrAdd(BeginInfo.PackageName);
	for (const FMessage& Message : Messages)
	{
		FMsg::Logf(__FILE__, __LINE__, LogIncrementalValidate.GetCategoryName(), Message.Verbosity,
			TEXT("%s"), *ResolveText(Message.Text));
	}
}

void FIncrementalValidatePackageWriter::Save()
{
	FString IncrementalValidatePath = GetIncrementalValidatePath();
	TUniquePtr<FArchive> DiskArchive(IFileManager::Get().CreateFileWriter(*IncrementalValidatePath));

	if (!DiskArchive)
	{
		UE_LOGF(LogIncrementalValidate, Error,
			"Could not write to file %ls. This file is needed to store results for the -IncrementalValidate cook.",
			*IncrementalValidatePath);
		return;
	}
	FNameAsStringProxyArchive Ar(*DiskArchive);
	Serialize(Ar);
}

void FIncrementalValidatePackageWriter::Load()
{
	FString IncrementalValidatePath = GetIncrementalValidatePath();
	TUniquePtr<FArchive> DiskArchive(IFileManager::Get().CreateFileReader(*IncrementalValidatePath));
	if (!DiskArchive)
	{
		UE_LOGF(LogIncrementalValidate, Fatal,
			"Could not load file %ls. This file is required and should have been written by the -IncrementalValidatePhase1 cook.",
			*IncrementalValidatePath);
		return;
	}
	FNameAsStringProxyArchive Ar(*DiskArchive);
	Serialize(Ar);
	if (Ar.IsError())
	{
		UE_LOGF(LogIncrementalValidate, Fatal, "Corrupt file %ls", *IncrementalValidatePath);
	}
}

void FIncrementalValidatePackageWriter::Serialize(FArchive& Ar)
{
	constexpr int32 LatestVersion = 0;
	int32 Version = LatestVersion;
	Ar << Version;
	if (Ar.IsLoading() && Version != LatestVersion)
	{
		Ar.SetError();
		return;
	}
	Ar << PackageStatusMap;
	Ar << PackageMessageMap;
}

FArchive& operator<<(FArchive& Ar, FIncrementalValidatePackageWriter::FMessage& Message)
{
	uint8 Verbosity = static_cast<uint8>(Message.Verbosity);
	Ar << Verbosity << Message.Text;
	if (Ar.IsLoading())
	{
		Message.Verbosity = static_cast<ELogVerbosity::Type>(Verbosity);
	}
	return Ar;
}

FString FIncrementalValidatePackageWriter::GetIncrementalValidatePath() const
{
	return FPaths::Combine(MetadataPath, FString(IncrementalValidateFilename));
}

FIncrementalValidatePackageWriter::EPackageStatus FIncrementalValidatePackageWriter::GetPackageStatus(FName PackageName) const
{
	if (const FPackageStatusInfo* Info = PackageStatusMap.Find(PackageName))
	{
		return Info->Status;
	}
	return EPackageStatus::NotYetProcessed;
}

void FIncrementalValidatePackageWriter::SetPackageStatus(FName PackageName, EPackageStatus NewStatus)
{
	FPackageStatusInfo& Info = PackageStatusMap.FindOrAdd(PackageName);
	Info.Status = NewStatus;
	switch (NewStatus)
	{
	case EPackageStatus::DeclaredUnmodified_ConfirmedUnmodified: [[fallthrough]];
	case EPackageStatus::DeclaredModified_WillNotVerify_Cooked: [[fallthrough]];
	case EPackageStatus::DeclaredModified_WillNotVerify_NotCooked:
		// For the non-error cases, clear the AssetClass so that we don't waste bandwidth or diskspace to store it.
		Info.AssetClass = FTopLevelAssetPath();
		break;
	default:
		break;
	}
}

FIncrementalValidatePackageWriter::FStatusCounts FIncrementalValidatePackageWriter::CountPackagesByStatus()
{
	FIncrementalValidatePackageWriter::FStatusCounts StatusCounts;

	for (const TPair<FName,FPackageStatusInfo>& Pair : PackageStatusMap)
	{
		StatusCounts[Pair.Value.Status]++;
	}

	return StatusCounts;
}

TMap<FTopLevelAssetPath, TArray<FName>> FIncrementalValidatePackageWriter::GetClassStatusSummary(EPackageStatus PackageStatus)
{
	TMap<FTopLevelAssetPath, TArray<FName>> Result;
	for (const TPair<FTopLevelAssetPath, TMap<EPackageStatus, TArray<FName>>>& Pair : ClassStatusSummary)
	{
		const TArray<FName>* Packages = Pair.Value.Find(PackageStatus);
		if (Packages)
		{
			Result.Add(Pair.Key, *Packages);
		}
	}

	Result.ValueSort([](const TArray<FName>& A, const TArray<FName>& B)
		{
			return A.Num() > B.Num();
		});
	return Result;
}
