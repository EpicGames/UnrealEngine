// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DiffPackageWriter.h"

class FIncrementalValidateMPCollector; 
namespace UE::Cook { struct FWorkerId; }

/**
 * A CookedPackageWriter that diffs the cook results of incrementally-unmodified packages between their last cook
 * results and the current cook.
 */
class FIncrementalValidatePackageWriter : public FDiffPackageWriter
{
public:
	using Super = FDiffPackageWriter;
	enum class EPhase
	{
		AllInOnePhase,
		Phase1,
		Phase2,
	};
	FIncrementalValidatePackageWriter(UCookOnTheFlyServer& InCOTFS, TUniquePtr<ICookedPackageWriter>&& InInner,
		EPhase InPhase, const FString& ResolvedMetadataPath,
		UE::Cook::FDeterminismManager* InDeterminismManager);

	// IPackageWriter
	virtual void BeginPackage(const FBeginPackageInfo& Info) override;
	virtual void CommitPackage(FCommitPackageInfo&& Info) override;
	virtual void WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
		const TArray<FFileRegion>& FileRegions) override;
	virtual void WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData,
		const TArray<FFileRegion>& FileRegions) override;
	virtual void WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData) override;
	virtual void WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info, const FIoBuffer& Data,
		const TArray<FFileRegion>& FileRegions) override;
	virtual void WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data) override;
	virtual int64 GetExportsFooterSize() override;
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) override;
	virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerExportsArchive(FName PackageName, UObject* Asset, uint16 MultiOutputIndex) override;
	virtual bool IsPreSaveCompleted() const override;

	// ICookedPackageWriter
	virtual FCookCapabilities GetCookCapabilities() const override;
	virtual void Initialize(const FCookInfo& CookInfo) override;
	virtual void UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context) override;
	virtual void BeginCook(const FCookInfo& Info) override;
	virtual void EndCook(const FCookInfo& Info) override;
	virtual void UpdateSaveArguments(FSavePackageArgs& SaveArgs) override;
	virtual bool IsAnotherSaveNeeded(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs) override;

	// FIncrementalValidatePackageWriter-specific public functions
	bool IsReadOnly() const;

protected:
	enum class ESaveAction : uint8
	{
		CheckForDiffs,
		SaveToInner,
		IgnoreResults,
	};

	enum class EPackageStatus : uint8
	{
		NotYetProcessed,
		DeclaredUnmodified_ConfirmedUnmodified,
		DeclaredUnmodified_FoundModified_IndeterminismOrFalsePositive,
		DeclaredUnmodified_FoundModified_Indeterminism,
		DeclaredUnmodified_FoundModified_FalsePositive,
		DeclaredUnmodified_FoundModified_OnIgnoreList,
		DeclaredUnmodified_NotYetProcessed,
		DeclaredModified_WillNotVerify_NotCooked,
		DeclaredModified_WillNotVerify_Cooked,
		Count
	};

	struct FMessage
	{
		FString Text;
		ELogVerbosity::Type Verbosity;
	};
	friend FArchive& operator<<(FArchive& Ar, FMessage& Message);

	struct FStatusCounts
	{
		FStatusCounts()
		{
			Data = MakeUniformStaticArray<uint32, (uint32)EPackageStatus::Count>(0);
		}

		uint32& operator[](EPackageStatus PackageStatus) 
		{ 
			return Data[(uint32)PackageStatus]; 
		}

	private:
		TStaticArray<uint32, (uint32)EPackageStatus::Count> Data;
	};

	struct FPackageStatusInfo
	{
		FTopLevelAssetPath AssetClass;
		EPackageStatus Status;
	};

protected:
	virtual void OnDiffWriterMessage(ELogVerbosity::Type Verbosity, FStringView Message) override;
	void LogIncrementalDifferences();
	void Save();
	void Load();
	void Serialize(FArchive& Ar);
	FString GetIncrementalValidatePath() const;
	EPackageStatus GetPackageStatus(FName PackageName) const;
	void SetPackageStatus(FName PackageName, EPackageStatus NewStatus);
	FStatusCounts CountPackagesByStatus();
	TMap<FTopLevelAssetPath, TArray<FName>> GetClassStatusSummary(EPackageStatus PackageStatus);
	bool IsAnotherSaveNeededInternal(FSavePackageResultStruct& PreviousResult, FSavePackageArgs& SaveArgs);
	void MarkPackageCompletedOnDirector(FName PackageName, UE::Cook::FWorkerId WorkerId);

protected:
	TMap<FName, FPackageStatusInfo> PackageStatusMap;
	TMap<FName, TArray<FMessage>> PackageMessageMap;
	TMap<EPackageStatus, int32> TotalStatusCounts; // Only populated on director
	TMap<FTopLevelAssetPath, TMap<EPackageStatus, TArray<FName>>> ClassStatusSummary; // Only populated on director
	TSet<FName> PackageIgnoreList;
	FString MetadataPath;
	int32 LoggingSoftMaximum = -1;
	EPhase Phase = EPhase::AllInOnePhase;
	ESaveAction SaveAction = ESaveAction::IgnoreResults;
	bool bPackageFirstPass = false;
	bool bReadOnly = true;

	friend class FIncrementalValidateMPCollector;
	friend FCbWriter& operator<<(FCbWriter& Writer, const FMessage& Message);
	friend bool LoadFromCompactBinary(FCbFieldView Field, FMessage& Path);
	friend FCbWriter& operator<<(FCbWriter& Writer, EPackageStatus Status);
	friend bool LoadFromCompactBinary(FCbFieldView Field, EPackageStatus& Status);
	friend FArchive& operator<<(FArchive& Writer, FPackageStatusInfo& Info);
	friend FCbWriter& operator<<(FCbWriter& Writer, const FPackageStatusInfo& Info);
	friend bool LoadFromCompactBinary(FCbFieldView Field, FPackageStatusInfo& Info);
};


