// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreCookOnTheFlyRequestManager.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "Cooker/ExternalCookOnTheFlyServer.h"
#include "CookOnTheFly.h"
#include "CookOnTheFlyMessages.h"
#include "CookOnTheFlyNetServer.h"
#include "CookOnTheFlyServerInterface.h"
#include "Engine/Engine.h"
#include "HAL/PlatformTime.h"
#include "IPAddress.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "NetworkMessage.h"
#include "PackageStoreWriter.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/BufferArchive.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Templates/Function.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/SavePackage.h"
#include "ZenStoreWriter.h"

class FPackageRegistry
{
public:
	void Add(TArrayView<FAssetData> Assets)
	{
		FScopeLock _(&CriticalSection);

		for (const FAssetData& Asset : Assets)
		{
			PackageIdToName.Add(FPackageId::FromName(Asset.PackageName), Asset.PackageName);
		}
	}

	void Add(FName PackageName)
	{
		FScopeLock _(&CriticalSection);
		PackageIdToName.Add(FPackageId::FromName(PackageName), PackageName);
	}

	FName Get(FPackageId PackageId) const
	{
		FScopeLock _(&CriticalSection);
		return PackageIdToName.FindRef(PackageId);
	}

	uint64 Num() const
	{
		FScopeLock _(&CriticalSection);
		return PackageIdToName.Num();
	}

	/**
	 * Iterate over all entries in the registry
	 */
	void ForEachPackage(TFunction<void(const FPackageId&, const FName&)> Visitor) const
	{
		FScopeLock _(&CriticalSection);
		for (const TPair<FPackageId, FName>& Entry : PackageIdToName)
		{
			Visitor(Entry.Key, Entry.Value);
		}
	}

private:
	TMap<FPackageId, FName> PackageIdToName;
	mutable FCriticalSection CriticalSection;
};

class FIoStoreCookOnTheFlyRequestManager final
	: public UE::Cook::ICookOnTheFlyRequestManager
{
public:
	FIoStoreCookOnTheFlyRequestManager(UE::Cook::ICookOnTheFlyServer& InCookOnTheFlyServer, const IAssetRegistry* AssetRegistry, TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> InConnectionServer)
		: CookOnTheFlyServer(InCookOnTheFlyServer)
		, ConnectionServer(InConnectionServer)
		, ServiceId(FExternalCookOnTheFlyServer::GenerateServiceId())
	{
		using namespace UE::Cook;

		ConnectionServer->OnClientConnected().AddRaw(this, &FIoStoreCookOnTheFlyRequestManager::OnClientConnected);
		ConnectionServer->OnClientDisconnected().AddRaw(this, &FIoStoreCookOnTheFlyRequestManager::OnClientDisconnected);
		ConnectionServer->OnRequest(ECookOnTheFlyMessage::GetCookedPackages).BindRaw(this, &FIoStoreCookOnTheFlyRequestManager::HandleClientRequest);
		ConnectionServer->OnRequest(ECookOnTheFlyMessage::CookPackage).BindRaw(this, &FIoStoreCookOnTheFlyRequestManager::HandleClientRequest);
		ConnectionServer->OnRequest(ECookOnTheFlyMessage::RecookPackages).BindRaw(this, &FIoStoreCookOnTheFlyRequestManager::HandleClientRequest);
		ConnectionServer->OnRequest(ECookOnTheFlyMessage::GetAllOnDiskPackages).BindRaw(this, &FIoStoreCookOnTheFlyRequestManager::HandleClientRequest);

		MessageEndpoint = FMessageEndpoint::Builder("FCookOnTheFly");
	}

private:
	class FPlatformContext
	{
	public:
		enum class EPackageStatus
		{
			None,
			Cooking,
			Cooked,
			Failed,
		};

		struct FPackage
		{
			EPackageStatus Status = EPackageStatus::None;
			FPackageStoreEntryResource Entry;
		};

		FPlatformContext(FIoStoreCookOnTheFlyRequestManager& InManager, FName InPlatformName, IPackageStoreWriter* InPackageWriter)
			: PlatformName(InPlatformName)
			, Manager(InManager)
			, PackageWriter(InPackageWriter)
		{
		}

		~FPlatformContext()
		{
			if (PackageCookedEvent)
			{
				FPlatformProcess::ReturnSynchEventToPool(PackageCookedEvent);
			}
			FScopeLock _(&GetLock());
			if (bCookSessionRequested)
			{
				// The request can only be in progress if there is another thread calling RequestCookSession,
				// and all threads should have completed before we are allowed to destroy. If we ever destroy
				// when requests are still in progress, we will need to add a cancel mechanic and wait for the
				// cancel here.
				check(!bCookSessionRequestInProgress);
				Manager.CookOnTheFlyServer.RemovePlatform(PlatformName);
				bCookSessionRequested = false;
				CookerReadyForRequests->Reset();
			}
		}

		FCriticalSection& GetLock()
		{
			return CriticalSection;
		}

		void GetCompletedPackages(UE::ZenCookOnTheFly::Messaging::FCompletedPackages& OutCompletedPackages)
		{
			OutCompletedPackages.CookedPackages.Reserve(OutCompletedPackages.CookedPackages.Num() + Packages.Num());
			for (const auto& KV : Packages)
			{
				if (KV.Value->Status == EPackageStatus::Cooked)
				{
					OutCompletedPackages.CookedPackages.Add(KV.Value->Entry);
				}
				else if (KV.Value->Status == EPackageStatus::Failed)
				{
					OutCompletedPackages.FailedPackages.Add(KV.Key);
				}
			}
		}

		void OnSessionStarted()
		{
			FScopeLock _(&GetLock());
			if (bCookSessionRequested)
			{
				return;
			}

			if (bCookSessionRequestInProgress)
			{
				bCookSessionRequested = true;
				bCookSessionRequestInProgress = false;
			}
			CookerReadyForRequests->Trigger();
		}

		void RequestCookSessionAndWait()
		{
			bool bCallOnSessionStartedManually = false;
			{
				FScopeLock _(&GetLock());
				if (bCookSessionRequested)
				{
					return;
				}
				if (!bCookSessionRequestInProgress)
				{
					bCookSessionRequestInProgress = true;
					bool bAlreadyInitialized = false;
					Manager.CookOnTheFlyServer.AddPlatform(PlatformName, bAlreadyInitialized);
					if (bAlreadyInitialized)
					{
						// The platform has already been initialized by an AddPlatform call from elsewhere
						// and the cooker will not be sending an OnSessionStarted event. Trigger it now.
						bCallOnSessionStartedManually = true;
					}
				}
			}
			if (bCallOnSessionStartedManually)
			{
				Manager.OnSessionStarted(PlatformName, false /* bFirstSessionInThisProcess */);
			}

			if (CookerReadyForRequests->Wait(0))
			{
				return;
			}
			// AddPlatform always queues the request rather than handling it immediately,
			// and it handles it later in the tick on the cooker's scheduler thread. So if
			// we try to wait for the event from the scheduler thread we will deadlock.
			check(!Manager.CookOnTheFlyServer.IsSchedulerThread());
			CookerReadyForRequests->Wait();
		}

		EPackageStoreEntryStatus RequestCook(UE::Cook::ICookOnTheFlyServer& InCookOnTheFlyServer, const FPackageId& PackageId, FPackageStoreEntryResource& OutEntry)
		{
			FPackage& Package = GetPackage(PackageId);
			check(Manager.bCookerGlobalsInitialized); // Caller should have called RequestCookSessionAndWait
			FName PackageName = Manager.PackageRegistry.Get(PackageId);
			if (Package.Status == EPackageStatus::Cooked)
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "0x%ls was already cooked", *LexToString(PackageId));
				OutEntry = Package.Entry;
				return EPackageStoreEntryStatus::Ok;
			}
			else if (Package.Status == EPackageStatus::Failed)
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Failed to cook package 0x%ls '%ls'", *LexToString(PackageId), *PackageName.ToString());
				return EPackageStoreEntryStatus::Missing;
			}
			else if (Package.Status == EPackageStatus::Cooking)
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "0x%ls was already cooking", *LexToString(PackageId));
				return EPackageStoreEntryStatus::Pending;
			}
			if (PackageName.IsNone())
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Received cook request for unknown package 0x%ls", *LexToString(PackageId));
				return EPackageStoreEntryStatus::Missing;
			}
			FString Filename;
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageName.ToString(), Filename))
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "Cooking package 0x%ls '%ls'", *LexToString(PackageId), *PackageName.ToString());
				Package.Status = EPackageStatus::Cooking;
				const bool bEnqueued = InCookOnTheFlyServer.EnqueueCookRequest(UE::Cook::FCookPackageRequest{ PlatformName, Filename });
				check(bEnqueued);
				return EPackageStoreEntryStatus::Pending;
			}
			else
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Failed to cook package 0x%ls '%ls' (File not found)", *LexToString(PackageId), *PackageName.ToString());
				Package.Status = EPackageStatus::Failed;
				return EPackageStoreEntryStatus::Missing;
			}
		}

		void RequestRecook(UE::Cook::ICookOnTheFlyServer& InCookOnTheFlyServer, const FPackageId& PackageId, const FName& PackageName)
		{
			FPackage& Package = GetPackage(PackageId);
			if (Package.Status != EPackageStatus::Cooked && Package.Status != EPackageStatus::Failed)
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "Skipping recook of package 0x%ls '%ls' that was not cooked", *LexToString(PackageId), *PackageName.ToString());
				return;
			}
			FString Filename;
			if (FPackageName::TryConvertLongPackageNameToFilename(PackageName.ToString(), Filename))
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "Recooking package 0x%ls '%ls'", *LexToString(PackageId), *PackageName.ToString());
				Package.Status = EPackageStatus::Cooking;
				const bool bEnqueued = InCookOnTheFlyServer.EnqueueCookRequest(UE::Cook::FCookPackageRequest{ PlatformName, Filename });
				check(bEnqueued);
			}
			else
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Failed to recook package 0x%ls '%ls' (File not found)", *LexToString(PackageId), *PackageName.ToString());
				Package.Status = EPackageStatus::Failed;
			}
		}

		void MarkAsFailed(FPackageId PackageId, FName PackageName, UE::ZenCookOnTheFly::Messaging::FCompletedPackages& OutCompletedPackages)
		{
			// Unsolicited packages that fail to save because they contain editoronly data will be passed into this function
			// Mark them as failed, but do not report them to the client and do not give a warning unless the client requests them.
			FPackage& Package = GetPackage(PackageId);
			Package.Status = EPackageStatus::Failed;
			if (PackageCookedEvent)
			{
				PackageCookedEvent->Trigger();
			}
		}

		void MarkAsCooked(FPackageId PackageId, const FPackageStoreEntryResource& Entry, UE::ZenCookOnTheFly::Messaging::FCompletedPackages& OutCompletedPackages)
		{
			UE_LOGF(LogCookOnTheFly, Verbose, "0x%ls cooked", *LexToString(PackageId));
			FPackage& Package = GetPackage(PackageId);
			Package.Status = EPackageStatus::Cooked;
			Package.Entry = Entry;
			OutCompletedPackages.CookedPackages.Add(Entry);
			if (PackageCookedEvent)
			{
				PackageCookedEvent->Trigger();
			}
		}

		void AddExistingPackages(TArrayView<const FPackageStoreEntryResource> Entries, TArrayView<const IPackageStoreWriter::FOplogCookInfo> CookInfos)
		{
			Packages.Reserve(Entries.Num());

			for (int32 EntryIndex = 0, Num = Entries.Num(); EntryIndex < Num; ++EntryIndex)
			{
				const FPackageStoreEntryResource& Entry = Entries[EntryIndex];
				const IPackageStoreWriter::FOplogCookInfo& CookInfo = CookInfos[EntryIndex];
				const FPackageId PackageId = Entry.GetPackageId();

				FPackage& Package = GetPackage(PackageId);

				if (CookInfo.bUpToDate)
				{
					Package.Status = EPackageStatus::Cooked;
					Package.Entry = Entry;
				}
				else
				{
					Package.Status = EPackageStatus::None;
				}
			}
		}

		FPackage& GetPackage(FPackageId PackageId)
		{
			TUniquePtr<FPackage>& Package = Packages.FindOrAdd(PackageId, MakeUnique<FPackage>());
			check(Package.IsValid());
			return *Package;
		}

		void AddSingleThreadedClient()
		{
			++SingleThreadedClientsCount;
			if (!PackageCookedEvent)
			{
				PackageCookedEvent = FPlatformProcess::GetSynchEventFromPool();
			}
		}

		void RemoveSingleThreadedClient()
		{
			check(SingleThreadedClientsCount >= 0);
			if (!SingleThreadedClientsCount)
			{
				FPlatformProcess::ReturnSynchEventToPool(PackageCookedEvent);
				PackageCookedEvent = nullptr;
			}
		}

		void WaitForCook()
		{
			check(PackageCookedEvent);
			PackageCookedEvent->Wait();
		}

	private:
		FCriticalSection CriticalSection;
		FName PlatformName;
		TMap<FPackageId, TUniquePtr<FPackage>> Packages;
		FIoStoreCookOnTheFlyRequestManager& Manager;
		IPackageStoreWriter* PackageWriter = nullptr;
		int32 SingleThreadedClientsCount = 0;
		FEvent* PackageCookedEvent = nullptr;
		FEventRef CookerReadyForRequests;
		bool bCookSessionRequested = false;
		bool bCookSessionRequestInProgress = false;
	};

	virtual bool Initialize() override
	{
		using namespace UE::Cook;

		TArray<TSharedPtr<FInternetAddr>> ListenAddresses;
		if (MessageEndpoint.IsValid() && ConnectionServer->GetAddressList(ListenAddresses))
		{
			FZenCookOnTheFlyRegisterServiceMessage* RegisterServiceMessage = FMessageEndpoint::MakeMessage<FZenCookOnTheFlyRegisterServiceMessage>();
			RegisterServiceMessage->ServiceId = ServiceId;
			RegisterServiceMessage->Port = ListenAddresses[0]->GetPort();
			MessageEndpoint->Publish(RegisterServiceMessage);
		}
		return true;
	}

	void InitializeCookSessionGlobals()
	{
		// This function is only called from Cooker's scheduler thread via OnSessionStarted, so we do not need
		// to worry about synchronization within it. Threads reading the information written here must call 
		// RequestCookSessionAndWait or assert that bCookerGlobalsInitialized is true.
		if (bCookerGlobalsInitialized)
		{
			return;
		}
		ON_SCOPE_EXIT{ bCookerGlobalsInitialized = true; };

		TArray<FAssetData> AllAssets;
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		AssetRegistry->GetAllAssets(AllAssets, true);
		PackageRegistry.Add(AllAssets);
	}

	virtual void OnSessionStarted(FName PlatformName, bool bFirstSessionInThisProcess) override
	{
		InitializeCookSessionGlobals();
		FPlatformContext* Context = FindContext(PlatformName);
		if (Context)
		{
			Context->OnSessionStarted();
		}
	}

	virtual void Shutdown() override
	{
	}

	virtual void OnPackageGenerated(const FName& PackageName) override
	{
		FPackageId PackageId = FPackageId::FromName(PackageName);
		UE_LOGF(LogCookOnTheFly, Verbose, "Package 0x%ls '%ls' generated", *LexToString(PackageId), *PackageName.ToString());
		PackageRegistry.Add(PackageName);
	}

	void TickRecookPackages()
	{
		TArray<FPackageId, TInlineAllocator<128>> PackageIds;
		{
			FScopeLock _(&PackagesToRecookCritical);
			if (PackagesToRecook.IsEmpty())
			{
				return;
			}
			PackageIds = PackagesToRecook.Array();
			PackagesToRecook.Empty();
		}

		// PackageRegistry is not initialized until InitializeCookSessionGlobals, and afterwards is supposed to be immutable
		// so it can be accessed from multiple threads without a lock. We need to access it in this function, so we have to 
		// assert it has been initialized. It must be initialized if we have any PackagesToRecook, because all the functions
		// that add elements to PackagesToRecook call RequestCookSessionAndWait first
		check(bCookerGlobalsInitialized);

		TArray<FName, TInlineAllocator<128>> PackageNames;
		PackageNames.Reserve(PackageIds.Num());

		CollectGarbage(RF_NoFlags);

		for (FPackageId PackageId : PackageIds)
		{
			FName PackageName = PackageRegistry.Get(PackageId);
			if (!PackageName.IsNone())
			{
				PackageNames.Add(PackageName);

				UPackage* Package = FindObjectFast<UPackage>(nullptr, PackageName);
				if (Package)
				{
					UE_LOGF(LogCookOnTheFly, Warning, "Can't recook package '%ls'", *PackageName.ToString());
					FReferenceChainSearch::FindAndPrintStaleReferencesToObject(Package, EPrintStaleReferencesOptions::Display);
				}
				else
				{
					UE_LOGF(LogCookOnTheFly, Verbose, "Recooking package '%ls'", *PackageName.ToString());
				}
			}
		}

		for (const FName& PackageName : PackageNames)
		{
			CookOnTheFlyServer.MarkPackageDirty(PackageName);
		}

		ForEachContext([this, &PackageIds, &PackageNames](FPlatformContext& Context)
			{
				FScopeLock _(&Context.GetLock());
				for (int32 PackageIndex = 0; PackageIndex < PackageNames.Num(); ++PackageIndex)
				{
					const FPackageId& PackageId = PackageIds[PackageIndex];
					const FName& PackageName = PackageNames[PackageIndex];
					if (!PackageName.IsNone())
					{
						Context.RequestRecook(CookOnTheFlyServer, PackageId, PackageName);
					}
				}
				return true;
			});
	}

	virtual void Tick() override
	{
		TickRecookPackages();
	}

	virtual bool ShouldUseLegacyScheduling() override
	{
		return false;
	}

private:
	void OnClientConnected(UE::Cook::ICookOnTheFlyClientConnection& Connection)
	{
		const ITargetPlatform* TargetPlatform = Connection.GetTargetPlatform();
		if (TargetPlatform)
		{
			IPackageStoreWriter* PackageWriter = CookOnTheFlyServer.GetPackageWriter(TargetPlatform).AsPackageStoreWriter();
			check(PackageWriter); // This class should not be used except when COTFS is using an IPackageStoreWriter
			FZenStoreWriter* ZenStoreWriter = PackageWriter->AsZenStoreWriter();
			if (ZenStoreWriter != nullptr)
			{
				FZenStoreWriter::ZenHostInfo HostInfo = ZenStoreWriter->GetHostInfo();
				Connection.SetZenInfo(HostInfo.ProjectId, HostInfo.OplogId, HostInfo.HostName, HostInfo.HostPort);
			}
			FName PlatformName = Connection.GetPlatformName();
			FScopeLock _(&ContextsCriticalSection);
			if (!PlatformContexts.Contains(PlatformName))
			{
				TUniquePtr<FPlatformContext>& Context = PlatformContexts.Add(PlatformName,
					MakeUnique<FPlatformContext>(*this, PlatformName, PackageWriter));

				PackageWriter->GetEntries([&Context](TArrayView<const FPackageStoreEntryResource> Entries,
					TArrayView<const IPackageStoreWriter::FOplogCookInfo> CookInfos)
					{
						Context->AddExistingPackages(Entries, CookInfos);
					});

				PackageWriter->OnEntryCreated().AddRaw(this, &FIoStoreCookOnTheFlyRequestManager::OnPackageStoreEntryCreated);
				PackageWriter->OnCommit().AddRaw(this, &FIoStoreCookOnTheFlyRequestManager::OnPackageCooked);
				PackageWriter->OnMarkUpToDate().AddRaw(this, &FIoStoreCookOnTheFlyRequestManager::OnPackagesMarkedUpToDate);
			}
			FPlatformContext& Context = GetContext(PlatformName);
			FScopeLock __(&Context.GetLock());
			if (Connection.GetIsSingleThreaded())
			{
				Context.AddSingleThreadedClient();
			}
		}

		UE_LOGF(LogCookOnTheFly, Display, "New client connected");
		{
			FScopeLock _(&ClientsCriticalSection);
			Clients.Add(&Connection);
		}
	}

	void OnClientDisconnected(UE::Cook::ICookOnTheFlyClientConnection& Connection)
	{
		{
			FScopeLock _(&ClientsCriticalSection);
			Clients.Remove(&Connection);
		}
		FName PlatformName = Connection.GetPlatformName();
		if (!PlatformName.IsNone())
		{
			if (Connection.GetIsSingleThreaded())
			{
				FPlatformContext& Context = GetContext(PlatformName);
				FScopeLock __(&Context.GetLock());
				Context.RemoveSingleThreadedClient();
			}
		}
	}

	bool HandleClientRequest(UE::Cook::ICookOnTheFlyClientConnection& Connection, const UE::Cook::FCookOnTheFlyRequest& Request)
	{
		using namespace UE::Cook;

		const double StartTime = FPlatformTime::Seconds();

		UE_LOGF(LogCookOnTheFly, Verbose, "Received: %ls, Size='%lld'", *Request.GetHeader().ToString(), Request.TotalSize());

		FCookOnTheFlyResponse Response(Request);
		bool bRequestOk = false;

		UE_LOGF(LogCookOnTheFly, Verbose, "New request, Type='%ls', Client='%ls'", LexToString(Request.GetHeader().MessageType), *Connection.GetPlatformName().ToString());

		switch (Request.GetMessageType())
		{ 
			case UE::Cook::ECookOnTheFlyMessage::CookPackage:
				bRequestOk = HandleCookPackageRequest(Connection.GetPlatformName(), Connection.GetIsSingleThreaded(), Request, Response);
				break;
			case UE::Cook::ECookOnTheFlyMessage::GetCookedPackages:
				bRequestOk = HandleGetCookedPackagesRequest(Connection.GetPlatformName(), Request, Response);
				break;
			case UE::Cook::ECookOnTheFlyMessage::RecookPackages:
				bRequestOk = HandleRecookPackagesRequest(Connection.GetPlatformName(), Request, Response);
				break;
			case UE::Cook::ECookOnTheFlyMessage::GetAllOnDiskPackages:
				bRequestOk = HandleGetAllOnDiskPackagesRequest(Connection.GetPlatformName(), Request, Response);
				break;
			default:
				return false;
		}

		if (!bRequestOk)
		{
			Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Error);
		}
		bRequestOk = Connection.SendMessage(Response);
		
		const double Duration = FPlatformTime::Seconds() - StartTime;

		UE_LOGF(LogCookOnTheFly, Verbose, "Request handled, Type='%ls', Client='%ls', Status='%ls', Duration='%.6lfs'",
			LexToString(Request.GetMessageType()),
			*Connection.GetPlatformName().ToString(),
			bRequestOk ? TEXT("Ok") : TEXT("Failed"),
			Duration);

		return bRequestOk;
	}

	bool HandleGetCookedPackagesRequest(const FName& PlatformName, const UE::Cook::FCookOnTheFlyRequest& Request, UE::Cook::FCookOnTheFlyResponse& Response)
	{
		using namespace UE::Cook;
		using namespace UE::ZenCookOnTheFly::Messaging;

		if (PlatformName.IsNone())
		{
			UE_LOGF(LogCookOnTheFly, Warning, "GetCookedPackagesRequest from editor client");
			Response.SetStatus(ECookOnTheFlyMessageStatus::Error);
			return true;
		}
		else
		{
			FPlatformContext& Context = GetContext(PlatformName);
			Context.RequestCookSessionAndWait();
			FScopeLock __(&Context.GetLock());
			FCompletedPackages CompletedPackages;
			Context.GetCompletedPackages(CompletedPackages);

			Response.SetBodyTo(MoveTemp(CompletedPackages));
		}

		Response.SetStatus(ECookOnTheFlyMessageStatus::Ok);

		return true;
	}

	bool HandleCookPackageRequest(const FName& PlatformName, bool bIsSingleThreaded, const UE::Cook::FCookOnTheFlyRequest& Request, UE::Cook::FCookOnTheFlyResponse& Response)
	{
		using namespace UE::ZenCookOnTheFly::Messaging;

		if (PlatformName.IsNone())
		{
			UE_LOGF(LogCookOnTheFly, Warning, "Received cook package request for unknown platform '%ls'", *PlatformName.ToString());
			Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Error);
			return true;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::HandleCookPackageRequest);

		FPlatformContext& Context = GetContext(PlatformName);
		Context.RequestCookSessionAndWait();

		FCookPackageRequest CookRequest = Request.GetBodyAs<FCookPackageRequest>();
		FCookPackageResponse CookResponse;

		for (const FPackageId& PackageId : CookRequest.PackageIds)
		{
			UE_LOGF(LogCookOnTheFly, Verbose, "Received cook request 0x%ls", *LexToString(PackageId));

			FPackageStoreEntryResource Entry;
			EPackageStoreEntryStatus PackageStatus = EPackageStoreEntryStatus::Pending;
			if (bIsSingleThreaded)
			{
				for (;;)
				{
					{
						FScopeLock _(&Context.GetLock());
						PackageStatus = Context.RequestCook(CookOnTheFlyServer, PackageId, Entry);
					}
					if (PackageStatus != EPackageStoreEntryStatus::Pending)
					{
						break;
					}
					Context.WaitForCook();
				}
			}
			else
			{
				FScopeLock _(&Context.GetLock());
				PackageStatus = Context.RequestCook(CookOnTheFlyServer, PackageId, Entry);
			}

			if (PackageStatus == EPackageStoreEntryStatus::Ok)
			{
				CookResponse.CookedPackages.Add(MoveTemp(Entry));
			}
			else if (PackageStatus == EPackageStoreEntryStatus::Missing)
			{
				CookResponse.FailedPackages.Add(PackageId);
			}
		}

		Response.SetBodyTo(MoveTemp(CookResponse));
		Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Ok);

		return true;
	}

	bool HandleRecookPackagesRequest(const FName& PlatformName, const UE::Cook::FCookOnTheFlyRequest& Request, UE::Cook::FCookOnTheFlyResponse& Response)
	{
		using namespace UE::ZenCookOnTheFly::Messaging;

		TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::HandleRecookPackagesRequest);

		FPlatformContext& Context = GetContext(PlatformName);
		Context.RequestCookSessionAndWait();

		FRecookPackagesRequest RecookRequest = Request.GetBodyAs<FRecookPackagesRequest>();

		UE_LOGF(LogCookOnTheFly, Display, "Received recook request for %d packages", RecookRequest.PackageIds.Num());

		{
			FScopeLock _(&PackagesToRecookCritical);
			for (FPackageId PackageId : RecookRequest.PackageIds)
			{
				PackagesToRecook.Add(PackageId);
			}
		}

		Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Ok);

		return true;
	}

	bool HandleGetAllOnDiskPackagesRequest(const FName& PlatformName, const UE::Cook::FCookOnTheFlyRequest& Request, UE::Cook::FCookOnTheFlyResponse& Response)
	{
		using namespace UE::ZenCookOnTheFly::Messaging;

		TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::HandleGetAllOnDiskPackagesRequest);

		if (PlatformName.IsNone())		// only accept this request from non-editor clients
		{
			UE_LOGF(LogCookOnTheFly, Warning, "GetAllOnDiskPackages from editor client");
			Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Error);
			return true;
		}
		else
		{
			FPlatformContext& Context = GetContext(PlatformName);
			Context.RequestCookSessionAndWait();		

			// After this point the packages have been enumerated into FPlatformContext::PackageRegistry
			check(bCookerGlobalsInitialized);		

			FGetAllOnDiskPackagesRequest AllOnDiskPackages;
			AllOnDiskPackages.PackageIds.Reserve(PackageRegistry.Num());
			{
				FScopeLock __(&Context.GetLock());
				PackageRegistry.ForEachPackage([&AllOnDiskPackages](const FPackageId& ID, const FName& Name)
					{
						AllOnDiskPackages.PackageIds.Emplace(ID);
					});
			}

			Response.SetBodyTo(MoveTemp(AllOnDiskPackages));
		}

		Response.SetStatus(UE::Cook::ECookOnTheFlyMessageStatus::Ok);

		return true;
	}

	bool BroadcastMessage(const UE::Cook::FCookOnTheFlyMessage& Message, const FName& PlatformName = NAME_None)
	{
		using namespace UE::Cook;

		UE_LOGF(LogCookOnTheFly, Verbose, "Sending: %ls, Size='%lld'", *Message.GetHeader().ToString(), Message.TotalSize());

		TArray<ICookOnTheFlyClientConnection*, TInlineAllocator<4>> ClientsToBroadcast;
		{
			FScopeLock _(&ClientsCriticalSection);

			for (ICookOnTheFlyClientConnection* Client : Clients)
			{
				if (PlatformName.IsNone() || Client->GetPlatformName() == PlatformName)
				{
					ClientsToBroadcast.Add(Client);
				}
			}
		}

		bool bBroadcasted = true;
		for (ICookOnTheFlyClientConnection* Client : ClientsToBroadcast)
		{
			if (!Client->SendMessage(Message))
			{
				UE_LOGF(LogCookOnTheFly, Warning, "Failed to send message '%ls' to client (Platform='%ls')",
					LexToString(Message.GetHeader().MessageType), *Client->GetPlatformName().ToString());

				bBroadcasted = false;
			}
		}

		return bBroadcasted;
	}

	void OnPackageStoreEntryCreated(const IPackageStoreWriter::FEntryCreatedEventArgs& EventArgs)
	{
		FPlatformContext& Context = GetContext(EventArgs.PlatformName);
		Context.RequestCookSessionAndWait();

		FScopeLock _(&Context.GetLock());
		for (const FPackageId& ImportedPackageId : EventArgs.Entry.ImportedPackageIds)
		{
			FPackageStoreEntryResource DummyEntry;
			EPackageStoreEntryStatus PackageStatus = Context.RequestCook(CookOnTheFlyServer, ImportedPackageId, DummyEntry);
		}
	}

	void OnPackageCooked(const IPackageStoreWriter::FCommitEventArgs& EventArgs)
	{
		using namespace UE::Cook;
		using namespace UE::ZenCookOnTheFly::Messaging;

		TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::OnPackageCooked);

		if (EventArgs.AdditionalFiles.Num())
		{
			TArray<FString> Filenames;
			TArray<FIoChunkId> ChunkIds;

			for (const ICookedPackageWriter::FAdditionalFileInfo& FileInfo : EventArgs.AdditionalFiles)
			{
				UE_LOGF(LogCookOnTheFly, Verbose, "Sending additional cooked file '%ls'", *FileInfo.Filename);
				Filenames.Add(FileInfo.Filename);
				ChunkIds.Add(FileInfo.ChunkId);
			}

			FCookOnTheFlyMessage Message(ECookOnTheFlyMessage::FilesAdded);
			{
				TUniquePtr<FArchive> Ar = Message.WriteBody();
				*Ar << Filenames;
				*Ar << ChunkIds;
			}

			BroadcastMessage(Message, EventArgs.PlatformName);
		}

		FCompletedPackages NewCompletedPackages;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::MarkAsCooked);

			FPlatformContext& Context = GetContext(EventArgs.PlatformName);
			FScopeLock _(&Context.GetLock());

			if (EventArgs.EntryIndex >= 0)
			{
				Context.MarkAsCooked(FPackageId::FromName(EventArgs.PackageName), EventArgs.Entries[EventArgs.EntryIndex], NewCompletedPackages);
			}
			else
			{
				Context.MarkAsFailed(FPackageId::FromName(EventArgs.PackageName), EventArgs.PackageName, NewCompletedPackages);
			}
		}
		BroadcastCompletedPackages(EventArgs.PlatformName, MoveTemp(NewCompletedPackages));
	}

	void OnPackagesMarkedUpToDate(const IPackageStoreWriter::FMarkUpToDateEventArgs& EventArgs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::OnPackagesMarkedUpToDate);

		UE::ZenCookOnTheFly::Messaging::FCompletedPackages NewCompletedPackages;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::MarkAsCooked);

			FPlatformContext& Context = GetContext(EventArgs.PlatformName);
			FScopeLock _(&Context.GetLock());

			for (int32 EntryIndex : EventArgs.PackageIndexes)
			{
				FName PackageName = EventArgs.Entries[EntryIndex].PackageName;
				Context.MarkAsCooked(FPackageId::FromName(PackageName), EventArgs.Entries[EntryIndex], NewCompletedPackages);
			}
		}
		BroadcastCompletedPackages(EventArgs.PlatformName, MoveTemp(NewCompletedPackages));
	}

	void BroadcastCompletedPackages(FName PlatformName, UE::ZenCookOnTheFly::Messaging::FCompletedPackages&& NewCompletedPackages)
	{
		using namespace UE::Cook;
		using namespace UE::ZenCookOnTheFly::Messaging;
		if (NewCompletedPackages.CookedPackages.Num() || NewCompletedPackages.FailedPackages.Num())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CookOnTheFly::SendCookedPackagesMessage);

			UE_LOGF(LogCookOnTheFly, Verbose, "Sending '%ls' message, Cooked='%d', Failed='%d'",
				LexToString(ECookOnTheFlyMessage::PackagesCooked),
				NewCompletedPackages.CookedPackages.Num(),
				NewCompletedPackages.FailedPackages.Num());

			FCookOnTheFlyMessage Message(ECookOnTheFlyMessage::PackagesCooked);
			Message.SetBodyTo<FCompletedPackages>(MoveTemp(NewCompletedPackages));
			BroadcastMessage(Message, PlatformName);
		}
	}
	
	FPlatformContext* FindContext(FName PlatformName)
	{
		FScopeLock _(&ContextsCriticalSection);
		TUniquePtr<FPlatformContext>* Ctx = PlatformContexts.Find(PlatformName);
		if (!Ctx)
		{
			return nullptr;
		}
		check(Ctx->IsValid());
		return Ctx->Get();
	}

	FPlatformContext& GetContext(FName PlatformName)
	{
		FScopeLock _(&ContextsCriticalSection);
		TUniquePtr<FPlatformContext>& Ctx = PlatformContexts.FindChecked(PlatformName);
		check(Ctx.IsValid());
		return *Ctx;
	}

	void ForEachContext(TFunctionRef<bool(FPlatformContext&)> Callback)
	{
		FScopeLock _(&ContextsCriticalSection);
		for (auto& KV : PlatformContexts)
		{
			TUniquePtr<FPlatformContext>& Ctx = KV.Value;
			check(Ctx.IsValid());
			if (!Callback(*Ctx))
			{
				return;
			}
		}
	}

	UE::Cook::ICookOnTheFlyServer& CookOnTheFlyServer;
	TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> ConnectionServer;
	FCriticalSection ClientsCriticalSection;
	TArray<UE::Cook::ICookOnTheFlyClientConnection*> Clients;
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	const FString ServiceId;
	FCriticalSection ContextsCriticalSection;
	TMap<FName, TUniquePtr<FPlatformContext>> PlatformContexts;
	FPackageRegistry PackageRegistry;
	FCriticalSection PackagesToRecookCritical;
	TSet<FPackageId> PackagesToRecook;
	bool bCookerGlobalsInitialized = false;
};

namespace UE { namespace Cook
{

TUniquePtr<ICookOnTheFlyRequestManager> MakeIoStoreCookOnTheFlyRequestManager(ICookOnTheFlyServer& CookOnTheFlyServer, const IAssetRegistry* AssetRegistry, TSharedRef<ICookOnTheFlyNetworkServer> ConnectionServer)
{
	return MakeUnique<FIoStoreCookOnTheFlyRequestManager>(CookOnTheFlyServer, AssetRegistry, ConnectionServer);
}

}}

