// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class IAssetRegistry;

namespace UE::Cook
{

class ICookOnTheFlyServer;
class ICookOnTheFlyRequestManager;
class ICookOnTheFlyNetworkServer;

TUniquePtr<ICookOnTheFlyRequestManager> MakeIoStoreCookOnTheFlyRequestManager(ICookOnTheFlyServer& CookOnTheFlyServer,
	const IAssetRegistry* AssetRegistry, TSharedRef<ICookOnTheFlyNetworkServer> ConnectionServer);

} // namespace UE::Cook
