// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadBalanceCookBurden.h"

#include "Algo/GraphConvert.h"
#include "Algo/GraphLoadBalance.h"
#include "Algo/GraphReachability.h"
#include "Algo/Reverse.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/BitArray.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookTypes.h"
#include "Misc/Optional.h"
#include "Templates/UnrealTemplate.h"

namespace UE::Cook
{

/** Implements LoadBalanceCookBurden. */
class FCookGraphLoadBalancer
{
public:
	/**
	 * Constructor stores and calculates variables shared by helper functions.
	 *
	 * @param InWorkerIds Element n is the id to store in OutAssignments[j] if package j is assigned to worker n.
	 * @param InGraph Dependency graph of packages, vertex j in this graph is a package and we write its assignment into
	 *        OutAssignments[j]. For format of the graph, @see Algo::Graph::ConvertToGraph. The vertices of the graph
	 *        must be topologically sorted from root to leaf.
	 * @param OutAssignments Out vector overwritten with OutAssignments[j] <- the worker package j is assigned to.
	 */
	FCookGraphLoadBalancer(TConstArrayView<FWorkerId> InWorkerIds, TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> InGraph,
		TArray<FWorkerId>& OutAssignments, bool bInLogResults);

	/** Run the LoadBalancing assignment and populate the OutAssignments array that was passed into the constructor. */
	void LoadBalance();

private:
	/** Tracks the packages assigned to load and save on each Worker. */
	struct FWorkerLoad
	{
		FWorkerLoad(FCookGraphLoadBalancer& InBalancer) : Balancer(InBalancer) {}

		void UpdateAllCosts(TConstArrayView<int32> VertexSharedWorkerCount);
		void UpdateTotalCost();
		void UpdateLoadBurdenFromSaveAssignment();

		TArray<Algo::Graph::FVertex> SaveAssignment;
		TArray<Algo::Graph::FVertex> LoadBurden;
		TArray<Algo::Graph::FVertex> UnassignedLoads;
		double TotalCost = 0;
		double SaveCost = 0;
		double LoadCost = 0;
		double ExpectedSaveCostDueToLoadBurden = 0;
		FCookGraphLoadBalancer& Balancer;
	};

	/** Read packagedata to calculate the savecost of the package's vertex. */
	double GetSaveCost(Algo::Graph::FVertex Vertex);
	/** Read packagedata to calculate the loadcost of the package's vertex. */
	double GetLoadCost(Algo::Graph::FVertex Vertex);
	/** Split the graph into buckets that minimize the long pole of load costs. */
	void BalanceLoadBurdens();
	/** Starting from each worker's loadcost, assign saves to reduce the total cost. */
	void BalanceSaveBurdens();
	/** Assign the vertices that are loaded by only a single worker to that worker. */
	void AssignSingleWorkerSaves(TArray<TBitArray<>>& VertexLoadedBys, TArray<int32>& VertexSharedWorkerCount);
	/** Assign vertices loaded by multiple workers to the worker that minimizes cookcost longpoleandspread. */
	void AssignSharedWorkerSaves(TArray<TBitArray<>>& VertexLoadedBys, TArray<int32>& VertexSharedWorkerCount);
	void LogResults();

	TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> Graph;
	TConstArrayView<FWorkerId> WorkerIds;
	TArray<FWorkerId>& Assignments;
	TArray64<Algo::Graph::FVertex> ReachabilityGraphBuffer;
	TArray<TConstArrayView<Algo::Graph::FVertex>> ReachabilityGraph;
	TArray<FWorkerLoad> WorkerLoads;
	int32 NumVertices = 0;
	int32 NumWorkers = 0;
	bool bLogResults = true;
};

void LoadBalanceStriped(TConstArrayView<FWorkerId> AllWorkers, TArrayView<FPackageData*> Requests,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments,
	bool bInLogResults)
{
	int32 AllWorkersIndex = 0;
	int32 NumWorkers = AllWorkers.Num();
	for (FPackageData* Request : Requests)
	{
		OutAssignments.Add(AllWorkers[AllWorkersIndex]);
		AllWorkersIndex = (AllWorkersIndex + 1) % NumWorkers;
	}
}

void LoadBalanceCookBurden(TConstArrayView<FWorkerId> AllWorkers, TArrayView<FPackageData*> Requests,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments,
	bool bInLogResults)
{
	using namespace Algo::Graph;

	if (AllWorkers.Num() <= 1 || Requests.Num() <= 2)
	{
		return LoadBalanceStriped(AllWorkers, Requests, MoveTemp(RequestGraph), OutAssignments, bInLogResults);
	}

	// Convert RequestGraph into a Normalized Graph Form which is required by our graph algorithms
	// Note this reduces the memory size from 64-bit pointer to 32-bit Algo::Graph::FVertex

	// Our input requests are guaranteed sorted in leaf to root order. FCookGraphLoadBalancer requires the vertices be
	// sorted in root to leaf order. So reverse the index when labeling vertices.
	int32 NumVertices = Requests.Num();
	TArray<FPackageData*> RootToLeafPackages;
	RootToLeafPackages.Reserve(NumVertices);
	for (int32 Index = NumVertices - 1; Index >= 0; --Index)
	{
		RootToLeafPackages.Add(Requests[Index]);
	}

	TArray64<FVertex> EdgesBuffer;
	TArray<TConstArrayView<FVertex>> Edges;

	ConvertToGraph(RootToLeafPackages, [&RequestGraph](FPackageData* Vertex)
		{
			TArray<FPackageData*>* Edges = RequestGraph.Find(Vertex);
			if (!Edges)
			{
				return TConstArrayView<FPackageData*>();
			}
			return TConstArrayView<FPackageData*>(*Edges);
		},
		EdgesBuffer, Edges, EConvertToGraphOptions::Shrink);

	// Free memory of structures we no longer need as soon as we're done with them; the LoadBalancer is memory intensive
	RequestGraph.Empty();

	FCookGraphLoadBalancer LoadBalancer(AllWorkers, Edges, OutAssignments, bInLogResults);
	LoadBalancer.LoadBalance();

	// LoadBalancer gives the assignment for vertex i in element OutAssignments[i],
	// but our caller expects OutAssignments[i] to hold the assignment of Requests[i],
	// and Requests[i] is vertex NumVertices - 1 - i. So reverse the list.
	for (int32 Index = 0; Index < NumVertices / 2; ++Index)
	{
		Swap(OutAssignments[Index], OutAssignments[NumVertices - 1 - Index]);
	}
}

FCookGraphLoadBalancer::FCookGraphLoadBalancer(TConstArrayView<FWorkerId> InWorkerIds,
	TConstArrayView<TConstArrayView<Algo::Graph::FVertex>> InGraph, TArray<FWorkerId>& OutAssignments,
	bool bInLogResults)
	: Graph(InGraph)
	, WorkerIds(InWorkerIds)
	, Assignments(OutAssignments)
	, bLogResults(bInLogResults)
{
	Algo::Graph::ConstructReachabilityGraph(Graph, ReachabilityGraphBuffer, ReachabilityGraph);

	OutAssignments.SetNumUninitialized(Graph.Num());
	for (FWorkerId& WorkerId : OutAssignments)
	{
		WorkerId = FWorkerId::Invalid();
	}

	NumVertices = Graph.Num();
	NumWorkers = WorkerIds.Num();
	check(NumVertices >= 1 && NumWorkers >= 2);

	WorkerLoads.Reserve(NumWorkers);
	for (int32 WorkerLoad = 0; WorkerLoad < NumWorkers; ++WorkerLoad)
	{
		WorkerLoads.Emplace(*this);
	}
}

void FCookGraphLoadBalancer::LoadBalance()
{
	BalanceLoadBurdens();
	BalanceSaveBurdens();
	if (bLogResults)
	{
		LogResults();
	}
}

void FCookGraphLoadBalancer::BalanceLoadBurdens()
{
	using namespace Algo::Graph;

	FLoadBalanceContext Context;
	TArray<TArray<FVertex>> LoadBurdens;
	Context.Graph = Graph;
	Context.ReachabilityGraph = ReachabilityGraph;
	Context.OutAssignments = &LoadBurdens;
	Context.OutRootAssignments = nullptr;
	Context.NumBuckets = NumWorkers;
	ConstructLoadBalance(Context);
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		FWorkerLoad& WorkerLoad = WorkerLoads[WorkerIndex];
		WorkerLoad.LoadBurden = MoveTemp(LoadBurdens[WorkerIndex]);
	}
}


void FCookGraphLoadBalancer::BalanceSaveBurdens()
{
	using namespace Algo::Graph;

	// Calculate VertexLoadedBy, VertexSharedWorkerCount, and Worker.UnassignedLoads
	TArray<TBitArray<>> VertexLoadedBys;
	TArray<int32> VertexSharedWorkerCount;
	VertexLoadedBys.SetNum(NumVertices);
	VertexSharedWorkerCount.SetNumZeroed(NumVertices);
	for (TBitArray<>& VertexLoadedBy : VertexLoadedBys)
	{
		VertexLoadedBy.Init(false, NumWorkers);
	}
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		FWorkerLoad& Worker = WorkerLoads[WorkerIndex];
		for (FVertex Vertex : Worker.LoadBurden)
		{
			VertexLoadedBys[Vertex][WorkerIndex] = true;
			++VertexSharedWorkerCount[Vertex];
		}
		Worker.UnassignedLoads.Reset();
		Worker.UnassignedLoads.Append(Worker.LoadBurden);
	}

	// Assign SingleWorkerSaves first since they can't go to another worker without increasing load burden
	AssignSingleWorkerSaves(VertexLoadedBys, VertexSharedWorkerCount);

	// Assign the SharedWorkerSaves in a way that pulls them off of the long pole
	AssignSharedWorkerSaves(VertexLoadedBys, VertexSharedWorkerCount);

	// TODO: For some graphs the long pole will have a large set of vertices that are only reached by a single root
	// and will still be longer than the other workers at this point. To reduce further we need to move the save
	// assignment of those vertices onto another worker even though that increases the load burden.
}

void FCookGraphLoadBalancer::AssignSingleWorkerSaves(TArray<TBitArray<>>& VertexLoadedBys,
	TArray<int32>& VertexSharedWorkerCount)
{
	using namespace Algo::Graph;

	// For each Vertex that is reachable from only a single Worker, assign that vertex
	// to the worker. This will not change any load burdens
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		FWorkerLoad& Worker = WorkerLoads[WorkerIndex];
		for (TArray<FVertex>::TIterator Iter(Worker.UnassignedLoads); Iter; ++Iter)
		{
			FVertex Vertex = *Iter;
			// ConstructLoadBalance guarantees that all vertices are assigned to a bucket/worker
			check(VertexSharedWorkerCount[Vertex] >= 1);
			if (VertexSharedWorkerCount[Vertex] > 1)
			{
				continue;
			}

			// It is load-assigned to one worker, which must be this one, so we should not have save-assigned it yet
			check(Assignments[Vertex].IsInvalid());
			Assignments[Vertex] = WorkerIds[WorkerIndex];
			Worker.SaveAssignment.Add(Vertex);
			Iter.RemoveCurrentSwap();
		}
	}
}

void FCookGraphLoadBalancer::AssignSharedWorkerSaves(TArray<TBitArray<>>& VertexLoadedBys,
	TArray<int32>& VertexSharedWorkerCount)
{
	using namespace Algo::Graph;

	constexpr double Epsilon = 1e-3;

	// Calculate current costs for each worker
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		FWorkerLoad& Worker = WorkerLoads[WorkerIndex];
		Worker.UpdateAllCosts(VertexSharedWorkerCount);
	}

	// While there are remaining vertices, find the lowest-cost worker that has load-burden vertices that are not yet
	// assigned to a Worker, and out of all of its unassigned load-burden vertices, find one that has the highest-cost
	// other worker and assign it to the lowest-cost worker.
	TArray<int32> WorkersHighToLow;
	WorkersHighToLow.Reserve(NumWorkers);
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers; ++WorkerIndex)
	{
		WorkersHighToLow.Add(WorkerIndex);
	}
	auto WorkerCostIsHigher = [this](int32 A, int32 B)
	{
		return WorkerLoads[A].TotalCost > WorkerLoads[B].TotalCost;
	};
	Algo::Sort(WorkersHighToLow, WorkerCostIsHigher);

	while (true)
	{
		// Find the minimum-cost worker that has unassigned loads
		int32 MinWorkerIndexInSorted = WorkersHighToLow.FindLastByPredicate([this](int32 WorkerIndexInWorkerLoads)
			{ return !WorkerLoads[WorkerIndexInWorkerLoads].UnassignedLoads.IsEmpty(); });
		if (MinWorkerIndexInSorted == INDEX_NONE)
		{
			// If there are none then all vertices are assigned and we are done with this step
			break;
		}
		int32 MinWorkerIndex = WorkersHighToLow[MinWorkerIndexInSorted];
		FWorkerLoad& MinWorker = WorkerLoads[MinWorkerIndex];

		// Find the unassigned vertex in this worker that is shared with the highest-cost other worker
		FVertex BestSharedVertex = InvalidVertex;
		int32 BestSharedVertexIndexInUnassignedLoads = INDEX_NONE;
		int32 BestSharedVertexIndexInWorkersHighToLow = NumWorkers;
		for (TArray<FVertex>::TIterator Iter(MinWorker.UnassignedLoads); Iter; ++Iter)
		{
			FVertex Vertex = *Iter;
			if (Assignments[Vertex].IsValid())
			{
				// Already assigned, remove it
				Iter.RemoveCurrentSwap();
				continue;
			}

			TBitArray<>& VertexLoadedBy = VertexLoadedBys[Vertex];
			int32 IndexInWorkersHighToLow = 0;
			for (; IndexInWorkersHighToLow < NumWorkers; ++IndexInWorkersHighToLow)
			{
				int32 WorkerIndex = WorkersHighToLow[IndexInWorkersHighToLow];
				if (VertexLoadedBy[WorkerIndex])
				{
					break;
				}
			}
			check(IndexInWorkersHighToLow < NumWorkers); // It will be loaded by at least two workers
			if (IndexInWorkersHighToLow < BestSharedVertexIndexInWorkersHighToLow)
			{
				BestSharedVertex = Vertex;
				BestSharedVertexIndexInWorkersHighToLow = IndexInWorkersHighToLow;
				BestSharedVertexIndexInUnassignedLoads = Iter.GetIndex();
				if (IndexInWorkersHighToLow == 0)
				{
					break;
				}
			}
		}
		if (BestSharedVertex == InvalidVertex)
		{
			// After removing the already-assigned vertices, this worker that we thought had unassignedloads no longer
			// has unassignedloads. Restart the loop to find the next cheapest worker with unassignedloads.
			continue;
		}

		double VertexSaveCost = GetSaveCost(BestSharedVertex);
		double ExpectedVertexSaveCostDueToLoadBurden = VertexSaveCost / VertexSharedWorkerCount[BestSharedVertex];

		// Assign the vertex to the MinWorker
		check(Assignments[BestSharedVertex].IsInvalid());
		Assignments[BestSharedVertex] = WorkerIds[MinWorkerIndex];
		MinWorker.SaveAssignment.Add(BestSharedVertex);
		MinWorker.SaveCost += VertexSaveCost;
		MinWorker.TotalCost += VertexSaveCost;

		// Remove the vertex from this worker. It will be removed from other workers when iterate their UnassignedLoads next
		MinWorker.UnassignedLoads.RemoveAtSwap(BestSharedVertexIndexInUnassignedLoads);

		// Remove the cost of the unassigned load from every vertex that included it
		for (TConstSetBitIterator<> VertexLoadedByIt(VertexLoadedBys[BestSharedVertex]); VertexLoadedByIt;
			++VertexLoadedByIt)
		{
			int32 WorkerIndex = VertexLoadedByIt.GetIndex();
			FWorkerLoad& Worker = WorkerLoads[WorkerIndex];
			Worker.ExpectedSaveCostDueToLoadBurden -= ExpectedVertexSaveCostDueToLoadBurden;
			Worker.TotalCost -= ExpectedVertexSaveCostDueToLoadBurden;
			// We're subtracting costs we added , it shouldn't drop below 0 except for numerical precision issues
			check(Worker.ExpectedSaveCostDueToLoadBurden > -Epsilon);
		}
		Algo::Sort(WorkersHighToLow, WorkerCostIsHigher);
	}

	// All vertices are now assigned for saving to a worker
	int32 NumAssignedVertices = 0;
	for (FWorkerLoad& Worker : WorkerLoads)
	{
		NumAssignedVertices += Worker.SaveAssignment.Num();
		check(Worker.UnassignedLoads.IsEmpty());
		check(Worker.ExpectedSaveCostDueToLoadBurden < Epsilon);
		Worker.ExpectedSaveCostDueToLoadBurden = 0;
		Worker.UpdateTotalCost();
	}
	check(NumAssignedVertices == NumVertices);
}

void FCookGraphLoadBalancer::LogResults()
{
	using namespace Algo::Graph;

	for (FWorkerId& WorkerId : Assignments)
	{
		checkf(WorkerId.IsValid(), TEXT("Vertex was not assigned, which should be impossible."));
	}
	for (FWorkerLoad& Worker : WorkerLoads)
	{
		Worker.UpdateLoadBurdenFromSaveAssignment();
		Worker.UpdateAllCosts(TConstArrayView<int32>());
	}
	double SingleProcessCost = 0.;
	for (FVertex Vertex = 0; Vertex < NumVertices; ++Vertex)
	{
		SingleProcessCost += GetSaveCost(Vertex) + GetLoadCost(Vertex);
	}
	int32 WorkerWideLoadCount = 0;
	int32 WorkerWideSaveCount = 0;
	UE_LOGF(LogCook, Display, "CookMultiprocess loadbalanced %d packages across %d CookWorkers.",
		NumVertices, WorkerIds.Num());
	UE_LOGF(LogCook, Display, "\t   SingleProcess: Saves=%d, Loads=%d, Cost=%.3f, FractionOfSingleProcessCost=%.3f",
		NumVertices, NumVertices, SingleProcessCost, SingleProcessCost / SingleProcessCost);
	for (int32 WorkerIndex = 0; WorkerIndex < WorkerIds.Num(); ++WorkerIndex)
	{
		FWorkerLoad& WorkerLoad = WorkerLoads[WorkerIndex];
		FWorkerId WorkerId = WorkerIds[WorkerIndex];
		int32 NumSaves = WorkerLoad.SaveAssignment.Num();
		int32 NumLoads = WorkerLoad.LoadBurden.Num();
		WorkerWideSaveCount += NumSaves;
		WorkerWideLoadCount += NumLoads;
		UE_LOGF(LogCook, Display, "\tCookWorker %5ls: Saves=%d, Loads=%d, Cost=%.3f, FractionOfSingleProcessCost=%.3f",
			WorkerId.IsLocal() ? TEXT("Local") : *WriteToString<16>(WorkerId.GetRemoteIndex()), NumSaves,
			NumLoads, WorkerLoad.TotalCost, WorkerLoad.TotalCost/SingleProcessCost);
	}
	check(WorkerWideSaveCount == NumVertices);
	UE_LOGF(LogCook, Display, "\tCookWorker duplicated load effort: %d/%d loads = %.3f",
		WorkerWideLoadCount, NumVertices, ((float)WorkerWideLoadCount) / NumVertices);
}


double FCookGraphLoadBalancer::GetSaveCost(Algo::Graph::FVertex Vertex)
{
	// TOOD: Tweak the savecost and load cost of packages once we have data on their relative expense
	// We may need to change them per package based on the class of the most important asset in the package
	return 1.;
}
double FCookGraphLoadBalancer::GetLoadCost(Algo::Graph::FVertex Vertex)
{
	return 3.;
}

void FCookGraphLoadBalancer::FWorkerLoad::UpdateTotalCost()
{
	TotalCost = LoadCost + SaveCost + ExpectedSaveCostDueToLoadBurden;
}

void FCookGraphLoadBalancer::FWorkerLoad::UpdateAllCosts(TConstArrayView<int32> VertexSharedWorkerCount)
{
	using namespace Algo::Graph;

	LoadCost = 0;
	for (FVertex Vertex : LoadBurden)
	{
		LoadCost += Balancer.GetLoadCost(Vertex);
	}

	SaveCost = 0;
	for (FVertex Vertex : SaveAssignment)
	{
		SaveCost += Balancer.GetSaveCost(Vertex);
	}

	ExpectedSaveCostDueToLoadBurden = 0;
	if (VertexSharedWorkerCount.Num() == Balancer.NumVertices)
	{
		for (FVertex Vertex : UnassignedLoads)
		{
			if (VertexSharedWorkerCount[Vertex] > 0)
			{
				ExpectedSaveCostDueToLoadBurden += Balancer.GetSaveCost(Vertex) / VertexSharedWorkerCount[Vertex];
			}
		}
	}
}

void FCookGraphLoadBalancer::FWorkerLoad::UpdateLoadBurdenFromSaveAssignment()
{
	using namespace Algo::Graph;

	LoadBurden.Empty();
	TSet<FVertex> LoadBurdenSet;
	LoadCost = 0;
	for (FVertex Vertex : SaveAssignment)
	{
		for (FVertex Reachable : Balancer.ReachabilityGraph[Vertex])
		{
			bool bExisted;
			LoadBurdenSet.Add(Reachable, &bExisted);
			if (!bExisted)
			{
				LoadCost += Balancer.GetLoadCost(Reachable);
			}
		}
	}
	LoadBurden = LoadBurdenSet.Array();
	LoadBurdenSet.Reset();
	UpdateTotalCost();
}

}