// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"

namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FWorkerId; }

namespace UE::Cook
{

/** Divides requests evenly without considering dependencies or load/save burden. */
void LoadBalanceStriped(TConstArrayView<FWorkerId> Workers, TArrayView<FPackageData*> Requests,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments, bool bInLogResults);

/** Balances "CookBurden" (considers load and save cost) across the CookWorkers. */
void LoadBalanceCookBurden(TConstArrayView<FWorkerId> Workers, TArrayView<FPackageData*> Requests,
	TMap<FPackageData*, TArray<FPackageData*>>&& RequestGraph, TArray<FWorkerId>& OutAssignments, bool bInLogResults);

}