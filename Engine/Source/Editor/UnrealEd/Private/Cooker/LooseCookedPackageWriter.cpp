// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/LooseCookedPackageWriter.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "Cooker/AsyncIODelete.h"
#include "Cooker/CompactBinaryTCP.h"
#include "Cooker/CookSandbox.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "LooseFilesCookArtifactReader.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "PackageStoreOptimizer.h"
#include "Serialization/ArchiveStackTrace.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/FilePackageWriterUtil.h"
#include "Serialization/LargeMemoryWriter.h"
#include "UObject/SavePackage.h"

LLM_DEFINE_TAG(Cooker_PackageStoreManifest);

FLooseCookedPackageWriter::FLooseCookedPackageWriter(const FString& InOutputPath,
	const FString& InMetadataDirectoryPath, const ITargetPlatform* InTargetPlatform, FAsyncIODelete& InAsyncIODelete,
	UE::Cook::FCookSandbox& InSandboxFile, TSharedRef<FLooseFilesCookArtifactReader> InCookArtifactReader)
	: CookArtifactReader(InCookArtifactReader)
	, OutputPath(InOutputPath)
	, MetadataDirectoryPath(InMetadataDirectoryPath)
	, TargetPlatform(*InTargetPlatform)
	, SandboxFile(InSandboxFile)
	, AsyncIODelete(InAsyncIODelete)
{
}

FLooseCookedPackageWriter::~FLooseCookedPackageWriter()
{
}

void FLooseCookedPackageWriter::BeginPackage(const FBeginPackageInfo& Info)
{
	Super::BeginPackage(Info);
	LLM_SCOPE_BYTAG(Cooker_PackageStoreManifest);
	FScopeLock Lock(&OplogLock);
	FOplogPackageInfo& PackageInfo = Oplog.FindOrAdd(Info.PackageName);
	PackageInfo.PackageName = Info.PackageName;
	PackageInfo.PackageDataChunks.Reset();
	PackageInfo.BulkDataChunks.Reset();
}

void FLooseCookedPackageWriter::CommitPackageInternal(FPackageWriterRecords::FPackage&& BaseRecord, const FCommitPackageInfo& Info)
{
	{
		FFilePackageWriterUtil::FRecord& Record = static_cast<FFilePackageWriterUtil::FRecord&>(BaseRecord);
		FFilePackageWriterUtil::FWritePackageParameters Parameters(Record, Info);
		Parameters.Cooker = Cooker;
		Parameters.AllPackageHashes = &AllPackageHashes;
		Parameters.PackageHashesLock = &PackageHashesLock;
		Parameters.bProvidePerPackageResult = bProvidePerPackageResults;
		FFilePackageWriterUtil::WritePackage(Parameters);
	}

	if (Info.Status != IPackageWriter::ECommitStatus::Canceled && Info.Status != IPackageWriter::ECommitStatus::NotCommitted)
	{
		FPackageWriterRecords::FPackage& Record = static_cast<FPackageWriterRecords::FPackage&>(BaseRecord);
		UpdateManifest(Record);
	}
}

void FLooseCookedPackageWriter::CompleteExportsArchiveForDiff(FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive)
{
	FPackageWriterRecords::FPackage& BaseRecord = Records.FindRecordChecked(Info.PackageName);
	FFilePackageWriterUtil::FRecord& Record = static_cast<FFilePackageWriterUtil::FRecord&>(BaseRecord);
	Record.bCompletedExportsArchiveForDiff = true;

	// Add on all the attachments which are usually added on during Commit. The order must match AsyncSave.
	for (FBulkDataRecord& BulkRecord : Record.BulkDatas)
	{
		if (BulkRecord.Info.BulkDataType == FBulkDataInfo::AppendToExports && BulkRecord.Info.MultiOutputIndex == Info.MultiOutputIndex)
		{
			ExportsArchive.Serialize(const_cast<void*>(BulkRecord.Buffer.GetData()),
				BulkRecord.Buffer.GetSize());
		}
	}
	for (FLinkerAdditionalDataRecord& AdditionalRecord : Record.LinkerAdditionalDatas)
	{
		if (AdditionalRecord.Info.MultiOutputIndex == Info.MultiOutputIndex)
		{
			ExportsArchive.Serialize(const_cast<void*>(AdditionalRecord.Buffer.GetData()),
				AdditionalRecord.Buffer.GetSize());
		}
	}

	uint32 FooterData = PACKAGE_FILE_TAG;
	ExportsArchive << FooterData;

	for (FPackageTrailerRecord& PackageTrailer : Record.PackageTrailers)
	{
		if (PackageTrailer.Info.MultiOutputIndex == Info.MultiOutputIndex)
		{
			ExportsArchive.Serialize(const_cast<void*>(PackageTrailer.Buffer.GetData()),
				PackageTrailer.Buffer.GetSize());
		}
	}
}

int64 FLooseCookedPackageWriter::GetExportsFooterSize()
{
	return sizeof(uint32);
}

FPackageWriterRecords::FPackage* FLooseCookedPackageWriter::ConstructRecord()
{
	//Watch out : if you change the type of record, you need to go through this file to make sure there is not a static cast that would not work anymore.
	return new FFilePackageWriterUtil::FRecord();
}

void FLooseCookedPackageWriter::UpdateManifest(const FPackageWriterRecords::FPackage& Record)
{
	LLM_SCOPE_BYTAG(Cooker_PackageStoreManifest);
	FScopeLock Lock(&OplogLock);
	for (const FPackageWriterRecords::FWritePackage& Package : Record.Packages)
	{
		FOplogPackageInfo* PackageInfo = Oplog.Find(Package.Info.PackageName);
		check(PackageInfo);
		FOplogChunkInfo& ChunkInfo = PackageInfo->PackageDataChunks.AddDefaulted_GetRef();
		ChunkInfo.ChunkId = Package.Info.ChunkId;
		FStringView RelativePathView;
		FPathViews::TryMakeChildPathRelativeTo(Package.Info.LooseFilePath, OutputPath, RelativePathView);
		ChunkInfo.RelativeFileName = RelativePathView;

	}
	for (const FPackageWriterRecords::FBulkData& BulkData : Record.BulkDatas)
	{
		FOplogPackageInfo* PackageInfo = Oplog.Find(BulkData.Info.PackageName);
		check(PackageInfo);
		FOplogChunkInfo& ChunkInfo = PackageInfo->BulkDataChunks.AddDefaulted_GetRef();
		ChunkInfo.ChunkId = BulkData.Info.ChunkId;
		FStringView RelativePathView;
		FPathViews::TryMakeChildPathRelativeTo(BulkData.Info.LooseFilePath, OutputPath, RelativePathView);
		ChunkInfo.RelativeFileName = RelativePathView;
	}
}

bool FLooseCookedPackageWriter::GetPreviousCookedBytes(const FPackageInfo& Info, FPreviousCookedBytesData& OutData)
{
	UE::ArchiveStackTrace::FPackageData ExistingPackageData;
	TUniquePtr<uint8, UE::ArchiveStackTrace::FDeleteByFree> Bytes;
	UE::ArchiveStackTrace::LoadPackageIntoMemory(*Info.LooseFilePath, ExistingPackageData, Bytes);
	OutData.Size = ExistingPackageData.Size;
	OutData.HeaderSize = ExistingPackageData.HeaderSize;
	OutData.StartOffset = ExistingPackageData.StartOffset;
	OutData.Data.Reset(Bytes.Release());
	return OutData.Data.IsValid();
}

FDateTime FLooseCookedPackageWriter::GetPreviousCookTime() const
{
	const FString PreviousAssetRegistry = FPaths::Combine(MetadataDirectoryPath, GetDevelopmentAssetRegistryFilename());
	return IFileManager::Get().GetTimeStamp(*PreviousAssetRegistry);
}

void FLooseCookedPackageWriter::RegisterDeterminismHelper(UObject* SourceObject,
	const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	if (Cooker)
	{
		Cooker->RegisterDeterminismHelper(this, SourceObject, DeterminismHelper);
	}
}

void FLooseCookedPackageWriter::Initialize(const FCookInfo& Info)
{
	bLegacyIterativeSharedBuild = Info.bLegacyIterativeSharedBuild;
	if (Info.bFullBuild && !Info.bWorkerOnSharedSandbox)
	{
		DeleteSandboxDirectory();
	}
	if (!Info.bWorkerOnSharedSandbox)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveScriptObjects);
		FPackageStoreOptimizer PackageStoreOptimizer;
		PackageStoreOptimizer.Initialize();
		FIoBuffer ScriptObjectsBuffer = PackageStoreOptimizer.CreateScriptObjectsBuffer();
		FFileHelper::SaveArrayToFile(
			MakeArrayView(ScriptObjectsBuffer.Data(), ScriptObjectsBuffer.DataSize()),
			*(MetadataDirectoryPath / TEXT("scriptobjects.bin")));
	}
}

EPackageWriterResult FLooseCookedPackageWriter::BeginCacheForCookedPlatformData(
	FBeginCacheForCookedPlatformDataInfo& Info)
{
	check(Cooker); // Fallback for non-cooker case not yet implemented
	return Cooker->CookerBeginCacheForCookedPlatformData(Info);
}

void FLooseCookedPackageWriter::WriteOplogEntry(FCbWriter& Writer, const FOplogPackageInfo& PackageInfo)
{
	Writer.BeginObject();

	Writer.BeginObject("packagestoreentry");
	Writer << "packagename" << PackageInfo.PackageName;
	Writer.EndObject();

	Writer.BeginArray("packagedata");
	for (const FOplogChunkInfo& ChunkInfo : PackageInfo.PackageDataChunks)
	{
		Writer.BeginObject();
		Writer << "id" << ChunkInfo.ChunkId;
		Writer << "filename" << ChunkInfo.RelativeFileName;
		Writer.EndObject();
	}
	Writer.EndArray();

	Writer.BeginArray("bulkdata");
	for (const FOplogChunkInfo& ChunkInfo : PackageInfo.BulkDataChunks)
	{
		Writer.BeginObject();
		Writer << "id" << ChunkInfo.ChunkId;
		Writer << "filename" << ChunkInfo.RelativeFileName;
		Writer.EndObject();
	}
	Writer.EndArray();

	Writer.EndObject();
}

bool FLooseCookedPackageWriter::ReadOplogEntry(FOplogPackageInfo& PackageInfo, const FCbFieldView& Field)
{
	FCbObjectView PackageStoreEntryObjectView = Field["packagestoreentry"].AsObjectView();
	if (!PackageStoreEntryObjectView)
	{
		return false;
	}
	PackageInfo.PackageName = FName(PackageStoreEntryObjectView["packagename"].AsString());

	FCbArrayView PackageDataView = Field["packagedata"].AsArrayView();
	PackageInfo.PackageDataChunks.Reset();
	PackageInfo.PackageDataChunks.Reserve(PackageDataView.Num());
	for (const FCbFieldView& ChunkEntry : PackageDataView)
	{
		FOplogChunkInfo& ChunkInfo = PackageInfo.PackageDataChunks.AddDefaulted_GetRef();
		ChunkInfo.ChunkId.Set(ChunkEntry["id"].AsObjectId().GetView());
		ChunkInfo.RelativeFileName = FString(ChunkEntry["filename"].AsString());
	}

	FCbArrayView BulkDataView = Field["bulkdata"].AsArrayView();
	PackageInfo.BulkDataChunks.Reset();
	PackageInfo.BulkDataChunks.Reserve(BulkDataView.Num());
	for (const FCbFieldView& ChunkEntry : BulkDataView)
	{
		FOplogChunkInfo& ChunkInfo = PackageInfo.BulkDataChunks.AddDefaulted_GetRef();
		ChunkInfo.ChunkId.Set(ChunkEntry["id"].AsObjectId().GetView());
		ChunkInfo.RelativeFileName = FString(ChunkEntry["filename"].AsString());
	}

	return true;
}

TMap<FName, TRefCountPtr<FPackageHashes>>& FLooseCookedPackageWriter::GetPackageHashes()
{
	return AllPackageHashes;
}

void FLooseCookedPackageWriter::BeginCook(const FCookInfo& Info)
{
	if (!Info.bWorkerOnSharedSandbox)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadPackageStoreManifest);
		FString PackageStoreManifestFilePath = FPaths::Combine(MetadataDirectoryPath, TEXT("packagestore.manifest"));
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*PackageStoreManifestFilePath));
		if (Ar)
		{
			FCbField ManifestField = LoadCompactBinary(*Ar);
			FCbField OplogField = ManifestField["oplog"];
			if (OplogField)
			{
				FCbArray EntriesArray = OplogField["entries"].AsArray();
				FScopeLock Lock(&OplogLock);
				Oplog.Reserve(EntriesArray.Num());
				for (const FCbField& EntryField : EntriesArray)
				{
					FOplogPackageInfo PackageInfo;
					ReadOplogEntry(PackageInfo, EntryField);
					Oplog.Add(PackageInfo.PackageName, MoveTemp(PackageInfo));
				}
			}
		}
	}
	else
	{
		bProvidePerPackageResults = true;
	}
	AllPackageHashes.Empty();
}

void FLooseCookedPackageWriter::EndCook(const FCookInfo& Info)
{
	if (!Info.bWorkerOnSharedSandbox)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SavePackageStoreManifest);
		FScopeLock Lock(&OplogLock);
		// Sort packages for determinism
		TArray<const FOplogPackageInfo*> SortedPackages;
		SortedPackages.Reserve(Oplog.Num());
		for (const auto& KV : Oplog)
		{
			SortedPackages.Add(&KV.Value);
		}
		Algo::Sort(SortedPackages, [](const FOplogPackageInfo* A, const FOplogPackageInfo* B)
			{
				return A->PackageName.LexicalLess(B->PackageName);
			});
		FCbWriter ManifestWriter;
		ManifestWriter.BeginObject();
		ManifestWriter.BeginObject("oplog");
		ManifestWriter.BeginArray("entries");
		for (const FOplogPackageInfo* Package : SortedPackages)
		{
			WriteOplogEntry(ManifestWriter, *Package);
		}
		ManifestWriter.EndArray();
		ManifestWriter.EndObject();
		ManifestWriter.EndObject();

		FString PackageStoreManifestFilePath = FPaths::Combine(MetadataDirectoryPath, TEXT("packagestore.manifest"));
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*PackageStoreManifestFilePath));
		if (Ar)
		{
			SaveCompactBinary(*Ar, ManifestWriter.Save());
		}
		else
		{
			UE_LOGF(LogSavePackage, Error, "Failed saving package store manifest file '%ls'", *PackageStoreManifestFilePath);
		}
	}
}

void FLooseCookedPackageWriter::PopulateOplog(const FAssetRegistryState& PreviousState, int32& OutNumPackagesInOplog)
{
	PackageNameToCookedFiles.Reset();

	// If we are legacyiterative from a shared build the cooked files do not exist in the local cooked directory;
	// we assume they are packaged in the pak file (which we don't want to extract to confirm) and keep them all.
	if (!bLegacyIterativeSharedBuild)
	{
		// Remove from disk the cooked files that no longer exist in the editor.
		GetAllCookedFiles();
		TSet<FName> RemoveFromDisk;
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		for (const TPair<FName, TArray<FString>>& Pair : PackageNameToCookedFiles)
		{
			FName PackageName = Pair.Key;

			if (!AssetRegistry.DoesPackageExistOnDisk(PackageName))
			{
				// Check that the package is not an exception before removing from cooked.
				// Generated packages do not exist uncooked.
				bool bIsCookedOnly = false;
				PreviousState.EnumerateAssetsByPackageName(PackageName, [&bIsCookedOnly](const FAssetData* AssetData)
					{
						bIsCookedOnly |= !!(AssetData->PackageFlags & PKG_CookGenerated);
						return true; // Keep iterating
					});
				if (!bIsCookedOnly)
				{
					RemoveFromDisk.Add(PackageName);
				}
			}
		}
		if (RemoveFromDisk.Num())
		{
			RemoveCookedPackagesByPackageName(RemoveFromDisk.Array(), true /* bRemoveRecords */);
		}
		OutNumPackagesInOplog = PackageNameToCookedFiles.Num();
	}
	else
	{
		// Clean the sandbox, since we are getting files from the SharedBuild location rather than from the local cook
		// directory.
		DeleteSandboxDirectory();
		// If we are legacyiterative from a shared build the cooked files do not exist in the local cooked directory;
		// we assume they are packaged in the pak file (which we don't want to extract to confirm) and keep them all.
		OutNumPackagesInOplog = PreviousState.GetNumPackages();
	}
}

void FLooseCookedPackageWriter::UpdateLastReferenceDateAndPruneStaleOps(
	UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext)
{
	// LooseCookedPackageWriter does not support setting LastReferenceDate and pruning stale ops; it only does
	// pruning based on packages that no longer exist in editor. That more limited pruning was already done in
	// PopulateOplog.
}

TArray<FName> FLooseCookedPackageWriter::GetOplogPackageNames()
{
	TArray<FName> Result;
	PackageNameToCookedFiles.GenerateKeyArray(Result);
	return Result;
}

FCbObject FLooseCookedPackageWriter::GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey)
{
	/** Oplog attachments are not implemented by FLooseCookedPackageWriter */
	return FCbObject();
}

void FLooseCookedPackageWriter::GetOplogAttachments(TArrayView<FName> PackageNames,
	TArrayView<FUtf8StringView> AttachmentKeys,
	TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback)
{
	/** Oplog attachments are not implemented by FLooseCookedPackageWriter */
	for (FName PackageName: PackageNames)
	{
		for (FUtf8StringView AttachmentKey: AttachmentKeys)
		{
			Callback(PackageName, AttachmentKey, FCbObject());
		}
	}
}

void FLooseCookedPackageWriter::GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
	TArrayView<FUtf8StringView> AttachmentKeys,
	TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback)
{
	/** Oplog attachments are not implemented by FLooseCookedPackageWriter */
	for (FName PackageName : PackageNames)
	{
		for (FUtf8StringView AttachmentKey : AttachmentKeys)
		{
			Callback(PackageName, AttachmentKey, FCbObject());
		}
	}
}

IPackageWriter::ECommitStatus FLooseCookedPackageWriter::GetCommitStatus(FName PackageName)
{
	// GetCommitStatus is not implemented by FLooseCookedPackageWriter
	return ECommitStatus::NotCommitted;
}

void FLooseCookedPackageWriter::RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove)
{
	if (PackageNameToCookedFiles.IsEmpty())
	{
		FindAndDeleteCookedFilesForPackages(PackageNamesToRemove);
		return;
	}

	// if we are going to clear the cooked packages it is conceivable that we will recook the packages which we just cooked 
	// that means it's also conceivable that we will recook the same package which currently has an outstanding async write request
	UPackage::WaitForAsyncFileWrites();
	RemoveCookedPackagesByPackageName(PackageNamesToRemove, false /* bRemoveRecords */);
	// PackageNameToCookedFiles is no longer used after the RemoveCookedPackages call at the beginning of the cook.
	PackageNameToCookedFiles.Empty();
}

void FLooseCookedPackageWriter::RemoveCookedPackagesByPackageName(TArrayView<const FName> PackageNamesToRemove, bool bRemoveRecords)
{
	auto DeletePackageLambda = [&PackageNamesToRemove, this](int32 PackageIndex)
	{
		FName PackageName = PackageNamesToRemove[PackageIndex];
		TArray<FString>* CookedFileNames = PackageNameToCookedFiles.Find(PackageName);
		if (CookedFileNames)
		{
			for (const FString& CookedFileName : *CookedFileNames)
			{
				IFileManager::Get().Delete(*CookedFileName, true, true, true);
			}
		}
	};
	ParallelFor(PackageNamesToRemove.Num(), DeletePackageLambda);

	if (bRemoveRecords)
	{
		for (FName PackageName : PackageNamesToRemove)
		{
			PackageNameToCookedFiles.Remove(PackageName);
		}
	}
}

TFuture<FCbObject> FLooseCookedPackageWriter::WriteMPCookMessageForPackage(FName PackageName)
{
	FCbFieldIterator OplogEntryField;
	{
		FOplogPackageInfo PackageInfo;
		bool bValid = false;
		{
			FScopeLock Lock(&OplogLock);
			bValid = Oplog.RemoveAndCopyValue(PackageName, PackageInfo);
		}
		if (bValid)
		{
			check(PackageName == PackageInfo.PackageName);
			FCbWriter OplogEntryWriter;
			WriteOplogEntry(OplogEntryWriter, PackageInfo);
			OplogEntryField = OplogEntryWriter.Save();
		}
	}

	TRefCountPtr<FPackageHashes> PackageHashes;
	{
		FScopeLock PackageHashesScopeLock(&PackageHashesLock);
		AllPackageHashes.RemoveAndCopyValue(PackageName, PackageHashes);
	}

	auto ComposeMessage = [OplogEntryField = MoveTemp(OplogEntryField)](FPackageHashes* PackageHashes)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		if (OplogEntryField.HasValue())
		{
			Writer << "OplogEntry" << OplogEntryField;
		}
		if (PackageHashes)
		{
			Writer << "PackageHash" << PackageHashes->PackageHash;
			Writer << "ChunkHashes" << PackageHashes->ChunkHashes;
		}
		Writer.EndObject();
		return Writer.Save().AsObject();
	};

	if (PackageHashes && PackageHashes->CompletionFuture.IsValid())
	{
		TUniquePtr<TPromise<FCbObject>> Promise(new TPromise<FCbObject>());
		TFuture<FCbObject> ResultFuture = Promise->GetFuture();
		PackageHashes->CompletionFuture.Next(
			[PackageHashes, Promise = MoveTemp(Promise), ComposeMessage = MoveTemp(ComposeMessage)]
			(int)
			{
				Promise->SetValue(ComposeMessage(PackageHashes.GetReference()));
			});
		return ResultFuture;
	}
	else
	{
		TPromise<FCbObject> Promise;
		Promise.SetValue(ComposeMessage(PackageHashes.GetReference()));
		return Promise.GetFuture();
	}
}

bool FLooseCookedPackageWriter::TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message)
{
	FOplogPackageInfo PackageInfo;
	FCbFieldView OplogEntryField(Message["OplogEntry"]);
	if (ReadOplogEntry(PackageInfo, OplogEntryField))
	{
		check(PackageName == PackageInfo.PackageName);
		FScopeLock Lock(&OplogLock);
		Oplog.Add(PackageName, MoveTemp(PackageInfo));
	}

	bool bOk = true;
	TRefCountPtr<FPackageHashes> ThisPackageHashes(new FPackageHashes());
	if (LoadFromCompactBinary(Message["PackageHash"], ThisPackageHashes->PackageHash))
	{
		bOk = LoadFromCompactBinary(Message["ChunkHashes"], ThisPackageHashes->ChunkHashes) & bOk;
		if (bOk)
		{
			bool bAlreadyExisted = false;
			{
				FScopeLock PackageHashesScopeLock(&PackageHashesLock);
				TRefCountPtr<FPackageHashes>& ExistingPackageHashes = AllPackageHashes.FindOrAdd(PackageName);
				bAlreadyExisted = ExistingPackageHashes.IsValid();
				ExistingPackageHashes = ThisPackageHashes;
			}
			if (bAlreadyExisted)
			{
				UE_LOGF(LogSavePackage, Error, "FLooseCookedPackageWriter encountered the same package twice in a cook! (%ls)",
					*PackageName.ToString());
			}
		}
	}

	return bOk;
}

void FLooseCookedPackageWriter::RemoveCookedPackages()
{
	DeleteSandboxDirectory();
}

void FLooseCookedPackageWriter::DeleteSandboxDirectory()
{
	// if we are going to clear the cooked packages it is conceivable that we will recook the packages which we just cooked 
	// that means it's also conceivable that we will recook the same package which currently has an outstanding async write request
	UPackage::WaitForAsyncFileWrites();

	FString SandboxDirectory = OutputPath;
	FPaths::NormalizeDirectoryName(SandboxDirectory);

	AsyncIODelete.DeleteDirectory(SandboxDirectory);
}

class FPackageSearchVisitor : public IPlatformFile::FDirectoryVisitor
{
	TArray<FString>& FoundFiles;
public:
	FPackageSearchVisitor(TArray<FString>& InFoundFiles)
		: FoundFiles(InFoundFiles)
	{}
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (bIsDirectory == false)
		{
			FString Filename(FilenameOrDirectory);
			FStringView Extension(FPathViews::GetExtension(Filename, true /* bIncludeDot */));
			if (!Extension.IsEmpty())
			{
				EPackageExtension ExtensionEnum = FPackagePath::ParseExtension(Extension);
				if (ExtensionEnum != EPackageExtension::Unspecified && ExtensionEnum != EPackageExtension::Custom)
				{
					FoundFiles.Add(Filename);
				}
			}
		}
		return true;
	}
};

void FLooseCookedPackageWriter::GetAllCookedFiles()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLooseCookedPackageWriter::GetAllCookedFiles);

	const FString& SandboxRootDir = OutputPath;
	TArray<FString> CookedFiles;
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FPackageSearchVisitor PackageSearch(CookedFiles);
		PlatformFile.IterateDirectoryRecursively(*SandboxRootDir, PackageSearch);
	}

	const FString SandboxProjectDir = FPaths::Combine(OutputPath, FApp::GetProjectName()) + TEXT("/");

	UE::Cook::FCookSandboxConvertCookedPathToPackageNameContext Context;
	Context.SandboxRootDir = SandboxRootDir;
	Context.SandboxProjectDir = SandboxProjectDir;
	SandboxFile.FillContext(Context);

	for (const FString& CookedFile : CookedFiles)
	{
		const FName PackageName = SandboxFile.ConvertCookedPathToPackageName(CookedFile, Context);
		if (!PackageName.IsNone())
		{
			PackageNameToCookedFiles.FindOrAdd(PackageName).Add(CookedFile);
		}
	}
}

void FLooseCookedPackageWriter::FindAndDeleteCookedFilesForPackages(TConstArrayView<FName> PackageNames)
{
	const FString& SandboxRootDir = OutputPath;
	const FString SandboxProjectDir = FPaths::Combine(OutputPath, FApp::GetProjectName()) + TEXT("/");

	UE::Cook::FCookSandboxConvertCookedPathToPackageNameContext Context;
	Context.SandboxRootDir = SandboxRootDir;
	Context.SandboxProjectDir = SandboxProjectDir;
	SandboxFile.FillContext(Context);

	for (FName PackageName : PackageNames)
	{
		FString CookedFileName = SandboxFile.ConvertPackageNameToCookedPath(WriteToString<256>(PackageName), Context);
		if (CookedFileName.IsEmpty())
		{
			continue;
		}
		FStringView ParentDir;
		FStringView BaseName;
		FStringView Extension;
		FPathViews::Split(CookedFileName, ParentDir, BaseName, Extension);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		TArray<FString, TInlineAllocator<3>> FilesToRemove;
		PlatformFile.IterateDirectory(*WriteToString<1024>(ParentDir),
			[BaseName, ParentDir, &FilesToRemove](const TCHAR* FoundFullPath, bool bDirectory)
			{
				FStringView FoundParentDir;
				FStringView FoundBaseName;
				FStringView FoundExtension;
				FPathViews::Split(FoundFullPath, FoundParentDir, FoundBaseName, FoundExtension);
				if (FoundBaseName == BaseName)
				{
					if (FoundParentDir.IsEmpty())
					{
						FilesToRemove.Add(FPaths::ConvertRelativePathToFull(FString(ParentDir), FString(FoundFullPath)));
					}
					else
					{
						FilesToRemove.Add(FString(FoundFullPath));
					}
				}
				return true;
			});
		for (const FString& FileName : FilesToRemove)
		{
			PlatformFile.DeleteFile(*FileName);
		}
	}
}

EPackageExtension FLooseCookedPackageWriter::BulkDataTypeToExtension(FBulkDataInfo::EType BulkDataType)
{
	switch (BulkDataType)
	{
	case FBulkDataInfo::AppendToExports:
		return EPackageExtension::Exports;
	case FBulkDataInfo::BulkSegment:
		return EPackageExtension::BulkDataDefault;
	case FBulkDataInfo::Mmap:
		return EPackageExtension::BulkDataMemoryMapped;
	case FBulkDataInfo::Optional:
		return EPackageExtension::BulkDataOptional;
	default:
		checkNoEntry();
		return EPackageExtension::Unspecified;
	}
}