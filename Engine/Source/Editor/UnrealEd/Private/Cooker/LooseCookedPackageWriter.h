// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoChunkId.h"
#include "Misc/PackagePath.h"
#include "Serialization/PackageWriterToSharedBuffer.h"
#include "Templates/SharedPointer.h"

class FAssetRegistryState;
class FAsyncIODelete;
class FLargeMemoryWriter;
class FLooseFilesCookArtifactReader;
class FMD5;
class ITargetPlatform;
template <typename ReferencedType> class TRefCountPtr;
namespace UE::Cook { class FCookSandbox; }
namespace UE::Cook { struct FCookSandboxConvertCookedPathToPackageNameContext; }

/** A CookedPackageWriter that saves cooked packages in separate .uasset,.uexp,.ubulk files in the Saved\Cooked\[Platform] directory. */
class FLooseCookedPackageWriter : public TPackageWriterToSharedBuffer<FBaseCookedPackageWriter>
{
public:
	using Super = TPackageWriterToSharedBuffer<FBaseCookedPackageWriter>;

	FLooseCookedPackageWriter(const FString& OutputPath, const FString& MetadataDirectoryPath,
		const ITargetPlatform* TargetPlatform, FAsyncIODelete& InAsyncIODelete,
		UE::Cook::FCookSandbox& InSandboxFile, TSharedRef<FLooseFilesCookArtifactReader> CookArtifactReader);
	~FLooseCookedPackageWriter();

	virtual void SetCooker(UE::PackageWriter::Private::ICookerInterface* CookerInterface) override
	{
		Cooker = CookerInterface;
	}

	virtual FCapabilities GetCapabilities() const override
	{
		FCapabilities Result = Super::GetCapabilities();
		Result.bDeterminismDebug = Cooker ? Cooker->IsDeterminismDebug() : false;
		return Result;
	}

	virtual FCookCapabilities GetCookCapabilities() const override
	{
		FCookCapabilities Result = Super::GetCookCapabilities();
		Result.bDiffModeSupported = true;
		return Result;
	}

	virtual void BeginPackage(const FBeginPackageInfo& Info) override;
	virtual int64 GetExportsFooterSize() override;

	virtual FDateTime GetPreviousCookTime() const override;
	virtual void RegisterDeterminismHelper(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) override;
	virtual void Initialize(const FCookInfo& Info) override;
	virtual void BeginCook(const FCookInfo& Info) override;
	virtual void EndCook(const FCookInfo& Info) override;
	virtual void PopulateOplog(const FAssetRegistryState& PreviousState, int32& OutNumOplogPackages) override;
	virtual void UpdateLastReferenceDateAndPruneStaleOps(
		UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext) override;
	virtual TArray<FName> GetOplogPackageNames() override;
	virtual FCbObject GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey) override;
	virtual void GetOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override;
	virtual void GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override;
	virtual ECommitStatus GetCommitStatus(FName PackageName) override;
	virtual void RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove) override;
	virtual void RemoveCookedPackages() override;
	virtual TFuture<FCbObject> WriteMPCookMessageForPackage(FName PackageName) override;
	virtual bool TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message) override;
	virtual bool GetPreviousCookedBytes(const FPackageInfo& Info, FPreviousCookedBytesData& OutData) override;
	virtual void CompleteExportsArchiveForDiff(FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive) override;
	virtual EPackageWriterResult BeginCacheForCookedPlatformData(FBeginCacheForCookedPlatformDataInfo& Info) override;

	virtual void CommitPackageInternal(FPackageWriterRecords::FPackage&& BaseRecord, const FCommitPackageInfo& Info) override;
	virtual FPackageWriterRecords::FPackage* ConstructRecord() override;

	static EPackageExtension BulkDataTypeToExtension(FBulkDataInfo::EType BulkDataType);

private:

	struct FOplogChunkInfo
	{
		FString RelativeFileName;
		FIoChunkId ChunkId;
	};

	struct FOplogPackageInfo
	{
		FName PackageName;
		TArray<FOplogChunkInfo, TInlineAllocator<1>> PackageDataChunks;
		TArray<FOplogChunkInfo> BulkDataChunks;
	};

	/* Delete the sandbox directory (asynchronously) in preparation for a clean cook */
	void DeleteSandboxDirectory();
	/** Searches the disk for all the cooked files in the sandbox. Stores results in PackageNameToCookedFiles. */
	void GetAllCookedFiles();
	void FindAndDeleteCookedFilesForPackages(TConstArrayView<FName> PackageNames);

	void RemoveCookedPackagesByPackageName(TArrayView<const FName> PackageNamesToRemove, bool bRemoveRecords);
	
	void UpdateManifest(const FPackageWriterRecords::FPackage& Record);
	void WriteOplogEntry(FCbWriter& Writer, const FOplogPackageInfo& PackageInfo);
	bool ReadOplogEntry(FOplogPackageInfo& PackageInfo, const FCbFieldView& Field);

	TMap<FName, TRefCountPtr<FPackageHashes>>& GetPackageHashes() override;

	TSharedRef<FLooseFilesCookArtifactReader> CookArtifactReader;

	// If EWriteOptions::ComputeHash is not set, the package will not get added to this.
	TMap<FName, TRefCountPtr<FPackageHashes>> AllPackageHashes;

	TMap<FName, TArray<FString>> PackageNameToCookedFiles;
	/** CommitPackage can be called in parallel if using recursive save, so we need a lock for shared containers used during CommitPackage */
	FCriticalSection PackageHashesLock;
	FCriticalSection OplogLock;
	FString OutputPath;
	FString MetadataDirectoryPath;
	const ITargetPlatform& TargetPlatform;
	TMap<FName, FOplogPackageInfo> Oplog;
	UE::Cook::FCookSandbox& SandboxFile;
	FAsyncIODelete& AsyncIODelete;
	UE::PackageWriter::Private::ICookerInterface* Cooker = nullptr;
	bool bLegacyIterativeSharedBuild = false;
	bool bProvidePerPackageResults = false;
};
