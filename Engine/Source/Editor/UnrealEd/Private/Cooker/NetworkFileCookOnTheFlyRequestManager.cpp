// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/NetworkFileCookOnTheFlyRequestManager.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CookOnTheFlyServerInterface.h"
#include "Cooker/CookTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "INetworkFileServer.h"
#include "INetworkFileSystemModule.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

namespace UE::Cook { struct FPackageData; }

class FNetworkFileCookOnTheFlyRequestManager final
	: public UE::Cook::ICookOnTheFlyRequestManager
{
public:
	FNetworkFileCookOnTheFlyRequestManager(UE::Cook::ICookOnTheFlyServer& InCookOnTheFlyServer, TSharedRef<UE::Cook::ICookOnTheFlyNetworkServer> NetworkServer)
		: CookOnTheFlyServer(InCookOnTheFlyServer)
	{
		FNetworkFileDelegateContainer Delegates;
		Delegates.FileRequestDelegate				= FFileRequestDelegate::CreateRaw(this, &FNetworkFileCookOnTheFlyRequestManager::OnFileRequest);
		Delegates.SandboxPathOverrideDelegate		= FSandboxPathDelegate::CreateRaw(this, &FNetworkFileCookOnTheFlyRequestManager::OnGetSandboxPath);
		Delegates.OnFileModifiedCallback			= &FileModifiedDelegate;

		NetworkFileServer.Reset(FModuleManager::LoadModuleChecked<INetworkFileSystemModule>("NetworkFileSystem")
			.CreateNetworkFileServer(NetworkServer, Delegates));
	}

	virtual ~FNetworkFileCookOnTheFlyRequestManager()
	{
		Shutdown();
	}

	virtual bool Initialize() override
	{
		return NetworkFileServer.IsValid();
	}

	virtual void Tick() override
	{
	}

	virtual void Shutdown() override
	{
		if (NetworkFileServer.IsValid())
		{
			NetworkFileServer->Shutdown();
			NetworkFileServer.Reset();
		}
	}

	virtual void OnPackageGenerated(const FName& PackageName) override
	{
	}

	virtual bool ShouldUseLegacyScheduling() override
	{
		return true;
	}

private:
	void OnFileRequest(FString& Filename, const FString& PlatformNameStr, TArray<FString>& UnsolicitedFiles)
	{
		using namespace UE::Cook;

		FName PlatformName(*PlatformNameStr);
		const bool bIsCookable = FPackageName::IsPackageExtension(*FPaths::GetExtension(Filename, true));

		if (bIsCookable)
		{
			FEvent* CookCompletedEvent = FPlatformProcess::GetSynchEventFromPool();
			FCompletionCallback CookRequestCompleted = [this, CookCompletedEvent](UE::Cook::FPackageData*)
			{
				CookCompletedEvent->Trigger();
			};

			const bool bEnqueued = CookOnTheFlyServer.EnqueueCookRequest(UE::Cook::FCookPackageRequest
			{ 
				PlatformName,
				Filename,
				MoveTemp(CookRequestCompleted)
			});
			check(bEnqueued);

			CookCompletedEvent->Wait();
			FPlatformProcess::ReturnSynchEventToPool(CookCompletedEvent);
			CookOnTheFlyServer.GetUnsolicitedFiles(PlatformName, Filename, bIsCookable, UnsolicitedFiles);
		}
		else
		{
			CookOnTheFlyServer.GetUnsolicitedFiles(PlatformName, Filename, bIsCookable, UnsolicitedFiles);
		}
	}

	FString OnGetSandboxPath()
	{
		return CookOnTheFlyServer.GetSandboxDirectory();
	}

	UE::Cook::ICookOnTheFlyServer& CookOnTheFlyServer;
	TUniquePtr<INetworkFileServer> NetworkFileServer;
	FOnFileModifiedDelegate FileModifiedDelegate;
};

namespace UE { namespace Cook
{

TUniquePtr<ICookOnTheFlyRequestManager> MakeNetworkFileCookOnTheFlyRequestManager(ICookOnTheFlyServer& CookOnTheFlyServer, TSharedRef<ICookOnTheFlyNetworkServer> NetworkServer)
{
	return MakeUnique<FNetworkFileCookOnTheFlyRequestManager>(CookOnTheFlyServer, NetworkServer);
}

}} // namespace UE::Cook
