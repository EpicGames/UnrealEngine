// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

namespace UE::Cook
{

class ICookOnTheFlyNetworkServer;
class ICookOnTheFlyRequestManager;
class ICookOnTheFlyServer;

TUniquePtr<ICookOnTheFlyRequestManager> MakeNetworkFileCookOnTheFlyRequestManager(ICookOnTheFlyServer& CookOnTheFlyServer,
	TSharedRef<ICookOnTheFlyNetworkServer> NetworkServer);

} // namespace UE::Cook
