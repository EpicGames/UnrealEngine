// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/ObjectRedirectCookDependency.h"

#include "Cooker/CookDependency.h"
#include "Cooker/CookDependencyContext.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/CoreRedirects.h"

namespace UE::Cook
{
	UE_COOK_DEPENDENCY_FUNCTION(ObjectRedirectDependency, ValidateObjectRedirectDependency);

	FCookDependency CreateObjectRedirectDependency(const FName& PackageName)
	{
		FCbWriter Writer;
		Writer.AddString(PackageName.ToString());
		
		FCbFieldIterator Fields = Writer.Save();

		return FCookDependency::Function(UE_COOK_DEPENDENCY_FUNCTION_CALL(ObjectRedirectDependency), MoveTemp(Fields));
	}

	void ValidateObjectRedirectDependency(FCbFieldViewIterator Args, FCookDependencyContext& Context)
	{
		FUtf8StringView PackageNameAsString = Args.AsString();
		FName PackageName(PackageNameAsString);

		FBlake3 Hasher;
		FCoreRedirects::GetObjectRedirectHashAffectingPackage(PackageName, Hasher);

		FBlake3Hash Hash = Hasher.Finalize();
		Context.Update(&Hash, sizeof(Hash));
	}
}