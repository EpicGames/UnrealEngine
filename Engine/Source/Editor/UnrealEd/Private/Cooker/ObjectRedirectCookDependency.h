// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/CompactBinary.h"
#include "UObject/NameTypes.h"

namespace UE::Cook { class FCookDependency; }
namespace UE::Cook { struct FCookDependencyContext; }

namespace UE::Cook
{
	/** Create a function dependency tracking the object redirect for a given package. */
	FCookDependency CreateObjectRedirectDependency(const FName& PackageName);

	/** Deserialize the function dependency parameters and recompute the hash value. */
	void ValidateObjectRedirectDependency(FCbFieldViewIterator Args, FCookDependencyContext& Context);
}
