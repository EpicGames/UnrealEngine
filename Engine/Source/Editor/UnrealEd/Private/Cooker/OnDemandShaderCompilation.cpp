// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/OnDemandShaderCompilation.h"

#include "Async/TaskGraphInterfaces.h"
#include "Async/Async.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "MaterialShared.h"
#include "UObject/CoreRedirects.h"
#include "PackageTools.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"


int32 GODSCShaderMapsLifetime = 25;
static FAutoConsoleVariableRef CVarODSCShaderMapsLifetime(
	TEXT("odsc.shadermaps.lifetime"),
	GODSCShaderMapsLifetime,
	TEXT("Controls how many shader recompiles can happen before deleting an unused shadermap. Higher values means more memory, but faster iteration time\n")
	TEXT("-1 means we never delete shader maps\n"),
	ECVF_Default);

int32 GODSCNumShaderMapsBeforeGC = 5000;
static FAutoConsoleVariableRef CVarODSCNumShaderMapsBeforeGC(
	TEXT("odsc.shadermaps.numbeforegc"),
	GODSCNumShaderMapsBeforeGC,
	TEXT("Controls how many shader maps we keep in memory before we start deleting them. Higher values means more memory, but faster iteration time\n")
	TEXT("-1 means we never delete shader maps\n"),
	ECVF_Default);

FString GODSCExludedClasses = "/Script/Engine.GameMode";
static FAutoConsoleVariableRef CVarODSCExludedClasses(
	TEXT("odsc.excludedclasses"),
	GODSCExludedClasses,
	TEXT("Controls what packages will be ignored during material loading to speed it up. This will exclude uassets inheriting from these classes\n")
	TEXT("This list can contain multiple classes, separated by '|'\n"),
	ECVF_Default);


namespace UE::Cook
{

class FODSCClientDataAccess
{
public:
	static FODSCClientData::FWorldPartitionAssets* TryFindInContentBundle(FSoftObjectPath& AssetSoftPath);
};

void FODSCClientData::OnClientConnected(const void* ConnectionPtr)
{
}

void FODSCClientData::OnClientDisconnected(const void* ConnectionPtr)
{
}

void FODSCClientData::PurgeMaterialShaderMaps(int32 Lifetime, int32 NumMapsToDelete, FODSCClientPersistentData::Value& MaterialShaderMapsKeptAlive)
{
	// Don't start counting shader lifetime until we go over the limit of shadermaps we want to keep in memory
	if (NumMapsToDelete <= 0)
	{
		return;
	}

	for (FODSCClientPersistentData::Value::TIterator Iter = MaterialShaderMapsKeptAlive.CreateIterator(); Iter; ++Iter)
	{
		int32& ShaderMapLifetime = Iter.Value();

		++ShaderMapLifetime;
		FMaterialShaderMap* MaterialShaderMap = Iter.Key();
		if ((Lifetime >= 0) && (ShaderMapLifetime > Lifetime) && (NumMapsToDelete > 0))
		{
			MaterialShaderMap->RemoveCompilingMaterialExternalDependency();
			Iter.RemoveCurrent();
			--NumMapsToDelete;
		}
	}
}

void FODSCClientData::FlushClientPersistentData(const void* ConnectionPtr)
{
	FScopeLock PollablesScopeLock(&ODSCClientPersistentDataLock);
	int32 NumMapsToDelete = (GODSCNumShaderMapsBeforeGC >= 0) ? ODSCClientPersistentData.MaterialShaderMapsKeptAlive.Num() - GODSCNumShaderMapsBeforeGC : 0;
	PurgeMaterialShaderMaps(GODSCShaderMapsLifetime, NumMapsToDelete, ODSCClientPersistentData.MaterialShaderMapsKeptAlive);
}

void FODSCClientData::KeepClientPersistentData(const void* ConnectionPtr, const TArray<TStrongObjectPtr<UMaterialInterface>>& LoadedMaterialsToRecompile)
{
	FScopeLock PollablesScopeLock(&ODSCClientPersistentDataLock);
	for (const TStrongObjectPtr<UMaterialInterface>& MaterialInterface : LoadedMaterialsToRecompile)
	{
		for (int32 QualityLevelIndex = 0; QualityLevelIndex <= EMaterialQualityLevel::Num; ++QualityLevelIndex)
		{
			for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < EShaderPlatform::SP_NumPlatforms; ++ShaderPlatformIndex)
			{
				const FMaterialResource* MaterialResource = MaterialInterface->GetMaterialResource((EShaderPlatform)ShaderPlatformIndex, (EMaterialQualityLevel::Type)QualityLevelIndex);
				if (!MaterialResource)
				{
					continue;
				}
				FMaterialShaderMap* CompilingShaderMap = FMaterialShaderMap::FindCompilingShaderMap(MaterialResource->GetGameThreadCompilingShaderMapId());
				if (!CompilingShaderMap)
				{
					continue;
				}

				int32& Lifetime = ODSCClientPersistentData.MaterialShaderMapsKeptAlive.FindOrAdd(CompilingShaderMap, -1);
				// On first insertion, we add the external dependency, and set to 0 such that the call to PurgeMaterialShaderMaps will always work with positive value
				if (Lifetime == -1)
				{
					CompilingShaderMap->AddCompilingMaterialExternalDependency();
				}
				Lifetime = 0;
			}
		}

		CleanupWorldPartitionAssets();
	}
}

TMap<FString, FODSCClientData::FWorldPartitionAssets> FODSCClientData::WorldPartitionAssets;
TSet<FString> FODSCClientData::ScannedWorldPartitionPaths;
TSet<FName> FODSCClientData::ExcludedPackageNames;

void FODSCClientData::CleanupWorldPartitionAssets()
{
	TSet<UPackage*> PackagesToUnloadSet;
	for (auto Iter = WorldPartitionAssets.CreateIterator(); Iter; ++Iter)
	{
		FWorldPartitionAssets& DynamicMaterialData = Iter.Value();
		if (DynamicMaterialData.PackagePtr.IsValid())
		{
			PackagesToUnloadSet.FindOrAdd(DynamicMaterialData.PackagePtr.Get());
			DynamicMaterialData.PackagePtr = nullptr;
		}
	}

	TArray<UPackage*> PackagesToUnloadArray;
	for (UPackage* Package : PackagesToUnloadSet)
	{
		PackagesToUnloadArray.Add(Package);
	}

	FText OutErrorMessage;
	// bUnloadDirtyPackages=true because some systems (UPCGGraphInstance::RefreshParameters for example) mark the package dirty 
	// and prevent the unloading from happening
	UPackageTools::UnloadPackages(PackagesToUnloadArray, OutErrorMessage, true /*bUnloadDirtyPackages*/);

	if (!OutErrorMessage.IsEmpty())
	{
		UE_LOGF(LogShaders, Error, "UPackageTools::UnloadPackages: %ls", *OutErrorMessage.ToString());
	}
}

bool ExtractMaterialPath(FSoftObjectPath& MaterialPath, FSoftObjectPath& ActorPath, const FString& MaterialKey)
{
	FString MaterialPathString;
	FString ActorPathString;
	int32 ActorSeparatorIndex = MaterialKey.Find(":::");
	if (ActorSeparatorIndex != INDEX_NONE)
	{
		MaterialPathString = MaterialKey.Left(ActorSeparatorIndex);
		ActorPathString = MaterialKey.Right(MaterialKey.Len() - ActorSeparatorIndex - 3);
	}
	else
	{
		MaterialPathString = MaterialKey;
	}

	bool bValidActorPath = false;
	if (!ActorPathString.IsEmpty())
	{
		if (!FWorldPartitionHelpers::ConvertRuntimePathToEditorPath(ActorPathString, ActorPath))
		{
			ActorPathString.ReplaceInline(TEXT("/_Generated_/"), TEXT("/"));
			ActorPath = ActorPathString;
		}
		else
		{
			bValidActorPath = true;
		}
	}

	if (!FWorldPartitionHelpers::ConvertRuntimePathToEditorPath(MaterialPathString, MaterialPath))
	{
		MaterialPathString.ReplaceInline(TEXT("/_Generated_/"), TEXT("/"));
		MaterialPath = MaterialPathString;
		if (!bValidActorPath)
		{
			return false;
		}
	}
	return true;
}

UMaterialInterface* FODSCClientData::FindMaterial(const FString& InMaterialKey)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FODSCClientData_FindMaterial %s"), *InMaterialKey));

	UE_CALL_ONCE([](){ SetupClassExclusionList(); });

	// Setup some packages exclusion as loading some packages may take a long time and are not necessary in the context of ODSC
	TArray<FCoreRedirect> CoreRedirectExcludedPackages; 
	FName InvalidPath(TEXT("/ODSC/Invalid/Path"));
	{
		CoreRedirectExcludedPackages.Reserve(ExcludedPackageNames.Num());
		for (FName ExcludedPackage : ExcludedPackageNames)
		{
			CoreRedirectExcludedPackages.Add(FCoreRedirect(ECoreRedirectFlags::Type_Package, ExcludedPackage.ToString(), InvalidPath.ToString()));
		}
	}

	FCoreRedirects::AddRedirectList(CoreRedirectExcludedPackages, InvalidPath.ToString());

	ON_SCOPE_EXIT
	{
		FCoreRedirects::RemoveRedirectList(CoreRedirectExcludedPackages, InvalidPath.ToString());
	};

	FSoftObjectPath MaterialPath;
	FSoftObjectPath ActorPath;
	bool bIsWorldPartitionPath = ExtractMaterialPath(MaterialPath, ActorPath, InMaterialKey);

	UMaterialInterface* MaterialInterface = nullptr;

	if (bIsWorldPartitionPath)
	{
		MaterialInterface = TryFindWorldPartitionMaterial(MaterialPath, ActorPath);
		if (MaterialInterface)
		{
			return MaterialInterface;
		}
	}

	MaterialInterface = FindObject<UMaterialInterface>(nullptr, *MaterialPath.ToString());
	if (MaterialInterface)
	{
		return MaterialInterface;
	}

	MaterialInterface = LoadObject<UMaterialInterface>(nullptr, *MaterialPath.ToString());	

	return MaterialInterface;
}

FSoftObjectPath GetWorldPartitionActorPath(const FSoftObjectPath& InMaterialSoftPath)
{
	FString PathToProxy = InMaterialSoftPath.ToString();
	// Remove the landscape prefix since Landscape MIC are embedded in the package of their proxy
	FString LandscapeMaterialInstanceConstantClassName = ULandscapeMaterialInstanceConstant::StaticClass()->GetFName().ToString();
	int32 SubObjectNameChopIndex = PathToProxy.Find(FString(".") + LandscapeMaterialInstanceConstantClassName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (SubObjectNameChopIndex != INDEX_NONE)
	{
		PathToProxy.LeftChopInline(PathToProxy.Len() - SubObjectNameChopIndex);
		return FSoftObjectPath(PathToProxy);
	}

	return FSoftObjectPath();
}

FODSCClientData::FWorldPartitionAssets* FODSCClientDataAccess::TryFindInContentBundle(FSoftObjectPath& AssetSoftPath)
{
	TStringBuilder<256> ActualMountPointPackageName;
	TStringBuilder<256> ActualMountPointFilePath;
	TStringBuilder<256> ActualRelPath;

	FPackageName::TryGetMountPointForPath(AssetSoftPath.ToString(), ActualMountPointPackageName, ActualMountPointFilePath, ActualRelPath);

	FString ActualRelPathString(ActualRelPath);
	
	FODSCClientData::FWorldPartitionAssets* DynamicMaterialData = nullptr;
	// WP actors with a path like /MyMountPoint/CB/ have their paths actually remapped to /Game/
	if (ActualRelPathString.StartsWith(TEXT("CB/")))
	{
		FString ActualRelPathStringCopy(TEXT("/Game"));
		ActualRelPathStringCopy += ActualRelPathString.Right(ActualRelPathString.Len() - 2);
		DynamicMaterialData = FODSCClientData::WorldPartitionAssets.Find(ActualRelPathStringCopy);
		if (DynamicMaterialData)
		{
			AssetSoftPath.SetPath(ActualRelPathStringCopy);
		}
	}

	return DynamicMaterialData;
}


UMaterialInterface* FODSCClientData::TryFindWorldPartitionMaterial(const FSoftObjectPath& InMaterialSoftPath, const FSoftObjectPath& InActorSoftPath)
{
	FSoftObjectPath ActorSoftPath(InActorSoftPath);
	FSoftObjectPath MaterialSoftPath(InMaterialSoftPath);

	ScanWorldPartitionAssets(MaterialSoftPath.GetAssetPath().GetPackageName().ToString());
	ScanWorldPartitionAssets(ActorSoftPath.GetAssetPath().GetPackageName().ToString());

	FWorldPartitionAssets* DynamicMaterialData = WorldPartitionAssets.Find(*MaterialSoftPath.ToString());
	// Landscape sometimes issues requests without actors. Try to reconstruct the path 
	if (DynamicMaterialData == nullptr && !ActorSoftPath.IsValid())
	{
		ActorSoftPath = GetWorldPartitionActorPath(MaterialSoftPath);
	}

	if (DynamicMaterialData == nullptr && ActorSoftPath.IsValid())
	{
		DynamicMaterialData = WorldPartitionAssets.Find(ActorSoftPath.ToString());
	}

	if (DynamicMaterialData == nullptr && MaterialSoftPath.IsValid())
	{
		DynamicMaterialData = FODSCClientDataAccess::TryFindInContentBundle(MaterialSoftPath);
	}

	if (DynamicMaterialData == nullptr && ActorSoftPath.IsValid())
	{
		DynamicMaterialData = FODSCClientDataAccess::TryFindInContentBundle(ActorSoftPath);
	}

	if (DynamicMaterialData == nullptr)
	{
		return nullptr;
	}

	if (!DynamicMaterialData->PackagePtr)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FODSCClientData_LoadPackage %s"), *DynamicMaterialData->PackageName));
		DynamicMaterialData->PackagePtr = TStrongObjectPtr<UPackage>(LoadPackage(nullptr, *DynamicMaterialData->PackageName, LOAD_None));
	}

	TArray<FString> MaterialPathsToTry;
	MaterialPathsToTry.Add(MaterialSoftPath.ToString());

	{
		// When the provided material path doesn't work, try to replace the base path by the package's
		FTopLevelAssetPath MaterialTopPath;
		MaterialTopPath.TrySetPath(FName(DynamicMaterialData->PackageName), MaterialSoftPath.GetAssetPath().GetAssetName());
		FSoftObjectPath MaterialSoftPathCopy(MaterialTopPath, MaterialSoftPath.GetSubPathString());
		MaterialPathsToTry.Add(MaterialSoftPathCopy.ToString());
	}

	UMaterialInterface* MaterialInterface = nullptr;
	for (const FString& MaterialPathToTry : MaterialPathsToTry)
	{
		MaterialInterface = FindObject<UMaterialInterface>(nullptr, *MaterialPathToTry);
		if (MaterialInterface)
		{
			return MaterialInterface;
		}
	}

	for (const FString& MaterialPathToTry : MaterialPathsToTry)
	{
		MaterialInterface = LoadObject<UMaterialInterface>(nullptr, *MaterialPathToTry);
		if (MaterialInterface)
		{
			return MaterialInterface;
		}
	}

	return MaterialInterface;

}

void FODSCClientData::ScanWorldPartitionAssets(const FString& InAssetPath)
{
	if (InAssetPath.IsEmpty())
	{
		return;
	}

	bool bAlreadySeenPath;
	ScannedWorldPartitionPaths.FindOrAdd(InAssetPath, &bAlreadySeenPath);
	if (bAlreadySeenPath)
	{
		return;
	}

	FString AssetPath = InAssetPath;

	TArray<FString> PathsToScan;
	PathsToScan.Add(AssetPath);
	PathsToScan.Append(ULevel::GetExternalObjectsPaths(AssetPath));

	{
		TStringBuilder<256> ActualMountPointPackageName;
		TStringBuilder<256> ActualMountPointFilePath;
		TStringBuilder<256> ActualRelPath;

		FPackageName::TryGetMountPointForPath(InAssetPath, ActualMountPointPackageName, ActualMountPointFilePath, ActualRelPath);

		// If we have /MyOtherMountPoint/CB/ as a base path, try scanning the external folders' content bundle as well
		FString ActualMountPointPackageNameString(ActualMountPointPackageName);
		if (!ActualMountPointPackageNameString.StartsWith(TEXT("/Game/")))
		{
			FString ActualRelPathStr(ActualRelPath);
			if (ActualRelPathStr.StartsWith(TEXT("CB/")))
			{
				PathsToScan.Add(ActualMountPointPackageNameString + FPackagePath::GetExternalActorsFolderName() + TEXT("/ContentBundle/"));
				PathsToScan.Add(ActualMountPointPackageNameString + FPackagePath::GetExternalObjectsFolderName() + TEXT("/ContentBundle/"));
			}
		}
	}


	// Do a synchronous scan of the level external actors path.					
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FODSCClientData_ScanSynchronous);
		AssetRegistry.ScanSynchronous(PathsToScan, TArray<FString>(), UE::AssetRegistry::EScanFlags::IgnoreInvalidPathWarning);
	}

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.bIncludeOnlyOnDiskAssets = true;
	for (FString& PackagePath : PathsToScan)
	{
		Filter.PackagePaths.Add(*PackagePath);
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(GetAssets);
	TArray<FAssetData> Assets;
	AssetRegistry.GetAssets(Filter, Assets);

	for (FAssetData& AssetData : Assets)
	{
		FWorldPartitionAssets DynamicMaterialData{};
		DynamicMaterialData.PackageName = AssetData.PackageName.ToString();
		WorldPartitionAssets.Add(AssetData.GetSoftObjectPath().ToString(), DynamicMaterialData);

        // Uncomment to see all the WP actors collected
//		UE_LOGF(LogShaders, Display, "ScanAssets: %ls->%ls", *AssetData.GetSoftObjectPath().ToString(), *DynamicMaterialData.PackageName);

	}
}

static bool DetectIsUAssetByNames(FStringView PackageName, FStringView ObjectPathName)
{
	FStringView PackageBaseName;
	{
		// Get everything after the last slash
		int32 IndexOfLastSlash = INDEX_NONE;
		PackageName.FindLastChar(TEXT('/'), IndexOfLastSlash);
		PackageBaseName = PackageName.Mid(IndexOfLastSlash + 1);
	}

	return PackageBaseName.Equals(ObjectPathName, ESearchCase::IgnoreCase);
}

static void AddUClassPackagesToExclusionList(TSet<FName>& ExcludedPackageNames, const FString& ClassName)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(FTopLevelAssetPath(ClassName));
	TSet<FTopLevelAssetPath> DerivedClassNames;
	AssetRegistry.GetDerivedClassNames(Filter.ClassPaths, {}, DerivedClassNames);

	// Still keep script packages
	for (FTopLevelAssetPath& DerivedClassName : DerivedClassNames)
	{
		FString ObjectName = DerivedClassName.GetAssetName().ToString();
		// Remove suffix for compiled blueprint classes
		ObjectName.RemoveFromEnd(TEXT("_C"));
		
		bool bIsUAsset = DetectIsUAssetByNames(DerivedClassName.GetPackageName().ToString(), ObjectName);
		if (!bIsUAsset)
		{
			continue;
		}

		UE_LOGF(LogShaders, Display, "FODSCClientData excluding package '%ls' because '%ls' is a uasset inheriting from %ls", 
			   *DerivedClassName.GetPackageName().ToString(), *DerivedClassName.ToString(), *ClassName);

		ExcludedPackageNames.Add(FName(DerivedClassName.GetPackageName().ToString()));
	}
}

void FODSCClientData::SetupClassExclusionList()
{
	TArray<FString> ClassNames;
	
	GODSCExludedClasses.ParseIntoArray(ClassNames, TEXT("|"));
	for (FString& ClassName : ClassNames)
	{
		AddUClassPackagesToExclusionList(ExcludedPackageNames, ClassName);
	}
}

}
