// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"
#include "Materials/MaterialInterface.h"

class FMaterialShaderMap;
class UMaterialInterface;

namespace UE::Cook
{

class FODSCClientData
{
public:
	void OnClientConnected(const void* ConnectionPtr);
	void OnClientDisconnected(const void* ConnectionPtr); 
	void KeepClientPersistentData(const void* ConnectionPtr, const TArray<TStrongObjectPtr<UMaterialInterface>>& LoadedMaterialsToRecompile);
	void FlushClientPersistentData(const void* ConnectionPtr);

	static UMaterialInterface* FindMaterial(const FString& MaterialKey);

private:
	
	friend class FODSCClientDataAccess;
	static UMaterialInterface* TryFindWorldPartitionMaterial(const FSoftObjectPath& MaterialSoftPath, const FSoftObjectPath& ActorSoftPath);

	static void ScanWorldPartitionAssets(const FString& AssetPath);
	static void SetupClassExclusionList();

	static void CleanupWorldPartitionAssets();
	struct FWorldPartitionAssets
	{
		FString PackageName;
		TStrongObjectPtr<UPackage> PackagePtr;
	};

	static TMap<FString, FWorldPartitionAssets> WorldPartitionAssets;
	static TSet<FString> ScannedWorldPartitionPaths;
	static TSet<FName> ExcludedPackageNames;

	struct FODSCClientPersistentData
	{
		typedef TMap<TRefCountPtr<FMaterialShaderMap>, int32> Value;
		Value MaterialShaderMapsKeptAlive;
	};

	static void PurgeMaterialShaderMaps(int32 Lifetime, int32 NumMapsToDelete, FODSCClientPersistentData::Value& MaterialShaderMapsKeptAlive);

	FODSCClientPersistentData ODSCClientPersistentData;
	FCriticalSection ODSCClientPersistentDataLock;
};

}