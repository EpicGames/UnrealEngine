// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageBuildDependencyTracker.h"
#include "Misc/ConfigAccessTracking.h"

#if UE_WITH_PACKAGE_ACCESS_TRACKING
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "Async/UniqueLock.h"
#include "Cooker/CookConfigAccessTracker.h"
#include "Cooker/CookDependency.h"
#include "CookOnTheSide/CookLog.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#endif

#if UE_WITH_PACKAGE_ACCESS_TRACKING

DEFINE_LOG_CATEGORY_STATIC(LogPackageBuildDependencyTracker, Log, All);

FPackageBuildDependencyTracker FPackageBuildDependencyTracker::Singleton;

void FPackageBuildDependencyTracker::Disable()
{
	if (bEnabled)
	{
		UE::CoreUObject::RemoveObjectHandleReadCallback(ObjectHandleReadHandle);
		ObjectHandleReadHandle = UE::CoreUObject::FObjectHandleTrackingCallbackId {};
		FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

		bEnabled = false;
	}
}

bool FPackageBuildDependencyTracker::IsEnabled() const
{
	return bEnabled;
}

void FPackageBuildDependencyTracker::DumpStats() const
{
	if (!IsEnabled())
	{
		return;
	}

	UE::TUniqueLock RecordsScopeLock(RecordsLock);
	uint64 ReferencingPackageCount = 0;
	uint64 ReferenceCount = 0;
	for (const TPair<FName, TMap<FBuildDependencyAccessData, FResultProjectionList>>& PackageAccessRecord : Records)
	{
		++ReferencingPackageCount;
		for (const TPair<FBuildDependencyAccessData, FResultProjectionList>& AccessedData : PackageAccessRecord.Value)
		{
			++ReferenceCount;
		}
	}
	UE_LOGF(LogPackageBuildDependencyTracker, Display, "Package Accesses (%llu referencing packages with a total of %llu unique accesses)", ReferencingPackageCount, ReferenceCount);

	constexpr bool bDetailedDump = false;
	if (bDetailedDump)
	{
		UE_LOGF(LogPackageBuildDependencyTracker, Display, "=========================================================================");
		for (const TPair<FName, TMap<FBuildDependencyAccessData, FResultProjectionList>>& PackageAccessRecord : Records)
		{
			UE_LOGF(LogPackageBuildDependencyTracker, Display, "%ls:", *PackageAccessRecord.Key.ToString());
			for (const TPair<FBuildDependencyAccessData, FResultProjectionList>& AccessedData : PackageAccessRecord.Value)
			{
				UE_LOGF(LogPackageBuildDependencyTracker, Display, "    %ls", *AccessedData.Key.ReferencedPackage.ToString());
			}
		}
	}
}

TArray<TPair<FBuildDependencyAccessData, FResultProjectionList>> FPackageBuildDependencyTracker::GetAccessDatas(FName ReferencerPackage) const
{
	UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
	const TMap<FBuildDependencyAccessData, FResultProjectionList>* ReferencerMap = Records.Find(ReferencerPackage);
	if (!ReferencerMap)
	{
		return TArray<TPair<FBuildDependencyAccessData, FResultProjectionList>>();
	}
	return ReferencerMap->Array();
}

FPackageBuildDependencyTracker::FPackageBuildDependencyTracker()
{
	ObjectHandleReadHandle = UE::CoreUObject::AddObjectHandleReadCallback(StaticOnObjectHandleRead);
	FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FPackageBuildDependencyTracker::PostGarbageCollect);

	bEnabled = true;
}

FPackageBuildDependencyTracker::~FPackageBuildDependencyTracker()
{
	Disable();
}

static bool ShouldSkipDependency(const UObject* Object)
{
	if (!Object)
	{
		return true;
	}
	if (!GUObjectArray.IsValidIndex(Object))
	{
		return true;
	}
	if (!Object->HasAnyFlags(RF_Public))
	{
		return true;
	}
	if (Object->GetClass() == UClass::StaticClass())
	{
		return true;
	}
	// Optimization to skip GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn) for other native types like enums and script structs.
	// CDOs and their sub objects will still be handled in the slower path.
	if (Object->IsNative()) 
	{
		return true;
	}
	return false;
}

static FName NAME_EngineTransient(TEXT("/Engine/Transient"));

void FPackageBuildDependencyTracker::StaticOnObjectHandleRead(const TArrayView<const UObject* const>& Objects)
{
	const int32 Count = Objects.Num();
	if (Count == 0 || (Count == 1 && ShouldSkipDependency(Objects[0])))
	{
		return;
	}

	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (!AccumulatedScopeData)
	{
		return;
	}

	FName Referencer = AccumulatedScopeData->PackageName;
	FName CookResultProjection = AccumulatedScopeData->CookResultProjection;
	if (AccumulatedScopeData->BuildOpName.IsNone()
		| CookResultProjection == UE::Cook::ResultProjection::None // -V792
		| Referencer.IsNone() // -V792
		| Referencer == NAME_EngineTransient) // -V792
	{
		return;
	}

	for (const UObject* ReadObject : Objects)
	{
		// No need to re-evaluate ShouldSkipDependency when Count == 1
		if (Count > 1 && ShouldSkipDependency(ReadObject))
		{
			continue;
		}

		UPackage* ReferencedPackage = ReadObject->GetOutermost();
		FName Referenced = ReferencedPackage->GetFName();
		if ((Referencer == Referenced) || ReferencedPackage->HasAnyPackageFlags(PKG_CompiledIn)
			|| Referenced == NAME_EngineTransient)
		{
			continue;
		}

		if (AccumulatedScopeData->OpName == PackageAccessTrackingOps::NAME_NoAccessExpected)
		{
			UE_LOGF(LogPackageBuildDependencyTracker, Warning, "Object %ls is referencing object %ls inside of a NAME_NoAccessExpected scope. Programmer should narrow the scope or debug the reference.",
				*Referencer.ToString(), *Referenced.ToString());
		}

		LLM_SCOPE_BYNAME("PackageBuildDependencyTracker");

		FBuildDependencyAccessData AccessData{ Referenced, AccumulatedScopeData->TargetPlatform };
		UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);

		bool bNeedsAdd = true;
		if (Referencer == Singleton.LastReferencer)
		{
			if (AccessData == Singleton.LastAccessData)
			{
				if (CookResultProjection == UE::Cook::ResultProjection::All
					&& Singleton.LastCookResultProjection == UE::Cook::ResultProjection::All)
				{
					bNeedsAdd = false;
				}
				else
				{
					Singleton.LastCookResultProjection = CookResultProjection;
				}
			}
			else
			{
				Singleton.LastCookResultProjection = CookResultProjection;
				Singleton.LastAccessData = AccessData;
			}
		}
		else
		{
			Singleton.LastCookResultProjection = CookResultProjection;
			Singleton.LastAccessData = AccessData;
			Singleton.LastReferencer = Referencer;
			Singleton.LastReferencerMap = &Singleton.Records.FindOrAdd(Referencer);
		}
		if (bNeedsAdd)
		{
			if (CookResultProjection == UE::Cook::ResultProjection::All)
			{
				Singleton.LastReferencerMap->FindOrAdd(AccessData).AddProjectionAll();
			}
			else
			{
				Singleton.LastReferencerMap->FindOrAdd(AccessData).AddProjection(CookResultProjection, ReadObject->GetClass()->GetClassPathName());
			}

		}
	}
}

void FPackageBuildDependencyTracker::PostGarbageCollect()
{
	// We expect not to be asked for the records for any package no longer in memory;
	// Only package saves trigger a request for records, and they only request records for themselves
	// and for their build dependencies, and they and their build dependencies should be in memory when
	// they are saved.
	// Drop these no longer needed records to save memory.
	// We will recreate them if the package loads again.
	UE::TUniqueLock RecordsScopeLock(Singleton.RecordsLock);
	for (TMap<FName, TMap<FBuildDependencyAccessData, FResultProjectionList>>::TIterator Iter(Records); Iter; ++Iter)
	{
		FName PackageName = Iter.Key();
		if (!FindPackage(nullptr, *WriteToString<256>(PackageName)))
		{
			Iter.RemoveCurrent();
		}
	}
	Singleton.LastReferencer = NAME_None;
	Singleton.LastReferencerMap = nullptr;
	Singleton.LastAccessData = FBuildDependencyAccessData{ NAME_None, nullptr };
	Singleton.LastCookResultProjection = NAME_None;
}

void FResultProjectionList::AddProjectionAll(bool* bOutExists)
{
	if (bHasAll)
	{
		if (bOutExists)
		{
			*bOutExists = true;
		}
		return;
	}
	bHasAll = true;
	Classes.Empty();
	ResultProjections.Empty();
	if (bOutExists)
	{
		*bOutExists = false;
	}
}

void FResultProjectionList::AddProjection(FName CookResultProjection, FTopLevelAssetPath ClassPath, bool* bOutExists)
{
	if (bHasAll)
	{
		if (bOutExists)
		{
			*bOutExists = true;
		}
		return;
	}

	if (CookResultProjection == UE::Cook::ResultProjection::PackageAndClass)
	{
		Classes.Add(ClassPath, bOutExists);
		ResultProjections.Add(CookResultProjection);
	}
	else
	{
		// ResultProjection::None and ResultProjection::All should have been handled by caller and this function not called
		check(CookResultProjection != UE::Cook::ResultProjection::None);
		check(CookResultProjection != UE::Cook::ResultProjection::All);
		ResultProjections.Add(CookResultProjection, bOutExists);
	}
}

#endif // UE_WITH_PACKAGE_ACCESS_TRACKING

void DumpBuildDependencyTrackerStats()
{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FPackageBuildDependencyTracker::Get().DumpStats();
#endif
#if UE_WITH_CONFIG_TRACKING && UE_WITH_OBJECT_HANDLE_TRACKING
	UE::ConfigAccessTracking::FCookConfigAccessTracker::Get().DumpStats();
#endif
}
