// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/PackageAccessTracking.h"

#if UE_WITH_PACKAGE_ACCESS_TRACKING
#include "Async/Mutex.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/ObjectHandle.h"
#include "UObject/TopLevelAssetPath.h"
#endif

#if UE_WITH_PACKAGE_ACCESS_TRACKING

struct FBuildDependencyAccessData
{
	FName ReferencedPackage;
	const ITargetPlatform* TargetPlatform;
	friend uint32 GetTypeHash(const FBuildDependencyAccessData& Data)
	{
		return HashCombine(GetTypeHash(Data.ReferencedPackage), GetTypeHash(Data.TargetPlatform));
	}
	bool operator==(const FBuildDependencyAccessData& Other) const
	{
		return ReferencedPackage == Other.ReferencedPackage && TargetPlatform == Other.TargetPlatform;
	}
	bool operator!=(const FBuildDependencyAccessData& Other) const
	{
		return !(*this == Other); 
	}
};

struct FResultProjectionList
{
	TSet<FTopLevelAssetPath> Classes;
	TSet<FName> ResultProjections;
	bool bHasAll = false;

	void AddProjectionAll(bool* bOutExists = nullptr);
	void AddProjection(FName CookResultProjection, FTopLevelAssetPath ClassPath, bool* bOutExists = nullptr);
};

class FPackageBuildDependencyTracker: public FNoncopyable
{
public:
	static FPackageBuildDependencyTracker& Get() { return Singleton; }

	void Disable();
	bool IsEnabled() const;
	void DumpStats() const;
	TArray<TPair<FBuildDependencyAccessData, FResultProjectionList>> GetAccessDatas(FName ReferencerPackage) const;

private:
	FPackageBuildDependencyTracker();
	virtual ~FPackageBuildDependencyTracker();
	void PostGarbageCollect();

	/** Track object reference reads */
	static void StaticOnObjectHandleRead(const TArrayView<const UObject* const>& Objects);

	// Use a mutex rather than a critical section for synchronization.  Calls into system libraries, such as windows critical section
	// functions, are 50 times more expensive on build farm VMs, radically affecting cook times, which this avoids.  Saves 5% of total
	// cook time for shader invalidation on a large project.
	mutable UE::FMutex RecordsLock;

	TMap<FName, TMap<FBuildDependencyAccessData, FResultProjectionList>> Records;
	FName LastReferencer = NAME_None;
	FBuildDependencyAccessData LastAccessData{ NAME_None, nullptr };
	FName LastCookResultProjection = NAME_None;
	TMap<FBuildDependencyAccessData, FResultProjectionList>* LastReferencerMap = nullptr;
	UE::CoreUObject::FObjectHandleTrackingCallbackId ObjectHandleReadHandle;
	bool bEnabled = false;
	static FPackageBuildDependencyTracker Singleton;
};

#endif // UE_WITH_PACKAGE_ACCESS_TRACKING

void DumpBuildDependencyTrackerStats();
