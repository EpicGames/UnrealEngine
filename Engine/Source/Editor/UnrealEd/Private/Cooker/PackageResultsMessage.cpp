// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageResultsMessage.h"

#include "CookPackageData.h"
#include "Misc/ScopeLock.h"

namespace UE::Cook
{

TArray<UE::CompactBinaryTCP::FMarshalledMessage> FPackageRemoteResult::FPlatformResult::ReleaseMessages()
{
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> Result = MoveTemp(Messages);
	Messages.Empty();
	return Result;
}

TArray<UE::CompactBinaryTCP::FMarshalledMessage> FPackageRemoteResult::ReleaseMessages()
{
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> Result = MoveTemp(Messages);
	Messages.Empty();
	return Result;
}

void FPackageRemoteResult::AddPackageMessage(const FGuid& MessageType, FCbObject&& Object)
{
	Messages.Add(UE::CompactBinaryTCP::FMarshalledMessage{ MessageType, MoveTemp(Object) });
}

void FPackageRemoteResult::AddAsyncPackageMessage(const FGuid& MessageType, TFuture<FCbObject>&& ObjectFuture)
{
	FAsyncMessage& AsyncMessage = AsyncMessages.Emplace_GetRef();
	AsyncMessage.MessageType = MessageType;
	AsyncMessage.Future = MoveTemp(ObjectFuture);
}

void FPackageRemoteResult::AddPlatformMessage(const ITargetPlatform* TargetPlatform, const FGuid& MessageType,
	FCbObject&& Object)
{
	check(TargetPlatform != nullptr);
	FPlatformResult* Result = Platforms.FindByPredicate([TargetPlatform](const FPlatformResult& Result)
		{ return Result.Platform == TargetPlatform; }
	);
	check(Result);
	Result->Messages.Add(UE::CompactBinaryTCP::FMarshalledMessage{ MessageType, MoveTemp(Object) });
}

void FPackageRemoteResult::AddAsyncPlatformMessage(const ITargetPlatform* TargetPlatform, const FGuid& MessageType,
	TFuture<FCbObject>&& ObjectFuture)
{
	check(TargetPlatform != nullptr);
	FAsyncMessage& AsyncMessage = AsyncMessages.Emplace_GetRef();
	AsyncMessage.MessageType = MessageType;
	AsyncMessage.Future = MoveTemp(ObjectFuture);
	AsyncMessage.TargetPlatform = TargetPlatform;
}

bool FPackageRemoteResult::IsComplete()
{
	FinalizeAsyncMessages();
	FScopeLock AsyncWorkScopeLock(&AsyncSupport->AsyncWorkLock);
	return bAsyncMessagesComplete;
}

TFuture<int> FPackageRemoteResult::GetCompletionFuture()
{
	FinalizeAsyncMessages();
	return AsyncSupport->CompletionFuture.GetFuture();
}

void FPackageRemoteResult::FinalizeAsyncMessages()
{
	if (bAsyncMessagesFinalized)
	{
		return;
	}
	bAsyncMessagesFinalized = true;
	AsyncSupport = MakeUnique<FAsyncSupport>();

	if (AsyncMessages.IsEmpty())
	{
		bAsyncMessagesComplete = true;
		AsyncSupport->CompletionFuture.EmplaceValue(0);
		return;
	}

	// As of now we have no multithreading that accesses this. The first time it can occur is after calling the first Next.
	NumIncompleteAsyncWork = AsyncMessages.Num();
	for (FAsyncMessage& Message : AsyncMessages)
	{
		Message.Future.Next([this, Message = &Message](FCbObject Object)
		{
			bool bLocalAsyncMessagesComplete = false;
			{
				FScopeLock AsyncWorkScopeLock(&AsyncSupport->AsyncWorkLock);
				check(!Message->bCompleted);
				Message->bCompleted = true;
				if (!Message->TargetPlatform)
				{
					AddPackageMessage(Message->MessageType, MoveTemp(Object));
				}
				else
				{
					AddPlatformMessage(Message->TargetPlatform, Message->MessageType, MoveTemp(Object));
				}
				if (--NumIncompleteAsyncWork == 0)
				{
					bAsyncMessagesComplete = true;
					bLocalAsyncMessagesComplete = true;
				}
			}
			if (bLocalAsyncMessagesComplete)
			{
				AsyncSupport->CompletionFuture.EmplaceValue(0);
			}
		});
	}
}

void FPackageRemoteResult::SetPlatforms(TConstArrayView<ITargetPlatform*> OrderedSessionPlatforms)
{
	Platforms.Reserve(OrderedSessionPlatforms.Num());
	for (ITargetPlatform* TargetPlatform : OrderedSessionPlatforms)
	{
		FPackageRemoteResult::FPlatformResult& PlatformResult = Platforms.Emplace_GetRef();
		PlatformResult.Platform = TargetPlatform;
	}
}

void FPackageResultsMessage::Write(FCbWriter& Writer) const
{
	Writer.BeginArray("R");
	for (const FPackageRemoteResult& Result : Results)
	{
		Writer.BeginObject();
		{
			Writer << "N" << Result.PackageName;
			Writer << "R" << (uint8) Result.SuppressCookReason;
			WriteMessagesArray(Writer, Result.Messages);
			Writer.BeginArray("P");
			for (const FPackageRemoteResult::FPlatformResult& PlatformResult : Result.Platforms)
			{
				Writer.BeginObject();
				Writer << "C" << PlatformResult.WasCommitted();
				Writer << "S" << (uint8) PlatformResult.GetCookResults();
				WriteMessagesArray(Writer, PlatformResult.Messages);
				Writer.EndObject();
			}
			Writer.EndArray();
			if (!Result.ExternalActorDependencies.IsEmpty())
			{
				Writer << "EAD" << Result.ExternalActorDependencies;
			}
		}
		Writer.EndObject();
	}
	Writer.EndArray();
}

void FPackageResultsMessage::WriteMessagesArray(FCbWriter& Writer,
	TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage> InMessages)
{
	if (!InMessages.IsEmpty())
	{
		// We write a nonhomogenous array of length 2N. 2N+0 is the Message type, 2N+1 is the message object.
		Writer.BeginArray("M");
		for (const UE::CompactBinaryTCP::FMarshalledMessage& Message : InMessages)
		{
			Writer << Message.MessageType;
			Writer << Message.Object;
		}
		Writer.EndArray();
	}
}

bool FPackageResultsMessage::TryRead(FCbObjectView Object)
{
	Results.Reset();
	for (FCbFieldView ResultField : Object["R"])
	{
		FCbObjectView ResultObject = ResultField.AsObjectView();
		FPackageRemoteResult& Result = Results.Emplace_GetRef();
		LoadFromCompactBinary(ResultObject["N"], Result.PackageName);
		if (Result.PackageName.IsNone())
		{
			return false;
		}
		int32 LocalSuppressCookReason = ResultObject["R"].AsUInt8(MAX_uint8);
		if (LocalSuppressCookReason == MAX_uint8)
		{
			return false;
		}
		Result.SuppressCookReason = static_cast<ESuppressCookReason>(LocalSuppressCookReason);
		if (!TryReadMessagesArray(ResultObject, Result.Messages))
		{
			return false;
		}
		Result.Platforms.Reset();
		for (FCbFieldView PlatformField : ResultObject["P"])
		{
			FPackageRemoteResult::FPlatformResult& PlatformResult = Result.Platforms.Emplace_GetRef();
			FCbObjectView PlatformObject = PlatformField.AsObjectView();

			PlatformResult.SetWasCommitted(PlatformObject["C"].AsBool());
			uint8 CookResultsInt = PlatformObject["S"].AsUInt8();
			if (CookResultsInt >= (uint8)ECookResult::Count)
			{
				return false;
			}
			PlatformResult.SetCookResults((ECookResult)CookResultsInt);

			if (!TryReadMessagesArray(PlatformObject, PlatformResult.Messages))
			{
				return false;
			}
		}
		FCbFieldView ExternalActorDependenciesField = ResultObject["EAD"];
		if (ExternalActorDependenciesField.IsArray())
		{
			if (!LoadFromCompactBinary(ExternalActorDependenciesField, Result.ExternalActorDependencies))
			{
				return false;
			}
		}
	}
	return true;
}

bool FPackageResultsMessage::TryReadMessagesArray(FCbObjectView ObjectWithMessageField,
	TArray<UE::CompactBinaryTCP::FMarshalledMessage>& InMessages)
{
	// We read a nonhomogenous array of length 2N. 2N+0 is the Message type, 2N+1 is the message object.
	FCbArrayView MessagesArray = ObjectWithMessageField["M"].AsArrayView();
	InMessages.Reset(MessagesArray.Num() / 2);
	FCbFieldViewIterator MessageField = MessagesArray.CreateViewIterator();
	while (MessageField)
	{
		UE::CompactBinaryTCP::FMarshalledMessage& Message = InMessages.Emplace_GetRef();
		Message.MessageType = MessageField->AsUuid();
		if (MessageField.HasError())
		{
			return false;
		}
		++MessageField;
		Message.Object = FCbObject::Clone(MessageField->AsObjectView());
		if (MessageField.HasError())
		{
			return false;
		}
		++MessageField;
	}
	return true;
}

FGuid FPackageResultsMessage::MessageType(TEXT("4631C6C0F6DC4CEFB2B09D3FB0B524DB"));

}