// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CompactBinaryTCP.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Cooker/CookTypes.h"
#include "Cooker/MPCollector.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class ITargetPlatform;
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FPackageResultsMessage; }

namespace UE::Cook
{

/**
 * Helper struct for FPackageResultsMessage.
 * Holds replication information about the result of a Package's save, including per-platform results and
 * system-specific messages from other systems
 */
struct FPackageRemoteResult
{
public:
	/** Information about the results for a single platform */
	struct FPlatformResult
	{
	public:
		const ITargetPlatform* GetPlatform() const;
		void SetPlatform(const ITargetPlatform* InPlatform);

		TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage> GetMessages() const;
		TArray<UE::CompactBinaryTCP::FMarshalledMessage> ReleaseMessages();

		bool WasCommitted() const;
		void SetWasCommitted(bool bValue);

		ECookResult GetCookResults() const;
		void SetCookResults(ECookResult Value);

	private:
		TArray<UE::CompactBinaryTCP::FMarshalledMessage> Messages;
		const ITargetPlatform* Platform = nullptr;
		ECookResult CookResults = ECookResult::NotAttempted;
		bool bWasCommitted = false;

		friend FPackageRemoteResult;
		friend FPackageResultsMessage;
	};

	FPackageRemoteResult() = default;
	FPackageRemoteResult(FPackageRemoteResult&&) = default;
	FPackageRemoteResult(const FPackageRemoteResult&) = delete;
	FPackageRemoteResult& operator=(FPackageRemoteResult&&) = default;
	FPackageRemoteResult& operator=(const FPackageRemoteResult&) = delete;

	FName GetPackageName() const;
	void SetPackageName(FName InPackageName);

	ESuppressCookReason GetSuppressCookReason() const;
	void SetSuppressCookReason(ESuppressCookReason InSuppressCookReason);

	void AddPackageMessage(const FGuid& MessageType, FCbObject&& Object);
	void AddAsyncPackageMessage(const FGuid& MessageType, TFuture<FCbObject>&& ObjectFuture);
	void AddPlatformMessage(const ITargetPlatform* TargetPlatform, const FGuid& MessageType, FCbObject&& Object);
	void AddAsyncPlatformMessage(const ITargetPlatform* TargetPlatform, const FGuid& MessageType, TFuture<FCbObject>&& ObjectFuture);

	// GetMessages and ReleaseMessages are not thread-safe until IsComplete returns true or GetCompletionFuture().Get()/.Next().
	TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage> GetMessages() const;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> ReleaseMessages();

	bool IsComplete();
	TFuture<int> GetCompletionFuture();

	TArray<FPlatformResult, TInlineAllocator<1>>& GetPlatforms();
	void SetPlatforms(TConstArrayView<ITargetPlatform*> OrderedSessionPlatforms);

	void SetExternalActorDependencies(TArray<FName>&& InExternalActorDependencies);
	TConstArrayView<FName> GetExternalActorDependencies() const;

	/**
	 * A non-atomic RefCount that can be used for storage of a refcount by the user (e.g. CookWorkerClient)
	 * If used from multiple threads, the user must access it only within the user's external critical section.
	 */
	int32& GetUserRefCount();

private:
	/** A TFuture and status data that was received from an asynchronous IMPCollector. */
	struct FAsyncMessage
	{
		FAsyncMessage() = default;
		FAsyncMessage(FAsyncMessage&&) = default;
		FAsyncMessage(const FAsyncMessage&) = delete;

		FGuid MessageType;
		TFuture<FCbObject> Future;
		const ITargetPlatform* TargetPlatform = nullptr;
		bool bCompleted = false;
	};
	/**
	 * Some of the fields used when writing async messages on clients; these fields are otherwise unused.
	 * These fields do not support Move construction or assignment, or memmove, so to support TArray
	 * of FPackageRemoteResult we have to store these fields in a separate allocation.
	 */
	struct FAsyncSupport
	{
		TPromise<int> CompletionFuture;
		FCriticalSection AsyncWorkLock;
	};

	/**
	 * If any async messages have been stored, subscribe to their Futures to pull their resultant messages
	 * and trigger this struct's ComplectionFuture when they are all done.
	 */
	void FinalizeAsyncMessages();

private:
	// Fields read/writable only from the owner thread.
	TArray<FAsyncMessage> AsyncMessages;
	TArray<FName> ExternalActorDependencies;
	FName PackageName;
	/** If failure reason is NotSuppressed, it was saved. Otherwise, holds the suppression reason */
	ESuppressCookReason SuppressCookReason;

	// Fields guarded by AsyncSupport->AsyncWorkLock. They can only be read or written if either AsyncSupport is nullptr
	// or if within AsyncSupport->AsyncWorkLock.
	TArray<FPlatformResult, TInlineAllocator<1>> Platforms;
	TArray<UE::CompactBinaryTCP::FMarshalledMessage> Messages;
	TUniquePtr<FAsyncSupport> AsyncSupport;
	int32 NumIncompleteAsyncWork = 0;
	bool bAsyncMessagesFinalized = false;
	bool bAsyncMessagesComplete = false;

	// Fields Read/Write only within an external critical section
	int32 UserRefCount = 0;

	friend FPackageResultsMessage;
};

/** Message from Client to Server giving the results for saved or refused-to-cook packages. */
struct FPackageResultsMessage : public IMPCollectorMessage
{
public:
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override;
	virtual const TCHAR* GetDebugName() const override
	{
		return TEXT("PackageResultsMessage");
	}

public:
	TArray<FPackageRemoteResult> Results;

	static FGuid MessageType;

private:
	static void WriteMessagesArray(FCbWriter& Writer,
		TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage> InMessages);
	static bool TryReadMessagesArray(FCbObjectView ObjectWithMessageField,
		TArray<UE::CompactBinaryTCP::FMarshalledMessage>& InMessages);
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline const ITargetPlatform* FPackageRemoteResult::FPlatformResult::GetPlatform() const
{
	return Platform;
}

inline void FPackageRemoteResult::FPlatformResult::SetPlatform(const ITargetPlatform* InPlatform)
{
	Platform = InPlatform;
}

inline TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage>
FPackageRemoteResult::FPlatformResult::GetMessages() const
{
	return Messages;
}

inline bool FPackageRemoteResult::FPlatformResult::WasCommitted() const
{
	return bWasCommitted;
}

inline void FPackageRemoteResult::FPlatformResult::SetWasCommitted(bool bValue)
{
	bWasCommitted = bValue;
}

inline ECookResult FPackageRemoteResult::FPlatformResult::GetCookResults() const
{
	return CookResults;
}

inline void FPackageRemoteResult::FPlatformResult::SetCookResults(ECookResult Value)
{
	CookResults = Value;
}

inline FName FPackageRemoteResult::GetPackageName() const
{
	return PackageName;
}

inline void FPackageRemoteResult::SetPackageName(FName InPackageName)
{
	PackageName = InPackageName;
}

inline ESuppressCookReason FPackageRemoteResult::GetSuppressCookReason() const
{
	return SuppressCookReason;
}

inline void FPackageRemoteResult::SetSuppressCookReason(ESuppressCookReason InSuppressCookReason)
{
	SuppressCookReason = InSuppressCookReason;
}

inline TConstArrayView<UE::CompactBinaryTCP::FMarshalledMessage> FPackageRemoteResult::GetMessages() const
{
	return Messages;
}

inline TArray<FPackageRemoteResult::FPlatformResult, TInlineAllocator<1>>& FPackageRemoteResult::GetPlatforms()
{
	return Platforms;
}

inline void FPackageRemoteResult::SetExternalActorDependencies(TArray<FName>&& InExternalActorDependencies)
{
	ExternalActorDependencies = MoveTemp(InExternalActorDependencies);
}

inline TConstArrayView<FName> FPackageRemoteResult::GetExternalActorDependencies() const
{
	return ExternalActorDependencies;
}

inline int32& FPackageRemoteResult::GetUserRefCount()
{
	return UserRefCount;
}

inline FGuid FPackageResultsMessage::GetMessageType() const
{
	return MessageType;
}

}
