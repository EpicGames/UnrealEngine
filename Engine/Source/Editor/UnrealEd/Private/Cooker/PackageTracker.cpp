// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageTracker.h"

#include "CookOnTheFlyServerInterface.h"
#include "CookPackageData.h"
#include "CookPlatformManager.h"
#include "CookProfilingPrivate.h"
#include "Engine/Level.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

namespace UE::Cook
{

void FThreadSafeUnsolicitedPackagesList::AddCookedPackage(const FFilePlatformRequest& PlatformRequest)
{
	FScopeLock S(&SyncObject);
	CookedPackages.Add(PlatformRequest);
}

void FThreadSafeUnsolicitedPackagesList::GetPackagesForPlatformAndRemove(const ITargetPlatform* Platform, TArray<FName>& PackageNames)
{
	FScopeLock _(&SyncObject);

	for (int I = CookedPackages.Num() - 1; I >= 0; --I)
	{
		FFilePlatformRequest& Request = CookedPackages[I];

		if (Request.GetPlatforms().Contains(Platform))
		{
			// remove the platform
			Request.RemovePlatform(Platform);
			PackageNames.Emplace(Request.GetFilename());

			if (Request.GetPlatforms().Num() == 0)
			{
				CookedPackages.RemoveAt(I);
			}
		}
	}
}

void FThreadSafeUnsolicitedPackagesList::Empty()
{
	FScopeLock _(&SyncObject);
	CookedPackages.Empty();
}

FPackageTracker::FPackageTracker(UCookOnTheFlyServer& InCOTFS)
	:COTFS(InCOTFS)
{
	ActiveInstances = new FPackageStreamInstancedPackageContainer(); // RefCounted
}

FPackageTracker::~FPackageTracker()
{
	Unsubscribe();
}

void FPackageTracker::Unsubscribe()
{
	if (!bSubscribed)
	{
		return;
	}

	bSubscribed = false;
	GUObjectArray.RemoveUObjectDeleteListener(this);
	GUObjectArray.RemoveUObjectCreateListener(this);
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
}

void FPackageTracker::InitializeTracking(TSet<FName>& OutStartupPackages)
{
	check(!bTrackingInitialized);

	LLM_SCOPE_BYTAG(Cooker);

	FWriteScopeLock ScopeLock(Lock);
	
	TMap<FName, UE::Cook::FInstigator> MapOfNewPackages;
	FCookLoadScope::GetCookerStartupPackages(MapOfNewPackages);

	GUObjectArray.AddUObjectDeleteListener(this);
	GUObjectArray.AddUObjectCreateListener(this);
	FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FPackageTracker::OnEndLoadPackage);

	bSubscribed = true;

	bTrackingInitialized = true;

	PackageStream.Reserve(MapOfNewPackages.Num());
	OutStartupPackages.Empty(MapOfNewPackages.Num());
	for (TPair<FName, UE::Cook::FInstigator>& Pair : MapOfNewPackages)
	{
		switch (Pair.Value.Category)
		{
		case UE::Cook::EInstigator::StartupPackage:
		case UE::Cook::EInstigator::StartupPackageCookLoadScope:
			OutStartupPackages.Add(Pair.Key);
			break;
		case UE::Cook::EInstigator::EditorOnlyLoad:
		default:
			break;
		}
		PackageStream.Add({ Pair.Key, MoveTemp(Pair.Value), EPackageStreamEvent::PackageLoad });
	}

	for (TObjectIterator<UPackage> Iter; Iter; ++Iter)
	{
		LoadedPackages.Add(*Iter);
	}
}

TArray<FPackageStreamEvent> FPackageTracker::GetPackageStream()
{
	check(bTrackingInitialized);

	FWriteScopeLock ScopeLock(Lock);
	TArray<FPackageStreamEvent> Result = MoveTemp(PackageStream);
	PackageStream.Reset();
	return Result;
}

void FPackageTracker::NotifyUObjectCreated(const class UObjectBase* Object, int32 Index)
{
	if (Object->GetClass() == UPackage::StaticClass())
	{
		UPackage* Package = const_cast<UPackage*>(static_cast<const UPackage*>(Object));
		if (Package->GetOuter() == nullptr) // Nested packages are no longer created, but can still exist in old data.
		{
			OnCreatePackage(Package->GetFName());

			FWriteScopeLock ScopeLock(Lock);
			LoadedPackages.Add(Package);
		}
	}
}

FInstigator FPackageTracker::GetPackageCreationInstigator() const
{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData =
		PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	FName ReferencerName(AccumulatedScopeData ? AccumulatedScopeData->PackageName : NAME_None);
#else
	FName ReferencerName(NAME_None);
#endif
	EInstigator InstigatorType;
	switch (FCookLoadScope::GetCurrentValue())
	{
	case ECookLoadType::EditorOnly:
		InstigatorType = EInstigator::EditorOnlyLoad;
		break;
	case ECookLoadType::UsedInGame:
		InstigatorType = EInstigator::SaveTimeSoftDependency;
		break;
	default:
		InstigatorType = EInstigator::Unsolicited;
		break;
	}
	return FInstigator(InstigatorType, ReferencerName);
}

void FPackageTracker::OnCreatePackage(FName PackageName)
{
	LLM_SCOPE_BYTAG(Cooker);
#if ENABLE_COOK_STATS
	++DetailedCookStats::NumDetectedLoads;
#endif
	FInstigator Instigator = GetPackageCreationInstigator();
	if (Instigator.Category == EInstigator::Unsolicited && COTFS.bHiddenDependenciesDebug)
	{
		COTFS.OnDiscoveredPackageDebug(PackageName, Instigator);
	}

	FWriteScopeLock ScopeLock(Lock);
	if (ExpectedNeverLoadPackages.Contains(PackageName))
	{
		UE_LOGF(LogCook, Verbose, "SoftGC PoorPerformance: Reloaded package %ls.",
			*WriteToString<256>(PackageName));
	}

	// We store packages by name rather than by pointer, because they might have their name changed. When
	// external actors are moved out of their external package, we rename the package to <PackageName>_Trash.
	// We want to report a load dependency on the package as it was originally loaded; we don't want to report
	// the renamed packagename if it gets renamed after load.
	ActivePackageInstigators.FindOrAdd(PackageName, Instigator);
	PackageStream.Add({ PackageName, MoveTemp(Instigator), EPackageStreamEvent::PackageLoad });
}

void FPackageTracker::NotifyUObjectDeleted(const class UObjectBase* ObjectBase, int32 Index)
{
	if (ObjectBase->GetClass() == UPackage::StaticClass())
	{
		UPackage* Package = const_cast<UPackage*>(static_cast<const UPackage*>(ObjectBase));

		FWriteScopeLock ScopeLock(Lock);
		LoadedPackages.Remove(Package);
	}
	if (!bCollectingGarbage)
	{
		const UObject* Object = static_cast<const UObject*>(ObjectBase);
		COTFS.PackageDatas->CachedCookedPlatformDataObjectsOnDestroyedOutsideOfGC(Object);
	}
}

void FPackageTracker::OnUObjectArrayShutdown()
{
	Unsubscribe();
}

void FPackageTracker::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	RemapMapKeys(PlatformSpecificNeverCookPackages, Remap);
}

void FPackageTracker::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	using namespace UE::AssetRegistry;

	// OnEndLoadPackage is the hook we use to test for whether a load is instanced; the CreatePackage hook is too early
	// (LoadPath is unknown in the case of instanced loads), and the OnSyncLoadPackage and OnAsyncLoadPackage hooks are
	// called too frequently - they are called even for packages that have already loaded, before LoadPackage checks
	// for whether they can early exit. OnEndLoadPackage has the information we need and is only called for packages
	// when they transition from unloaded to loaded.
	// 
	// Use this hook to record instanced loads in our lookup map; ProcessUnsolicitedPackages will respond to the
	// creation event for the packages by looking them up in the map and using the information we provide about their
	// AssetRegistry dependencies non-instanced referencers.
	TArray<TRefCountPtr<FPackageStreamInstancedPackage>, TInlineAllocator<10>> LoadedInstances;
	for (UPackage* Package : Context.LoadedPackages)
	{
		FName PackageName = Package->GetFName();
		FName LoadedName = Package->GetLoadedPath().GetPackageFName();
		if (PackageName == LoadedName || LoadedName.IsNone())
		{
			continue;
		}

		FInstigator Instigator(EInstigator::Unsolicited);
		{
			FWriteScopeLock ScopeLock(Lock);
			FInstigator* ActiveInstigator = ActivePackageInstigators.Find(PackageName);
			if (ActiveInstigator)
			{
				Instigator = *ActiveInstigator;
			}
		}

		TArray<FAssetDependency> PackageDependencies;
		COTFS.AssetRegistry->GetDependencies(FAssetIdentifier(LoadedName), PackageDependencies,
			EDependencyCategory::Package, EDependencyQuery::Hard);

		FWriteScopeLock ActiveInstancesScopeLock(ActiveInstances->Lock);
		FPackageStreamInstancedPackage*& Existing = ActiveInstances->Map.FindOrAdd(PackageName, nullptr);
		if (Existing)
		{
			if (Existing->LoadedName != LoadedName)
			{
				UE_LOGF(LogCook, Error,
					"OnBeginLoadPackage was called twice for the same package with two different LoadedPaths. Ignoring the second call. PackageName: %ls. LoadedPath1: %ls. LoadedPath2: %ls.",
					*PackageName.ToString(), *Existing->LoadedName.ToString(), *LoadedName.ToString());
				continue;
			}
			LoadedInstances.Emplace(Existing);
			continue;
		}

		TRefCountPtr<FPackageStreamInstancedPackage> InstancedPackage = new FPackageStreamInstancedPackage(*ActiveInstances);
		Existing = InstancedPackage.GetReference();

		InstancedPackage->PackageName = PackageName;
		InstancedPackage->LoadedName = LoadedName;
		InstancedPackage->Instigator = MoveTemp(Instigator);
		InstancedPackage->Dependencies.Reserve(PackageDependencies.Num());
		for (FAssetDependency& PackageDependency : PackageDependencies)
		{
			InstancedPackage->Dependencies.Add(PackageDependency.AssetId.PackageName, PackageDependency.Properties);
		}

		LoadedInstances.Emplace(MoveTemp(InstancedPackage));
	}

	if (LoadedInstances.IsEmpty())
	{
		// The usual path through this function is that there were no loaded instances. Clear the
		// ActivePackageInstigators to fulfill our design of removing that memory when we no longer need it, and then
		// return without further work.
		FWriteScopeLock ScopeLock(Lock);
		ActivePackageInstigators.Reset();
	}
	else
	{
		// Now that FPackageStreamInstancedPackage have been registered for all of the instanced loads that occurred
		// during the top-most LoadPackage call, calculate the ancestor non-instanced package referencer for each of
		// the instanced package loads.
		{
			TMap<FName, TOptional<TMap<FName, EDependencyProperty>>> DependenciesOfNonInstancedReferencers;
			FReadScopeLock ActiveInstancesScopeLock(ActiveInstances->Lock);
			for (TRefCountPtr<FPackageStreamInstancedPackage>& InstancedPackage : LoadedInstances)
			{
				TSet<FPackageStreamInstancedPackage*> Visited;
				InstancedPackage->FlattenReferencer(ActiveInstancesScopeLock, Visited,
					DependenciesOfNonInstancedReferencers, *this);
			}
		}

		// The FPackageStreamInstancedPackage we created need to remain available until all of the package creation
		// records created during their load and that we have added into the PackageStream have been processed by
		// ProcessUnsolicitedPackages. Add end-of-data-lifetime markers for the FPackageStreamInstancedPackages
		// into the PackageStream, after all of those creation records.
		FWriteScopeLock ScopeLock(Lock);
		for (TRefCountPtr<FPackageStreamInstancedPackage>& Instance : LoadedInstances)
		{
			PackageStream.Add({ Instance->PackageName, FInstigator(), EPackageStreamEvent::InstancedPackageEndLoad,
				TRefCountPtr<const FPackageStreamInstancedPackage>(Instance.GetReference()) });
		}

		// Clear ActivePackageInstigators to fulfill our design of removing that memory when we no longer need it.
		ActivePackageInstigators.Reset();
	}
}

TRefCountPtr<const FPackageStreamInstancedPackage> FPackageTracker::FindInstancedPackage(FName PackageName)
{
	FWriteScopeLock ActiveInstancesScopeLock(ActiveInstances->Lock);
	FPackageStreamInstancedPackage** Existing = ActiveInstances->Map.Find(PackageName);
	return TRefCountPtr<const FPackageStreamInstancedPackage>(Existing ? *Existing : nullptr);
}

EInstigator FPackageTracker::MergeReferenceCategories(EInstigator Parent, EInstigator Child)
{
	// EditorOnly -> 0, Unsolicited -> 1, UsedInGame -> 2. Return Min(Child, Parent)
	switch (Parent)
	{
	case EInstigator::EditorOnlyLoad: [[fallthrough]];
	case EInstigator::HardEditorOnlyDependency:
		return Parent;
	case EInstigator::Unsolicited:
		switch (Child)
		{
		case EInstigator::EditorOnlyLoad: [[fallthrough]];
		case EInstigator::HardEditorOnlyDependency:
			return Child;
		default:
			// Child is Unsolicited or UsedInGame, 
			return Parent;
		}
	default:
		return Child;
	}
}

FPackageStreamInstancedPackage::FPackageStreamInstancedPackage(FPackageStreamInstancedPackageContainer& InContainer)
	: Container(&InContainer)
{
}

FPackageStreamInstancedPackage::~FPackageStreamInstancedPackage()
{
	{
		FWriteScopeLock ActiveInstancesScopeLock(Container->Lock);
		Container->Map.Remove(PackageName);
	}
	Container.SafeRelease();
}

void FPackageStreamInstancedPackage::FlattenReferencer(FReadScopeLock& CalledInsideActiveInstancesLock,
	TSet<FPackageStreamInstancedPackage*>& Visited,
	TMap<FName, TOptional<TMap<FName, UE::AssetRegistry::EDependencyProperty>>>& DependenciesOfNonInstancedReferencers,
	FPackageTracker& PackageTracker)
{
	using namespace UE::AssetRegistry;

	bool bAlreadyVisited;
	Visited.Add(this, &bAlreadyVisited);
	if (bAlreadyVisited)
	{
		UE_LOGF(LogCook, Error, "Cycle detected in InstancedPackage referencers. PackageName == %ls",
			*PackageName.ToString());
		return;
	}
	if (Instigator.Referencer.IsNone())
	{
		return;
	}

	// If the Instigator of this instanced package is itself an instanced package, recurse into it.
	// And in either case, instanced package or not, get the instigator's AssetRegistry dependencies
	// in case the Instigator.Category is EInstigator::Unsolicited.
	FPackageStreamInstancedPackage** ReferencerPtr = Container->Map.Find(Instigator.Referencer);
	FPackageStreamInstancedPackage* Referencer = ReferencerPtr ? *ReferencerPtr : nullptr;
	TMap<FName, UE::AssetRegistry::EDependencyProperty>* ReferencerDependencies = nullptr;
	if (ReferencerPtr)
	{
		// All Dependencies on FPackageStreamInstancedPackage are intialized when the
		// FPackageStreamInstancedPackage is constructed, no need to initialize them here.
		ReferencerDependencies = &Referencer->Dependencies;
	}
	else
	{
		TOptional<TMap<FName, UE::AssetRegistry::EDependencyProperty>>& ExistingDependencies
			= DependenciesOfNonInstancedReferencers.FindOrAdd(Instigator.Referencer);
		if (!ExistingDependencies)
		{
			TArray<FAssetDependency> PackageDependencies;
			IAssetRegistry::GetChecked().GetDependencies(FAssetIdentifier(Instigator.Referencer), PackageDependencies,
				EDependencyCategory::Package, EDependencyQuery::Hard);
			ExistingDependencies.Emplace();
			for (FAssetDependency& PackageDependency : PackageDependencies)
			{
				ExistingDependencies->Add(PackageDependency.AssetId.PackageName, PackageDependency.Properties);
			}
		}
		ReferencerDependencies = &(*ExistingDependencies);
	}

	// If the Instigator.Category is Unsolicited, try to remap it based on AssetRegistry dependencies; all hard
	// imports start off as being marked Unsolicited, and we need to convert them to HardDependency (or
	// to EditorOnlyLoad if they are an EditorOnly HardDependency). It is also possible that this PackageLoad
	// was not due to a hard import, and is indeed Unsolicited, in which case it is not present in the AR
	// dependencies and we keep it marked as Unsolicited.
	if (Instigator.Category != EInstigator::ForceExplorableSaveTimeSoftDependency
		&& PackageTracker.NeverCookPackageList.Contains(LoadedName))
	{
		TStringBuilder<256> PackageNameStr(InPlace, LoadedName);
		if (INDEX_NONE != UE::String::FindFirst(PackageNameStr,
			ULevel::GetExternalActorsFolderName(), ESearchCase::IgnoreCase)
			|| INDEX_NONE != UE::String::FindFirst(PackageNameStr,
				FPackagePath::GetExternalObjectsFolderName(), ESearchCase::IgnoreCase))
		{
			// ONLYEDITORONLY_TODO: WorldPartition should mark these loads as ForceExplorableSaveTimeSoftDependency
			// rather than needing to use a naming convention. We should also mark them once, during Save, rather than
			// marking them and reexploring them every time they are loaded. 
			Instigator.Category = EInstigator::ForceExplorableSaveTimeSoftDependency;
		}
	}

	if (Instigator.Category == EInstigator::Unsolicited)
	{
		const EDependencyProperty* DependencyProperty = ReferencerDependencies->Find(LoadedName);
		if (DependencyProperty)
		{
			Instigator.Category = EnumHasAnyFlags(*DependencyProperty, EDependencyProperty::Game)
				? EInstigator::HardDependency : EInstigator::EditorOnlyLoad;
		}
	}
	if (!ReferencerPtr)
	{
		// Now that we have remapped the category if necessary, if our parent is not an instanced package
		// then we have no further work to do.
		return;
	}

	// If our parent is an instanced package, flatten its instigator recursively and then merge
	// its instigator with the data from its instigation of this instanced package.
	Referencer->FlattenReferencer(CalledInsideActiveInstancesLock, Visited,
		DependenciesOfNonInstancedReferencers, PackageTracker);

	Instigator.Referencer = Referencer->Instigator.Referencer;
	Instigator.Category = FPackageTracker::MergeReferenceCategories(
		Referencer->Instigator.Category, Instigator.Category);
}

} // namespace UE::Cook
