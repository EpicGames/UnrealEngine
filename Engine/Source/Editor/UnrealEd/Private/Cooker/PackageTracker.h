// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "Containers/Set.h"
#include "CookRequests.h"
#include "CookTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectArray.h"

#include <atomic>

class FReadScopeLock;
class ITargetPlatform;
class UPackage;
namespace UE::Cook { struct FInstigator; }
namespace UE::Cook { struct FPackageData; }
namespace UE::Cook { struct FRecompileShaderRequest; }

namespace UE::Cook
{

struct FPackageStreamInstancedPackage;

template<typename Type>
struct FThreadSafeQueue
{
private:
	mutable FCriticalSection SynchronizationObject; // made this mutable so this class can have const functions and still be thread safe
	TRingBuffer<Type> Items;
public:
	void Enqueue(const Type& Item)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Add(Item);
	}
		
	void Enqueue(Type&& Item)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Add(MoveTempIfPossible(Item));
	}

	void EnqueueUnique(const Type& Item)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		for (const Type& Existing : Items)
		{
			if (Existing == Item)
			{
				return;
			}
		}
		Items.PushBack(Item);
	}

	bool Dequeue(Type* Result)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		if (Items.Num())
		{
			*Result = Items.PopFrontValue();
			return true;
		}
		return false;
	}

	void DequeueAll(TArray<Type>& Results)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Results.Reserve(Results.Num() + Items.Num());
		while (!Items.IsEmpty())
		{
			Results.Add(Items.PopFrontValue());
		}
	}

	bool HasItems() const
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		return Items.Num() > 0;
	}

	void Remove(const Type& Item)
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Remove(Item);
	}

	void CopyItems(const TArray<Type>& InItems) const
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Empty(InItems.Num());
		for (const Type& Item : InItems)
		{
			Items.PushBack(Item);
		}
	}

	int Num() const
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		return Items.Num();
	}

	void Empty()
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		Items.Empty();
	}
};

/** Simple thread safe proxy for TSet<FName> */
template <typename T>
class FThreadSafeSet
{
	TSet<T> InnerSet;
	FCriticalSection SetCritical;
public:
	void Add(T InValue)
	{
		FScopeLock SetLock(&SetCritical);
		InnerSet.Add(InValue);
	}
	bool AddUnique(T InValue)
	{
		FScopeLock SetLock(&SetCritical);
		if (!InnerSet.Contains(InValue))
		{
			InnerSet.Add(InValue);
			return true;
		}
		return false;
	}
	bool Contains(T InValue)
	{
		FScopeLock SetLock(&SetCritical);
		return InnerSet.Contains(InValue);
	}
	void Remove(T InValue)
	{
		FScopeLock SetLock(&SetCritical);
		InnerSet.Remove(InValue);
	}
	void Empty()
	{
		FScopeLock SetLock(&SetCritical);
		InnerSet.Empty();
	}

	void GetValues(TSet<T>& OutSet)
	{
		FScopeLock SetLock(&SetCritical);
		OutSet.Append(InnerSet);
	}
};

struct FThreadSafeUnsolicitedPackagesList
{
	void AddCookedPackage(const FFilePlatformRequest& PlatformRequest);
	void GetPackagesForPlatformAndRemove(const ITargetPlatform* Platform, TArray<FName>& PackageNames);
	void Empty();

private:
	FCriticalSection				SyncObject;
	TArray<FFilePlatformRequest>	CookedPackages;
};

/** Container for name -> data for FPackageStreamInstancedPackage held by the PackageTracker. */
struct FPackageStreamInstancedPackageContainer : public FRefCountedObject
{
	FRWLock Lock;
	TMap<FName, FPackageStreamInstancedPackage*> Map;
};

/**
 * Data about an instanced package load: a package was created with a non-existing name and another package was
 * loaded into it. We need to handle the recording of packages imported by instanced package loads in a special manner
 * since their AssetRegistry dependencies will not be found by querying the AssetRegistry for their name.
 */
struct FPackageStreamInstancedPackage : public FRefCountedObject
{
public:
	// FPackageStreamInstancedPackages are only createable by FPackageTracker, and they
	// are registered with PackageTracker in a map lookup by name to raw pointer. They
	// remove themselves from the map when they are destructed.
	~FPackageStreamInstancedPackage();

	FName PackageName;
	FName LoadedName;
	FInstigator Instigator;
	TMap<FName, UE::AssetRegistry::EDependencyProperty> Dependencies;
	
private:
	FPackageStreamInstancedPackage(FPackageStreamInstancedPackageContainer& InContainer);
	/**
	 * Set the Referencer to the first ancestor in the referencer chain that is non-instanced. Recursively called
	 * on the parent referencers in between this and the ancestor.
	 */
	void FlattenReferencer(FReadScopeLock& CalledInsideActiveInstancesLock,
		TSet<FPackageStreamInstancedPackage*>& Visited,
		TMap<FName, TOptional<TMap<FName, UE::AssetRegistry::EDependencyProperty>>>&
			DependenciesOfNonInstancedReferencers, FPackageTracker& PackageTracker);

private:
	TRefCountPtr<FPackageStreamInstancedPackageContainer> Container;

	friend FPackageTracker;
};

/**
 * The different types of events that can occur about the loads of packages, as collected by the PackageTracker and
 * reported to the cooker via GetPackageStream.
 */
enum class EPackageStreamEvent : uint8
{
	PackageLoad,
	InstancedPackageEndLoad,
};

/**
 * Data about an event in the in-order PackageStream collected by the PackageTracker to communicate events about the
 * load of packages to the cooker.
 */
struct FPackageStreamEvent
{
	FName PackageName;
	FInstigator Instigator;
	EPackageStreamEvent EventType;

	/**
	 * Only used by EPackageStreamEvent::InstancedPackageEndLoad, to keep the instanced package referenced until
	 * the event about it has been processed.
	 */
	TRefCountPtr<const FPackageStreamInstancedPackage> InstancedPackage;
};

struct FPackageTracker : public FUObjectArray::FUObjectCreateListener, public FUObjectArray::FUObjectDeleteListener
{
public:
	FPackageTracker(UCookOnTheFlyServer& InCOTFS);
	~FPackageTracker();

	void InitializeTracking(TSet<FName>& OutStartupPackages);

	/** Returns all packages that have been loaded since the last time GetNewPackages was called */
	TArray<FPackageStreamEvent> GetPackageStream();
	TRefCountPtr<const FPackageStreamInstancedPackage> FindInstancedPackage(FName PackageName);

	virtual void NotifyUObjectCreated(const class UObjectBase* Object, int32 Index) override;
	virtual void NotifyUObjectDeleted(const class UObjectBase* Object, int32 Index) override;
	virtual void OnUObjectArrayShutdown() override;

	void OnEndLoadPackage(const FEndLoadPackageContext& Context);

	/** Swap all ITargetPlatform* stored on this instance according to the mapping in @param Remap. */
	void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap);

	UCookOnTheFlyServer& COTFS;

	FThreadSafeUnsolicitedPackagesList UnsolicitedCookedPackages;
	FThreadSafeQueue<FRecompileShaderRequest> RecompileRequests;

	/** Packages to never cook - entries are LongPackageNames. */
	FThreadSafeSet<FName> NeverCookPackageList;
	TMap<const ITargetPlatform*, TSet<FName>> PlatformSpecificNeverCookPackages;

	// Thread-safe enumeration of loaded package. 
	// A lock is held during enumeration, keep code simple and optimal so the lock is released as fast as possible.
	template <typename FunctionType>
	void ForEachLoadedPackage(FunctionType Function)
	{
		FReadScopeLock ScopeLock(Lock);
		for (UPackage* Package : LoadedPackages)
		{
			Function(Package);
		}
	}
	int32 NumLoadedPackages()
	{
		FReadScopeLock ScopeLock(Lock);
		return LoadedPackages.Num();
	}
	void AddExpectedNeverLoadPackages(const TSet<FName>& PackageNames)
	{
		FWriteScopeLock ScopeLock(Lock);
		ExpectedNeverLoadPackages.Append(PackageNames);
	}
	void ClearExpectedNeverLoadPackages()
	{
		FWriteScopeLock ScopeLock(Lock);
		ExpectedNeverLoadPackages.Empty();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		FReadScopeLock ActiveInstancesScopeLock(ActiveInstances->Lock);
		return LoadedPackages.GetAllocatedSize() + ExpectedNeverLoadPackages.GetAllocatedSize()
			+ PackageStream.GetAllocatedSize()
			+ sizeof(*ActiveInstances)
			+ ActiveInstances->Map.GetAllocatedSize()
			+ ActiveInstances->Map.Num()*sizeof(FPackageStreamInstancedPackage);
	}

	void SetCollectingGarbage(bool bInCollectingGarbage)
	{
		bCollectingGarbage = bInCollectingGarbage;
	}

	/**
	 * When package A loads B loads C, and we don't want to tell the cooker about B (because it is e.g. an
	 * insetanced package), we need to calculate the reason that A loaded C by merging the reason A loaded B with
	 * the reason B loaded C. The PackageTracker sets EInstigator types to represent those reasons, and this function
	 * does the merging of a parent and child EInstigators to get the merged reason that the GrandParent loads the
	 * Child.
	 */
	static EInstigator MergeReferenceCategories(EInstigator Parent, EInstigator Child);

private:
	void Unsubscribe();
	void OnCreatePackage(FName LoadedPackageName);
	FInstigator GetPackageCreationInstigator() const;

	/** Thread synchronizer around data we read/write during hooks from package creation threads. */
	FRWLock Lock;

	// Begin Data Guarded by Lock

	/** All packages currently loaded by the engine. */
	TSet<UPackage*> LoadedPackages;
	/** List of packages from COTFS that we expect will never be loaded again; log a warning if we see them. */
	TSet<FName> ExpectedNeverLoadPackages;
	/* Events about UPackages created since the last call to GetPackageStream, plus related data such as the
	 * end-of-data-lifetime marker for the data we record about instanced package loads.
	 */
	TArray<FPackageStreamEvent> PackageStream;
	/**
	 * List of the Instigators we record for each created package; we keep a record of these instigators for use in our
	 * OnEndLoadPackage hook, and remove them when that hook is complete.
	 */
	TMap<FName, FInstigator> ActivePackageInstigators;

	// End Data Guarded by Lock

	// Begin Data ReadOnlyWhileSubscribed
	// 
	// This data is read-only while we are subscribed to the hooks that can be called from package creation threads,
	// and is writable only from scheduler thread, and only when we are not subscribed.
	/**
	 * Our pointer to the container used for Instanced packages. The pointer is read only; the container has its own
	 * internal lock.
	 */
	TRefCountPtr<FPackageStreamInstancedPackageContainer> ActiveInstances;
	bool bTrackingInitialized = false;
	bool bSubscribed = false;

	// End Data ReadOnlyWhileSubscribed

	// Begin Data ReadWrite from scheduler thread only
	bool bCollectingGarbage = false;

	// End Data ReadWrite from scheduler thread only
};

} // namespace UE::Cook
