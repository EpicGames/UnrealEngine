// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/ShaderLibraryCooking.h"

#include "Commandlets/AssetRegistryGenerator.h"
#include "Commandlets/ShaderPipelineCacheToolsCommandlet.h"
#include "Cooker/CookDirector.h"
#include "Cooker/CookOnTheFlyServerInterface.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookPlatformManager.h"
#include "Cooker/CookTypes.h"
#include "Cooker/CookWorkerClient.h"
#include "Cooker/OnDemandShaderCompilation.h"
#include "Cooker/PackageTracker.h"
#include "CookOnTheSide/CookLog.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CoreGlobals.h"
#include "GlobalShader.h"
#include "HAL/ConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "LayeredCookArtifactReader.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "PipelineCacheChunkDataGenerator.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Settings/ProjectPackagingSettings.h"
#include "ShaderCodeLibrary.h"
#include "ShaderCompiler.h"
#include "ShaderLibraryChunkDataGenerator.h"
#include "ShaderStats.h"
#include "ShaderStatsCollector.h"

// Include RHIShaderFormatDefinitions.inl only after all header includes; it defines some static variables,
// used by UpdateShaderCookingCVars.
#include "RHIShaderFormatDefinitions.inl"


bool bDisableShaderCompilationDuringCookOnTheFly = false;
static FAutoConsoleVariableRef CVarDisableShaderCompilationDuringCookOnTheFly(
	TEXT("Cook.OnTheFly.ShaderCompilationDisabled"),
	bDisableShaderCompilationDuringCookOnTheFly,
	TEXT("Controls whether shader compilation is disabled during cook on the fly\n"),
	ECVF_Default);

bool bAllowIncompleteShaderMapsDuringCookOnTheFly = false;
static FAutoConsoleVariableRef CVarAllowEmptyShaderMapsDuringCookOnTheFly(
	TEXT("Cook.OnTheFly.AllowIncompleteShadermaps"),
	bAllowIncompleteShaderMapsDuringCookOnTheFly,
	TEXT("Controls whether incomplete shader maps are processed during cook on the fly\n"),
	ECVF_Default);

static void GenerateAutogenShaderHeadersForAllShaderFormats(const TArray<ITargetPlatform*>& Platforms)
{
	for (const ITargetPlatform* Platform : Platforms)
	{
		TArray<FName> ShaderPlatformNameArray;
		Platform->GetAllTargetedShaderFormats(ShaderPlatformNameArray);

		for (const FName& ShaderPlatformName : ShaderPlatformNameArray)
		{
			FShaderCompileUtilities::GenerateBrdfHeaders(ShaderPlatformName);
		}
	}
}

bool UCookOnTheFlyServer::IsUsingShaderCodeLibrary() const
{
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	return IsDirectorCookByTheBook() && AllowShaderCompiling() && PackagingSettings->bShareMaterialShaderCode;
}

namespace UE::Cook
{

void TickShaderCompilingManager(UE::Cook::FTickStackData& StackData)
{
	UE_SCOPED_COOKTIMER_AND_DURATION(CookByTheBook_ShaderProcessAsync, DetailedCookStats::TickLoopShaderProcessAsyncResultsTimeSec);
	GShaderCompilingManager->ProcessAsyncResults(StackData.Timer.GetActionTimeSlice(), false);
}

} // namespace UE::Cook

void UCookOnTheFlyServer::TickRecompileShaderRequestsPrivate(UE::Cook::FTickStackData& StackData)
{
	using namespace UE::Cook;

	// try to pull off a request
	FRecompileShaderRequest RecompileShaderRequest;
	bool bProcessedRequests = false;
	if (PackageTracker->RecompileRequests.Dequeue(&RecompileShaderRequest))
	{
		if (RecompileShaderRequest.RecompileArguments.CommandType != ODSCRecompileCommand::ResetMaterialCache)
		{
			RecompileShadersForRemote(RecompileShaderRequest.RecompileArguments, GetSandboxDirectory(RecompileShaderRequest.RecompileArguments.PlatformName));
		}

		RecompileShaderRequest.CompletionCallback();
		bProcessedRequests = true;
	}
	if (PackageTracker->RecompileRequests.HasItems())
	{
		RecompileRequestsPollable->Trigger(*this);
	}

	if (bProcessedRequests)
	{
		// Ask for GC to run again when we processed some shaders requests to ensure material get evicted and we don't keep their package open
		StackData.ResultFlags |= COSR_RequiresGC | COSR_RequiresGC_Periodic | COSR_YieldTick;
	}
}

namespace UE::Cook::CVarControl
{

void UpdateShaderCookingCVars(ITargetPlatformManagerModule* TPM, int32 CookTimeCVarControl,
	const ITargetPlatform* Platform, FName PlatformName)
{
	// for now, only do this if we are performing any cvar control modes
	if (CookTimeCVarControl != 0)
	{
		if (Platform != TPM->GetRunningTargetPlatform())
		{
			// register that when we cook for this platform's shaderplatforms, we will want to use the given platform when looking up cvars
			TArray<FName> ShaderFormats;
			Platform->GetAllTargetedShaderFormats(ShaderFormats);
			for (FName SF : ShaderFormats)
			{
				EShaderPlatform SP = ShaderFormatNameToShaderPlatform(SF);
				ConsoleVariablePlatformMapping::RegisterShaderPlatformToPlatformMapping((int)SP, PlatformName);
			}
		}
	}
}

} // namespace UE::Cook::CVarControl


void UCookOnTheFlyServer::SaveGlobalShaderMapFiles(const TArrayView<const ITargetPlatform* const>& Platforms, ODSCRecompileCommand RecompileCommand)
{
	LLM_SCOPE(ELLMTag::Shaders);
	check(!IsCookingDLC()); // GlobalShaderMapFiles are not supported when cooking DLC
	check(IsInGameThread());
	for (const ITargetPlatform* TargetPlatform : Platforms)
	{
		const FString& PlatformName = TargetPlatform->PlatformName();
		UE_LOGF(LogCook, Display, "Compiling global%ls shaders for platform '%ls'",
			RecompileCommand == ODSCRecompileCommand::Changed ? TEXT(" changed") : TEXT(""), *PlatformName);

		TArray<uint8> GlobalShaderMap;
		FShaderRecompileData RecompileData(PlatformName, SP_NumPlatforms, RecompileCommand, nullptr, nullptr, &GlobalShaderMap);
		RecompileData.ODSCCustomLoadMaterial = &UE::Cook::FODSCClientData::FindMaterial;

		RecompileShadersForRemote(RecompileData, GetSandboxDirectory(PlatformName));
	}
}

namespace UE::Cook
{

FGuid FShaderLibraryCollector::MessageType(TEXT("4DF3B36BBA2F4E04A846E894E24EB2C4"));

void FShaderLibraryCollector::ClientTick(FMPCollectorClientTickContext& Context)
{
	constexpr int32 NumLoopsPerWarning = 100;
	for (int32 NumMessages = 0; ; ++NumMessages)
	{
		// Maximum size for a message is 1GB. Caller will crash if we go over that.
		// Provide a maximum shader size of half that because CopyToCompactBinaryAndClear adds on additional
		// small amounts of data beyond the shader size limit.
		const int64 MaximumSize = UE::CompactBinaryTCP::MaxOSPacketSize / 2;

		bool bHasData;
		bool bRanOutOfRoom;
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.SetName(RootObjectID);
		FShaderLibraryCooker::CopyToCompactBinaryAndClear(Writer, bHasData, bRanOutOfRoom, MaximumSize);
		if (bHasData)
		{
			Writer.EndObject();
			Context.AddMessage(Writer.Save().AsObject());
		}
		if (!bRanOutOfRoom)
		{
			break;
		}
		if (NumMessages > 0 && NumMessages % NumLoopsPerWarning == 0)
		{
			UE_LOGF(LogCook, Warning, "FShaderLibraryCollector::ClientTick has an unexpectedly high number of loops. Infinite loop?")
		}
	}
}

void FShaderLibraryCollector::ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message)
{
	bool bSuccessful = FShaderLibraryCooker::AppendFromCompactBinary(Message[RootObjectID]);
	UE_CLOGF(!bSuccessful, LogCook, Error,
		"Corrupt message received from CookWorker when replicating ShaderLibrary. Shaders will be missing from the cook.");
}

const TCHAR* TEXT_ShaderLibrary(TEXT("ShaderLibrary"));

FShaderLibraryCookArtifact::FShaderLibraryCookArtifact(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
{
}

FString FShaderLibraryCookArtifact::GetArtifactName() const
{
	return TEXT_ShaderLibrary;
}

void FShaderLibraryCookArtifact::OnFullRecook(const ITargetPlatform* TargetPlatform)
{
	CleanIntermediateFiles(TargetPlatform);
}

void FShaderLibraryCookArtifact::CleanIntermediateFiles(const ITargetPlatform* TargetPlatform)
{
	TArray<FName> ShaderFormats;
	TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
	if (ShaderFormats.Num() > 0)
	{
		FShaderLibraryCooker::CleanDirectories(ShaderFormats);
	}
}

void FShaderLibraryCookArtifact::UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
{
	FShaderLibraryCooker::UpdateOplogPackages(Context);
}

} // namespace UE::Cook

void UCookOnTheFlyServer::RegisterShaderLibraryIncrementalCookArtifact(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(RegisterShaderLibraryIncrementalCookArtifact);
	if (IsCookWorkerMode() || !IsUsingShaderCodeLibrary())
	{
		return;
	}
	RegisterArtifact(new UE::Cook::FShaderLibraryCookArtifact(*this));
}

void UCookOnTheFlyServer::BeginCookStartShaderCodeLibrary(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookStartShaderCodeLibrary);
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	if (IsUsingShaderCodeLibrary())
	{
		FShaderLibraryCooker::InitForCooking(PackagingSettings->bSharedMaterialNativeLibraries, AllContextArtifactReader.Get());

		bool bAllPlatformsNeedStableKeys = false;
		// support setting without Hungarian prefix for the compatibility, but allow newer one to override
		GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("NeedsShaderStableKeys"), bAllPlatformsNeedStableKeys, GEngineIni);
		GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bNeedsShaderStableKeys"), bAllPlatformsNeedStableKeys, GEngineIni);

		// See if we desire stable keys in DLC, PSO manual cache in DLC is not supported, so they are disabled by default
		// They have other uses so allow them to be optionally enabled
		bool bAllowShaderStableKeysInDLC = false;
		GConfig->GetBool(TEXT("DevOptions.Shaders"), TEXT("bAllowShaderStableKeysInDLC"), bAllowShaderStableKeysInDLC, GEngineIni);
		bAllPlatformsNeedStableKeys = bAllPlatformsNeedStableKeys && (!IsCookingDLC() || bAllowShaderStableKeysInDLC);

		for (const ITargetPlatform* TargetPlatform : BeginContext.TargetPlatforms)
		{
			// Find out if this platform requires stable shader keys, by reading the platform setting file.
			// Stable shader keys are needed if we are going to create a PSO cache.
			bool bNeedShaderStableKeys = bAllPlatformsNeedStableKeys;
			FConfigFile PlatformIniFile;
			FConfigCacheIni::LoadLocalIniFile(PlatformIniFile, TEXT("Engine"), true, *TargetPlatform->IniPlatformName());
			PlatformIniFile.GetBool(TEXT("DevOptions.Shaders"), TEXT("NeedsShaderStableKeys"), bNeedShaderStableKeys);
			PlatformIniFile.GetBool(TEXT("DevOptions.Shaders"), TEXT("bNeedsShaderStableKeys"), bNeedShaderStableKeys);

			bool bNeedsDeterministicOrder = PackagingSettings->bDeterministicShaderCodeOrder;
			FConfigFile PlatformGameIniFile;
			FConfigCacheIni::LoadLocalIniFile(PlatformGameIniFile, TEXT("Game"), true, *TargetPlatform->IniPlatformName());
			PlatformGameIniFile.GetBool(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("bDeterministicShaderCodeOrder"), bNeedsDeterministicOrder);

			TArray<FName> ShaderFormats;
			TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
			TArray<FShaderLibraryCooker::FShaderFormatDescriptor> ShaderFormatsWithStableKeys;
			for (FName& Format : ShaderFormats)
			{
				FShaderLibraryCooker::FShaderFormatDescriptor NewDesc;
				NewDesc.ShaderFormat = Format;
				NewDesc.bNeedsStableKeys = bNeedShaderStableKeys;
				NewDesc.bNeedsDeterministicOrder = bNeedsDeterministicOrder;
				ShaderFormatsWithStableKeys.Push(NewDesc);
			}

			if (ShaderFormats.Num() > 0)
			{
				FShaderLibraryCooker::CookShaderFormats(ShaderFormatsWithStableKeys);
			}
		}

		if (CookDirector)
		{
			CookDirector->Register(new UE::Cook::FShaderLibraryCollector());
		}
		else if (CookWorkerClient)
		{
			CookWorkerClient->Register(new UE::Cook::FShaderLibraryCollector());
		}
	}

	if (CookDirector)
	{
		CookDirector->Register(new FShaderStatsAggregator(FShaderStatsAggregator::EMode::Director));
	}
	else if (CookWorkerClient)
	{
		CookWorkerClient->Register(new FShaderStatsAggregator(FShaderStatsAggregator::EMode::Worker));
	}
}

void UCookOnTheFlyServer::BeginCookFinishShaderCodeLibrary(FBeginCookContext& BeginContext)
{
	UE_SCOPED_COOKTIMER(BeginCookFinishShaderCodeLibrary);
	check(IsDirectorCookByTheBook()); // CookByTheBook only for now

	// Generate the AutogenShaderHeaders.usd files for every shader platforms we are about to cook. Those files can be hashed and used
	// in dependencies so they need to be present. Only do this for the director.
	bool bIsCookWorker = IsCookWorkerMode();
	if (!bIsCookWorker)
	{
		GenerateAutogenShaderHeadersForAllShaderFormats(BeginContext.TargetPlatforms);
	}

	// don't resave the global shader map files in dlc
	if (!bIsCookWorker && !IsCookingDLC() && !EnumHasAnyFlags(CookByTheBookOptions->StartupOptions, ECookByTheBookOptions::ForceDisableSaveGlobalShaders))
	{
		OpenGlobalShaderLibrary();

		// make sure global shaders are up to date!
		SaveGlobalShaderMapFiles(BeginContext.TargetPlatforms, ODSCRecompileCommand::Changed);

		SaveAndCloseGlobalShaderLibrary();
	}

	// Open the shader code library for the current project or the current DLC pack, depending on which we are cooking
	FString LibraryName = GetProjectShaderLibraryName();
	check(!LibraryName.IsEmpty());
	OpenShaderLibrary(LibraryName);
}

void UCookOnTheFlyServer::RegisterShaderChunkDataGenerator()
{
	check(!IsCookWorkerMode());
	// add shader library and PSO cache chunkers
	FString LibraryName = GetProjectShaderLibraryName();
	for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
	{
		FAssetRegistryGenerator& RegistryGenerator = *(PlatformManager->GetPlatformData(TargetPlatform)->RegistryGenerator);
		RegistryGenerator.RegisterChunkDataGenerator(MakeShared<FShaderLibraryChunkDataGenerator>(*this, TargetPlatform));
		RegistryGenerator.RegisterChunkDataGenerator(MakeShared<FPipelineCacheChunkDataGenerator>(TargetPlatform, LibraryName));
	}
}

FString UCookOnTheFlyServer::GetProjectShaderLibraryName() const
{
	static FString ShaderLibraryName = [this]()
		{
			FString OverrideShaderLibraryName;
			if (FParse::Value(FCommandLine::Get(), TEXT("OverrideShaderLibraryName="), OverrideShaderLibraryName))
			{
				return OverrideShaderLibraryName;
			}

			if (!IsCookingDLC())
			{
				FString Result = FApp::GetProjectName();
				if (Result.IsEmpty())
				{
					Result = TEXT("UnrealGame");
				}
				return Result;
			}
			else
			{
				return CookByTheBookOptions->DlcName;
			}
		}();
	return ShaderLibraryName;
}

static FString GenerateShaderCodeLibraryName(const FString& Name, bool bIsLegacyIterativeSharedBuild)
{
	FString ActualName = (!bIsLegacyIterativeSharedBuild) ? Name : Name + TEXT("_SC");
	return ActualName;
}

void UCookOnTheFlyServer::OpenGlobalShaderLibrary()
{
	const bool bCacheShaderLibraries = IsUsingShaderCodeLibrary();
	if (bCacheShaderLibraries)
	{
		const TCHAR* GlobalShaderLibName = TEXT("Global");
		FString ActualName = GenerateShaderCodeLibraryName(GlobalShaderLibName, IsCookFlagSet(ECookInitializationFlags::LegacyIterativeSharedBuild));

		// The shader code library directory doesn't matter while cooking
		FShaderLibraryCooker::BeginCookingLibrary(ActualName);
	}
}

void UCookOnTheFlyServer::OpenShaderLibrary(FString const& Name)
{
	const bool bCacheShaderLibraries = IsUsingShaderCodeLibrary();
	if (bCacheShaderLibraries)
	{
		FString ActualName = GenerateShaderCodeLibraryName(Name, IsCookFlagSet(ECookInitializationFlags::LegacyIterativeSharedBuild));

		// The shader code library directory doesn't matter while cooking
		FShaderLibraryCooker::BeginCookingLibrary(ActualName);
	}
}

void UCookOnTheFlyServer::CreatePipelineCache(const ITargetPlatform* TargetPlatform, const FString& LibraryName)
{
	// make sure we have a registry generated for all the platforms 
	const FString TargetPlatformName = TargetPlatform->PlatformName();
	TArray<FString>* SCLCSVPaths = OutSCLCSVPaths.Find(FName(TargetPlatformName));
	if (SCLCSVPaths && SCLCSVPaths->Num())
	{
		TArray<FName> ShaderFormats;
		TargetPlatform->GetAllTargetedShaderFormats(ShaderFormats);
		for (FName ShaderFormat : ShaderFormats)
		{
			const FString StablePCDir = FPaths::ProjectDir() / TEXT("Build") / TargetPlatform->IniPlatformName() / TEXT("PipelineCaches");
			// look for the new binary format for stable pipeline cache - spc
			const FString StablePCBinary = StablePCDir / FString::Printf(TEXT("*%s_%s.spc"), *LibraryName, *ShaderFormat.ToString());

			bool bBinaryStablePipelineCacheFilesFound = [&StablePCBinary]()
				{
					TArray<FString> ExpandedFiles;
					IFileManager::Get().FindFilesRecursive(ExpandedFiles, *FPaths::GetPath(StablePCBinary), *FPaths::GetCleanFilename(StablePCBinary), true, false, false);
					return ExpandedFiles.Num() > 0;
				}();

			// for now, also look for the older *stablepc.csv or *stablepc.csv.compressed
			const FString StablePCTextual = StablePCDir / FString::Printf(TEXT("*%s_%s.stablepc.csv"), *LibraryName, *ShaderFormat.ToString());
			const FString StablePCTextualCompressed = StablePCTextual + TEXT(".compressed");

			bool bTextualStablePipelineCacheFilesFound = [&StablePCTextual, &StablePCTextualCompressed]()
				{
					TArray<FString> ExpandedFiles;
					IFileManager::Get().FindFilesRecursive(ExpandedFiles, *FPaths::GetPath(StablePCTextual), *FPaths::GetCleanFilename(StablePCTextual), true, false, false);
					IFileManager::Get().FindFilesRecursive(ExpandedFiles, *FPaths::GetPath(StablePCTextualCompressed), *FPaths::GetCleanFilename(StablePCTextualCompressed), true, false, false);
					return ExpandedFiles.Num() > 0;
				}();

			// because of the compute shaders that are cached directly from stable shader keys files, we need to run this also if we have stable keys (which is pretty much always)
			static const IConsoleVariable* CVarIncludeComputePSOsDuringCook = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ShaderPipelineCacheTools.IncludeComputePSODuringCook"));
			const bool bIncludeComputePSOsDuringCook = CVarIncludeComputePSOsDuringCook && CVarIncludeComputePSOsDuringCook->GetInt() >= 1;
			if (!bBinaryStablePipelineCacheFilesFound && !bTextualStablePipelineCacheFilesFound && !bIncludeComputePSOsDuringCook)
			{
				UE_LOGF(LogCook, Display, "---- NOT Running UShaderPipelineCacheToolsCommandlet for platform %ls  shader format %ls, no files found at %ls, and either no stable keys or not including compute PSOs during the cook", *TargetPlatformName, *ShaderFormat.ToString(), *StablePCDir);
			}
			else
			{
				UE_LOGF(LogCook, Display, "---- Running UShaderPipelineCacheToolsCommandlet for platform %ls  shader format %ls", *TargetPlatformName, *ShaderFormat.ToString());

				const FString OutFilename = FString::Printf(TEXT("%s_%s.stable.upipelinecache"), *LibraryName, *ShaderFormat.ToString());
				const FString PCUncookedPath = FPaths::ProjectDir() / TEXT("Content") / TEXT("PipelineCaches") / TargetPlatform->IniPlatformName() / OutFilename;

				if (IFileManager::Get().FileExists(*PCUncookedPath))
				{
					UE_LOGF(LogCook, Warning, "Deleting %ls, cooked data doesn't belong here.", *PCUncookedPath);
					IFileManager::Get().Delete(*PCUncookedPath, false, true);
				}

				const FString PCCookedPath = ConvertToFullSandboxPath(*PCUncookedPath, true);
				const FString PCPath = PCCookedPath.Replace(TEXT("[Platform]"), *TargetPlatformName);


				FString Args(TEXT("build "));
				if (bBinaryStablePipelineCacheFilesFound)
				{
					Args += TEXT("\"");
					Args += StablePCBinary;
					Args += TEXT("\" ");
				}
				if (bTextualStablePipelineCacheFilesFound)
				{
					Args += TEXT("\"");
					Args += StablePCTextual;
					Args += TEXT("\" ");
				}

				int32 NumMatched = 0;
				for (int32 Index = 0; Index < SCLCSVPaths->Num(); Index++)
				{
					if (!(*SCLCSVPaths)[Index].Contains(ShaderFormat.ToString()))
					{
						continue;
					}
					NumMatched++;
					Args += TEXT(" ");
					Args += TEXT("\"");
					Args += (*SCLCSVPaths)[Index];
					Args += TEXT("\"");
				}
				if (!NumMatched)
				{
					UE_LOGF(LogCook, Warning, "Shader format %ls for platform %ls had stable pipeline cache files, but no stable keys files.", *ShaderFormat.ToString(), *TargetPlatformName);
					for (int32 Index = 0; Index < SCLCSVPaths->Num(); Index++)
					{
						UE_LOGF(LogCook, Warning, "    stable keys file: %ls", *((*SCLCSVPaths)[Index]));
					}
					continue;
				}

				Args += TEXT(" -chunkinfodir=\"");
				Args += ConvertToFullSandboxPath(FPaths::ProjectDir() / TEXT("Content"), true).Replace(TEXT("[Platform]"), *TargetPlatformName);
				Args += TEXT("\" ");
				Args += TEXT(" -library=");
				Args += LibraryName;
				Args += TEXT(" ");
				Args += TEXT(" -platform=");
				Args += TargetPlatformName;
				Args += TEXT(" ");
				Args += TEXT("\"");
				Args += PCPath;
				Args += TEXT("\"");
				UE_LOGF(LogCook, Display, "  With Args: %ls", *Args);

				int32 Result = UShaderPipelineCacheToolsCommandlet::StaticMain(Args);

				if (Result)
				{
					LogCookerMessage(FString::Printf(TEXT("UShaderPipelineCacheToolsCommandlet failed %d"), Result), EMessageSeverity::Error);
				}
				else
				{
					UE_LOGF(LogCook, Display, "---- Done running UShaderPipelineCacheToolsCommandlet for platform %ls", *TargetPlatformName);

					// copy the resulting file to metadata for easier examination later
					if (IFileManager::Get().FileExists(*PCPath))
					{
						const FString RootPipelineCacheMetadataPath = GetMetadataDirectory() / TEXT("PipelineCaches");
						const FString PipelineCacheMetadataPathSB = ConvertToFullSandboxPath(*RootPipelineCacheMetadataPath, true);
						const FString PipelineCacheMetadataPath = PipelineCacheMetadataPathSB.Replace(TEXT("[Platform]"), *TargetPlatform->PlatformName());
						const FString PipelineCacheMetadataFileName = PipelineCacheMetadataPath / OutFilename;

						UE_LOGF(LogCook, Display, "Copying the binary PSO cache file %ls to %ls.", *PCPath, *PipelineCacheMetadataFileName);
						if (IFileManager::Get().Copy(*PipelineCacheMetadataFileName, *PCPath) != COPY_OK)
						{
							UE_LOGF(LogCook, Warning, "Failed to copy the binary PSO cache file %ls to %ls.", *PCPath, *PipelineCacheMetadataFileName);
						}
					}
				}
			}
		}
	}
}

void UCookOnTheFlyServer::SaveAndCloseGlobalShaderLibrary()
{
	const bool bCacheShaderLibraries = IsUsingShaderCodeLibrary();
	if (bCacheShaderLibraries)
	{
		const TCHAR* GlobalShaderLibName = TEXT("Global");
		FString ActualName = GenerateShaderCodeLibraryName(GlobalShaderLibName, IsCookFlagSet(ECookInitializationFlags::LegacyIterativeSharedBuild));

		// Save shader code map - cleaning directories is deliberately a separate loop here as we open the cache once per shader platform and we don't assume that they can't be shared across target platforms.
		for (const ITargetPlatform* TargetPlatform : PlatformManager->GetSessionPlatforms())
		{
			FinishPopulateShaderLibrary(TargetPlatform, GlobalShaderLibName, false /* bInitializeReferenceTracking */);
			SaveShaderLibrary(TargetPlatform, GlobalShaderLibName);
		}

		FShaderLibraryCooker::EndCookingLibrary(ActualName);
	}
}

void UCookOnTheFlyServer::GetShaderLibraryPaths(const ITargetPlatform* TargetPlatform,
	FString& OutShaderCodeDir, FString& OutMetaDataPath, bool bUseProjectDirForDLC)
{
	// TODO: Saving ShaderChunks into the DLC directory currently does not work, so we have the bUseProjectDirForDLC arg to save to Project
	const FString BasePath = (!IsCookingDLC() || bUseProjectDirForDLC) ? FPaths::ProjectContentDir() : GetContentDirectoryForDLC();
	OutShaderCodeDir = ConvertToFullSandboxPath(*BasePath, true, TargetPlatform->PlatformName());

	const FString RootMetaDataPath = GetMetadataDirectory() / TEXT("PipelineCaches");
	OutMetaDataPath = ConvertToFullSandboxPath(*RootMetaDataPath, true, *TargetPlatform->PlatformName());
}

void UCookOnTheFlyServer::FinishPopulateShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name, bool bInitializeReferenceTracking)
{
	FString ShaderCodeDir;
	FString MetaDataPath;
	GetShaderLibraryPaths(TargetPlatform, ShaderCodeDir, MetaDataPath);

	FShaderLibraryCooker::FinishPopulateShaderLibrary(TargetPlatform, Name, ShaderCodeDir, MetaDataPath, bInitializeReferenceTracking);
}

void UCookOnTheFlyServer::PruneShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name)
{
	FShaderLibraryCooker::PruneShaderLibrary(TargetPlatform, Name);
}

void UCookOnTheFlyServer::SaveShaderLibrary(const ITargetPlatform* TargetPlatform, const FString& Name)
{
	FString ShaderCodeDir;
	FString MetaDataPath;
	GetShaderLibraryPaths(TargetPlatform, ShaderCodeDir, MetaDataPath);

	TArray<FString>& PlatformSCLCSVPaths = OutSCLCSVPaths.FindOrAdd(FName(TargetPlatform->PlatformName()));
	FString ErrorString;
	bool bHasData;
	UE::Cook::EPackagingSystem PackagingSystem = IsUsingIoStore() ? UE::Cook::EPackagingSystem::IoStore
		: UE::Cook::EPackagingSystem::Loose;
	if (!FShaderLibraryCooker::SaveShaderLibraryWithoutChunking(TargetPlatform, Name, ShaderCodeDir, MetaDataPath,
		PlatformSCLCSVPaths, PackagingSystem, ErrorString, bHasData))
	{
		// This is fatal - In this case we should cancel any launch on device operation or package write but we don't want to assert and crash the editor
		LogCookerMessage(FString::Printf(TEXT("%s"), *ErrorString), EMessageSeverity::Error);
	}
	else if (bHasData)
	{
		for (const FString& Item : PlatformSCLCSVPaths)
		{
			UE_LOGF(LogCook, Display, "Saved scl.csv %ls for platform %ls, %lld bytes", *Item, *TargetPlatform->PlatformName(),
				IFileManager::Get().FileSize(*Item));
		}
	}
}

void UCookOnTheFlyServer::ShutdownShaderLibraryCookerAndCompilers(const FString& ProjectShaderLibraryName)
{
	FString ActualLibraryName = GenerateShaderCodeLibraryName(ProjectShaderLibraryName,
		IsCookFlagSet(ECookInitializationFlags::LegacyIterativeSharedBuild));
	FShaderLibraryCooker::EndCookingLibrary(ActualLibraryName);
	FShaderLibraryCooker::Shutdown();
	ShutdownShaderCompilers(PlatformManager->GetSessionPlatforms());
}

void UCookOnTheFlyServer::DumpShaderTypeStats(const ITargetPlatform* TargetPlatform, const FString& PlatformNameString)
{
	using namespace UE::Cook;

	auto GetCookStatus = [this, TargetPlatform]
	(FName AssetName, bool& bOutImplemented, bool& bOutReferenced, bool& bOutRecooked)
		{
			bOutImplemented = true;
			bOutReferenced = false;
			bOutRecooked = false;

			FPackageData* PackageData = PackageDatas->FindPackageDataByPackageName(AssetName);
			if (!PackageData)
			{
				return;
			}
			FPackagePlatformData& PlatformData = PackageData->FindOrAddPlatformData(TargetPlatform);
			bOutReferenced = PlatformData.IsCookSucceeded();
			bOutRecooked = !PlatformData.IsIncrementallySkipped();
		};
	FShaderLibraryCooker::DumpShaderTypeStats(
		GShaderCompilingManager->GetAbsoluteShaderDebugInfoDirectory(),
		ConvertToFullSandboxPath(GetMetadataDirectory(), true, PlatformNameString),
		GetCookStatus);
}

bool UCookOnTheFlyServer::RecompileChangedShaders(const TArray<const ITargetPlatform*>& TargetPlatforms)
{
	bool bShadersRecompiled = false;
	for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		bShadersRecompiled |= RecompileChangedShadersForPlatform(TargetPlatform->PlatformName());
	}
	return bShadersRecompiled;
}