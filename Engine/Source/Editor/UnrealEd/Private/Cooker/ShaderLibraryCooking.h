// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/CookArtifact.h"
#include "Cooker/MPCollector.h"
#include "CoreTypes.h"
#include "UObject/NameTypes.h"

class ITargetPlatform;
class ITargetPlatformManagerModule;
namespace UE::Cook { struct FTickStackData; }


extern bool bDisableShaderCompilationDuringCookOnTheFly;
extern bool bAllowIncompleteShaderMapsDuringCookOnTheFly;

namespace UE::Cook
{

void TickShaderCompilingManager(UE::Cook::FTickStackData& StackData);

/** CookMultiprocess collector for ShaderLibrary data. */
class FShaderLibraryCollector : public IMPCollector
{
public:
	virtual FGuid GetMessageType() const override;
	virtual const TCHAR* GetDebugName() const override;

	virtual void ClientTick(FMPCollectorClientTickContext& Context) override;
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override;

	static FGuid MessageType;
	static constexpr ANSICHAR RootObjectID[2]{ "S" };
};

/** CookArtifact to manage the shaderlibraries in the cook's OutputFolder. */
class FShaderLibraryCookArtifact : public ICookArtifact
{
public:
	FShaderLibraryCookArtifact(UCookOnTheFlyServer& InCOTFS);
	virtual FString GetArtifactName() const override;
	virtual void OnFullRecook(const ITargetPlatform* TargetPlatform) override;
	virtual void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context) override;

	void CleanIntermediateFiles(const ITargetPlatform* TargetPlatform);

private:
	UCookOnTheFlyServer& COTFS;
};

} // namespace UE::Cook

namespace UE::Cook::CVarControl
{

void UpdateShaderCookingCVars(ITargetPlatformManagerModule* TPM, int32 CookTimeCVarControl,
	const ITargetPlatform* Platform, FName PlatformName);

} // namespace UE::Cook::CVarControl


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

namespace UE::Cook
{

inline FGuid FShaderLibraryCollector::GetMessageType() const
{
	return MessageType;
}

inline const TCHAR* FShaderLibraryCollector::GetDebugName() const
{
	return TEXT("FShaderLibraryCollector");
}

} // namespace UE::Cook