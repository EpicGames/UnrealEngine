// Copyright Epic Games, Inc. All Rights Reserved.

#include "StallDetector.h"

namespace UE::Cook
{
	// Number of iteration without progress after which the cook is considered stalled for the number of cooked packages.
	const static int32 StalledIterationLimitPackageCooked = 10;

	// Number of iteration without progress after which the cook is considered stalled for the number of in progress packages.
	const static int32 StalledIterationLimitPackageInProgress = 10;

	bool FStallDetector::IsStalled(int32 NewPackageCooked, int32 NewPackageInProgress)
	{
		// First check if the number of cooked package is stalled
		StalledPackageCooked.Update(NewPackageCooked);
		if (StalledPackageCooked.StalledIterationCount < UE::Cook::StalledIterationLimitPackageCooked)
		{
			return false;
		}

		// Now check if the number of package in progress is stalled
		StalledPackageInProgress.Update(NewPackageInProgress);

		// If the number of package in progress is zero, then there is no cook requested so it is not a stall.
		if (StalledPackageInProgress.StalledIterationCount < UE::Cook::StalledIterationLimitPackageInProgress || StalledPackageInProgress.Value == 0)
		{
			return false;
		}

		// No progress so the cook is stalled
		return true;
	}
}
