// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StalledIterationCounter.h"

namespace UE::Cook
{

	/**
	 * Class used to detect when the cook is not making progress. It does it by keeping track of the number
	 * of cooked packages and packages in progress.
	 */
	class FStallDetector
	{
	public:
		bool IsStalled(int32 NewPackageCooked, int32 NewPackageInProgress);

	private:
		FStalledIterationCounter StalledPackageCooked;
		FStalledIterationCounter StalledPackageInProgress;
	};
}
