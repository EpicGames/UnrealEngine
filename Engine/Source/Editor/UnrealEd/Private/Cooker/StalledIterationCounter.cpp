// Copyright Epic Games, Inc. All Rights Reserved.

#include "StalledIterationCounter.h"

namespace UE::Cook
{

	FStalledIterationCounter::FStalledIterationCounter()
		: Value(0)
		, StalledIterationCount(0)
	{}

	void FStalledIterationCounter::Update(int32 NewValue)
	{
		if (Value != NewValue)
		{
			Value = NewValue;
			StalledIterationCount = 0;
		}

		++StalledIterationCount;
	}
}
