// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Cook
{

	/**
	 * Small structure used to keep track of a value and for how many iteration the value didn't change. It's used
	 * more specifically in the cooker to detect when the cooker is stalling and no progress is being done.
	 */
	struct FStalledIterationCounter
	{
		FStalledIterationCounter();
		void Update(int32 NewValue);

		int32 Value;
		int32 StalledIterationCount;
	};
}
