// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

/**
 * An allocator that batches allocation calls into blocks to reduce malloc costs.
 * 
 * Not threadsafe; must be guarded by critical section if used from multiple threads.
 * 
 * Block size is adaptive; it doubles with each new block, up to MaxBlockSize.
 * 
 * Allocated memory is not released until Empty is called.
 * 
 * This is the base version and does not keep a free list. Freed elements will never be
 * reused and will only be freed by Empty() or destructor.
 * In this base class, destructors for allocated elements are not automatically called;
 * users of the allocator must call destructor manually or must call DeleteElement on each
 * element they called NewElement on.
 */
template <typename ElementType>
class TTypedBlockAllocatorBase
{
public:
	/** Set the MinimumBlockSize. Default is 1024. All new blocks will be this size or larger. */
	void SetMinBlockSize(int32 BlockSize);
	int32 GetMinBlockSize() const;
	/**
	 * Set the MaximumBlockSize. Default is 65536. New blocks added by Alloc will be this size or
	 * less (ignored if MaximumBlockSize < MinimumBlockSize). Blocks added by Reserve are not
	 * limited by MaximumBlockSize. 
	 */
	void SetMaxBlockSize(int32 BlockSize);
	int32 GetMaxBlockSize() const;

	/** Return the memory for an Element without calling a constructor. */
	ElementType* Alloc();
	/**
	 * Make the memory for an Element returned from Alloc or Construct available again to Alloc.
	 * Does not call the ElementType destructor. Does not return the memory to the inner allocator.
	 */
	void Free(ElementType* Element);
	/** Allocate an Element and call constructor with the given arguments. */
	template <typename... ArgsType>
	ElementType* NewElement(ArgsType&&... Args);
	/** Call destructor on the given element and call Free on its memory. */
	void DeleteElement(ElementType* Element);

	/**
	 * Call the given Callback on every element that has been returned from Alloc. Not valid to call if any
	 * elements have been Freed because it is too expensive to prevent calling the Callback on the freed elements.
	 */
	template <typename CallbackType>
	void EnumerateAllocations(CallbackType&& Callback);

	/**
	 * Allocate enough memory from the inner allocator to ensure that AllocationCount more calls to Alloc can be made
	 * without further calls to the inner allocator.
	 * 
	 * @param AllocationCount The number of future calls to Alloc that should reserved.
	 * @param InMaxBlockSize If non-zero, the maximum capacity of any Blocks allocated by ReserveDelta.
	 *        If zero, MaxBlockSize is used.
	 */
	void ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize = 0);
	/**
	 * Release all allocated memory. For performance, Empty does not require that allocations have been destructed
	 * or freed; caller is responsible for calling any necessary ElementType destructors and for dropping all
	 * references to the allocated ElementTypes.
	 */
	void Empty();

protected:
	struct FAllocationBlock
	{
		FAllocationBlock(int32 InCapacity);

		TUniquePtr<TTypeCompatibleBytes<ElementType>[]> Elements;
		int32 NextIndex;
		int32 Capacity;
	};

	ElementType* BaseAlloc();
	template <typename CallbackType>
	void BaseEnumerateAllocations(CallbackType&& Callback);
	void BaseReserveDelta(int32 AllocationCount, int32 InMaxBlockSize, int32 NumAvailableOnFreeList);
	void BaseEmpty();
	void CallDestructorInternal(ElementType* Element);

protected:
	TArray<FAllocationBlock> Blocks;
	int32 NextBlock = 0;
	int32 NumAllocations = 0;
	int32 NumFreed = 0;
	int32 MinBlockSize = 1024;
	int32 MaxBlockSize = 65536;
};

/**
 * A TTypedAllocatorBase with a freelist. Freed elements are destructed, but their memory
 * is reused and the constructor called again on it in future allocation.
 *
 * In this subclass, destructors for allocated elements are not automatically called;
 * users of the allocator must call destructor manually or must call DeleteElement on each
 * element they called NewElement on.
 */
template <typename ElementType>
class TTypedBlockAllocatorFreeList : public TTypedBlockAllocatorBase<ElementType>
{
public:
	ElementType* Alloc();
	void Free(ElementType* Element);
	template <typename... ArgsType> ElementType* NewElement(ArgsType&&... Args);
	void DeleteElement(ElementType* Element);
	void ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize = 0);
	void Empty();

protected:
	using TTypedBlockAllocatorBase<ElementType>::BaseAlloc;
	using TTypedBlockAllocatorBase<ElementType>::BaseEnumerateAllocations;
	using TTypedBlockAllocatorBase<ElementType>::BaseReserveDelta;
	using TTypedBlockAllocatorBase<ElementType>::BaseEmpty;
	using TTypedBlockAllocatorBase<ElementType>::CallDestructorInternal;
	using TTypedBlockAllocatorBase<ElementType>::NumAllocations;
	using TTypedBlockAllocatorBase<ElementType>::NumFreed;

protected:
	ElementType* FreeList = nullptr;
};

/**
 * A TTypedAllocatorBase with a resetlist. Freed elements are not destructed until empty is called. When NewElement is
 * called and the reset list is available, a previously-freed element is returned without having had the destructor and
 * a second call to the constructor called on it.
 * 
 * Destructors for elements added via NewElement are called automatically from Empty or from the allocator's
 * destructor.
 */
template <typename ElementType>
class TTypedBlockAllocatorResetList : public TTypedBlockAllocatorBase<ElementType>
{
public:
	~TTypedBlockAllocatorResetList();

	void Free(ElementType* Element);
	template <typename... ArgsType> ElementType* NewElement(ArgsType&&... Args);
	void ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize = 0);
	void Empty();

protected:
	ElementType* Alloc(); // Not public on this subclass; every allocation must have constructor called
	using TTypedBlockAllocatorBase<ElementType>::DeleteElement; // Not supported on this subclass
	void ReallocFreeList(int32 NewCapacity);

protected:
	using TTypedBlockAllocatorBase<ElementType>::BaseAlloc;
	using TTypedBlockAllocatorBase<ElementType>::BaseEnumerateAllocations;
	using TTypedBlockAllocatorBase<ElementType>::BaseReserveDelta;
	using TTypedBlockAllocatorBase<ElementType>::BaseEmpty;
	using TTypedBlockAllocatorBase<ElementType>::CallDestructorInternal;
	using TTypedBlockAllocatorBase<ElementType>::NumAllocations;
	using TTypedBlockAllocatorBase<ElementType>::NumFreed;

protected:
	TUniquePtr<ElementType* []> FreeList;
	int32 CapacityFreed = 0;
};


template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::SetMinBlockSize(int32 BlockSize)
{
	check(BlockSize >= 0);
	MinBlockSize = FMath::Max(BlockSize, 1);
}

template <typename ElementType>
int32 TTypedBlockAllocatorBase<ElementType>::GetMinBlockSize() const
{
	return MinBlockSize;
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::SetMaxBlockSize(int32 BlockSize)
{
	check(BlockSize >= 0);
	MaxBlockSize = BlockSize;
}

template <typename ElementType>
int32 TTypedBlockAllocatorBase<ElementType>::GetMaxBlockSize() const
{
	return MaxBlockSize;
}

template <typename ElementType>
ElementType* TTypedBlockAllocatorBase<ElementType>::Alloc()
{
	return BaseAlloc();
}

template <typename ElementType>
ElementType* TTypedBlockAllocatorBase<ElementType>::BaseAlloc()
{
	ElementType* Result = nullptr;
	FAllocationBlock* Block = nullptr;
	for (; NextBlock < Blocks.Num(); ++NextBlock)
	{
		if (Blocks[NextBlock].NextIndex < Blocks[NextBlock].Capacity)
		{
			Block = &Blocks[NextBlock];
			break;
		}
	}

	if (!Block)
	{
		check(NextBlock == Blocks.Num());
		// Double our NumAllocations with each new block, up to a maximum block size
		int32 BlockCapacity = FMath::Max(NumAllocations, MinBlockSize);
		if (MaxBlockSize > MinBlockSize)
		{
			BlockCapacity = FMath::Min(BlockCapacity, MaxBlockSize);
		}
		Block = &Blocks.Emplace_GetRef(BlockCapacity);
	}

	Result = Block->Elements[Block->NextIndex].GetTypedPtr();
	++Block->NextIndex;
	++NumAllocations;

	return Result;
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::Free(ElementType* Element)
{
	check(NumAllocations > 0);
	--NumAllocations;
	++NumFreed;
	// The Element will remain in memory, in its block, until Empty is called.
}

template <typename ElementType>
template <typename... ArgsType>
ElementType* TTypedBlockAllocatorBase<ElementType>::NewElement(ArgsType&&... Args)
{
	return new(Alloc()) ElementType(Forward<ArgsType>(Args)...);
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::DeleteElement(ElementType* Element)
{
	CallDestructorInternal(Element);
	Free(Element);
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::CallDestructorInternal(ElementType* Element)
{
	// We need a typedef here because VC won't compile the destructor call below if ElementType itself has a member
	// called ElementType
	typedef ElementType TTypedBlockAllocatorDestructElementType;
	((TTypedBlockAllocatorDestructElementType*)Element)->
		TTypedBlockAllocatorDestructElementType::~TTypedBlockAllocatorDestructElementType();
}

template <typename ElementType>
template <typename CallbackType>
void TTypedBlockAllocatorBase<ElementType>::EnumerateAllocations(CallbackType&& Callback)
{
	checkf(NumFreed == 0, TEXT("It is invalid to call EnumerateAllocations after calling Free or Destruct."));
	BaseEnumerateAllocations(Forward<CallbackType>(Callback));
}

template <typename ElementType>
template <typename CallbackType>
void TTypedBlockAllocatorBase<ElementType>::BaseEnumerateAllocations(CallbackType&& Callback)
{
	for (const FAllocationBlock& Block : Blocks)
	{
		for (TTypeCompatibleBytes<ElementType>& Bytes :
			TArrayView<TTypeCompatibleBytes<ElementType>>(Block.Elements.Get(), Block.NextIndex))
		{
			ElementType* Element = Bytes.GetTypedPtr();
			Callback(Element);
		}
	}
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize)
{
	BaseReserveDelta(AllocationCount, InMaxBlockSize, 0);
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::BaseReserveDelta(int32 AllocationCount,
	int32 InMaxBlockSize, int32 NumAvailableOnFreeList)
{
	int32 DeltaAllocationCount = AllocationCount - NumAvailableOnFreeList;
	for (const FAllocationBlock& Block : TConstArrayView<FAllocationBlock>(Blocks).RightChop(NextBlock))
	{
		DeltaAllocationCount -= Block.Capacity - Block.NextIndex;
	}
	if (InMaxBlockSize == 0)
	{
		InMaxBlockSize = MaxBlockSize;
	}
	if (DeltaAllocationCount <= 0)
	{
		return;
	}

	// Allocate blocks until we have enough capacity to cover the Reservation. As with blocks allocated from
	// Alloc, set the unclamped capacity high enough to double our number of allocations. But also set it high
	// enough to cover the remaining count, and use the different MaxBlockSize if passed in.
	int32 BlockCapacity = FMath::Max(DeltaAllocationCount, NumAllocations);
	BlockCapacity = FMath::Max(BlockCapacity, MinBlockSize);
	if (InMaxBlockSize > MinBlockSize)
	{
		BlockCapacity = FMath::Min(BlockCapacity, InMaxBlockSize);
	}
	Blocks.Emplace(BlockCapacity);
	DeltaAllocationCount -= BlockCapacity;

	if (DeltaAllocationCount > 0)
	{
		// For further blocks, use the MaxBlockSize
		check(InMaxBlockSize > MinBlockSize); // Otherwise we would have allocated all of DeltaAllocationCount in a single block
		BlockCapacity = InMaxBlockSize;
		while (DeltaAllocationCount > 0)
		{
			Blocks.Emplace(BlockCapacity);
			DeltaAllocationCount -= BlockCapacity;
		}
	}
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::Empty()
{
	BaseEmpty();
}

template <typename ElementType>
void TTypedBlockAllocatorBase<ElementType>::BaseEmpty()
{
	Blocks.Empty();
	NextBlock = 0;
	NumAllocations = 0;
	NumFreed = 0;
}

template <typename ElementType>
TTypedBlockAllocatorBase<ElementType>::FAllocationBlock::FAllocationBlock(int32 InCapacity)
	: Elements(InCapacity > 0 ? new TTypeCompatibleBytes<ElementType>[InCapacity] : nullptr)
	, NextIndex(0)
	, Capacity(InCapacity)
{
}

template <typename ElementType>
ElementType* TTypedBlockAllocatorFreeList<ElementType>::Alloc()
{
	ElementType* Result = nullptr;
	if (NumFreed > 0)
	{
		check(FreeList);
		Result = FreeList;
		FreeList = *reinterpret_cast<ElementType**>(FreeList);
		++NumAllocations;
		--NumFreed;
		return Result;
	}
	return BaseAlloc();
}

template <typename ElementType>
void TTypedBlockAllocatorFreeList<ElementType>::Free(ElementType* Element)
{
	if constexpr (sizeof(ElementType) <= sizeof(ElementType*))
	{
		static_assert(sizeof(ElementType) >= sizeof(ElementType*),
			"The FreeList is implemented by storing pointers within each freed element. To use the FreeList, elementsize must be >= pointer size.");
	}
	check(NumAllocations > 0);

	*reinterpret_cast<ElementType**>(Element) = FreeList;
	FreeList = Element;
	--NumAllocations;
	++NumFreed;
}

template <typename ElementType>
template <typename... ArgsType>
ElementType* TTypedBlockAllocatorFreeList<ElementType>::NewElement(ArgsType&&... Args)
{
	return new(Alloc()) ElementType(Forward<ArgsType>(Args)...);
}

template <typename ElementType>
void TTypedBlockAllocatorFreeList<ElementType>::DeleteElement(ElementType* Element)
{
	CallDestructorInternal(Element);
	Free(Element);
}

template <typename ElementType>
void TTypedBlockAllocatorFreeList<ElementType>::ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize)
{
	BaseReserveDelta(AllocationCount, InMaxBlockSize, NumFreed);
}

template <typename ElementType>
void TTypedBlockAllocatorFreeList<ElementType>::Empty()
{
	BaseEmpty();
	FreeList = nullptr;
}


template <typename ElementType>
TTypedBlockAllocatorResetList<ElementType>::~TTypedBlockAllocatorResetList()
{
	Empty();
}

template <typename ElementType>
ElementType* TTypedBlockAllocatorResetList<ElementType>::Alloc()
{
	ElementType* Result = nullptr;
	if (NumFreed > 0)
	{
		check(FreeList);
		Result = FreeList[--NumFreed];
		if (8 < NumFreed && NumFreed*2 < CapacityFreed)
		{
			ReallocFreeList(NumFreed * 3 / 2);
		}
		++NumAllocations;
		return Result;
	}
	return BaseAlloc();
}

template <typename ElementType>
void TTypedBlockAllocatorResetList<ElementType>::ReallocFreeList(int32 NewCapacity)
{
	TUniquePtr<ElementType* []> Copy = MakeUnique<ElementType* []>(NewCapacity);
	FMemory::Memcpy(Copy.Get(), FreeList.Get(), NumFreed * sizeof(ElementType*));
	FreeList = MoveTemp(Copy);
	CapacityFreed = NewCapacity;
}

template <typename ElementType>
void TTypedBlockAllocatorResetList<ElementType>::Free(ElementType* Element)
{
	check(NumAllocations > 0);
	if (CapacityFreed == NumFreed)
	{
		ReallocFreeList(NumFreed < 8 ? 8 : NumFreed * 3 / 2);
	}
	FreeList[NumFreed++] = Element;
	--NumAllocations;
}

template <typename ElementType>
template <typename... ArgsType>
ElementType* TTypedBlockAllocatorResetList<ElementType>::NewElement(ArgsType&&... Args)
{
	return new(Alloc()) ElementType(Forward<ArgsType>(Args)...);
}

template <typename ElementType>
void TTypedBlockAllocatorResetList<ElementType>::ReserveDelta(int32 AllocationCount, int32 InMaxBlockSize)
{
	BaseReserveDelta(AllocationCount, InMaxBlockSize, NumFreed);
}

template <typename ElementType>
void TTypedBlockAllocatorResetList<ElementType>::Empty()
{
	BaseEnumerateAllocations([this](ElementType* Element)
		{
			CallDestructorInternal(Element);
		});
	BaseEmpty();
	FreeList.Reset();
	CapacityFreed = 0;
}
