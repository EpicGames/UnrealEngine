// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/ValidationCookArtifact.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Cooker/CookArtifactReader.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "Engine/Level.h"
#include "Hash/Blake3.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

namespace UE::Cook
{

namespace Private
{

static const FString ConfigSectionName = TEXT("ValidationSettings");
static const TCHAR JsonKey_PackageValidationCache[] = TEXT("packageValidationCache");
static const TCHAR JsonKey_ExternalActors[] = TEXT("externalActors");
static const TCHAR JsonKey_ExternalPackages[] = TEXT("externalPackages");
static const TCHAR JsonKey_LastAssetValidationBuildVersion[] = TEXT("lastAssetValidationBuildVersion");

static bool bIncrementalExternalPackageValidationEnabled = false;
static FAutoConsoleVariableRef CVarIncrementalExternalPackageValidationEnabled(
	TEXT("cook.validation.IncrementalExternalPackageValidationEnabled"),
	bIncrementalExternalPackageValidationEnabled,
	TEXT("True if incremental external package validation is enabled, where we only validate changed external packages (eg, external actors in a partitioned world)"));

static bool bIncrementalValidationRequiresSameBuildVersion = false;
static FAutoConsoleVariableRef CVarIncrementalValidationRequiresSameBuildVersion(
	TEXT("cook.validation.IncrementalValidationRequiresSameBuildVersion"),
	bIncrementalValidationRequiresSameBuildVersion,
	TEXT("True if a different build version needs to re-validate everything, or False if incremental asset validation persists over different build versions"));

} // namespace Private

TArray<FAssetData> GetExternalObjectsForPackageValidation(const FName PackageName, const bool bIncludeActorPackages)
{
	TArray<FAssetData> ExternalObjects;
	if (bIncludeActorPackages)
	{
		const FString ExternalActorsPathForWorld = ULevel::GetExternalActorsPath(PackageName.ToString());
		IAssetRegistry::GetChecked().GetAssetsByPath(*ExternalActorsPathForWorld, ExternalObjects, /*bRecursive*/true, /*bIncludeOnlyOnDiskAssets*/true);
	}
	if (TArray<FAssetData> AssetsWithPackageAsOuter = UAssetRegistryHelpers::GetAssetsWithOuterForPaths({ PackageName }, PackageName, EAssetsWithOuterForPathsFlags::RecursivePaths | EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset);
		!AssetsWithPackageAsOuter.IsEmpty())
	{
		ExternalObjects.Append(MoveTemp(AssetsWithPackageAsOuter));
	}
	return ExternalObjects;
}

FValidationCookArtifact::FValidationCookArtifact(UCookOnTheFlyServer& InCOTFS)
	: COTFS(InCOTFS)
	, CurrentBuildVersion(FApp::GetBuildVersion())
	, bIsIncrementalExternalPackageValidationEnabled(Private::bIncrementalExternalPackageValidationEnabled)
{
	// We currently only support incremental external package validation for SP ByTheBook cooks (MP cooks would require a IMPCollector)
	if (bIsIncrementalExternalPackageValidationEnabled && (COTFS.GetProcessType() != UE::Cook::EProcessType::SingleProcess || COTFS.GetCookType() != UE::Cook::ECookType::ByTheBook))
	{
		UE_LOGF(LogCook, Warning, "Incremental external package validation was requested, but it is only supported by single-process cook-by-the-book cooks. Incremental external package validation will be disabled for this cook.");
		bIsIncrementalExternalPackageValidationEnabled = false;
	}

#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.AddRaw(this, &FValidationCookArtifact::LogCookStats);
#endif
}

FValidationCookArtifact::~FValidationCookArtifact()
{
#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.RemoveAll(this);
#endif
}

FString FValidationCookArtifact::GetArtifactName() const
{
	return TEXT("validation");
}

FConfigFile FValidationCookArtifact::CalculateCurrentSettings(ICookInfo& CookInfo, const ITargetPlatform* TargetPlatform)
{
	// Initialize the artifact data for this platform
	FArtifactData& PlatformArtifactData = PerPlatformData.Add(TargetPlatform);
	PlatformArtifactData.Filename = FPaths::Combine(CookInfo.GetCookMetadataOutputFolder(TargetPlatform), TEXTVIEW("ValidationCookArtifact.json"));

#if ENABLE_COOK_STATS
	PerPlatformMetrics.Add(TargetPlatform);
#endif

	// Validation doesn't have any settings that affect incremental cooking

	return FConfigFile();
}

void FValidationCookArtifact::CompareSettings(UE::Cook::Artifact::FCompareSettingsContext& Context)
{
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	FArtifactData& PlatformArtifactData = PerPlatformData.FindChecked(TargetPlatform);

	// Validation doesn't have any settings that affect incremental cooking
	
	if (LoadCookArtifactFile(Context, PlatformArtifactData))
	{
		UE_LOGF(LogCook, Display, "CookArtifact '%ls' loaded.", *PlatformArtifactData.Filename);
	}

	if (IsAssetValidationEnabledForCook())
	{
		if (Private::bIncrementalValidationRequiresSameBuildVersion && PlatformArtifactData.LastAssetValidationBuildVersion != CurrentBuildVersion)
		{
			UE_LOGF(LogCook, Display, "CookArtifact '%ls' invalidated cook because IncrementalValidationRequiresSameBuildVersion is true and LastAssetValidationBuildVersion != CurrentBuildVersion (old: '%ls', new: '%ls').",
				*PlatformArtifactData.Filename, *PlatformArtifactData.LastAssetValidationBuildVersion, *CurrentBuildVersion);
			Context.RequestFullRecook(true);
			return;
		}

		if (!bIsIncrementalExternalPackageValidationEnabled)
		{
			PlatformArtifactData.PackageValidationCache.Empty();
		}
	}
}

void FValidationCookArtifact::OnInvalidate(const ITargetPlatform* TargetPlatform)
{
	InvalidateCache(TargetPlatform);
}

void FValidationCookArtifact::OnFullRecook(const ITargetPlatform* TargetPlatform)
{
	InvalidateCache(TargetPlatform);
}

void FValidationCookArtifact::UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
{
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	FArtifactData& PlatformArtifactData = PerPlatformData.FindChecked(TargetPlatform);

	// Remove any stale data from the persistent artifact, and track cook stats if requested
	{
		const TMap<FName, UE::Cook::Artifact::FOplogPackageData>& OplogPackages = Context.GetOplogPackages();
		for (auto It = PlatformArtifactData.PackageValidationCache.CreateIterator(); It; ++It)
		{
			if (!OplogPackages.Contains(It->Key))
			{
				// Package is no longer known to the cook; remove it
				It.RemoveCurrent();
				continue;
			}
		}
#if ENABLE_COOK_STATS
		if (IsAssetValidationEnabledForCook())
		{
			FPackageValidationMetrics& PlatformMetrics = PerPlatformMetrics.FindChecked(TargetPlatform);
			for (const TTuple<FName, UE::Cook::Artifact::FOplogPackageData>& OplogPackagePair : OplogPackages)
			{
				// Note: Packages re-saved during cook were already tracked via FilterExternalPackagesToValidate, 
				//       which is why we only need track packages that were skipped during an incremental cook here
				// Skip Verse packages, as the Verse compiler validates those
				if (OplogPackagePair.Value.WasIncrementallySkipped() && !FPackageName::IsVersePackage(FNameBuilder(OplogPackagePair.Key)))
				{
					const int32 NumExternalPackages = GetExternalObjectsForPackageValidation(OplogPackagePair.Key).Num();
					PlatformMetrics.NumTotalPackages += NumExternalPackages + 1; // +1 for the main package
					PlatformMetrics.NumPackagesSkipped += NumExternalPackages + 1; // +1 for the main package
				}
			}
		}
#endif
	}

	// If validation was enabled then we can also update the last build version in the persistent artifact
	if (IsAssetValidationEnabledForCook())
	{
		PlatformArtifactData.LastAssetValidationBuildVersion = CurrentBuildVersion;
	}

	if (SaveCookArtifactFile(PlatformArtifactData))
	{
		UE_LOGF(LogCook, Display, "CookArtifact '%ls' saved.", *PlatformArtifactData.Filename);
	}
	else
	{
		UE_LOGF(LogCook, Warning, "CookArtifact '%ls' failed to save.", *PlatformArtifactData.Filename);
	}
}

void FValidationCookArtifact::FilterExternalPackagesToValidate(const FName PackageName, TArray<FAssetData>& InOutExternalPackages, const bool bUpdateCache)
{
	if (PerPlatformData.IsEmpty())
	{
		return;
	}

	auto TrackValidationMetrics = 
		[this, InitialNumExternalPackages = InOutExternalPackages.Num(), &InOutExternalPackages]()
		{
#if ENABLE_COOK_STATS
			const int32 NumSkippedExternalPackages = InitialNumExternalPackages - InOutExternalPackages.Num();
			for (TTuple<const ITargetPlatform*, FPackageValidationMetrics>& PerPlatformMetricsPair : PerPlatformMetrics)
			{
				FPackageValidationMetrics& PlatformMetrics = PerPlatformMetricsPair.Value;
				PlatformMetrics.NumTotalPackages += InitialNumExternalPackages + 1; // +1 for the main package
				PlatformMetrics.NumPackagesSkipped += NumSkippedExternalPackages;
				PlatformMetrics.NumPackagesModifiedOrUncached += InOutExternalPackages.Num() + 1; // +1 for the main package
			}
#endif
		};

	if (!bIsIncrementalExternalPackageValidationEnabled)
	{
		TrackValidationMetrics();
		return;
	}

	// Read the current hashes for all external packages (external actors and non-external-actor packages combined)
	TMap<FName, FIoHash> NewExternalPackageHashes;
	if (InOutExternalPackages.Num() > 0)
	{
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

		auto ComputeExternalPackageValidationHash = 
			[&AssetRegistry](const FName PackageName, const FAssetPackageData& PackageData) -> FIoHash
			{
				FBlake3 Hasher;

				// Include the package saved hash
				{
					const FIoHash SavedHash = PackageData.GetPackageSavedHash();
					Hasher.Update(MakeMemoryView(SavedHash.GetBytes(), sizeof(FIoHash::ByteArray)));
				}

				// Include the schema hash of each imported class to detect property layout changes without a resave
				{
					TArray<FName> ImportedClasses = PackageData.ImportedClasses;
					ImportedClasses.Sort(FNameLexicalLess()); // Sort for determinism
					for (const FName ClassName : ImportedClasses)
					{
						FNameBuilder ClassNameStr(ClassName);

						// Note: This only includes C++ and Verse classes, as BP classes will be handled via the dependency code below (without needing to load them)
						if (FPackageName::IsScriptPackage(ClassNameStr.ToView()) || FPackageName::IsVersePackage(ClassNameStr.ToView()))
						{
							if (const UClass* ImportedClass = LoadObject<UClass>(nullptr, ClassNameStr.ToView()))
							{
								const FBlake3Hash& SchemaHash = ImportedClass->GetSchemaHash(/*bSkipEditorOnly*/false);
								Hasher.Update(MakeMemoryView(SchemaHash.GetBytes(), sizeof(FBlake3Hash)));
							}
						}
					}
				}

				// Include the saved hash of each direct hard package dependency to detect changes in referenced assets
				{
					TArray<FName> Dependencies;
					if (AssetRegistry.GetDependencies(PackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard))
					{
						Dependencies.Sort(FNameLexicalLess()); // Sort for determinism
						for (const FName Dependency : Dependencies)
						{
							FNameBuilder DependencyStr(Dependency);

							// Skip C++ and Verse packages, as they were handled via the class imports above
							if (!FPackageName::IsScriptPackage(DependencyStr.ToView()) && !FPackageName::IsVersePackage(DependencyStr.ToView()))
							{
								if (FAssetPackageData DependencyPackageData;
									AssetRegistry.TryGetAssetPackageData(Dependency, DependencyPackageData) == AssetRegistry::EExists::Exists)
								{
									const FIoHash DependencySavedHash = DependencyPackageData.GetPackageSavedHash();
									Hasher.Update(MakeMemoryView(DependencySavedHash.GetBytes(), sizeof(FIoHash::ByteArray)));
								}
							}
						}
					}
				}

				return FIoHash(Hasher.Finalize());
			};

		for (const FAssetData& ExternalPackage : InOutExternalPackages)
		{
			// Note: TryGetAssetPackageData only takes a read-lock, so this could be parallelized if needed
			FAssetPackageData ExternalPackageData;
			if (AssetRegistry.TryGetAssetPackageData(ExternalPackage.PackageName, ExternalPackageData) == AssetRegistry::EExists::Exists)
			{
				NewExternalPackageHashes.Add(ExternalPackage.PackageName, ComputeExternalPackageValidationHash(ExternalPackage.PackageName, ExternalPackageData));
			}
		}
	}

	// Compare these hashes against any previous state to know what's changed
	// Note: Validation is platform agnostic, so we just use the data for the first platform
	if (InOutExternalPackages.Num() > 0)
	{
		FArtifactData& PlatformArtifactData = PerPlatformData.begin()->Value;
		if (const TMap<FName, FIoHash>* ExternalPackageHashes = PlatformArtifactData.PackageValidationCache.Find(PackageName))
		{
			InOutExternalPackages.RemoveAll(
				[&NewExternalPackageHashes, ExternalPackageHashes](const FAssetData& ExternalPackage)
				{
					const FIoHash* CurrentHash  = NewExternalPackageHashes.Find(ExternalPackage.PackageName);
					const FIoHash* PreviousHash = ExternalPackageHashes->Find(ExternalPackage.PackageName);
					return CurrentHash && PreviousHash && *CurrentHash == *PreviousHash && !CurrentHash->IsZero();
				});
		}
	}

	TrackValidationMetrics();

	// Persist the current hashes if we were asked to
	if (bUpdateCache)
	{
		for (TTuple<const ITargetPlatform*, FArtifactData>& PerPlatformDataPair : PerPlatformData)
		{
			if (NewExternalPackageHashes.IsEmpty())
			{
				// Packages without external packages should not be stored in the cache
				PerPlatformDataPair.Value.PackageValidationCache.Remove(PackageName);
			}
			else
			{
				PerPlatformDataPair.Value.PackageValidationCache.Add(PackageName, NewExternalPackageHashes);
			}
		}
	}
}

void FValidationCookArtifact::MarkPackageRequiresFullValidation(const FName PackageName)
{
	for (TTuple<const ITargetPlatform*, FArtifactData>& PerPlatformDataPair : PerPlatformData)
	{
		PerPlatformDataPair.Value.PackageValidationCache.Remove(PackageName);
	}
}

bool FValidationCookArtifact::IsAssetValidationEnabledForCook() const
{
	return EnumHasAnyFlags(COTFS.GetCookValidationOptions(), UE::Cook::ECookValidationOptions::RunAssetValidation);
}

void FValidationCookArtifact::InvalidateCache(const ITargetPlatform* TargetPlatform)
{
	FArtifactData& PlatformArtifactData = PerPlatformData.FindChecked(TargetPlatform);
	FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PlatformArtifactData.Filename);
	PlatformArtifactData.PackageValidationCache.Reset();
}

bool FValidationCookArtifact::LoadCookArtifactFile(UE::Cook::Artifact::FCompareSettingsContext& Context, FArtifactData& PlatformArtifactData)
{
	TUniquePtr<FArchive> Reader(Context.GetArtifactReader().CreateFileReader(*PlatformArtifactData.Filename));
	if (!Reader)
	{
		return false;
	}

	FString PlatformArtifactJsonString;
	if (!FFileHelper::LoadFileToString(PlatformArtifactJsonString, *Reader.Get()))
	{
		return false;
	}

	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> PlatformArtifactJsonDocument = UE::Json::ParseInPlace(PlatformArtifactJsonString);
	if (PlatformArtifactJsonDocument.HasError())
	{
		return false;
	}

	TOptional<UE::Json::FConstObject> PlatformArtifactJsonRootObject = UE::Json::GetRootObject(PlatformArtifactJsonDocument.GetValue());
	if (!PlatformArtifactJsonRootObject.IsSet())
	{
		return false;
	}

	// Read PackageValidationCache
	if (TOptional<UE::Json::FConstObject> PackageValidationCacheJsonObject = UE::Json::GetObjectField(PlatformArtifactJsonRootObject.GetValue(), Private::JsonKey_PackageValidationCache))
	{
		for (const UE::Json::FMember& PackageCacheJsonEntry : PackageValidationCacheJsonObject.GetValue())
		{
			if (!PackageCacheJsonEntry.value.IsObject())
			{
				continue;
			}

			const FString PackageName            = FString(UE::Json::ValueAsStringView(PackageCacheJsonEntry.name));
			const FString ExternalActorsPrefix   = ULevel::GetExternalActorsPath(PackageName) / TEXT("");
			const FString ExternalPackagesPrefix = PackageName / TEXT("");

			TMap<FName, FIoHash>& ExternalPackageHashes = PlatformArtifactData.PackageValidationCache.Add(*PackageName);

			// Read a hash sub-object into the combined map, re-adding the given prefix to each stored path
			auto ReadHashesIntoMap =
				[&ExternalPackageHashes](const UE::Json::FConstObject& ContainerObject, const TCHAR* JsonKey, const FString& Prefix)
				{
					if (TOptional<UE::Json::FConstObject> HashesJsonObject = UE::Json::GetObjectField(ContainerObject, JsonKey))
					{
						for (const UE::Json::FMember& HashJsonEntry : HashesJsonObject.GetValue())
						{
							if (HashJsonEntry.value.IsString())
							{
								// Re-add the prefix stripped off during save
								FString Path = FString(UE::Json::ValueAsStringView(HashJsonEntry.name));
								if (Path.Len() > 0 && Path[0] != TEXT('/'))
								{
									Path.InsertAt(0, Prefix);
								}
								ExternalPackageHashes.Add(*Path, FIoHash(UE::Json::ValueAsStringView(HashJsonEntry.value)));
							}
						}
					}
				};
			ReadHashesIntoMap(PackageCacheJsonEntry.value.GetObject(), Private::JsonKey_ExternalActors,   ExternalActorsPrefix);
			ReadHashesIntoMap(PackageCacheJsonEntry.value.GetObject(), Private::JsonKey_ExternalPackages, ExternalPackagesPrefix);
		}
	}

	// Read LastAssetValidationBuildVersion
	if (TOptional<FStringView> LastAssetValidationBuildVersionJsonString = UE::Json::GetStringField(PlatformArtifactJsonRootObject.GetValue(), Private::JsonKey_LastAssetValidationBuildVersion))
	{
		PlatformArtifactData.LastAssetValidationBuildVersion = LastAssetValidationBuildVersionJsonString.GetValue();
	}

	return true;
}

bool FValidationCookArtifact::SaveCookArtifactFile(FArtifactData& PlatformArtifactData)
{
	UE::Json::FDocument PlatformArtifactJsonDocument(rapidjson::kObjectType);
	{
		UE::Json::FObject PlatformArtifactJsonRootObject = PlatformArtifactJsonDocument.GetObject();
		UE::Json::FAllocator& PlatformArtifactJsonAllocator = PlatformArtifactJsonDocument.GetAllocator();

		// Add PackageValidationCache
		{
			UE::Json::FValue PackageValidationCacheJsonObject(rapidjson::kObjectType);
			PlatformArtifactData.PackageValidationCache.KeySort(FNameLexicalLess());
			for (TTuple<FName, TMap<FName, FIoHash>>& PackageValidationCachePair : PlatformArtifactData.PackageValidationCache)
			{
				const FString PackageName            = PackageValidationCachePair.Key.ToString();
				const FString ExternalActorsPrefix   = ULevel::GetExternalActorsPath(PackageName) / TEXT("");
				const FString ExternalPackagesPrefix = PackageName / TEXT("");

				// Split the combined external package hashes into external actors and non-external-actor packages for compactness,
				// stripping the common root path prefix from each group
				UE::Json::FValue ExternalActorsJsonObject(rapidjson::kObjectType);
				UE::Json::FValue ExternalPackagesJsonObject(rapidjson::kObjectType);

				PackageValidationCachePair.Value.KeySort(FNameLexicalLess());
				for (const TTuple<FName, FIoHash>& ExternalPackageHashesPair : PackageValidationCachePair.Value)
				{
					FNameBuilder PathBuilder(ExternalPackageHashesPair.Key);
					FStringView Path = PathBuilder.ToView();

					UE::Json::FValue* TargetJsonObject = &ExternalPackagesJsonObject;
					if (Path.StartsWith(ExternalActorsPrefix))
					{
						Path.RemovePrefix(ExternalActorsPrefix.Len());
						TargetJsonObject = &ExternalActorsJsonObject;
					}
					else if (Path.StartsWith(ExternalPackagesPrefix))
					{
						Path.RemovePrefix(ExternalPackagesPrefix.Len());
					}

					TargetJsonObject->AddMember(
						UE::Json::MakeStringValue(Path, PlatformArtifactJsonAllocator),
						UE::Json::MakeStringValue(LexToString(ExternalPackageHashesPair.Value), PlatformArtifactJsonAllocator),
						PlatformArtifactJsonAllocator);
				}

				UE::Json::FValue PackageEntryJsonObject(rapidjson::kObjectType);
				if (ExternalActorsJsonObject.MemberCount() > 0)
				{
					PackageEntryJsonObject.AddMember(UE::Json::MakeStringRef(Private::JsonKey_ExternalActors), MoveTemp(ExternalActorsJsonObject), PlatformArtifactJsonAllocator);
				}
				if (ExternalPackagesJsonObject.MemberCount() > 0)
				{
					PackageEntryJsonObject.AddMember(UE::Json::MakeStringRef(Private::JsonKey_ExternalPackages), MoveTemp(ExternalPackagesJsonObject), PlatformArtifactJsonAllocator);
				}

				PackageValidationCacheJsonObject.AddMember(
					UE::Json::MakeStringValue(FNameBuilder(PackageValidationCachePair.Key).ToView(), PlatformArtifactJsonAllocator),
					MoveTemp(PackageEntryJsonObject),
					PlatformArtifactJsonAllocator);
			}
			PlatformArtifactJsonRootObject.AddMember(
				UE::Json::MakeStringRef(Private::JsonKey_PackageValidationCache),
				MoveTemp(PackageValidationCacheJsonObject),
				PlatformArtifactJsonAllocator);
		}

		// Add LastAssetValidationBuildVersion
		PlatformArtifactJsonRootObject.AddMember(
			UE::Json::MakeStringRef(Private::JsonKey_LastAssetValidationBuildVersion), 
			UE::Json::MakeStringValue(PlatformArtifactData.LastAssetValidationBuildVersion, PlatformArtifactJsonAllocator), 
			PlatformArtifactJsonAllocator);
	}

	const FString PlatformArtifactJsonString = UE::Json::WritePretty(PlatformArtifactJsonDocument);
	if (!FFileHelper::SaveStringToFile(PlatformArtifactJsonString, *PlatformArtifactData.Filename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		return false;
	}

	return true;
}

#if ENABLE_COOK_STATS
void FValidationCookArtifact::LogCookStats(FCookStatsManager::AddStatFuncRef AddStat) const
{
	if (!IsAssetValidationEnabledForCook() || PerPlatformMetrics.Num() == 0)
	{
		return;
	}

	// Note: Validation is platform agnostic, so we just use the data for the first platform
	const FPackageValidationMetrics& PlatformMetrics = PerPlatformMetrics.begin()->Value;

	UE::Json::FDocument JsonDocument(rapidjson::kObjectType);
	{
		UE::Json::FObject JsonRootObject = JsonDocument.GetObject();
		UE::Json::FAllocator& JsonAllocator = JsonDocument.GetAllocator();

		auto AddMetric =
			[&JsonRootObject, &JsonAllocator](FStringView Key, int32 Value)
			{
				JsonRootObject.AddMember(UE::Json::MakeStringRef(Key), Value, JsonAllocator);
			};

		AddMetric(TEXTVIEW("num_total"), PlatformMetrics.NumTotalPackages);
		AddMetric(TEXTVIEW("num_skipped"), PlatformMetrics.NumPackagesSkipped);
		AddMetric(TEXTVIEW("num_modified_or_uncached"), PlatformMetrics.NumPackagesModifiedOrUncached);
	}
	// StatName ("Validation") is combined with each AttrName ("Metrics") in FEditorTelemetry::RecordEvent_Cooking, to form "{StatName}_{AttrName}"
	AddStat(TEXT("Validation"), FCookStatsManager::CreateKeyValueArray(TEXT("Metrics"), UE::Json::WriteCompact(JsonDocument)));
}
#endif

} // namespace UE::Cook
