// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cooker/CookArtifact.h"
#include "IO/IoHash.h"
#include "ProfilingDebugging/CookStats.h"

struct FAssetData;
class UCookOnTheFlyServer;

namespace UE::Cook
{

/**
 * Given a package, get all the external objects that should be validated alongside it.
 */
TArray<FAssetData> GetExternalObjectsForPackageValidation(const FName PackageName, const bool bIncludeActorPackages = true);

/**
 * CookArtifact that stores information related to incremental source package validation during incremental cooking.
 * 
 * @note Most incremental validation is handled automatically, as incremental cooking only loads/validates changed packages, however things made up of multiple 
 *       source packages (like worlds) need special handling since validation runs on load, so before things like world cell generation have run during save.
 */
class FValidationCookArtifact : public ICookArtifact
{
public:
	explicit FValidationCookArtifact(UCookOnTheFlyServer& InCOTFS);
	virtual ~FValidationCookArtifact();

	virtual FString GetArtifactName() const override;
	virtual FConfigFile CalculateCurrentSettings(ICookInfo& CookInfo, const ITargetPlatform* TargetPlatform) override;
	virtual void CompareSettings(UE::Cook::Artifact::FCompareSettingsContext& Context) override;
	virtual void OnInvalidate(const ITargetPlatform* TargetPlatform) override;
	virtual void OnFullRecook(const ITargetPlatform* TargetPlatform) override;
	virtual void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context) override;

	/**
	 * Given the complete set of external packages (external actors and non-external-actor packages) associated with the given package,
	 * filter the set down to only those that need validation.
	 *
	 * @param PackageName				The name of the package being validated.
	 * @param InOutExternalPackages		The combined set of external actors and non-external-actor packages to filter.
	 * @param bUpdateCache				If set, update the cache with the current hashes, so the next call uses them as the baseline for comparison.
	 */
	void FilterExternalPackagesToValidate(const FName PackageName, TArray<FAssetData>& InOutExternalPackages, const bool bUpdateCache = true);

	/**
	 * Clear any incremental validation state for the given package (eg, if it failed validation).
	 */
	void MarkPackageRequiresFullValidation(const FName PackageName);

private:
	bool IsAssetValidationEnabledForCook() const;

	void InvalidateCache(const ITargetPlatform* TargetPlatform);

	struct FArtifactData
	{
		/** Package -> (External Package Name -> Hash); external actors and non-external-actor packages stored together */
		TMap<FName, TMap<FName, FIoHash>> PackageValidationCache;

		/** FApp::GetBuildVersion() that was used when we last ran asset validation (@see IsAssetValidationEnabledForCook) */
		FString LastAssetValidationBuildVersion;

		/** Not serialized */
		FString Filename;
	};

	bool LoadCookArtifactFile(UE::Cook::Artifact::FCompareSettingsContext& Context, FArtifactData& PlatformArtifactData);
	bool SaveCookArtifactFile(FArtifactData& PlatformArtifactData);

#if ENABLE_COOK_STATS
	void LogCookStats(FCookStatsManager::AddStatFuncRef AddStat) const;
#endif

	UCookOnTheFlyServer& COTFS;
	FString CurrentBuildVersion;
	bool bIsIncrementalExternalPackageValidationEnabled = false;

	TSortedMap<const ITargetPlatform*, FArtifactData> PerPlatformData;

#if ENABLE_COOK_STATS
	struct FPackageValidationMetrics
	{
		int32 NumTotalPackages = 0;
		int32 NumPackagesSkipped = 0;
		int32 NumPackagesModifiedOrUncached = 0;
	};
	TSortedMap<const ITargetPlatform*, FPackageValidationMetrics> PerPlatformMetrics;
#endif
};

} // namespace UE::Cook
