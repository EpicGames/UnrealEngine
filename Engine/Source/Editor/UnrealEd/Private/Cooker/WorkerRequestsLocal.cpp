// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkerRequestsLocal.h"

#include "CookTypes.h"
#include "Cooker/CookGenerationHelper.h"
#include "Cooker/CookPackageData.h"
#include "Cooker/CookRequests.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FConfigFile;
class ITargetPlatform;

namespace UE::Cook
{
struct FInstigator;

bool FWorkerRequestsLocal::HasExternalRequests() const
{
	return ExternalRequests.HasRequests();
}

int32 FWorkerRequestsLocal::GetNumExternalRequests() const
{
	return ExternalRequests.GetNumRequests();
}

EExternalRequestType FWorkerRequestsLocal::DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks,
	TArray<FFilePlatformRequest>& OutBuildRequests)
{
	return ExternalRequests.DequeueNextCluster(OutCallbacks, OutBuildRequests);
}

bool FWorkerRequestsLocal::DequeueSchedulerCallbacks(TArray<FSchedulerCallback>& OutCallbacks)
{
	return ExternalRequests.DequeueCallbacks(OutCallbacks);
}

void FWorkerRequestsLocal::DequeueAllExternal(TArray<FSchedulerCallback>& OutCallbacks,
	TArray<FFilePlatformRequest>& OutCookRequests)
{
	ExternalRequests.DequeueAll(OutCallbacks, OutCookRequests);
}

void FWorkerRequestsLocal::QueueDiscoveredPackage(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
	FInstigator&& Instigator, FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency Urgency,
	FGenerationHelper* ParentGenerationHelper)
{
	// FWorkerRequestsRemote needs to send ParentGenerationHelper data to the director, but in the local case we
	// already consumed the data in FGenerationHelper::StartQueueGeneratedPackages, so we don't use the value here.
	(void)ParentGenerationHelper;
	COTFS.QueueDiscoveredPackageOnDirector(PackageData, MoveTemp(Instigator), MoveTemp(ReachablePlatforms), Urgency);
}

void FWorkerRequestsLocal::EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS,
	FGenerationHelper& GenerationHelper)
{
	GenerationHelper.EndQueueGeneratedPackagesOnDirector(COTFS, FWorkerId::Local());
}

void FWorkerRequestsLocal::AddStartCookByTheBookRequest(FFilePlatformRequest&& Request)
{
	ExternalRequests.EnqueueUnique(MoveTemp(Request));
}

void FWorkerRequestsLocal::InitializeCookOnTheFly()
{
	ExternalRequests.CookRequestEvent.Reset(FPlatformProcess::GetSynchEventFromPool());
}

void FWorkerRequestsLocal::AddCookOnTheFlyRequest(FFilePlatformRequest&& Request)
{
	ExternalRequests.EnqueueUnique(MoveTemp(Request), true /* bForceFrontOfQueue */);
	if (ExternalRequests.CookRequestEvent)
	{
		ExternalRequests.CookRequestEvent->Trigger();
	}
}

void FWorkerRequestsLocal::AddCookOnTheFlyCallback(FSchedulerCallback&& Callback)
{
	ExternalRequests.AddCallback(MoveTemp(Callback));
	if (ExternalRequests.CookRequestEvent)
	{
		ExternalRequests.CookRequestEvent->Trigger();
	}
}

void FWorkerRequestsLocal::WaitForCookOnTheFlyEvents(int TimeoutMs)
{
	if (ExternalRequests.CookRequestEvent)
	{
		ExternalRequests.CookRequestEvent->Wait(TimeoutMs, true /* bIgnoreThreadIdleStats */);
	}
}

void FWorkerRequestsLocal::AddEditorActionCallback(FSchedulerCallback&& Callback)
{
	ExternalRequests.AddCallback(MoveTemp(Callback));
}

void FWorkerRequestsLocal::AddPublicInterfaceRequest(FFilePlatformRequest&& Request, bool bForceFrontOfQueue)
{
	ExternalRequests.EnqueueUnique(MoveTemp(Request), bForceFrontOfQueue);
}

void FWorkerRequestsLocal::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	ExternalRequests.RemapTargetPlatforms(Remap);
}

void FWorkerRequestsLocal::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform)
{
	ExternalRequests.OnRemoveSessionPlatform(TargetPlatform);
}

void FWorkerRequestsLocal::ReportDemotion(UE::Cook::FPackageData& PackageData, ESuppressCookReason Reason)
{
}

void FWorkerRequestsLocal::ReportPromoteToSaveComplete(UE::Cook::FPackageData& PackageData)
{
}

void FWorkerRequestsLocal::GetInitializeConfigSettings(UCookOnTheFlyServer& COTFS,
	const FString& OutputDirectoryOverride, UE::Cook::FInitializeConfigSettings& Settings)
{
	Settings.LoadLocal(OutputDirectoryOverride);
}

void FWorkerRequestsLocal::GetBeginCookConfigSettings(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext,
	UE::Cook::FBeginCookConfigSettings& Settings)
{
	Settings.LoadLocal(BeginContext);
}

void FWorkerRequestsLocal::GetBeginCookIncrementalFlags(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext)
{
	BeginContext.COTFS.LoadBeginCookIncrementalFlagsLocal(BeginContext);
}

ECookMode::Type FWorkerRequestsLocal::GetDirectorCookMode(UCookOnTheFlyServer& COTFS)
{
	return COTFS.GetCookMode();
}

void FWorkerRequestsLocal::LogAllRequestedFiles()
{
	ExternalRequests.LogAllRequestedFiles();
}

}