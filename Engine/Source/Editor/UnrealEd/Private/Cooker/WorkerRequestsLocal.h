// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "CookRequests.h"
#include "Cooker/CookTypes.h"
#include "HAL/Platform.h"
#include "IWorkerRequests.h"

class FConfigFile;
class ITargetPlatform;

namespace UE::Cook
{
struct FInstigator;
struct FPackageData;

/** An IWorkerRequests for SingleProcess cooks: functions are passed through to the CookOnTheFlyServer. */
class FWorkerRequestsLocal : public IWorkerRequests
{
public:
	virtual bool HasExternalRequests() const override;
	virtual int32 GetNumExternalRequests() const override;
	virtual EExternalRequestType DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks,
		TArray<FFilePlatformRequest>& OutBuildRequests) override;
	virtual bool DequeueSchedulerCallbacks(TArray<FSchedulerCallback>& OutCallbacks) override;
	virtual void DequeueAllExternal(TArray<FSchedulerCallback>& OutCallbacks,
		TArray<FFilePlatformRequest>& OutCookRequests) override;
	virtual void RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap) override;
	virtual void OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform) override;
	virtual void QueueDiscoveredPackage(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
		FInstigator&& Instigator, FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency Urgency,
		FGenerationHelper* ParentGenerationHelper) override;
	virtual void EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS, FGenerationHelper& GenerationHelper) override;
	virtual void AddStartCookByTheBookRequest(FFilePlatformRequest&& Request) override;
	virtual void InitializeCookOnTheFly() override;
	virtual void AddCookOnTheFlyRequest(FFilePlatformRequest&& Request) override;
	virtual void AddCookOnTheFlyCallback(FSchedulerCallback&& Callback) override;
	virtual void WaitForCookOnTheFlyEvents(int TimeoutMs) override;
	virtual void AddEditorActionCallback(FSchedulerCallback&& Callback) override;
	virtual void AddPublicInterfaceRequest(FFilePlatformRequest&& Request, bool bForceFrontOfQueue) override;

	virtual void ReportDemotion(UE::Cook::FPackageData& PackageData, ESuppressCookReason Reason) override;
	virtual void ReportPromoteToSaveComplete(UE::Cook::FPackageData& PackageData) override;
	virtual void GetInitializeConfigSettings(UCookOnTheFlyServer& COTFS, const FString& OutputOverrideDirectory,
		UE::Cook::FInitializeConfigSettings& Settings) override;
	virtual void GetBeginCookConfigSettings(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext,
		UE::Cook::FBeginCookConfigSettings& Settings) override;
	virtual void GetBeginCookIncrementalFlags(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext) override;
	virtual ECookMode::Type GetDirectorCookMode(UCookOnTheFlyServer& COTFS) override;

	virtual void LogAllRequestedFiles() override;

private:
	FExternalRequests ExternalRequests;
};

}