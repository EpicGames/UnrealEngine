// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorkerRequestsRemote.h"

#include "Algo/Find.h"
#include "Containers/SparseArray.h"
#include "CookOnTheSide/CookLog.h"
#include "CookPlatformManager.h"
#include "CookTypes.h"
#include "Cooker/CookRequests.h"
#include "Interfaces/ITargetPlatform.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Trace/Detail/Channel.h"

class FConfigFile;
class ITargetPlatform;

namespace UE::Cook
{
struct FInstigator;
struct FPackageData;

FWorkerRequestsRemote::FWorkerRequestsRemote(UCookOnTheFlyServer& InCOTFS)
	: CookWorkerClient(*InCOTFS.CookWorkerClient)
{
}

bool FWorkerRequestsRemote::HasExternalRequests() const
{
	return ExternalRequests.HasRequests();
}

int32 FWorkerRequestsRemote::GetNumExternalRequests() const
{
	return ExternalRequests.GetNumRequests();
}

EExternalRequestType FWorkerRequestsRemote::DequeueNextCluster(TArray<FSchedulerCallback>& OutCallbacks,
	TArray<FFilePlatformRequest>& OutBuildRequests)
{
	return ExternalRequests.DequeueNextCluster(OutCallbacks, OutBuildRequests);
}

bool FWorkerRequestsRemote::DequeueSchedulerCallbacks(TArray<FSchedulerCallback>& OutCallbacks)
{
	return ExternalRequests.DequeueCallbacks(OutCallbacks);
}

void FWorkerRequestsRemote::DequeueAllExternal(TArray<FSchedulerCallback>& OutCallbacks,
	TArray<FFilePlatformRequest>& OutCookRequests)
{
	ExternalRequests.DequeueAll(OutCallbacks, OutCookRequests);
}

void FWorkerRequestsRemote::QueueDiscoveredPackage(UCookOnTheFlyServer& COTFS, FPackageData& PackageData,
	FInstigator&& Instigator, FDiscoveredPlatformSet&& ReachablePlatforms, EUrgency Urgency,
	FGenerationHelper* ParentGenerationHelper)
{
	CookWorkerClient.ReportDiscoveredPackage(PackageData, MoveTemp(Instigator), MoveTemp(ReachablePlatforms),
		ParentGenerationHelper, Urgency);
}

void FWorkerRequestsRemote::EndQueueGeneratedPackages(UCookOnTheFlyServer& COTFS,
	FGenerationHelper& GenerationHelper)
{
	CookWorkerClient.ReportGeneratorQueuedGeneratedPackages(GenerationHelper);
}

void FWorkerRequestsRemote::AddStartCookByTheBookRequest(FFilePlatformRequest&& Request)
{
	LogCalledCookByTheBookError(TEXT("AddStartCookByTheBookRequest"));
}

void FWorkerRequestsRemote::InitializeCookOnTheFly()
{
	LogCalledCookOnTheFlyError(TEXT("InitializeCookOnTheFly"));
}

void FWorkerRequestsRemote::AddCookOnTheFlyRequest(FFilePlatformRequest&& Request)
{
	LogCalledCookOnTheFlyError(TEXT("AddCookOnTheFlyRequest"));
}

void FWorkerRequestsRemote::AddCookOnTheFlyCallback(FSchedulerCallback&& Callback)
{
	LogCalledCookOnTheFlyError(TEXT("AddCookOnTheFlyCallback"));
}

void FWorkerRequestsRemote::WaitForCookOnTheFlyEvents(int TimeoutMs)
{
	LogCalledCookOnTheFlyError(TEXT("WaitForCookOnTheFlyEvents"));
}

void FWorkerRequestsRemote::AddEditorActionCallback(FSchedulerCallback&& Callback)
{
	LogCalledEditorActionError(TEXT("AddEditorActionCallback"));
}

void FWorkerRequestsRemote::AddPublicInterfaceRequest(FFilePlatformRequest&& Request, bool bForceFrontOfQueue)
{
	LogCalledPublicInterfaceError(TEXT("AddPublicInterfaceRequest"));
}

void FWorkerRequestsRemote::RemapTargetPlatforms(const TMap<ITargetPlatform*, ITargetPlatform*>& Remap)
{
	ExternalRequests.RemapTargetPlatforms(Remap);
}

void FWorkerRequestsRemote::OnRemoveSessionPlatform(const ITargetPlatform* TargetPlatform)
{
	ExternalRequests.OnRemoveSessionPlatform(TargetPlatform);
}

void FWorkerRequestsRemote::ReportDemotion(UE::Cook::FPackageData& PackageData, ESuppressCookReason Reason)
{
	CookWorkerClient.ReportDemotion(PackageData, Reason);
}

void FWorkerRequestsRemote::ReportPromoteToSaveComplete(UE::Cook::FPackageData& PackageData)
{
	CookWorkerClient.ReportPromoteToSaveComplete(PackageData);
}

void FWorkerRequestsRemote::GetInitializeConfigSettings(UCookOnTheFlyServer& COTFS,
	const FString& OutputDirectoryOverride, UE::Cook::FInitializeConfigSettings& Settings)
{
	Settings = CookWorkerClient.ConsumeInitializeConfigSettings();
}

void FWorkerRequestsRemote::GetBeginCookConfigSettings(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext,
	UE::Cook::FBeginCookConfigSettings& Settings)
{
	Settings = CookWorkerClient.ConsumeBeginCookConfigSettings();
}

void FWorkerRequestsRemote::GetBeginCookIncrementalFlags(UCookOnTheFlyServer& COTFS, FBeginCookContext& BeginContext)
{
	const FBeginCookContextForWorker& DirectorBeginContext = CookWorkerClient.GetBeginCookContext();

	for (FBeginCookContextPlatform& PlatformContext : BeginContext.PlatformContexts)
	{
		const ITargetPlatform* TargetPlatform = PlatformContext.TargetPlatform;
		const FBeginCookContextForWorkerPlatform* DirectorPlatformContext = Algo::FindByPredicate(
			DirectorBeginContext.PlatformContexts, [TargetPlatform](const FBeginCookContextForWorkerPlatform& Platform)
			{
				return Platform.TargetPlatform == TargetPlatform;
			});
		checkf(DirectorPlatformContext,
			TEXT("Director sent TargetPlatform %s, but this platform is not found in the DirectorBeginContext."),
			*TargetPlatform->PlatformName());

		UE::Cook::FPlatformData* PlatformData = PlatformContext.PlatformData;
		PlatformContext.CookArtifactsCurrentSettings.Empty(); // Not needed on CookWorkerClients
		PlatformContext.bHasMemoryResults = false; // Not needed on CookWorkerClients
		PlatformContext.bFullBuild = DirectorPlatformContext->bFullBuild;
		PlatformContext.bClearMemoryResults = false; // Not needed on CookWorkerClients
		PlatformContext.bPopulateMemoryResultsFromDiskResults = false; // Not needed on CookWorkerClients
		PlatformContext.bLegacyIterativeSharedBuild = false; // Not needed on CookWorkerClients
		PlatformContext.bWorkerOnSharedSandbox = true;
		PlatformData->bFullBuild = PlatformContext.bFullBuild;
		PlatformData->bLegacyIterativeSharedBuild = PlatformContext.bLegacyIterativeSharedBuild;
		PlatformData->bWorkerOnSharedSandbox = PlatformContext.bWorkerOnSharedSandbox;
	}
}

ECookMode::Type FWorkerRequestsRemote::GetDirectorCookMode(UCookOnTheFlyServer& COTFS)
{
	return CookWorkerClient.GetDirectorCookMode();
}

void FWorkerRequestsRemote::LogCalledCookByTheBookError(const TCHAR* FunctionName) const
{
	check(FunctionName);
	UE_LOGF(LogCook, Error, "Calling %ls (a CookByTheBook function) is not allowed in a CookWorker.",
		FunctionName);
}

void FWorkerRequestsRemote::LogCalledCookOnTheFlyError(const TCHAR* FunctionName) const
{
	check(FunctionName);
	UE_LOGF(LogCook, Error, "Calling %ls (a CookOnTheFly function) is not allowed in a CookWorker.", FunctionName);
}

void FWorkerRequestsRemote::LogCalledPublicInterfaceError(const TCHAR* FunctionName) const
{
	check(FunctionName);
	UE_LOGF(LogCook, Error,
		"Calling %ls (a CookOnTheFlyServer public interface function) is not allowed in a CookWorker.",
		FunctionName);
}

void FWorkerRequestsRemote::LogCalledEditorActionError(const TCHAR* FunctionName) const
{
	check(FunctionName);
	UE_LOGF(LogCook, Error, "Calling %ls (an editor-mode-only function) is not allowed in a CookWorker.",
		FunctionName);
}

void FWorkerRequestsRemote::LogAllRequestedFiles()
{
	ExternalRequests.LogAllRequestedFiles();
}

}