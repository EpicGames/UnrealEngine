// Copyright Epic Games, Inc. All Rights Reserved.


#include "CreditsScreen.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineVersionBase.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "SlateGlobals.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SRichTextBlock.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CreditsScreen"

void SCreditsScreen::Construct(const FArguments& InArgs)
{
	PreviousScrollPosition = 0.0f;
	ScrollPixelsPerSecond = 50.0f;
	bIsPlaying = true;

	ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SCreditsScreen::RollCredits ) );

	const FString Version = FEngineVersion::Current().ToString(FEngineBuildSettings::IsPerforceBuild() ? EVersionComponent::Branch : EVersionComponent::Patch);

	FString CreditsText;
	FFileHelper::LoadFileToString(CreditsText, *( FPaths::EngineContentDir() + TEXT("Editor/Credits.rt") ));
	CreditsText.ReplaceInline(TEXT("%VERSION%"), *Version);

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		[
			SAssignNew(ScrollBox, SScrollBox)
			.Style( FAppStyle::Get(), "ScrollBox" )
			.OnUserScrolled(this, &SCreditsScreen::HandleUserScrolled)

			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SRichTextBlock)
					.Text(FText::FromString(CreditsText))
					.TextStyle(FAppStyle::Get(), "Credits.Normal")
					.DecoratorStyleSet(&FAppStyle::Get())
					.Justification(ETextJustify::Center)
					+ SRichTextBlock::HyperlinkDecorator(TEXT("browser"), this, &SCreditsScreen::OnBrowserLinkClicked)
				]
			]
		]

		+ SOverlay::Slot()
		.VAlign(VAlign_Bottom)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Credits.Button")
				.OnClicked(this, &SCreditsScreen::HandleTogglePlayPause)
				[
					SNew(SImage)
					.Image(this, &SCreditsScreen::GetTogglePlayPauseBrush)
				]
			]
		]
	];
}

EActiveTimerReturnType SCreditsScreen::RollCredits( double InCurrentTime, float InDeltaTime )
{
	float NewPixelOffset = ( ScrollPixelsPerSecond * InDeltaTime );
	ScrollBox->SetScrollOffset( ScrollBox->GetScrollOffset() + NewPixelOffset );
	PreviousScrollPosition = ScrollBox->GetScrollOffset();

	return EActiveTimerReturnType::Continue;
}

FReply SCreditsScreen::HandleTogglePlayPause()
{
	if ( bIsPlaying )
	{
		bIsPlaying = false;
		if ( ActiveTimerHandle.IsValid() )
		{
			UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
		}
	}
	else
	{
		bIsPlaying = true;
		if ( !ActiveTimerHandle.IsValid() )
		{
			ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SCreditsScreen::RollCredits ) );
		}
	}

	return FReply::Handled();
}

void SCreditsScreen::HandleUserScrolled(float ScrollOffset)
{
	// If the user scrolls up, and we're currently playing, then stop playing.
	if ( bIsPlaying && ScrollOffset < PreviousScrollPosition )
	{
		bIsPlaying = false;
		if ( ActiveTimerHandle.IsValid() )
		{
			UnRegisterActiveTimer( ActiveTimerHandle.Pin().ToSharedRef() );
		}
	}

	PreviousScrollPosition = ScrollOffset;
}

const FSlateBrush* SCreditsScreen::GetTogglePlayPauseBrush() const
{
	static FName PauseIcon(TEXT("Credits.Pause"));
	static FName PlayIcon(TEXT("Credits.Play"));

	if ( bIsPlaying )
	{
		return FAppStyle::GetBrush(PauseIcon);
	}
	else
	{
		return FAppStyle::GetBrush(PlayIcon);
	}
}

void SCreditsScreen::OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* url = Metadata.Find(TEXT("href"));

	if ( url != NULL )
	{
		FPlatformProcess::LaunchURL(**url, NULL, NULL);
	}
}

#undef LOCTEXT_NAMESPACE
