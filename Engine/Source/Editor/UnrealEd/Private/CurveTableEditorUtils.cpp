// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableEditorUtils.h"

#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "CurveTableEditorUtils"

FCurveTableEditorUtils::FCurveTableEditorManager& FCurveTableEditorUtils::FCurveTableEditorManager::Get()
{
	static TSharedRef< FCurveTableEditorManager > EditorManager(new FCurveTableEditorManager());
	return *EditorManager;
}

void FCurveTableEditorUtils::BroadcastPreChange(UCurveTable* CurveTable, ECurveTableChangeInfo Info)
{
	FCurveTableEditorManager::Get().PreChange(CurveTable, Info);
}

void FCurveTableEditorUtils::BroadcastPostChange(UCurveTable* CurveTable, ECurveTableChangeInfo Info)
{
	FCurveTableEditorManager::Get().PostChange(CurveTable, Info);
}

#undef LOCTEXT_NAMESPACE
