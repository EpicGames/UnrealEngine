// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableEditorUtils.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Styling/SlateTypes.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Styling/AppStyle.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Misc/StringUtility.h"
#include "ScopedTransaction.h"
#include "K2Node_GetDataTableRow.h"
#include "Input/Reply.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DetailWidgetRow.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "DataTableEditorUtils"

/** Combobox that allows selecting a struct row for a data table. Based off of SSearchableComboBox */
class SDataTableStructComboBox : public SComboButton
{
public:
	/** Type of list used for showing menu options. */
	typedef SListView< TSharedPtr<FString> > SComboListType;
	/** Delegate type used to generate widgets that represent Options */
	typedef typename TSlateDelegates< TSharedPtr<FString> >::FOnGenerateWidget FOnGenerateWidget;
	typedef typename TSlateDelegates< TSharedPtr<FString> >::FOnSelectionChanged FOnSelectionChanged;
	DECLARE_DELEGATE_OneParam(FOnFillComboBoxStrings, TArray<TSharedPtr<FString>>&);

	SLATE_BEGIN_ARGS(SDataTableStructComboBox)
		: _Content()
		, _ComboBoxStyle(&FCoreStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ItemStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _OnStructSelected()
		, _InitiallySelectedItem(nullptr)
		, _Method()
		, _MaxListHeight(450.0f)
		, _HasDownArrow(true)
	{}

	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

	/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
	SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

	SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)

	SLATE_ATTRIBUTE(FMargin, ContentPadding)
	SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

	SLATE_EVENT(FDataTableEditorUtils::FOnDataTableStructSelected, OnStructSelected)
	
	/** The custom scrollbar to use in the ListView */
	SLATE_ARGUMENT(TSharedPtr<SScrollBar>, CustomScrollbar)

	/** The option that should be selected when the combo box is first created */
	SLATE_ARGUMENT(TSharedPtr<FString>, InitiallySelectedItem)

	SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

	/** The max height of the combo box menu */
	SLATE_ARGUMENT(float, MaxListHeight)

	/**
	 * When false, the down arrow is not generated and it is up to the API consumer
	 * to make their own visual hint that this is a drop down.
	 */
	SLATE_ARGUMENT(bool, HasDownArrow)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	void Construct(const FArguments& InArgs);

	void ClearSelection();

	void SetSelectedItem(TSharedPtr<FString> InSelectedItem);

	/** @return the item currently selected by the combo box. */
	TSharedPtr<FString> GetSelectedItem();

	/**
	 * Requests a list refresh after updating options
	 * Call SetSelectedItem to update the selected item if required
	 * @see SetSelectedItem
	 */
	void RefreshOptions();

	/** Returns the asset data for a specific string, or null if not found */
	const FAssetData* FindAssetDataForString(TSharedPtr<FString> StringOption) const;

	/** Returns struct from AssetData, possibly loading it */
	UScriptStruct* GetOrLoadStruct(const FAssetData* AssetData);

private:

	/** Generate a row for the InItem in the combo box's list (passed in as OwnerTable). Do this by calling the user-specified OnGenerateWidget */
	TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called if the menu is closed */
	void OnMenuOpenChanged(bool bOpen);

	/** Invoked when the selection in the list changes */
	void OnSelectionChanged_Internal(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo);

	/** Invoked when the search text changes */
	void OnSearchTextChanged(const FText& ChangedText);

	/** Text to display inside box */
	FText GetSelectedText() const;

	/** Show tooltip text for a specific option */
	FText GetTooltipText(TSharedPtr<FString> StringOption);

	/** Handle clicking on the content menu */
	virtual FReply OnButtonClicked() override;

	/** The item style to use. */
	const FTableRowStyle* ItemStyle;

private:
	/** Delegate that is invoked when the selected item in the combo box changes */
	FDataTableEditorUtils::FOnDataTableStructSelected OnStructSelected;
	/** The padding around each menu row */
	FMargin MenuRowPadding;
	/** The item currently selected in the combo box */
	TSharedPtr<FString> SelectedItem;
	/** The search field used for the combox box's contents */
	TSharedPtr< SEditableTextBox > SearchField;
	/** The ListView that we pop up; visualized the available options. */
	TSharedPtr< SComboListType > ComboListView;
	/** The Scrollbar used in the ListView. */
	TSharedPtr< SScrollBar > CustomScrollbar;

	/** List of names to show in combo box, there is a 1:1 mapping to PossibleStructs */
	TArray< TSharedPtr<FString> > CurrentOptions;
	/** List of AssetData representing rows */
	TArray<FAssetData> PossibleStructs;
};

void SDataTableStructComboBox::Construct(const FArguments& InArgs)
{
	check(InArgs._ComboBoxStyle);

	ItemStyle = InArgs._ItemStyle;

	MenuRowPadding = InArgs._ComboBoxStyle->MenuRowPadding;

	// Work out which values we should use based on whether we were given an override, or should use the style's version
	const FComboButtonStyle& OurComboButtonStyle = InArgs._ComboBoxStyle->ComboButtonStyle;
	const FButtonStyle* const OurButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &OurComboButtonStyle.ButtonStyle;

	this->OnStructSelected = InArgs._OnStructSelected;

	CustomScrollbar = InArgs._CustomScrollbar;

	TSharedRef<SWidget> ComboBoxMenuContent =
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxListHeight)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(this->SearchField, SEditableTextBox)
				.HintText(LOCTEXT("Search", "Search"))
				.OnTextChanged(this, &SDataTableStructComboBox::OnSearchTextChanged)
			]

			+ SVerticalBox::Slot()
			[
				SAssignNew(this->ComboListView, SComboListType)
				.ListItemsSource(&CurrentOptions)
				.OnGenerateRow(this, &SDataTableStructComboBox::GenerateMenuItemRow)
				.OnSelectionChanged(this, &SDataTableStructComboBox::OnSelectionChanged_Internal)
				.SelectionMode(ESelectionMode::Single)
				.ExternalScrollbar(InArgs._CustomScrollbar)
			]
		];

	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock)
			.Text(this, &SDataTableStructComboBox::GetSelectedText);
	}

	SComboButton::Construct(SComboButton::FArguments()
		.ComboButtonStyle(&OurComboButtonStyle)
		.ButtonStyle(OurButtonStyle)
		.Method(InArgs._Method)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		.MenuContent()
		[
			ComboBoxMenuContent
		]
		.HasDownArrow(InArgs._HasDownArrow)
		.ContentPadding(InArgs._ContentPadding)
		.ForegroundColor(InArgs._ForegroundColor)
		.OnMenuOpenChanged(this, &SDataTableStructComboBox::OnMenuOpenChanged)
		.IsFocusable(true)
		);


	// Better to select search field so you can type right away
	SetMenuContentWidgetToFocus(SearchField);

	// Refresh options now
	RefreshOptions();

	// Need to establish the selected item at point of construction so its available for querying
	// NB: If you need a selection to fire use SetItemSelection rather than setting an IntiallySelectedItem
	SelectedItem = InArgs._InitiallySelectedItem;
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
	{
		ComboListView->Private_SetItemSelection(SelectedItem, true);
	}
}

void SDataTableStructComboBox::ClearSelection()
{
	ComboListView->ClearSelection();
}

void SDataTableStructComboBox::SetSelectedItem(TSharedPtr<FString> InSelectedItem)
{
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(InSelectedItem))
	{
		ComboListView->SetSelection(InSelectedItem);
	}
	else
	{
		ComboListView->ClearSelection();
	}
}

TSharedPtr<FString> SDataTableStructComboBox::GetSelectedItem()
{
	return SelectedItem;
}

FText SDataTableStructComboBox::GetSelectedText() const
{
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

FText SDataTableStructComboBox::GetTooltipText(TSharedPtr<FString> StringOption)
{
	const FAssetData* FoundAsset = FindAssetDataForString(StringOption);

	if (FoundAsset)
	{
		return FText::FromString(FoundAsset->PackageName.ToString());
	}
	return FText::GetEmpty();
}

void SDataTableStructComboBox::RefreshOptions()
{
	if (PossibleStructs.Num() == 0)
	{
		FDataTableEditorUtils::GetPossibleStructAssetData(PossibleStructs);

		CurrentOptions.Reset();
		for (const FAssetData& FoundStruct : PossibleStructs)
		{
			CurrentOptions.Add(MakeShareable(new FString(FoundStruct.AssetName.ToString())));
		}
	}

	if (!ComboListView->IsPendingRefresh())
	{
		ComboListView->RequestListRefresh();
	}
}

const FAssetData* SDataTableStructComboBox::FindAssetDataForString(TSharedPtr<FString> StringOption) const
{
	check(CurrentOptions.Num() == PossibleStructs.Num());
	for (int32 i = 0; i < CurrentOptions.Num(); i++)
	{
		if (StringOption == CurrentOptions[i])
		{
			return &PossibleStructs[i];
		}
	}
	return nullptr;
}

UScriptStruct* SDataTableStructComboBox::GetOrLoadStruct(const FAssetData* AssetData)
{
	if (!AssetData)
	{
		return nullptr;
	}

	return Cast<UScriptStruct>(AssetData->GetAsset());
}

TSharedRef<ITableRow> SDataTableStructComboBox::GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FString SearchToken = SearchField->GetText().ToString().ToLower();
	EVisibility WidgetVisibility = EVisibility::Visible;
	if (!SearchToken.IsEmpty())
	{
		if (InItem->ToLower().Find(SearchToken) < 0)
		{
			WidgetVisibility = EVisibility::Collapsed;
		}
	}
	
	TAttribute<FText> OnGetToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SDataTableStructComboBox::GetTooltipText, InItem));

	return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
		.Style(ItemStyle)
		.Visibility(WidgetVisibility)
		.Padding(MenuRowPadding)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*InItem))
			.ToolTipText(OnGetToolTip)
		];
}

void SDataTableStructComboBox::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
		{
			// Ensure the ListView selection is set back to the last committed selection
			ComboListView->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
			ComboListView->RequestScrollIntoView(SelectedItem, 0);
		}

		// Set focus back to ComboBox for users focusing the ListView that just closed
		TSharedRef<SWidget> ThisRef = AsShared();
		FSlateApplication::Get().ForEachUser([&ThisRef](FSlateUser& User) {
			if (User.HasFocusedDescendants(ThisRef))
			{
				User.SetFocus(ThisRef, EFocusCause::SetDirectly);
			}
		});
	}
}

void SDataTableStructComboBox::OnSelectionChanged_Internal(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo)
{
	// Ensure that the proposed selection is different
	if (SelectInfo != ESelectInfo::OnNavigation)
	{
		// Ensure that the proposed selection is different from selected
		if (ProposedSelection != SelectedItem)
		{
			SelectedItem = ProposedSelection;
			
			UScriptStruct* SelectedStruct = GetOrLoadStruct(FindAssetDataForString(SelectedItem));

			OnStructSelected.ExecuteIfBound(SelectedStruct);

		}
		// close combo even if user reselected item
		this->SetIsOpen(false);
	}
}

void SDataTableStructComboBox::OnSearchTextChanged(const FText& ChangedText)
{
	FString SearchToken = ChangedText.ToString().ToLower();
	for (int32 i = 0; i < CurrentOptions.Num(); i++)
	{
		TSharedPtr<ITableRow> Row = ComboListView->WidgetFromItem(CurrentOptions[i]);
		if (Row)
		{
			if (SearchToken.IsEmpty())
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else if (CurrentOptions[i]->ToLower().Find(SearchToken) >= 0)
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else
			{
				Row->AsWidget()->SetVisibility(EVisibility::Collapsed);
			}
		}
	}

	ComboListView->RequestListRefresh();

	SelectedItem = TSharedPtr< FString >();
}

FReply SDataTableStructComboBox::OnButtonClicked()
{
	// if user clicked to close the combo menu
	if (this->IsOpen())
	{
		// Re-select first selected item, just in case it was selected by navigation previously
		TArray<TSharedPtr<FString>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
		}
	}
	else
	{
		SearchField->SetText(FText::GetEmpty());
		RefreshOptions();
	}

	return SComboButton::OnButtonClicked();
}

const FString FDataTableEditorUtils::VariableTypesTooltipDocLink = TEXT("Shared/Editor/Blueprint/VariableTypes");

TSharedRef<SWidget> FDataTableEditorUtils::MakeRowStructureComboBox(FOnDataTableStructSelected OnSelected)
{
	TSharedRef<SDataTableStructComboBox> ComboBox = SNew(SDataTableStructComboBox)
		.OnStructSelected(OnSelected);

	return ComboBox;
}

FDataTableEditorUtils::FDataTableEditorManager& FDataTableEditorUtils::FDataTableEditorManager::Get()
{
	static TSharedRef< FDataTableEditorManager > EditorManager(new FDataTableEditorManager());
	return *EditorManager;
}

bool FDataTableEditorUtils::RemoveRow(UDataTable* DataTable, FName Name)
{
	bool bResult = false;
	if (DataTable && DataTable->RowStruct)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveDataTableRow", "Remove Data Table Row"));

		BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
		DataTable->Modify();
		uint8* RowData = nullptr;
		const bool bRemoved = DataTable->GetNonConstRowMap().RemoveAndCopyValueStable(Name, RowData);
		if (bRemoved && RowData)
		{
			DataTable->RowStruct->DestroyStruct(RowData);
			FMemory::Free(RowData);
			bResult = true;
		}
		BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);
	}
	return bResult;
}

uint8* FDataTableEditorUtils::AddRow(UDataTable* DataTable, FName RowName)
{
	if (!DataTable || (RowName == NAME_None) || (DataTable->GetRowMap().Find(RowName) != nullptr) || !DataTable->RowStruct)
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddDataTableRow", "Add Data Table Row"));

	BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
	DataTable->Modify();
	// Allocate data to store information, using UScriptStruct to know its size
	uint8* RowData = (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());
	DataTable->RowStruct->InitializeStruct(RowData);
	// And be sure to call DestroyScriptStruct later

	// Add to row map
	DataTable->AddRowInternal(RowName, RowData);
	DataTable->HandleDataTableChanged(RowName);
	BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);
	return RowData;
}

uint8* FDataTableEditorUtils::AddRowAboveOrBelowSelection(UDataTable* DataTable, const FName& RowName, const FName& NewRowName, ERowInsertionPosition InsertPosition)
{
	if (!DataTable || (NewRowName == NAME_None) || (DataTable->GetRowMap().Find(NewRowName) != nullptr) || !DataTable->RowStruct)
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddDataTableRowAboveBelow", "Add Data Table Row Above or Below"));

	TArray<FName> OrderedRowNames;
	DataTable->GetRowMap().GenerateKeyArray(OrderedRowNames);

	int32 CurrentRowIndex = OrderedRowNames.IndexOfByKey(RowName);
	if (CurrentRowIndex == INDEX_NONE)
	{
		return nullptr;
	}

	if (InsertPosition == ERowInsertionPosition::Below)
	{
		CurrentRowIndex += 1;
	}

	OrderedRowNames.Insert(NewRowName, CurrentRowIndex);
	
	// Build a name -> index map as the KeySort will hit this a lot
	TMap<FName, int32> NamesToNewIndex;
	for (int32 NameIndex = 0; NameIndex < OrderedRowNames.Num(); ++NameIndex)
	{
		NamesToNewIndex.Add(OrderedRowNames[NameIndex], NameIndex);
	}
	
	
	BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
	
	DataTable->Modify();
	
	// Allocate data to store information, using UScriptStruct to know its size
	uint8* RowData = (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());

	// And be sure to call DestroyScriptStruct later
	DataTable->RowStruct->InitializeStruct(RowData);

	// Add to row map
	DataTable->AddRowInternal(NewRowName, RowData);

	// Re-sort the map keys to match the new order
	DataTable->GetNonConstRowMap().KeySort([&NamesToNewIndex](const FName& One, const FName& Two) -> bool
	{
		const int32 OneIndex = NamesToNewIndex.FindRef(One);
		const int32 TwoIndex = NamesToNewIndex.FindRef(Two);
		return OneIndex < TwoIndex;
	});

	BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);

	return RowData;
}


uint8* FDataTableEditorUtils::DuplicateRow(UDataTable* DataTable, FName SourceRowName, FName RowName)
{
	if (!DataTable || (SourceRowName == NAME_None) || !DataTable->RowMap.Contains(SourceRowName) || DataTable->RowMap.Contains(RowName) || !DataTable->RowStruct)
	{
		return NULL;
	}

	const FScopedTransaction Transaction(LOCTEXT("DuplicateDataTableRow", "Duplicate Data Table Row"));

	BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
	DataTable->Modify();

	// Allocate data to store information, using UScriptStruct to know its size
	uint8* OldRowData = *DataTable->RowMap.Find(SourceRowName);
	uint8* NewRowData = (uint8*)FMemory::Malloc(DataTable->RowStruct->GetStructureSize());

	DataTable->RowStruct->InitializeStruct(NewRowData);
	DataTable->RowStruct->CopyScriptStruct(NewRowData, OldRowData);

	// Add to row map
	DataTable->RowMap.Add(RowName, NewRowData);
	BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);
	return NewRowData;
}

bool FDataTableEditorUtils::RenameRow(UDataTable* DataTable, FName OldName, FName NewName)
{
	bool bResult = false;
	if (DataTable)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameDataTableRow", "Rename Data Table Row"));

		BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
		DataTable->Modify();

		uint8* RowData = nullptr;
		const bool bValidnewName = (NewName != NAME_None) && !DataTable->GetRowMap().Find(NewName);
		const bool bRemoved = bValidnewName && DataTable->GetNonConstRowMap().RemoveAndCopyValue(OldName, RowData);
		if (bRemoved)
		{
			DataTable->GetNonConstRowMap().FindOrAdd(NewName) = RowData;
			bResult = true;
		}
		BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);
	}
	return bResult;
}

bool FDataTableEditorUtils::MoveRow(UDataTable* DataTable, FName RowName, ERowMoveDirection Direction, int32 NumRowsToMoveBy)
{
	if (!DataTable)
	{
		return false;
	}
	
	// Our maps are ordered which is why we can get away with this
	// If we ever change our map implementation, we'll need to preserve this order information in a separate array and 
	// make sure that order dependent code (such as exporting and the data table viewer) use that when dealing with rows
	// This may also require making RowMap private and fixing up all the existing code that references it directly
	TArray<FName> OrderedRowNames;
	DataTable->GetRowMap().GenerateKeyArray(OrderedRowNames);

	const int32 CurrentRowIndex = OrderedRowNames.IndexOfByKey(RowName);
	if (CurrentRowIndex == INDEX_NONE)
	{
		return false;
	}
	
	// Calculate our new row index, clamped to the available rows
	int32 NewRowIndex = INDEX_NONE;
	switch(Direction)
	{
	case ERowMoveDirection::Up:
		NewRowIndex = FMath::Clamp(CurrentRowIndex - NumRowsToMoveBy, 0, OrderedRowNames.Num() - 1);
		break;

	case ERowMoveDirection::Down:
		NewRowIndex = FMath::Clamp(CurrentRowIndex + NumRowsToMoveBy, 0, OrderedRowNames.Num() - 1);
		break;

	default:
		break;
	}

	if (NewRowIndex == INDEX_NONE)
	{
		return false;
	}

	if (CurrentRowIndex == NewRowIndex)
	{
		// Nothing to do, but not an error
		return true;
	}

	// Swap the order around as requested
	OrderedRowNames.RemoveAt(CurrentRowIndex, EAllowShrinking::No);
	OrderedRowNames.Insert(RowName, NewRowIndex);

	// Build a name -> index map as the KeySort will hit this a lot
	TMap<FName, int32> NamesToNewIndex;
	for (int32 NameIndex = 0; NameIndex < OrderedRowNames.Num(); ++NameIndex)
	{
		NamesToNewIndex.Add(OrderedRowNames[NameIndex], NameIndex);
	}

	const FScopedTransaction Transaction(LOCTEXT("MoveDataTableRow", "Move Data Table Row"));

	BroadcastPreChange(DataTable, EDataTableChangeInfo::RowList);
	DataTable->Modify();

	// Re-sort the map keys to match the new order
	DataTable->GetNonConstRowMap().KeySort([&NamesToNewIndex](const FName& One, const FName& Two) -> bool
	{
		const int32 OneIndex = NamesToNewIndex.FindRef(One);
		const int32 TwoIndex = NamesToNewIndex.FindRef(Two);
		return OneIndex < TwoIndex;
	});

	BroadcastPostChange(DataTable, EDataTableChangeInfo::RowList);

	return true;
}

bool FDataTableEditorUtils::SelectRow(const UDataTable* DataTable, FName RowName)
{
	for (auto Listener : FDataTableEditorManager::Get().GetListeners())
	{
		static_cast<INotifyOnDataTableChanged*>(Listener)->SelectionChange(DataTable, RowName);
	}
	return true;
}

bool FDataTableEditorUtils::DiffersFromDefault(UDataTable* DataTable, FName RowName)
{
	bool bDiffers = false;

	if (DataTable && DataTable->GetRowMap().Contains(RowName))
	{
		uint8* RowData = DataTable->GetRowMap()[RowName];

		if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(DataTable->RowStruct))
		{
			return !UDStruct->CompareScriptStruct(RowData, UDStruct->GetDefaultInstance(), PPF_None);
		}
	}

	return bDiffers;
}

bool FDataTableEditorUtils::ResetToDefault(UDataTable* DataTable, FName RowName)
{
	bool bResult = false;

	if (DataTable && DataTable->GetRowMap().Contains(RowName))
	{
		const FScopedTransaction Transaction(LOCTEXT("ResetDataTableRowToDefault", "Reset Data Table Row to Default Values"));

		BroadcastPreChange(DataTable, EDataTableChangeInfo::RowData);
		DataTable->Modify();

		uint8* RowData = DataTable->GetRowMap()[RowName];

		if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(DataTable->RowStruct))
		{
			UDStruct->InitializeDefaultValue(RowData);
			bResult = true;
		}

		BroadcastPostChange(DataTable, EDataTableChangeInfo::RowData);
	}

	return bResult;
}

void FDataTableEditorUtils::BroadcastPreChange(UDataTable* DataTable, EDataTableChangeInfo Info)
{
	FDataTableEditorManager::Get().PreChange(DataTable, Info);
}

void FDataTableEditorUtils::BroadcastPostChange(UDataTable* DataTable, EDataTableChangeInfo Info)
{
	if (DataTable && (EDataTableChangeInfo::RowList == Info))
	{
		for (TObjectIterator<UK2Node_GetDataTableRow> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
		{
			It->OnDataTableRowListChanged(DataTable);
		}
	}
	FDataTableEditorManager::Get().PostChange(DataTable, Info);
	DataTable->OnDataTableChanged().Broadcast();
}

void FDataTableEditorUtils::CacheDataTableForEditing(const UDataTable* DataTable, TArray<FDataTableEditorColumnHeaderDataPtr>& OutAvailableColumns, TArray<FDataTableEditorRowListViewDataPtr>& OutAvailableRows)
{
	if (!DataTable || !DataTable->RowStruct)
	{
		OutAvailableColumns.Empty();
		OutAvailableRows.Empty();
		return;
	}

	CacheDataForEditing(DataTable->RowStruct, DataTable->GetRowMap(), OutAvailableColumns, OutAvailableRows);
}

void FDataTableEditorUtils::CacheDataForEditing(const UScriptStruct* RowStruct, const TMap<FName, uint8*>& RowMap, TArray<FDataTableEditorColumnHeaderDataPtr>& OutAvailableColumns, TArray<FDataTableEditorRowListViewDataPtr>& OutAvailableRows)
{
	TArray<FDataTableEditorColumnHeaderDataPtr> OldColumns = OutAvailableColumns;
	TArray<FDataTableEditorRowListViewDataPtr> OldRows = OutAvailableRows;

	// First build array of properties
	TArray<const FProperty*> StructProps;
	for (TFieldIterator<const FProperty> It(RowStruct); It; ++It)
	{
		const FProperty* Prop = *It;
		check(Prop);
		if (!Prop->HasAnyPropertyFlags(CPF_Deprecated) && !Prop->HasMetaData(FName(TEXT("HideFromDataTableEditorColumn"))))
		{
			StructProps.Add(Prop);
		}
	}

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	static const float CellPadding = 10.0f;

	// Populate the column data
	OutAvailableColumns.Reset(StructProps.Num());
	for (int32 Index = 0; Index < StructProps.Num(); ++Index)
	{
		const FProperty* Prop = StructProps[Index];
		const FText PropertyDisplayName = DataTableUtils::GetPropertyDisplayName(Prop, FName::NameToDisplayString(Prop->GetName(), Prop->IsA<FBoolProperty>()));

		FDataTableEditorColumnHeaderDataPtr CachedColumnData;
		
		// If at all possible, attempt to reuse previous columns if their data has not changed
		if (Index >= OldColumns.Num() || OldColumns[Index]->ColumnId != Prop->GetFName() || !OldColumns[Index]->DisplayName.EqualTo(PropertyDisplayName))
		{
			CachedColumnData = MakeShareable(new FDataTableEditorColumnHeaderData());
			CachedColumnData->ColumnId = Prop->GetFName();
			CachedColumnData->DisplayName = PropertyDisplayName;
			CachedColumnData->Property = Prop;
		}
		else
		{
			CachedColumnData = OldColumns[Index];

			// Need to update property hard pointer in case it got reconstructed
			CachedColumnData->Property = Prop;
		}

		CachedColumnData->DesiredColumnWidth = static_cast<float>(FontMeasure->Measure(CachedColumnData->DisplayName, CellTextStyle.Font).X + CellPadding);

		OutAvailableColumns.Add(CachedColumnData);
	}

	// Populate the row data
	OutAvailableRows.Reset(RowMap.Num());
	int32 Index = 0;
	for (auto RowIt = RowMap.CreateConstIterator(); RowIt; ++RowIt, ++Index)
	{
		FText RowName = FText::FromName(RowIt->Key);
		FDataTableEditorRowListViewDataPtr CachedRowData;

		// If at all possible, attempt to reuse previous rows if their data has not changed.
		if (Index >= OldRows.Num() || OldRows[Index]->RowId != RowIt->Key || !OldRows[Index]->DisplayName.EqualTo(RowName))
		{
			CachedRowData = MakeShareable(new FDataTableEditorRowListViewData());
			CachedRowData->RowId = RowIt->Key;
			CachedRowData->DisplayName = RowName;
			CachedRowData->CellData.Reserve(StructProps.Num());
		}
		else
		{
			CachedRowData = OldRows[Index];
			CachedRowData->CellData.Reset(StructProps.Num());
		}

		CachedRowData->DesiredRowHeight = FontMeasure->GetMaxCharacterHeight(CellTextStyle.Font);
		CachedRowData->RowNum = Index + 1;

		// Always rebuild cell data
		{
			const uint8* RowData = RowIt.Value();
			for (int32 ColumnIndex = 0; ColumnIndex < StructProps.Num(); ++ColumnIndex)
			{
				const FProperty* Prop = StructProps[ColumnIndex];
				FDataTableEditorColumnHeaderDataPtr CachedColumnData = OutAvailableColumns[ColumnIndex];

				const FText CellText = DataTableUtils::GetPropertyValueAsText(Prop, RowData);
				CachedRowData->CellData.Add(CellText);

				const FVector2D CellTextSize = FontMeasure->Measure(CellText, CellTextStyle.Font);

				CachedRowData->DesiredRowHeight = static_cast<float>(FMath::Max(CachedRowData->DesiredRowHeight, CellTextSize.Y));

				const float CellWidth = static_cast<float>(CellTextSize.X + CellPadding);
				CachedColumnData->DesiredColumnWidth = FMath::Max(CachedColumnData->DesiredColumnWidth, CellWidth);
			}
		}

		OutAvailableRows.Add(CachedRowData);
	}
}

TArray<UScriptStruct*> FDataTableEditorUtils::GetPossibleStructs()
{
	TArray< UScriptStruct* > RowStructs;

	// Make combo of table rowstruct options
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		UScriptStruct* Struct = *It;
		if (IsValidTableStruct(Struct))
		{
			RowStructs.Add(Struct);
		}
	}

	RowStructs.Sort();

	return RowStructs;
}

void FDataTableEditorUtils::GetPossibleStructAssetData(TArray<FAssetData>& StructAssets)
{
	StructAssets.Reset();

	// Make combo of table rowstruct options
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		UScriptStruct* Struct = *It;
		if (IsValidTableStruct(Struct))
		{
			StructAssets.Add(FAssetData(Struct));
		}
	}

	// Now get unloaded ones
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<FAssetData> AssetData;
	AssetRegistryModule.Get().GetAssetsByClass(UUserDefinedStruct::StaticClass()->GetClassPathName(), AssetData);

	for (int32 AssetIndex = 0; AssetIndex < AssetData.Num(); ++AssetIndex)
	{
		const FAssetData& Asset = AssetData[AssetIndex];
		if (Asset.IsValid() && !Asset.IsAssetLoaded())
		{
			StructAssets.Add(Asset);
		}
	}

	StructAssets.Sort([](const FAssetData& A, const FAssetData& B) { return A.AssetName.LexicalLess(B.AssetName); });
}

bool FDataTableEditorUtils::IsValidTableStruct(const UScriptStruct* Struct)
{
	const UScriptStruct* TableRowStruct = FTableRowBase::StaticStruct();

	// If a child of the table row struct base, but not itself
	const bool bBasedOnTableRowBase = TableRowStruct && Struct->IsChildOf(TableRowStruct) && (Struct != TableRowStruct);
	const bool bUDStruct = Struct->IsA<UUserDefinedStruct>();
	const bool bValidStruct = (Struct->GetOutermost() != GetTransientPackage());

	return (bBasedOnTableRowBase || bUDStruct) && bValidStruct;
}

void FDataTableEditorUtils::AddSearchForReferencesContextMenu(FDetailWidgetRow& RowNameDetailWidget, FExecuteAction SearchForReferencesAction)
{
	if (SearchForReferencesAction.IsBound() && FEditorDelegates::OnOpenReferenceViewer.IsBound())
	{
		RowNameDetailWidget.AddCustomContextMenuAction(FUIAction(SearchForReferencesAction),
			NSLOCTEXT("FDataTableRowUtils", "FDataTableRowUtils_SearchForReferences", "Find Row References"),
			NSLOCTEXT("FDataTableRowUtils", "FDataTableRowUtils_SearchForReferencesTooltip", "Find assets that reference this Row"),
			FSlateIcon());
	}
}

FText FDataTableEditorUtils::GetHandleShortDescription(const UObject* TableAsset, FName RowName)
{
	FText TableNameText = LOCTEXT("Description_None", "None");
	FText RowNameText = TableNameText;
	const int32 MaxChars = 15;
	FString More = TEXT("...");

	if (!TableAsset && RowName.IsNone())
	{
		// Just display None on it's own
		return TableNameText;
	}

	if (TableAsset)
	{
		FString TempString = TableAsset->GetName();

		// Chop off end if needed
		if (TempString.Len() > MaxChars)
		{
			TempString.LeftInline(MaxChars - More.Len());
			TempString.Append(More);
		}

		TableNameText = FText::AsCultureInvariant(TempString);
	}

	if (!RowName.IsNone())
	{
		FString TempString = RowName.ToString();

		// Show right side if too long, usually more important
		if (TempString.Len() > MaxChars)
		{
			TempString.RightInline(MaxChars - More.Len());
			TempString.InsertAt(0, More);
		}

		RowNameText = FText::AsCultureInvariant(TempString);
	}

	return FText::Format(LOCTEXT("HandlePreviewFormat", "{0}[{1}]"), TableNameText, RowNameText);
}

FText FDataTableEditorUtils::GetRowTypeInfoTooltipText(FDataTableEditorColumnHeaderDataPtr ColumnHeaderDataPtr)
{
	if (ColumnHeaderDataPtr.IsValid())
	{
		const FProperty* Property = ColumnHeaderDataPtr->Property;
		if (Property)
		{
			const FFieldClass* PropertyClass = Property->GetClass();
			const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
			if (StructProp)
			{
				FString TypeName = FName::NameToDisplayString(Property->GetCPPType(), Property->IsA<FBoolProperty>());
				if (TypeName.Len())
				{
					// If type name starts with F and another capital letter, assume standard naming and remove F in the string shown to the user
					if (TypeName.StartsWith("F", ESearchCase::CaseSensitive) && TypeName.Len() > 1 && FChar::IsUpper(TypeName.GetCharArray()[1]))
					{
						TypeName.RemoveFromStart("F");
					}
					return FText::FromString(TypeName);
				}
			}
			if (PropertyClass)
			{
				return FText::FromString(PropertyClass->GetDescription());
			}
			
		}
	}

	return FText::GetEmpty();
}

FString FDataTableEditorUtils::GetRowTypeTooltipDocExcerptName(FDataTableEditorColumnHeaderDataPtr ColumnHeaderDataPtr)
{
	if (ColumnHeaderDataPtr.IsValid())
	{
		const FProperty* Property = ColumnHeaderDataPtr->Property;
		if (Property)
		{
			const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
			if (StructProp)
			{
				if (StructProp->Struct == TBaseStructure<FSoftObjectPath>::Get())
				{
					return "SoftObject";
				}
				if (StructProp->Struct == TBaseStructure<FSoftClassPath>::Get())
				{
					return "SoftClass";
				}
				FString TypeName = FName::NameToDisplayString(Property->GetCPPType(), Property->IsA<FBoolProperty>());
				if (TypeName.Len())
				{
					// If type name starts with F and another capital letter, assume standard naming and remove F to match the doc excerpt name
					if (TypeName.StartsWith("F", ESearchCase::CaseSensitive) && TypeName.Len() > 1 && FChar::IsUpper(TypeName.GetCharArray()[1]))
					{
						TypeName.RemoveFromStart("F");
					}
					return TypeName;
				}
			}
			const FFieldClass* PropertyClass = Property->GetClass();
			if (PropertyClass)
			{
				if (PropertyClass == FStrProperty::StaticClass())
				{
					return "String";
				}
				FString PropertyClassName = PropertyClass->GetName();
				PropertyClassName.RemoveFromEnd("Property");
				return PropertyClassName;
			}
		}
	}

	return "";
}

#undef LOCTEXT_NAMESPACE
