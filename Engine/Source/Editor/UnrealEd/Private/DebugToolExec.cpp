// Copyright Epic Games, Inc. All Rights Reserved.


#include "DebugToolExec.h"
#include "CollisionQueryParams.h"
#include "Engine/GameInstance.h"
#include "Engine/HitResult.h"
#include "GameFramework/Pawn.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "Engine/GameEngine.h"
#include "GameFramework/PlayerController.h"
#include "EngineUtils.h"
#include "EngineGlobals.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"


/**
 * Brings up a property window to edit the passed in object.
 *
 * @param Object	property to edit
 * @param bShouldShowNonEditable	whether to show properties that are normally not editable under "None"
 */
void FDebugToolExec::EditObject(UObject* Object, bool bShouldShowNonEditable)
{
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	// the effects of this cannot be easily reversed, so prevent the user from playing network games without restarting to avoid potential exploits
	GDisallowNetworkTravel = true;
#endif

	struct Local
	{
		/** Delegate to show all properties */
		static bool IsPropertyVisible(  const FPropertyAndParent& PropertyAndParent, bool bInShouldShowNonEditable )
		{
			return bInShouldShowNonEditable;
		}
	};

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedPtr<IDetailsView> DetailsView = PropertyModule.CreateDetailView(Args);
	DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&Local::IsPropertyVisible, bShouldShowNonEditable));
	DetailsView->SetObject(Object);
	
	// create Slate property window
	FSlateApplication::Get().AddWindow
	( 
		SNew(SWindow)
		.ClientSize(FVector2D(400,600))
		.Title( FText::FromString( Object->GetName() ) )
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					DetailsView.ToSharedRef()
				]
			]
		]
	);
}

/**
 * Exec handler, parsing the passed in command
 *
 * @param InWorld World Context
 * @param Cmd	Command to parse
 * @param Ar	output device used for logging
 */
bool FDebugToolExec::Exec_Editor( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	// these commands are only allowed in standalone games
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (GEngine->GetNetMode(InWorld) != NM_Standalone || (GEngine->GetWorldContextFromWorldChecked(InWorld).PendingNetGame != NULL))
	{
		return 0;
	}
	// Edits the class defaults.
	else
#endif
	if( FParse::Command(&Cmd,TEXT("EDITDEFAULT")) )
	{
		// not allowed in the editor as this command can have far reaching effects such as impacting serialization
		if (!GIsEditor)
		{
			UClass* Class = nullptr;
			FString ClassName;
			if (FParse::Value(Cmd, TEXT("CLASS="), ClassName))
			{
				Class = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing FDebugToolExec class"));
			}
			else if (FParse::Token(Cmd, ClassName, true))
			{
				Class = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing FDebugToolExec class"));
			}			

			if (Class)
			{
				EditObject(Class->GetDefaultObject(), true);
			}
			else
			{
				Ar.Logf( TEXT("Missing class") );
			}
		}
		return 1;
	}
	else if (FParse::Command(&Cmd,TEXT("EDITOBJECT")))
	{
		UClass* SearchClass = nullptr;
		UObject* FoundObj = nullptr;
		FString ClassName;
		// Search by class.
		if (FParse::Value(Cmd, TEXT("CLASS="), ClassName))
		{
			SearchClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing FDebugToolExec class"));
		}
		if (SearchClass)
		{
			// pick the first valid object
			for (FThreadSafeObjectIterator It(SearchClass); It && FoundObj == NULL; ++It)
			{
				if (IsValid(*It) && !It->IsTemplate())
				{
					FoundObj = *It;
				}
			}
		}
		// Search by name.
		else
		{
			FName searchName;
			FString SearchPathName;
			if ( FParse::Value(Cmd, TEXT("NAME="), searchName) )
			{
				// Look for actor by name.
				for( TObjectIterator<UObject> It; It && FoundObj == NULL; ++It )
				{
					if (It->GetFName() == searchName) 
					{
						FoundObj = *It;
					}
				}
			}
			else if ( FParse::Token(Cmd,SearchPathName, true) )
			{
				FoundObj = FindFirstObject<UObject>(*SearchPathName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing FDebugToolExec object"));
			}
		}

		// Bring up an property editing window for the found object.
		if (FoundObj != NULL)
		{
			// not allowed in the editor unless it is a PIE object as this command can have far reaching effects such as impacting serialization
			if (!GIsEditor || (!FoundObj->IsTemplate() && FoundObj->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor)))
			{
				EditObject(FoundObj, true);
			}
		}
		else
		{
			Ar.Logf(TEXT("Target not found"));
		}
		return 1;
	}
	else if (FParse::Command(&Cmd,TEXT("EDITARCHETYPE")))
	{
		UObject* foundObj = NULL;
		// require fully qualified path name
		FString SearchPathName;
		if (FParse::Token(Cmd, SearchPathName, true))
		{
			foundObj = FindFirstObject<UObject>(*SearchPathName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("EDITARCHETYPE"));
		}

		// Bring up an property editing window for the found object.
		if (foundObj != NULL)
		{
			// not allowed in the editor unless it is a PIE object as this command can have far reaching effects such as impacting serialization
			if (!GIsEditor || (!foundObj->IsTemplate() && foundObj->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor)))
			{
				EditObject(foundObj, false);
			}
		}
		else
		{
			Ar.Logf(TEXT("Target not found"));
		}
		return 1;
	}
	// Edits an objects properties or copies them to the clipboard.
	else if( FParse::Command(&Cmd,TEXT("EDITACTOR")) )
	{
		UClass*	Class = nullptr;
		AActor*	Found = nullptr;
		FString ClassName;

		if (FParse::Command(&Cmd, TEXT("TRACE")))
		{
			APlayerController* PlayerController = InWorld->GetGameInstance() ? InWorld->GetGameInstance()->GetFirstLocalPlayerController() : nullptr;
			if (PlayerController != nullptr)
			{
				// Do a trace in the player's facing direction and edit anything that's hit.
				FVector PlayerLocation;
				FRotator PlayerRotation;
				PlayerController->GetPlayerViewPoint(PlayerLocation, PlayerRotation);
				FHitResult Hit(1.0f);
				PlayerController->GetWorld()->LineTraceSingleByChannel(Hit, PlayerLocation, PlayerLocation + PlayerRotation.Vector() * 10000.f, ECC_Pawn, FCollisionQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true, PlayerController->GetPawn()));
				Found = Hit.GetHitObjectHandle().FetchActor();
			}
		}
		// Search by class.
		else if (FParse::Value(Cmd, TEXT("CLASS="), ClassName))
		{
			Class = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("parsing FDebugToolExec class"));
			if (Class)
			{
				UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);

				// Look for the closest actor of this class to the player.
				FVector PlayerLocation(0.0f);
				APlayerController* PlayerController = InWorld->GetGameInstance() ? InWorld->GetGameInstance()->GetFirstLocalPlayerController() : nullptr;
				if (PlayerController != NULL)
				{
					FRotator DummyRotation;
					PlayerController->GetPlayerViewPoint(PlayerLocation, DummyRotation);
				}

				float   MinDist = FLT_MAX;
				for (TActorIterator<AActor> It(InWorld, Class); It; ++It)
				{
					if (IsValid(*It))
					{
						const double Dist = (PlayerController && It->GetRootComponent()) ? FVector::Dist(It->GetActorLocation(), PlayerLocation) : 0.f;
						if (Dist < MinDist)
						{
							MinDist = Dist;
							Found = *It;
						}
					}
				}
			}
		}
		// Search by name.
		else
		{
			FName ActorName;
			if( FParse::Value( Cmd, TEXT("NAME="), ActorName ) )
			{
				// Look for actor by name.
				for( FActorIterator It(InWorld); It; ++It )
				{
					if( It->GetFName() == ActorName )
					{
						Found = *It;
						break;
					}
				}
			}
		}

		// Bring up an property editing window for the found object.
		if( Found )
		{
			// not allowed in the editor unless it is a PIE object as this command can have far reaching effects such as impacting serialization
			if (!GIsEditor || (!Found->IsTemplate() && Found->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor)))
			{
				EditObject(Found, true);
			}
		}
		else
		{
			Ar.Logf( TEXT("Target not found") );
		}

		return 1;
	}
	else
	{
		return 0;
	}
}
