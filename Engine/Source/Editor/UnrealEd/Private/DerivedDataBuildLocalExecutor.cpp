// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataBuildLocalExecutor.h"

#include "Async/ManualResetEvent.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SharedString.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildAction.h"
#include "DerivedDataBuildInputs.h"
#include "DerivedDataBuildOutput.h"
#include "DerivedDataBuildTypes.h"
#include "DerivedDataBuildWorker.h"
#include "DerivedDataRequestTypes.h"
#include "DerivedDataValue.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IO/IoHash.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"

#include <atomic>

namespace UE::DerivedData
{

DEFINE_LOG_CATEGORY_STATIC(LogDerivedDataBuildLocalExecutor, Log, All);

/**
 * This implements a simple local build worker executor which spawns all jobs
 * as local processes. This is intentionally as simple as possible and
 * everything is synchronous. This is not meant to be used in production,
 * and is perhaps primarily useful as a debugging aid.
 */
class FLocalBuildWorkerExecutor final : public IBuildWorkerExecutor
{
public:
	FLocalBuildWorkerExecutor()
	{
		// Delete previous local execution state from the primary process.
		if (GetMultiprocessId() == 0)
		{
			TStringBuilder<256> DeleteRoot(InPlace, SandboxRoot, '.', FGuid::NewGuid());
			if (IFileManager::Get().DirectoryExists(*SandboxRoot))
			{
				UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Display, "Deleting existing local execution state from '{SandboxRoot}'.", *SandboxRoot);
				IFileManager::Get().Move(*DeleteRoot, *SandboxRoot);
				IFileManager::Get().DeleteDirectory(*DeleteRoot, /*bRequireExists*/ false, /*bTree*/ true);
			}
		}

		IModularFeatures::Get().RegisterModularFeature(IBuildWorkerExecutor::FeatureName, this);
	}

	~FLocalBuildWorkerExecutor() final
	{
		IModularFeatures::Get().UnregisterModularFeature(IBuildWorkerExecutor::FeatureName, this);
	}

	void Build(
		const FBuildAction& Action,
		const FOptionalBuildInputs& Inputs,
		const FBuildPolicy& Policy,
		const FBuildWorker& Worker,
		IBuild& BuildSystem,
		IRequestOwner& Owner,
		FOnBuildWorkerActionComplete&& OnComplete) final
	{
		// Review build action inputs to determine if they need to be materialized/propagated
		// (right now, they always will be)

		TArray<FUtf8StringView> MissingInputs;

		Action.IterateInputs([&Inputs, &MissingInputs](FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
		{
			if (Inputs.IsNull() || Inputs.Get().FindInput(Key).IsNull())
			{
				MissingInputs.Emplace(Key);
			}
		});

		if (!MissingInputs.IsEmpty())
		{
			// Report missing inputs
			return OnComplete({Action.GetKey(), {}, MissingInputs, EStatus::Ok});
		}

		// This path will execute the build action synchronously in a scratch directory
		//
		// At this stage, all inputs are available in process
		//
		// Currently no cleanup whatsoever is performed, so inputs/outputs can be inspected
		// this could be problematic for large runs so we should probably add support for
		// configurable cleanup policies

		TStringBuilder<256> WorkerPath;
		if (!ManifestWorker(Worker, WorkerPath))
		{
			return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
		}
		FPathViews::Append(WorkerPath, Worker.GetPath());

		// Create a scratch directory for the build at {SandboxRoot}/Builds/######/####.
		TStringBuilder<256> BuildRoot;
		{
			static std::atomic<int32> SerialNo = 0;
			const int32 BuildSerialNo = ++SerialNo;
			TStringBuilder<8> OuterDir, InnerDir;
			OuterDir.Appendf(TEXT("%02d%04d"), GetMultiprocessId(), BuildSerialNo / 10000);
			InnerDir.Appendf(TEXT("%04d"), BuildSerialNo % 10000);
			FPathViews::Append(BuildRoot, SandboxRoot, TEXTVIEW("Builds"), OuterDir, InnerDir);
		}

		// Manifest inputs under the build root.
		if (!Inputs.IsNull())
		{
			std::atomic<bool> bOk = true;
			Inputs.Get().IterateInputs([&BuildRoot, &Action, &Worker, &bOk](FUtf8StringView Key, const FCompressedBuffer& Buffer)
			{
				TStringBuilder<256> Path;
				FPathViews::Append(Path, BuildRoot, TEXT("Inputs"), FIoHash(Buffer.GetRawHash()));
				if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileWriter(*Path, FILEWRITE_Silent)})
				{
					Buffer.Save(*Ar);
				}
				else
				{
					UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
						"Failed to open '{Path}' to write input for build worker {WorkerName} for '{Name}'.",
						Path, Worker.GetName(), Action.GetName());
					bOk = false;
				}
			});
			if (!bOk)
			{
				return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
			}
		}

		// Manifest the action under the build root.
		{
			TStringBuilder<256> Path;
			FPathViews::Append(Path, BuildRoot, TEXT("Build.action"));
			if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileWriter(*Path, FILEWRITE_Silent)})
			{
				FCbWriter Writer;
				Action.Save(Writer);
				Writer.Save(*Ar);
			}
			else
			{
				UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
					"Failed to open '{Path}' to write action for build worker {WorkerName} for '{Name}'.",
					Path, Worker.GetName(), Action.GetName());
				return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
			}
		}

		// Launch the worker process and wait for it to complete.

		// TODO: Handle Worker.IterateEnvironment(...)

		const bool bLaunchDetached = false;
		const bool bLaunchHidden = false;
		const bool bLaunchReallyHidden = false;
		uint32 ProcessID = 0;
		const int PriorityModifier = 0;
		const TCHAR* WorkingDirectory = *BuildRoot;
		FProcHandle ProcHandle = FPlatformProcess::CreateProc(
			*WorkerPath,
			TEXT("-Build=Build.action"),
			bLaunchDetached,
			bLaunchHidden,
			bLaunchReallyHidden,
			&ProcessID,
			PriorityModifier,
			WorkingDirectory,
			nullptr,
			nullptr);

		FPlatformProcess::WaitForProc(ProcHandle);

		int32 ExitCode = -1;
		if (!FPlatformProcess::GetProcReturnCode(ProcHandle, &ExitCode) || ExitCode != 0)
		{
			UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
				"Worker {WorkerName} exited with code {ExitCode} for '{Name}' at '{BuildRoot}' with worker '{WorkerPath}'.",
				Worker.GetName(), ExitCode, Action.GetName(), BuildRoot, WorkerPath);
			return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
		}

		// Load the output object from the build root.
		FOptionalBuildOutput RemoteBuildOutput;
		{
			TStringBuilder<256> Path;
			FPathViews::Append(Path, BuildRoot, TEXT("Build.output"));
			if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*Path, FILEREAD_Silent)})
			{
				FCbObject BuildOutput = LoadCompactBinary(*Ar).AsObject();
				if (Ar->IsError())
				{
					UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
						"Worker {WorkerName} build output '{Path}' is not valid compact binary for '{Name}' at '{BuildRoot}' with worker '{WorkerPath}'.",
						Worker.GetName(), Path, Action.GetName(), BuildRoot, WorkerPath);
					return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
				}
				RemoteBuildOutput = FBuildOutput::Load(Action.GetName(), Action.GetFunction(), BuildOutput);
			}
			if (RemoteBuildOutput.IsNull())
			{
				UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
					"Worker {WorkerName} is missing build output '{Path}' for '{Name}' at '{BuildRoot}' with worker '{WorkerPath}'.",
					Worker.GetName(), Path, Action.GetName(), BuildRoot, WorkerPath);
				return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
			}
		}

		// Load the output values from the build root.
		FBuildOutputBuilder OutputBuilder = BuildSystem.CreateOutput(Action.GetName(), Action.GetFunction());
		for (const FValueWithId& Value : RemoteBuildOutput.Get().GetValues())
		{
			if (EnumHasAnyFlags(Policy.GetValuePolicy(Value.GetId()), EBuildPolicy::SkipData))
			{
				OutputBuilder.AddValue(Value.GetId(), Value);
			}
			else
			{
				FCompressedBuffer OutputData;

				TStringBuilder<128> Path;
				FPathViews::Append(Path, BuildRoot, TEXT("Outputs"), FIoHash(Value.GetRawHash()));
				if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*Path, FILEREAD_Silent)})
				{
					OutputData = FCompressedBuffer::Load(*Ar);
				}

				if (OutputData.IsNull())
				{
					UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
						"Worker {WorkerName} is missing build output data '{Path}' for '{Name}' at '{BuildRoot}' with worker '{WorkerPath}'.",
						Worker.GetName(), Path, Action.GetName(), BuildRoot, WorkerPath);
					return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
				}

				if (OutputData.GetRawHash() != Value.GetRawHash() || OutputData.GetRawSize() != Value.GetRawSize())
				{
					UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Warning,
						"Worker {WorkerName} has mismatched build output data at '{Path}' for '{Name}' at '{BuildRoot}' with worker '{WorkerPath}'. "
						"Expected hash {ExpectedHash} and size {ExpectedSize}. "
						"Received hash {ReceivedHash} and size {ReceivedSize}.",
						Worker.GetName(), Path, Action.GetName(), BuildRoot, WorkerPath,
						Value.GetRawHash(), Value.GetRawSize(),
						OutputData.GetRawHash(), OutputData.GetRawSize());
					return OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
				}

				OutputBuilder.AddValue(Value.GetId(), FValue(MoveTemp(OutputData)));
			}
		}

		FBuildOutput BuildOutput = OutputBuilder.Build();
		return OnComplete({Action.GetKey(), BuildOutput, {}, EStatus::Ok});
	}

	[[nodiscard]] bool ManifestWorker(const FBuildWorker& Worker, FStringBuilderBase& OutPath)
	{
		OutPath.Reset();
		FPathViews::Append(OutPath, SandboxRoot, TEXTVIEW("Workers"), Worker.GetKey().Hash);

		// Return if a worker exists with this worker hash.
		IFileManager& FileManager = IFileManager::Get();
		if (FileManager.DirectoryExists(*OutPath))
		{
			return true;
		}

		// Allow the first thread in this process to manifest the worker and have the others wait.
		// Multiple processes may race to rename their temporary directory.
		bool bWaitForManifest = false;
		FManifestWorkerState* State;
		{
			TUniqueLock Lock(ManifestingWorkersMutex);
			TUniquePtr<FManifestWorkerState>& StatePtr = ManifestingWorkers.FindOrAdd(Worker.GetKey());
			if (StatePtr)
			{
				bWaitForManifest = true;
			}
			else
			{
				StatePtr = MakeUnique<FManifestWorkerState>();
			}
			State = StatePtr.Get();
			++State->RefCount;
		}
		ON_SCOPE_EXIT
		{
			TUniqueLock Lock(ManifestingWorkersMutex);
			if (--State->RefCount == 0)
			{
				ManifestingWorkers.Remove(Worker.GetKey());
			}
		};
		if (bWaitForManifest)
		{
			State->ReadyEvent.Wait();
			return State->bOk;
		}

		UE_LOGFMT(LogDerivedDataBuildLocalExecutor, Display,
			"Manifesting build worker {WorkerName} in '{WorkerPath}'...", Worker.GetName(), OutPath);
		State->bOk = Worker.Extract(OutPath);
		State->ReadyEvent.Notify();
		return State->bOk;
	}

	TConstArrayView<FUtf8StringView> GetHostPlatforms() const final
	{
		static constexpr FUtf8StringView HostPlatforms[]{UTF8TEXTVIEW("Win64")};
		return HostPlatforms;
	}

	void DumpStats()
	{
	}

private:
	struct FManifestWorkerState
	{
		int32 RefCount = 0;
		bool bOk = false;
		FManualResetEvent ReadyEvent;
	};

	FString SandboxRoot = FPaths::EngineSavedDir() / TEXT("LocalExec");
	TMap<FBuildWorkerKey, TUniquePtr<FManifestWorkerState>> ManifestingWorkers;
	FMutex ManifestingWorkersMutex;
};

} // UE::DerivedData

TOptional<UE::DerivedData::FLocalBuildWorkerExecutor> GLocalBuildWorkerExecutor;

void InitDerivedDataBuildLocalExecutor()
{
	if (!GLocalBuildWorkerExecutor.IsSet())
	{
		GLocalBuildWorkerExecutor.Emplace();
	}
}

void DumpDerivedDataBuildLocalExecutorStats()
{
	static bool bHasRun = false;
	if (GLocalBuildWorkerExecutor.IsSet() && !bHasRun)
	{
		bHasRun = true;
		GLocalBuildWorkerExecutor->DumpStats();
	}
}
