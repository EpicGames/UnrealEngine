// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataBuildRemoteExecutor.h"

#include "Algo/Find.h"
#include "Containers/Queue.h"
#include "Containers/SharedString.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildAction.h"
#include "DerivedDataBuildInputs.h"
#include "DerivedDataBuildOutput.h"
#include "DerivedDataBuildTypes.h"
#include "DerivedDataBuildWorker.h"
#include "DerivedDataRequest.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"
#include "Experimental/ZenServerInterface.h"
#include "Features/IModularFeatures.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"
#include "ZenServerHttp.h"

#include <atomic>

namespace UE::DerivedData
{

DEFINE_LOG_CATEGORY_STATIC(LogDerivedDataBuildRemoteExecutor, Log, All);

class FRemoteBuildWorkerExecutor;

/**
 * This implements a remote execution interface for Derived Data Build
 * using Zen Structured Compute interfaces to get work done. 
 
 * All build actions are typically submitted to the same Zen server which 
 * is responsible for build outputs (Project Store) and local DDC. This Zen 
 * server is responsible for dispatching the work elsewhere but can also 
 * execute it locally when appropriate.
 * 
 * The current implementation uses the "legacy" REST API for simplicity,
 * but should be upgraded to use the new queue-oriented API as soon
 * as possible since that is more efficient and more flexible.
 * 
 */

class FRemoteBuildExecutionRequest final : public FRequestBase
{
public:
	FRemoteBuildExecutionRequest(
		FRemoteBuildWorkerExecutor& InExecutor,
		const FBuildAction& Action,
		const FOptionalBuildInputs& Inputs,
		const FBuildPolicy& Policy,
		const FBuildWorker& Worker,
		IBuild& BuildSystem,
		IRequestOwner& Owner,
		FOnBuildWorkerActionComplete&& OnComplete);

	~FRemoteBuildExecutionRequest() final;

	// IRequest interface
	void SetPriority(EPriority Priority) final
	{
	}

	void Cancel() final
	{
		bCancelPending.store(true, std::memory_order_relaxed);
		Wait();
	}

	void Wait() final
	{
		CompletionEvent->Wait();
	}

private:

	struct FRemoteExecutionState
	{
		const FBuildAction& BuildAction;
		const FOptionalBuildInputs& BuildInputs;
		const FBuildWorker& BuildWorker;
		IBuild& BuildSystem;
		IRequestOwner& Owner;
		FBuildPolicy BuildPolicy;

		FIoHash WorkerDescriptionId;
		FIoHash ActionId;

		FCbObject WorkerDescriptor;
		FCbObject Action;

		// Step 1: Query if worker exists
		// Step 1a: Post Worker object
		// Step 1b: Post Worker package which contains missing blobs
		// Step 2: Post Action object
		// Step 2a: Post Action package which contains missing blobs
		// Step 3: Wait for results, return Package

		UE::Zen::FZenHttpRequest::Result Result;
		int ResponseCode;
		TSet<FIoHash> NeedHashes;
		FCbPackage ResultPackage;
	};

	FRemoteExecutionState State;
	FOnBuildWorkerActionComplete CompletionCallback;
	FRemoteBuildWorkerExecutor& Executor;
	FEventRef CompletionEvent{EEventMode::ManualReset};
	std::atomic<bool> bCancelPending;
	bool bHeuristicBuildStarted;

	// General utility methods
	static FCbObject BuildWorkerDescriptor(const FBuildWorker& Worker, const int TimeoutSeconds);
	void PrepareActionForPost();

	bool ProcessCancellation();
	bool IsResultOk(const UE::Zen::FZenHttpRequest::Result& Result, const TCHAR* OperationDesc);
	bool IsResponseOk(const int ResponseCode, const TCHAR* OperationDesc);

	// Async steps
	void DetermineIfWorkerExists_Async();
	void PostWorkerObject_Async();
	void PostWorkerPackage_Async();
	void PostActionObject_Async();
	void PostActionPackage_Async();
	void GetResultPackage_Async();

	void QueueGetResultPackage();

	// Post-step flow
	void OnWorkerExistsDetermined(const UE::Zen::FZenHttpRequest::Result& Result);
	void OnPostWorkerObjectComplete(const UE::Zen::FZenHttpRequest::Result& Result);
	void OnPostWorkerPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result);
	void OnPostActionObjectComplete(const UE::Zen::FZenHttpRequest::Result& Result);
	void OnPostActionPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result);
	void OnGetResultPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result);
};

class FRemoteBuildWorkerExecutor final: public IBuildWorkerExecutor, FRunnable
{
public:
	FRemoteBuildWorkerExecutor()
	: GlobalExecutionTimeoutSeconds(-1)
	, bEnabled(false)
	{
		check(IsInGameThread()); // initialization from the main thread is expected to allow config reading for the limiting heuristics
		check(GConfig && GConfig->IsReadyForUse());

		bool bConfigEnabled = false;
		GConfig->GetBool(TEXT("DerivedDataBuildRemoteExecutor"), TEXT("bEnabled"), bConfigEnabled, GEngineIni);
		GConfig->GetBool(TEXT("DerivedDataBuildRemoteExecutor"), TEXT("bContentNegotiationEnabled"), bContentNegotiationEnabled, GEngineIni);
		GConfig->GetInt(TEXT("DerivedDataBuildRemoteExecutor"), TEXT("GlobalExecutionTimeoutSeconds"), GlobalExecutionTimeoutSeconds, GEngineIni);

		if (bConfigEnabled || FParse::Param(FCommandLine::Get(), TEXT("DDC2RemoteExecution")))
		{
			ScopeZenService = MakeUnique<UE::Zen::FScopeZenService>();
			UE::Zen::FZenServiceInstance& ZenServiceInstance = ScopeZenService->GetInstance();
			if (ZenServiceInstance.IsServiceRunning())
			{
				
				ProcessingThreadEvent = FPlatformProcess::GetSynchEventFromPool(/* bIsManualReset = */ false);
				ProcessingThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FRemoteBuildWorkerExecutor"), 0, TPri_BelowNormal));
				bEnabled = ProcessingThread.IsValid();
			}
			
			if (bEnabled)
			{
				RequestPool = MakeUnique<UE::Zen::FZenHttpRequestPool>(ZenServiceInstance.GetEndpoint());
				IModularFeatures::Get().RegisterModularFeature(IBuildWorkerExecutor::FeatureName, this);
			}
			else
			{
				bProcessingThreadRunning = false;
				ProcessingThreadEvent->Trigger();
				ProcessingThread->WaitForCompletion();
				ScopeZenService.Reset();
				ProcessingThread.Reset();
				FPlatformProcess::ReturnSynchEventToPool(ProcessingThreadEvent);
			}
		}

	}

	virtual ~FRemoteBuildWorkerExecutor()
	{
		if (bEnabled)
		{
			IModularFeatures::Get().UnregisterModularFeature(IBuildWorkerExecutor::FeatureName, this);
			bProcessingThreadRunning = false;
			ProcessingThreadEvent->Trigger();
			ProcessingThread->WaitForCompletion();
			FPlatformProcess::ReturnSynchEventToPool(ProcessingThreadEvent);
		}
	}	

	void Build(
		const FBuildAction& Action,
		const FOptionalBuildInputs& Inputs,
		const FBuildPolicy& Policy,
		const FBuildWorker& Worker,
		IBuild& BuildSystem,
		IRequestOwner& Owner,
		FOnBuildWorkerActionComplete&& OnComplete) final
	{
		{
			TArray<FUtf8StringView> MissingInputs;
			uint64 TotalInputSize = 0;
			uint64 TotalMissingInputSize = 0;

			Action.IterateInputs([&MissingInputs, &Inputs, &TotalInputSize, &TotalMissingInputSize] (FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
				{
					if (Inputs.IsNull() || Inputs.Get().FindInput(Key).IsNull())
					{
						MissingInputs.Emplace(Key);
						TotalMissingInputSize += RawSize;
					}
					TotalInputSize += RawSize;
				});

			if (!LimitingHeuristics.PassesPreResolveRequirements(TotalInputSize, TotalMissingInputSize))
			{
				OnComplete({Action.GetKey(), {}, {}, EStatus::Error});
				return;
			}

			if (!MissingInputs.IsEmpty())
			{
				OnComplete({Action.GetKey(), {}, MissingInputs, EStatus::Ok});
				return;
			}
		}

		new FRemoteBuildExecutionRequest(*this, Action, Inputs, Policy, Worker, BuildSystem, Owner, MoveTemp(OnComplete));
	}

	TConstArrayView<FUtf8StringView> GetHostPlatforms() const final
	{
		static constexpr FUtf8StringView HostPlatforms[]{UTF8TEXTVIEW("Win64"), UTF8TEXTVIEW("Linux"), UTF8TEXTVIEW("Mac")};
		return HostPlatforms;
	}

	void DumpStats()
	{
		if (Stats.TotalRemoteBuilds.load() == 0)
		{
			return;
		}

		Stats.Dump();
	}

	uint32 Run() final
	{
		bProcessingThreadRunning = true;
		while (bProcessingThreadRunning)
		{
			ProcessingThreadEvent->Wait();
			TUniqueFunction<void(bool)> Function;
			while (PendingRequests.Dequeue(Function))
			{
				Function(true);
			}
			if (bProcessingThreadRunning)
			{
				FPlatformProcess::Sleep(1.0f);
			}
		}
		return 0;
	}

	void Stop() final
	{
		bProcessingThreadRunning = false;
	}
	
	void Exit() final
	{
		TUniqueFunction<void(bool)> Function;
		while (PendingRequests.Dequeue(Function))
		{
			Function(false);
		}
	}

	void AddResultWaitRequest(TUniqueFunction<void(bool)>&& Function)
	{
		PendingRequests.Enqueue(MoveTemp(Function));
		ProcessingThreadEvent->Trigger();
	}

	bool IsContentNegotiationEnabled() const
	{
		return bContentNegotiationEnabled;
	}

private:
	struct FStats
	{
		std::atomic<uint64> TotalRemoteBuilds{0};
		std::atomic<uint32> InFlightRemoteBuilds{0};

		std::atomic<uint64> TotalSuccessfulRemoteBuilds{0};
		std::atomic<uint64> TotalTimedOutRemoteBuilds{0};

		struct FBlobStat
		{
			std::atomic<uint64> Quantity{0};
			std::atomic<uint64> Bytes{0};

			void AddBlob(uint64 InBytes)
			{
				Quantity.fetch_add(1, std::memory_order_relaxed);
				Bytes.fetch_add(InBytes, std::memory_order_relaxed);
			}
		};
		FBlobStat TotalWorkerObjectsUploaded;
		FBlobStat TotalWorkerPackagesUploaded;
		FBlobStat TotalActionObjectsUploaded;
		FBlobStat TotalActionPackagesUploaded;

		FBlobStat TotalObjectsDownloaded;
		FBlobStat TotalPackagesDownloaded;

		void Dump()
		{
			UE_LOGF(LogDerivedDataBuildRemoteExecutor, Display, "");
			UE_LOGF(LogDerivedDataBuildRemoteExecutor, Display, "DDC Remote Execution Stats");
			UE_LOGF(LogDerivedDataBuildRemoteExecutor, Display, "==========================");
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Total remote builds"), TotalRemoteBuilds.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Successful remote builds"), TotalSuccessfulRemoteBuilds.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Timed out remote builds"), TotalTimedOutRemoteBuilds.load());

			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded worker objects (quantity)"), TotalWorkerObjectsUploaded.Quantity.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded worker objects (KB)"), TotalWorkerObjectsUploaded.Bytes.load()/1024);
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded worker packages (quantity)"), TotalWorkerPackagesUploaded.Quantity.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded worker packages (KB)"), TotalWorkerPackagesUploaded.Bytes.load() / 1024);

			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded action objects (quantity)"), TotalActionObjectsUploaded.Quantity.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded action objects (KB)"), TotalActionObjectsUploaded.Bytes.load() / 1024);
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded action packages (quantity)"), TotalActionPackagesUploaded.Quantity.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Uploaded action packages (KB)"), TotalActionPackagesUploaded.Bytes.load() / 1024);

			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Downloaded packages (quantity)"), TotalPackagesDownloaded.Quantity.load());
			UE_LOG(LogDerivedDataBuildRemoteExecutor, Display, TEXT("%-36s=%10") UINT64_FMT, TEXT("Downloaded packages (KB)"), TotalPackagesDownloaded.Bytes.load() / 1024);
		}
	};

	// Temporary heuristics until a scheduler makes higher level decisions about how to limit remote execution of builds
	class FLimitingHeuristics
	{
	public:
		FLimitingHeuristics()
		{
			check(IsInGameThread()); // initialization from the main thread is expected to allow config reading for the limiting heuristics
			check(GConfig && GConfig->IsReadyForUse());
			const TCHAR* Section = TEXT("DerivedDataBuildRemoteExecutor.LimitingHeuristics");
			GConfig->GetBool(Section, TEXT("bEnableLimits"), bEnableLimits, GEngineIni);

			int32 SignedMaxTotalRemoteBuilds{MAX_int32};
			GConfig->GetInt(Section, TEXT("MaxTotalRemoteBuilds"), SignedMaxTotalRemoteBuilds, GEngineIni);
			if ((SignedMaxTotalRemoteBuilds >= 0) && (SignedMaxTotalRemoteBuilds < MAX_int32))
			{
				MaxTotalRemoteBuilds = (uint64)SignedMaxTotalRemoteBuilds;
			}

			int32 SignedMaxInFlightRemoteBuilds{MAX_int32};
			GConfig->GetInt(Section, TEXT("MaxInFlightRemoteBuilds"), SignedMaxInFlightRemoteBuilds, GEngineIni);
			if ((SignedMaxInFlightRemoteBuilds >= 0) && (SignedMaxInFlightRemoteBuilds < MAX_int32))
			{
				MaxInFlightRemoteBuilds = (uint32)SignedMaxInFlightRemoteBuilds;
			}

			int32 SignedMinInputSizeForRemoteBuilds{0};
			GConfig->GetInt(Section, TEXT("MinInputSizeForRemoteBuilds"), SignedMinInputSizeForRemoteBuilds, GEngineIni);
			if ((SignedMinInputSizeForRemoteBuilds >= 0) && (SignedMinInputSizeForRemoteBuilds < MAX_int32))
			{
				MinInputSizeForRemoteBuilds = (uint64)SignedMinInputSizeForRemoteBuilds;
			}

			int32 SignedMaxInputSizeForRemoteBuilds{MAX_int32};
			GConfig->GetInt(Section, TEXT("MaxInputSizeForRemoteBuilds"), SignedMaxInputSizeForRemoteBuilds, GEngineIni);
			if ((SignedMaxInputSizeForRemoteBuilds >= 0) && (SignedMaxInputSizeForRemoteBuilds < MAX_int32))
			{
				MaxInputSizeForRemoteBuilds = (uint64)SignedMaxInputSizeForRemoteBuilds;
			}

			int32 SignedMaxMissingInputSizeForRemoteBuilds{MAX_int32};
			GConfig->GetInt(Section, TEXT("MaxMissingInputSizeForRemoteBuilds"), SignedMaxMissingInputSizeForRemoteBuilds, GEngineIni);
			if ((SignedMaxMissingInputSizeForRemoteBuilds >= 0) && (SignedMaxMissingInputSizeForRemoteBuilds < MAX_int32))
			{
				MaxMissingInputSizeForRemoteBuilds = (uint64)SignedMaxMissingInputSizeForRemoteBuilds;
			}
		}

		bool PassesPreResolveRequirements(uint64 InputSize, uint64 MissingInputSize)
		{
			if (!bEnableLimits)
			{
				return true;
			}

			if (InputSize < MinInputSizeForRemoteBuilds)
			{
				return false;
			}

			if (InputSize > MaxInputSizeForRemoteBuilds)
			{
				return false;
			}

			if (MissingInputSize > MaxMissingInputSizeForRemoteBuilds)
			{
				return false;
			}

			return true;
		}

		bool TryStartNewBuild(FStats& InStats)
		{
			if (!bEnableLimits)
			{
				return true;
			}

			if ((InStats.TotalRemoteBuilds.fetch_add(1, std::memory_order_relaxed) >= MaxTotalRemoteBuilds))
			{
				InStats.TotalRemoteBuilds.fetch_sub(1, std::memory_order_relaxed);
				return false;
			}

			if ((InStats.InFlightRemoteBuilds.fetch_add(1, std::memory_order_relaxed) >= MaxInFlightRemoteBuilds))
			{
				InStats.TotalRemoteBuilds.fetch_sub(1, std::memory_order_relaxed);
				InStats.InFlightRemoteBuilds.fetch_sub(1, std::memory_order_relaxed);
				return false;
			}

			return true;
		}

		void FinishBuild(FStats& InStats)
		{
			InStats.InFlightRemoteBuilds.fetch_sub(1, std::memory_order_relaxed);
		}

	private:
		uint64 MaxTotalRemoteBuilds{MAX_uint64};
		uint32 MaxInFlightRemoteBuilds{MAX_uint32};
		uint64 MinInputSizeForRemoteBuilds{0};
		uint64 MaxInputSizeForRemoteBuilds{MAX_uint64};
		uint64 MaxMissingInputSizeForRemoteBuilds{MAX_uint64};
		bool bEnableLimits{false};
	};

	friend class FRemoteBuildExecutionRequest;

	FStats Stats;
	FLimitingHeuristics LimitingHeuristics;
	TUniquePtr<FRunnableThread> ProcessingThread;
	FEvent* ProcessingThreadEvent;
	TQueue<TUniqueFunction<void(bool)>, EQueueMode::Mpsc> PendingRequests;
	int GlobalExecutionTimeoutSeconds;
	TUniquePtr<UE::Zen::FScopeZenService> ScopeZenService;
	TUniquePtr<UE::Zen::FZenHttpRequestPool> RequestPool;
	bool bEnabled;
	bool bContentNegotiationEnabled = false;
	std::atomic<bool> bProcessingThreadRunning;
};

FRemoteBuildExecutionRequest::FRemoteBuildExecutionRequest(
	FRemoteBuildWorkerExecutor& InExecutor,
	const FBuildAction& Action,
	const FOptionalBuildInputs& Inputs,
	const FBuildPolicy& Policy,
	const FBuildWorker& Worker,
	IBuild& BuildSystem,
	IRequestOwner& Owner,
	FOnBuildWorkerActionComplete&& OnComplete)
: State{Action, Inputs, Worker, BuildSystem, Owner, Policy}
, CompletionCallback(MoveTemp(OnComplete))
, Executor(InExecutor)
, bCancelPending(false)
, bHeuristicBuildStarted(false)
{
	Owner.Begin(this);

	if (!Executor.LimitingHeuristics.TryStartNewBuild(Executor.Stats))
	{
		State.Owner.End(this, [this]
			{
				CompletionCallback({ State.BuildAction.GetKey(), {}, {}, EStatus::Error });
				CompletionEvent->Trigger();
			});
		return;
	}

	bHeuristicBuildStarted = true;

	DetermineIfWorkerExists_Async();
}

FRemoteBuildExecutionRequest::~FRemoteBuildExecutionRequest()
{
	if (bHeuristicBuildStarted)
	{
		Executor.LimitingHeuristics.FinishBuild(Executor.Stats);
	}
}

FCbObject FRemoteBuildExecutionRequest::BuildWorkerDescriptor(const FBuildWorker& Worker, const int TimeoutSeconds)
{
	FCbWriter WorkerDescriptor;
	WorkerDescriptor.BeginObject();

	WorkerDescriptor.AddString(ANSITEXTVIEW("name"), Worker.GetName());
	WorkerDescriptor.AddString(ANSITEXTVIEW("path"), Worker.GetPath());
	WorkerDescriptor.AddString(ANSITEXTVIEW("host"), Worker.GetHostPlatform());
	WorkerDescriptor.AddUuid(ANSITEXTVIEW("buildsystem_version"), Worker.GetBuildSystemVersion());
	WorkerDescriptor.AddInteger(ANSITEXTVIEW("timeout"), TimeoutSeconds);
	WorkerDescriptor.AddInteger(ANSITEXTVIEW("cores"), 1);
	//WorkerDescriptor.AddInteger(ANSITEXTVIEW("memory"), 1 * 1024 * 1024 * 1024);

	WorkerDescriptor.BeginArray(ANSITEXTVIEW("environment"));
	Worker.IterateEnvironment([&WorkerDescriptor](FUtf8StringView Name, FUtf8StringView Value)
		{
			WorkerDescriptor.AddString(WriteToUtf8String<256>(Name, "=", Value));
		});
	WorkerDescriptor.EndArray();

	WorkerDescriptor.BeginArray(ANSITEXTVIEW("executables"));
	Worker.IterateExecutables([&WorkerDescriptor](FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
		{
			WorkerDescriptor.BeginObject();
			WorkerDescriptor.AddString(ANSITEXTVIEW("name"), Key);
			WorkerDescriptor.AddBinaryAttachment(ANSITEXTVIEW("hash"), RawHash);
			WorkerDescriptor.AddInteger(ANSITEXTVIEW("size"), RawSize);
			WorkerDescriptor.EndObject();
		});
	WorkerDescriptor.EndArray();

	WorkerDescriptor.BeginArray("files");
	Worker.IterateFiles([&WorkerDescriptor](FUtf8StringView Key, const FIoHash& RawHash, uint64 RawSize)
		{
			WorkerDescriptor.BeginObject();
			WorkerDescriptor.AddString(ANSITEXTVIEW("name"), Key);
			WorkerDescriptor.AddBinaryAttachment(ANSITEXTVIEW("hash"), RawHash);
			WorkerDescriptor.AddInteger(ANSITEXTVIEW("size"), RawSize);
			WorkerDescriptor.EndObject();
		});
	WorkerDescriptor.EndArray();

	WorkerDescriptor.BeginArray(ANSITEXTVIEW("dirs"));
	WorkerDescriptor.AddString(WriteToString<256>("Engine/Binaries/", Worker.GetHostPlatform()));
	WorkerDescriptor.EndArray();

	WorkerDescriptor.BeginArray(ANSITEXTVIEW("functions"));
	Worker.IterateFunctions([&WorkerDescriptor](FUtf8StringView Name, const FGuid& Version)
		{
			WorkerDescriptor.BeginObject();
			WorkerDescriptor.AddString(ANSITEXTVIEW("name"), Name);
			WorkerDescriptor.AddUuid(ANSITEXTVIEW("version"), Version);
			WorkerDescriptor.EndObject();
		});
	WorkerDescriptor.EndArray();

	WorkerDescriptor.EndObject();

	return WorkerDescriptor.Save().AsObject();
}

bool FRemoteBuildExecutionRequest::ProcessCancellation()
{
	if (bCancelPending.load(std::memory_order_relaxed))
	{
		State.Owner.End(this, [this]
		{
			CompletionCallback({State.BuildAction.GetKey(), {}, {}, EStatus::Canceled});
			CompletionEvent->Trigger();
		});
		return true;
	}
	return false;
}

bool FRemoteBuildExecutionRequest::IsResultOk(const UE::Zen::FZenHttpRequest::Result& Result, const TCHAR* OperationDesc)
{
	if (Result == Zen::FZenHttpRequest::Result::Failed)
	{
		UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: operation '%ls' produced a failed result!", OperationDesc);

		State.Owner.End(this, [this]
		{
			CompletionCallback({State.BuildAction.GetKey(), {}, {}, EStatus::Error});
			CompletionEvent->Trigger();
		});
		return false;
	}
	return true;
}

bool FRemoteBuildExecutionRequest::IsResponseOk(const int ResponseCode, const TCHAR* OperationDesc)
{
	if (!UE::Zen::IsSuccessCode(ResponseCode))
	{
		UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: operation '%ls' produced an error result (%d)!", OperationDesc, ResponseCode);

		State.Owner.End(this, [this]
			{
				CompletionCallback({ State.BuildAction.GetKey(), {}, {}, EStatus::Error });
				CompletionEvent->Trigger();
			});
		return false;
	}
	return true;
}

void FRemoteBuildExecutionRequest::DetermineIfWorkerExists_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::GetWorkerExists"), [this]
		{
			State.WorkerDescriptor = BuildWorkerDescriptor(State.BuildWorker, Executor.GlobalExecutionTimeoutSeconds);
			State.WorkerDescriptionId = State.WorkerDescriptor.GetHash();

			TStringBuilder<128> WorkerUri;
			WorkerUri.Append("/compute/workers/");
			WorkerUri << State.WorkerDescriptionId;

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingDownload(WorkerUri, nullptr, Zen::EContentType::CbObject);
			State.ResponseCode = Request->GetResponseCode();
			
			OnWorkerExistsDetermined(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::PostWorkerObject_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::PostWorkerObject"), [this]
		{
			TStringBuilder<128> WorkerUri;
			WorkerUri.Append("/compute/workers/");
			WorkerUri << State.WorkerDescriptionId;

			Executor.Stats.TotalWorkerObjectsUploaded.AddBlob(State.WorkerDescriptor.GetSize());

			State.NeedHashes.Empty();

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingPost(WorkerUri, State.WorkerDescriptor);
			State.ResponseCode = Request->GetResponseCode();

			if (Request->GetResponseCode() == 404)
			{
				FCbObjectView Response = Request->GetResponseAsObject();
				FCbArrayView NeedArray = Response["need"].AsArrayView();

				for (auto& It : NeedArray)
				{
					State.NeedHashes.Add(It.AsHash());
				}
			}

			OnPostWorkerObjectComplete(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::PostWorkerPackage_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::PostWorkerPackage"), [this]
		{
			TStringBuilder<128> WorkerUri;
			WorkerUri.Append("/compute/workers/");
			WorkerUri << State.WorkerDescriptionId;

			uint64_t AttachmentBytes{};
			FCbPackage Package;

			{
				TSet<FIoHash> WorkerFileHashes;
				State.BuildWorker.IterateExecutables([NeedHashes = State.NeedHashes, &WorkerFileHashes](FUtf8StringView Path, const FIoHash& RawHash, uint64 RawSize)
					{
						if (NeedHashes.Contains(RawHash))
						{
							WorkerFileHashes.Emplace(RawHash);
						}
					});

				State.BuildWorker.IterateFiles([NeedHashes = State.NeedHashes, &WorkerFileHashes](FUtf8StringView Path, const FIoHash& RawHash, uint64 RawSize)
					{
						if (NeedHashes.Contains(RawHash))
						{
							WorkerFileHashes.Emplace(RawHash);
						}
					});

				if (State.NeedHashes.Num() != WorkerFileHashes.Num())
				{
					UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: build input file missing!");
					State.Result = UE::Zen::FZenHttpRequest::Result::Failed;
					State.ResponseCode = 0;
					OnPostWorkerPackageComplete(State.Result);
					return;
				}

				FRequestOwner BlockingOwner(EPriority::Blocking);
				State.BuildWorker.Resolve(WorkerFileHashes.Array(), BlockingOwner, [&Package, &AttachmentBytes](FBuildWorkerResolvedParams&& Params)
					{
						for (const FCompressedBuffer& Buffer : Params.Files)
						{
							Package.AddAttachment(FCbAttachment{ Buffer });
							AttachmentBytes += Buffer.GetCompressedSize();
						}
					});
				BlockingOwner.Wait();
			}

			Package.SetObject(State.WorkerDescriptor);

			Executor.Stats.TotalWorkerPackagesUploaded.AddBlob(AttachmentBytes + State.WorkerDescriptor.GetSize());

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingPostPackage(WorkerUri, Package);
			State.ResponseCode = Request->GetResponseCode();
			
			OnPostWorkerPackageComplete(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::PrepareActionForPost()
{
	if (!State.Action) 
	{
		FCbWriter BuildActionWriter;
		State.BuildAction.Save(BuildActionWriter);
		State.Action = BuildActionWriter.Save().AsObject();
		State.ActionId = State.Action.GetHash();
	}
}

void FRemoteBuildExecutionRequest::PostActionObject_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::PostActionObject"), [this]
		{
			PrepareActionForPost();

			TStringBuilder<128> ActionUri;
			ActionUri.Append("/compute/jobs/");
			ActionUri << State.WorkerDescriptionId;
			ActionUri << "?priority=" << (uint8)State.Owner.GetPriority();

			State.NeedHashes.Empty();

			Executor.Stats.TotalActionObjectsUploaded.AddBlob(State.Action.GetSize());

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingPost(ActionUri, State.Action);
			State.ResponseCode = Request->GetResponseCode();

			if (Request->GetResponseCode() == 404)
			{
				FCbObjectView Response = Request->GetResponseAsObject();
				FCbArrayView NeedArray = Response["need"].AsArrayView();

				for (auto& It : NeedArray)
				{
					State.NeedHashes.Add(It.AsHash());
				}
			}

			OnPostActionObjectComplete(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::PostActionPackage_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::PostActionPackage"), [this]
		{
			PrepareActionForPost();

			uint64_t AttachmentBytes{};
			FCbPackage ActionPackage;

			// It's valid for NeedHashes to be empty, in the case we go ahead and skip
			// the initial "needs" probe because the inputs are very small so we may
			// as well go ahead and send all of them at once without prior negotiation.

			State.BuildInputs.Get().IterateInputs([NeedHashes = State.NeedHashes, &ActionPackage, &AttachmentBytes](FUtf8StringView Key, const FCompressedBuffer& Buffer)
				{
					if (NeedHashes.IsEmpty() || NeedHashes.Contains(Buffer.GetRawHash()))
					{
						ActionPackage.AddAttachment(FCbAttachment{ Buffer });
						AttachmentBytes += Buffer.GetCompressedSize();
					}
				});

			if (!State.NeedHashes.IsEmpty() && (State.NeedHashes.Num() != ActionPackage.GetAttachments().Num()))
			{
				UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: build input attachment missing!");
				State.Result = UE::Zen::FZenHttpRequest::Result::Failed;
				State.ResponseCode = 0;
				OnPostActionPackageComplete(State.Result);
				return;
			}

			ActionPackage.SetObject(State.Action);

			TStringBuilder<128> ActionUri;
			ActionUri.Append("/compute/jobs/");
			ActionUri << State.WorkerDescriptionId;
			ActionUri << "?priority=" << (uint8)State.Owner.GetPriority();

			Executor.Stats.TotalActionPackagesUploaded.AddBlob(AttachmentBytes + State.Action.GetSize());

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingPostPackage(ActionUri, ActionPackage);
			State.ResponseCode = Request->GetResponseCode();
			
			OnPostActionPackageComplete(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::GetResultPackage_Async()
{
	UE::Tasks::Launch(TEXT("FRemoteBuildExecutionRequest::GetResultPackage"), [this]
		{
			TStringBuilder<128> JobGetUri;
			JobGetUri.Append("/compute/jobs/");
			JobGetUri << State.WorkerDescriptionId;
			JobGetUri << "/";
			JobGetUri << State.ActionId;

			UE::Zen::FZenScopedRequestPtr Request(Executor.RequestPool.Get(), false);
			State.Result = Request->PerformBlockingDownload(JobGetUri.ToString(), nullptr, UE::Zen::EContentType::CbPackage);
			State.ResponseCode = Request->GetResponseCode();
			if (State.ResponseCode == 200)
			{
				Executor.Stats.TotalPackagesDownloaded.AddBlob(Request->GetResponseBuffer().Num());
				State.ResultPackage = Request->GetResponseAsPackage();
			}
			else if (State.ResponseCode == 404)
			{
				Executor.Stats.TotalTimedOutRemoteBuilds.fetch_add(1, std::memory_order_relaxed);

				UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "result package missing (uri: %ls)", *JobGetUri);
			}
			
			OnGetResultPackageComplete(State.Result);
		}
	);
}

void FRemoteBuildExecutionRequest::QueueGetResultPackage()
{
	auto Callback = [this](const bool Continue) mutable
	{
		if (!Continue)
		{
			State.Owner.End(this, [this]
				{
					CompletionCallback({ State.BuildAction.GetKey(), {}, {}, EStatus::Canceled });
					CompletionEvent->Trigger();
				});
			return;
		}
		GetResultPackage_Async();
	};

	Executor.AddResultWaitRequest(MoveTemp(Callback));
}

void FRemoteBuildExecutionRequest::OnWorkerExistsDetermined(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("GetWorkerExists")))
	{
		return;
	}

	if (State.ResponseCode == 404)
	{
		// Worker missing, proceed to posting Object
		PostWorkerObject_Async();
		return;
	}

	if (!IsResponseOk(State.ResponseCode, TEXT("GetWorkerExists")))
	{
		return;
	}

	// Worker exists, proceed to action
	if (Executor.IsContentNegotiationEnabled())
	{
		PostActionObject_Async();
	}
	else
	{
		// Skip the content negotiation and just post the action if
		// the inputs are small enough or if we just know that it's
		// very unlikely that the remote end has the content.
		PostActionPackage_Async();
	}
}

void FRemoteBuildExecutionRequest::OnPostWorkerObjectComplete(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("PostWorkerObject")))
	{
		return;
	}

	if (State.ResponseCode == 404)
	{
		// Worker content missing, proceed to posting Package
		PostWorkerPackage_Async();
		return;
	}

	if (!IsResponseOk(State.ResponseCode, TEXT("PostWorkerObject")))
	{
		return;
	}

	// Worker exists and is not missing anything, proceed to action
	PostActionObject_Async();
}
void FRemoteBuildExecutionRequest::OnPostWorkerPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("PostWorkerPackage")) || !IsResponseOk(State.ResponseCode, TEXT("PostWorkerPackage")))
	{
		return;
	}

	// Worker exists and is not missing anything, proceed to action
	PostActionObject_Async();
}

void FRemoteBuildExecutionRequest::OnPostActionObjectComplete(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("PostActionObject")))
	{
		return;
	}

	if (State.ResponseCode == 404)
	{
		// Action content missing, proceed to posting Package
		PostActionPackage_Async();
		return;
	}

	if (!IsResponseOk(State.ResponseCode, TEXT("PostActionObject")))
	{
		return;
	}

	// Action posted, proceed to waiting
	QueueGetResultPackage();
}

void FRemoteBuildExecutionRequest::OnPostActionPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("PostActionPackage")) || !IsResponseOk(State.ResponseCode, TEXT("PostActionPackage")))
	{
		return;
	}

	// Action posted, proceed to waiting
	QueueGetResultPackage();
}

void FRemoteBuildExecutionRequest::OnGetResultPackageComplete(const UE::Zen::FZenHttpRequest::Result& Result)
{
	if (ProcessCancellation())
	{
		return;
	}

	if (!IsResultOk(Result, TEXT("GetResultPackage")))
	{
		return;
	}

	if (State.ResponseCode == 202 /* Accepted */)
	{
		// Result is not yet available but the job is pending/running
		QueueGetResultPackage();
		return;
	}

	if (!IsResponseOk(State.ResponseCode, TEXT("GetResultPackage")))
	{
		return;
	}

	// We're done!

	FOptionalBuildOutput RemoteBuildOutput = FBuildOutput::Load(State.BuildAction.GetName(), State.BuildAction.GetFunction(), State.ResultPackage.GetObject());

	if (RemoteBuildOutput.IsNull())
	{
		UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: build output blob missing!");
		State.Owner.End(this, [this]
			{
				CompletionCallback({ State.BuildAction.GetKey(), {}, {}, EStatus::Error });
				CompletionEvent->Trigger();
			});
		return;
	}

	FBuildOutputBuilder OutputBuilder = State.BuildSystem.CreateOutput(State.BuildAction.GetName(), State.BuildAction.GetFunction());

	for (const FBuildOutputMessage& Message : RemoteBuildOutput.Get().GetMessages())
	{
		OutputBuilder.AddMessage(Message);
	}

	for (const FBuildOutputLog& Log : RemoteBuildOutput.Get().GetLogs())
	{
		OutputBuilder.AddLog(Log);
	}

	for (const FValueWithId& Value : RemoteBuildOutput.Get().GetValues())
	{
		FCompressedBuffer BufferForValue;

		if (const FCbAttachment* Attachment = State.ResultPackage.FindAttachment(Value.GetRawHash()))
		{
			BufferForValue = Attachment->AsCompressedBinary();
		}

		if (BufferForValue.IsNull())
		{
			UE_LOGF(LogDerivedDataBuildRemoteExecutor, Warning, "Remote execution system error: payload blob missing!");
			State.Owner.End(this, [this]
				{
					CompletionCallback({ State.BuildAction.GetKey(), {}, {}, EStatus::Error });
					CompletionEvent->Trigger();
				});
			return;
		}

		OutputBuilder.AddValue(Value.GetId(), FValue(MoveTemp(BufferForValue)));
	}

	FBuildOutput BuildOutput = OutputBuilder.Build();

	Executor.Stats.TotalSuccessfulRemoteBuilds.fetch_add(1, std::memory_order_relaxed);

	State.Owner.End(this, [this, &BuildOutput]() mutable
		{
			CompletionCallback({ State.BuildAction.GetKey(), MoveTemp(BuildOutput), {}, EStatus::Ok });
			CompletionEvent->Trigger();
		});
}

} // namespace UE::DerivedData

TOptional<UE::DerivedData::FRemoteBuildWorkerExecutor> GRemoteBuildWorkerExecutor;

void InitDerivedDataBuildRemoteExecutor()
{
	if (!GRemoteBuildWorkerExecutor.IsSet())
	{
		GRemoteBuildWorkerExecutor.Emplace();
	}
}

void DumpDerivedDataBuildRemoteExecutorStats()
{
	static bool bHasRun = false;
	if (GRemoteBuildWorkerExecutor.IsSet() && !bHasRun)
	{
		bHasRun = true;
		GRemoteBuildWorkerExecutor->DumpStats();
	}
}
