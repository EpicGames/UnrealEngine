// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataBuildWorkers.h"

#include "Containers/StringView.h"
#include "Misc/Optional.h"
#include "MultiPlatformTargetReceiptBuildWorkers.h"

struct FDerivedDataBuildWorkers
{
	FMultiPlatformTargetReceiptBuildWorkers BaseTextureBuildWorkerFactory{TEXTVIEW("$(EngineDir)/Binaries/$(Platform)/BaseTextureBuildWorker.target")};
	FMultiPlatformTargetReceiptBuildWorkers ShaderBuildWorkerFactory{TEXTVIEW("$(EngineDir)/Binaries/$(Platform)/ShaderBuildWorker.target")};
};

TOptional<FDerivedDataBuildWorkers> GBuildWorkers;

void InitDerivedDataBuildWorkers()
{
	if (!GBuildWorkers.IsSet())
	{
		GBuildWorkers.Emplace();
	}
}

void ShutdownDerivedDataBuildWorkers()
{
	GBuildWorkers.Reset();
}
