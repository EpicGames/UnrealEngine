// Copyright Epic Games, Inc. All Rights Reserved.


#include "Dialogs/Dialogs.h"

#include "Dialog/DialogUtils.h"
#include "Dialogs/DialogsPrivate.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MessageDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "ObjectTools.h"
#include "DesktopPlatformModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformMisc.h"
#include "SPrimaryButton.h"
#include "Dialog/SMessageDialog.h"

DEFINE_LOG_CATEGORY_STATIC(LogDialogs, Log, All);

#define LOCTEXT_NAMESPACE "Dialogs"

///////////////////////////////////////////////////////////////////////////////
//
// Local classes.
//
///////////////////////////////////////////////////////////////////////////////

class SChoiceDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SChoiceDialog )	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Message)	
		SLATE_ATTRIBUTE(float, WrapMessageAt)
		SLATE_ARGUMENT(EAppMsgCategory, MessageCategory)
		SLATE_ARGUMENT(EAppMsgType::Type, MessageType)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{
		ParentWindow = InArgs._ParentWindow;
		Response = EAppReturnType::Cancel;
		MessageType = InArgs._MessageType;

		FSlateFontInfo MessageFont( FAppStyle::GetFontStyle("StandardDialog.LargeFont"));
		MyMessage = InArgs._Message;

		TSharedPtr<SUniformGridPanel> ButtonBox;

		const FSlateBrush* IconBrush = FDialogUtils::GetMessageCategoryIcon(InArgs._MessageCategory);

		this->ChildSlot
		[	
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(16.f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(1.0f)
				.MaxHeight(550)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(24.f, 24.f))
						.Image(IconBrush)
					]

					+SHorizontalBox::Slot()
					.Padding(16.f, 0.f, 0.f, 0.f)
					[
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(MyMessage)
							.Font(MessageFont)
							.WrapTextAt(InArgs._WrapMessageAt)
						]
					]
				]

				+SVerticalBox::Slot()
				.Padding(0.f, 32.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked(this, &SChoiceDialog::HandleCopyMessageButtonClicked)
						.ToolTipText(NSLOCTEXT("SChoiceDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
						.ContentPadding(2.f)
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(FMargin(16.f, 0.f, 0.f, 0.f))
						[
							SNew(SCheckBox)
							.IsChecked(ECheckBoxState::Unchecked)
							.OnCheckStateChanged(this, &SChoiceDialog::OnCheckboxClicked)
							.Visibility(this, &SChoiceDialog::GetCheckboxVisibility)
							.ToolTipText(NSLOCTEXT("SChoiceDialog", "ApplyToAllTooltip", "Make your choice of Yes or No apply to all remaining items in the current operation"))
							[
								SNew(STextBlock)
								.WrapTextAt(615.0f)
								.Text(NSLOCTEXT("SChoiceDialog", "ApplyToAllLabel", "Apply to All"))
							]
						]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SAssignNew( ButtonBox, SUniformGridPanel )
						.SlotPadding(FMargin(8.f, 0.f, 0.f, 0.f))
						.MinDesiredSlotWidth(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
						.MinDesiredSlotHeight(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
					]
				]
			]
		];

		TSharedPtr<SButton> PrimaryButton;
		int32 SlotIndex = 0;

#define ADD_SLOT(Button)\
		ButtonBox->AddSlot(SlotIndex++,0)\
		[\
			SNew( SButton )\
			.VAlign(VAlign_Center)\
			.HAlign(HAlign_Center)\
			.Text( EAppReturnTypeToText(EAppReturnType::Button) )\
			.OnClicked( this, &SChoiceDialog::HandleButtonClicked, EAppReturnType::Button )\
		];

#define ADD_SLOT_PRIMARY(Button)\
		ButtonBox->AddSlot(SlotIndex++,0)\
		[\
			SAssignNew( PrimaryButton, SButton )\
			.VAlign(VAlign_Center)\
			.HAlign(HAlign_Center)\
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))\
			.Text( EAppReturnTypeToText(EAppReturnType::Button) )\
			.OnClicked( this, &SChoiceDialog::HandleButtonClicked, EAppReturnType::Button )\
		];


		switch ( MessageType )
		{	
		case EAppMsgType::Ok:
			ADD_SLOT_PRIMARY(Ok)
			break;
		case EAppMsgType::YesNo:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			break;
		case EAppMsgType::OkCancel:
			ADD_SLOT_PRIMARY(Ok)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoCancel:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::CancelRetryContinue:
			ADD_SLOT_PRIMARY(Continue)
			ADD_SLOT(Retry)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoYesAllNoAll:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			break;
		case EAppMsgType::YesNoYesAllNoAllCancel:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoYesAll:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(YesAll)
			break;
		default:
			UE_LOGF(LogDialogs, Fatal, "Invalid Message Type");
		}

		// Route user focus to the default dialog button. 
		if (ensure(PrimaryButton.IsValid()))
		{
			// Dialog should have a primary slot, allowing users to navigate options via esc, space, enter, and tab keys
			ParentWindow->SetWidgetToFocusOnActivate(PrimaryButton);
		}
		else
		{
			// Fallback to entire dialog. This will only support esc key
			ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));
		}

#undef ADD_SLOT
#undef ADD_SLOT_PRIMARY
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


	EAppReturnType::Type GetResponse()
	{
		return Response;
	}

	virtual	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		//see if we pressed the Escape to cancel choice
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return HandleButtonClicked(EAppReturnType::Cancel);
		}

		if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
		{
			CopyMessageToClipboard();

			return FReply::Handled();
		}

		//if it was some other button, ignore it
		return FReply::Unhandled();
	}

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const
	{
		return true;
	}

	/** Converts an EAppReturnType into a localized FText */
	static FText EAppReturnTypeToText(EAppReturnType::Type ReturnType)
	{
		switch(ReturnType)
		{
		case EAppReturnType::No:
			return FDialogButtonTexts::Get().No;
		case EAppReturnType::Yes:
			return FDialogButtonTexts::Get().Yes;
		case EAppReturnType::YesAll:
			return FDialogButtonTexts::Get().YesAll;
		case EAppReturnType::NoAll:
			return FDialogButtonTexts::Get().NoAll;
		case EAppReturnType::Cancel:
			return FDialogButtonTexts::Get().Cancel;
		case EAppReturnType::Ok:
			return FDialogButtonTexts::Get().Ok;
		case EAppReturnType::Retry:
			return FDialogButtonTexts::Get().Retry;
		case EAppReturnType::Continue:
			return FDialogButtonTexts::Get().Continue;
		default:
			return LOCTEXT("MissingType", "MISSING RETURN TYPE");
		}
	}

protected:

	/**
	 * Copies the message text to the clipboard.
	 */
	void CopyMessageToClipboard( )
	{
		FPlatformApplicationMisc::ClipboardCopy( *MyMessage.Get().ToString() );
	}


private:

	// Handles clicking a message box button.
	FReply HandleButtonClicked( EAppReturnType::Type InResponse )
	{
		Response = InResponse;
		if ((MessageType == EAppMsgType::YesNoYesAllNoAll || MessageType == EAppMsgType::YesNoYesAllNoAllCancel)
			&& bApplyToAllChecked)
		{
			if (Response == EAppReturnType::Yes)
			{
				Response = EAppReturnType::YesAll;
			}
			else if (Response == EAppReturnType::No)
			{
				Response = EAppReturnType::NoAll;
			}
		}

		ResultCallback.ExecuteIfBound(ParentWindow.ToSharedRef(), Response);


		ParentWindow->RequestDestroyWindow();

		return FReply::Handled();
	}

	// Handles clicking the 'Copy Message' button.
	FReply HandleCopyMessageButtonClicked()
	{
		CopyMessageToClipboard();
		return FReply::Handled();
	}
		
	// Used as a delegate for the OnClicked property of the Apply to All checkbox
	void OnCheckboxClicked(ECheckBoxState InNewState)
	{
		bApplyToAllChecked = InNewState == ECheckBoxState::Checked;
	}

	// Used as a delegate for the Visibility property of the Apply to All checkbox
	EVisibility GetCheckboxVisibility() const
	{
		if (MessageType == EAppMsgType::YesNoYesAllNoAll || MessageType == EAppMsgType::YesNoYesAllNoAllCancel)
		{
			return EVisibility::Visible;
		}
		return EVisibility::Hidden;
	}

public:
	/** Callback delegate that is triggered, when the dialog is run in non-modal mode */
	FOnMsgDlgResult ResultCallback;

private:

	EAppReturnType::Type Response;
	TSharedPtr<SWindow> ParentWindow;
	TAttribute<FText> MyMessage;
	bool bApplyToAllChecked;
	EAppMsgType::Type MessageType;
};


void CreateMsgDlgWindow(TSharedPtr<SWindow>& OutWindow, TSharedPtr<SChoiceDialog>& OutDialog, EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType,
						const FText& InMessage, const FText& InTitle, FOnMsgDlgResult ResultCallback=NULL)
{
	OutWindow = SNew(SWindow)
		.Title(InTitle)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false).SupportsMaximize(false);

	OutDialog = SNew(SChoiceDialog)
		.ParentWindow(OutWindow)
		.Message(InMessage)
		.WrapMessageAt(512.0f)
		.MessageCategory(InMessageCategory)
		.MessageType(InMessageType);

	OutDialog->ResultCallback = ResultCallback;

	OutWindow->SetContent(OutDialog.ToSharedRef());
}

EAppReturnType::Type OpenMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, EAppReturnType::Type InDefaultValue, const FText& InMessage, const FText& InTitle)
{
	EAppReturnType::Type Response = InDefaultValue;
	if (FApp::IsUnattended() == true || GIsRunningUnattendedScript)
	{
		UE_LOGF(LogDialogs, Log, "Message Dialog was triggered in unattended script mode without a default value. %d will be used.", (int32)InDefaultValue);
	}
	else
	{
		TSharedPtr<SWindow> MsgWindow = NULL;
		TSharedPtr<SChoiceDialog> MsgDialog = NULL;

		CreateMsgDlgWindow(MsgWindow, MsgDialog, InMessageCategory, InMessageType, InMessage, InTitle);

		GEditor->EditorAddModalWindow(MsgWindow.ToSharedRef());

		Response = MsgDialog->GetResponse();
	}

	return Response;
}

EAppReturnType::Type OpenMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle)
{
	EAppReturnType::Type DefaultValue = EAppReturnType::Yes;
	switch (InMessageType)
	{
	case EAppMsgType::Ok:
		DefaultValue = EAppReturnType::Ok;
		break;
	case EAppMsgType::YesNo:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::OkCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::CancelRetryContinue:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
	default:
		DefaultValue = EAppReturnType::Yes;
		break;
	}

	if (GIsRunningUnattendedScript && InMessageType != EAppMsgType::Ok)
	{
		UE_LOGF(LogDialogs, Error, "Message Dialog was triggered in unattended script mode without a default value. %d will be used.", (int32)DefaultValue);
		if (FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
		else
		{
			FDebug::DumpStackTraceToLog(ELogVerbosity::Error);
		}
	}

	return OpenMessageDialog_Internal(InMessageCategory, InMessageType, DefaultValue, InMessage, InTitle);
}

EAppReturnType::Type OpenMsgDlgInt(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle)
{
	return OpenMessageDialog_Internal(EAppMsgCategory::Warning, InMessageType, InMessage, InTitle);
}

EAppReturnType::Type OpenMsgDlgInt(EAppMsgType::Type InMessageType, EAppReturnType::Type InDefaultValue, const FText& InMessage, const FText& InTitle)
{
	return OpenMessageDialog_Internal(EAppMsgCategory::Warning, InMessageType, InDefaultValue, InMessage, InTitle);
}

TSharedRef<SWindow> OpenMsgDlgInt_NonModal(EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle,
							FOnMsgDlgResult ResultCallback)
{
	TSharedPtr<SWindow> MsgWindow = NULL;
	TSharedPtr<SChoiceDialog> MsgDialog = NULL;

	CreateMsgDlgWindow(MsgWindow, MsgDialog, EAppMsgCategory::Warning, InMessageType, InMessage, InTitle, ResultCallback);

	FSlateApplication::Get().AddWindow(MsgWindow.ToSharedRef());

	return MsgWindow.ToSharedRef();
}

class SModalDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SModalDialog ){}
		SLATE_ARGUMENT(FText,Message)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		MyMessage = InArgs._Message;
		this->ChildSlot
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(5.0f)
			[
				SNew( STextBlock )
				.WrapTextAt(615.0f)	// 400.0f
				.Text( MyMessage )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SButton )
					.Text( NSLOCTEXT("UnrealEd", "Yes", "Yes") )
					.OnClicked( this, &SModalDialog::OnYesClicked )
					.ContentPadding(7.0f)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SButton )
					.Text( NSLOCTEXT("UnrealEd", "No", "No") )
					.OnClicked( this, &SModalDialog::OnNoClicked )
					.ContentPadding(7.0f)
				]
			]
		];
	}

	
	SModalDialog()
	: bUserResponse( false )
	{
	}

	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		MyWindow = InWindow;
	}

	bool GetResponse() const { return bUserResponse; }
	virtual bool SupportsKeyboardFocus() const override { return true; }

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
		{
			FPlatformApplicationMisc::ClipboardCopy( *MyMessage.Get().ToString() );
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

private:
	
	FReply OnYesClicked()
	{
		bUserResponse = true;
		MyWindow->RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnNoClicked()
	{
		bUserResponse = false;
		MyWindow->RequestDestroyWindow();
		return FReply::Handled();
	}

	TSharedPtr<SWindow> MyWindow;
	bool bUserResponse;
	TAttribute<FText> MyMessage;
};

/**
 * SModalDialogWithCheckbox 
 * Modal dialog with one or two buttons and a checkbox. 
 * All text and images contained are customizable.
 * Setup so Escape acts as cancel.
 */
class SModalDialogWithCheckbox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SModalDialogWithCheckbox )
	: _bHasCancelButton(false)
	, _WrapMessageAt(512.0f)
	{}
		/** Warning message displayed on the dialog */
		SLATE_ATTRIBUTE(FText, Message)

		/** Message Displayed next to the checkbox */
		SLATE_ATTRIBUTE(FText, CheckboxMessage)
		
		/** Text to display on the confirm button */
		SLATE_ATTRIBUTE(FText, ConfirmText)

		/** Text to display on the cancel button */
		SLATE_ATTRIBUTE(FText, CancelText)

		/** If true an extra button is displayed to be used as a cancel button */
		SLATE_ARGUMENT(bool, bHasCancelButton)

		/** Default value of the checkbox */
		SLATE_ARGUMENT(bool, bDefaultCheckValue)

		/** Wrap message at specified length, zero or negative number will disable the wrapping */
		SLATE_ARGUMENT(float, WrapMessageAt)

		/** Typically an icon to help the user more easily identify the nature of the issue */
		SLATE_ATTRIBUTE( const FSlateBrush*, Image )

		/** Window in which this widget resides */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)

	SLATE_END_ARGS()

	/** Used to construct widgets */
	void Construct( const FArguments& InArgs )
	{
		bCheckboxResult = InArgs._bDefaultCheckValue;
		MyWindow = InArgs._ParentWindow.Get();
		MyMessage = InArgs._Message;
		MyCheckboxMessage = InArgs._CheckboxMessage;

		FSlateFontInfo MessageFont( FAppStyle::GetFontStyle("StandardDialog.LargeFont"));

		ChildSlot
		[
			SNew(SBorder)
			.Padding(16.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(1.0f)
				.Padding(0)
				.MaxHeight(550)
				[
					SNew( SScrollBox )
					+ SScrollBox::Slot()
					[
						SNew(SHorizontalBox)
				
						// warning image
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew( SImage )
							.Image(InArgs._Image)
						]

						// main warning message
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(16.f, 0.f, 0.f, 0.f))
						[
							SNew( STextBlock )
							.WrapTextAt(InArgs._WrapMessageAt)
							.Text(MyMessage)
							.Font(MessageFont)
						]
					]
				]
			
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.Padding(FMargin(0.f, 32.f, 0.f, 0.f))
				[
					ConstructConditionalInternals(InArgs)
				]
			]
		];
	}
	
	/** 
	 * Constructs the widget components which require conditional checks.
	 *
	 * @param InArgs - classes custom arguments passed though from construct
	 *
	 * @returns A Shared reference to a SHorizontalBox containing all newly construct widgets.
	 */
	TSharedRef<SHorizontalBox> ConstructConditionalInternals( const FArguments& InArgs ) 
	{
		TSharedPtr<SPrimaryButton> PrimaryButton;
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
		
		// checkbox with user specified text
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Left)
		.FillWidth(1.0)
		[
			SNew(SCheckBox)
			.IsChecked(InArgs._bDefaultCheckValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged(this, &SModalDialogWithCheckbox::OnCheckboxClicked)
			.Visibility(this, &SModalDialogWithCheckbox::GetCheckboxVisibility)
			[
				SNew( STextBlock )
				.WrapTextAt(615.0f)
				.Text( MyCheckboxMessage )
			]
		];
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Right)
		.Padding(FMargin(16.f, 0.f, 0.f, 0.f)) // currently hardcoded until we adjust StandardDialog.SlotPadding
		.AutoWidth()
		[
			SAssignNew(PrimaryButton, SPrimaryButton)
			.Text(InArgs._ConfirmText)
			.OnClicked(this, &SModalDialogWithCheckbox::OnConfirmClicked)			
		];


		// Only add a cancel button if required
		if (InArgs._bHasCancelButton)
		{
			// cancel/stop/abort button
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Right)
			.Padding(FMargin(8.f, 0.f, 0.f, 0.f)) // currently hardcoded until we adjust StandardDialog.SlotPadding
			.AutoWidth()
			[
				SNew( SButton )
				.Text( InArgs._CancelText )
				.OnClicked( this, &SModalDialogWithCheckbox::OnCancelClicked )
			];
		}
	
		// Route user focus to the default dialog button. 
		if (ensure(MyWindow.Pin()))
		{
			MyWindow.Pin()->SetWidgetToFocusOnActivate(PrimaryButton);
		}

		return HorizontalBox;
	}

	SModalDialogWithCheckbox()
	: bUserResponse(false)
	, bCheckboxResult(false)
	{
	}
	
	/** Returns true if the user pressed the confirm button, otherwise false. */
	bool GetResponse() const { return bUserResponse; }

	/** Returns true if the user activated the checkbox */
	bool GetCheckBoxState() const { return bCheckboxResult; }
	
	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const
	{
		return true;
	}

	/** Used to intercept Escape key presses, then interprets them as cancel */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		// Pressing escape returns as if the user canceled
		if ( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancelClicked();
		}

		if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
		{
			FPlatformApplicationMisc::ClipboardCopy( *MyMessage.Get().ToString() );
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

private:

	/** Used as a delegate for the confirm buttons OnClicked method */
	FReply OnConfirmClicked()
	{
		bUserResponse = true;
		MyWindow.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}
	
	/** Used as a delegate for the Cancel buttons OnClicked method */
	FReply OnCancelClicked()
	{
		bUserResponse = false;
		MyWindow.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	/** Used as a delegate for the Checkboxs OnClicked method */
	void OnCheckboxClicked(ECheckBoxState InNewState)
	{
		bCheckboxResult = InNewState == ECheckBoxState::Checked;
	}

	/** Used as a delegate for the Checkboxs Visibile method */
	EVisibility GetCheckboxVisibility() const
	{
		return (MyCheckboxMessage.Get().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible);
	}

	/** Used to cache the users response to the warning */
	bool bUserResponse;

	/**  Used to cache whether the user activated the checkbox*/
	bool bCheckboxResult;

	/** Pointer to the window which holds this Widget, required for modal control */
	TWeakPtr<SWindow> MyWindow;

	TAttribute<FText> MyMessage;
	TAttribute<FText> MyCheckboxMessage;
};

TSet<FString> FSuppressableWarningDialog::SuppressedInTheSession = {};

FString SuppressableWarningDialogGetSessionKey(const FString& IniSettingName, const FString& IniSettingFileName)
{
	return IniSettingFileName + TEXT("_") + IniSettingName;
}

FSuppressableWarningDialog::FSuppressableWarningDialog(const FSetupInfo& Info)
{
	// Ensure proper usage of the suppression warning.
	checkf(!Info.ConfirmText.IsEmpty(), TEXT("All warnings should have ConfirmText set!"));

	const static FString ConfigSection = TEXT("SuppressableDialogs");
	
	bool bShouldSuppressDialog = false;

	// Cache the value suppression value to be check and possible reset in ShowModal.
	IniSettingName = Info.IniSettingName;
	IniSettingFileName = Info.IniSettingFileName;
	Prompt = Info.Message;
	ResponseIniSettingName = Info.IniSettingName + TEXT("_ConfirmResponse");

	// bDontPersistSuppressionAcrossSessions takes precedence until removed
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Info.bDontPersistSuppressionAcrossSessions)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		DialogMode = EMode::DontPersistSuppressionAcrossSessions;
	}
	else
	{
		DialogMode = Info.DialogMode;
	}

	if (DialogMode == EMode::DontPersistSuppressionAcrossSessions)
	{
		bShouldSuppressDialog = SuppressedInTheSession.Contains(SuppressableWarningDialogGetSessionKey(IniSettingName, IniSettingFileName));
	}
	else
	{
		GConfig->GetBool( *ConfigSection, *IniSettingName, bShouldSuppressDialog, IniSettingFileName );
	}
	
	if (!bShouldSuppressDialog && FSlateApplication::IsInitialized())
	{
		ModalWindow = SNew(SWindow)
		.Title(Info.Title)
		.SizingRule( ESizingRule::Autosized )
		.SupportsMaximize(false) .SupportsMinimize(false);

		// Cache a default image to be used as most cases will not provide their own.
		static const FSlateBrush* DefaultImage = FAppStyle::GetBrush("Icons.WarningWithColor.Large");

		MessageBox = SNew(SModalDialogWithCheckbox)
			.Message(Prompt)
			.bHasCancelButton(!Info.CancelText.IsEmpty())
			.ConfirmText(Info.ConfirmText)
			.CancelText(Info.CancelText)
			.bDefaultCheckValue(Info.bDefaultToSuppressInTheFuture)
			.CheckboxMessage(Info.CheckBoxText)
			.ParentWindow(ModalWindow)
			.Image((Info.Image != NULL) ? Info.Image : DefaultImage)
			.WrapMessageAt(Info.WrapMessageAt);

		ModalWindow->SetContent( MessageBox.ToSharedRef() );
	}
}

FSuppressableWarningDialog::EResult FSuppressableWarningDialog::ShowModal() const
{
	const FString ConfigSection = TEXT("SuppressableDialogs");
	// Assume we should not suppress the dialog
	bool bShouldSuppressDialog = false;

	// Get the setting from the config file.
	if (DialogMode == EMode::DontPersistSuppressionAcrossSessions)
	{
		bShouldSuppressDialog = SuppressedInTheSession.Contains(SuppressableWarningDialogGetSessionKey(IniSettingName, IniSettingFileName));
	}
	else
	{
		GConfig->GetBool( *ConfigSection, *IniSettingName, bShouldSuppressDialog, IniSettingFileName );
	}

	EResult RetCode = Suppressed;
	if( !bShouldSuppressDialog )
	{
		GEditor->EditorAddModalWindow(ModalWindow.ToSharedRef());
		RetCode = (MessageBox->GetResponse()) ? Confirm : Cancel;
		
		// Set the ini variable to the state of the disable check box
		bShouldSuppressDialog = MessageBox->GetCheckBoxState();

		if( RetCode == Confirm )
		{
			if (DialogMode == EMode::DontPersistSuppressionAcrossSessions)
			{
				if (bShouldSuppressDialog)
				{
					SuppressedInTheSession.Add(SuppressableWarningDialogGetSessionKey(IniSettingName, IniSettingFileName));
				}
			}
			else
			{
				GConfig->SetBool(*ConfigSection, *IniSettingName, bShouldSuppressDialog, IniSettingFileName);

				if (DialogMode == EMode::PersistUserResponse)
				{
					GConfig->SetBool(*ConfigSection, *ResponseIniSettingName, true, IniSettingFileName);
				}
				GConfig->Flush(false, IniSettingFileName);
			}
		}
		else
		{
			if (DialogMode == EMode::PersistUserResponse)
			{
				GConfig->SetBool(*ConfigSection, *IniSettingName, bShouldSuppressDialog, IniSettingFileName);
				GConfig->SetBool(*ConfigSection, *ResponseIniSettingName, false, IniSettingFileName);
				GConfig->Flush(false, IniSettingFileName);
			}
		}
	}
	else
	{
		// If the dialog is suppressed, log the warning
		UE_LOGF(LogDialogs, Warning, "Suppressed: %ls", *Prompt.ToString());

		if (DialogMode == EMode::PersistUserResponse)
		{
			// Get the saved response
			bool bWasConfirmed = false;
			GConfig->GetBool(*ConfigSection, *ResponseIniSettingName, bWasConfirmed, IniSettingFileName);

			RetCode = bWasConfirmed ? Confirm : Cancel;
		}
	}

	return RetCode;
}

bool PromptUserForDirectory(FString& OutDirectory, const FString& Message, const FString& DefaultPath)
{
	bool bFolderSelected = false;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		FString FolderName;
		bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			Message,
			DefaultPath,
			FolderName
			);

		if ( bFolderSelected )
		{
			OutDirectory = FolderName;
		}
	}
	return bFolderSelected;
}

bool PromptUserIfExistingObject(const FString& Name, const FString& Package, const FString& Group, UPackage*& Pkg)
{
	return PromptUserIfExistingObject(Name, Package, Pkg);
}

bool PromptUserIfExistingObject(const FString& Name, const FString& Package, UPackage* &Pkg)
{
	FString	QualifiedName = Package + TEXT(".");
	QualifiedName += Name;

	// Check for an existing object
	UObject* ExistingObject = StaticFindObject( UObject::StaticClass(), nullptr, *QualifiedName );
	if( ExistingObject != nullptr )
	{
		// Object already exists in either the specified package or another package.  Check to see if the user wants
		// to replace the object.
		TSharedRef<SMessageDialog> ConfirmDialog = SNew(SMessageDialog)
			.Icon(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
			.Title(FText(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Title", "Overwrite Existing Object")))
			.Message(FText::Format(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_Message", "An object already exists with this name.\n\n\tName: {0}\n\tClass: {1}\n\tAsset path: {2}\n\nOverwrite the existing object?"),
				FText::FromString(Name),
				FText::FromString(ExistingObject->GetClass()->GetName()),
				FText::FromString(Package)))
			.Buttons({
				SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonOverwrite", "Overwrite")).SetPrimary(true),
				SCustomDialog::FButton(NSLOCTEXT("UnrealEd", "ReplaceExistingObjectInPackageConfirmation_ButtonCancel", "Cancel")),
				})
			.ContentMinWidth(300.0f);
		uint32 ConfirmationResult = ConfirmDialog->ShowModal();

		bool bWantReplace = ConfirmationResult == 0;

		if( bWantReplace )
		{
			// Replacing an object.  Here we go!
			//Try to Delete the existing object
			if (ObjectTools::DeleteSingleObject( ExistingObject ))
			{
				// Force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
				CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

				// Old package will be GC'ed... create a new one here
				Pkg = CreatePackage(*Package);
			}
			else //failed to delete
			{
				// Notify the user that the operation failed b/c the existing asset couldn't be deleted
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("DlgNewGeneric", "ContentBrowser_CannotDeleteExistingAsset", "The new asset wasn't created due to a problem while attempting\nto delete the existing '{0}' asset."), FText::FromString( Name ) ) );
				return false;
			}
		}
		else
		{
			// User chose not to replace the object; they'll need to enter a new name
			return false;
		}
	}
	return true;
}

void SGenericDialogWidget::Construct( const FArguments& InArgs )
{
	TSharedPtr<SWidget> ContentWidget;
	if (InArgs._UseScrollBox)
	{
		ContentWidget = 
			SNew(SBox)
			.MaxDesiredHeight(static_cast<float>(InArgs._ScrollBoxMaxHeight))
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					InArgs._Content.Widget
				]
			];
	}
	else
	{
		ContentWidget = InArgs._Content.Widget;
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			ContentWidget.ToSharedRef()
		]

		+SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoHeight()
		.Padding(0.f, 2.f, 0.f, 0.f)
		[
			SNew(SButton)
			.Text( NSLOCTEXT("UnrealEd", "OK", "OK") )
			.OnClicked(this, &SGenericDialogWidget::OnOK_Clicked)
		]
	];

	OkPressedDelegate = InArgs._OnOkPressed;
}

void SGenericDialogWidget::OpenDialog(const FText& InDialogTitle, const TSharedRef< SWidget >& DisplayContent, const FArguments& InArgs, bool bAsModalDialog)
{
	TSharedPtr< SWindow > Window;
	TSharedPtr< SGenericDialogWidget > GenericDialogWidget;

	Window = SNew(SWindow)
		.Title(InDialogTitle)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew( SBorder )
			.Padding( 4.f )
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SAssignNew(GenericDialogWidget, SGenericDialogWidget)
				.UseScrollBox(InArgs._UseScrollBox)
				.ScrollBoxMaxHeight(InArgs._ScrollBoxMaxHeight)
				.OnOkPressed(InArgs._OnOkPressed)
				[
					DisplayContent
				]
			]
		];

	GenericDialogWidget->SetWindow(Window);

	if(bAsModalDialog)
	{
		GEditor->EditorAddModalWindow(Window.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow( Window.ToSharedRef() );
	}
}

FReply SGenericDialogWidget::OnOK_Clicked(void)
{
	OkPressedDelegate.ExecuteIfBound();
	
	MyWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

TSharedRef<SWindow> UE::Private::CreateModalDialogWindow(
	const FText& InTitle,
	TSharedRef<SWidget> Contents,
	ESizingRule Sizing,
	FVector2D MinDimensions)
{
	// clang-format off
	return SNew(SWindow)
		.Title(InTitle)
		.SizingRule(Sizing)
		.MinWidth(MinDimensions.X)
		.MinHeight(MinDimensions.Y)
		.ClientSize(MinDimensions)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.HasCloseButton(false)
		[
			SNew(SBorder)
			.Padding(4.f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				MoveTemp(Contents)
			]
		];
	// clang-format on
}

void UE::Private::ShowModalDialogWindow(TSharedRef<SWindow> Window)
{
	GEditor->EditorAddModalWindow(Window);
}

void UE::Private::ShowNonModalDialogWindow(TSharedRef<SWindow> Window)
{
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	if (MainFrameModule.GetParentWindow().IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}
}

#undef LOCTEXT_NAMESPACE 
