// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Opens a modal/blocking message box dialog (with an additional 'copy message text' button), and returns the result immediately
 * For internal use only, use FMessageDialog::Open() instead.
 *
 * @param InMessageCategory	The category of message box to display (ie, which icon to use)
 * @param InMessageType		The type of message box to display (e.g. 'ok', or 'yes'/'no' etc.)
 * @param InMessage			The message to display in the message box
 * @param InTitle			The title to display for the message box
 * @return					Returns the result of the user input
 */
EAppReturnType::Type OpenMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle);
