// Copyright Epic Games, Inc. All Rights Reserved.


#include "Dialogs/DlgMoveAssets.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	SDlgMoveAsset
//
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class SDlgMoveAsset : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDlgMoveAsset )
		{}
		
		/** This is used ether as the whole Path or as just the package information when bUseLegacyMapPackage is set */
		SLATE_ATTRIBUTE(FString, AssetPackage)

		/** Group information, this is only displayed when bUseLegacyMapPackage is set */
		SLATE_ATTRIBUTE(FString, AssetGroup)

		/** Name information, this is only displayed when bUseLegacyMapPackage is set. Otherwise it is added onto Package */
		SLATE_ATTRIBUTE(FString, AssetName)

		/** If True the window will display the window used for legacy or map packages */
		SLATE_ATTRIBUTE(bool, bUseLegacyMapPackage)
		
		/** Window in which this widget resides */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		
	SLATE_END_ARGS()

	/** Used to construct widgets */
	void Construct( const FArguments& InArgs )
	{
		// Set this widget as focused, to allow users to hit ESC to cancel.
		ParentWindow = InArgs._ParentWindow.Get();
		ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));
	
		// setup names with their original values. 
		AssetPackage = InArgs._AssetPackage.Get();
		AssetGroup = InArgs._AssetGroup.Get();
		AssetName = InArgs._AssetName.Get();

		// Cache whether its a Legacy or Map package as we may need to 
		// call validate later. 
		bLegacyOrMapPackage = InArgs._bUseLegacyMapPackage.Get();

		EVisibility LegacyOrMapPackageVisibility = EVisibility::Collapsed;
		FText LegacyOrMapPackagePathText = NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_Path", "Path");
		
		if (bLegacyOrMapPackage)
		{
			// Make the Group and Name fields visible, and change the Path field to be called Package.
			LegacyOrMapPackageVisibility = EVisibility::Visible;
			LegacyOrMapPackagePathText = NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_Package", "Package");
		}
		
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SNew(SVerticalBox)
			
				// Add user input block
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(3.0f)
						[
							SNew(SGridPanel)

							+SGridPanel::Slot(0,0)
							.Padding(0,0,10,5)
							[
								SNew(STextBlock)
								.Text(LegacyOrMapPackagePathText)
							]

							// This box will not be editable if the package is a map or legacy.
							+SGridPanel::Slot(1,0)
							.Padding(0,0,0,5)
							[
								SNew(SEditableTextBox)
								.Text( FText::FromString(AssetPackage) )
								.OnTextCommitted(this, &SDlgMoveAsset::OnPathChange)
								.IsEnabled(!bLegacyOrMapPackage)
								.MinDesiredWidth(250)
							]

							// The visibility of the following components is conditional upon
							// the package being legacy or a map package.
							+SGridPanel::Slot(0,1)
							.Padding(0,0,10,5)
							[
								SNew(STextBlock)
								.Text(NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_Group", "Group"))
								.Visibility(LegacyOrMapPackageVisibility)
							]

							+SGridPanel::Slot(1,1)
							.Padding(0,0,0,5)
							[
								SNew(SEditableTextBox)
								.Text(FText::FromString(AssetGroup))
								.OnTextCommitted(this, &SDlgMoveAsset::OnGroupChange)
								.MinDesiredWidth(250)
								.Visibility(LegacyOrMapPackageVisibility)
							]

							+SGridPanel::Slot(0,2)
							.Padding(0,0,10,0)
							[
								SNew(STextBlock)
								.Text(NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_Name", "Name"))
								.Visibility(LegacyOrMapPackageVisibility)
							]

							+SGridPanel::Slot(1,2)
							[
								SNew(SEditableTextBox)
								.Text(FText::FromString(AssetName))
								.OnTextCommitted(this, &SDlgMoveAsset::OnNameChange)
								.MinDesiredWidth(250)
								.Visibility(LegacyOrMapPackageVisibility)
							]
						]
					]
					
				// Add Ok, Ok to all and Cancel buttons.
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(5)
				.HAlign(HAlign_Right)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text( NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_OK", "OK") )
						.OnClicked( this, &SDlgMoveAsset::OnButtonClick, FDlgMoveAsset::OK )
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text( NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_OKToAll", "OK to All") )
						.OnClicked( this, &SDlgMoveAsset::OnButtonClick, FDlgMoveAsset::OKToAll )
					]
					+SUniformGridPanel::Slot(2,0)
					[
						SNew(SButton) 
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text( NSLOCTEXT("ModalDialogs", "SDlgMoveAsset_Cancel", "Cancel") )
						.OnClicked( this, &SDlgMoveAsset::OnButtonClick, FDlgMoveAsset::Cancel )
					]
				]
			]
		];
	}
	
	SDlgMoveAsset()
	: bUserResponse(FDlgMoveAsset::Cancel)
	, bLegacyOrMapPackage(false)
	{
	}
	
	/** 
	 * Returns the EResult of the button which the user pressed, if the user
	 * canceled the action using ESC it will return as if canceled. 
	 */
	FDlgMoveAsset::EResult GetUserResponse() const 
	{
		return bUserResponse; 
	}
		
	/** Accesses the ObjectPackage value */
	FString GetNewPackage() const
	{
		return AssetPackage;
	}

	/** Accesses the ObjectGroup value */
	FString GetNewGroup() const
	{
		return AssetGroup;
	}

	/** Accesses the ObjectName value */
	FString GetNewName() const
	{
		return AssetName;
	}

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const
	{
		return true;
	}

	/** Used to intercept Escape key presses, then interprets them as cancel */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		// Pressing escape returns as if the user canceled
		if ( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnButtonClick(FDlgMoveAsset::Cancel);
		}

		return FReply::Unhandled();
	}

private:

	/** 
	 * Handles when a button is pressed, should be bound with appropriate EResult Key
	 * 
	 * @param ButtonID - The return type of the button which has been pressed.
	 */
	FReply OnButtonClick(FDlgMoveAsset::EResult ButtonID)
	{
		ParentWindow->RequestDestroyWindow();
		bUserResponse = ButtonID;

		if (ButtonID != FDlgMoveAsset::Cancel)
		{
			if (!ValidatePackage())
			{
				// act as if user canceled if the package is invalid
				bUserResponse = FDlgMoveAsset::Cancel;
			}
		}

		return FReply::Handled();
	}

	/** Used as a delegate for when the Path/Package value changes. */
	void OnPathChange(const FText& NewPackage, ETextCommit::Type CommitInfo)
	{
		AssetPackage = NewPackage.ToString();
	}

	/** Used as a delegate for when the Group value changes. */
	void OnGroupChange(const FText& NewGroup,  ETextCommit::Type CommitInfo)
	{
		AssetGroup = NewGroup.ToString();
	}
	
	/** Used as a delegate for when the Name value changes. */
	void OnNameChange(const FText& NewName,  ETextCommit::Type CommitInfo)
	{
		AssetName = NewName.ToString();
	}

	/** Ensures supplied package name and group information is valid */
	bool ValidatePackage()
	{
		if ( !bLegacyOrMapPackage )
		{
			// Package is the full path, not using groups, and name is determined by the last element in the path
			AssetGroup = TEXT("");
			AssetName = FPackageName::GetLongPackageAssetName(AssetPackage);
		}

		FText Reason;
		if( !FPackageName::IsValidLongPackageName( AssetPackage, false, &Reason )
			||	!FName(*AssetGroup).IsValidGroupName( Reason, true )
			||	!FName(*AssetName).IsValidObjectName( Reason ) )
		{
			FMessageDialog::Open( EAppMsgType::Ok, Reason );
			return 0;
		}
	
		return true;
	}

	/** Used to cache the users response to the warning */
	FDlgMoveAsset::EResult bUserResponse;

	/** Pointer to the window which holds this Widget, required for modal control */
	TSharedPtr<SWindow> ParentWindow;

	/** Hold the data pertaining to the current object */
	FString AssetPackage, AssetGroup, AssetName;

	/** Caches whether this is a legacy or map package */
	bool bLegacyOrMapPackage;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	FDlgMoveAsset
//
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FDlgMoveAsset::FDlgMoveAsset(bool bInLegacyOrMapPackage, const FString& InPackage, const FString& InGroup, const FString& InName, const FText& InTitle)
{
	if (FSlateApplication::IsInitialized())
	{
		MoveAssetWindow = SNew(SWindow)
			.Title(InTitle)
			.SupportsMinimize(false) .SupportsMaximize(false)
			.SizingRule( ESizingRule::Autosized );

		FString CurrentAssetPackage, CurrentAssetGroup;

		if (!bInLegacyOrMapPackage)
		{
			CurrentAssetGroup = TEXT("");
			CurrentAssetPackage = FPackageName::GetLongPackagePath(InPackage) + TEXT("/") + InName;
		}
		else
		{
			CurrentAssetPackage = InPackage;
			CurrentAssetGroup = InGroup;
		}

		MoveAssetWidget = SNew(SDlgMoveAsset)
		.AssetName(InName)
		.AssetGroup(CurrentAssetGroup)
		.AssetPackage(CurrentAssetPackage)
		.bUseLegacyMapPackage(bInLegacyOrMapPackage)
		.ParentWindow(MoveAssetWindow);

		MoveAssetWindow->SetContent( MoveAssetWidget.ToSharedRef() );
	}
}

FDlgMoveAsset::EResult FDlgMoveAsset::ShowModal()
{
	GEditor->EditorAddModalWindow(MoveAssetWindow.ToSharedRef());
	return (EResult)MoveAssetWidget->GetUserResponse();
}

FString FDlgMoveAsset::GetNewPackage() const
{
	check(MoveAssetWidget.IsValid());
	return MoveAssetWidget->GetNewPackage();

}

FString FDlgMoveAsset::GetNewGroup() const
{
	check(MoveAssetWidget.IsValid());
	return MoveAssetWidget->GetNewGroup();
}

FString FDlgMoveAsset::GetNewName() const
{
	check(MoveAssetWidget.IsValid());
	return MoveAssetWidget->GetNewName();
}
