// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/DlgPickAssetPath.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "DlgPickAssetPath"

void SDlgPickAssetPath::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));
	AssetName = FText::FromString(FPackageName::GetLongPackageAssetName(InArgs._DefaultAssetPath.ToString()));

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SDlgPickAssetPath::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;
	PathPickerConfig.bAllowClassesFolder = true;
	PathPickerConfig.bAllowReadOnlyFolders = InArgs._AllowReadOnlyFolders;
	PathPickerConfig.bCanShowDevelopersFolder = InArgs._CanShowDevelopersFolder;
	PathPickerConfig.bShowViewOptions = InArgs._ShowViewOptions;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450,450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2,2,2,4)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3.0f)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(3.0f)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 10, 0)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Name", "Name"))
						]

						+SHorizontalBox::Slot()
						[
							SNew(SEditableTextBox)
							.Text(AssetName)
							.OnTextCommitted(this, &SDlgPickAssetPath::OnNameChange)
							.MinDesiredWidth(250)
						]
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(8.f, 16.f)
			[
				SNew(SUniformGridPanel)
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))

				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("Save", "Save"))
					.OnClicked(this, &SDlgPickAssetPath::OnButtonClick, EAppReturnType::Ok)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SDlgPickAssetPath::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SDlgPickAssetPath::OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo)
{
	AssetName = NewName;
}

void SDlgPickAssetPath::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
}

FReply SDlgPickAssetPath::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	
	if (ButtonID == EAppReturnType::Cancel || ValidatePackage())
	{
		// Only close the window if canceling or if the asset name is valid
		RequestDestroyWindow();
	}
	else
	{
		// reset the user response in case the window is closed using 'x'.
		UserResponse = EAppReturnType::Cancel;
	}

	return FReply::Handled();
}

/** Ensures supplied package name information is valid */
bool SDlgPickAssetPath::ValidatePackage()
{
	FText Reason;
	if (!FPackageName::IsValidLongPackageName(GetFullAssetPath().ToString(), false, &Reason)
		|| !FName(*AssetName.ToString()).IsValidObjectName(Reason))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason );
		return false;
	}

	if (FPackageName::DoesPackageExist(GetFullAssetPath().ToString()) || FindObject<UObject>(NULL, *(AssetPath.ToString() + "/" + AssetName.ToString() + "." + AssetName.ToString())) != NULL)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("AssetAlreadyExists", "Asset {0} already exists."), GetFullAssetPath()));
		return false;
	}

	return true;
}

EAppReturnType::Type SDlgPickAssetPath::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

const FText& SDlgPickAssetPath::GetAssetPath()
{
	return AssetPath;
}

const FText& SDlgPickAssetPath::GetAssetName()
{
	return AssetName;
}

FText SDlgPickAssetPath::GetFullAssetPath()
{
	return FText::FromString(AssetPath.ToString() + "/" + AssetName.ToString());
}


#undef LOCTEXT_NAMESPACE
