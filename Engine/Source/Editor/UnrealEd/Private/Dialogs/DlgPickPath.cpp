// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/DlgPickPath.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "DlgPickPath"

void SDlgPickPath::Construct(const FArguments& InArgs)
{
	Path = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultPath.ToString()));

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = Path.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SDlgPickPath::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;
	PathPickerConfig.bAllowReadOnlyFolders = InArgs._AllowReadOnlyFolders;
	PathPickerConfig.bCanShowDevelopersFolder = InArgs._CanShowDevelopersFolder;
	PathPickerConfig.bShowViewOptions = InArgs._ShowViewOptions;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450,450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2,2,2,4)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3.0f)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("OK", "OK"))
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SDlgPickPath::OnButtonClick, EAppReturnType::Ok)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &SDlgPickPath::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SDlgPickPath::OnPathChange(const FString& NewPath)
{
	Path = FText::FromString(NewPath);
}

FReply SDlgPickPath::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	if (ButtonID == EAppReturnType::Cancel || ValidatePath())
	{
		// Only close the window if canceling or if the asset name is valid
		RequestDestroyWindow();
	}

	return FReply::Handled();
}

/** Ensures supplied package name information is valid */
bool SDlgPickPath::ValidatePath()
{
	FText Reason;
	if (Path.IsEmpty())
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoPathChooseError", "You must select a path."));
		return false;
	}

	return true;
}

EAppReturnType::Type SDlgPickPath::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

const FText& SDlgPickPath::GetPath()
{
	return Path;
}

#undef LOCTEXT_NAMESPACE
