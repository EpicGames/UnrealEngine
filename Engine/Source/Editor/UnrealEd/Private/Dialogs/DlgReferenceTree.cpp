// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/DlgReferenceTree.h"
#include "Containers/VersePath.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"


#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"

FArchiveGenerateReferenceGraph::FArchiveGenerateReferenceGraph( FReferenceGraph& OutGraph ) 
	: CurrentObject(NULL),
	  ObjectGraph(OutGraph)
{

	ArIsObjectReferenceCollector = true;
	ArIgnoreOuterRef = true;

	// Iterate over each object..
	for( FThreadSafeObjectIterator It; It; ++It )
	{
		UObject* Object	= *It;

		// Skip transient and those about to be deleted
		if( !Object->HasAnyFlags( RF_Transient ) && IsValid(Object) )
		{
			// only serialize non actors objects which have not been visited.
			// actors are skipped because we have don't need them to show the reference tree
			// @todo, may need to serialize them later for full reference graph.
			if( !VisitedObjects.Find( Object ) && !Object->IsA( AActor::StaticClass() ) )
			{
				// Set the current object to the one we are about to serialize
				CurrentObject = Object;
				// This object has been visited.  Any serializations after this should skip this object
				VisitedObjects.Add( Object );
				Object->Serialize( *this );
			}
		}
	}
}

FArchive& FArchiveGenerateReferenceGraph::operator<<( UObject*& Object )
{
	// Only look at objects which are valid
	const bool bValidObject = 
		IsValid(Object) &&	// Object should be valid
		!Object->HasAnyFlags( RF_Transient ) && // Should not be transient 
		(Cast<UClass>(Object) == NULL); // skip UClasses

	if( bValidObject )
	{
		// Determine if a node for the referenced object has already been created
		FReferenceGraphNode* ReferencedNode = ObjectGraph.FindRef( Object );
		if ( ReferencedNode == NULL )
		{
			// If no node has been created, create one now
			ReferencedNode = ObjectGraph.Add( Object, new FReferenceGraphNode( Object ) );
		}

		// Find a node for the referencer object.  CurrentObject references Object
		FReferenceGraphNode* ReferencerNode = ObjectGraph.FindRef( CurrentObject );
		if( ReferencerNode == NULL )
		{
			// If node node has been created, create one now
			ReferencerNode = ObjectGraph.Add( CurrentObject, new FReferenceGraphNode( CurrentObject ) );
		}

		// Ignore self referencing objects
		if( Object != CurrentObject )
		{
			// Add a new link from the node to what references it.  
			// Links represent references to the object contained in ReferencedNode
			ReferencedNode->Links.Add( ReferencerNode );
		}
		
		if( !VisitedObjects.Find( Object ) && !Object->IsA( AActor::StaticClass() ) )
		{
			// If this object hasnt been visited and is not an actor, serialize it

			// Store the current object for when we return from serialization 
			UObject* PrevObject = CurrentObject;
			// Set the new current object
			CurrentObject = Object;
			// This object has now been visited
			VisitedObjects.Add( Object );
			// Serialize
			Object->Serialize( *this );
			// Restore the current object
			CurrentObject = PrevObject;
		}
	}

	return *this;
}

namespace ReferenceTreeView
{
	namespace Helpers
	{
		/** Shows the passed in object in the content browser (if browsable) or the level (if actor) */
		void SelectObjectInEditor( UObject* InObjectToSelect )
		{
			AActor* Actor = Cast<AActor>( InObjectToSelect );
			if( Actor ) 
			{
				// Do not attempt to select script based objects
				if( !Actor->HasAnyFlags(RF_ClassDefaultObject) )
				{
					// Select and focus in on the actor
					GEditor->SelectNone( false, true );
					GEditor->SelectActor( Actor, true, true, true );
					GEditor->MoveViewportCamerasToActor( *Actor, true );
				}
			}
			else
			{
				// Show the object in the content browser.
				TArray<UObject*> ObjectsToSync;
				ObjectsToSync.Add(InObjectToSelect);
				GEditor->SyncBrowserToObjects(ObjectsToSync);
			}
		}
	}
}
static const FName ColumnID_ReferenceLabel( "Reference" );

struct FReferenceTreeDataContainer
{
	UObject* Object;
	TArray< FReferenceTreeItemPtr > ChildrenReferences;

	FReferenceTreeDataContainer(UObject* InObject) : Object(InObject)
	{}

	FText GetToolTipText() const
	{
		if (IAssetTools::Get().ShowingContentVersePath())
		{
			UE::Core::FVersePath VersePath = Object->GetVersePath();
			if (VersePath.IsValid())
			{
				return FText::FromString(MoveTemp(VersePath).ToString());
			}
		}

		return FText::FromString(Object->GetPathName());
	}
};

/** Widget that represents a row in the sound wave compression's tree control.  Generates widgets for each column on demand. */
class SReferenceTreeRow
	: public STableRow< FReferenceTreeItemPtr >
{

public:

	SLATE_BEGIN_ARGS( SReferenceTreeRow ){}
		SLATE_ARGUMENT(FReferenceTreeItemPtr, Item)
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		Item = InArgs._Item;

		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), FText::FromString( Item->Object->GetClass()->GetName() ) );
		Args.Add( TEXT("ObjectName"), FText::FromString( Item->Object->GetName() ) );

		this->ChildSlot
		[	
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
				.AutoWidth()
			[
				SNew( SExpanderArrow, SharedThis(this) )
			]

			+SHorizontalBox::Slot()
				.AutoWidth()
			[
				SNew(STextBlock)
				.Text( FText::Format( NSLOCTEXT("ReferenceTree", "ReferenceTree_Object/ClassTitle", "{ClassName}({ObjectName})"), Args ) )
			]
		];

		STableRow< FReferenceTreeItemPtr >::ConstructInternal(
			STableRow::FArguments()
				.ShowSelection(true),
			InOwnerTableView
		);
	}

private:
	/** Called when a tree item is double clicked. */
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override
	{
		// Show the object in the editor.  I.E show the object in the level if its an actor, or content browser otherwise.
		if( Item->Object )
		{
			ReferenceTreeView::Helpers::SelectObjectInEditor( Item->Object );
		}

		return FReply::Handled();
	}

private:
	/** The data this row represents. */
	FReferenceTreeItemPtr Item;
};

TWeakPtr< SWindow > SReferenceTree::SingletonInstance;

void SReferenceTree::OpenDialog(UObject* InObject)
{
	if(SingletonInstance.IsValid())
	{
		SingletonInstance.Pin()->RequestDestroyWindow();
	}

	TSharedPtr< SWindow > Window;
	TSharedPtr< SReferenceTree > ReferenceTreeWidget;

	Window = SNew(SWindow)
		.Title(NSLOCTEXT("ReferenceTree", "ReferenceTree_Title", "Reference Tree"))
		.ClientSize(FVector2D(300.0f, 400.0f))
		.SupportsMaximize(false) .SupportsMinimize(false)
		[
			SNew( SBorder )
			.Padding( 4.f )
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SAssignNew(ReferenceTreeWidget, SReferenceTree)
					.Object(InObject)
			]
		];

	ReferenceTreeWidget->SetWindow(Window);

	SingletonInstance = Window;

	FSlateApplication::Get().AddWindow( Window.ToSharedRef() );
}

void SReferenceTree::Construct( const FArguments& InArgs )
{
	bShowScriptRefs = false;

	TSharedRef< SHeaderRow > HeaderRowWidget =
		SNew( SHeaderRow )

		.Visibility( EVisibility::Collapsed )

		// Quality label column
		+SHeaderRow::Column( ColumnID_ReferenceLabel )
			.DefaultLabel( NSLOCTEXT("SoundWaveOptions", "ReferenceColumnLabel", "Reference") )
			.FillWidth( 1.0f );

	// Build the top menu.
	TSharedRef< FUICommandList > CommandList(new FUICommandList());
	FMenuBarBuilder MenuBarBuilder( CommandList );
	{
		MenuBarBuilder.AddPullDownMenu( NSLOCTEXT("ReferenceTreeView", "View", "View"), NSLOCTEXT("ReferenceTreeView", "View_Tooltip", "View settings for the reference tree."), FNewMenuDelegate::CreateRaw( this, &SReferenceTree::FillViewEntries ) );
		MenuBarBuilder.AddPullDownMenu( NSLOCTEXT("ReferenceTreeView", "Options", "Options"), NSLOCTEXT("ReferenceTreeView", "Options_Tooltip", "Options for the reference tree."), FNewMenuDelegate::CreateRaw( this, &SReferenceTree::FillOptionsEntries ) );
	}

	this->ChildSlot
		[
			SNew(SVerticalBox)
				
			+SVerticalBox::Slot()
				.AutoHeight()
			[
				MenuBarBuilder.MakeWidget()			
			]

			+SVerticalBox::Slot()
				.FillHeight(1.0f)
			[
				SAssignNew(ReferenceTreeView, SReferenceTreeView)
				// Point the tree to our array of root-level items.  Whenever this changes, we'll call RequestTreeRefresh()
				.TreeItemsSource( &ReferenceTreeRoot )

				// Called to child items for any given parent item
				.OnGetChildren( this, &SReferenceTree::OnGetChildrenForReferenceTree )

				// Generates the actual widget for a list item
				.OnGenerateRow( this, &SReferenceTree::OnGenerateRowForReferenceTree ) 

				// Generates the right click menu.
				.OnContextMenuOpening( this, &SReferenceTree::BuildMenuWidget )

				// Header for the tree
				.HeaderRow(HeaderRowWidget)
			]

			+SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SButton)
						.Text( NSLOCTEXT("UnrealEd", "OK", "OK") )
						.OnClicked(this, &SReferenceTree::OnOK_Clicked)
				]
		];

	PopulateTree(InArgs._Object);

	FEditorDelegates::MapChange.AddRaw(this, &SReferenceTree::OnEditorMapChange);
}

SReferenceTree::~SReferenceTree()
{
	FEditorDelegates::MapChange.RemoveAll(this);

	DestroyGraphAndTree();
}

void SReferenceTree::PopulateTree( UObject* InRootObject )
{
	if( InRootObject == NULL )
	{
		return;
	}

	// Always regenerate.
	DestroyGraphAndTree();

	FArchiveGenerateReferenceGraph GenerateReferenceGraph( ReferenceGraph );


	// Empty all items currently in the tree.
	ReferenceTreeRoot.Empty();

	// Add a root node tree item
	ReferenceTreeRoot.Add(MakeShareable(new FReferenceTreeDataContainer( InRootObject )));

	const FReferenceGraphNode* RootGraphNode = ReferenceGraph.FindRef( InRootObject );

	bool bIsReferenced = false;
	if( RootGraphNode )
	{
		// For each node that references the root node, recurse over its links to generate the tree.
		for( TSet<FReferenceGraphNode*>::TConstIterator It(RootGraphNode->Links); It; ++It )
		{
			FReferenceGraphNode* Link = *It;
			UObject* Reference = Link->Object;
			if( ( bShowScriptRefs || !Reference->HasAnyFlags( RF_ClassDefaultObject ) ) && ( Reference->IsA( UActorComponent::StaticClass() ) || ObjectTools::IsObjectBrowsable( Reference ) ) && !Reference->HasAnyFlags( RF_Transient ) )
			{
				bIsReferenced = true;
				// Skip default objects unless we are showing script references and transient objects.
				// Populate links to browsable objects and actor components (we will actually display the actor or script reference for components)
				PopulateTreeRecursive( *Link, ReferenceTreeRoot[0] );
			}
		}
	}

	// Expand all tree nodes and ensure the root item is visible
	SetAllExpansionStates(true);
	ReferenceTreeView->RequestTreeRefresh();
}

bool SReferenceTree::PopulateTreeRecursive( FReferenceGraphNode& InNode, FReferenceTreeItemPtr InParentNode )
{
	// Prevent circular references.  This node has now been visited for this path.
	InNode.bVisited = true;

	bool bNodesWereAdded = false;

	UObject* ObjectToDisplay = InNode.GetObjectToDisplay( bShowScriptRefs );

	if( ObjectToDisplay )
	{
		// Make a tree node for this object. If the object is a component, display the components outer instead.
		InParentNode->ChildrenReferences.Add(MakeShareable(new FReferenceTreeDataContainer( ObjectToDisplay )));

		// We just added a node. Inform the parent.
		bNodesWereAdded = true;

		uint32 NumChildrenAdded = 0;
		// @todo: Move to INI or menu option?
		const uint32 MaxChildrenPerNodeToDisplay = 50;

		// Iterate over all this nodes links and add them to the tree
		for( TSet<FReferenceGraphNode*>::TConstIterator It(InNode.Links); It; ++It )
		{
			if( NumChildrenAdded == MaxChildrenPerNodeToDisplay )
			{
				// The tree is getting too large to be usable
				// We will display a node saying how many other nodes there are that cant be displayed
				// @todo: provide the ability to expand this node and populate the tree with the skipped nodes.

				// stop populating
				break;
			}

			FReferenceGraphNode* Link = *It;
			UObject* Object = Link->Object;

			// Only recurse into unvisited nodes which are components or are visible in the content browser.  
			// Components are acceptable so their actor references can be added to the tree.
			bool bObjectIsValid = !Object->HasAnyFlags( RF_Transient ) &&
				( Object->IsA( UActorComponent::StaticClass() ) || // Allow actor components to pass so that their actors can be displayed
				Object->IsA( UPolys::StaticClass() ) || // Allow polys to pass so BSP can be displayed
				ObjectTools::IsObjectBrowsable( Object ) ); // Allow all browsable objects through

			if( Link->bVisited == false && bObjectIsValid )
			{
				if( PopulateTreeRecursive( *Link, InParentNode->ChildrenReferences[InParentNode->ChildrenReferences.Num() - 1] ) )
				{
					++NumChildrenAdded;
				}
			}
		}
	}

	// We can safely visit this node again, all of its links have been visited.
	// Any other way this node is visited represents a new path.
	InNode.bVisited = false;

	return bNodesWereAdded;
}

void SReferenceTree::OnGetChildrenForReferenceTree( FReferenceTreeItemPtr InParent, TArray< FReferenceTreeItemPtr >& OutChildren )
{
	// Simply return the children, it's already setup.
	OutChildren = InParent->ChildrenReferences;
}

TSharedPtr< SWidget > SReferenceTree::BuildMenuWidget()
{
	// Empty list of commands.
	TSharedPtr< FUICommandList > Commands;

	const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, Commands);
	{
		if(ReferenceTreeView->GetSelectedItems().Num())
		{
			UObject* SelectedObject = ReferenceTreeView->GetSelectedItems()[0]->Object;
			AActor* Actor = Cast<AActor>( SelectedObject );
			if( Actor ) 
			{	
				FUIAction SelectActorAction( FExecuteAction::CreateStatic( ReferenceTreeView::Helpers::SelectObjectInEditor, SelectedObject ) );
				MenuBuilder.AddMenuEntry(NSLOCTEXT("ReferenceTreeView", "SelectActor", "Select Actor"), NSLOCTEXT("ReferenceTreeView", "SelectActor_Tooltip", "Select the actor in the viewport."), FSlateIcon(), SelectActorAction);

				FUIAction ViewPropertiesAction( FExecuteAction::CreateRaw( this, &SReferenceTree::OnMenuViewProperties, Actor ) );
				MenuBuilder.AddMenuEntry(NSLOCTEXT("ReferenceTreeView", "ViewProperties", "View Properties"), NSLOCTEXT("ReferenceTreeView", "ViewProperties_Tooltip", "View the actor's properties."), FSlateIcon(), ViewPropertiesAction);	
			}
			else
			{
				FUIAction OpenEditorAction( FExecuteAction::CreateRaw( this, &SReferenceTree::OnMenuShowEditor, SelectedObject ) );
				MenuBuilder.AddMenuEntry(NSLOCTEXT("ReferenceTreeView", "OpenEditor", "Open Editor"), NSLOCTEXT("ReferenceTreeView", "OpenEditor_ToolTip", "Opens the asset's editor."), FSlateIcon(), OpenEditorAction);

				FUIAction ShowInContentBrowserAction( FExecuteAction::CreateStatic( ReferenceTreeView::Helpers::SelectObjectInEditor, SelectedObject ) );
				MenuBuilder.AddMenuEntry(NSLOCTEXT("ReferenceTreeView", "ShowInContentBrowser", "Show in Content Browser"), NSLOCTEXT("ReferenceTreeView", "ShowInContentBrowser_Tooltip", "Shows the asset in the Content Browser."), FSlateIcon(), ShowInContentBrowserAction);
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void SReferenceTree::FillViewEntries( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.AddMenuEntry( NSLOCTEXT("ReferenceTreeView", "RebuildTree", "Rebuild Tree"), NSLOCTEXT("ReferenceTreeView", "RebuildTree_Tooltip", "Rebuilds the tree."), FSlateIcon(), 
		FUIAction(FExecuteAction::CreateRaw(this, &SReferenceTree::PopulateTree, ReferenceTreeRoot.Num()? ReferenceTreeRoot[0]->Object : NULL) ) );
	MenuBuilder.AddMenuEntry( NSLOCTEXT("ReferenceTreeView", "CollapseAll", "Collapse All"), NSLOCTEXT("ReferenceTreeView", "CollapseAll_Tooltip", "Collapses all items in the tree."), FSlateIcon(), 
		FUIAction(FExecuteAction::CreateRaw(this, &SReferenceTree::SetAllExpansionStates, false) ) );
	MenuBuilder.AddMenuEntry( NSLOCTEXT("ReferenceTreeView", "ExpandAll", "Expand All"), NSLOCTEXT("ReferenceTreeView", "ExpandAll_Tooltip", "Expands all items in the tree."), FSlateIcon(), 
		FUIAction(FExecuteAction::CreateRaw(this, &SReferenceTree::SetAllExpansionStates, true) ) );
}

void SReferenceTree::FillOptionsEntries( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.AddMenuEntry( NSLOCTEXT("ReferenceTreeView", "ShowScriptObjects", "Show Script Objects"), NSLOCTEXT("ReferenceTreeView", "ShowScriptObjects_Tooltip", "Toggles displaying script objects in the tree."), FSlateIcon(), 
		FUIAction(FExecuteAction::CreateRaw(this, &SReferenceTree::OnShowScriptReferences), FCanExecuteAction(), FIsActionChecked::CreateRaw(this, &SReferenceTree::OnShowScriptReferences_Checked)), NAME_None, EUserInterfaceActionType::Check );
}

void SReferenceTree::SetAllExpansionStates(bool bInExpansionState)
{
	// Go through all the items in the root of the tree and recursively visit their children to set every item in the tree.
	for(int32 ChildIndex = 0; ChildIndex < ReferenceTreeRoot.Num(); ChildIndex++)
	{
		SetAllExpansionStates_Helper( ReferenceTreeRoot[ChildIndex], bInExpansionState );
	}
}

void SReferenceTree::SetAllExpansionStates_Helper(FReferenceTreeItemPtr InNode, bool bInExpansionState)
{
	ReferenceTreeView->SetItemExpansion(InNode, bInExpansionState);

	// Recursively go through the children.
	for(int32 ChildIndex = 0; ChildIndex < InNode->ChildrenReferences.Num(); ChildIndex++)
	{
		SetAllExpansionStates_Helper( InNode->ChildrenReferences[ChildIndex], bInExpansionState );
	}
}

void SReferenceTree::OnEditorMapChange(uint32 InMapChangeFlags)
{
	if( MapChangeEventFlags::WorldTornDown)
	{
		// If a map is changing and the world was torn down, destroy the graph
		DestroyGraphAndTree();
	}
}

void SReferenceTree::DestroyGraphAndTree()
{
	// Remove all items from the tree
	ReferenceTreeRoot.Empty();

	// Delete every node in the graph.
	for( FReferenceGraph::TIterator It(ReferenceGraph); It; ++It )
	{
		delete It.Value();
		It.Value() = NULL;
	}

	// Empty graph
	ReferenceGraph.Empty();
}

void SReferenceTree::OnMenuViewProperties( AActor* InActor )
{
	// Show the property windows and create one if necessary
	GUnrealEd->ShowActorProperties();

	// Show the property window for the actor
	TArray<AActor*> Actors;
	Actors.Add(InActor);
	GUnrealEd->UpdateFloatingPropertyWindowsFromActorList(Actors);
}

void SReferenceTree::OnMenuShowEditor( UObject* InObject )
{
	// Show the editor for this object
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InObject);
}

void SReferenceTree::OnShowScriptReferences()
{
	bShowScriptRefs = !bShowScriptRefs;

	if(ReferenceTreeRoot.Num())
	{
		PopulateTree(ReferenceTreeRoot[0]->Object);
	}
}

FReply SReferenceTree::OnOK_Clicked(void)
{
	MyWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

TSharedRef< ITableRow > SReferenceTree::OnGenerateRowForReferenceTree( FReferenceTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable )
{
	return
		SNew( SReferenceTreeRow, OwnerTable )
		.Item( Item )
		.ToolTipText( Item.ToSharedRef(), &FReferenceTreeDataContainer::GetToolTipText );
}
