// Copyright Epic Games, Inc. All Rights Reserved.


#include "Dialogs/SBuildProgress.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "UnrealEdMisc.h"


SBuildProgressWidget::SBuildProgressWidget() 
{			
}

SBuildProgressWidget::~SBuildProgressWidget()
{
}

void SBuildProgressWidget::Construct( const FArguments& InArgs )
{
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
		.Padding(0.0f)
		[
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoHeight()
			.Padding(10.0f, 10.0f, 10.0f, 2.0f )
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("BuildProgress", "BuildStatusLabel", "Build Status"))
				.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.Padding(10.0f, 2.0f)
			[
				SNew( SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( STextBlock )
					.Text( this, &SBuildProgressWidget::OnGetBuildTimeText )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(10.0f, 0, 10.0f, 7.0f)
				[
					SNew( STextBlock )
					.Text( this, &SBuildProgressWidget::OnGetProgressText )
				]
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding(10.0f, 1.0f)
			[
				SNew( STextBlock )
				.Text( NSLOCTEXT("BuildProgress", "BuildProgressLabel", "Build Progress") )
			]
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoHeight()
			.Padding( 10.0f, 7.0f, 10.0f, 7.0f )
			[
				SNew(SProgressBar)
				.Percent( this, &SBuildProgressWidget::OnGetProgressFraction )	
			]
			+SVerticalBox::Slot()
			.Padding(15.0f, 10.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SNew(SButton)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text( NSLOCTEXT("BuildProgress", "StopBuildButtonLabel", "Stop Build") )
				.OnClicked( this, &SBuildProgressWidget::OnStopBuild )
			]
		]
	];

	// Reset progress indicators
	BuildStartTime = -1;
	bStoppingBuild = false;
	SetBuildStatusText( FText::GetEmpty() );
	SetBuildProgressPercent( 0, 100 );
}

FText SBuildProgressWidget::OnGetProgressText() const
{
	return ProgressStatusText;
}

void SBuildProgressWidget::UpdateProgressText()
{
	if( ProgressNumerator > 0 && ProgressDenominator > 0 )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("StatusText"), BuildStatusText );
		Args.Add( TEXT("ProgressCompletePercentage"), FText::AsPercent( (float)ProgressNumerator/ProgressDenominator) );
		ProgressStatusText = FText::Format( NSLOCTEXT("BuildProgress", "ProgressStatusFormat", "{StatusText} ({ProgressCompletePercentage})"), Args );
	}
	else
	{
		ProgressStatusText = BuildStatusText;
	}
}

FText SBuildProgressWidget::OnGetBuildTimeText() const
{
	// Only show a percentage if there is something interesting to report
	return BuildStatusTime; 
}

TOptional<float> SBuildProgressWidget::OnGetProgressFraction() const
{
	// Only show a percentage if there is something interesting to report
	if( ProgressNumerator > 0 && ProgressDenominator > 0 )
	{
		return (float)ProgressNumerator/ProgressDenominator;
	}
	else
	{
		// Return non-value to indicate marquee mode
		// for the progress bar.
		return TOptional<float>();
	}
}

void SBuildProgressWidget::SetBuildType(EBuildType InBuildType)
{
	BuildType = InBuildType;
}

FText SBuildProgressWidget::BuildElapsedTimeText() const
{
	// Display elapsed build time.
	return FText::AsTimespan( FDateTime::Now() - BuildStartTime );
}

void SBuildProgressWidget::UpdateTime()
{
	BuildStatusTime = BuildElapsedTimeText();
}

void SBuildProgressWidget::SetBuildStatusText( const FText& StatusText )
{
	UpdateTime();
	
	// Only update the text if we haven't canceled the build.
	if( !bStoppingBuild )
	{
		BuildStatusText = StatusText;
		UpdateProgressText();
	}
}

void SBuildProgressWidget::SetBuildProgressPercent( int32 InProgressNumerator, int32 InProgressDenominator )
{
	UpdateTime();

	// Only update the progress bar if we haven't canceled the build.
	if( !bStoppingBuild )
	{
		ProgressNumerator = InProgressNumerator;
		ProgressDenominator = InProgressDenominator;
		UpdateProgressText();
	}
}

void SBuildProgressWidget::MarkBuildStartTime()
{
	BuildStartTime = FDateTime::Now();
}

FReply SBuildProgressWidget::OnStopBuild()
{
	FUnrealEdMisc::Get().SetMapBuildCancelled( true );
	
	SetBuildStatusText( NSLOCTEXT("UnrealEd", "StoppingMapBuild", "Stopping Map Build...") );

	bStoppingBuild = true;

	return FReply::Handled();
}
