// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/SDeleteAssetsDialog.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "Misc/MessageDialog.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Editor/Transactor.h"

#include "Framework/Commands/GenericCommands.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SDeleteAssetsDialog"

namespace DeleteAssetsView
{
	/** IDs for list columns */
	static const FName ColumnID_Asset( "Asset" );
	static const FName ColumnID_AssetClass( "Class" );
	static const FName ColumnID_DiskReferences( "DiskReferences" );
	static const FName ColumnID_MemoryReferences( "MemoryReferences" );
	static const FName ColumnID_Folder( "Folder" );
	static const FName ColumnID_Path( "Path" );
}

static FLinearColor DangerColor( 0.715465432f, 0.034230207f, 0 );
static FLinearColor WarningColor( 1, 1, 0 );


//////////////////////////////////////////////////////////////////////////
// SPendingDeleteRow

class SPendingDeleteRow : public SMultiColumnTableRow< TSharedPtr<FPendingDelete> >
{
public:

	SLATE_BEGIN_ARGS( SPendingDeleteRow ) { }
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FPendingDelete> InItem )
	{
		Item = InItem;

		SMultiColumnTableRow< TSharedPtr<FPendingDelete> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName )
	{
		if ( ColumnName == DeleteAssetsView::ColumnID_Asset )
		{
			return SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 3, 0, 0, 0 )
				[
					SNew( STextBlock )
					.Text(FText::FromString(Item->GetObject()->GetName()))
				];
		}
		else if ( ColumnName == DeleteAssetsView::ColumnID_AssetClass )
		{
			return SNew( STextBlock )
				.Text(FText::FromString(Item->GetObject()->GetClass()->GetName()));
		}
		else if ( ColumnName == DeleteAssetsView::ColumnID_DiskReferences )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT( "AssetCount" ), FText::AsNumber( Item->RemainingDiskReferences ) );

			FText OnDiskCountText = Item->RemainingDiskReferences > 1 ? FText::Format( LOCTEXT( "OnDiskAssetReferences", "{AssetCount} References" ), Args ) : FText::Format( LOCTEXT( "OnDiskAssetReference", "{AssetCount} Reference" ), Args );

			return SNew( STextBlock )
				.Text( OnDiskCountText )
				.Visibility( Item->RemainingDiskReferences > 0 ? EVisibility::Visible : EVisibility::Hidden );
		}
		else if ( ColumnName == DeleteAssetsView::ColumnID_MemoryReferences )
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT( "ReferenceCount" ), FText::AsNumber( Item->RemainingMemoryReferences ) );

			FText InMemoryCountText = Item->RemainingMemoryReferences > 1 ? FText::Format( LOCTEXT( "InMemoryReferences", "{ReferenceCount} References" ), Args ) : FText::Format( LOCTEXT( "OnDiskReference", "{ReferenceCount} Reference" ), Args );

			return SNew( STextBlock )
				.Text( InMemoryCountText )
				.Visibility( Item->RemainingMemoryReferences > 0 ? EVisibility::Visible : EVisibility::Hidden );
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FPendingDelete> Item;
};

class SPendingFolderDeleteRow : public SMultiColumnTableRow<TSharedPtr<FString>>
{
public:
	SLATE_BEGIN_ARGS(SPendingFolderDeleteRow)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FString> InItem)
	{
		Item = InItem;
		SMultiColumnTableRow<TSharedPtr<FString>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName)
	{
		if (!Item)
		{
			return SNullWidget::NullWidget;
		}

		if (InColumnName == DeleteAssetsView::ColumnID_Folder)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FPaths::GetBaseFilename(*Item)))
				];
		}
		else if (InColumnName == DeleteAssetsView::ColumnID_Path)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(*Item));
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FString> Item;
};

//////////////////////////////////////////////////////////////////////////
// SDeleteAssetsDialog

SDeleteAssetsDialog::~SDeleteAssetsDialog()
{
	DeleteModel->OnStateChanged().RemoveAll( this );
}

void SDeleteAssetsDialog::Construct( const FArguments& InArgs, TSharedRef<FAssetDeleteModel> InDeleteModel )
{
	//bIsActiveTimerRegistered = false;
	bIsActiveTimerRegistered = true;
	RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SDeleteAssetsDialog::TickDeleteModel ) );

	// Model handles objects/assets to delete
	DeleteModel = InDeleteModel;

	// Save additional folders to delete
	FoldersToDelete.Reset(InArgs._FoldersToDelete.Num());
	for (const FString& Folder : InArgs._FoldersToDelete)
	{
		FoldersToDelete.Add(MakeShared<FString>(Folder));
	}

	// Save off the attributes
	ParentWindow = InArgs._ParentWindow;

	OnDialogClosed = InArgs._OnDialogClosed;

	ReferencerCommands = TSharedPtr< FUICommandList >(new FUICommandList);

	ReferencerCommands->MapAction(FGenericCommands::Get().Delete, FUIAction(
		FExecuteAction::CreateSP( this, &SDeleteAssetsDialog::ExecuteDeleteReferencers ),
		FCanExecuteAction::CreateSP( this, &SDeleteAssetsDialog::CanExecuteDeleteReferencers )
		) );

	// Create the widgets
	ChildSlot
	[
		SAssignNew(RootContainer, SBorder)
		.BorderImage( FAppStyle::GetBrush( "AssetDeleteDialog.Background" ) )
		.Padding(10.0f)
	];

	DeleteModel->OnStateChanged().AddRaw(this, &SDeleteAssetsDialog::HandleDeleteModelStateChanged);

	// Manually fire the state changed event so that we are setup for the initial state.
	HandleDeleteModelStateChanged(DeleteModel->GetState());
}

TSharedRef<SWidget> SDeleteAssetsDialog::BuildProgressDialog()
{
	return SNew( SVerticalBox )

	// Show Progress Text
	+ SVerticalBox::Slot()
	.VAlign(VAlign_Center)
	.FillHeight( 1.0f )
	[
		SNew( SVerticalBox )

		+ SVerticalBox::Slot()
		.Padding( 5.0f, 0 )
		[
			SNew( STextBlock )
			.Text( this, &SDeleteAssetsDialog::ScanningText )
		]

		// Show Progress
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 5.0f, 10.0f )
		[
			SNew( SProgressBar )
			.Percent( this, &SDeleteAssetsDialog::ScanningProgressFraction )
		]
	];
}

TSharedRef<SWidget> SDeleteAssetsDialog::BuildDeleteDialog()
{
	const auto* LoadingSavingSettings = GetDefault<UEditorLoadingSavingSettings>();

	FFormatNamedArguments Args;
	Args.Add( TEXT( "OnDiskReferences" ), FText::AsNumber( DeleteModel->GetAssetReferences().Num() ) );

	const bool bShowFolderList = !FoldersToDelete.IsEmpty();
	const bool bShowAssetList = !DeleteModel->GetPendingDeletedAssets()->IsEmpty();

	return SNew( SVerticalBox )

	// The folders to be deleted
	+ SVerticalBox::Slot()
	.FillHeight( bShowFolderList ? 0.4f : 0.f )
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.Visibility(bShowFolderList ? EVisibility::Visible : EVisibility::Collapsed)
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Padding( FMargin(0, 0, 0, 3) )
		[
			SNew( SVerticalBox )

			// Attempting delete text
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "DetailsView.CategoryTop" ) )
				.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
				.Padding(3.0f)
				[
					SNew( STextBlock )
					.Text( LOCTEXT( "AttemptingFolderDelete", "Pending Deleted Folders" ) )
					.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
					.ShadowOffset( FVector2D( 1.0f, 1.0f ) )
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew( FoldersToDeleteList, SListView< TSharedPtr<FString> > )
				.ListItemsSource( &FoldersToDelete )
				.OnGenerateRow( this, &SDeleteAssetsDialog::HandleGenerateFolderRow )
				.HeaderRow(
					SNew( SHeaderRow )
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_Folder )
                    .DefaultLabel( LOCTEXT( "Column_FolderName", "Folder" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth(0.25f)
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_Path )
                    .DefaultLabel( LOCTEXT( "Column_FolderPath", "Path" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth( 0.75f )
				)
			]
		]
	]

	// The assets to be deleted
	+ SVerticalBox::Slot()
	.FillHeight( bShowAssetList ? 0.6f : 0.f )
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Padding( FMargin(0, 0, 0, 3) )
		[
			SNew( SVerticalBox )
			.Visibility(bShowAssetList ? EVisibility::Visible : EVisibility::Collapsed)
			// Attempting delete text
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "DetailsView.CategoryTop" ) )
				.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
				.Padding(3.0f)
				[
					SNew( STextBlock )
					.Text( LOCTEXT( "AttemptingDelete", "Pending Deleted Assets" ) )
					.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
					.ShadowOffset( FVector2D( 1.0f, 1.0f ) )
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew( ObjectsToDeleteList, SListView< TSharedPtr<FPendingDelete> > )
				.ListItemsSource( DeleteModel->GetPendingDeletedAssets() )
				.OnGenerateRow( this, &SDeleteAssetsDialog::HandleGenerateAssetRow )
				.HeaderRow(
					SNew( SHeaderRow )
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_Asset )
                    .DefaultLabel( LOCTEXT( "Column_AssetName", "Asset" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth(0.5f)
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_AssetClass )
                    .DefaultLabel( LOCTEXT( "Column_AssetClass", "Class" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth( 0.25f )
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_DiskReferences )
                    .DefaultLabel( LOCTEXT( "Column_DiskReferences", "Asset Referencers" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth( 0.25f )
                    + SHeaderRow::Column( DeleteAssetsView::ColumnID_MemoryReferences )
                    .DefaultLabel( LOCTEXT( "Column_MemoryReferences", "Memory References" ) )
                    .HAlignHeader( EHorizontalAlignment::HAlign_Left )
                    .FillWidth( 0.25f )
				)
			]
		]
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.BorderBackgroundColor( FLinearColor::Red )
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Visibility( this, &SDeleteAssetsDialog::GetReferencesVisiblity )
		.Padding(5.0f)
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "References", "Some of the assets being deleted are still referenced in memory." ) )
		]
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.BorderBackgroundColor( FLinearColor::Yellow )
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Visibility( this, &SDeleteAssetsDialog::GetUndoVisiblity )
		.Padding( 5.0f )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "DeleteUndo", "There are references in the undo history, so the undo history will be cleared." ) )
		]
	]

	+ SVerticalBox::Slot()
	.FillHeight( 0.5f )
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Padding( FMargin( 0, 0, 0, 3 ) )
		.Visibility( this, &SDeleteAssetsDialog::GetAssetReferencesVisiblity)
		[
			SNew( SVerticalBox )

			// Pending Deletes
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "DetailsView.CategoryTop" ) )
				.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
				.Padding( 3.0f )
				[
					SNew( STextBlock )
					.Text( LOCTEXT( "AssetsReferencingPendingDeletedAssets", "Assets Referencing the Pending Deleted Assets" ) )
					.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
					.ShadowOffset( FVector2D( 1.0f, 1.0f ) )
				]
			]

			// The Assets Still Using The To-Be-Deleted Assets
			+ SVerticalBox::Slot()
			.FillHeight( 1.0f )
			[
				SDeleteAssetsDialog::MakeAssetViewForReferencerAssets()
			]
		]
	]
	
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding( 5.0f )
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
		.Padding( 0.0f )
		[
			SNew( SVerticalBox )

			// How do you want to handle this?
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "DetailsView.CategoryTop" ) )
				.BorderBackgroundColor( FLinearColor( .6f, .6f, .6f, 1.0f ) )
				.Padding( 3.0f )
				[
					SNew( SHorizontalBox )

					+ SHorizontalBox::Slot()
					.FillWidth( 1.0f )
					.HAlign( HAlign_Center )
					[
						SNew( STextBlock )
						.Text( this, &SDeleteAssetsDialog::GetHandleText )
						.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
						.ShadowOffset( FVector2D( 1.0f, 1.0f ) )
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(6,4)
			[
				SAssignNew(DeleteSourceFilesCheckbox, SCheckBox)
				.Visibility(this, &SDeleteAssetsDialog::GetDeleteSourceFilesVisibility)
				.IsChecked(LoadingSavingSettings->bDeleteSourceFilesWithAssets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DeleteSourceFiles", "Also delete related source content files"))
					.ToolTip(
						SNew(SToolTip)
						.Text(this, &SDeleteAssetsDialog::GetDeleteSourceContentTooltip)
					)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0,4)
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.Padding( 6, 0 )
				[
					SNew( SBorder )
					.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
					.Visibility( this, &SDeleteAssetsDialog::GetReplaceReferencesVisibility )
					[
						( DeleteModel->CanReplaceReferences() ? BuildReplaceReferencesWidget() : BuildCantUseReplaceReferencesWidget() )
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.Padding( 6, 0 )
				[
					SAssignNew( ForceDeleteButton, SBorder )
					.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
					.Visibility( this, &SDeleteAssetsDialog::GetForceDeleteVisibility )
					[
						BuildForceDeleteWidget()
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.Padding( 6, 0 )
				[
					SNew( SBorder )
					.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
					.Visibility( this, &SDeleteAssetsDialog::GetDeleteVisibility )
					[
						SAssignNew(DeleteButton, SButton )
						.HAlign( HAlign_Center )
						.Text( LOCTEXT( "Delete", "Delete" ) )
						.ToolTipText( LOCTEXT( "DeleteTooltipText", "Perform the delete" ) )
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
						.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
						.OnClicked(this, &SDeleteAssetsDialog::Delete)
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.Padding( 6, 0 )
				[
					SNew( SBorder )
					.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
					.VAlign( EVerticalAlignment::VAlign_Bottom )
					[
						SNew( SButton )
						.HAlign( HAlign_Center )
						.Text( LOCTEXT( "Cancel", "Cancel" ) )
						.ToolTipText( LOCTEXT( "CancelDeleteTooltipText", "Cancel the delete" ) )
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
						.OnClicked( this, &SDeleteAssetsDialog::Cancel )
					]
				]
			]
		]
	];
}

FText SDeleteAssetsDialog::GetHandleText() const
{
	if ( CanDelete() )
	{
		return LOCTEXT( "AreYouSure", "Are you sure you want to delete these items?" );
	}
	else
	{
		return LOCTEXT( "HandleIt", "How do you want to handle this?" );
	}
}

FText SDeleteAssetsDialog::GetDeleteSourceContentTooltip() const
{
	FString AllFiles;

	static const int32 MaxNumPathsToShow = 25;
	const auto& AllFileCounts = DeleteModel->GetPendingDeletedSourceFileCounts();
	int32 TotalCount = 0, NumPrinted = 0;
	for (const auto& PathAndAssetCount : AllFileCounts)
	{
		// If this path is no longer referenced by deleted files, it's toast.
		if (PathAndAssetCount.Value == 0)
		{
			++TotalCount;

			if (TotalCount <= MaxNumPathsToShow)
			{
				if (NumPrinted != 0)
				{
					AllFiles += TEXT("\n");
				}

				AllFiles += PathAndAssetCount.Key;
				++NumPrinted;
			}
		}
	}

	FText RootText;

	FFormatOrderedArguments Args;
	Args.Add(FText::FromString(AllFiles));

	if (NumPrinted < TotalCount)
	{
		Args.Add(FText::AsNumber(TotalCount - NumPrinted));
		RootText = LOCTEXT("DeleteSourceFilesAndMore_Tooltip", "When checked, the following source content files will also be deleted along with the assets:\n\n{0}\n... and {1} more.");
	}
	else
	{
		RootText = LOCTEXT("DeleteSourceFiles_Tooltip", "When checked, the following source content files will also be deleted along with the assets:\n\n{0}");
	}
	
	return FText::Format(RootText, Args);
}

EVisibility SDeleteAssetsDialog::GetAssetReferencesVisiblity() const
{
	return DeleteModel->GetAssetReferences().Num() == 0 ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget> SDeleteAssetsDialog::BuildCantUseReplaceReferencesWidget()
{
	return SNew( STextBlock )
		.AutoWrapText( true )
		.Text( LOCTEXT( "ReplaceReferencesNotAvailabeText", "Not all objects are compatible, so Replace References is unavailable." ) );
}

TSharedRef<SWidget> SDeleteAssetsDialog::BuildReplaceReferencesWidget()
{
	return SNew( SVerticalBox )

	+ SVerticalBox::Slot()
	.FillHeight( 1 )
	.Padding( 0, 0, 0, 3 )
	[
		SNew( STextBlock )
		.AutoWrapText( true )
		//.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
		.Text( LOCTEXT( "ReplaceReferencesText", "Delete the assets and update referencers to point at an asset of your choosing." ) )
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SAssignNew( ConsolidationPickerComboButton, SComboButton )
		.HAlign( EHorizontalAlignment::HAlign_Fill )
		.VAlign( EVerticalAlignment::VAlign_Center )
		.ComboButtonStyle( FAppStyle::Get(), "ToolbarComboButton" )
		.ForegroundColor( FLinearColor::White )
		.ContentPadding(3.0f)
		.MenuPlacement( EMenuPlacement::MenuPlacement_BelowAnchor )
		.OnGetMenuContent( this, &SDeleteAssetsDialog::MakeConsolidationAssetPicker )
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				CreateThumbnailWidget()
			]

			+ SHorizontalBox::Slot()
			.FillWidth( 1.0f )
			.VAlign( VAlign_Center )
			.Padding(5, 0)
			[
				SNew( STextBlock )
				.Text( this, &SDeleteAssetsDialog::GetConsolidateAssetName )
			]
		]
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew( SButton )
		.HAlign( HAlign_Center )
		.Text( LOCTEXT( "Replace References", "Replace References" ) )
		.OnClicked( this, &SDeleteAssetsDialog::ReplaceReferences )
		.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
		.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
	];
}

TSharedRef<SWidget> SDeleteAssetsDialog::BuildForceDeleteWidget()
{
	return SNew( SVerticalBox )

	+ SVerticalBox::Slot()
	.FillHeight( 1.0f )
	.Padding( 0, 0, 0, 3.0f )
	[
		SNew( STextBlock )
		.AutoWrapText( true )
		//.Font( FAppStyle::GetFontStyle( "BoldFont" ) )
		.Text( LOCTEXT( "ForceDeleteText", "Delete the asset anyway, but referencers may not work correctly anymore.\n\nUse as a last resort." ) )
	]

	+ SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew( SButton )
		.HAlign( HAlign_Center )
		.Text( LOCTEXT( "ForceDelete", "Force Delete" ) )
		.ToolTipText( LOCTEXT( "ForceDeleteTooltipText", "Force Delete will obliterate all references to this asset and is dangerous.\n\nUse as a last resort." ) )
		.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
		.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
		.OnClicked(this, &SDeleteAssetsDialog::ForceDelete)
	];
}

FReply SDeleteAssetsDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( ReferencerCommands->ProcessCommandBindings(InKeyEvent) )
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SDeleteAssetsDialog::OnCancelHotkeyPressed()
{
	Cancel();
}

void SDeleteAssetsDialog::OnConfirmHotkeyPressed()
{
	// If we're already in the process of deleting, we don't want to do anything
	if (DeleteModel->GetState() != FAssetDeleteModel::Finished)
	{
		return;
	}

	// If Force Delete is available, trigger that
	if (CanForceDelete())
	{
		ForceDelete();
	}
	// Otherwise trigger the normal Delete
	else
	{
		Delete();
	}
}

TSharedPtr<SWidget> SDeleteAssetsDialog::GetWidgetToFocusOnActivate()
{
	return CanForceDelete() ? ForceDeleteButton : DeleteButton;
}

EActiveTimerReturnType SDeleteAssetsDialog::TickDeleteModel( double InCurrentTime, float InDeltaTime )
{
	DeleteModel->Tick( InDeltaTime );

	if ( DeleteModel->GetState() == FAssetDeleteModel::EState::Finished )
	{
		bIsActiveTimerRegistered = false;
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SDeleteAssetsDialog::HandleDeleteModelStateChanged(FAssetDeleteModel::EState NewState)
{
	switch ( NewState )
	{
	case FAssetDeleteModel::StartScanning:
		RootContainer->SetContent( BuildProgressDialog() );
		break;
	case FAssetDeleteModel::Finished:
		RootContainer->SetContent( BuildDeleteDialog() );
		break;
	case FAssetDeleteModel::Scanning:
	case FAssetDeleteModel::UpdateActions:
	case FAssetDeleteModel::Waiting:
		break;
	}
}

FText SDeleteAssetsDialog::ScanningText() const
{
	return DeleteModel->GetProgressText();
}

TOptional< float > SDeleteAssetsDialog::ScanningProgressFraction() const
{
	return DeleteModel->GetProgress();
}

TSharedRef<SWidget> SDeleteAssetsDialog::CreateThumbnailWidget()
{
	ConsolidationAssetThumbnail = MakeShareable( new FAssetThumbnail( NULL, 40, 40, UThumbnailManager::Get().GetSharedThumbnailPool()) );

	return SNew( SBox )
		.WidthOverride( 40.0f )
		.HeightOverride( 40.0f )
		[
			ConsolidationAssetThumbnail->MakeThumbnailWidget()
		];
}

EVisibility SDeleteAssetsDialog::GetReferencesVisiblity() const
{
	return DeleteModel->IsAnythingReferencedInMemoryByNonUndo() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDeleteAssetsDialog::GetUndoVisiblity() const
{
	return DeleteModel->IsAnythingReferencedInMemoryByUndo() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<ITableRow> SDeleteAssetsDialog::HandleGenerateAssetRow( TSharedPtr<FPendingDelete> InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	return SNew(SPendingDeleteRow, OwnerTable, InItem)
		.Visibility(InItem->IsInternal() ? EVisibility::Collapsed : EVisibility::Visible);
}

TSharedRef<ITableRow> SDeleteAssetsDialog::HandleGenerateFolderRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SPendingFolderDeleteRow, InOwnerTable, InItem);
}

void SDeleteAssetsDialog::DeleteRelevantSourceContent()
{
	if (DeleteModel->HasAnySourceContentFilesToDelete())
	{
		auto* Settings = GetMutableDefault<UEditorLoadingSavingSettings>();
		if (DeleteSourceFilesCheckbox->GetCheckedState() == ECheckBoxState::Checked)
		{
			Settings->bDeleteSourceFilesWithAssets = true;
			DeleteModel->DeleteSourceContentFiles();
		}
		else
		{
			Settings->bDeleteSourceFilesWithAssets = false;			
		}
	}
}

FReply SDeleteAssetsDialog::Delete()
{
	OnDialogClosed.ExecuteIfBound(EResult::Delete);

	if (TSharedPtr<SWindow> OwningWindow = ParentWindow.Pin())
	{
		OwningWindow->RequestDestroyWindow();
	}

	if (DeleteModel->IsAnythingReferencedInMemoryByUndo())
	{
		GEditor->Trans->Reset(LOCTEXT("DeleteSelectedItem", "Delete Selected Item"));
	}

	DeleteRelevantSourceContent();
	DeleteModel->DoDelete();

	return FReply::Handled();
}

FReply SDeleteAssetsDialog::Cancel()
{
	OnDialogClosed.ExecuteIfBound(EResult::Cancel);

	if (TSharedPtr<SWindow> OwningWindow = ParentWindow.Pin())
	{
		OwningWindow->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SDeleteAssetsDialog::ForceDelete()
{
	OnDialogClosed.ExecuteIfBound(EResult::ForceDelete);

	if (TSharedPtr<SWindow> OwningWindow = ParentWindow.Pin())
	{
		OwningWindow->RequestDestroyWindow();
	}

	if( DeleteModel->IsAnythingReferencedInMemoryByUndo() )
	{
		GEditor->Trans->Reset( LOCTEXT("DeleteSelectedItem", "Delete Selected Item") );
	}

	DeleteRelevantSourceContent();
	DeleteModel->DoForceDelete();

	return FReply::Handled();
}

FText SDeleteAssetsDialog::GetConsolidateAssetName() const
{
	if ( !ConsolidationAsset.IsValid() )
	{
		return LOCTEXT( "None", "None" );
	}
	else
	{
		return FText::FromName(ConsolidationAsset.AssetName);
	}
}

FReply SDeleteAssetsDialog::ReplaceReferences()
{
	if ( !ConsolidationAsset.IsValid() )
	{
		return FReply::Handled();
	}

	FText Message = FText::Format( LOCTEXT( "ReplaceMessage", "This will replace any reference to the pending deleted assets with {0}; and then delete them.\n\nAre you sure?" ), FText::FromName( ConsolidationAsset.AssetName ) );
	FText Title = LOCTEXT( "ReplaceTitle", "Replace References?" );

	if ( EAppReturnType::Ok == FMessageDialog::Open( EAppMsgType::OkCancel, Message, Title ) )
	{
		if (TSharedPtr<SWindow> OwningWindow = ParentWindow.Pin())
		{
			OwningWindow->RequestDestroyWindow();
		}
		DeleteRelevantSourceContent();
		DeleteModel->DoReplaceReferences( ConsolidationAsset );
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SDeleteAssetsDialog::MakeAssetViewForReferencerAssets()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;

	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.bAutohideSearchBar = true;

	AssetPickerConfig.AssetShowWarningText = TAttribute<FText>( this, &SDeleteAssetsDialog::GetReferencingAssetsEmptyText );
	AssetPickerConfig.AssetShowWarningTextJustification = ETextJustify::Left;
	AssetPickerConfig.AssetShowWarningTextSelectable = true;

	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SDeleteAssetsDialog::OnAssetsActivated);
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SDeleteAssetsDialog::OnShouldFilterAsset);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP( this, &SDeleteAssetsDialog::OnGetAssetContextMenu );
	AssetPickerConfig.GetCurrentSelectionDelegates.Add( &GetSelectedReferencerAssets );

	return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
}

TSharedRef<SWidget> SDeleteAssetsDialog::MakeConsolidationAssetPicker()
{
	FAssetPickerConfig AssetPickerConfig;
	//AssetPickerConfig.Filter.ClassPaths.Add( UStaticMesh::StaticClass()->GetFName() );
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP( this, &SDeleteAssetsDialog::OnAssetSelectedFromConsolidationPicker );
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP( this, &SDeleteAssetsDialog::OnShouldConsolidationFilterAsset );
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bShowBottomToolbar = true;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.bAddFilterUI = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>( TEXT( "ContentBrowser" ) );

	return SNew( SBox )
		.HeightOverride( 250.0f )
		.WidthOverride( 300.0f )
		[
			ContentBrowserModule.Get().CreateAssetPicker( AssetPickerConfig )
		];
}

FText SDeleteAssetsDialog::GetReferencingAssetsEmptyText() const
{
	FString DiskReferences = "There Are Some Non-Displayable References\n\n";

	static FName NAME_ActorLabel(TEXT("ActorLabel"));
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	for ( const FName& DiskReference : DeleteModel->GetAssetReferences() )
	{
		FString ReferenceToAppend;

		FARFilter Filter;
		Filter.PackageNames = { DiskReference };
		
		TArray<FAssetData> Assets;
		if (AssetRegistry.GetAssets(Filter, Assets))
		{
			for (const FAssetData& Asset : Assets)
			{
				FString ActorLabel;
				if (Asset.GetTagValue(NAME_ActorLabel, ActorLabel))
				{
					ReferenceToAppend = Asset.GetOptionalOuterPathName().ToString() + TEXT(".") + ActorLabel;
					break;
				}
			}
		}
		
		if (ReferenceToAppend.IsEmpty())
		{
			ReferenceToAppend = DiskReference.ToString();
		}

		DiskReferences += ReferenceToAppend + TEXT("\n");
	}

	return FText::FromString( DiskReferences );
}

/** Handler for when the user double clicks, presses enter, or presses space on an asset */
void SDeleteAssetsDialog::OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod)
{
	// Open a simple asset editor for all assets which do not have asset type actions if activating with enter or double click
	if ( ActivationMethod == EAssetTypeActivationMethod::DoubleClicked || ActivationMethod == EAssetTypeActivationMethod::Opened )
	{
		if (TSharedPtr<SWindow> OwningWindow = ParentWindow.Pin())
		{
			OwningWindow->RequestDestroyWindow();
		}

		for(const FAssetData& ActivatedAsset : ActivatedAssets)
		{
			FString MapFilePath;
			if (FEditorFileUtils::IsMapPackageAsset(ActivatedAsset.GetObjectPathString(), MapFilePath))
			{
				if ( ActivatedAsset.IsAssetLoaded() )
				{
					DeleteModel->GoToNextReferenceInLevel();
				}
				else
				{
					if ( !GIsDemoMode )
					{
						// If there are any unsaved changes to the current level, see if the user wants to save those first.
						bool bPromptUserToSave = true;
						bool bSaveMapPackages = true;
						bool bSaveContentPackages = true;
						if ( FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages) == false )
						{
							// something went wrong or the user pressed cancel.  Return to the editor so the user doesn't lose their changes		
							return;
						}
					}

					FEditorDirectories::Get().SetLastDirectory(ELastDirectory::LEVEL, FPaths::GetPath(MapFilePath));
					FEditorFileUtils::LoadMap(MapFilePath, false, true);

					// @todo ndarnell Now that the map is loading how do i select the first reference...
				}
			}
			else
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ActivatedAsset.GetAsset());

				// @todo ndarnell select in content browser maybe as well?
			}
		}
	}
}

EVisibility SDeleteAssetsDialog::GetReplaceReferencesVisibility() const
{
	// We can't replace references if nobody is referencing the pending deleted assets
	if ( DeleteModel->GetAssetReferences().Num() == 0 )
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SDeleteAssetsDialog::GetForceDeleteVisibility() const
{
	return CanForceDelete() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDeleteAssetsDialog::GetDeleteVisibility() const
{
	return CanDelete() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDeleteAssetsDialog::GetDeleteSourceFilesVisibility() const
{
	return DeleteModel->HasAnySourceContentFilesToDelete() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SDeleteAssetsDialog::CanReplaceReferences() const
{
	return DeleteModel->CanReplaceReferences();
}

bool SDeleteAssetsDialog::CanForceDelete() const
{
	return DeleteModel->CanForceDelete();
}

bool SDeleteAssetsDialog::CanDelete() const
{
	return DeleteModel->CanDelete();
}

bool SDeleteAssetsDialog::OnShouldConsolidationFilterAsset( const FAssetData& InAssetData ) const
{
	return DeleteModel->CanReplaceReferencesWith( InAssetData );
}

TSharedPtr<SWidget> SDeleteAssetsDialog::OnGetAssetContextMenu( const TArray<FAssetData>& SelectedAssets )
{
	// Get all menu extenders for this context menu from the content browser module
	/*FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT( "ContentBrowser" ) );
	TArray<FContentBrowserMenuExtender_SelectedAssets> MenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for ( int32 i = 0; i < MenuExtenderDelegates.Num(); ++i )
	{
		if ( MenuExtenderDelegates[i].IsBound() )
		{
			Extenders.Add( MenuExtenderDelegates[i].Execute( SelectedAssets ) );
		}
	}*/
	//TSharedPtr<FExtender> MenuExtender = FExtender::Combine( Extenders );

	FMenuBuilder MenuBuilder(true, ReferencerCommands);

	MenuBuilder.BeginSection("AssetOptions", LOCTEXT("AssetOptionsText", "Asset Options"));
	{
		MenuBuilder.AddMenuEntry( FGenericCommands::Get().Delete, NAME_None,
			LOCTEXT( "AddPendingDelete", "Add to Pending Deletes" ),
			LOCTEXT( "AddPendingDeleteTooltip", "Adds the selected assets to the list of pending deleted assets." )
			);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SDeleteAssetsDialog::CanExecuteDeleteReferencers() const
{
	TArray<FAssetData> SelectedAssets = GetSelectedReferencerAssets.Execute();
	return SelectedAssets.Num() > 0;
}

void SDeleteAssetsDialog::ExecuteDeleteReferencers()
{
	TArray<FAssetData> SelectedAssets = GetSelectedReferencerAssets.Execute();

	for ( const FAssetData& SelectedAsset : SelectedAssets )
	{
		UObject* ObjectToDelete = SelectedAsset.GetAsset();
		if (ObjectToDelete)
		{
			DeleteModel->AddObjectToDelete(ObjectToDelete);
		}
		if ( !bIsActiveTimerRegistered )
		{
			bIsActiveTimerRegistered = true;
			RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SDeleteAssetsDialog::TickDeleteModel ) );
		}
	}
}

bool SDeleteAssetsDialog::OnShouldFilterAsset( const FAssetData& InAssetData ) const
{
	// Filter out any redirectors that are not to the main UAsset
	if ( InAssetData.IsRedirector() && !InAssetData.IsUAsset() )
	{
		return true;
	}

	// If it's in the set of references then don't filter it.
	if ( DeleteModel->GetAssetReferences().Contains( InAssetData.PackageName ) )
	{
		return false;
	}

	return true;
}

void SDeleteAssetsDialog::OnAssetSelectedFromConsolidationPicker(const FAssetData& AssetData)
{
	ConsolidationAssetThumbnail->SetAsset( AssetData );
	ConsolidationAssetThumbnail->RefreshThumbnail();

	ConsolidationAsset = AssetData;
	ConsolidationPickerComboButton->SetIsOpen( false );
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
