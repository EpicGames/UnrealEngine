// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/SOutputLogDialog.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Images/SImage.h"
#include "HAL/PlatformApplicationMisc.h"

void SOutputLogDialog::Open( const FText& InTitle, const FText& InHeader, const FText& InLog, const FText& InFooter )
{
	TArray<FText> Buttons;
	Buttons.Add(NSLOCTEXT("SOutputLogDialog", "Ok", "Ok"));
	Open(InTitle, InHeader, InLog, InFooter, Buttons);
}

int32 SOutputLogDialog::Open( const FText& InTitle, const FText& InHeader, const FText& InLog, const FText& InFooter, const TArray<FText>& InButtons )
{
	TSharedRef<SWindow> ModalWindow = SNew(SWindow)
		.Title( InTitle )
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false) 
		.SupportsMaximize(false);

	TSharedRef<SOutputLogDialog> MessageBox = SNew(SOutputLogDialog)
		.ParentWindow(ModalWindow)
		.Header( InHeader )
		.Log( InLog )
		.Footer( InFooter )
		.Buttons( InButtons );

	ModalWindow->SetContent( MessageBox );

	GEditor->EditorAddModalWindow(ModalWindow);

	return MessageBox->Response;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SOutputLogDialog::Construct( const FArguments& InArgs )
{
	ParentWindow = InArgs._ParentWindow.Get();
	ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));

	FSlateFontInfo MessageFont( FAppStyle::GetFontStyle("StandardDialog.LargeFont"));
	Header = InArgs._Header.Get();
	Log = InArgs._Log.Get();
	Footer = InArgs._Footer.Get();
	Buttons = InArgs._Buttons.Get();

	const float DPIFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(ParentWindow->GetPositionInScreen().X, ParentWindow->GetPositionInScreen().Y);
	MaxWidth = FSlateApplication::Get().GetPreferredWorkArea().GetSize().X * 0.8f / DPIFactor;

	TSharedPtr<SUniformGridPanel> ButtonBox;

	this->ChildSlot
		[	
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.AutoHeight()
						.Padding(12.0f)
						[
							SNew(STextBlock)
								.Text(Header)
								.Visibility(Header.IsEmptyOrWhitespace()? EVisibility::Hidden : EVisibility::Visible)
								.Font(MessageFont)
								.WrapTextAt(MaxWidth - 50.0f)
						]

					+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.FillHeight(1.0f)
						.MaxHeight(550)
						.Padding(12.0f, 0.0f, 12.0f, 12.0f)
						[
							SNew(SMultiLineEditableTextBox)
								.Style(FAppStyle::Get(), "Log.TextBox")
								.ForegroundColor(FLinearColor::Gray)
								.Text(FText::TrimTrailing(Log))
								.IsReadOnly(true)
								.AlwaysShowScrollbars(true)
						]

					+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.AutoHeight()
						.Padding(12.0f, 0.0f, 12.0f, Footer.IsEmptyOrWhitespace()? 0.0f : 12.0f)
						[
							SNew(STextBlock)
								.Text(Footer)
								.Visibility(Footer.IsEmptyOrWhitespace()? EVisibility::Hidden : EVisibility::Visible)
								.Font(MessageFont)
								.WrapTextAt(MaxWidth - 50.0f)
						]

					+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(12.0f, 0.0f, 12.0f, 12.0f)
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(0.0f)
								[
									SNew(SButton)
									.ButtonStyle(FAppStyle::Get(), "SimpleButton")
									.OnClicked(this, &SOutputLogDialog::HandleCopyMessageButtonClicked)
									.ToolTipText(NSLOCTEXT("SOutputLogDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
									.ContentPadding(2.f)
									.Content()
									[
										SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Right)
								.VAlign(VAlign_Center)
								.Padding(0.0f)
								[
									SAssignNew( ButtonBox, SUniformGridPanel )
										.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
										.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
										.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
								]
						]
				]
		];

	for(int32 Idx = 0; Idx < Buttons.Num(); Idx++)
	{
		ButtonBox->AddSlot(Idx, 0)
			[
				SNew( SButton )
				.Text( Buttons[Idx] )
				.OnClicked( this, &SOutputLogDialog::HandleButtonClicked, Idx )
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.HAlign(HAlign_Center)
			];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SOutputLogDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
	{
		CopyMessageToClipboard();
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

bool SOutputLogDialog::SupportsKeyboardFocus() const
{
	return true;
}

FVector2D SOutputLogDialog::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	FVector2D OutputLogDesiredSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	OutputLogDesiredSize.X = FMath::Min(OutputLogDesiredSize.X, MaxWidth);
	return OutputLogDesiredSize;
}

void SOutputLogDialog::CopyMessageToClipboard( )
{
	FString FullMessage = FString::Printf(TEXT("%s") LINE_TERMINATOR LINE_TERMINATOR TEXT("%s") LINE_TERMINATOR LINE_TERMINATOR TEXT("%s"), *Header.ToString(), *Log.ToString(), *Footer.ToString()).TrimStart();
	FPlatformApplicationMisc::ClipboardCopy( *FullMessage );
}

FReply SOutputLogDialog::HandleButtonClicked( int32 InResponse )
{
	Response = InResponse;
	ParentWindow->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SOutputLogDialog::HandleCopyMessageButtonClicked( )
{
	CopyMessageToClipboard();
	return FReply::Handled();
}
