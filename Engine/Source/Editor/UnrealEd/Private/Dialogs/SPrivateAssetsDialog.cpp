// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialogs/SPrivateAssetsDialog.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "SPrimaryButton.h"
#include "SWarningOrErrorBox.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Editor.h"
#include "Editor/Transactor.h"

#define LOCTEXT_NAMESPACE "SPrivateAssetsDialog"

namespace PrivatizeAssetsView
{
	static const FName ColumnID_Asset("Asset");
	static const FName ColumnID_AssetClass("Class");
	static const FName ColumnID_DiskReferences("DiskReferences");
	static const FName ColumnID_MemoryReferences("MemoryReferences");
}

//////////////////////////////////////////////////////////////////////////
// SPendingPrivateAssetRow

class SPendingPrivateAssetRow : public SMultiColumnTableRow<TSharedPtr<FPendingPrivateAsset>>
{
public:

	SLATE_BEGIN_ARGS(SPendingPrivateAssetRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FPendingPrivateAsset> InItem)
	{
		Item = InItem;

		SMultiColumnTableRow<TSharedPtr<FPendingPrivateAsset>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == PrivatizeAssetsView::ColumnID_Asset)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Item->GetObject()->GetName()))
				];
		}
		else if (ColumnName == PrivatizeAssetsView::ColumnID_AssetClass)
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->GetObject()->GetClass()->GetName()));
		}
		else if (ColumnName == PrivatizeAssetsView::ColumnID_DiskReferences)
		{
			int32 IllegalDiskReferenceCount = Item->IllegalDiskReferences.Num();
			FFormatNamedArguments Args;
			Args.Add(TEXT("AssetCount"), FText::AsNumber(IllegalDiskReferenceCount));

			FText OnDiskCountText = IllegalDiskReferenceCount > 1 ?
				FText::Format(LOCTEXT("InAssetReferences", "{AssetCount} References"), Args) : FText::Format(LOCTEXT("InAssetReference", "{AssetCount} Reference"), Args);

			return SNew(STextBlock)
				.Text(OnDiskCountText)
				.Visibility(IllegalDiskReferenceCount > 0 ? EVisibility::Visible : EVisibility::Hidden);
		}
		else if (ColumnName == PrivatizeAssetsView::ColumnID_MemoryReferences)
		{
			int32 IllegalMemoryReferenceCount = Item->IllegalMemoryReferences.ExternalReferences.Num();
			FFormatNamedArguments Args;
			Args.Add(TEXT("ReferenceCount"), FText::AsNumber(IllegalMemoryReferenceCount));

			FText InMemoryCountText = IllegalMemoryReferenceCount > 1 ? 
				FText::Format(LOCTEXT("InMemoryReferences", "{ReferenceCount} References"), Args) : FText::Format(LOCTEXT("InMemoryReference", "{ReferenceCount} Reference"), Args);

			return SNew(STextBlock)
				.Text(InMemoryCountText)
				.Visibility(IllegalMemoryReferenceCount > 0 ? EVisibility::Visible : EVisibility::Hidden);
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FPendingPrivateAsset> Item;
};

//////////////////////////////////////////////////////////////////////////
// SPrivateAssetsDialog

SPrivateAssetsDialog::~SPrivateAssetsDialog()
{
	PrivatizeModel->OnStateChanged().RemoveAll(this);
}

void SPrivateAssetsDialog::Construct(const FArguments& InArgs, const TSharedRef<FAssetPrivatizeModel> InPrivatizeModel)
{
	bIsActiveTimerRegistered = true;
	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SPrivateAssetsDialog::TickPrivatizeModel));

	PrivatizeModel = InPrivatizeModel;

	ParentWindow = InArgs._ParentWindow;

	ChildSlot
	[
		SAssignNew(RootContainer, SBorder)
		.BorderImage(FAppStyle::GetBrush("AssetDeleteDialog.Background"))
		.Padding(10)
	];

	PrivatizeModel->OnStateChanged().AddRaw(this, &SPrivateAssetsDialog::HandlePrivatizeModelStateChanged);

	HandlePrivatizeModelStateChanged(PrivatizeModel->GetState());
}

FReply SPrivateAssetsDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{

	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		ParentWindow.Get()->RequestDestroyWindow();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedRef<SWidget> SPrivateAssetsDialog::BuildProgressDialog()
{
	// Shows a progress bar during the scanning of references for each pending private asset
	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.VAlign(VAlign_Center)
	.FillHeight(1.0f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f, 0)
		[
			SNew(STextBlock)
			.Text(this, &SPrivateAssetsDialog::ScanningText)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f, 10.0f)
		[
			SNew(SProgressBar)
			.Percent(this, &SPrivateAssetsDialog::ScanningProgressFraction)
		]
	];
}

TSharedRef<SWidget> SPrivateAssetsDialog::BuildPrivatizeDialog()
{
	
	TSharedRef<SHeaderRow> HeaderRowWidget =
		SNew(SHeaderRow)
		+SHeaderRow::Column(PrivatizeAssetsView::ColumnID_Asset)
		.DefaultLabel(LOCTEXT("Column_AssetName", "Asset"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Left)
		.FillWidth(0.5f)
		+SHeaderRow::Column(PrivatizeAssetsView::ColumnID_AssetClass)
		.DefaultLabel(LOCTEXT("Column_AssetClass", "Class"))
		.HAlignHeader(HAlign_Left)
		.FillWidth(0.25f)
		+SHeaderRow::Column(PrivatizeAssetsView::ColumnID_DiskReferences)
		.DefaultLabel(LOCTEXT("Column_DiskReferences", "Asset Referencers"))
		.HAlignHeader(HAlign_Left)
		.FillWidth(0.25f)
		+SHeaderRow::Column(PrivatizeAssetsView::ColumnID_MemoryReferences)
		.DefaultLabel(LOCTEXT("Column_MemoryReferences", "Memory Referencers"))
		.HAlignHeader(HAlign_Left)
		.FillWidth(0.25f);

	return SNew(SVerticalBox)
	// The to be made private assets and their reference counts
	+SVerticalBox::Slot()
	.FillHeight(0.5f)
	.Padding(5.0f)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(0, 0, 0, 3))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
				.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
				.Padding(3.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AttemptingPrivatize", "The following assets will be made private:"))
					.Font(FAppStyle::GetFontStyle("BoldFont"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(ObjectsToPrivatizeList, SListView<TSharedPtr<FPendingPrivateAsset>>)
				.ListItemsSource(PrivatizeModel->GetPendingPrivateAssets())
				.OnGenerateRow(this, &SPrivateAssetsDialog::HandleGenerateAssetRow)
				.HeaderRow(HeaderRowWidget)
			]
		]
	]
	// An asset list of all on disk references that would become illegal if the pending assets are made private
	+SVerticalBox::Slot()
	.FillHeight(1.0f)
	.Padding(5.0f)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		.Visibility(this, &SPrivateAssetsDialog::GetAssetReferencesVisibility)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
				.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f, 1.0f))
				.Padding(3.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetsReferencingPendingPrivateAssets", "References to the above assets will be removed from the following:"))
					.Font(FAppStyle::GetFontStyle("BoldFont"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				BuildAssetViewForReferencerAssets()
			]
		]
	]
	// Details any extra warning information about the operation
	+SVerticalBox::Slot()
	.AutoHeight()
	.Padding(FMargin(8.0f, 5.0f))
	[
		SNew(SWarningOrErrorBox)
		.Visibility(this, &SPrivateAssetsDialog::GetWarningTextVisibility)
		.MessageStyle(EMessageStyle::Warning)
		.Message(this, &SPrivateAssetsDialog::GetWarningText)
	]
	// Confirm or cancel buttons
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		.Padding(0)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.Padding(6, 0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("Privatize", "Make Private"))
						.ToolTipText(LOCTEXT("PrivatizeTooltipText", "Make private"))
						.OnClicked(this, &SPrivateAssetsDialog::ForcePrivatize)
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.Padding(6, 0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.VAlign(VAlign_Bottom)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.ToolTipText(LOCTEXT("CancelPrivatizeTooltipText", "Cancel making selection private"))
						.OnClicked(this, &SPrivateAssetsDialog::Cancel)
					]
				]
			]
		]
	];
}

TSharedRef<SWidget> SPrivateAssetsDialog::BuildAssetViewForReferencerAssets()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;

	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.bAutohideSearchBar = true;

	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SPrivateAssetsDialog::OnShouldFilterAsset);

	return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
}

EActiveTimerReturnType SPrivateAssetsDialog::TickPrivatizeModel(double InCurrentTime, float InDeltaTime)
{
	PrivatizeModel->Tick(InDeltaTime);

	if (PrivatizeModel->GetState() == FAssetPrivatizeModel::EState::Finished)
	{
		bIsActiveTimerRegistered = false;
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

void SPrivateAssetsDialog::HandlePrivatizeModelStateChanged(FAssetPrivatizeModel::EState NewState)
{
	switch (NewState)
	{
	case FAssetPrivatizeModel::StartScanning:
		RootContainer->SetContent(BuildProgressDialog());
		break;
	case FAssetPrivatizeModel::Finished:
		// If in our scan we realize we can cleanly perform the privatize operation without any illegal reference fix up
		// We can skip the confirmation dialog and close down
		if (PrivatizeModel->CanPrivatize())
		{
			Privatize();
		}
		RootContainer->SetContent(BuildPrivatizeDialog());
		break;
	default:
		break;
	}
}

bool SPrivateAssetsDialog::OnShouldFilterAsset(const FAssetData& InAssetData) const
{
	if (InAssetData.IsRedirector() && !InAssetData.IsUAsset())
	{
		return true;
	}

	if (PrivatizeModel->GetIllegalAssetReferences().Contains(InAssetData.PackageName))
	{
		return false;
	}

	return true;
}

FReply SPrivateAssetsDialog::Privatize()
{
	ParentWindow.Get()->RequestDestroyWindow();

	PrivatizeModel->DoPrivatize();

	return FReply::Handled();
}

FReply SPrivateAssetsDialog::ForcePrivatize()
{
	ParentWindow.Get()->RequestDestroyWindow();

	// If anything is referenced in undo we need to purge the undo stack
	// Otherwise you could undo back to restoring a reference which would now be illegal
	if (PrivatizeModel->IsAnythingReferencedInMemoryByUndo())
	{
		GEditor->Trans->Reset(LOCTEXT("PrivatizeSelectedItem", "Privatize Selected Item"));
	}

	PrivatizeModel->DoForcePrivatize();

	return FReply::Handled();
}

FReply SPrivateAssetsDialog::Cancel()
{
	ParentWindow.Get()->RequestDestroyWindow();

	return FReply::Handled();
}

FText SPrivateAssetsDialog::GetWarningText() const
{
	bool bReferencesFoundInUndo = PrivatizeModel->IsAnythingReferencedInMemoryByUndo();
	bool bReferencesFoundInNonUndo = PrivatizeModel->IsAnythingReferencedInMemoryByNonUndo();

	FText MemoryWarning = LOCTEXT("PrivateAssetMemoryReferenceWarning", "- Some assets are still referenced in memory");
	FText UndoWarning = LOCTEXT("PrivateAssetUndoWarning", "- Undo stack will be cleared as a result of this action");

	FText WarningText;

	if (bReferencesFoundInNonUndo)
	{
		WarningText = MemoryWarning;
	}

	if (bReferencesFoundInUndo)
	{
		WarningText = FText::Join(FText::FromString("\n"), WarningText, UndoWarning);
	}

	return WarningText;
}

EVisibility SPrivateAssetsDialog::GetWarningTextVisibility() const
{
	return (PrivatizeModel->IsAnythingReferencedInMemoryByUndo() || 
		PrivatizeModel->IsAnythingReferencedInMemoryByNonUndo()) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPrivateAssetsDialog::GetAssetReferencesVisibility() const
{
	return PrivatizeModel->GetIllegalAssetReferences().Num() == 0 ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SPrivateAssetsDialog::ScanningText() const
{
	return PrivatizeModel->GetProgressText();
}

TOptional<float> SPrivateAssetsDialog::ScanningProgressFraction() const
{
	return PrivatizeModel->GetProgress();
}

TSharedRef<ITableRow> SPrivateAssetsDialog::HandleGenerateAssetRow(TSharedPtr<FPendingPrivateAsset> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPendingPrivateAssetRow, OwnerTable, InItem);
}

#undef LOCTEXT_NAMESPACE