// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiffUtils.h"

#include "AssetDefinitionRegistry.h"
#include "Components/ActorComponent.h"
#include "Containers/BitArray.h"
#include "EditorCategoryUtils.h"
#include "IAssetTools.h"
#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "IAssetTypeActions.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "ObjectEditorUtils.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/SavePackage.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "HAL/PlatformFileManager.h"
#include "UnrealEngine.h"
#include "UObject/Linker.h"

class ITableRow;
class SWidget;

namespace UEDiffUtils_Private
{
	static FProperty* Resolve( const UStruct* Class, FName PropertyName )
	{
		if(Class == nullptr )
		{
			return nullptr;
		}

		for (FProperty* Prop : TFieldRange<FProperty>(Class))
		{
			if( Prop->GetFName() == PropertyName )
			{
				return Prop;
			}
		}

		return nullptr;
	}

	static FPropertySoftPathSet GetPropertyNameSet(const UStruct* ForStruct)
	{
		return FPropertySoftPathSet(DiffUtils::GetVisiblePropertiesInOrderDeclared(ForStruct));
	}
	
	static const FString DiffSyntaxHelp = TEXT("format: 'diff <lhs> <rhs>");
	static const FString MergeSyntaxHelp = TEXT("format: 'merge <remote> <local> <base> [-o out_path]' or 'merge <local> [-o out_path]'");
	static void RunDiffCommand(const TArray<FString>& Args);
	static void RunMergeCommand(const TArray<FString>& Args);

	FAutoConsoleCommand MergeConsoleCommand(
		TEXT("merge"),
		*FString::Format(TEXT("Either merge three assets or a single conflicted asset.\n{0}"), {MergeSyntaxHelp}),
		FConsoleCommandWithArgsDelegate::CreateStatic(&RunMergeCommand),
		ECVF_Default
	);

	FAutoConsoleCommand DiffConsoleCommand(
		TEXT("diff"),
		*FString::Format(TEXT("diff two assets against one another.\n{0}"), {DiffSyntaxHelp}),
		FConsoleCommandWithArgsDelegate::CreateStatic(&RunDiffCommand),
		ECVF_Default
	);
}

static UObject* LoadAssetFromExternalPath(FString Path)
{
	FPackagePath PackagePath;
	if (!FPackagePath::TryFromPackageName(Path, PackagePath))
	{
		// copy to the temp directory so it can be loaded properly
		FString File = FPaths::GetBaseFilename(Path) + TEXT("-");
		for (const ANSICHAR Char : "#(){}[].")
		{
			File.ReplaceCharInline(Char, '-');
		}
		const FString Extension = TEXT(".") + FPaths::GetExtension(Path);
		const FString SourcePath = Path;
		FPlatformFileManager::Get().GetPlatformFile().CreateDirectory(*FPaths::DiffDir());
		Path = FPaths::CreateTempFilename(*FPaths::DiffDir(), *File, *Extension);
		Path = FPaths::ConvertRelativePathToFull(Path);
		if (!FPlatformFileManager::Get().GetPlatformFile().CopyFile(*Path, *SourcePath))
		{
			UE_LOGF(LogEngine, Display, "Failed to Copy %ls", *SourcePath);
			return nullptr;
		}
		// load the temp package
		PackagePath = FPackagePath::FromLocalPath(Path);
	}
	if (PackagePath.IsEmpty())
	{
		UE_LOGF(LogEngine, Display, "Invalid Path: %ls", *Path);
		return nullptr;
	}
	if (const UPackage* TempPackage = DiffUtils::LoadPackageForDiff(PackagePath, {}))
	{
		if (UObject* Object = TempPackage->FindAssetInPackage())
		{
			return Object;
		}
	}
	UE_LOGF(LogEngine, Display, "Failed to load: %ls", *Path);
	return nullptr;
}

static void UEDiffUtils_Private::RunDiffCommand(const TArray<FString>& Args)
{
	if (Args.Num() != 2)
	{
		UE_LOGF(LogEngine, Display, "%ls", *DiffSyntaxHelp);
		return;
	}
	
	UObject* LHS = LoadAssetFromExternalPath(Args[0]);
	UObject* RHS = LoadAssetFromExternalPath(Args[1]);
	if (LHS && RHS)
	{
		IAssetTools::Get().DiffAssets(LHS, RHS, {}, {});
	}
}

namespace UE::CmdLink
{
	// CmdLinkServerModule will set these methods if loaded.
	// they're used by the merge command because we need to keep the CmdLink client running until the user closes the merge window and saves the output
	UNREALED_API void(*GBeginAsyncCommand)(const FString&, const TArray<FString>&) = [](const FString&, const TArray<FString>&){};
	UNREALED_API void(*GEndAsyncCommand)(const FString&, const TArray<FString>&) = [](const FString&, const TArray<FString>&){};
}

static void UEDiffUtils_Private::RunMergeCommand(const TArray<FString>& Args)
{
	UObject* Local = nullptr;
	UObject* Base = nullptr;
	UObject* Remote = nullptr;
	FString OutDirectory;
	bool bThreeWayMerge = false;
	bool bInvalidSyntax = false;
	switch (Args.Num())
	{
	case 1: // merge <local>
		Local = LoadAssetFromExternalPath(Args[0]);
		bThreeWayMerge = false;
		break;
	case 3:
		if (Args[1] == TEXT("-o")) // merge <local> -o <output_file>
		{
			Local = LoadAssetFromExternalPath(Args[0]);
			OutDirectory = Args[2];
			bThreeWayMerge = false;
		}
		else // merge <local> <base> <remote>
		{
			Remote = LoadAssetFromExternalPath(Args[0]);
			Local = LoadAssetFromExternalPath(Args[1]);
			Base = LoadAssetFromExternalPath(Args[2]);
			bThreeWayMerge = true;
		}
		break;
	case 5: // merge <local> <base> <remote> -o <output_file>
		if (Args[3] == TEXT("-o"))
		{
			Remote = LoadAssetFromExternalPath(Args[0]);
			Local = LoadAssetFromExternalPath(Args[1]);
			Base = LoadAssetFromExternalPath(Args[2]);
			OutDirectory = Args[4];
			bThreeWayMerge = true;
			break;
		}

		// 5 parameters requires output file at the end
		bInvalidSyntax = true;
		break;
	default:
		// unsupported parameter count
		bInvalidSyntax = true;
		break;
	}

	if (bInvalidSyntax)
	{
		// invalid syntax. display help.
		UE_LOGF(LogEngine, Display, "%ls", *MergeSyntaxHelp);
		return;
	}

	const FOnAssetMergeResolved ResolutionCallback = FOnAssetMergeResolved::CreateLambda([Args, Local, OutDirectory](const FAssetMergeResults& Results)
	{
		if (!OutDirectory.IsEmpty() && Results.Result == EAssetMergeResult::Completed)
		{
			// save a copy of the asset to the output directory
			
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.Error = GLog;
			UPackage::SavePackage(Results.MergedPackage, Local, *OutDirectory, SaveArgs);
			ResetLoaders(Results.MergedPackage);
		}
		UE::CmdLink::GEndAsyncCommand(TEXT("merge"), Args);
	});

	if (bThreeWayMerge)
	{
		FAssetManualMergeArgs MergeArgs;
		MergeArgs.LocalAsset = Local;
		MergeArgs.BaseAsset = Base;
		MergeArgs.RemoteAsset = Remote;
		MergeArgs.ResolutionCallback = ResolutionCallback;
		MergeArgs.Flags = MF_NONE;

		if (MergeArgs.LocalAsset && MergeArgs.BaseAsset && MergeArgs.RemoteAsset)
		{
			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Local->GetClass());
			if (!AssetDefinition->CanMerge())
			{
				UE_LOGF(LogEngine, Error, "%ls of class type %ls does not support merging", *Local->GetName(), *Local->GetClass()->GetName());
				return;
			}
			UE::CmdLink::GBeginAsyncCommand(TEXT("merge"), Args);
			AssetDefinition->Merge(MergeArgs);
		}
	}
	else
	{
		FAssetAutomaticMergeArgs MergeArgs;
		MergeArgs.LocalAsset = Local;
		MergeArgs.ResolutionCallback = ResolutionCallback;
		MergeArgs.Flags = MF_NONE;
		
		if (MergeArgs.LocalAsset)
		{
			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(Local->GetClass());
			if (!AssetDefinition->CanMerge())
			{
				UE_LOGF(LogEngine, Error, "%ls does not support merging", *Local->GetName());
				return;
			}
			UE::CmdLink::GBeginAsyncCommand(TEXT("merge"), Args);
			AssetDefinition->Merge(MergeArgs);
		}
	}
}

FPropertySoftPath::FPropertySoftPath()
	: RootTypeHint(nullptr)
{
}

FPropertySoftPath::FPropertySoftPath(const FProperty* Property)
	: FPropertySoftPath()
{
	PropertyChain.Push(FChainElement(Property));
}

FPropertySoftPath::FPropertySoftPath(TArray<FName> InPropertyChain)
	: FPropertySoftPath()
{
	for (FName PropertyName : InPropertyChain)
	{
		PropertyChain.Push(FChainElement(PropertyName));
	}
}

FPropertySoftPath::FPropertySoftPath(FPropertyPath InPropertyPath)
	: FPropertySoftPath()
{
	for (int32 PropertyIndex = 0, end = InPropertyPath.GetNumProperties(); PropertyIndex != end; ++PropertyIndex)
	{
		const FPropertyInfo& Info = InPropertyPath.GetPropertyInfo(PropertyIndex);
		if (Info.ArrayIndex != INDEX_NONE)
		{
			PropertyChain.Push(FName(*FString::FromInt(Info.ArrayIndex)));
		}
		else
		{
			PropertyChain.Push(FChainElement(Info.Property.Get()));
		}
	}

	if (InPropertyPath.GetNumProperties() > 0)
	{
		const FProperty* RootProperty = InPropertyPath.GetPropertyInfo(0).Property.Get();
		if(RootProperty)
		{
			RootTypeHint = RootProperty->GetOwnerStruct();
		}
	}
}

FPropertySoftPath::FPropertySoftPath(const FPropertySoftPath& MainPropertyPath, const FPropertySoftPath& SubPropertyPath)
	: PropertyChain(MainPropertyPath.PropertyChain)
	, RootTypeHint(MainPropertyPath.RootTypeHint)
{
	PropertyChain.Append(SubPropertyPath.PropertyChain);
}

FPropertySoftPath::FPropertySoftPath(const FPropertySoftPath& SubPropertyPath, const FProperty* LeafProperty)
	: PropertyChain(SubPropertyPath.PropertyChain)
	, RootTypeHint(SubPropertyPath.RootTypeHint)
{
	PropertyChain.Push(FChainElement(LeafProperty));
}

FPropertySoftPath::FPropertySoftPath(const FPropertySoftPath& SubPropertyPath, int32 ContainerIndex)
	: PropertyChain(SubPropertyPath.PropertyChain)
	, RootTypeHint(SubPropertyPath.RootTypeHint)
{
	PropertyChain.Push(FName(*FString::FromInt(ContainerIndex)));
}

FResolvedProperty FPropertySoftPath::Resolve(const UObject* Object) const
{
	FResolvedProperty ResolvedProperty = Resolve(Object->GetClass(), Object);
	return ResolvedProperty;
}

int32 FPropertySoftPath::TryReadIndex(const TArray<FChainElement>& LocalPropertyChain, int32& OutIndex)
{
	if(OutIndex + 1 < LocalPropertyChain.Num())
	{
		FString AsString = LocalPropertyChain[OutIndex + 1].DisplayString;
		if(AsString.IsNumeric())
		{
			++OutIndex;
			return FCString::Atoi(*AsString);
		}
	}
	return INDEX_NONE;
};

FResolvedProperty FPropertySoftPath::Resolve(const UStruct* Struct, const void* StructData) const
{
	if (RootTypeHint && RootTypeHint != Struct)
	{
		if (const UClass* AsClass = Cast<UClass>(Struct))
		{
			const UScriptStruct* SparseClassDataStruct = AsClass->GetSparseClassDataStruct();
			if (SparseClassDataStruct && SparseClassDataStruct->IsChildOf(RootTypeHint))
			{
				if (const void* SparseClassData = const_cast<UClass*>(AsClass)->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull))
				{
					return Resolve(RootTypeHint, SparseClassData);
				}
			}
		}
	}

	// dig into the object, finding nested objects, etc:
	const void* CurrentBlock = StructData;
	const UStruct* NextClass = Struct;
	const void* NextBlock = CurrentBlock;
	const FProperty* Property = nullptr;

	for (int32 i = 0; i < PropertyChain.Num(); ++i)
	{
		CurrentBlock = NextBlock;
		const FProperty* NextProperty = UEDiffUtils_Private::Resolve(NextClass, PropertyChain[i].PropertyName);
		if (!NextProperty)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
            {
            	StructProperty->FindInnerPropertyInstance(PropertyChain[i].PropertyName, CurrentBlock, NextProperty, NextBlock);
            }
		}
		CurrentBlock = NextBlock;

		// if an index was provided, resolve it
		const int32 PropertyIndex = TryReadIndex(PropertyChain, i);
		if (NextProperty && PropertyIndex != INDEX_NONE)
		{
			Property = NextProperty;
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, Property->ContainerPtrToValuePtr<UObject*>(CurrentBlock));
				if (ArrayHelper.IsValidIndex(PropertyIndex))
				{
					NextProperty = ArrayProperty->Inner;
					NextBlock = ArrayHelper.GetRawPtr(PropertyIndex);
				}
			}
			else if( const FSetProperty* SetProperty = CastField<FSetProperty>(Property) )
			{
				FScriptSetHelper SetHelper(SetProperty, Property->ContainerPtrToValuePtr<UObject*>(CurrentBlock));
				NextProperty = SetHelper.GetElementProperty();
				NextBlock = SetHelper.FindNthElementPtr(PropertyIndex);
			}
			else if( const FMapProperty* MapProperty = CastField<FMapProperty>(Property) )
			{
				FScriptMapHelper MapHelper(MapProperty, Property->ContainerPtrToValuePtr<UObject*>(CurrentBlock));
				NextProperty = MapHelper.GetValueProperty();
				NextBlock = MapHelper.FindNthPairPtr(PropertyIndex);
			}
		}
		
		CurrentBlock = NextBlock;
		if (NextProperty)
		{
			Property = NextProperty;
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				const UObject* NextObject = ObjectProperty->GetObjectPropertyValue(Property->ContainerPtrToValuePtr<UObject*>(CurrentBlock));
				NextBlock = NextObject;
				NextClass = NextObject ? NextObject->GetClass() : nullptr;
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				NextBlock = StructProperty->ContainerPtrToValuePtr<void>(CurrentBlock);
				NextClass = StructProperty->Struct;
			}
			else
			{
				break;
			}
		}
		else
		{
			break;
		}
	}

	return FResolvedProperty(CurrentBlock, Property);
}

FPropertyPath FPropertySoftPath::ResolvePath(const UObject* Object) const
{
	auto UpdateContainerAddress = [](const FProperty* Property, const void* Instance, const void*& OutContainerAddress, const UStruct*& OutContainerStruct)
	{
		if( ensure(Instance) )
		{
			if(const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				const UObject* const* InstanceObject = reinterpret_cast<const UObject* const*>(Instance);
				if( *InstanceObject)
				{
					OutContainerAddress = *InstanceObject;
					OutContainerStruct = (*InstanceObject)->GetClass();
				}
			}
			else if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				OutContainerAddress = Instance;
				OutContainerStruct = StructProperty->Struct;
			}
		}
	};

	const void* ContainerAddress = Object;
	const UStruct* ContainerStruct = (Object ? Object->GetClass() : nullptr);

	FPropertyPath Ret;
	for( int32 I = 0; I < PropertyChain.Num(); ++I )
	{
		FName PropertyIdentifier = PropertyChain[I].PropertyName;
		FProperty* ResolvedProperty = UEDiffUtils_Private::Resolve(ContainerStruct, PropertyIdentifier);

		// if property wasn't found in ContainerStruct, check for it in SparseClassData
		if (!ResolvedProperty && RootTypeHint && RootTypeHint != ContainerStruct)
		{
			if (const UClass* AsClass = Cast<UClass>(ContainerStruct))
			{
				const UScriptStruct* SparseClassDataStruct = AsClass->GetSparseClassDataStruct();
				if (SparseClassDataStruct && SparseClassDataStruct->IsChildOf(RootTypeHint))
				{
					ResolvedProperty = UEDiffUtils_Private::Resolve(RootTypeHint, PropertyIdentifier);
					// return if null won't mutate...
					ContainerAddress = const_cast<UClass*>(AsClass)->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull);
				}
			}
		}

		// If the property didn't exist inside the container, return an invalid property
		if (!ResolvedProperty)
		{
			return FPropertyPath();
		}
		
		FPropertyInfo Info(ResolvedProperty, INDEX_NONE);
		Ret.AddProperty(Info);

		int32 PropertyIndex = TryReadIndex(PropertyChain, I);
		
		
		// calculate offset so we can continue resolving object properties/structproperties:
		if( const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty) )
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<const void*>( ContainerAddress ));
			if (ArrayHelper.IsValidIndex(PropertyIndex))
			{
				UpdateContainerAddress( ArrayProperty->Inner, ArrayHelper.GetRawPtr(PropertyIndex), ContainerAddress, ContainerStruct );

				FPropertyInfo ArrayInfo(ArrayProperty->Inner, PropertyIndex);
				Ret.AddProperty(ArrayInfo);
			}
		}
		else if( const FSetProperty* SetProperty = CastField<FSetProperty>(ResolvedProperty) )
		{
			FScriptSetHelper SetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<const void*>( ContainerAddress ));
			if (SetHelper.IsValidIndex(PropertyIndex))
			{
				const int32 InternalIndex = SetHelper.FindInternalIndex(PropertyIndex);
				UpdateContainerAddress( SetProperty->ElementProp, SetHelper.GetElementPtr(InternalIndex), ContainerAddress, ContainerStruct );

				FPropertyInfo SetInfo(SetProperty->ElementProp, PropertyIndex);
				Ret.AddProperty(SetInfo);
			}
		}
		else if( const FMapProperty* MapProperty = CastField<FMapProperty>(ResolvedProperty) )
		{
			FScriptMapHelper MapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<const void*>( ContainerAddress ));
			if (MapHelper.IsValidIndex(PropertyIndex))
			{
				const int32 InternalIndex = MapHelper.FindInternalIndex(PropertyIndex);
				// we have an index, but are we looking into a key or value? Peek ahead to find out:
				if(ensure((I + 1 < PropertyChain.Num())))
				{
					if(PropertyChain[I+1].PropertyName == MapProperty->KeyProp->GetFName())
					{
						++I;

						UpdateContainerAddress( MapProperty->KeyProp, MapHelper.GetKeyPtr(InternalIndex), ContainerAddress, ContainerStruct );

						FPropertyInfo MakKeyInfo(MapProperty->KeyProp, PropertyIndex);
						Ret.AddProperty(MakKeyInfo);
					}
					else if(ensure( PropertyChain[I+1].PropertyName == MapProperty->ValueProp->GetFName() ))
					{
						++I;

						UpdateContainerAddress( MapProperty->ValueProp, MapHelper.GetValuePtr(InternalIndex), ContainerAddress, ContainerStruct );

						FPropertyInfo MapValueInfo(MapProperty->ValueProp, PropertyIndex);
						Ret.AddProperty(MapValueInfo);
					}
				}
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ResolvedProperty))
		{
			UpdateContainerAddress( ObjectProperty, ObjectProperty->ContainerPtrToValuePtr<const void*>( ContainerAddress, FMath::Max(PropertyIndex, 0) ), ContainerAddress, ContainerStruct );
			
			// handle static arrays:
			if(PropertyIndex != INDEX_NONE )
			{
				FPropertyInfo ObjectInfo(ResolvedProperty, PropertyIndex);
				Ret.AddProperty(ObjectInfo);
			}
		}
		else if( const FStructProperty* StructProperty = CastField<FStructProperty>(ResolvedProperty) )
		{
			UpdateContainerAddress( StructProperty, StructProperty->ContainerPtrToValuePtr<const void*>( ContainerAddress, FMath::Max(PropertyIndex, 0) ), ContainerAddress, ContainerStruct );
			
			// handle static arrays:
			if(PropertyIndex != INDEX_NONE )
			{
				FPropertyInfo StructInfo(ResolvedProperty, PropertyIndex);
				Ret.AddProperty(StructInfo);
			}
		}
		else
		{
			// handle static arrays:
			if(PropertyIndex != INDEX_NONE )
			{
				FPropertyInfo StaticArrayInfo(ResolvedProperty, PropertyIndex);
				Ret.AddProperty(StaticArrayInfo);
			}
		}
	}
	return Ret;
}

FString FPropertySoftPath::ToDisplayName(const int32 NumberOfElements /*= INDEX_NONE*/) const
{
	FString Ret;
	const int32 Count = NumberOfElements != INDEX_NONE
		? FMath::Clamp(NumberOfElements, 1, PropertyChain.Num())
		: PropertyChain.Num();
	for (int32 i = PropertyChain.Num() - Count; i < PropertyChain.Num(); i++)
	{
		FString PropertyAsString = PropertyChain[i].DisplayString;
		if (Ret.IsEmpty())
		{
			Ret.Append(PropertyAsString);
		}
		else if (PropertyAsString.IsNumeric())
		{
			Ret.AppendChar('[');
			Ret.Append(PropertyAsString);
			Ret.AppendChar(']');
		}
		else
		{
			Ret.AppendChar(' ');
			Ret.Append(PropertyAsString);
		}
	}
	return Ret;
}

FPropertySoftPath FPropertySoftPath::GetRootProperty(int32 Depth) const
{
	FPropertySoftPath RootPath;
	Depth = FMath::Clamp(Depth, 1, PropertyChain.Num());
	for (int32 ElementIndex = 0; ElementIndex < Depth; ElementIndex++)
	{
		RootPath.PropertyChain.Push(PropertyChain[ElementIndex]);
	}
	return RootPath;
}

int32 FPropertySoftPath::TryReadIndex(int32 Index) const
{
	return TryReadIndex(PropertyChain, Index);
};

const UObject* DiffUtils::GetCDO(const UBlueprint* ForBlueprint)
{
	if (!ForBlueprint
		|| !ForBlueprint->GeneratedClass)
	{
		return NULL;
	}

	return ForBlueprint->GeneratedClass->GetDefaultObject(false);
}

void DiffUtils::CompareUnrelatedStructs(const UStruct* StructA, const void* A, const UStruct* StructB, const void* B,
	TArray<FSingleObjectDiffEntry>& OutDifferingProperties)
{
	CompareUnrelatedStructs(StructA, A, nullptr, StructB, B, nullptr, OutDifferingProperties);
}

void DiffUtils::CompareUnrelatedStructs(const UStruct* StructA, const void* A, const UObject* OwningOuterA, const UStruct* StructB, const void* B,
                                        const UObject* OwningOuterB, TArray<FSingleObjectDiffEntry>& OutDifferingProperties)
{
	FPropertySoftPathSet PropertiesInA = UEDiffUtils_Private::GetPropertyNameSet(StructA);
	FPropertySoftPathSet PropertiesInB = UEDiffUtils_Private::GetPropertyNameSet(StructB);

	// any properties in A that aren't in B are differing:
	auto AddedToA = PropertiesInA.Difference(PropertiesInB).Array();
	for (const auto& Entry : AddedToA)
	{
		OutDifferingProperties.Push(FSingleObjectDiffEntry(Entry, EPropertyDiffType::PropertyAddedToA));
	}

	// and the converse:
	auto AddedToB = PropertiesInB.Difference(PropertiesInA).Array();
	for (const auto& Entry : AddedToB)
	{
		OutDifferingProperties.Push(FSingleObjectDiffEntry(Entry, EPropertyDiffType::PropertyAddedToB));
	}

	// for properties in common, dig out the uproperties and determine if they're identical:
	if (A && B)
	{
		FPropertySoftPathSet Common = PropertiesInA.Intersect(PropertiesInB);
		for (const auto& PropertyName : Common)
		{
			FResolvedProperty AProp = PropertyName.Resolve(StructA, A);
			FResolvedProperty BProp = PropertyName.Resolve(StructB, B);

			check(AProp != FResolvedProperty() && BProp != FResolvedProperty());
			TArray<FPropertySoftPath> DifferingSubProperties;
			if (!DiffUtils::Identical(AProp, BProp, OwningOuterA, OwningOuterB, FDiffParameters(PropertyName), DifferingSubProperties))
			{
				for (int DifferingIndex = 0; DifferingIndex < DifferingSubProperties.Num(); DifferingIndex++)
				{
					OutDifferingProperties.Push(FSingleObjectDiffEntry(DifferingSubProperties[DifferingIndex], EPropertyDiffType::PropertyValueChanged));
				}
			}
		}
	}
}

void DiffUtils::CompareUnrelatedObjects(const UObject* A, const UObject* B, TArray<FSingleObjectDiffEntry>& OutDifferingProperties)
{
	if (A && B)
	{
		return CompareUnrelatedStructs(A->GetClass(), A, A->GetPackage(), B->GetClass(), B, B->GetPackage(), OutDifferingProperties);
	}
}

void DiffUtils::CompareUnrelatedSCS(const UBlueprint* Old, const TArray< FSCSResolvedIdentifier >& OldHierarchy, const UBlueprint* New, const TArray< FSCSResolvedIdentifier >& NewHierarchy, FSCSDiffRoot& OutDifferingEntries )
{
	const auto FindEntry = [](TArray< FSCSResolvedIdentifier > const& InArray, const FSCSIdentifier* Value) -> const FSCSResolvedIdentifier*
	{
		const FSCSResolvedIdentifier* BestMatch = nullptr;

		for (const auto& Node : InArray)
		{
			if (Node.Identifier.Name == Value->Name)
			{
				if (Node.Identifier.TreeLocation == Value->TreeLocation)
				{
					return &Node;
				}
				else if (BestMatch == nullptr)
				{
					BestMatch = &Node;
				}
			}
		}
		return BestMatch;
	};

	for (const auto& OldNode : OldHierarchy)
	{
		const FSCSResolvedIdentifier* NewEntry = FindEntry(NewHierarchy, &OldNode.Identifier);

		if (NewEntry != nullptr)
		{
			bool bShouldDiffProperties = true;

			// did it change class?
			const bool bObjectTypesDiffer = OldNode.Object != nullptr && NewEntry->Object != nullptr && OldNode.Object->GetClass() != NewEntry->Object->GetClass();
			if (bObjectTypesDiffer)
			{
				FSCSDiffEntry Diff = { OldNode.Identifier, ETreeDiffType::NODE_TYPE_CHANGED, FSingleObjectDiffEntry() };
				OutDifferingEntries.Entries.Push(Diff);

				// Only diff properties if we're still within the same class inheritance hierarchy.
				bShouldDiffProperties = OldNode.Object->GetClass()->IsChildOf(NewEntry->Object->GetClass()) || NewEntry->Object->GetClass()->IsChildOf(OldNode.Object->GetClass());
			}

			// did a property change?
			if(bShouldDiffProperties)
			{
				TArray<FSingleObjectDiffEntry> DifferingProperties;
				DiffUtils::CompareUnrelatedObjects(OldNode.Object, NewEntry->Object, DifferingProperties);
				for (const auto& Property : DifferingProperties)
				{
					// Only include property value change entries if the object types differ.
					if (!bObjectTypesDiffer || Property.DiffType == EPropertyDiffType::PropertyValueChanged)
					{
						FSCSDiffEntry Diff = { OldNode.Identifier, ETreeDiffType::NODE_PROPERTY_CHANGED, Property };
						OutDifferingEntries.Entries.Push(Diff);
					}
				}
			}

			// did it move?
			if( NewEntry->Identifier.TreeLocation != OldNode.Identifier.TreeLocation )
			{
				FSCSDiffEntry Diff = { OldNode.Identifier, ETreeDiffType::NODE_MOVED, FSingleObjectDiffEntry() };
				OutDifferingEntries.Entries.Push(Diff);
			}

			// did it become corrupted? or stop being corrupted?
			const bool bNewIsCorrupt = New->GeneratedClass && NewEntry->Object && !NewEntry->Object->IsIn(New->GeneratedClass);
			const bool bOldIsCorrupt = Old->GeneratedClass && OldNode.Object && !OldNode.Object->IsIn(Old->GeneratedClass);
			if (bNewIsCorrupt != bOldIsCorrupt)
			{
				if (bNewIsCorrupt)
				{
					FSCSDiffEntry Diff = { NewEntry->Identifier, ETreeDiffType::NODE_CORRUPTED, FSingleObjectDiffEntry() };
					OutDifferingEntries.Entries.Push(Diff);
				}
				else
				{
					FSCSDiffEntry Diff = { OldNode.Identifier, ETreeDiffType::NODE_FIXED, FSingleObjectDiffEntry() };
					OutDifferingEntries.Entries.Push(Diff);
				}
			}

			// no change! Do nothing.
		}
		else
		{
			// not found in the new data, must have been deleted:
			FSCSDiffEntry Entry = { OldNode.Identifier, ETreeDiffType::NODE_REMOVED, FSingleObjectDiffEntry() };
			OutDifferingEntries.Entries.Push( Entry );
		}
	}

	for (const auto& NewNode : NewHierarchy)
	{
		const FSCSResolvedIdentifier* OldEntry = FindEntry(OldHierarchy, &NewNode.Identifier);

		if (OldEntry == nullptr)
		{
			FSCSDiffEntry Entry = { NewNode.Identifier, ETreeDiffType::NODE_ADDED, FSingleObjectDiffEntry() };
			OutDifferingEntries.Entries.Push( Entry );
		}
	}
}

static void IdenticalHelper(const FProperty* AProperty, const FProperty* BProperty, const void* AValue, const void* BValue,
	const UObject* OwningOuterA, const UObject* OwningOuterB, DiffUtils::FDiffParameters DiffParameters,
	TArray<FPropertySoftPath>& DifferingSubProperties)
{
	if (AProperty == nullptr || BProperty == nullptr || AProperty->ArrayDim != BProperty->ArrayDim || AProperty->GetClass() != BProperty->GetClass())
	{
		DifferingSubProperties.Push(DiffParameters.RootPath);
		return;
	}

	if (AValue == nullptr || BValue == nullptr)
	{
		if (AValue != BValue)
		{
			DifferingSubProperties.Push(DiffParameters.RootPath);
		}
		return;
	}

	if (DiffParameters.ShouldIgnorePropertyPredicate
		&& DiffParameters.ShouldIgnorePropertyPredicate(*AProperty))
	{
		return;
	}

	// Keep copy of the initial RootPath since we reuse DiffParameters with
	// an updated path to nested calls to 'IdenticalHelper'
	const FPropertySoftPath RootPath = DiffParameters.RootPath;

	if (DiffParameters.bShouldDiffArrayElements && AProperty->ArrayDim != 1)
	{
		// Identical does not handle static array case automatically and we have to do the offset calculation ourself because 
		// our container (e.g. the struct or class or dynamic array) has already done the initial offset calculation:
		for (int32 I = 0; I < AProperty->ArrayDim; ++I)
		{
			int32 Offset = AProperty->GetElementSize() * I;

			DiffParameters.RootPath = FPropertySoftPath(RootPath, I);
			DiffParameters.bShouldDiffArrayElements = false;

			const void* CurAValue = reinterpret_cast<const void*>(reinterpret_cast<const uint8*>(AValue) + Offset);
			const void* CurBValue = reinterpret_cast<const void*>(reinterpret_cast<const uint8*>(BValue) + Offset);

			IdenticalHelper(AProperty, BProperty, CurAValue, CurBValue, OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
		}

		return;
	}
	
	const FStructProperty* APropAsStruct = CastField<FStructProperty>(AProperty);
	const FArrayProperty* APropAsArray = CastField<FArrayProperty>(AProperty);
	const FSetProperty* APropAsSet = CastField<FSetProperty>(AProperty);
	const FMapProperty* APropAsMap = CastField<FMapProperty>(AProperty);
	const FObjectProperty* APropAsObject = CastField<FObjectProperty>(AProperty);
	if (APropAsStruct != nullptr)
	{
		const FStructProperty* BPropAsStruct = CastFieldChecked<FStructProperty>(const_cast<FProperty*>(BProperty));
		if (BPropAsStruct->Struct == APropAsStruct->Struct)
		{
			if (APropAsStruct->Struct->StructFlags & STRUCT_IdenticalNative)
			{
				// If the struct uses CPP identical tests then we need to honor that
				if (!AProperty->Identical(AValue, BValue, PPF_DeepComparison))
				{
					DifferingSubProperties.Push(RootPath);
				}
			}
			else
			{
				// Compare sub-properties to detect more granular changes
				for (TFieldIterator<FProperty> PropertyIt(APropAsStruct->Struct); PropertyIt; ++PropertyIt)
				{
					const FProperty* StructProp = *PropertyIt;

					DiffParameters.RootPath = FPropertySoftPath(RootPath, StructProp);

					const void* SubValueA = StructProp->ContainerPtrToValuePtr<void>(AValue, 0);
					const void* SubValueB = StructProp->ContainerPtrToValuePtr<void>(BValue, 0);
					IdenticalHelper(StructProp, StructProp, SubValueA, SubValueB,
						OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
				}
			}
		}
		else
		{
			DifferingSubProperties.Push(RootPath);
		}
	}
	else if (APropAsArray != nullptr)
	{
		const FArrayProperty* BPropAsArray = CastFieldChecked<const FArrayProperty>(BProperty);
		if(BPropAsArray->Inner->GetClass() == APropAsArray->Inner->GetClass())
		{
			FScriptArrayHelper ArrayHelperA(APropAsArray, AValue);
			FScriptArrayHelper ArrayHelperB(BPropAsArray, BValue);
		
			// note any differences in contained types:
			for (int32 ArrayIndex = 0; ArrayIndex < ArrayHelperA.Num() && ArrayIndex < ArrayHelperB.Num(); ArrayIndex++)
			{
				DiffParameters.RootPath = FPropertySoftPath(RootPath, ArrayIndex);

				const void* SubValueA = ArrayHelperA.GetRawPtr(ArrayIndex);
				const void* SubValueB = ArrayHelperB.GetRawPtr(ArrayIndex);
				IdenticalHelper(APropAsArray->Inner, BPropAsArray->Inner, SubValueA, SubValueB,
					OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
			}

			// note any size difference:
			if (ArrayHelperA.Num() != ArrayHelperB.Num())
			{
				DifferingSubProperties.Push(RootPath);
			}
		}
		else
		{
			DifferingSubProperties.Push(RootPath);
		}
	}
	else if(APropAsSet != nullptr)
	{
		const FSetProperty* BPropAsSet = CastFieldChecked<const FSetProperty>(BProperty);
		if(BPropAsSet->ElementProp->GetClass() == APropAsSet->ElementProp->GetClass())
		{
			FScriptSetHelper SetHelperA(APropAsSet, AValue);
			FScriptSetHelper SetHelperB(BPropAsSet, BValue);

			if (SetHelperA.Num() != SetHelperB.Num())
			{
				// API not robust enough to indicate changes made to # of set elements, would
				// need to return something more detailed than DifferingSubProperties array:
				DifferingSubProperties.Push(RootPath);
			}

			// note any differences in contained elements:
			FScriptSetHelper::FIterator IteratorA(SetHelperA);
			FScriptSetHelper::FIterator IteratorB(SetHelperB);
			for (; IteratorA && IteratorB; ++IteratorA, ++IteratorB)
			{
				DiffParameters.RootPath = FPropertySoftPath(RootPath, IteratorA.GetLogicalIndex());

				const void* SubValueA = SetHelperA.GetElementPtr(IteratorA);
				const void* SubValueB = SetHelperB.GetElementPtr(IteratorB);
				IdenticalHelper(APropAsSet->ElementProp, BPropAsSet->ElementProp, SubValueA, SubValueB,
					OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
			}
		}
		else
		{
			DifferingSubProperties.Push(RootPath);
		}
	}
	else if(APropAsMap != nullptr)
	{
		const FMapProperty* BPropAsMap = CastFieldChecked<const FMapProperty>(BProperty);
		if(APropAsMap->KeyProp->GetClass() == BPropAsMap->KeyProp->GetClass() && APropAsMap->ValueProp->GetClass() == BPropAsMap->ValueProp->GetClass())
		{
			FScriptMapHelper MapHelperA(APropAsMap, AValue);
			FScriptMapHelper MapHelperB(BPropAsMap, BValue);
			if (MapHelperA.Num() != MapHelperB.Num())
			{
				// API not robust enough to indicate changes made to # of set elements, would
				// need to return something more detailed than DifferingSubProperties array:
				DifferingSubProperties.Push(RootPath);
			}

			FScriptMapHelper::FIterator IteratorA(MapHelperA);
			FScriptMapHelper::FIterator IteratorB(MapHelperB);
			for (; IteratorA && IteratorB; ++IteratorA, ++IteratorB)
			{
				const FPropertySoftPath EntryIndexPath = FPropertySoftPath(RootPath, IteratorA.GetLogicalIndex());

				DiffParameters.RootPath = FPropertySoftPath(EntryIndexPath, BPropAsMap->KeyProp);
				IdenticalHelper(APropAsMap->KeyProp, BPropAsMap->KeyProp, MapHelperA.GetKeyPtr(IteratorA), MapHelperB.GetKeyPtr(IteratorB),
					OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
					
				DiffParameters.RootPath = FPropertySoftPath(EntryIndexPath, BPropAsMap->ValueProp);
				IdenticalHelper(APropAsMap->ValueProp, BPropAsMap->ValueProp, MapHelperA.GetValuePtr(IteratorA), MapHelperB.GetValuePtr(IteratorB),
					OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
			}
		}
		else
		{
			DifferingSubProperties.Push(RootPath);
		}
	}
	else if(APropAsObject != nullptr)
	{
		// Past container check, do a normal identical check now before going into components
		if (AProperty->Identical(AValue, BValue, PPF_DeepComparison))
		{
			return;
		}

		const FObjectProperty* BPropAsObject = CastFieldChecked<const FObjectProperty>(BProperty);

		const UObject* A = *((const UObject* const*)AValue);
		const UObject* B = *((const UObject* const*)BValue);

		// dig into the objects if they are in the same package as our initial object
		if (A && B && A->GetClass() == B->GetClass() && OwningOuterA && OwningOuterB && A->IsIn(OwningOuterA) && B->IsIn(OwningOuterB))
		{
			const UClass* AClass = A->GetClass(); // BClass and AClass are identical!

			// We only want to recurse if this is EditInlineNew and not a component
			// Other instanced refs are likely to form a type-specific web so recursion doesn't make sense and won't be displayed properly in the details pane
			if (AClass->HasAnyClassFlags(CLASS_EditInlineNew) && !AClass->IsChildOf(UActorComponent::StaticClass()))
			{
				for (TFieldIterator<FProperty> PropertyIt(AClass); PropertyIt; ++PropertyIt)
				{
					const FProperty* ClassProp = *PropertyIt;
					DiffParameters.RootPath = FPropertySoftPath(RootPath, ClassProp);

					const void* SubValueA = ClassProp->ContainerPtrToValuePtr<void>(A, 0);
					const void* SubValueB = ClassProp->ContainerPtrToValuePtr<void>(B, 0);
					IdenticalHelper(ClassProp, ClassProp, SubValueA, SubValueB,
						OwningOuterA, OwningOuterB, DiffParameters, DifferingSubProperties);
				}
			}
			else
			{
				const FString PathA = A->GetPathName(OwningOuterA);
				const FString PathB = B->GetPathName(OwningOuterB);
				if (PathA != PathB)
				{
					DifferingSubProperties.Push(RootPath);
				}
			}
		}
		else
		{
			DifferingSubProperties.Push(RootPath);
		}
	}
	else
	{
		// Passed all container tests that would check for nested properties being wrong
		if (AProperty->Identical(AValue, BValue, PPF_DeepComparison))
		{
			return;
		}

		DifferingSubProperties.Push(RootPath);
	}
}

bool DiffUtils::Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const FPropertySoftPath& RootPath, TArray<FPropertySoftPath>& DifferingProperties)
{
	if( AProp.Property == nullptr && BProp.Property == nullptr )
	{
		return true;
	}
	else if( AProp.Property == nullptr || BProp.Property == nullptr )
	{
		return false;
	}

	const void* AValue = AProp.Property->ContainerPtrToValuePtr<void>(AProp.Object);
	const void* BValue = BProp.Property->ContainerPtrToValuePtr<void>(BProp.Object);

	// We _could_ just ask the property for comparison but that would make the "identical" functions significantly more complex.
	// Instead let's write a new function, specific to DiffUtils, that handles the sub properties
	// NOTE: For Static Arrays, AValue and BValue were, and are, only references to the value at index 0.  So changes to values past index 0 didn't show up before and
	// won't show up now.  Changes to index 0 will show up as a change to the entire array.
	IdenticalHelper(AProp.Property, BProp.Property, AValue, BValue, nullptr, nullptr, FDiffParameters(RootPath), DifferingProperties);
	
	return DifferingProperties.Num() == 0;
}

bool DiffUtils::Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA,
	const UObject* OwningOuterB)
{
	TArray<FPropertySoftPath> DifferingProperties;
	return Identical(AProp, BProp, OwningOuterA, OwningOuterB, FDiffParameters{}, DifferingProperties);
}

bool DiffUtils::Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA,
	const UObject* OwningOuterB, const FPropertySoftPath& RootPath, TArray<FPropertySoftPath>& DifferingProperties)
{
	return Identical(AProp, BProp, OwningOuterA, OwningOuterB, FDiffParameters(RootPath), DifferingProperties);
}

bool DiffUtils::Identical(const FResolvedProperty& AProp, const FResolvedProperty& BProp, const UObject* OwningOuterA,
	const UObject* OwningOuterB, FDiffParameters DiffParameters, TArray<FPropertySoftPath>& DifferingProperties)
{
	if( AProp.Property == nullptr && BProp.Property == nullptr )
	{
		return true;
	}
	else if( AProp.Property == nullptr || BProp.Property == nullptr )
	{
		return false;
	}

	const void* AValue = AProp.Property->ContainerPtrToValuePtr<void>(AProp.Object);
	const void* BValue = BProp.Property->ContainerPtrToValuePtr<void>(BProp.Object);

	// note that we're not directly calling FProperty::Identical because sub-object properties should be weakly compared based on
	// their paths instead of their pointers or data
	IdenticalHelper(AProp.Property, BProp.Property, AValue, BValue, OwningOuterA, OwningOuterB, DiffParameters, DifferingProperties);
	
	return DifferingProperties.Num() == 0;
}

bool DiffUtils::Identical(const TSharedPtr<IPropertyHandle>& PropertyHandleA, const TSharedPtr<IPropertyHandle>& PropertyHandleB,
                          const TArray<TWeakObjectPtr<UObject>>& OwningOutersA, const TArray<TWeakObjectPtr<UObject>>& OwningOutersB)
{
	TArray<FPropertySoftPath> DifferingProperties;
	return Identical(DifferingProperties, PropertyHandleA, PropertyHandleB, OwningOutersA, OwningOutersB);
}

bool DiffUtils::Identical(
	TArray<FPropertySoftPath>& OutDifferingProperties,
	const TSharedPtr<IPropertyHandle>& PropertyHandleA,
	const TSharedPtr<IPropertyHandle>& PropertyHandleB,
	const TArray<TWeakObjectPtr<UObject>>& OwningOutersA,
	const TArray<TWeakObjectPtr<UObject>>& OwningOutersB
	)
{
	TArray<void*> ValuesA;
	TArray<void*> ValuesB;
	PropertyHandleA->AccessRawData(ValuesA);
	PropertyHandleB->AccessRawData(ValuesB);

	// if OwningOuters weren't provided, fallback to using the property handles to find them
	TArray<UObject*> HandleOutersA;
	TArray<UObject*> HandleOutersB;
	if (OwningOutersA.IsEmpty())
	{
		PropertyHandleA->GetOuterObjects(HandleOutersA);
	}
	if (OwningOutersB.IsEmpty())
	{
		PropertyHandleB->GetOuterObjects(HandleOutersB);
	}
	
	if (!ensure(ValuesA.Num() == OwningOutersA.Num()))
	{
		// Outer count mismatch
		return false;
	}
	if (!ensure(ValuesB.Num() == OwningOutersB.Num()))
	{
		// Outer count mismatch
		return false;
	}

	auto IsIdenticalAtIndex = [&](int32 IndexA, int32 IndexB)
	{
		const void* ValueA = ValuesA[IndexA];
		const void* ValueB = ValuesB[IndexB];

		const UObject* OwningOuterA = OwningOutersA.IsEmpty() ? HandleOutersA[IndexA] : OwningOutersA[IndexA].Get();
		const UObject* OwningOuterB = OwningOutersB.IsEmpty() ? HandleOutersB[IndexB] : OwningOutersB[IndexB].Get();

		if (!OwningOuterA || !OwningOuterB)
		{
			// objects were Garbage Collected!
			return !OwningOuterA && !OwningOuterB;
		}

		// note that we're not directly calling FProperty::Identical because sub-object properties should be weakly compared based on
		// their paths instead of their pointers or data
		FDiffParameters DiffParameters;
		DiffParameters.bShouldDiffArrayElements = false;

		IdenticalHelper(PropertyHandleA->GetProperty(), PropertyHandleB->GetProperty(), ValueA, ValueB,
			OwningOuterA, OwningOuterB, DiffParameters, OutDifferingProperties);

		return OutDifferingProperties.IsEmpty();
	};
	
	if (ValuesA.Num() == ValuesB.Num())
	{
		// compare AValues[I] with BValues[I]
		for (int32 I = 0; I < ValuesA.Num(); ++I)
		{
			if (!IsIdenticalAtIndex(I,I))
			{
				return false;
			}
		}
	}
	else if (ValuesA.Num() == 1)
	{
		// compare AValues[0] with BValues[0...N]
		for (int32 I = 0; I < ValuesB.Num(); ++I)
		{
			if (!IsIdenticalAtIndex(0,I))
			{
				return false;
			}
		}
	}
	else if (ValuesB.Num() == 1)
	{
		// compare BValues[0] with AValues[0...N]
		for (int32 I = 0; I < ValuesA.Num(); ++I)
		{
			if (!IsIdenticalAtIndex(I,0))
			{
				return false;
			}
		}
	}
	else
	{
		// number of values doesn't match... this cannot be compared
		return ensure(false);
	}
	
	return true;
}

TArray<FPropertySoftPath> DiffUtils::GetVisiblePropertiesInOrderDeclared(const UStruct* ForStruct, const FPropertySoftPath& Scope /*= TArray<FName>()*/)
{
	TArray<FPropertySoftPath> Ret;
	if (ForStruct)
	{
		TSet<FString> HiddenCategories = FEditorCategoryUtils::GetHiddenCategories(ForStruct);
		for (TFieldIterator<FProperty> PropertyIt(ForStruct); PropertyIt; ++PropertyIt)
		{
			FName CategoryName = FObjectEditorUtils::GetCategoryFName(*PropertyIt);
			if (!HiddenCategories.Contains(CategoryName.ToString()))
			{
				if (PropertyIt->PropertyFlags&CPF_Edit)
				{
					// We don't need to recurse into objects/structs as those will be picked up in the Identical check later
					FPropertySoftPath NewPath(Scope, *PropertyIt);
					Ret.Push(NewPath);
				}
			}
		}
	}
	return Ret;
}

TArray<FPropertyPath> DiffUtils::ResolveAll(const UObject* Object, const TArray<FPropertySoftPath>& InSoftProperties)
{
	TArray< FPropertyPath > Ret;
	for (const auto& Path : InSoftProperties)
	{
		Ret.Push(Path.ResolvePath(Object));
	}
	return Ret;
}

TArray<FPropertyPath> DiffUtils::ResolveAll(const UObject* Object, const TArray<FSingleObjectDiffEntry>& InDifferences)
{
	TArray< FPropertyPath > Ret;
	for (const auto& Difference : InDifferences)
	{
		Ret.Push(Difference.Identifier.ResolvePath(Object));
	}
	return Ret;
}

UPackage* DiffUtils::LoadPackageForDiff(const FPackagePath& InTempPackagePath, const FPackagePath& InOriginalPackagePath)
{
	// if this is a local asset, load it normally
	if (!FPackageName::IsTempPackage(InTempPackagePath.GetPackageName()))
	{
		return LoadPackage(nullptr, *InTempPackagePath.GetPackageName(), LOAD_None);
	}
	
	// set up instancing context
	FLinkerInstancingContext Context;
	if (!InOriginalPackagePath.GetLocalFullPath().IsEmpty())
	{
		Context.AddTag(ULevel::DontLoadExternalObjectsTag);
		Context.AddPackageMapping(InOriginalPackagePath.GetPackageFName(), InTempPackagePath.GetPackageFName());
	}
	
	return LoadPackage(nullptr, *InTempPackagePath.GetPackageName(),
		LOAD_ForDiff | LOAD_DisableCompileOnLoad | LOAD_DisableEngineVersionChecks, nullptr, &Context);
}

UPackage* DiffUtils::LoadPackageForDiff(TSharedPtr<ISourceControlRevision> Revision)
{
	FString TempFileName;
	if(Revision->Get(TempFileName))
	{
		// Try and load that package
		const FPackagePath TempPackagePath = FPackagePath::FromLocalPath(TempFileName);
		const FPackagePath OriginalPackagePath = FPackagePath::FromLocalPath(Revision->GetFilename());
		return LoadPackageForDiff(TempPackagePath, OriginalPackagePath);
	}
	return nullptr;
}

TSharedPtr<FBlueprintDifferenceTreeEntry> FBlueprintDifferenceTreeEntry::NoDifferencesEntry()
{
	// This just generates a widget that tells the user that no differences were detected. Without this
	// the treeview displaying differences is confusing when no differences are present because it is not obvious
	// that the control is a treeview (a treeview with no children looks like a listview).
	const auto GenerateWidget = []() -> TSharedRef<SWidget>
	{
		return SNew(STextBlock)
			.ColorAndOpacity(FLinearColor(.7f, .7f, .7f))
			.TextStyle(FAppStyle::Get(), TEXT("BlueprintDif.ItalicText"))
			.Text(NSLOCTEXT("FBlueprintDifferenceTreeEntry", "NoDifferencesLabel", "No differences detected..."));
	};

	return TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
		FOnDiffEntryFocused()
		, FGenerateDiffEntryWidget::CreateStatic(GenerateWidget)
		, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >()
	) );
}

TSharedPtr<FBlueprintDifferenceTreeEntry> FBlueprintDifferenceTreeEntry::UnknownDifferencesEntry()
{
	// Warn about there being unknown differences
	const auto GenerateWidget = []() -> TSharedRef<SWidget>
	{
		return SNew(STextBlock)
			.ColorAndOpacity(FLinearColor(.7f, .7f, .7f))
			.TextStyle(FAppStyle::Get(), TEXT("BlueprintDif.ItalicText"))
			.Text(NSLOCTEXT("FBlueprintDifferenceTreeEntry", "BlueprintTypeNotSupported", "Warning: Detecting differences in this Blueprint type specific data is not yet supported..."));
	};

	return TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
		FOnDiffEntryFocused()
		, FGenerateDiffEntryWidget::CreateStatic(GenerateWidget)
		, TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >()
	));
}

TSharedPtr<FBlueprintDifferenceTreeEntry> FBlueprintDifferenceTreeEntry::CreateCategoryEntry(const FText& LabelText, const FText& ToolTipText, FOnDiffEntryFocused FocusCallback, const TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& Children, bool bHasDifferences)
{
	const auto CreateDefaultsRootEntry = [](FText LabelText, FText ToolTipText, FLinearColor Color) -> TSharedRef<SWidget>
	{
		return SNew(STextBlock)
			.ToolTipText(ToolTipText)
			.ColorAndOpacity(Color)
			.Text(LabelText);
	};

	return TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
		FocusCallback
		, FGenerateDiffEntryWidget::CreateStatic(CreateDefaultsRootEntry, LabelText, ToolTipText, DiffViewUtils::LookupColor(bHasDifferences))
		, Children
	));
}

TSharedPtr<FBlueprintDifferenceTreeEntry> FBlueprintDifferenceTreeEntry::CreateCategoryEntryForMerge(const FText& LabelText, const FText& ToolTipText, FOnDiffEntryFocused FocusCallback, const TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >& Children, bool bHasRemoteDifferences, bool bHasLocalDifferences, bool bHasConflicts)
{
	const auto CreateDefaultsRootEntry = [](FText LabelText, FText ToolTipText, bool bInHasRemoteDifferences, bool bInHasLocalDifferences, bool bInHasConflicts) -> TSharedRef<SWidget>
	{
		const FLinearColor BaseColor = DiffViewUtils::LookupColor(bInHasRemoteDifferences || bInHasLocalDifferences, bInHasConflicts);
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.ToolTipText(ToolTipText)
				.ColorAndOpacity(BaseColor)
				.Text(LabelText)
			]
			+ DiffViewUtils::Box(true, DiffViewUtils::LookupColor(bInHasRemoteDifferences, bInHasConflicts))
			+ DiffViewUtils::Box(true, BaseColor)
			+ DiffViewUtils::Box(true, DiffViewUtils::LookupColor(bInHasLocalDifferences, bInHasConflicts));
	};

	return TSharedPtr<FBlueprintDifferenceTreeEntry>(new FBlueprintDifferenceTreeEntry(
		FocusCallback
		, FGenerateDiffEntryWidget::CreateStatic(CreateDefaultsRootEntry, LabelText, ToolTipText, bHasRemoteDifferences, bHasLocalDifferences, bHasConflicts)
		, Children
	));
}

TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > DiffTreeView::CreateTreeView(TArray< TSharedPtr<FBlueprintDifferenceTreeEntry> >* DifferencesList)
{
	const auto RowGenerator = [](TSharedPtr< FBlueprintDifferenceTreeEntry > Entry, const TSharedRef<STableViewBase>& Owner) -> TSharedRef< ITableRow >
	{
		return SNew(STableRow<TSharedPtr<FBlueprintDifferenceTreeEntry> >, Owner)
			[
				Entry->GenerateWidget.Execute()
			];
	};

	const auto ChildrenAccessor = [](TSharedPtr<FBlueprintDifferenceTreeEntry> InTreeItem, TArray< TSharedPtr< FBlueprintDifferenceTreeEntry > >& OutChildren)
	{
		OutChildren = InTreeItem->Children;
	};

	const auto Selector = [](TSharedPtr<FBlueprintDifferenceTreeEntry> InTreeItem, ESelectInfo::Type Type)
	{
		if (InTreeItem.IsValid())
		{
			InTreeItem->OnFocus.ExecuteIfBound();
		}
	};

	return SNew(STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > >)
		.OnGenerateRow(STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > >::FOnGenerateRow::CreateStatic(RowGenerator))
		.OnGetChildren(STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > >::FOnGetChildren::CreateStatic(ChildrenAccessor))
		.OnSelectionChanged(STreeView< TSharedPtr< FBlueprintDifferenceTreeEntry > >::FOnSelectionChanged::CreateStatic(Selector))
		.TreeItemsSource(DifferencesList);
}

int32 DiffTreeView::CurrentDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences)
{
	auto SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return INDEX_NONE;
	}

	for (int32 Iter = 0; Iter < SelectedItems.Num(); ++Iter)
	{
		int32 Index = Differences.Find(SelectedItems[Iter]);
		if (Index != INDEX_NONE)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

void DiffTreeView::HighlightNextDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& RootDifferences)
{
	int32 CurrentIndex = CurrentDifference(TreeView, Differences);

	auto Next = Differences[CurrentIndex + 1];
	// we have to manually expand our parent:
	for (auto& Test : RootDifferences)
	{
		if (Test->Children.Contains(Next))
		{
			TreeView->SetItemExpansion(Test, true);
			break;
		}
	}

	TreeView->SetSelection(Next);
	TreeView->RequestScrollIntoView(Next);
}

void DiffTreeView::HighlightPrevDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& RootDifferences)
{
	int32 CurrentIndex = CurrentDifference(TreeView, Differences);

	auto Prev = Differences[CurrentIndex - 1];
	// we have to manually expand our parent:
	for (auto& Test : RootDifferences)
	{
		if (Test->Children.Contains(Prev))
		{
			TreeView->SetItemExpansion(Test, true);
			break;
		}
	}

	TreeView->SetSelection(Prev);
	TreeView->RequestScrollIntoView(Prev);
}

bool DiffTreeView::HasNextDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences)
{
	int32 CurrentIndex = CurrentDifference(TreeView, Differences);
	return Differences.IsValidIndex(CurrentIndex + 1);
}

bool DiffTreeView::HasPrevDifference(TSharedRef< STreeView<TSharedPtr< FBlueprintDifferenceTreeEntry > > > TreeView, const TArray< TSharedPtr<class FBlueprintDifferenceTreeEntry> >& Differences)
{
	int32 CurrentIndex = CurrentDifference(TreeView, Differences);
	return Differences.IsValidIndex(CurrentIndex - 1);
}

FLinearColor DiffViewUtils::LookupColor(bool bDiffers, bool bConflicts)
{
	if( bConflicts )
	{
		return DiffViewUtils::Conflicting();
	}
	else if( bDiffers )
	{
		return DiffViewUtils::Differs();
	}
	else
	{
		return DiffViewUtils::Identical();
	}
}

FLinearColor DiffViewUtils::Differs()
{
	// yellow color
	return FLinearColor(0.85f,0.71f,0.25f);
}

FLinearColor DiffViewUtils::Identical()
{
	const static FLinearColor ForegroundColor = FAppStyle::GetColor("Graph.ForegroundColor");
	return ForegroundColor;
}

FLinearColor DiffViewUtils::Missing()
{
	// blue color
	return FLinearColor(0.3f,0.3f,1.f);
}

FLinearColor DiffViewUtils::Conflicting()
{
	// red color
	return FLinearColor(1.0f,0.2f,0.3f);
}

FText DiffViewUtils::PropertyDiffMessage(FSingleObjectDiffEntry Difference, FText ObjectName)
{
	FText Message;
	FString PropertyName = Difference.Identifier.ToDisplayName();
	switch (Difference.DiffType)
	{
	case EPropertyDiffType::PropertyAddedToA:
		Message = FText::Format(NSLOCTEXT("DiffViewUtils", "PropertyValueChange_Removed", "{0} removed from {1}"), FText::FromString(PropertyName), ObjectName);
		break;
	case EPropertyDiffType::PropertyAddedToB:
		Message = FText::Format(NSLOCTEXT("DiffViewUtils", "PropertyValueChange_Added", "{0} added to {1}"), FText::FromString(PropertyName), ObjectName);
		break;
	case EPropertyDiffType::PropertyValueChanged:
		Message = FText::Format(NSLOCTEXT("DiffViewUtils", "PropertyValueChange", "{0} changed value in {1}"), FText::FromString(PropertyName), ObjectName);
		break;
	}
	return Message;
}

FText DiffViewUtils::SCSDiffMessage(const FSCSDiffEntry& Difference, FText ObjectName)
{
	const FText NodeName = FText::FromName(Difference.TreeIdentifier.Name);
	FText Text;
	switch (Difference.DiffType)
	{
	case ETreeDiffType::NODE_ADDED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeAdded", "Added Node {0} to {1}"), NodeName, ObjectName);
		break;
	case ETreeDiffType::NODE_REMOVED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeRemoved", "Removed Node {0} from {1}"), NodeName, ObjectName);
		break;
	case ETreeDiffType::NODE_TYPE_CHANGED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeTypeChanged", "Node {0} changed type in {1}"), NodeName, ObjectName);
		break;
	case ETreeDiffType::NODE_PROPERTY_CHANGED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodePropertyChanged", "{0} on {1}"), DiffViewUtils::PropertyDiffMessage(Difference.PropertyDiff, NodeName), ObjectName);
		break;
	case ETreeDiffType::NODE_MOVED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeMoved", "Moved Node {0} in {1}"), NodeName, ObjectName);
		break;
	case ETreeDiffType::NODE_CORRUPTED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeCorrupted", "Node {0} in {1} has corrupt outer - must be recreated"), NodeName, ObjectName);
		break;
	case ETreeDiffType::NODE_FIXED:
		Text = FText::Format(NSLOCTEXT("DiffViewUtils", "NodeFixed", "Node {0} in {1} has been recreated to correct outer"), NodeName, ObjectName);
		break;
	}
	return Text;
}

FText DiffViewUtils::GetPanelLabel(const UObject* Asset, const FRevisionInfo& Revision, FText Label)
{
	if( !Asset )
	{
		return NSLOCTEXT("DiffViewUtils", "NoBlueprint", "None" );
	}
	
	if( !Revision.Revision.IsEmpty() )
	{
		FText RevisionData;
		
		if(ISourceControlModule::Get().GetProvider().UsesChangelists())
		{
			RevisionData = FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionData", "Revision {0} - CL {1} - {2}")
				, FText::FromString(Revision.Revision)
				, FText::AsNumber(Revision.Changelist, &FNumberFormattingOptions::DefaultNoGrouping())
				, FText::AsDateTime(Revision.Date));
		}
		else
		{
			RevisionData = FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionDataNoChangelist", "Revision {0} - {1}")
				, FText::FromString(Revision.Revision)
				, FText::AsDateTime(Revision.Date));
		}

		if (Label.IsEmpty())
		{
			return FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionLabelTwoLines", "{0}\n{1}")
				, FText::FromString(Asset->GetName())
				, RevisionData);
		}
		else
		{
			return FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionLabel", "{0}\n{1}\n{2}")
				, Label
				, FText::FromString(Asset->GetName())
				, RevisionData);
		}
	}
	else
	{
		if (Label.IsEmpty())
		{
			return FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionLabelTwoLines", "{0}\n{1}")
				, FText::FromString(Asset->GetName())
				, NSLOCTEXT("DiffViewUtils", "LocalRevisionLabel", "Local Revision"));
		}
		else
		{
			return FText::Format(NSLOCTEXT("DiffViewUtils", "RevisionLabel", "{0}\n{1}\n{2}")
				, Label
				, FText::FromString(Asset->GetName())
				, NSLOCTEXT("DiffViewUtils", "LocalRevisionLabel", "Local Revision"));
		}
	}
}

SHorizontalBox::FSlot::FSlotArguments DiffViewUtils::Box(bool bIsPresent, FLinearColor Color)
{
	return MoveTemp(SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(0.5f, 0.f)
		[
			SNew(SImage)
			.ColorAndOpacity(Color)
			.Image(bIsPresent ? FAppStyle::GetBrush("BlueprintDif.HasGraph") : FAppStyle::GetBrush("BlueprintDif.MissingGraph"))
		]);
};

