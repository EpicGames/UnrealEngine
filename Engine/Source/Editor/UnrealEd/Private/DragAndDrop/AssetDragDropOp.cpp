// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/AssetDragDropOp.h"
#include "Engine/Level.h"
#include "ActorFactories/ActorFactory.h"
#include "GameFramework/Actor.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBox.h"
#include "AssetThumbnail.h"
#include "ClassIconFinder.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/Level.h"
#include "ContentBrowserUtils.h"

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(const FAssetData& InAssetData, TScriptInterface<IAssetFactoryInterface> AssetFactory)
{
	TArray<FAssetData> AssetDataArray;
	AssetDataArray.Emplace(InAssetData);
	return New(MoveTemp(AssetDataArray), TArray<FString>(), AssetFactory);
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(TArray<FAssetData> InAssetData, TScriptInterface<IAssetFactoryInterface> AssetFactory)
{
	return New(MoveTemp(InAssetData), TArray<FString>(), AssetFactory);
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(FString InAssetPath)
{
	TArray<FString> AssetPathsArray;
	AssetPathsArray.Emplace(MoveTemp(InAssetPath));
	return New(TArray<FAssetData>(), MoveTemp(AssetPathsArray), nullptr);
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(TArray<FString> InAssetPaths)
{
	return New(TArray<FAssetData>(), MoveTemp(InAssetPaths), nullptr);
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, TScriptInterface<IAssetFactoryInterface> AssetFactory)
{
	TSharedRef<FAssetDragDropOp> Operation = MakeShared<FAssetDragDropOp>();

	Operation->Init(MoveTemp(InAssetData), MoveTemp(InAssetPaths), AssetFactory);

	Operation->Construct();
	return Operation;
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(const FAssetData& InAssetData, UActorFactory* ActorFactory)
{
	return New(InAssetData, TScriptInterface<IAssetFactoryInterface>(ActorFactory));
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(TArray<FAssetData> InAssetData, UActorFactory* ActorFactory)
{
	return New(InAssetData, TScriptInterface<IAssetFactoryInterface>(ActorFactory));
}

TSharedRef<FAssetDragDropOp> FAssetDragDropOp::New(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, UActorFactory* ActorFactory)
{
	return New(InAssetData, InAssetPaths, TScriptInterface<IAssetFactoryInterface>(ActorFactory));
}

UActorFactory* FAssetDragDropOp::GetActorFactory() const
{
	return Cast<UActorFactory>(AssetFactory.GetObject());
}

TScriptInterface<IAssetFactoryInterface> FAssetDragDropOp::GetAssetFactory() const
{
	return AssetFactory.GetObject();
}

FAssetDragDropOp::~FAssetDragDropOp()
{
}

TSharedPtr<SWidget> FAssetDragDropOp::GetDefaultDecorator() const
{
	const int32 TotalCount = GetTotalCount();

	TSharedPtr<SWidget> ThumbnailWidget;
	if (CustomThumbnailWidget.IsSet())
	{
		ThumbnailWidget = CustomThumbnailWidget.Get();
	}
	else if (AssetThumbnail.IsValid())
	{
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			FAssetThumbnailConfig AssetThumbnailConfig;
			AssetThumbnailConfig.ShowAssetColor = true;
			AssetThumbnailConfig.BorderPadding = FMargin(1.f);
			AssetThumbnailConfig.AssetBorderImageOverride = FAppStyle::GetBrush("ContentBrowser.ThumbnailDragDropBackground");
			ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig);
		}
		else
		{
			ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();
		}
	}
	else if (HasFolders())
	{
		ThumbnailWidget = SNew(SImage)
				.Image(FAppStyle::GetBrush("ContentBrowser.ListViewFolderIcon"))
				.ColorAndOpacity(FLinearColor::Gray);
	}
	else
	{
		ThumbnailWidget = 
			SNew(SImage)
			.Image(FAppStyle::GetDefaultBrush());
	}
	
	const FSlateBrush* SubTypeBrush = FAppStyle::GetDefaultBrush();
	FLinearColor SubTypeColor = FLinearColor::White;
	if (AssetThumbnail.IsValid() && HasFolders())
	{
		SubTypeBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderClosed");
		SubTypeColor = FLinearColor::Gray;
	}
	else if (AssetFactory.IsValid() && HasFiles())
	{
		// TODO: Probably need to add a function in IAssetFactoryInterface for this and use that.
		if (UActorFactory* ActorFactory = Cast<UActorFactory>(AssetFactory.GetObject()))
		{
			AActor* DefaultActor = ActorFactory->GetDefaultActor(AssetData[0]);
			SubTypeBrush = FClassIconFinder::FindIconForActor(DefaultActor);
		}
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		static const FMargin AssetNumberSlotMargin = FMargin(6, 0, 0, 6);
		static constexpr int32 OverlayMinOrFixedSize = 16.f;

		return SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.DragDropBackground"))
				.Padding(0.f)
				[
					SNew(SHorizontalBox)

					// Left slot is for the thumbnail
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					[
						SNew(SBox)
						.WidthOverride(ThumbnailSize)
						.HeightOverride(ThumbnailSize)
						[
							SNew(SOverlay)

							+SOverlay::Slot()
							[
								ThumbnailWidget.ToSharedRef()
							]

							+SOverlay::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Bottom)
							.Padding(AssetNumberSlotMargin)
							[
								SNew(SBox)
								.MinDesiredWidth(OverlayMinOrFixedSize)
								.HeightOverride(OverlayMinOrFixedSize)
								[
									SNew(SBorder)
									.BorderImage(FAppStyle::GetBrush("ContentBrowser.DragDropAssetNumbersBorder"))
									.Visibility(TotalCount > 1 ? EVisibility::Visible : EVisibility::Collapsed)
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.Font(FAppStyle::GetFontStyle("AssetThumbnail.StatusOverflowFont"))
										.Text(FText::Format(NSLOCTEXT("AssetDragDropOp", "AssetDragAndDropNumber", "+{0}"), FText::AsNumber(TotalCount)))
									]
								]
							]
						]
					]

					// Right slot is for optional tooltip
					+SHorizontalBox::Slot()
					.Padding(8.f, 0.f, 0.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.MinDesiredWidth(80.0f)
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0.f, 0.f, 4.0f, 0.f)
							.VAlign(VAlign_Center)
							[
								SNew(SImage) 
								.Image(this, &FAssetDragDropOp::GetIcon)
							]

							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0,0,8,0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.WrapTextAt(800.f)
								.Text(this, &FAssetDragDropOp::GetDecoratorText)
							]
						]
					]
				];
	}
	return 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetDragDropTooltipBackground"))
		.Content()
		[
			SNew(SHorizontalBox)

			// Left slot is for the thumbnail
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox) 
				.WidthOverride(static_cast<float>(ThumbnailSize)) 
				.HeightOverride(static_cast<float>(ThumbnailSize))
				.Content()
				[
					SNew(SOverlay)

					+SOverlay::Slot()
					[
						ThumbnailWidget.ToSharedRef()
					]

					+SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Top)
					.Padding(FMargin(0, 4, 0, 0))
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("Menu.Background"))
						.Visibility(TotalCount > 1 ? EVisibility::Visible : EVisibility::Collapsed)
						.Content()
						[
							SNew(STextBlock)
							.Text(FText::AsNumber(TotalCount))
						]
					]

					+SOverlay::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Bottom)
					.Padding(FMargin(4, 4))
					[
						SNew(SImage)
						.Image(SubTypeBrush)
						.Visibility(SubTypeBrush != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed)
						.ColorAndOpacity(SubTypeColor)
					]
				]
			]

			// Right slot is for optional tooltip
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(80.0f)
				.Content()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(3.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage) 
						.Image(this, &FAssetDragDropOp::GetIcon)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0,0,3,0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.WrapTextAt(800.f)
						.Text(this, &FAssetDragDropOp::GetDecoratorText)
					]
				]
			]
		];
}

FText FAssetDragDropOp::GetDecoratorText() const
{
	if (CurrentHoverText.IsEmpty())
	{
		const int32 TotalCount = GetTotalCount();
		if (TotalCount > 0)
		{
			const FText FirstItemText = GetFirstItemText();
			return (TotalCount == 1)
				? FirstItemText
				: FText::Format(NSLOCTEXT("ContentBrowser", "AssetDragDropOpDescriptionMulti", "'{0}' and {1} {1}|plural(one=other,other=others)"), FirstItemText, TotalCount - 1);
		}
	}
	
	return CurrentHoverText;
}

void FAssetDragDropOp::Init(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, TScriptInterface<IAssetFactoryInterface> InAssetFactory)
{
	MouseCursor = EMouseCursor::GrabHandClosed;
	ThumbnailSize = 64;

	AssetData = MoveTemp(InAssetData);
	AssetPaths = MoveTemp(InAssetPaths);
	AssetFactory = InAssetFactory;

	// Load all assets first so that there is no loading going on while attempting to drag
	// Can cause unsafe frame reentry 
	for (FAssetData& Data : AssetData)
	{
		Data.GetAsset({ ULevel::LoadAllExternalObjectsTag });
	}

	InitThumbnail();
}

void FAssetDragDropOp::Init(TArray<FAssetData> InAssetData, TArray<FString> InAssetPaths, UActorFactory* InActorFactory)
{
	Init(InAssetData, InAssetPaths, TScriptInterface<IAssetFactoryInterface>(InActorFactory));
}

void FAssetDragDropOp::InitThumbnail()
{
	if (AssetData.Num() > 0 && ThumbnailSize > 0)
	{
		// Create a thumbnail pool to hold the single thumbnail rendered
		//ThumbnailPool = MakeShared<FAssetThumbnailPool>(1, /*InAreRealTileThumbnailsAllowed=*/false);

		// Create the thumbnail handle
		AssetThumbnail = MakeShared<FAssetThumbnail>(AssetData[0], ThumbnailSize, ThumbnailSize, UThumbnailManager::Get().GetSharedThumbnailPool());

		// Request the texture then tick the pool once to render the thumbnail
		AssetThumbnail->GetViewportRenderTargetTexture();
	}
}

bool FAssetDragDropOp::HasFiles() const
{
	return AssetData.Num() > 0;
}

bool FAssetDragDropOp::HasFolders() const
{
	return AssetPaths.Num() > 0;
}

int32 FAssetDragDropOp::GetTotalCount() const
{
	return AssetData.Num() + AssetPaths.Num();
}

FText FAssetDragDropOp::GetFirstItemText() const
{
	if (AssetData.Num() > 0)
	{
		return FText::FromName(AssetData[0].AssetName);
	}

	if (AssetPaths.Num() > 0)
	{
		return FText::FromString(AssetPaths[0]);
	}

	return FText::GetEmpty();
}

void FAssetDragDropOp::SetCustomThumbnailWidget(const TSharedRef<SWidget>& InNewThumbnailWidget)
{
	CustomThumbnailWidget = InNewThumbnailWidget;
}
