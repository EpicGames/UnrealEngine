// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/CollectionDragDropOp.h"

#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CollectionManagerModule.h"
#include "HAL/Platform.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SAssetTagItem.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"

class SWidget;

TSharedRef<FCollectionDragDropOp> FCollectionDragDropOp::New(TArray<FCollectionRef> InCollectionRefs, const EAssetTagItemViewMode InAssetTagViewMode)
{
	TSharedRef<FCollectionDragDropOp> Operation = MakeShareable(new FCollectionDragDropOp);

	Operation->AssetTagViewMode = InAssetTagViewMode;
	Operation->MouseCursor = EMouseCursor::GrabHandClosed;
	Operation->CollectionRefs = MoveTemp(InCollectionRefs);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Fill out deprecated Collections with game project Collections for backwards compatibility.
	Algo::TransformIf(
		Operation->CollectionRefs,
		Operation->Collections,
		[](const FCollectionRef& Collection) { return Collection.Container == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(); },
		[](const FCollectionRef& Collection) { return FCollectionNameType(Collection.Name, Collection.Type); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Operation->Construct();

	return Operation;
}

TSharedRef<FCollectionDragDropOp> FCollectionDragDropOp::New(TArray<FCollectionNameType> InCollections, const EAssetTagItemViewMode InAssetTagViewMode)
{
	TSharedRef<FCollectionDragDropOp> Operation = MakeShareable(new FCollectionDragDropOp);

	Operation->AssetTagViewMode = InAssetTagViewMode;
	Operation->MouseCursor = EMouseCursor::GrabHandClosed;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Operation->Collections = MoveTemp(InCollections);

	Operation->CollectionRefs.Reserve(Operation->Collections.Num());
	Algo::Transform(
		Operation->Collections,
		Operation->CollectionRefs,
		[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Operation->Construct();

	return Operation;
}

TArray<FAssetData> FCollectionDragDropOp::GetAssets() const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray<FSoftObjectPath> AssetPaths;
	for (const FCollectionRef& CollectionRef : CollectionRefs)
	{
		if (CollectionRef.Container)
		{
			CollectionRef.Container->GetAssetsInCollection(CollectionRef.Name, CollectionRef.Type, AssetPaths);
		}
	}

	TArray<FAssetData> AssetDatas;
	AssetDatas.Reserve(AssetPaths.Num());
	for (const FSoftObjectPath& AssetPath : AssetPaths)
	{
		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(AssetPath);
		if (AssetData.IsValid())
		{
			AssetDatas.AddUnique(AssetData);
		}
	}
	
	return AssetDatas;
}

TSharedPtr<SWidget> FCollectionDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetDragDropTooltipBackground"))
		//.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SAssetTagItem)
			.ViewMode(AssetTagViewMode)
			.DisplayName(this, &FCollectionDragDropOp::GetDecoratorText)
		];
}

FText FCollectionDragDropOp::GetDecoratorText() const
{
	if (CurrentHoverText.IsEmpty() && CollectionRefs.Num() > 0)
	{
		return (CollectionRefs.Num() == 1)
			? FText::FromName(CollectionRefs[0].Name)
			: FText::Format(NSLOCTEXT("ContentBrowser", "CollectionDragDropDescription", "{0} and {1} {1}|plural(one=other,other=others)"), FText::FromName(CollectionRefs[0].Name), CollectionRefs.Num() - 1);
	}

	return CurrentHoverText;
}
