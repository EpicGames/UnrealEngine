// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragTool_BoxSelect.h"
#include "Components/PrimitiveComponent.h"
#include "CanvasItem.h"
#include "Model.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "GameFramework/Volume.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "EngineUtils.h"
#include "EditorModeManager.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "EditorModes.h"
#include "ActorEditorUtils.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Engine/LevelStreaming.h"
#include "CanvasTypes.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "LevelEditorSubsystem.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FDragTool_BoxSelect
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::LevelEditor::Private
{
	TArray<FTypedElementHandle> GetElementsIntersectingBox(const AActor* Actor,
		const FBox& InBox,
		const FEditorViewportClient* InEditorViewport,
		const FLevelEditorViewportClient* InLevelViewport,
		const FWorldSelectionElementArgs& SelectionArgs)
	{
		if (Actor && (!InEditorViewport || !Actor->IsA(AVolume::StaticClass()) || (InLevelViewport ? !InLevelViewport->IsVolumeVisibleInViewport(*Actor) : false)))
		{
			if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(ActorHandle))
				{
					return WorldElement.GetSelectionElementsInBox(InBox, SelectionArgs);
				}
			}
		}

		return {};
	}
}

/**
 * Starts a mouse drag behavior.  The start location is snapped to the editor constraints if bUseSnapping is true.
 *
 * @param	InViewportClient	The viewport client in which the drag event occurred.
 * @param	InStart				Where the mouse was when the drag started.
 */
void FDragTool_ActorBoxSelect::StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen)
{
	FDragTool::StartDrag(InViewportClient, InStart, InStartScreen);
	
	FIntPoint MousePos;
	InViewportClient->Viewport->GetMousePos(MousePos);

	Start = FVector(MousePos);
	End = EndWk = Start;

	FLevelEditorViewportClient::ClearHoverFromObjects();

	// Create a list of bsp models to check for intersection with the box
	ModelsToCheck.Reset();
	// Do not select BSP if its not visible
	if( InViewportClient->EngineShowFlags.BSP)
	{
		UWorld* World = InViewportClient->GetWorld();
		check(World);
		// Add the persistent level always
		ModelsToCheck.Add( World->PersistentLevel->Model );
		// Add all streaming level models
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			// Only add streaming level models if the level is visible
			if (StreamingLevel && StreamingLevel->GetShouldBeVisibleInEditor())
			{	
				if (ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					ModelsToCheck.Add( Level->Model );
				}
			}
		}
	}
}

void FDragTool_ActorBoxSelect::AddDelta( const FVector& InDelta )
{
	FDragTool::AddDelta( InDelta );

	FIntPoint MousePos;
	EditorViewportClient->Viewport->GetMousePos(MousePos);

	End = FVector(MousePos); 
	EndWk = End;
	
	const bool bUseHoverFeedback = GEditor != NULL && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;

	if( bUseHoverFeedback )
	{
		const bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

		UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet();

		UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
		const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

		FWorldSelectionElementArgs SeletionArgs
		{
			SelectionSet,
			ETypedElementSelectionMethod::Primary,
			FTypedElementSelectionOptions(),
			&(EditorViewportClient->EngineShowFlags),
			bStrictDragSelection,
			bGeometryMode
		};


		// If we are using over feedback calculate a new box from the one being dragged
		FBox SelBBox;
		CalculateBox( SelBBox );

		// Check every actor to see if it intersects the frustum created by the box
		// If it does, the actor will be selected and should be given a hover cue
		bool bSelectionChanged = false;
		UWorld* IteratorWorld = GWorld;
		for( FActorIterator It(IteratorWorld); It; ++It )
		{
			AActor& Actor = **It;
			const bool bActorHitByBox = !UE::LevelEditor::Private::GetElementsIntersectingBox( &Actor, SelBBox, EditorViewportClient, LevelViewportClient, SeletionArgs ).IsEmpty();


			if( bActorHitByBox )
			{
				// Apply a hover effect to any actor that will be selected
				AddHoverEffect( Actor );
			}
			else
			{
				// Remove any hover effect on this actor as it no longer will be selected by the current box
				RemoveHoverEffect( Actor );
			}
		}

		// Check each model to see if it will be selected
		for( int32 ModelIndex = 0; ModelIndex < ModelsToCheck.Num(); ++ModelIndex )
		{
			UModel& Model = *ModelsToCheck[ModelIndex];
			for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
			{
				if( IntersectsBox( Model, NodeIndex, SelBBox, bStrictDragSelection ) )
				{
					// Apply a hover effect to any bsp surface that will be selected
					AddHoverEffect( Model, Model.Nodes[NodeIndex].iSurf );
				}
				else
				{
					// Remove any hover effect on this bsp surface as it no longer will be selected by the current box
					RemoveHoverEffect( Model, Model.Nodes[NodeIndex].iSurf );
				}
			}
		}
	}
}
/**
* Ends a mouse drag behavior (the user has let go of the mouse button).
*/
void FDragTool_ActorBoxSelect::EndDrag()
{
	UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
	const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

	FScopedTransaction Transaction( NSLOCTEXT("ActorFrustumSelect", "MarqueeSelectTransation", "Marquee Select" ) );

	bool bShouldSelect = true;
	FBox SelBBox;
	CalculateBox( SelBBox );

	if( bControlDown )
	{
		// If control is down remove from selection
		bShouldSelect = false;
	}
	else if( !bShiftDown )
	{
		// If the user is selecting, but isn't holding down SHIFT, give modes a chance to clear selection
		ModeTools->SelectNone();
	}

	// Let the editor mode try to handle the box selection.
	const bool bEditorModeHandledBoxSelection = ModeTools->BoxSelect(SelBBox, bLeftMouseButtonDown);

	// Let the component visualizers try to handle the selection.
	const bool bComponentVisHandledSelection = !bEditorModeHandledBoxSelection && GUnrealEd->ComponentVisManager.HandleBoxSelect(SelBBox, EditorViewportClient, EditorViewportClient->Viewport);

	// If the edit mode didn't handle the selection, try normal actor box selection.
	if ( !bEditorModeHandledBoxSelection && !bComponentVisHandledSelection )
	{
		const bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

		UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet();
		FTypedElementSelectionOptions ElementSelectionOption;
		if (!bControlDown && !bShiftDown)
		{
			// If the user is selecting, but isn't holding down SHIFT, remove all current selections from the selection set
			SelectionSet->ClearSelection(ElementSelectionOption);
		}

		FWorldSelectionElementArgs SeletionArgs
		{
			SelectionSet,
			ETypedElementSelectionMethod::Primary,
			ElementSelectionOption,
			&(EditorViewportClient->EngineShowFlags),
			bStrictDragSelection,
			bGeometryMode
		};

		// Select all element that are within the selection box area.  Be aware that certain modes do special processing below.	
		bool bSelectionChanged = false;
		UWorld* IteratorWorld = GWorld;
		const TArray<FName>& HiddenLayers = LevelViewportClient ? LevelViewportClient->ViewHiddenLayers : TArray<FName>();
		TArray<FTypedElementHandle> Handles;

		for( FActorIterator It(IteratorWorld); It; ++It )
		{
			AActor* Actor = *It;
			
			bool bActorIsVisible = true;
			for ( auto Layer : Actor->Layers )
			{
				// Check the actor isn't in one of the layers hidden from this viewport.
				if( HiddenLayers.Contains( Layer ) )
				{
					bActorIsVisible = false;
					break;
				}
			}

			// Select the actor or its child elements
			if( bActorIsVisible )
			{
				Handles.Append( UE::LevelEditor::Private::GetElementsIntersectingBox( Actor, SelBBox, EditorViewportClient, LevelViewportClient, SeletionArgs ) );
			}
		}

		if (bShouldSelect)
		{
			SelectionSet->SelectElements( Handles, ElementSelectionOption );
		}
		else
		{
			SelectionSet->DeselectElements( Handles, ElementSelectionOption );
		}

		// Check every model to see if its BSP surfaces should be selected
		for( int32 ModelIndex = 0; ModelIndex < ModelsToCheck.Num(); ++ModelIndex )
		{
			UModel& Model = *ModelsToCheck[ModelIndex];
			// Check every node in the model
			for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
			{
				if( IntersectsBox( Model, NodeIndex, SelBBox, bStrictDragSelection ) )
				{
					// If the node intersected the frustum select the corresponding surface
					GEditor->SelectBSPSurf( &Model, Model.Nodes[NodeIndex].iSurf, bShouldSelect, false );
					bSelectionChanged = true;
				}
			}
		}

		if ( bSelectionChanged )
		{
			// If any selections were made.  Notify that now.
			GEditor->NoteSelectionChange();
		}
	}

	// Clear any hovered objects that might have been created while dragging
	FLevelEditorViewportClient::ClearHoverFromObjects();

	// Clean up.
	FDragTool::EndDrag();
}

void FDragTool_ActorBoxSelect::Render(const FSceneView* View, FCanvas* Canvas)
{
	FCanvasBoxItem BoxItem(FVector2D(Start.X, Start.Y) / Canvas->GetDPIScale(), FVector2D(End.X - Start.X, End.Y - Start.Y) / Canvas->GetDPIScale());
	BoxItem.SetColor(FLinearColor::White);
	Canvas->DrawItem(BoxItem);
}

void FDragTool_ActorBoxSelect::CalculateBox( FBox& OutBox )
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		EditorViewportClient->Viewport,
		EditorViewportClient->GetScene(),
		EditorViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(EditorViewportClient->IsRealtime()));

	FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily);

	FVector3f StartFloat{ Start };
	FVector3f EndFloat{ End };

	FVector4 StartScreenPos = View->PixelToScreen(StartFloat.X, StartFloat.Y, 0);
	FVector4 EndScreenPos = View->PixelToScreen(EndFloat.X, EndFloat.Y, 0);

	FVector TransformedStart = View->ScreenToWorld(View->PixelToScreen(StartFloat.X, StartFloat.Y, 0.5f));
	FVector TransformedEnd = View->ScreenToWorld(View->PixelToScreen(EndFloat.X, EndFloat.Y, 0.5f));

	// Create a bounding box based on the start/end points (normalizes the points).
	OutBox.Init();
	OutBox += TransformedStart;
	OutBox += TransformedEnd;

	switch(EditorViewportClient->ViewportType)
	{
	case LVT_OrthoXY:
	case LVT_OrthoNegativeXY:
		OutBox.Min.Z = -WORLD_MAX;
		OutBox.Max.Z = WORLD_MAX;
		break;
	case LVT_OrthoXZ:
	case LVT_OrthoNegativeXZ:
		OutBox.Min.Y = -WORLD_MAX;
		OutBox.Max.Y = WORLD_MAX;
		break;
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeYZ:
		OutBox.Min.X = -WORLD_MAX;
		OutBox.Max.X = WORLD_MAX;
		break;
	case LVT_OrthoFreelook:
	case LVT_Perspective:
		break;
	}
}

/** 
 * Returns true if the provided BSP node intersects with the provided frustum 
 *
 * @param InModel				The model containing BSP nodes to check
 * @param NodeIndex				The index to a BSP node in the model.  This node is used for the bounds check.
 * @param InFrustum				The frustum to check against.
 * @param bUseStrictSelection	true if the node must be entirely within the frustum
 */
bool FDragTool_ActorBoxSelect::IntersectsBox( const UModel& InModel, int32 NodeIndex, const FBox& InBox, bool bUseStrictSelection ) const
{
	FBox NodeBB;
	InModel.GetNodeBoundingBox( InModel.Nodes[NodeIndex], NodeBB );

	bool bFullyContained = false;
	bool bIntersects = false;
	if( !bUseStrictSelection )
	{
		bIntersects = InBox.Intersect( NodeBB );
	}
	else
	{
		bIntersects = InBox.IsInside( NodeBB.Max ) && InBox.IsInside( NodeBB.Min );
	}

	return bIntersects;
}

/** Adds a hover effect to the passed in actor */
void FDragTool_ActorBoxSelect::AddHoverEffect( AActor& InActor )
{
	FViewportHoverTarget HoverTarget( &InActor );
	FLevelEditorViewportClient::AddHoverEffect( HoverTarget );
	FLevelEditorViewportClient::HoveredObjects.Add( HoverTarget );
}

/** Removes a hover effect from the passed in actor */
void FDragTool_ActorBoxSelect::RemoveHoverEffect( AActor& InActor  )
{
	FViewportHoverTarget HoverTarget( &InActor );
	FSetElementId Id = FLevelEditorViewportClient::HoveredObjects.FindId( HoverTarget );
	if( Id.IsValidId() )
	{
		FLevelEditorViewportClient::RemoveHoverEffect( HoverTarget );
		FLevelEditorViewportClient::HoveredObjects.Remove( Id );
	}
}

/** Adds a hover effect to the passed in bsp surface */
void FDragTool_ActorBoxSelect::AddHoverEffect( UModel& InModel, int32 SurfIndex )
{
	FViewportHoverTarget HoverTarget( &InModel, SurfIndex );
	FLevelEditorViewportClient::AddHoverEffect( HoverTarget );
	FLevelEditorViewportClient::HoveredObjects.Add( HoverTarget );
}

/** Removes a hover effect from the passed in bsp surface */
void FDragTool_ActorBoxSelect::RemoveHoverEffect( UModel& InModel, int32 SurfIndex )
{
	FViewportHoverTarget HoverTarget( &InModel, SurfIndex );
	FSetElementId Id = FLevelEditorViewportClient::HoveredObjects.FindId( HoverTarget );
	if( Id.IsValidId() )
	{
		FLevelEditorViewportClient::RemoveHoverEffect( HoverTarget );
		FLevelEditorViewportClient::HoveredObjects.Remove( Id );
	}
}
