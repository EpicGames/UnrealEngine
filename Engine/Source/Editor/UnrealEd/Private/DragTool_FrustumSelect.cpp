// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragTool_FrustumSelect.h"

#include "Components/PrimitiveComponent.h"
#include "CanvasItem.h"
#include "Model.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "GameFramework/Volume.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "EngineUtils.h"
#include "Engine/Selection.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "ActorEditorUtils.h"
#include "ScopedTransaction.h"
#include "HModel.h"
#include "CanvasTypes.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "LevelEditorSubsystem.h"
#include "SceneView.h"
#include "EditorDragTools/FrustumSelectInteraction.h"

///////////////////////////////////////////////////////////////////////////////
//
// FDragTool_FrustumSelect
//
///////////////////////////////////////////////////////////////////////////////

namespace UE::LevelEditor::Private
{
	TArray<FTypedElementHandle> GetElementsIntersectingFrustum(const AActor* Actor,
		const FConvexVolume& InFrustum,
		const FEditorViewportClient* EditorViewport,
		const FLevelEditorViewportClient* LevelViewport,
		const FWorldSelectionElementArgs& SelectionArgs)
	{
		if (Actor && (!EditorViewport || !Actor->IsA(AVolume::StaticClass()) || (LevelViewport ? !LevelViewport->IsVolumeVisibleInViewport(*Actor) : false)))
		{
			if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(ActorHandle))
				{
					return WorldElement.GetSelectionElementsInConvexVolume(InFrustum, SelectionArgs);
				}
			}
		}

		return {};
	}
}

void FDragTool_ActorFrustumSelect::AddDelta( const FVector& InDelta )
{
	FIntPoint MousePos;
	EditorViewportClient->Viewport->GetMousePos(MousePos);

	EndWk = FVector(MousePos);
	End = EndWk;

	const bool bUseHoverFeedback = GEditor != NULL && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;
}

void FDragTool_ActorFrustumSelect::StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen)
{
	FDragTool::StartDrag(InViewportClient, InStart, InStartScreen);

	const bool bUseHoverFeedback = GEditor != NULL && GetDefault<ULevelEditorViewportSettings>()->bEnableViewportHoverFeedback;

	// Remove any active hover objects
	FLevelEditorViewportClient::ClearHoverFromObjects();

	FIntPoint MousePos;
	InViewportClient->Viewport->GetMousePos(MousePos);

	Start = FVector(InStartScreen.X, InStartScreen.Y, 0);
	End = EndWk = Start;
}

void FDragTool_ActorFrustumSelect::EndDrag()
{
	UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
	const bool bGeometryMode = BrushSubsystem ? BrushSubsystem->IsGeometryEditorModeActive() : false;

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(EditorViewportClient->Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags ));
	FSceneView* SceneView = EditorViewportClient->CalcSceneView(&ViewFamily);

	// Generate a frustum out of the dragged box
	FConvexVolume Frustum;
	CalculateFrustum( SceneView, Frustum, true );

	FScopedTransaction Transaction( NSLOCTEXT("ActorFrustumSelect", "MarqueeSelectTransation", "Marquee Select" ) );

	bool bShouldSelect = true;

	if( !bShiftDown )
	{
		// If the user is selecting, but isn't hold down SHIFT, remove all current selections.
		ModeTools->SelectNone();
	}

	// Does an actor have to be fully contained in the box to be selected
	const bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

	// Let the editor mode try to handle the selection.
	const bool bEditorModeHandledSelection = ModeTools->FrustumSelect(Frustum, EditorViewportClient, bLeftMouseButtonDown);

	// Let the component visualizers try to handle the selection.
	const bool bComponentVisHandledSelection = !bEditorModeHandledSelection && GUnrealEd->ComponentVisManager.HandleFrustumSelect(Frustum, EditorViewportClient, EditorViewportClient->Viewport);

	if( !bEditorModeHandledSelection && !bComponentVisHandledSelection)
	{
		UTypedElementSelectionSet* SelectionSet = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>()->GetSelectionSet();
		SelectionSet->Modify();

		FWorldSelectionElementArgs SeletionArgs
		{
			SelectionSet,
			ETypedElementSelectionMethod::Primary,
			FTypedElementSelectionOptions(),
			&(EditorViewportClient->EngineShowFlags),
			bStrictDragSelection,
			bGeometryMode
		};

		const int32 ViewportSizeX = EditorViewportClient->Viewport->GetSizeXY().X;
		const int32 ViewportSizeY = EditorViewportClient->Viewport->GetSizeXY().Y;

		if( Start.X > End.X )
		{
			Swap( Start.X, End.X );
		}

		if( Start.Y > End.Y )
		{
			Swap( Start.Y, End.Y );
		}

		TArray<FTypedElementHandle> ElementsToSelect;
		const bool bTransparentBoxSelection = GetDefault<ULevelEditorViewportSettings>()->bTransparentBoxSelection;
		if (bTransparentBoxSelection)
		{
			// Get a list of frustum-culled actors
			for(FActorIterator It(EditorViewportClient->GetWorld()); It; ++It)
			{
				AActor* Actor = *It;
				ElementsToSelect.Append(UE::LevelEditor::Private::GetElementsIntersectingFrustum(Actor, Frustum, EditorViewportClient, LevelViewportClient, SeletionArgs));
			}
		}
		else
		{
			// Extend the endpoint of the rect to get the actual line

			const int32 MinX = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, Start.X));
			const int32 MinY = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, Start.Y));
			const int32 MaxX = FMath::Min(ViewportSizeX, FMath::TruncToInt32(End.X + 1.0));
			const int32 MaxY = FMath::Min(ViewportSizeY, FMath::TruncToInt32(End.Y + 1.0));

			const FIntPoint Min{ MinX, MinY };
			const FIntPoint Max{ MaxX, MaxY };
			const FIntRect BoxRect{ Min, Max };

			// Typed Element selection
			{
				FTypedElementListRef ElementList = UTypedElementRegistry::GetInstance()->CreateElementList();
				EditorViewportClient->Viewport->GetElementHandlesInRect(BoxRect, ElementList);

				if (bStrictDragSelection)
				{
					ElementList->ForEachElement<ITypedElementWorldInterface>([bStrictDragSelection, &Frustum, &SelectionSet, &ElementsToSelect]
						(const TTypedElement<ITypedElementWorldInterface>& InElement)
						{
							if (InElement.IsElementInConvexVolume(Frustum, bStrictDragSelection))
							{
								ElementsToSelect.Add(SelectionSet->GetSelectionElement(InElement, ETypedElementSelectionMethod::Primary));
							}

							return true;
						});
				}
				else
				{
					// Grab only the selectable handles (this remove the components from the selection and select the actor instead)
					ElementList->ForEachElementHandle([&SelectionSet, &ElementsToSelect](const FTypedElementHandle& InHandle)
						{
							ElementsToSelect.Add(SelectionSet->GetSelectionElement(InHandle, ETypedElementSelectionMethod::Primary));
							return true;
						});
				}
			}

			// We need this old code to support the BSP
			TSet<AActor*> BSPActors;
			TSet<UModel*> HitModels;
			EditorViewportClient->Viewport->GetActorsAndModelsInHitProxy( BoxRect, BSPActors, HitModels );
			BSPActors.Empty(HitModels.Num());

			

			if (HitModels.Num() > 0)
			{
				// Check every model to see if its BSP surfaces should be selected
				for (auto It = HitModels.CreateConstIterator(); It; ++It)
				{
					UModel& Model = **It;
					// Check every node in the model
					for (int32 NodeIndex = 0; NodeIndex < Model.Nodes.Num(); NodeIndex++)
					{
						if (IntersectsFrustum(Model, NodeIndex, Frustum, bStrictDragSelection))
						{
							uint32 SurfaceIndex = Model.Nodes[NodeIndex].iSurf;
							FBspSurf& Surf = Model.Surfs[SurfaceIndex];
							BSPActors.Add( Surf.Actor );
						}
					}
				}
			}

			if( BSPActors.Num() > 0 )
			{
				for( auto It = BSPActors.CreateConstIterator(); It; ++It )
				{
					AActor* Actor = *It;
					if (bStrictDragSelection)
					{
						ElementsToSelect.Append(UE::LevelEditor::Private::GetElementsIntersectingFrustum(Actor, Frustum, EditorViewportClient, LevelViewportClient, SeletionArgs));
					}
					else
					{
						ElementsToSelect.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor));
					}
				}
			}
		}

		 FTypedElementSelectionOptions ElementSelectionOption;

		if (!bShiftDown)
		{
			// If the user is selecting, but isn't hold down SHIFT, remove the previous selections.
			SelectionSet->SetSelection(MoveTemp(ElementsToSelect), ElementSelectionOption);
		}
		else
		{
			SelectionSet->SelectElements(MoveTemp(ElementsToSelect), ElementSelectionOption);
		}
	}

	// Clear any hovered objects that might have been created while dragging
	FLevelEditorViewportClient::ClearHoverFromObjects();

	FDragTool::EndDrag();
}

void FDragTool_ActorFrustumSelect::Render(const FSceneView* View, FCanvas* Canvas )
{
	FCanvasBoxItem BoxItem( FVector2D(Start.X, Start.Y) / Canvas->GetDPIScale(), FVector2D(End.X-Start.X, End.Y-Start.Y) / Canvas->GetDPIScale());
	BoxItem.SetColor( FLinearColor::White );
	Canvas->DrawItem( BoxItem );
}

bool FDragTool_ActorFrustumSelect::IntersectsFrustum( const UModel& InModel, int32 NodeIndex, const FConvexVolume& InFrustum, bool bUseStrictSelection ) const
{
	FBox NodeBB;
	// Get a bounding box of the node being checked
	InModel.GetNodeBoundingBox( InModel.Nodes[NodeIndex], NodeBB );

	bool bFullyContained = false;

	// Does the box intersect the frustum
	bool bIntersects = InFrustum.IntersectBox( NodeBB.GetCenter(), NodeBB.GetExtent(), bFullyContained );

	return bIntersects && (!bUseStrictSelection || (bUseStrictSelection && bFullyContained));
}

void FDragTool_ActorFrustumSelect::CalculateFrustum( const FSceneView* InView, FConvexVolume& OutFrustum, bool bUseBoxFrustum ) const
{
	if (!InView || !EditorViewportClient)
	{
		return;
	}

	UE::EditorDragTools::FViewportFrustum ViewportFrustum(*EditorViewportClient, *InView, Start, End, bUseBoxFrustum);
	ViewportFrustum.Calculate(OutFrustum);
}

void FDragTool_ActorFrustumSelect::AddHoverEffect( AActor& InActor )
{
	FViewportHoverTarget HoverTarget( &InActor );
	FLevelEditorViewportClient::AddHoverEffect( HoverTarget );
	FLevelEditorViewportClient::HoveredObjects.Add( HoverTarget );
}

void FDragTool_ActorFrustumSelect::RemoveHoverEffect( AActor& InActor  )
{
	FViewportHoverTarget HoverTarget( &InActor );
	FSetElementId Id = FLevelEditorViewportClient::HoveredObjects.FindId( HoverTarget );
	if( Id.IsValidId() )
	{
		FLevelEditorViewportClient::RemoveHoverEffect( HoverTarget );
		FLevelEditorViewportClient::HoveredObjects.Remove( Id );
	}
}

void FDragTool_ActorFrustumSelect::AddHoverEffect( UModel& InModel, int32 SurfIndex )
{
	FViewportHoverTarget HoverTarget( &InModel, SurfIndex );
	FLevelEditorViewportClient::AddHoverEffect( HoverTarget );
	FLevelEditorViewportClient::HoveredObjects.Add( HoverTarget );
}

void FDragTool_ActorFrustumSelect::RemoveHoverEffect( UModel& InModel, int32 SurfIndex )
{
	FViewportHoverTarget HoverTarget( &InModel, SurfIndex );
	FSetElementId Id = FLevelEditorViewportClient::HoveredObjects.FindId( HoverTarget );
	if( Id.IsValidId() )
	{
		FLevelEditorViewportClient::RemoveHoverEffect( HoverTarget );
		FLevelEditorViewportClient::HoveredObjects.Remove( Id );
	}
}
