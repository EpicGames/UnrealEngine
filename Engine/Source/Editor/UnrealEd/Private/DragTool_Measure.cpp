// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragTool_Measure.h"
#include "SceneView.h"
#include "EditorViewportClient.h"
#include "CanvasItem.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "SnappingUtils.h"
#include "CanvasTypes.h"
#include "Settings/EditorStyleSettings.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FDragTool_Measure
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDragTool_Measure::FDragTool_Measure(FEditorViewportClient* InViewportClient)
	: FDragTool(InViewportClient->GetModeTools())
	, ViewportClient(InViewportClient)
{
	bUseSnapping = true;
	bConvertDelta = false;
}

FVector2D FDragTool_Measure::GetSnappedPixelPos(FVector2D PixelPos)
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(ViewportClient->IsRealtime()));

	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);

	// Put the mouse pos in world space
	FVector2f PixelPosFloat{ PixelPos };
	FVector WorldPos = View->ScreenToWorld(View->PixelToScreen(PixelPosFloat.X, PixelPosFloat.Y, 0.5f));;

	// Snap the world position
	const float GridSize = GEditor->GetGridSize();
	const FVector GridBase( GridSize, GridSize, GridSize );
	FSnappingUtils::SnapPointToGrid( WorldPos, GridBase );

	// And back into pixel-space (might fail, in which case we return the original)
	View->WorldToPixel(WorldPos, PixelPos);

	// The canvas we're going to render to factors in dpi scale to final position. 
	// Since we're basing our position off mouse coordinates it will already be pixel accurate and therefore we must back out the scale the canvas will apply
	PixelPos /= ViewportClient->GetDPIScale();

	return PixelPos;
}

void FDragTool_Measure::StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen)
{
	FDragTool::StartDrag(InViewportClient, InStart, InStartScreen);
	PixelStart = GetSnappedPixelPos(InStartScreen);
	PixelEnd = PixelStart;
}

void FDragTool_Measure::AddDelta(const FVector& InDelta)
{
	FDragTool::AddDelta(InDelta);
	
	FIntPoint MousePos;
	ViewportClient->Viewport->GetMousePos(MousePos);
	PixelEnd = GetSnappedPixelPos(FVector2D(MousePos));
}

void FDragTool_Measure::Render(const FSceneView* View, FCanvas* Canvas)
{
	const float OrthoUnitsPerPixel = ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport);
	const float Length = FMath::RoundToFloat(FVector2f{ PixelEnd - PixelStart }.Size() * OrthoUnitsPerPixel * ViewportClient->GetDPIScale());

	if (View != nullptr && Canvas != nullptr && Length >= 1.f)
	{
		const FLinearColor ToolColor = GetDefault<UEditorStyleSettings>()->ViewportToolOverlayColor;
		FCanvasLineItem LineItem( PixelStart, PixelEnd );
		LineItem.SetColor( ToolColor );
		Canvas->DrawItem( LineItem );

		const FVector2D PixelMid = FVector2D(PixelStart + ((PixelEnd - PixelStart) / 2));

		// Calculate number of decimal places to display, based on the current viewport zoom
		float Divisor = 1.0f;
		int32 DecimalPlaces = 0;
		const float OrderOfMagnitude = FMath::LogX(10.0f, OrthoUnitsPerPixel);

		switch (GetDefault<ULevelEditorViewportSettings>()->MeasuringToolUnits)
		{
		case MeasureUnits_Meters:
			Divisor = 100.0f;
			// Max one decimal place allowed for meters.
			DecimalPlaces = FMath::Clamp(FMath::FloorToInt(1.5f - OrderOfMagnitude), 0, 1);
			break;

		case MeasureUnits_Kilometers:
			Divisor = 100000.0f;
			// Max two decimal places allowed for kilometers.
			DecimalPlaces = FMath::Clamp(FMath::FloorToInt(4.5f - OrderOfMagnitude), 0, 2);
			break;
		}

		FNumberFormattingOptions Options;
		Options.UseGrouping = false;
		Options.MinimumFractionalDigits = DecimalPlaces;
		Options.MaximumFractionalDigits = DecimalPlaces;

		const FText LengthStr = FText::AsNumber( Length / Divisor, &Options );


		FCanvasTextItem TextItem( FVector2D( FMath::FloorToFloat(PixelMid.X), FMath::FloorToFloat(PixelMid.Y) ), LengthStr, GEngine->GetSmallFont(), ToolColor );
		TextItem.bCentreX = true;
		Canvas->DrawItem( TextItem );
	}
}
