// Copyright Epic Games, Inc. All Rights Reserved.


#include "DragTool_ViewportChange.h"
#include "CanvasItem.h"
#include "LevelEditorViewport.h"
#include "ILevelEditor.h"
#include "CanvasTypes.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FDragTool_ViewportChange
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDragTool_ViewportChange::FDragTool_ViewportChange(FLevelEditorViewportClient* InLevelViewportClient)
	: FDragTool(InLevelViewportClient->GetModeTools())
	, LevelViewportClient(InLevelViewportClient)
	, ViewOption(LVT_Perspective)
	, ViewOptionOffset(FVector2D(0.f,0.f))
{
	bUseSnapping = true;
	bConvertDelta = false;
}

void FDragTool_ViewportChange::StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& InStartScreen)
{
	FDragTool::StartDrag(InViewportClient, InStart, InStartScreen);

	FIntPoint MousePos;
	InViewportClient->Viewport->GetMousePos(MousePos);

	Start = FVector(InStartScreen.X, InStartScreen.Y, 0) / LevelViewportClient->GetDPIScale();
	End = EndWk = Start;
}

void FDragTool_ViewportChange::EndDrag()
{
	ViewOptionOffset.X = End.X - Start.X;
	ViewOptionOffset.Y = End.Y - Start.Y;

	if (ViewOptionOffset.Y == 0)
	{
		if (ViewOptionOffset.X == 0)
		{
			ViewOption = LVT_Perspective;
		}
		else if (ViewOptionOffset.X > 0)
		{
			ViewOption = LVT_OrthoRight; // Right
		}
		else
		{
			ViewOption = LVT_OrthoLeft; // Left
		}
	}
	else
	{
		double OffsetRatio = ViewOptionOffset.X / ViewOptionOffset.Y;
		double DragAngle = FMath::RadiansToDegrees(FMath::Atan(OffsetRatio));

		if (ViewOptionOffset.Y >= 0)
		{
			if (DragAngle >= -15.f && DragAngle <= 15.f)
			{
				ViewOption = LVT_OrthoBottom; // Bottom
			}
			else if (DragAngle > 75.f)
			{
				ViewOption = LVT_OrthoRight; // Right
			}
			else if (DragAngle < -75.f)
			{
				ViewOption = LVT_OrthoLeft; // Left
			}
		}
		else
		{
			if (DragAngle >= -15.f && DragAngle < 15.f)
			{
				ViewOption = LVT_OrthoTop; // Top
			}
			else if (DragAngle >= 15.f && DragAngle < 75.f)
			{
				ViewOption = LVT_OrthoFront; // Front
			}
			else if (DragAngle >= -75.f && DragAngle < -15.f)
			{
				ViewOption = LVT_OrthoBack; // Back
			}
			else if (DragAngle >= 75.f)
			{
				ViewOption = LVT_OrthoLeft; // Left
			}
			else if (DragAngle <= -75.f)
			{
				ViewOption = LVT_OrthoRight; // Right
			}
		}
	}

	double OffsetLength = FMath::RoundToFloat((End - Start).Size());
	if (OffsetLength >= 125.15f)
	{
		LevelViewportClient->SetViewportType(ViewOption);
		return;
	}

	if (LevelViewportClient->ParentLevelEditor.IsValid())
	{
		LevelViewportClient->ParentLevelEditor.Pin()->SummonLevelViewportViewOptionMenu(ViewOption);
	}
}

void FDragTool_ViewportChange::AddDelta(const FVector& InDelta)
{
	FDragTool::AddDelta(InDelta);

	FIntPoint MousePos;
	LevelViewportClient->Viewport->GetMousePos(MousePos);

	EndWk = FVector(MousePos) / LevelViewportClient->GetDPIScale();
	End = EndWk;

	ViewOptionOffset.X = End.X - Start.X;
	ViewOptionOffset.Y = End.Y - Start.Y;
}

void FDragTool_ViewportChange::Render(const FSceneView* View, FCanvas* Canvas)
{
	FCanvasLineItem LineItem(Start, End);
	Canvas->DrawItem(LineItem);
}
