// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphNode_Comment.h"

#include "GraphEditorSettings.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Layout/SlateRect.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode_Comment)

class UEdGraphPin;

#define LOCTEXT_NAMESPACE "EdGraph"

namespace FEdGraphNode_Comment_Utils
{
	template<typename T>
	void SyncPropertyToValue(UEdGraphNode_Comment* InNode, FProperty* InProperty, const T& InValue)
	{
		if (InNode && InProperty)
		{
			T* CurrentValuePtr = InProperty->ContainerPtrToValuePtr<T>(InNode);
			if (*CurrentValuePtr != InValue)
			{
				*CurrentValuePtr = InValue;

				FPropertyChangedEvent PropertyChangedEvent(InProperty, EPropertyChangeType::Unspecified);
				InNode->PostEditChangeProperty(PropertyChangedEvent);
			}
		}
	}
}

/////////////////////////////////////////////////////
// UEdGraphNode_Comment

UEdGraphNode_Comment::UEdGraphNode_Comment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeWidth = 400;
	NodeHeight = 100;
	FontSize = 18;
	CommentColor = FLinearColor::White;
	bColorCommentBubble = false;
	MoveMode = ECommentBoxMode::GroupMovement;

	bCommentBubblePinned = true;
	bCommentBubbleVisible = true;
	bCommentBubbleVisible_InDetailsPanel = true;
	bCanResizeNode = true;
	bCanRenameNode = true;
	CommentDepth = -1;
}

void UEdGraphNode_Comment::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector) 
{
	UEdGraphNode_Comment* This = CastChecked<UEdGraphNode_Comment>(InThis);
	for (auto It = This->NodesUnderComment.CreateIterator(); It; ++It)
	{
		Collector.AddReferencedObject(*It, This);
	}

	Super::AddReferencedObjects(InThis, Collector);
}

void UEdGraphNode_Comment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UEdGraphNode_Comment, bCommentBubbleVisible_InDetailsPanel))
	{
		bCommentBubbleVisible = bCommentBubbleVisible_InDetailsPanel;
		bCommentBubblePinned = bCommentBubbleVisible_InDetailsPanel;
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UEdGraphNode_Comment::PostPlacedNewNode()
{
	const UClass* NodeClass = GetClass();
	const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();

	// This is done here instead of in the constructor so we can later change the default for newly placed
	// instances without changing all of the existing ones (due to delta serialization)
	FEdGraphNode_Comment_Utils::SyncPropertyToValue(this, NodeClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEdGraphNode_Comment, MoveMode)), GraphEditorSettings->DefaultCommentNodeMoveMode);
	FEdGraphNode_Comment_Utils::SyncPropertyToValue(this, NodeClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEdGraphNode_Comment, CommentColor)), GraphEditorSettings->DefaultCommentNodeTitleColor);
	FEdGraphNode_Comment_Utils::SyncPropertyToValue(this, NodeClass->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEdGraphNode_Comment, bCommentBubbleVisible_InDetailsPanel)), GraphEditorSettings->bShowCommentBubbleWhenZoomedOut);

	NodeComment = NSLOCTEXT("K2Node", "CommentBlock_NewEmptyComment", "Comment").ToString();
}

FText UEdGraphNode_Comment::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FText DetailsTooltipText = FText::GetEmpty();
		if (!NodeDetails.IsEmpty()) 
		{
			DetailsTooltipText = FText::Format(FText::FromString("\n\nDetails:\n{0}"), NodeDetails);
		}
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "CommentBlock_Tooltip", "Comment:\n{0}{1}"), FText::FromString(NodeComment), DetailsTooltipText), this);
	}
	return CachedTooltip;
}

FString UEdGraphNode_Comment::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Common");
}

FString UEdGraphNode_Comment::GetDocumentationExcerptName() const
{
	return TEXT("UEdGraphNode_Comment");
}

FSlateIcon UEdGraphNode_Comment::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = FLinearColor::White;
	
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Icons.Comment");
	return Icon;
}

FText UEdGraphNode_Comment::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(TitleType == ENodeTitleType::MenuTitle)
	{
		return NSLOCTEXT("K2Node", "NoComment_ListTitle", "Add Comment...");
	}
	else if(TitleType == ENodeTitleType::ListView)
	{
		return NSLOCTEXT("K2Node", "CommentBlock_ListTitle", "Comment");	
	}

	return FText::FromString(NodeComment);
}

FText UEdGraphNode_Comment::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	return GetNodeTitle(ENodeTitleType::ListView);
}

FLinearColor UEdGraphNode_Comment::GetNodeCommentColor() const
{
	// Only affects the 'zoomed out' comment bubble color, not the box itself
	return (bColorCommentBubble)
		? CommentColor 
		: FLinearColor::White;
}

void UEdGraphNode_Comment::ResizeNode(const FVector2f& NewSize)
{
	if (bCanResizeNode) 
	{
		NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(NewSize.Y);
		NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(NewSize.X);
	}
}

void UEdGraphNode_Comment::AddNodeUnderComment(UObject* Object)
{
	if( UEdGraphNode_Comment* ChildComment = Cast<UEdGraphNode_Comment>(Object))
	{
		CommentDepth = FMath::Min( CommentDepth, ChildComment->CommentDepth - 1 );
	}
	NodesUnderComment.Add(ObjectPtrWrap(Object));
}

void UEdGraphNode_Comment::ClearNodesUnderComment()
{
	NodesUnderComment.Empty();
}

void UEdGraphNode_Comment::SetBounds(const class FSlateRect& Rect)
{
	NodePosX = UE::LWC::FloatToIntCastChecked<int32>(Rect.Left);
	NodePosY = UE::LWC::FloatToIntCastChecked<int32>(Rect.Top);

	FVector2D Size = Rect.GetSize();
	NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(Size.X);
	NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(Size.Y);
}

const FCommentNodeSet& UEdGraphNode_Comment::GetNodesUnderComment() const
{
	return ObjectPtrDecay(NodesUnderComment);
}

void UEdGraphNode_Comment::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
	CachedTooltip.MarkDirty();
}

TSharedPtr<class INameValidatorInterface> UEdGraphNode_Comment::MakeNameValidator() const
{
	// Comments can be duplicated, etc...
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

bool UEdGraphNode_Comment::IsSelectedInEditor() const
{
	if (SelectionState == ESelectionState::Inherited)
	{
		return Super::IsSelectedInEditor();
	}
	return SelectionState == ESelectionState::Selected;
}

void UEdGraphNode_Comment::SetSelectionState(const ESelectionState InSelectionState)
{
	SelectionState = InSelectionState;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
