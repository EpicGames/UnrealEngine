// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphToken.h"

#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/UObjectToken.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"

void FEdGraphToken::Create(const UObject* InObject, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	return CreateInternal(InObject, Log, OutMessage, OutSourceNodes, nullptr);
}

void FEdGraphToken::Create(const UEdGraphPin* InPin, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	if(InPin && InPin->GetOwningNode())
	{
		return CreateInternal(InPin->GetOwningNode(), Log, OutMessage, OutSourceNodes, InPin);
	}
}

void FEdGraphToken::Create(const TCHAR* String, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNode)
{
	OutMessage.AddToken( FTextToken::Create(FText::FromString(FString(String))) );
}

void FEdGraphToken::Create(const FField* InField, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes)
{
	Create(*InField->GetPathName(), Log, OutMessage, OutSourceNodes);
}

const UEdGraphPin* FEdGraphToken::GetPin() const
{
	return PinBeingReferenced.Get();
}

const UObject* FEdGraphToken::GetGraphObject() const
{
	return ObjectBeingReferenced.Get();
}

FEdGraphToken::FEdGraphToken(const UObject* InObject, const UEdGraphPin* InPin)
	: ObjectBeingReferenced(InObject)
	, PinBeingReferenced(InPin)
{
	if (InPin)
	{
		CachedText = InPin->GetDisplayName();
		if (CachedText.IsEmpty())
		{
			CachedText = NSLOCTEXT("MessageLog", "UnnamedPin", "<Unnamed>");
		}
	}
	else if (InObject)
	{
		if (const UEdGraphNode* Node = Cast<UEdGraphNode>(InObject))
		{
			CachedText = Node->GetNodeTitle(ENodeTitleType::ListView);
		}
		else if(const UClass* Class = Cast<UClass>(InObject))
		{
			// Remove the trailing C if that is the users preference:
			CachedText = FBlueprintEditorUtils::GetFriendlyClassDisplayName(Class);
		}
		else if(const UField* Field = Cast<UField>(InObject))
		{
			CachedText = Field->GetDisplayNameText();
		}
		else
		{
			CachedText = FText::FromString(InObject->GetName());
		}
	}
	else
	{
		CachedText = NSLOCTEXT("MessageLog", "NoneObjectToken", "<None>");
	}
}

void FEdGraphToken::CreateInternal(const UObject* InObject, FCompilerResultsLog* Log, FTokenizedMessage &OutMessage, TArray<UEdGraphNode*>& OutSourceNodes, const UEdGraphPin* Pin)
{
	UObject* SourceObject = Log->FindSourceObject(const_cast<UObject*>(InObject));
	OutMessage.AddToken( MakeShareable(new FEdGraphToken(SourceObject, Pin ? Log->FindSourcePin(Pin) : nullptr)));
	if (UEdGraphNode* SourceNode = Cast<UEdGraphNode>(SourceObject))
	{
		OutSourceNodes.Add(SourceNode);
		
		// If this node came from a macro it actually has two source nodes, look up the other source node and add that as well:
		if(const UEdGraph* OwningGraph = SourceNode->GetGraph())
		{
			if(const UEdGraphSchema* Schema = OwningGraph->GetSchema())
			{
				if(Schema->GetGraphType(OwningGraph) == GT_Macro)
				{
					if(UObject* MacroSourceObject = Log->FindSourceMacroInstance(Cast<const UEdGraphNode>(InObject)))	
					{
						OutMessage.AddToken( FTextToken::Create( NSLOCTEXT("EdGraphToken", "FromMacroInstance", "generated from expanding")) );
						OutMessage.AddToken( MakeShareable(new FEdGraphToken(MacroSourceObject, nullptr)) );
						if(UEdGraphNode* MacroSourceNode = Cast<UEdGraphNode>(MacroSourceObject))
						{
							OutSourceNodes.Add(MacroSourceNode);
						}
					}
				}
			}
		}
	}
	
	if(SourceObject)
	{
		// The message link is only used when the user double clicks on the line, we'll jump to the first source object by default:
		OutMessage.SetMessageLink(FUObjectToken::Create(SourceObject));
	}
}
