// Copyright Epic Games, Inc. All Rights Reserved.


#include "EdGraphUtilities.h"

#include "BlueprintUtilities.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_EditablePinBase.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Math/IntRect.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StringOutputDevice.h"
#include "Stats/Stats.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectMarks.h"
#include "UnrealExporter.h"
#include "Widgets/Notifications/SNotificationList.h"

DECLARE_CYCLE_STAT(TEXT("Clone Graph"), EKismetCompilerStats_CloneGraph, STATGROUP_KismetCompiler);
DECLARE_CYCLE_STAT(TEXT("Clone Graph - Post Process"), EKismetCompilerStats_CloneGraph_PostProcess, STATGROUP_KismetCompiler);

/////////////////////////////////////////////////////
// Local namespace

namespace
{
	// Reconcile other pin links:
	//   - Links between nodes within the copied set are fine
	//   - Links to nodes that were not copied need to be fixed up if the copy-paste was in the same graph or broken completely
	void PostProcessPastedNodePinLinks(TSet<UEdGraphNode*>& InNodes)
	{
		for (TSet<UEdGraphNode*>::TIterator It(InNodes); It; ++It)
		{
			UEdGraphNode* Node = *It;
			UEdGraph* CurrentGraph = Node->GetGraph();

			for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* ThisPin = Node->Pins[PinIndex];

				// Ensure on any NULL entry, as it means there was a problem importing the pin from text, and we should be alerted to that.
				if (ensure(ThisPin))
				{
					for (int32 LinkIndex = 0; LinkIndex < ThisPin->LinkedTo.Num(); )
					{
						UEdGraphPin* OtherPin = ThisPin->LinkedTo[LinkIndex];

						if (OtherPin == nullptr)
						{
							// Totally bogus link
							ThisPin->LinkedTo.RemoveAtSwap(LinkIndex);
						}
						else if (!InNodes.Contains(OtherPin->GetOwningNode()))
						{
							// It's a link across the selection set, so it should be broken
							OtherPin->LinkedTo.RemoveSwap(ThisPin);
							ThisPin->LinkedTo.RemoveAtSwap(LinkIndex);
						}
						else if (!OtherPin->LinkedTo.Contains(ThisPin))
						{
							// The link needs to be reciprocal
							check(OtherPin->GetOwningNode()->GetGraph() == CurrentGraph);
							OtherPin->LinkedTo.Add(ThisPin);
							++LinkIndex;
						}
						else
						{
							// Everything seems fine but sanity check the graph
							check(OtherPin->GetOwningNode()->GetGraph() == CurrentGraph);
							++LinkIndex;
						}
					}
				}
				else
				{
					// Remove NULL entries; these will be replaced with a default value when the node is reconstructed below.
					Node->Pins.RemoveAt(PinIndex--);
				}
			}
		}
	}
}

/////////////////////////////////////////////////////
// FGraphObjectTextFactory

/** Helper class used to paste a buffer of text and create nodes and pins from it */
struct FGraphObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	TSet<UEdGraphNode*> SpawnedNodes;
	TSet<UEdGraphNode*> SubstituteNodes;
	const UEdGraph* DestinationGraph;
	TSet<FName> ExtraNamesInUse;
	TSet<UEdGraphNode*> NodesToDestroy;
public:
	FGraphObjectTextFactory(const UEdGraph* InDestinationGraph)
		: FCustomizableTextObjectFactory(GWarn)
		, DestinationGraph(InDestinationGraph)
	{
	}

protected:
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		if (const UEdGraphNode* DefaultNode = Cast<UEdGraphNode>(ObjectClass->GetDefaultObject()))
		{
			// if the root node can't be created, don't continue to check sub-
			// objects (for like collapsed graphs, or anim state-machine nodes)
			bOmitSubObjs = true;

			if (DefaultNode->CanDuplicateNode())
			{
				if (DestinationGraph != NULL)
				{
					if (DefaultNode->CanCreateUnderSpecifiedSchema(DestinationGraph->GetSchema()))
					{
						return true;
					}
				}
				else
				{
					return true;
				}
			}
		}

		return false;
	}

	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(CreatedObject))
		{
			UEdGraphNode* CreatedNode = Node;
			if (!Node->CanPasteHere(DestinationGraph))
			{
				// Attempt to create a substitute node if it cannot be pasted (note: the return value can be NULL, indicating that the node cannot be pasted into the graph)
				CreatedNode = DestinationGraph->GetSchema()->CreateSubstituteNode(CreatedNode, DestinationGraph, &InstanceGraph, ExtraNamesInUse);
				SubstituteNodes.Add(CreatedNode);
			}

			if (Node != CreatedNode)
			{
				NodesToDestroy.Add(Node);
			}

			if (CreatedNode)
			{
				SpawnedNodes.Add(CreatedNode);

				CreatedNode->GetGraph()->Nodes.Add(CreatedNode);
			}
		}
	}

	virtual void PostProcessConstructedObjects() override
	{
		if (SubstituteNodes.Num() > 0)
		{
			FText NotificationText;
			if (SubstituteNodes.Contains(nullptr))
			{
				if (SubstituteNodes.Num() > 1)
				{
					NotificationText = NSLOCTEXT("EdGraphUtilities", "SubstituteAndSkippedNodesWarning", "One or more copied nodes were substituted and/or could not be pasted into this graph!");
				}
				else
				{
					NotificationText = NSLOCTEXT("EdGraphUtilities", "SkippedNodesWarning", "One or more copied nodes could not be pasted into this graph!");
				}
			}
			else
			{
				NotificationText = NSLOCTEXT("EdGraphUtilities", "SubstituteNodesWarning", "One or more copied nodes were substituted during paste!");
			}

			// Display a notification to inform the user that one or more nodes were substituted rather than pasted into the new graph.
			FNotificationInfo Info(NotificationText);
			Info.ExpireDuration = 3.0f;
			Info.bUseLargeFont = false;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if (Notification.IsValid())
			{
				Notification->SetCompletionState(SNotificationItem::CS_None);
			}
		}

		// Fix up pin cross-links, etc. before removing incompatible nodes below. Otherwise, BreakAllNodeLinks() will complain about non-reciprocating pin links.
		PostProcessPastedNodePinLinks(NodesToDestroy);

		for (UEdGraphNode* Node : NodesToDestroy)
		{
			// Move the old node into the transient package so that it is GC'd
			Node->BreakAllNodeLinks();
			Node->Rename(NULL, GetTransientPackage());
			Node->MarkAsGarbage();
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// FEdGraphUtilities

TArray< TSharedPtr<FGraphPanelNodeFactory> > FEdGraphUtilities::VisualNodeFactories;
TArray< TSharedPtr<FGraphPanelPinFactory> > FEdGraphUtilities::VisualPinFactories;
TArray< TSharedPtr<FGraphPanelPinConnectionFactory> > FEdGraphUtilities::VisualPinConnectionFactories;

// Call PostPasteNode on each node
void FEdGraphUtilities::PostProcessPastedNodes(TSet<UEdGraphNode*>& SpawnedNodes)
{
	// Run thru and fix up the node's pin links; they may point to invalid pins if the paste was to another graph
	PostProcessPastedNodePinLinks(SpawnedNodes);

	// Give every node a chance to deep copy associated resources, etc...
	for (TSet<UEdGraphNode*>::TIterator It(SpawnedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;

		Node->PostPasteNode();
		Node->ReconstructNode();

		// Ensure we have RF_Transactional set on all pasted nodes, as its not copied in the T3D format
		Node->SetFlags(RF_Transactional);
	}
}

UEdGraphPin* FEdGraphUtilities::GetNetFromPin(UEdGraphPin* Pin)
{
	if ((Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() > 0))
	{
		return Pin->LinkedTo[0];
	}
	else
	{
		return Pin;
	}
}

// Clones (deep copies) a UEdGraph, including all of it's nodes and pins and their links,
// maintaining a mapping from the clone to the source nodes (even across multiple clonings)
UEdGraph* FEdGraphUtilities::CloneGraph(UEdGraph* InSource, UObject* NewOuter, FCompilerResultsLog* MessageLog, bool bCloningForCompile)
{
	BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CloneGraph);

	// Duplicate the graph, keeping track of what was duplicated
	TMap<UObject*, UObject*> DuplicatedObjectList;

	UObject* UseOuter = (NewOuter != nullptr) ? NewOuter : GetTransientPackage();
	FObjectDuplicationParameters Parameters(InSource, UseOuter);
	Parameters.CreatedObjects = &DuplicatedObjectList;

	if (bCloningForCompile || (NewOuter == nullptr))
	{
		Parameters.ApplyFlags |= RF_Transient;
		Parameters.FlagMask &= ~RF_Transactional;
	}

	UEdGraph* ClonedGraph = CastChecked<UEdGraph>(StaticDuplicateObjectEx(Parameters));

	// During compilation, do not clone disabled (e.g. ghost) nodes from non-transient source graphs.
	const bool bExcludeDisabledNodes = bCloningForCompile && InSource && !InSource->HasAnyFlags(RF_Transient);

	// Exclude disabled nodes and/or store backtrack links from each duplicated object to the original source object.
	if (bExcludeDisabledNodes || MessageLog != nullptr)
	{
		BP_SCOPED_COMPILER_EVENT_STAT(EKismetCompilerStats_CloneGraph_PostProcess);
		
		for (TMap<UObject*, UObject*>::TIterator It(DuplicatedObjectList); It; ++It)
		{
			bool bIsExcludedNode = false;

			UObject* const Source = It.Key();
			UObject* const Dest = It.Value();

			UEdGraphNode* SrcNode = Cast<UEdGraphNode>(Source);
			UEdGraphNode* DstNode = Cast<UEdGraphNode>(Dest);
			if (SrcNode && DstNode)
			{
				// Determine whether the source is a disabled node that should be excluded from the cloned graph.
				bIsExcludedNode = bExcludeDisabledNodes && !SrcNode->IsNodeEnabled();

				// associate pins, no known case of StaticDuplicateObjectEx resulting in a different number of pins, but
				// if that does happen we just associate as many pins as we can:
				ensure(SrcNode->Pins.Num() == DstNode->Pins.Num());
				for (int32 PinIdx = 0; PinIdx < SrcNode->Pins.Num() && PinIdx < DstNode->Pins.Num(); ++PinIdx)
				{
					UEdGraphPin* SrcPin = SrcNode->Pins[PinIdx];
					UEdGraphPin* DstPin = DstNode->Pins[PinIdx];
					if (ensure(SrcPin && DstPin))
					{
						// Don't record pins to the backtrack map if this is an excluded node.
						if (bIsExcludedNode)
						{
							// Patch any input links to pass through to the other side.
							if (DstPin->Direction == EGPD_Input && DstPin->LinkedTo.Num() > 0)
							{
								UEdGraphPin* PassThroughPin = DstNode->GetPassThroughPin(DstPin);
								if (PassThroughPin != nullptr && PassThroughPin->LinkedTo.Num() > 0)
								{
									for (UEdGraphPin* OutputPin : DstPin->LinkedTo)
									{
										for (UEdGraphPin* InputPin : PassThroughPin->LinkedTo)
										{
											InputPin->LinkedTo.Add(OutputPin);
											OutputPin->LinkedTo.Add(InputPin);
										}
									}
								}
							}
						}
						else if (MessageLog)
						{
							MessageLog->NotifyIntermediatePinCreation(DstPin, SrcPin);
						}
					}
				}

				if (bIsExcludedNode)
				{
					// Break remaining node links, if any exist.
					DstNode->BreakAllNodeLinks();

					// Remove the node from the cloned graph, if valid.
					if (ClonedGraph)
					{
						ClonedGraph->Nodes.Remove(DstNode);
					}
				}
			}

			// Don't record excluded nodes to the backtrack map.
			if (MessageLog && !bIsExcludedNode)
			{
				MessageLog->NotifyIntermediateObjectCreation(Dest, Source);
			}
		}
	}

	return ClonedGraph;
}

// Clones the content from SourceGraph and merges it into MergeTarget; including merging/flattening all of the children from the SourceGraph into MergeTarget
void FEdGraphUtilities::CloneAndMergeGraphIn(UEdGraph* MergeTarget, UEdGraph* SourceGraph, FCompilerResultsLog& MessageLog, bool bRequireSchemaMatch, bool bInIsCompiling/* = false*/, TArray<UEdGraphNode*>* OutClonedNodes)
{
	// Clone the graph, then move all of it's children
	UEdGraph* ClonedGraph = CloneGraph(SourceGraph, NULL, &MessageLog, true);
	MergeChildrenGraphsIn(ClonedGraph, ClonedGraph, bRequireSchemaMatch, false, &MessageLog);

	// Duplicate the list of cloned nodes
	if (OutClonedNodes != NULL)
	{
		OutClonedNodes->Append(ClonedGraph->Nodes);
	}

	// Determine if we are regenerating a blueprint on load
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(MergeTarget);
	const bool bIsLoading = Blueprint ? Blueprint->bIsRegeneratingOnLoad : false;

	// Move them all to the destination
	ClonedGraph->MoveNodesToAnotherGraph(MergeTarget, IsAsyncLoading() || bIsLoading, bInIsCompiling);
}

// Moves the contents of all of the children graphs (recursively) into the target graph.  This does not clone, it's destructive to the source
void FEdGraphUtilities::MergeChildrenGraphsIn(UEdGraph* MergeTarget, UEdGraph* ParentGraph, bool bRequireSchemaMatch, bool bInIsCompiling/* = false*/, FCompilerResultsLog* MessageLog/* = nullptr*/)
{
	// Determine if we are regenerating a blueprint on load
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(MergeTarget);
	const bool bIsLoading = Blueprint ? Blueprint->bIsRegeneratingOnLoad : false;

	// Merge all children nodes in too
	for (int32 Index = 0; Index < ParentGraph->SubGraphs.Num(); ++Index)
	{
		UEdGraph* ChildGraph = ParentGraph->SubGraphs[Index];

		auto NodeOwner = Cast<const UEdGraphNode>(ChildGraph ? ChildGraph->GetOuter() : nullptr);
		const bool bNonVirtualGraph = NodeOwner ? NodeOwner->ShouldMergeChildGraphs() : true;

		// Only merge children in with the same schema as the parent
		auto ChildSchema = ChildGraph ? ChildGraph->GetSchema() : nullptr;
		auto TargetSchema = MergeTarget ? MergeTarget->GetSchema() : nullptr;
		const bool bSchemaMatches = ChildSchema && TargetSchema && ChildSchema->GetClass()->IsChildOf(TargetSchema->GetClass());
		const bool bDoMerge = bNonVirtualGraph && (!bRequireSchemaMatch || bSchemaMatches);
		if (bDoMerge)
		{
			// Even if we don't require a match to recurse, we do to actually copy the nodes
			if (bSchemaMatches)
			{
				ChildGraph->MoveNodesToAnotherGraph(MergeTarget, IsAsyncLoading() || bIsLoading, bInIsCompiling);
			}
			else if (bRequireSchemaMatch && NodeOwner && MessageLog)
			{
				// If merging requires a schema match and we have a valid owner for the child graph, throw an error message on the outer node context.
				MessageLog->Error(*NSLOCTEXT("EdGraphUtilities", "CannotMergeChildGraph_Error", "Unable to merge child graph for node @@. The child graph's schema is incompatible with the outer graph.").ToString(), MessageLog->FindSourceObject(NodeOwner));
			}

			MergeChildrenGraphsIn(MergeTarget, ChildGraph, bRequireSchemaMatch, bInIsCompiling, MessageLog);
		}
	}
}

// Tries to rename the graph to have a name similar to BaseName
void FEdGraphUtilities::RenameGraphCloseToName(UEdGraph* Graph, const FString& BaseName, int32 StartIndex)
{
	FString NewName = BaseName;

	int32 NameIndex = StartIndex;
	for (;;)
	{
		if (Graph->Rename(*NewName, Graph->GetOuter(), REN_Test))
		{
			UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
			Graph->Rename(*NewName, Graph->GetOuter(), (BP->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
			return;
		}

		NewName = FString::Printf(TEXT("%s_%d"), *BaseName, NameIndex);
		++NameIndex;
	}
}

void FEdGraphUtilities::RenameGraphToNameOrCloseToName(UEdGraph* Graph, const FString& DesiredName)
{
	if (Graph->Rename(*DesiredName, Graph->GetOuter(), REN_Test))
	{
		UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
		Graph->Rename(*DesiredName, Graph->GetOuter(), (BP->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
	}
	else
	{
		RenameGraphCloseToName(Graph, DesiredName);
	}
}

// Exports a set of nodes to text
void FEdGraphUtilities::ExportNodesToText(TSet<UObject*> NodesToExport, /*out*/ FString& ExportedText)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	// Export each of the selected nodes
	UObject* LastOuter = NULL;
	for (TSet<UObject*>::TConstIterator NodeIt(NodesToExport); NodeIt; ++NodeIt)
	{
		UObject* Node = *NodeIt;

		// The nodes should all be from the same scope
		UObject* ThisOuter = Node->GetOuter();
		check((LastOuter == ThisOuter) || (LastOuter == NULL));
		LastOuter = ThisOuter;

		UExporter::ExportToOutputDevice(&Context, Node, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified|PPF_Copy|PPF_Delimited, false, ThisOuter);
	}

	ExportedText = Archive;
}

// Imports a set of previously exported nodes into a graph
void FEdGraphUtilities::ImportNodesFromText(UEdGraph* DestinationGraph, const FString& TextToImport, /*out*/ TSet<UEdGraphNode*>& ImportedNodeSet)
{
	// Turn the text buffer into objects
	FGraphObjectTextFactory Factory(DestinationGraph);
	Factory.ProcessBuffer(DestinationGraph, RF_Transactional, TextToImport);

	// Fix up pin cross-links, etc...
	FEdGraphUtilities::PostProcessPastedNodes(Factory.SpawnedNodes);

	// If a pin link wasn't resolved by now it was connected to something outside the clipboard and should be cleared:
	UEdGraphPin::ResolveAllPinReferences();

	ImportedNodeSet.Append(Factory.SpawnedNodes);
}

bool FEdGraphUtilities::CanImportNodesFromText(const UEdGraph* DestinationGraph, const FString& TextToImport )
{
	FGraphObjectTextFactory Factory(DestinationGraph);
	return Factory.CanCreateObjectsFromText(TextToImport);
}

FIntRect FEdGraphUtilities::CalculateApproximateNodeBoundaries(const TArray<UEdGraphNode*>& Nodes)
{
	int32 MinNodeX = +(int32)(1<<30);
	int32 MinNodeY = +(int32)(1<<30);
	int32 MaxNodeX = -(int32)(1<<30);
	int32 MaxNodeY = -(int32)(1<<30);

	for (auto NodeIt(Nodes.CreateConstIterator()); NodeIt; ++NodeIt)
	{
		UEdGraphNode* Node = *NodeIt;
		if (Node)
		{
			// Update stats
			MinNodeX = FMath::Min<int32>(MinNodeX, Node->NodePosX);
			MinNodeY = FMath::Min<int32>(MinNodeY, Node->NodePosY);
			MaxNodeX = FMath::Max<int32>(MaxNodeX, Node->NodePosX + Node->NodeWidth);
			MaxNodeY = FMath::Max<int32>(MaxNodeY, Node->NodePosY + Node->NodeHeight);
		}
	}

	const int32 AverageNodeWidth = 200;
	const int32 AverageNodeHeight = 128;

	return FIntRect(MinNodeX, MinNodeY, MaxNodeX + AverageNodeWidth, MaxNodeY + AverageNodeHeight);
}

void FEdGraphUtilities::CopyCommonState(UEdGraphNode* OldNode, UEdGraphNode* NewNode)
{
	// Copy common inheritable state (comment, location, etc...)
	NewNode->NodePosX = OldNode->NodePosX;
	NewNode->NodePosY = OldNode->NodePosY;
	NewNode->NodeWidth = OldNode->NodeWidth;
	NewNode->NodeHeight = OldNode->NodeHeight;
	NewNode->NodeComment = OldNode->NodeComment;
}

bool FEdGraphUtilities::IsSetParam(const UFunction* Function, const FName ParameterName)
{
	if (Function == nullptr)
	{
		return false;
	}

	const FString& RawMetaData = Function->GetMetaData(FBlueprintMetadata::MD_SetParam);
	if (RawMetaData.IsEmpty())
	{
		return false;
	}

	TArray<FString> SetParamPinGroups;
	{
		RawMetaData.ParseIntoArray(SetParamPinGroups, TEXT(","), true);
	}

	if (SetParamPinGroups.Num() > 0)
	{
		TArray<FString> GroupEntries;
		const FString ParameterNameStr = ParameterName.ToString();
		for (const FString& Entry : SetParamPinGroups)
		{
			GroupEntries.Reset();
			Entry.ParseIntoArray(GroupEntries, TEXT("|"), true);
			if (GroupEntries.Contains(ParameterNameStr))
			{
				return true;
			}
		}
	}

	return false;
}

bool FEdGraphUtilities::IsMapParam(const UFunction* Function, const FName ParameterName)
{
	if (Function == nullptr)
	{
		return false;
	}

	const FString& MapParamMetaData = Function->GetMetaData(FBlueprintMetadata::MD_MapParam);
	const FString& MapValueParamMetaData = Function->GetMetaData(FBlueprintMetadata::MD_MapValueParam);
	const FString& MapKeyParamMetaData = Function->GetMetaData(FBlueprintMetadata::MD_MapKeyParam);
	if (MapParamMetaData.IsEmpty() && MapValueParamMetaData.IsEmpty() && MapKeyParamMetaData.IsEmpty() )
	{
		return false;
	}

	TArray<FString> GroupEntries;
	const FString ParameterNameStr = ParameterName.ToString();

	const auto PipeSeparatedStringContains = [&GroupEntries, &ParameterNameStr](const FString& List)
	{
		GroupEntries.Reset();
		List.ParseIntoArray(GroupEntries, TEXT("|"), true);
		if (GroupEntries.Contains(ParameterNameStr))
		{
			return true;
		}
		return false;
	};

	return PipeSeparatedStringContains(MapParamMetaData)
		|| PipeSeparatedStringContains(MapValueParamMetaData) 
		|| PipeSeparatedStringContains(MapKeyParamMetaData);
}

bool FEdGraphUtilities::IsArrayDependentParam(const UFunction* Function, const FName ParameterName)
{
	if (Function == nullptr)
	{
		return false;
	}

	const FString& DependentPinMetaData = Function->GetMetaData(FBlueprintMetadata::MD_ArrayDependentParam);
	if( DependentPinMetaData.IsEmpty() )
	{
		return false;
	}

	TArray<FString> TypeDependentPinNames;
	DependentPinMetaData.ParseIntoArray(TypeDependentPinNames, TEXT(","), true);

	return TypeDependentPinNames.Contains(ParameterName.ToString());
}

bool FEdGraphUtilities::IsDynamicContainerParam(const UFunction* Function, const FName ParameterName)
{
	return FEdGraphUtilities::IsArrayDependentParam(Function, ParameterName) || FEdGraphUtilities::IsMapParam(Function, ParameterName) || FEdGraphUtilities::IsSetParam(Function, ParameterName);
}

UEdGraphPin* FEdGraphUtilities::FindArrayParamPin(const UFunction* Function, const UEdGraphNode* Node)
{
	return FEdGraphUtilities::FindPinFromMetaData(Function, Node, FBlueprintMetadata::MD_ArrayParam);
}

UEdGraphPin* FEdGraphUtilities::FindSetParamPin(const UFunction* Function, const UEdGraphNode* Node)
{
	return FEdGraphUtilities::FindPinFromMetaData(Function, Node, FBlueprintMetadata::MD_SetParam);
}
	
UEdGraphPin* FEdGraphUtilities::FindMapParamPin(const UFunction* Function, const UEdGraphNode* Node)
{
	return FEdGraphUtilities::FindPinFromMetaData(Function, Node, FBlueprintMetadata::MD_MapParam);
}

UEdGraphPin* FEdGraphUtilities::FindPinFromMetaData(const UFunction* Function, const UEdGraphNode* Node, FName MetaData )
{
	if(!Function || !Node)
	{
		return nullptr;
	}

	if(!Function->HasMetaData(MetaData))
	{
		return nullptr;
	}

	const FString& PinMetaData = Function->GetMetaData(MetaData);
	TArray<FString> ParamPinGroups;
	PinMetaData.ParseIntoArray(ParamPinGroups, TEXT(","), true);

	for (const FString& Entry : ParamPinGroups)
	{
		// split the group:
		TArray<FString> GroupEntries;
		Entry.ParseIntoArray(GroupEntries, TEXT("|"), true);
		// resolve pins
		for(const FString& PinName : GroupEntries)
		{
			if(UEdGraphPin* Pin = Node->FindPin(PinName))
			{
				return Pin;
			}
		}
	}

	return nullptr;
}

void FEdGraphUtilities::RegisterVisualNodeFactory(TSharedPtr<FGraphPanelNodeFactory> NewFactory)
{
	VisualNodeFactories.Add(NewFactory);
}

void FEdGraphUtilities::UnregisterVisualNodeFactory(TSharedPtr<FGraphPanelNodeFactory> OldFactory)
{
	VisualNodeFactories.Remove(OldFactory);
}

void FEdGraphUtilities::RegisterVisualPinFactory(TSharedPtr<FGraphPanelPinFactory> NewFactory)
{
	VisualPinFactories.Add(NewFactory);
}

void FEdGraphUtilities::UnregisterVisualPinFactory(TSharedPtr<FGraphPanelPinFactory> OldFactory)
{
	VisualPinFactories.Remove(OldFactory);
}

void FEdGraphUtilities::RegisterVisualPinConnectionFactory(TSharedPtr<FGraphPanelPinConnectionFactory> NewFactory)
{
    VisualPinConnectionFactories.Add(NewFactory);
}

void FEdGraphUtilities::UnregisterVisualPinConnectionFactory(TSharedPtr<FGraphPanelPinConnectionFactory> OldFactory)
{
    VisualPinConnectionFactories.Remove(OldFactory);
}


void FEdGraphUtilities::CopyPinDefaults(const UEdGraphNode* NodeFrom, UEdGraphNode* NodeTo)
{
	check(NodeFrom && NodeTo);

	UK2Node_EditablePinBase* WithUserDefinedPins = Cast<UK2Node_EditablePinBase>(NodeTo);

	for (UEdGraphPin* ToPin : NodeTo->Pins)
	{
		if (UEdGraphPin * FromPin = NodeFrom->FindPin(ToPin->GetName()))
		{
			ToPin->DefaultValue = FromPin->DefaultValue;
			ToPin->DefaultObject = FromPin->DefaultObject;
			ToPin->DefaultTextValue = FromPin->DefaultTextValue;

			if (WithUserDefinedPins)
			{
				// Copy defaults to user defined pins
				WithUserDefinedPins->PinDefaultValueChanged(FromPin);
			}
			else
			{
				// No user defined pins implies we're copying to a function call node.
				// Ensure that underlying changes to the entry node default are reflected to unmodified call nodes.
				ToPin->AutogeneratedDefaultValue = FromPin->DefaultValue;
			}
		}
	}
}

void FEdGraphUtilities::GetPinConnectionMap(const UEdGraphNode* Node, TMap<FString, TSet<UEdGraphPin*>>& OutPinConnections)
{
	check(Node);

	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (Pin != nullptr)
		{
			FString PinName = Pin->GetName();

			// If this is the first time seeing this pin name, add a new set
			if (!OutPinConnections.Contains(PinName))
			{
				OutPinConnections.Add(PinName, TSet<UEdGraphPin*>());
			}
			else
			{
				// There are no pins connected to this
				continue;
			}

			for (UEdGraphPin* ConnectedPin : Pin->LinkedTo)
			{
				if (ConnectedPin != nullptr)
				{
					// Add to the array of nodes at this pin name
					OutPinConnections[PinName].Add(ConnectedPin);
				}
			}
		}
	}
}

void FEdGraphUtilities::ReconnectPinMap(UEdGraphNode* Node, const TMap<FString, TSet<UEdGraphPin*>>& PinConnections)
{
	check(Node);

	for (UEdGraphPin* const NewPin : Node->Pins)
	{
		const FString& NewPinName = NewPin->GetName();
		if (PinConnections.Contains(NewPinName))
		{
			// Connect the new pins here
			for (UEdGraphPin* OldPin : PinConnections[NewPinName])
			{
				NewPin->MakeLinkTo(OldPin);
			}
		}
	}
}

void FEdGraphUtilities::FNodeVisitor::TraverseNodes(UEdGraphNode* Node)
{
	VisitedNodes.Add(Node);
	TouchNode(Node);

	// Follow every pin
	for (int32 i = 0; i < Node->Pins.Num(); ++i)
	{
		UEdGraphPin* MyPin = Node->Pins[i];

		// And every connection to the pin
		for (int32 j = 0; j < MyPin->LinkedTo.Num(); ++j)
		{
			UEdGraphPin* OtherPin = MyPin->LinkedTo[j];
			if (OtherPin)
			{
				UEdGraphNode* OtherNode = OtherPin->GetOwningNodeUnchecked();
				if (OtherNode && !VisitedNodes.Contains(OtherNode))
				{
					TraverseNodes(OtherNode);
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FWeakGraphPinPtr

void FWeakGraphPinPtr::operator=(const class UEdGraphPin* Pin)
{
	PinReference = Pin;
	if(Pin && !Pin->IsPendingKill())
	{
		PinName = Pin->PinName;
		NodeObjectPtr = Pin->GetOwningNode();
	}
	else
	{
		Reset();
	}
}

UEdGraphPin* FWeakGraphPinPtr::Get()
{
	if (UEdGraphNode* Node = NodeObjectPtr.Get())
	{
		// If pin is no longer valid or has a different owner, attempt to fix up the reference
		UEdGraphPin* Pin = PinReference.Get();
		if (Pin == nullptr || Pin->GetOuter() != Node)
		{
			for (UEdGraphPin* TestPin : Node->Pins)
			{
				if (TestPin->PinName == PinName)
				{
					Pin = TestPin;
					PinReference = Pin;
					break;
				}
			}
		}

		return Pin;
	}

	return nullptr;
}
