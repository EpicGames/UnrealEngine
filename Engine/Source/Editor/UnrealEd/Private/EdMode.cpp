// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdMode.h"
#include "EditorModeTools.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "CanvasItem.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "StaticMeshResources.h"
#include "Toolkits/BaseToolkit.h"

#include "CanvasTypes.h"

//////////////////////////////////
// FEdMode

FEdMode::FEdMode()
	: CurrentTool(nullptr)
{
	bDrawKillZ = true;
}

FEdMode::~FEdMode()
{
}

void FEdMode::OnModeUnregistered( FEditorModeID ModeID )
{
	if( ModeID == Info.ID )
	{
		// This should be synonymous with "delete this"
		Owner->DestroyMode(ModeID);
	}
}

bool FEdMode::MouseEnter( FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y )
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->MouseEnter( ViewportClient, Viewport, x, y );
	}

	return false;
}

bool FEdMode::MouseLeave( FEditorViewportClient* ViewportClient,FViewport* Viewport )
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->MouseLeave( ViewportClient, Viewport );
	}

	return false;
}

bool FEdMode::MouseMove(FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y)
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->MouseMove( ViewportClient, Viewport, x, y );
	}

	return false;
}

bool FEdMode::ReceivedFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport)
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->ReceivedFocus( ViewportClient, Viewport );
	}

	return false;
}

bool FEdMode::LostFocus(FEditorViewportClient* ViewportClient,FViewport* Viewport)
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->LostFocus( ViewportClient, Viewport );
	}

	return false;
}

bool FEdMode::CapturedMouseMove( FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY )
{
	if( GetCurrentTool() )
	{
		return GetCurrentTool()->CapturedMouseMove( InViewportClient, InViewport, InMouseX, InMouseY );
	}

	return false;
}

bool FEdMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	// First try the currently selected tool
	if ((GetCurrentTool() != nullptr) && GetCurrentTool()->InputKey(ViewportClient, Viewport, Key, Event))
	{
		return true;
	}

	return false;
}

bool FEdMode::InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	FModeTool* Tool = GetCurrentTool();
	if (Tool)
	{
		return Tool->InputAxis(InViewportClient, Viewport, ControllerId, Key, Delta, DeltaTime);
	}

	return false;
}

bool FEdMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{	
	if (FLegacyEdModeWidgetHelper::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale))
	{
		return true;
	}

	if (GetCurrentTool())
	{
		return GetCurrentTool()->InputDelta(InViewportClient,InViewport,InDrag,InRot,InScale);
	}

	return false;
}

bool FEdMode::UsesTransformWidget() const
{
	if (GetCurrentTool())
	{
		return GetCurrentTool()->UseWidget();
	}

	return true;
}

bool FEdMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return FLegacyEdModeWidgetHelper::UsesTransformWidget(CheckMode);
}

bool FEdMode::UsesPropertyWidgets() const
{
	return false;
}

bool FEdMode::BoxSelect( FBox& InBox, bool InSelect )
{
	bool bResult = false;
	if( GetCurrentTool() )
	{
		bResult = GetCurrentTool()->BoxSelect( InBox, InSelect );
	}
	return bResult;
}

bool FEdMode::FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect )
{
	bool bResult = false;
	if( GetCurrentTool() )
	{
		bResult = GetCurrentTool()->FrustumSelect( InFrustum, InViewportClient, InSelect );
	}
	return bResult;
}

void FEdMode::SelectNone()
{
	if( GetCurrentTool() )
	{
		GetCurrentTool()->SelectNone();
	}
}

void FEdMode::Tick(FEditorViewportClient* ViewportClient,float DeltaTime)
{
	if( GetCurrentTool() )
	{
		GetCurrentTool()->Tick(ViewportClient,DeltaTime);
	}
}

bool FEdMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click)
{
	if (FLegacyEdModeWidgetHelper::HandleClick(InViewportClient, HitProxy, Click))
	{
		return true;
	}

	return false;
}

void FEdMode::Enter()
{
	// Update components for selected actors, in case the mode we just exited
	// was hijacking selection events selection and not updating components.
	for ( FSelectionIterator It( *Owner->GetSelectedActors() ) ; It ; ++It )
	{
		AActor* SelectedActor = CastChecked<AActor>( *It );
		SelectedActor->MarkComponentsRenderStateDirty();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FEditorDelegates::EditorModeIDEnter.Broadcast(GetID());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FEdMode::Exit()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FEditorDelegates::EditorModeIDExit.Broadcast(GetID());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UTexture2D* FEdMode::GetVertexTexture()
{
	return GEngine->DefaultBSPVertexTexture;
}

void FEdMode::SetCurrentTool( EModeTools InID )
{
	CurrentTool = FindTool( InID );
	check( CurrentTool );	// Tool not found!  This can't happen.

	CurrentToolChanged();
}

void FEdMode::SetCurrentTool( FModeTool* InModeTool )
{
	CurrentTool = InModeTool;
	check(CurrentTool);

	CurrentToolChanged();
}

FModeTool* FEdMode::FindTool( EModeTools InID )
{
	for( int32 x = 0 ; x < Tools.Num() ; ++x )
	{
		if( Tools[x]->GetID() == InID )
		{
			return Tools[x];
		}
	}

	UE_LOGF(LogEditorModes, Fatal, "FEdMode::FindTool failed to find tool %d", (int32)InID);
	return NULL;
}

void FEdMode::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
	// Let the current mode tool render if it wants to
	if (FModeTool* Tool = GetCurrentTool())
	{
		Tool->Render( View, Viewport, PDI );
	}

	FLegacyEdModeWidgetHelper::Render(View, Viewport, PDI);
}

void FEdMode::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
	// Let the current mode tool draw a HUD if it wants to
	FModeTool* Tool = GetCurrentTool();
	if( Tool )
	{
		Tool->DrawHUD( ViewportClient, Viewport, View, Canvas );
	}

	// If this viewport doesn't show mode widgets or the mode itself doesn't want them, leave.
	if (ShowModeWidgets())
	{
		FLegacyEdModeWidgetHelper::DrawHUD(ViewportClient, Viewport, View, Canvas);
	}
}

bool FEdMode::UsesToolkits() const
{
	return false;
}

UWorld* FEdMode::GetWorld() const
{
	return Owner->GetWorld();
}

FEditorModeTools* FEdMode::GetModeManager() const
{
	return Owner;
}

void FEdMode::RequestDeletion()
{
	Owner->DeactivateMode(GetID());
}

bool FEdMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bool bResult = false;
	if( GetCurrentTool() )
	{
		bResult = GetCurrentTool()->StartModify();
	}
	return bResult;
}

bool FEdMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bool bResult = false;
	if( GetCurrentTool() )
	{
		bResult = GetCurrentTool()->EndModify();
	}
	return bResult;
}

bool FEdMode::IsSnapRotationEnabled()
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled;
}
