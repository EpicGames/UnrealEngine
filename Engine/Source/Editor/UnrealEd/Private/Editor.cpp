// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor.h"
#include "Factories/Factory.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Containers/ArrayView.h"
#include "EditorReimportHandler.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Factories/ReimportFbxSkeletalMeshFactory.h"
#include "Factories/ReimportFbxStaticMeshFactory.h"
#include "Factories/ReimportFbxSceneFactory.h"
#include "Factories/ReimportSoundFactory.h"
#include "Factories/ReimportTextureFactory.h"
#include "Factories/PhysicalMaterialMaskFactory.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UObjectIterator.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "UnrealEngine.h"

// needed for the RemotePropagator



#include "Engine/SimpleConstructionScript.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Engine/InheritableComponentHandler.h"



#include "Interfaces/IMainFrameModule.h"





#if PLATFORM_WINDOWS
// For WAVEFORMATEXTENSIBLE
	#include "Windows/AllowWindowsPlatformTypes.h"
#include <mmreg.h>
	#include "Windows/HideWindowsPlatformTypes.h"
#endif


#include "DesktopPlatformModule.h"
#include "ObjectTools.h"



#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineAnalytics.h"

#include "EditorFramework/AssetImportData.h"

// AIMdule

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "K2Node_AddComponent.h"

#include "AutoReimport/AutoReimportUtilities.h"
#include "AssetToolsModule.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeManager.h"
#include "InterchangeResultsContainer.h"

#include "AssetCompilingManager.h"

#if WITH_EDITOR
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "UnrealEd.Editor"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FSimpleMulticastDelegate								FEditorDelegates::NewCurrentLevel;
FEditorDelegates::FOnMapChanged							FEditorDelegates::MapChange;
FSimpleMulticastDelegate								FEditorDelegates::LayerChange;
FSimpleMulticastDelegate								FEditorDelegates::PostUndoRedo;
FEditorDelegates::FOnModeChanged						FEditorDelegates::ChangeEditorMode;
FSimpleMulticastDelegate								FEditorDelegates::SurfProps;
FSimpleMulticastDelegate								FEditorDelegates::SelectedProps;
FEditorDelegates::FOnFitTextureToSurface				FEditorDelegates::FitTextureToSurface;
FSimpleMulticastDelegate								FEditorDelegates::ActorPropertiesChange;
FSimpleMulticastDelegate								FEditorDelegates::RefreshEditor;
FSimpleMulticastDelegate								FEditorDelegates::RefreshAllBrowsers;
FSimpleMulticastDelegate								FEditorDelegates::RefreshLayerBrowser;
FSimpleMulticastDelegate								FEditorDelegates::RefreshLevelBrowser;
FSimpleMulticastDelegate								FEditorDelegates::RefreshPrimitiveStatsBrowser;
FSimpleMulticastDelegate								FEditorDelegates::LoadSelectedAssetsIfNeeded;
FSimpleMulticastDelegate								FEditorDelegates::DisplayLoadErrors;
FEditorDelegates::FOnEditorModeTransitioned				FEditorDelegates::EditorModeEnter;
FEditorDelegates::FOnEditorModeTransitioned				FEditorDelegates::EditorModeExit;
FEditorDelegates::FOnEditorModeIDTransitioned			FEditorDelegates::EditorModeIDEnter;
FEditorDelegates::FOnEditorModeIDTransitioned			FEditorDelegates::EditorModeIDExit;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::StartPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::PreBeginPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::BeginPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::PrePIEEnded;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::PostPIEStarted;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::EndPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::ShutdownPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::PausePIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::ResumePIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::SingleStepPIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::OnPreSwitchBeginPIEAndSIE;
FEditorDelegates::FOnPIEEvent							FEditorDelegates::OnSwitchBeginPIEAndSIE;
FSimpleMulticastDelegate								FEditorDelegates::CancelPIE;
FEditorDelegates::FOnStandaloneLocalPlayEvent			FEditorDelegates::BeginStandaloneLocalPlay;
FSimpleMulticastDelegate								FEditorDelegates::PropertySelectionChange;
FSimpleMulticastDelegate								FEditorDelegates::PostLandscapeLayerUpdated;
FEditorDelegates::FOnPreSaveWorldWithContext			FEditorDelegates::PreSaveWorldWithContext;
FEditorDelegates::FOnPostSaveWorldWithContext			FEditorDelegates::PostSaveWorldWithContext;
FEditorDelegates::FOnPreSaveExternalActors				FEditorDelegates::PreSaveExternalActors;
FEditorDelegates::FOnPostSaveExternalActors				FEditorDelegates::PostSaveExternalActors;
FSimpleMulticastDelegate								FEditorDelegates::OnPreAssetValidation;
FSimpleMulticastDelegate								FEditorDelegates::OnPostAssetValidation;
FEditorDelegates::FOnFinishPickingBlueprintClass		FEditorDelegates::OnFinishPickingBlueprintClass;
FEditorDelegates::FOnNewAssetCreation					FEditorDelegates::OnConfigureNewAssetProperties;
FEditorDelegates::FOnNewAssetCreation					FEditorDelegates::OnNewAssetCreated;
FEditorDelegates::FOnPreDestructiveAssetAction          FEditorDelegates::OnPreDestructiveAssetAction;
FEditorDelegates::FOnPreDestructiveAssetActionWithArgs  FEditorDelegates::OnPreDestructiveAssetActionWithArgs;
FEditorDelegates::FOnAssetPreImport						FEditorDelegates::OnAssetPreImport;
FEditorDelegates::FOnAssetPostImport					FEditorDelegates::OnAssetPostImport;
FEditorDelegates::FOnAssetReimport						FEditorDelegates::OnAssetReimport;
FEditorDelegates::FOnNewActorsDropped					FEditorDelegates::OnNewActorsDropped;
FEditorDelegates::FOnNewActorsPlaced					FEditorDelegates::OnNewActorsPlaced;
FEditorDelegates::FOnEditorActorReplaced				FEditorDelegates::OnEditorActorReplaced;
FEditorDelegates::FOnGridSnappingChanged				FEditorDelegates::OnGridSnappingChanged;
FSimpleMulticastDelegate								FEditorDelegates::OnLightingBuildStarted;
FSimpleMulticastDelegate								FEditorDelegates::OnLightingBuildKept;
FSimpleMulticastDelegate								FEditorDelegates::OnLightingBuildFailed;
FSimpleMulticastDelegate								FEditorDelegates::OnLightingBuildSucceeded;
FEditorDelegates::FOnApplyObjectToActor					FEditorDelegates::OnApplyObjectToActor;
FEditorDelegates::FOnFocusViewportOnActors				FEditorDelegates::OnFocusViewportOnActors;
FEditorDelegates::FOnMapLoad							FEditorDelegates::OnMapLoad;
FEditorDelegates::FOnMapOpened							FEditorDelegates::OnMapOpened;
FEditorDelegates::FOnEditorCameraMoved					FEditorDelegates::OnEditorCameraMoved;
FEditorDelegates::FOnDollyPerspectiveCamera				FEditorDelegates::OnDollyPerspectiveCamera;
FSimpleMulticastDelegate								FEditorDelegates::OnShutdownPostPackagesSaved;
FEditorDelegates::FOnPackageDeleted						FEditorDelegates::OnPackageDeleted;
FEditorDelegates::FOnAssetsCanDelete					FEditorDelegates::OnAssetsCanDelete;
FEditorDelegates::FOnAssetsAddExtraObjectsToDelete		FEditorDelegates::OnAssetsAddExtraObjectsToDelete;
FEditorDelegates::FOnAddExtraObjectsToDelete			FEditorDelegates::OnAddExtraObjectsToDelete;
FEditorDelegates::FOnAssetsPreDelete					FEditorDelegates::OnAssetsPreDelete;
FEditorDelegates::FOnAssetsDeleted						FEditorDelegates::OnAssetsDeleted;
FEditorDelegates::FOnAssetDragStarted					FEditorDelegates::OnAssetDragStarted;
FEditorDelegates::FOnPreForceDeleteObjects				FEditorDelegates::OnPreForceDeleteObjects;
FSimpleMulticastDelegate								FEditorDelegates::OnEnableGestureRecognizerChanged;
FSimpleMulticastDelegate								FEditorDelegates::OnActionAxisMappingsChanged;
FEditorDelegates::FOnAddLevelToWorld					FEditorDelegates::OnAddLevelToWorld;
FEditorDelegates::FOnEditCutActorsBegin					FEditorDelegates::OnEditCutActorsBegin;
FEditorDelegates::FOnEditCutActorsEnd					FEditorDelegates::OnEditCutActorsEnd;
FEditorDelegates::FOnEditCopyActorsBegin				FEditorDelegates::OnEditCopyActorsBegin;
FEditorDelegates::FOnEditCopyActorsEnd					FEditorDelegates::OnEditCopyActorsEnd;
FEditorDelegates::FOnEditPasteActorsBegin				FEditorDelegates::OnEditPasteActorsBegin;
FEditorDelegates::FOnEditPasteActorsEnd					FEditorDelegates::OnEditPasteActorsEnd;
FEditorDelegates::FOnDuplicateActorsBegin				FEditorDelegates::OnDuplicateActorsBegin;
FEditorDelegates::FOnDuplicateActorsEnd					FEditorDelegates::OnDuplicateActorsEnd;
FEditorDelegates::FOnDeleteActorsBegin					FEditorDelegates::OnDeleteActorsBegin;
FEditorDelegates::FOnDeleteActorsEnd					FEditorDelegates::OnDeleteActorsEnd;
FEditorDelegates::FOnOpenReferenceViewer				FEditorDelegates::OnOpenReferenceViewer;
FEditorDelegates::FOnViewAssetIdentifiers				FEditorDelegates::OnOpenSizeMap;
FEditorDelegates::FOnViewAssetIdentifiers				FEditorDelegates::OnOpenAssetAudit;
FEditorDelegates::FOnViewAssetIdentifiers				FEditorDelegates::OnEditAssetIdentifiers;
FEditorDelegates::FOnRestartRequested					FEditorDelegates::OnRestartRequested;
FEditorDelegates::FOnEditorBoot							FEditorDelegates::OnEditorBoot;
FEditorDelegates::FOnEditorLoadDefaultStartupMap		FEditorDelegates::OnEditorLoadDefaultStartupMap;
FEditorDelegates::FOnEditorInitialized					FEditorDelegates::OnEditorInitialized;
FSimpleMulticastDelegate								FEditorDelegates::OnEditorPreExit;
FEditorDelegates::FOnExternalContentResolved			FEditorDelegates::OnExternalContentResolved;
FEditorDelegates::FOnWorldGenerationProcess				FEditorDelegates::OnPreWorldGenerationProcess;
FEditorDelegates::FOnWorldGenerationProcess				FEditorDelegates::OnPostWorldGenerationProcess;


PRAGMA_ENABLE_DEPRECATION_WARNINGS

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/

UE_IMPLEMENT_STRUCT("/Script/UnrealEd", SlatePlayInEditorInfo);

//////////////////////////////////////////////////////////////////////////
// FReimportManager

FReimportManager* FReimportManager::Instance()
{
	static FReimportManager Inst;
	return &Inst;
}

void FReimportManager::RegisterHandler( FReimportHandler& InHandler )
{
	Handlers.AddUnique( &InHandler );
	bHandlersNeedSorting = true;
}

void FReimportManager::UnregisterHandler( FReimportHandler& InHandler )
{
	Handlers.Remove( &InHandler );
}

bool FReimportManager::CanReimport( UObject* Obj, TArray<FString> *ReimportSourceFilenames) const
{
	if ( Obj )
	{
		TArray<FString> SourceFilenames;
		for( int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex )
		{
			SourceFilenames.Empty();
			if ( Handlers[ HandlerIndex ]->CanReimport(Obj, SourceFilenames) )
			{
				if (ReimportSourceFilenames != nullptr)
				{
					(*ReimportSourceFilenames) = SourceFilenames;
				}
	
				return true;
			}
		}
	}
	
	if (ReimportSourceFilenames != nullptr)
	{
		ReimportSourceFilenames->Empty();
	}
	
	return false;
}

void FReimportManager::UpdateReimportPaths( UObject* Obj, const TArray<FString>& InFilenames )
{
	if (Obj)
	{
		SortHandlersIfNeeded();

		TArray<FString> UnusedExistingFilenames;
		auto* Handler = Handlers.FindByPredicate([&](FReimportHandler* InHandler){ return InHandler->CanReimport(Obj, UnusedExistingFilenames); });
		if (Handler)
		{
			(*Handler)->SetReimportPaths(Obj, InFilenames);
			Obj->MarkPackageDirty();
		}
	}
}

void FReimportManager::UpdateReimportPath(UObject* Obj, const FString& Filename, int32 SourceFileIndex)
{
	if (Obj)
	{
		SortHandlersIfNeeded();

		TArray<FString> UnusedExistingFilenames;
		auto* Handler = Handlers.FindByPredicate([&](FReimportHandler* InHandler) { return InHandler->CanReimport(Obj, UnusedExistingFilenames); });
		if (Handler)
		{
			if (SourceFileIndex == INDEX_NONE)
			{
				TArray<FString> Filenames;
				Filenames.Add(Filename);
				(*Handler)->SetReimportPaths(Obj, Filenames);
			}
			else
			{
				(*Handler)->SetReimportPaths(Obj, Filename, SourceFileIndex);
			}
			Obj->MarkPackageDirty();
		}
		else
		{
			//In case no Handler was found with the SourcefileName acquisition,
			// we then try to set the reimportpath 1 by 1 and break out as soon as the the SetReimportPaths then provided a CanReimport state.

			if (SourceFileIndex == INDEX_NONE)
			{
				TArray<FString> FileNames;
				FileNames.Add(Filename);

				TArray<FString> OutFileNames;

				for (int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex)
				{
					OutFileNames.Empty();
					Handlers[HandlerIndex]->SetReimportPaths(Obj, FileNames);
					if (Handlers[HandlerIndex]->CanReimport(Obj, OutFileNames))
					{
						break;
					}
				}
			}
			else
			{
				TArray<FString> FileNames;
				for (int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex)
				{
					FileNames.Empty();
					Handlers[HandlerIndex]->SetReimportPaths(Obj, Filename, SourceFileIndex);
					if (Handlers[HandlerIndex]->CanReimport(Obj, FileNames))
					{
						break;
					}
				}
			}

			Obj->MarkPackageDirty();
		}
	}
}


bool FReimportManager::Reimport(UObject* Obj, bool bAskForNewFileIfMissing, bool bShowNotification, FString PreferredReimportFile, FReimportHandler* SpecifiedReimportHandler, int32 SourceFileIndex, bool bForceNewFile /*= false*/, bool bAutomated /*= false*/, bool bInForceShowDialog /*= false*/)
{
	UE::Interchange::FAssetImportResultRef ImportResult = ReimportAsync(Obj, bAskForNewFileIfMissing, bShowNotification, PreferredReimportFile, SpecifiedReimportHandler, SourceFileIndex, bForceNewFile, bAutomated, bInForceShowDialog);
	ImportResult->WaitUntilDone();
	const TArray<UInterchangeResult*>& Results = ImportResult->GetResults()->GetResults();
	for (const UInterchangeResult* InterchangeResult : Results)
	{
		if (InterchangeResult->IsA<UInterchangeResultError_ReimportFail>())
		{
			return false;
		}
	}
	return true;
}

UE::Interchange::FAssetImportResultRef FReimportManager::ReimportAsync(UObject* Obj, bool bAskForNewFileIfMissing, bool bShowNotification, FString PreferredReimportFile, FReimportHandler* SpecifiedReimportHandler, int32 SourceFileIndex, bool bForceNewFile /*= false*/, bool bAutomated /*= false*/, bool bInForceShowDialog /*= false*/)
{
	UE::Interchange::FAssetImportResultRef ImportResultSynchronous = MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >();
	// Warn that were about to reimport, so prep for it
	PreReimport.Broadcast( Obj );

	bool bUseInterchangeFramework = UInterchangeManager::IsInterchangeImportEnabled();;
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	
	const int32 RealSourceFileIndex = SourceFileIndex == INDEX_NONE ? 0 : SourceFileIndex;
	
	bool bSuccess = false;
	if ( Obj )
	{
		SortHandlersIfNeeded();

		bool bValidSourceFilename = false;
		TArray<FString> SourceFilenames;

		FReimportHandler *CanReimportHandler = SpecifiedReimportHandler;
		if (CanReimportHandler)
		{
			CanReimportHandler->SetPreferredReimportPath(PreferredReimportFile);
		}
		auto TryToSetCanReimportHandler = [&]()
			{
				for (int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex)
				{
					SourceFilenames.Empty();
					Handlers[HandlerIndex]->SetPreferredReimportPath(PreferredReimportFile);
					if (Handlers[HandlerIndex]->CanReimport(Obj, SourceFilenames))
					{
						CanReimportHandler = Handlers[HandlerIndex];
						break;
					}
				}
			};

		if (CanReimportHandler == nullptr || !CanReimportHandler->CanReimport(Obj, SourceFilenames))
		{
			TryToSetCanReimportHandler();
		}

		bool bHasSourceFile = false;
		if (CanReimportHandler == nullptr)
		{
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Obj))
			{
				FAssetData AssetData(Obj);
				FAssetSourceFilesArgs GetSourceFilesArgs;
				GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&AssetData, 1);

				// Doesn't need to resolve the paths so it is a bit quicker
				GetSourceFilesArgs.FilePathFormat = EPathUse::Display;

				AssetDefinition->GetSourceFiles(GetSourceFilesArgs, [&bHasSourceFile](const FAssetSourceFilesResult& AssetImportInfo)
					{
						bHasSourceFile = true;

						return false;
					});
			}

			if (!bHasSourceFile)
			{
				if (SourceFilenames.IsValidIndex(RealSourceFileIndex))
				{
					SourceFilenames[RealSourceFileIndex].Empty();
				}
				else
				{
					//Add the missing entries
					SourceFilenames.AddDefaulted(RealSourceFileIndex - (SourceFilenames.Num() - 1));
				}

				//If Source file was not found, thenw e force a new file selection here so we can re-check the CanReimportHandler
				for (int32 FileIndex = 0; FileIndex < SourceFilenames.Num(); ++FileIndex)
				{
					GetNewReimportPath(Obj, SourceFilenames, FileIndex);
				}

				UpdateReimportPath(Obj, SourceFilenames[RealSourceFileIndex], RealSourceFileIndex);

				//Also setting the bForceNewFile to false as we already did force it.
				bForceNewFile = false;

				TryToSetCanReimportHandler();
			}
		}

		if (CanReimportHandler != nullptr)
		{
			TArray<int32> MissingFileIndex;

			// Check all filenames for missing files
			bool bMissingFiles = false;
			if (!bForceNewFile && SourceFilenames.Num() > 0)
			{
				for (int32 FileIndex = 0; FileIndex < SourceFilenames.Num(); ++FileIndex)
				{
					if (SourceFilenames[FileIndex].IsEmpty() || IFileManager::Get().FileSize(*SourceFilenames[FileIndex]) == INDEX_NONE || (bForceNewFile && SourceFileIndex == FileIndex))
					{
						if (SourceFileIndex == INDEX_NONE || SourceFileIndex == FileIndex)
						{
							MissingFileIndex.AddUnique(FileIndex);
							bMissingFiles = true;
						}
					}
				}
			}
			else
			{
				if (bForceNewFile)
				{
					if (SourceFilenames.IsValidIndex(RealSourceFileIndex))
					{
						SourceFilenames[RealSourceFileIndex].Empty();
					}
					else
					{
						//Add the missing entries
						SourceFilenames.AddDefaulted(RealSourceFileIndex - (SourceFilenames.Num() - 1));
					}
				}

				MissingFileIndex.AddUnique(RealSourceFileIndex);
				bMissingFiles = true;
			}

			bValidSourceFilename = true;
			if ((bAskForNewFileIfMissing || !PreferredReimportFile.IsEmpty()) && bMissingFiles )
			{
				if (!bAskForNewFileIfMissing && !PreferredReimportFile.IsEmpty())
				{
					SourceFilenames.Empty();
					SourceFilenames.Add(PreferredReimportFile);
				}
				else
				{
					for (int32 FileIndex : MissingFileIndex)
					{
						GetNewReimportPath(Obj, SourceFilenames, FileIndex);
					}
				}
				bool bAllSourceFileEmpty = true;
				for (int32 SourceIndex = 0; SourceIndex < SourceFilenames.Num(); ++SourceIndex)
				{
					if (!SourceFilenames[SourceIndex].IsEmpty())
					{
						bAllSourceFileEmpty = false;
						break;
					}
				}
				if ( SourceFilenames.Num() == 0 || bAllSourceFileEmpty)
				{
					// Failed to specify a new filename. Don't show a notification of the failure since the user exited on their own
					bValidSourceFilename = false;
					bShowNotification = false;
					SourceFilenames.Empty();
				}
				else
				{
					// A new filename was supplied, update the path
					for (int32 SourceIndex = 0; SourceIndex < SourceFilenames.Num(); ++SourceIndex)
					{
						if (!SourceFilenames[SourceIndex].IsEmpty())
						{
							CanReimportHandler->SetReimportPaths(Obj, SourceFilenames[SourceIndex], SourceIndex);
						}
					}
				}
			}
			else if (!PreferredReimportFile.IsEmpty() && !SourceFilenames.Contains(PreferredReimportFile))
			{
				// Reimporting the asset from a new file
				CanReimportHandler->SetReimportPaths(Obj, PreferredReimportFile, SourceFileIndex);
				//Update the local source file
				if (SourceFilenames.IsValidIndex(RealSourceFileIndex))
				{
					SourceFilenames[RealSourceFileIndex] = PreferredReimportFile;
				}
			}

			if ( bValidSourceFilename )
			{
				//Convert the import data if it's needed and choose a new valid reimport handler after the conversion is done.
				//This allow us to re-import:
				// Interchange -> Legacy Fbx    ---> Asset was imported with Interchange (gltf, fbx, obj, ...), Interchange is turn off for fbx and the provided source file is fbx
				// Legacy Fbx -> Interchange    ---> Asset was imported with Legacy Fbx, the file use for re-import is supported by Interchange (fbx, gltf, obj, ...)
				{
					const FString ReimportFilename = SourceFilenames.IsValidIndex(RealSourceFileIndex) ? SourceFilenames[RealSourceFileIndex] : FString();
					const FString ReimportFilenameExtension = FPaths::GetExtension(ReimportFilename).ToLower();
					//Convertion will return false if there is no conversion to do.
					if (InterchangeManager.ConvertImportData(Obj, ReimportFilenameExtension))
					{
						for (int32 NewFileHandlerIndex = 0; NewFileHandlerIndex < Handlers.Num(); ++NewFileHandlerIndex)
						{
							SourceFilenames.Empty();
							if (!PreferredReimportFile.IsEmpty())
							{
								Handlers[NewFileHandlerIndex]->SetPreferredReimportPath(PreferredReimportFile);
							}
							if (Handlers[NewFileHandlerIndex]->CanReimport(Obj, SourceFilenames))
							{
								CanReimportHandler = Handlers[NewFileHandlerIndex];
								break;
							}
						}
					}
				}

				if (bUseInterchangeFramework && CanReimportHandler->IsInterchangeFactory())
				{
					// Make sure SourceFilenames reflects the source filenames in Obj
					SourceFilenames.Empty();
					if ( CanReimportHandler->CanReimport(Obj, SourceFilenames) )
					{
						check( SourceFilenames.Num() > 0 );

						int32 RealValidSourceFileIndex = SourceFilenames.IsValidIndex(RealSourceFileIndex) ? RealSourceFileIndex : 0;
						UE::Interchange::FScopedSourceData ScopedSourceData(SourceFilenames[RealValidSourceFileIndex]);
						CanReimportHandler->SetReimportSourceIndex(Obj, SourceFileIndex);
						if (InterchangeManager.CanTranslateSourceData(ScopedSourceData.GetSourceData()))
						{
							FImportAssetParameters ImportAssetParameters;
							ImportAssetParameters.bIsAutomated = bAutomated || GIsAutomationTesting || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
							ImportAssetParameters.ReimportAsset = Obj;
							ImportAssetParameters.ReimportSourceIndex = SourceFileIndex;
							ImportAssetParameters.bForceShowDialog = bInForceShowDialog;
							UE::Interchange::FAssetImportResultRef ImportResult = InterchangeManager.ImportAssetAsync(FString(), ScopedSourceData.GetSourceData(), ImportAssetParameters);

							TFunction<void(UE::Interchange::FImportResult&)> AppendAndBroadcastImportResultIfNeeded =
								[](UE::Interchange::FImportResult& Result)
							{
								UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
								TStrongObjectPtr<UInterchangeResultsContainer> ResultsContainer(Result.GetResults());
								InterchangeManager.OnBatchImportComplete.Broadcast(ResultsContainer);
							};

							ImportResult->OnDone(AppendAndBroadcastImportResultIfNeeded);

							return ImportResult;
						}
					}
				}



				// Do the reimport
				const bool bOriginalAutomated = CanReimportHandler->IsAutomatedReimport();
				CanReimportHandler->SetAutomatedReimport(bAutomated);
				const bool bOriginalForceShowDialog = CanReimportHandler->IsForceShowDialog();
				CanReimportHandler->SetForceShowDialog(bInForceShowDialog);
				
				//Reimport the asset
				EReimportResult::Type Result = CanReimportHandler->Reimport( Obj, SourceFileIndex );
				
				CanReimportHandler->SetForceShowDialog(bOriginalForceShowDialog);
				CanReimportHandler->SetAutomatedReimport(bOriginalAutomated);
				// Even if the reimport has been successful, check that the originating object is still valid
				// The reimport might be a reimport to level which triggered the deletion of the object
				if( Result == EReimportResult::Succeeded && IsValid(Obj))
				{
					Obj->PostEditChange();
					GEditor->BroadcastObjectReimported(Obj);
					if (FEngineAnalytics::IsAvailable())
					{
						TArray<FAnalyticsEventAttribute> Attributes;
						Attributes.Add( FAnalyticsEventAttribute( TEXT( "ObjectType" ), Obj->GetClass()->GetName() ) );
						FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.AssetReimported"), Attributes);
					}
					bSuccess = true;
				}
				else if( Result == EReimportResult::Cancelled )
				{
					bShowNotification = false;
				}
			}
		}

		if ( bShowNotification )
		{
			// Send a notification of the results
			FText NotificationText;
			if ( bSuccess )
			{
				if ( bValidSourceFilename )
				{
					const FString FirstLeafFilename = FPaths::GetCleanFilename(SourceFilenames[0]);

					if ( SourceFilenames.Num() == 1 )
					{
						FFormatNamedArguments Args;
						Args.Add( TEXT("ObjectName"), FText::FromString( Obj->GetName() ) );
						Args.Add( TEXT("ObjectType"), FText::FromString( Obj->GetClass()->GetName() ) );
						Args.Add( TEXT("SourceFile"), FText::FromString( FirstLeafFilename ) );
						NotificationText = FText::Format( LOCTEXT("ReimportSuccessfulFrom", "Successfully Reimported: {ObjectName} ({ObjectType}) from file ({SourceFile})"), Args );
					}
					else
					{
						FFormatNamedArguments Args;
						Args.Add( TEXT("ObjectName"), FText::FromString( Obj->GetName() ) );
						Args.Add( TEXT("ObjectType"), FText::FromString( Obj->GetClass()->GetName() ) );
						Args.Add( TEXT("SourceFile"), FText::FromString( FirstLeafFilename ) );
						Args.Add( TEXT("Number"), SourceFilenames.Num() - 1 );
						NotificationText = FText::Format( LOCTEXT("ReimportSuccessfulMultiple", "Successfuly Reimported: {ObjectName} ({ObjectType}) from file ({SourceFile}) and {Number} more"), Args );
					}
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add( TEXT("ObjectName"), FText::FromString( Obj->GetName() ) );
					Args.Add( TEXT("ObjectType"), FText::FromString( Obj->GetClass()->GetName() ) );
					NotificationText = FText::Format( LOCTEXT("ReimportSuccessful", "Successfully Reimported: {ObjectName} ({ObjectType})"), Args );
				}
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("ObjectName"), FText::FromString( Obj->GetName() ) );
				Args.Add( TEXT("ObjectType"), FText::FromString( Obj->GetClass()->GetName() ) );
				NotificationText = FText::Format( LOCTEXT("ReimportFailed", "Failed to Reimport: {ObjectName} ({ObjectType})"), Args );
			}

			FNotificationInfo Info(NotificationText);
			Info.ExpireDuration = 3.0f;
			Info.bUseLargeFont = false;
			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if ( Notification.IsValid() )
			{
				Notification->SetCompletionState( bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail );
			}
		}
	}

	FAssetCompilingManager::Get().FinishCompilationForObjects({Obj});

	// Let listeners know whether the reimport was successful or not
	PostReimport.Broadcast( Obj, bSuccess );

	GEditor->RedrawAllViewports();

	if (!bSuccess)
	{
		//Add a ReimportFail message
		ImportResultSynchronous->GetResults()->Add<UInterchangeResultError_ReimportFail>();
	}
	ImportResultSynchronous->SetDone();

	return ImportResultSynchronous;
}

void FReimportManager::ValidateAllSourceFileAndReimport(TArray<UObject*> &ToImportObjects, bool bShowNotification, int32 SourceFileIndex, bool bForceNewFile /*= false*/, bool bAutomated /*= false*/)
{
	//Copy the array to prevent iteration assert if a reimport factory change the selection
	TArray<UObject*> CopyOfSelectedAssets;
	TMap<UObject*, TArray<int32>> MissingFileSelectedAssets;
	for (UObject *Asset : ToImportObjects)
	{
		TArray<FString> SourceFilenames;
		if (this->CanReimport(Asset, &SourceFilenames))
		{
			if (SourceFilenames.Num() == 0)
			{
				TArray<int32>& SourceIndexArray = MissingFileSelectedAssets.FindOrAdd(Asset);
				if (SourceIndexArray.Num() == 0)
				{
					// Insert an invalid index to indicate no file
					SourceIndexArray.Add(INDEX_NONE);
				}
			}
			else
			{
				bool bMissingFile = false;

				for (int32 FileIndex = 0; FileIndex < SourceFilenames.Num(); ++FileIndex)
				{
					FString SourceFilename = SourceFilenames[FileIndex];
					if (SourceFilename.IsEmpty() || IFileManager::Get().FileSize(*SourceFilename) == INDEX_NONE || (bForceNewFile && (FileIndex == SourceFileIndex || SourceFileIndex == INDEX_NONE)))
					{
						TArray<int32>& SourceIndexArray = MissingFileSelectedAssets.FindOrAdd(Asset);
						SourceIndexArray.Add(FileIndex);
						bMissingFile = true;
					}
				}

				if (!bMissingFile)
				{
					CopyOfSelectedAssets.Add(Asset);
				}
			}
		}
		else
		{
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Asset))
			{
				FAssetData AssetData(Asset);
				FAssetSourceFilesArgs GetSourceFilesArgs;
				GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&AssetData, 1);

				// Doesn't need to resolve the paths so it is a bit quicker
				GetSourceFilesArgs.FilePathFormat = EPathUse::Display;

				bool bHasSourceFile = false;
				AssetDefinition->GetSourceFiles(GetSourceFilesArgs, [&bHasSourceFile](const FAssetSourceFilesResult& AssetImportInfo)
					{
						bHasSourceFile = true;

						return false;
					});

				if (!bHasSourceFile)
				{
					TArray<int32>& SourceIndexArray = MissingFileSelectedAssets.FindOrAdd(Asset);
					if (SourceIndexArray.Num() == 0)
					{
						// Insert an invalid index to indicate no file
						SourceIndexArray.Add(INDEX_NONE);
					}
				}
			}
		}
	}

	if (MissingFileSelectedAssets.Num() > 0)
	{
		// Ask the user how to handle missing files before doing the re-import when there is more then one missing file and the "force new file" parameter is false
		// 1. Ask for missing file location for every missing file
		// 2. Ignore missing file asset when doing the re-import
		// 3. Cancel the whole reimport
		EAppReturnType::Type UserChoice = EAppReturnType::Type::Yes;
		if (!bForceNewFile && MissingFileSelectedAssets.Num() > 1)
		{
			//Pop the dialog box asking the question
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("MissingNumber"), FText::FromString(FString::FromInt(MissingFileSelectedAssets.Num())));
			int MaxListFile = 100;
			FString AssetToFileListString;
			for (auto Kvp : MissingFileSelectedAssets)
			{
				UObject *Asset = Kvp.Key;
				const TArray<int32>& SourceIndexArray = Kvp.Value;
				AssetToFileListString += TEXT("\n");
				if (MaxListFile == 0)
				{
					AssetToFileListString += TEXT("...");
					break;
				}
				TArray<FString> SourceFilenames;
				if (this->CanReimport(Asset, &SourceFilenames))
				{
					MaxListFile--;
					for (int32 FileIndex : SourceIndexArray)
					{
						if (SourceFilenames.IsValidIndex(FileIndex))
						{
							AssetToFileListString += FString::Printf(TEXT("Asset %s -> Missing file %s"), *(Asset->GetName()), *(SourceFilenames[FileIndex]));
						}
					}
				}
			}
			Arguments.Add(TEXT("AssetToFileList"), FText::FromString(AssetToFileListString));
			FText DialogText = FText::Format(LOCTEXT("ReimportMissingFileChoiceDialogMessage", "There is {MissingNumber} assets with missing source file path. Do you want to specify a new source file path for each asset?\n \"No\" will skip the reimport of all asset with a missing source file path.\n \"Cancel\" will cancel the whole reimport.\n{AssetToFileList}"), Arguments);
			const FText Title = LOCTEXT("ReimportMissingFileChoiceDialogMessageTitle", "Reimport missing files");

			UserChoice = FMessageDialog::Open(EAppMsgType::YesNoCancel, DialogText, Title);
		}

		//Ask missing file locations
		if (UserChoice == EAppReturnType::Type::Yes)
		{
			bool bCancelAll = true;
			//Ask the user for a new source reimport path for each asset
			for (auto Kvp : MissingFileSelectedAssets)
			{
				UObject *Asset = Kvp.Key;
				const TArray<int32>& SourceIndexArray = Kvp.Value;
				for (int32 FileIndex : SourceIndexArray)
				{
					TArray<FString> SourceFilenames;
					GetNewReimportPath(Asset, SourceFilenames, FileIndex);
					//The FileIndex can be INDEX_NONE in case the caller do not specify any source index, in that case we want to use the first index which is 0.
					int32 RealSourceFileIndex = FileIndex == INDEX_NONE ? 0 : FileIndex;
					if (!SourceFilenames.IsValidIndex(RealSourceFileIndex) || SourceFilenames[RealSourceFileIndex].IsEmpty())
					{
						continue;
					}
					bCancelAll = false;
					UpdateReimportPath(Asset, SourceFilenames[RealSourceFileIndex], RealSourceFileIndex);
					//We do not want to ask again the user for a file
					bForceNewFile = false;
				}
				//return if the operation is cancel and we have nothing to re-import
				if (bCancelAll)
				{
					return;
				}

				CopyOfSelectedAssets.Add(Asset);
			}
		}
		else if (UserChoice == EAppReturnType::Type::Cancel)
		{
			return;
		}
		//If user ignore those asset just not add them to CopyOfSelectedAssets
	}

	FReimportManager::Instance()->ReimportMultiple(CopyOfSelectedAssets, /*bAskForNewFileIfMissing=*/false, bShowNotification, TEXT(""), nullptr, SourceFileIndex, /*bForceNewFile=*/false, bAutomated);
}

void FReimportManager::AddReferencedObjects( FReferenceCollector& Collector )
{
	for(FReimportHandler* Handler : Handlers)
	{
		TObjectPtr<UObject>* Obj = Handler->GetFactoryObject();
		if(Obj && *Obj)
		{
			Collector.AddReferencedObject(*Obj);
		}
	}
}

void FReimportManager::SortHandlersIfNeeded()
{
	if (bHandlersNeedSorting)
	{
		// Use > operator because we want higher priorities earlier in the list
		Handlers.Sort([](const FReimportHandler& A, const FReimportHandler& B) { return A.GetPriority() > B.GetPriority(); });
		bHandlersNeedSorting = false;
	}
}

void FReimportManager::OnInterchangePostReimported(UObject* ReimportAsset) const
{
	if (!ReimportAsset)
	{
		return;
	}

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("ObjectType"), ReimportAsset->GetClass()->GetName()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.AssetReimported"), Attributes);
	}

	PostReimport.Broadcast(ReimportAsset, true);

	if (GEditor)
	{
		GEditor->BroadcastObjectReimported(ReimportAsset);
		GEditor->RedrawAllViewports();
	}
}

bool FReimportManager::ReimportMultiple(TArrayView<UObject*> Objects, bool bAskForNewFileIfMissing /*= false*/, bool bShowNotification /*= true*/, FString PreferredReimportFile /*= TEXT("")*/, FReimportHandler* SpecifiedReimportHandler /*= nullptr */, int32 SourceFileIndex /*= INDEX_NONE*/, bool bForceNewFile /*= false*/, bool bAutomated /*= false*/)
{
	bool bBulkSuccess = true;

	FScopedSlowTask BulkReimportTask((float)Objects.Num(), LOCTEXT("BulkReimport_Title", "Reimporting..."));

	for(UObject* CurrentObject : Objects)
	{
		if(CurrentObject)
		{
			FText SingleTaskTest = FText::Format(LOCTEXT("BulkReimport_SingleItem", "Reimporting {0}"), FText::FromString(CurrentObject->GetName()));
			FScopedSlowTask SingleObjectTask(1.0f, SingleTaskTest);
			SingleObjectTask.EnterProgressFrame(1.0f);
			UE::Interchange::FAssetImportResultRef ImportResult = ReimportAsync(CurrentObject, bAskForNewFileIfMissing, bShowNotification, PreferredReimportFile, SpecifiedReimportHandler, SourceFileIndex, bForceNewFile, bAutomated);
			const bool bAsync = ImportResult->GetStatus() == UE::Interchange::FImportResult::EStatus::InProgress;
			bool bResultSuccess = true;
			if (!bAsync)
			{
				const TArray<UInterchangeResult*>& Results = ImportResult->GetResults()->GetResults();
				for (const UInterchangeResult* InterchangeResult : Results)
				{
					if (InterchangeResult->IsA<UInterchangeResultError_ReimportFail>())
					{
						bResultSuccess = false;
					}
				}
			}
			bBulkSuccess = bBulkSuccess && bResultSuccess;
		}

		BulkReimportTask.EnterProgressFrame(1.0f);
	}

	//Cleanup the factories after using them
	for (int32 HandlerIndex = 0; HandlerIndex < Handlers.Num(); ++HandlerIndex)
	{
		Handlers[HandlerIndex]->PostImportCleanUp();
	}

	return bBulkSuccess;
}

void FReimportManager::GetNewReimportPath(UObject* Obj, TArray<FString>& InOutFilenames, int32 SourceFileIndex /*= INDEX_NONE*/)
{
	int32 RealSourceFileIndex = SourceFileIndex == INDEX_NONE ? 0 : SourceFileIndex;
	TArray<UObject*> ReturnObjects;
	FString FileTypes;
	FString AllExtensions;
	TArray<UFactory*> Factories;

	// Determine whether we will allow multi select and clear old filenames
	bool bAllowMultiSelect = SourceFileIndex == INDEX_NONE && InOutFilenames.Num() > 1;
	if (bAllowMultiSelect)
	{
		InOutFilenames.Empty();
	}
	else
	{
		if (!InOutFilenames.IsValidIndex(RealSourceFileIndex))
		{
			InOutFilenames.AddZeroed(RealSourceFileIndex - InOutFilenames.Num() + 1);
		}
		InOutFilenames[RealSourceFileIndex].Empty();
	}

	// Append the Interchange supported translator formats for this object
	TMultiMap<uint32, UFactory*> DummyFilterIndexToFactory;
	// Get the list of valid factories
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* CurrentClass = (*It);

		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->bEditorImport && Factory->DoesSupportClass(Obj->GetClass()))
			{
				Factories.Add(Factory);
			}
		}
	}
	//We now add all the factory extensions
	if (Factories.Num() > 0)
	{
		// Generate the file types and extensions represented by the selected factories
		ObjectTools::GenerateFactoryFileExtensions(Factories, FileTypes, AllExtensions, DummyFilterIndexToFactory);
	}

	if (UInterchangeManager::IsInterchangeImportEnabled())
	{
		//Get the extension interchange can translate for this object
		TArray<FString> TranslatorFormats = UInterchangeManager::GetInterchangeManager().GetSupportedFormatsForObject(Obj, RealSourceFileIndex);

		if (TranslatorFormats.Num() == 0)
		{
			//In case the asset was not imported or loose the sources file name, we want to be able to retrieve the possible formats.
			if (const UClass* FactoryClass = UInterchangeManager::GetInterchangeManager().GetRegisteredFactoryClass(Obj->GetClass()))
			{
				UInterchangeFactoryBase* FactoryBase = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
				EInterchangeTranslatorAssetType TranslatorAssetType = EInterchangeTranslatorAssetType::None;
				switch (FactoryBase->GetFactoryAssetType())
				{
					case EInterchangeFactoryAssetType::Animations:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Animations;
						break;
					case EInterchangeFactoryAssetType::Materials:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Materials;
						break;
					case EInterchangeFactoryAssetType::Meshes:
					case EInterchangeFactoryAssetType::Physics:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Meshes;
						break;
					case EInterchangeFactoryAssetType::Textures:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Textures;
						break;
					case EInterchangeFactoryAssetType::Grooms:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Grooms;
						break;
					case EInterchangeFactoryAssetType::Sounds:
						TranslatorAssetType = EInterchangeTranslatorAssetType::Sounds;
						break;
				};
				TranslatorFormats = UInterchangeManager::GetInterchangeManager().GetSupportedAssetTypeFormats(TranslatorAssetType, EInterchangeTranslatorType::Assets, false /*bStrictMatchTranslatorType*/);
			}
		}

		ObjectTools::AppendFormatsFileExtensions(TranslatorFormats, FileTypes, AllExtensions, DummyFilterIndexToFactory);
	}

	FString DefaultFolder;
	FString DefaultFile;
	
	TArray<FString> ExistingPaths = Utils::ExtractSourceFilePaths(Obj);
	if (ExistingPaths.Num() > 0)
	{
		DefaultFolder = FPaths::GetPath(ExistingPaths[0]);
		DefaultFile = FPaths::GetCleanFilename(ExistingPaths[0]);
		//Make sure we have at least the existing asset source files path extension.
		//If an asset is import with legacy importer and we switch importer to use interchange (or we go back to legacy)
		for (const FString& ExistingPath : ExistingPaths)
		{
			const FString& Extension = FPaths::GetExtension(ExistingPath);
			if (!AllExtensions.Contains(Extension))
			{
				AllExtensions.Append(TEXT(";*.") + Extension);
				FileTypes.Append(FString::Printf(TEXT("|Asset file source type (*.%s)|*.%s"), *Extension, *Extension));
			}
		}
	}

	FileTypes = FString::Printf(TEXT("All Files (%s)|%s|%s"), *AllExtensions, *AllExtensions, *FileTypes);

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if ( DesktopPlatform )
	{
		void* ParentWindowWindowHandle = NULL;

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if ( MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid() )
		{
			ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		FString Title = FString::Printf(TEXT("%s: %s"), *NSLOCTEXT("ReimportManager", "ImportDialogTitle", "Import For").ToString(), *Obj->GetName());
		if (SourceFileIndex != INDEX_NONE)
		{
			FAssetData AssetData(Obj);
			const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData);

			FAssetSourceFilesArgs GetSourceFilesArgs;
			GetSourceFilesArgs.Assets = TConstArrayView<FAssetData>(&AssetData, 1);
			GetSourceFilesArgs.FilePathFormat = EPathUse::Display;

			int32 SourceFileCount = 0;
			FString SourceDisplayLabel;
			AssetDefinition->GetSourceFiles(GetSourceFilesArgs, [&SourceFileCount, &SourceDisplayLabel, SourceFileIndex](const FAssetSourceFilesResult& AssetImportInfo)
			{
				++SourceFileCount;
				if (SourceFileIndex < SourceFileCount)
				{
					SourceDisplayLabel = AssetImportInfo.FilePath;
					return false;
				}

				return true;
			});
			
			if (SourceFileIndex >= 0 && SourceFileIndex < SourceFileCount)
			{
				Title = FString::Printf(TEXT("%s %s %s: %s"),
					*NSLOCTEXT("ReimportManager", "ImportDialogTitleLabelPart1", "Select").ToString(),
					*SourceDisplayLabel,
					*NSLOCTEXT("ReimportManager", "ImportDialogTitleLabelPart2", "Source File For").ToString(),
					*Obj->GetName());
			}
			else
			{
				FString SourceFileIndexStr = FString::FromInt(SourceFileIndex);
				Title = FString::Printf(TEXT("%s %s %s: %s"),
					*NSLOCTEXT("ReimportManager", "ImportDialogTitlePart1", "Select Source File Index").ToString(),
					*SourceFileIndexStr,
					*NSLOCTEXT("ReimportManager", "ImportDialogTitlePart2", "for").ToString(),
					*Obj->GetName());
			}
		}

		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			Title,
			*DefaultFolder,
			*DefaultFile,
			FileTypes,
			bAllowMultiSelect ? EFileDialogFlags::Multiple : EFileDialogFlags::None,
			OpenFilenames
			);
	}

	if ( bOpened )
	{
		if (bAllowMultiSelect)
		{
			for (int32 FileIndex = 0; FileIndex < OpenFilenames.Num(); ++FileIndex)
			{
				InOutFilenames.Add(OpenFilenames[FileIndex]);
			}
		}
		else
		{
			//Use the first valid entry
			if(OpenFilenames.Num() > 0)
			{
				InOutFilenames[RealSourceFileIndex] = OpenFilenames[0];
			}
		}
	}
}

FReimportManager::FReimportManager()
{
	// Create reimport handler for textures
	// NOTE: New factories can be created anywhere, inside or outside of editor
	// This is just here for convenience
	UReimportTextureFactory::StaticClass();

	// Create reimport handler for FBX static meshes
	UReimportFbxStaticMeshFactory::StaticClass();

	// Create reimport handler for FBX skeletal meshes
	UReimportFbxSkeletalMeshFactory::StaticClass();

	// Create reimport handler for FBX scene
	UReimportFbxSceneFactory::StaticClass();

	// Create reimport handler for PhysicalMaterialMasks
	UPhysicalMaterialMaskFactory::StaticClass();

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	InterchangePostReimportedDelegateHandle = InterchangeManager.OnAssetPostReimport.AddRaw(this, &FReimportManager::OnInterchangePostReimported);

	InterchangeManager.OnPreDestroyInterchangeManager.AddLambda([InterchangePostReimportedDelegateHandleClosure = InterchangePostReimportedDelegateHandle]()
		{
			if (InterchangePostReimportedDelegateHandleClosure.IsValid())
			{
				UInterchangeManager::GetInterchangeManager().OnAssetPostReimport.Remove(InterchangePostReimportedDelegateHandleClosure);
			}
		});
}

FReimportManager::~FReimportManager()
{
	Handlers.Empty();

	// you can't do much here because ~FReimportManager is called from FReimportManager::Instance at cexit shutdown time
}

int32 FReimportHandler::GetPriority() const
{
	return UFactory::GetDefaultImportPriority();
}

/*-----------------------------------------------------------------------------
	PIE helpers.
-----------------------------------------------------------------------------*/

/**
 * Sets GWorld to the passed in PlayWorld and sets a global flag indicating that we are playing
 * in the Editor.
 *
 * @param	PlayInEditorWorld		PlayWorld
 * @return	the original GWorld
 */
UWorld* SetPlayInEditorWorld( UWorld* PlayInEditorWorld )
{
	check(!GIsPlayInEditorWorld);
	UWorld* SavedWorld = GWorld;
	GIsPlayInEditorWorld = true;
	GWorld = PlayInEditorWorld;

	// Purge the existing scene interface from the editor world to avoid 2x GPU allocations with the additional play-in-editor world
	if (GEditor->EditorWorld != nullptr)
	{
		// Tear down the scene interface for the editor world
		GEditor->EditorWorld->PurgeScene();
	}

	if (FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(PlayInEditorWorld))
	{
		UE::SetPlayInEditorID(WorldContext->PIEInstance);
		UpdatePlayInEditorWorldDebugString(WorldContext);
	}

	return SavedWorld;
}

/**
 * Restores GWorld to the passed in one and reset the global flag indicating whether we are a PIE
 * world or not.
 *
 * @param EditorWorld	original world being edited
 */
void RestoreEditorWorld( UWorld* EditorWorld )
{
	check(GIsPlayInEditorWorld);
	GIsPlayInEditorWorld = false;
	GWorld = EditorWorld;
	UE::SetPlayInEditorID(INDEX_NONE);
	UpdatePlayInEditorWorldDebugString(nullptr);
}

/**
 * Takes an FName and checks to see that it is unique among all loaded objects.
 *
 * @param	InName		The name to check
 * @param	Outer		The context for validating this object name. Should be a group/package
 * @param	InReason	If the check fails, this string is filled in with the reason why.
 *
 * @return	1 if the name is valid, 0 if it is not
 */

bool IsUniqueObjectName( const FName& InName, UObject* Outer, FText* InReason )
{
	// See if the name is already in use.
	if( StaticFindObject( UObject::StaticClass(), Outer, *InName.ToString() ) != NULL )
	{
		if ( InReason != NULL )
		{
			*InReason = NSLOCTEXT("UnrealEd", "NameAlreadyInUse", "Name is already in use by another object.");
		}
		return false;
	}

	return true;
}

bool IsGloballyUniqueObjectName(const FName& InName, FText* InReason)
{
	// See if the name is already in use anywhere in the engine.
	if (StaticFindFirstObject(UObject::StaticClass(), *InName.ToString()) != NULL)
	{
		if (InReason != NULL)
		{
			*InReason = NSLOCTEXT("UnrealEd", "NameAlreadyInUse", "Name is already in use by another object.");
		}
		return false;
	}

	return true;
}

/**
 * Takes an FName and checks to see that it is unique among all loaded objects.
 *
 * @param	InName		The name to check
 * @param	Outer		The context for validating this object name. Should be a group/package.
 * @param	InReason	If the check fails, this string is filled in with the reason why.
 *
 * @return	1 if the name is valid, 0 if it is not
 */

bool IsUniqueObjectName( const FName& InName, UObject* Outer, FText& InReason )
{
	return IsUniqueObjectName(InName,Outer,&InReason);
}

//////////////////////////////////////////////////////////////////////////
// EditorUtilities

namespace EditorUtilities
{
	AActor* GetEditorWorldCounterpartActor( AActor* Actor )
	{
		const bool bIsSimActor = Actor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
		if( bIsSimActor && GEditor && GEditor->PlayWorld != NULL )
		{
			// Do we have a counterpart in the editor world?
			auto* SimWorldActor = Actor;
			if( GEditor->ObjectsThatExistInEditorWorld.Get( SimWorldActor ) )
			{
				// Find the counterpart level
				UWorld* EditorWorld = GEditor->EditorWorld;
				for( auto LevelIt( EditorWorld->GetLevelIterator() ); LevelIt; ++LevelIt )
				{
					auto* Level = *LevelIt;
					if( Level->GetFName() == SimWorldActor->GetLevel()->GetFName() )
					{
						// Find our counterpart actor
						AActor* EditorWorldActor = FindObject<AActor>( Level, *SimWorldActor->GetFName().ToString(), EFindObjectFlags::None ); // Don't match class exactly, because we support all classes derived from Actor as well!
						if( EditorWorldActor )
						{
							return EditorWorldActor;
						}
					}
				}
			}
		}

		return NULL;
	}

	AActor* GetSimWorldCounterpartActor( AActor* Actor )
	{
		const bool bIsSimActor = Actor->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
		if( !bIsSimActor && GEditor && GEditor->EditorWorld != NULL )
		{
			// Do we have a counterpart in the sim world?
			auto* EditorWorldActor = Actor;

			// Find the counterpart level
			UWorld* PlayWorld = GEditor->PlayWorld;
			if (PlayWorld != nullptr)
			{
				for (auto LevelIt(PlayWorld->GetLevelIterator()); LevelIt; ++LevelIt)
				{
					auto* Level = *LevelIt;
					if (Level->GetFName() == EditorWorldActor->GetLevel()->GetFName())
					{
						// Find our counterpart actor
						AActor* SimWorldActor = FindObject<AActor>(Level, *EditorWorldActor->GetFName().ToString(), EFindObjectFlags::None); // Don't match class exactly, because we support all classes derived from Actor as well!
						if (SimWorldActor && GEditor->ObjectsThatExistInEditorWorld.Get(SimWorldActor))
						{
							return SimWorldActor;
						}
					}
				}
			}
		}

		return NULL;
	}

	// Searches through the target components array of the target actor for the source component
	// TargetComponents array is passed in populated to avoid repeated refetching and StartIndex 
	// is updated as an optimization based on the assumption that the standard use case is iterating 
	// over two component arrays that will be parallel in order
	template<class AllocatorType = FDefaultAllocator>
	UActorComponent* FindMatchingComponentInstance( UActorComponent* SourceComponent, AActor* TargetActor, const TArray<UActorComponent*, AllocatorType>& TargetComponents, int32& StartIndex )
	{
		UActorComponent* TargetComponent = StartIndex < TargetComponents.Num() ? TargetComponents[ StartIndex ] : nullptr;

		// If the source and target components do not match (e.g. context-specific), attempt to find a match in the target's array elsewhere
		if( (SourceComponent != nullptr) 
			&& ((TargetComponent == nullptr) 
				|| (SourceComponent->GetClass() != TargetComponent->GetClass())
				|| (SourceComponent->GetFName() != TargetComponent->GetFName()) ))
		{
			const bool bSourceIsArchetype = SourceComponent->HasAnyFlags(RF_ArchetypeObject);
			// Reset the target component since it doesn't match the source
			TargetComponent = nullptr;

			const int32 NumTargetComponents = TargetComponents.Num();
			if (NumTargetComponents > 0)
			{
				// Attempt to locate a match elsewhere in the target's component list
				const int32 StartingIndex = (bSourceIsArchetype ? StartIndex : StartIndex + 1);
				int32 FindTargetComponentIndex = (StartingIndex >= NumTargetComponents) ? 0 : StartingIndex;
				do
				{
					UActorComponent* FindTargetComponent = TargetComponents[ FindTargetComponentIndex ];

					if (FindTargetComponent->GetClass() == SourceComponent->GetClass())
					{
						// In the case that the SourceComponent is an Archetype there is a better than even chance the name won't match due to the way the SCS
						// is set up, so we're actually going to reverse search the archetype chain
						if (bSourceIsArchetype)
						{
							UActorComponent* CheckComponent = FindTargetComponent;
							while (CheckComponent)
							{
								if ( SourceComponent == CheckComponent->GetArchetype())
								{
									TargetComponent = FindTargetComponent;
									StartIndex = FindTargetComponentIndex;
									break;
								}
								CheckComponent = Cast<UActorComponent>(CheckComponent->GetArchetype());
							}
							if (TargetComponent)
							{
								break;
							}
						}
						else
						{
							// If we found a match, update the target component and adjust the target index to the matching position
							if( FindTargetComponent != NULL && SourceComponent->GetFName() == FindTargetComponent->GetFName() )
							{
								TargetComponent = FindTargetComponent;
								StartIndex = FindTargetComponentIndex;
								break;
							}
						}
					}

					// Increment the index counter, and loop back to 0 if necessary
					if( ++FindTargetComponentIndex >= NumTargetComponents )
					{
						FindTargetComponentIndex = 0;
					}

				} while( FindTargetComponentIndex != StartIndex );
			}

			// If we still haven't found a match and we're targeting a class default object what we're really looking
			// for is an Archetype
			if(TargetComponent == nullptr && TargetActor->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
			{
				if (bSourceIsArchetype)
				{
					UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(SourceComponent->GetOuter());

					// If the target actor's class is a child of our owner and we're both archetypes, then we're actually looking for an overridden version of ourselves
					if (BPGC && TargetActor->GetClass()->IsChildOf(BPGC))
					{
						TargetComponent = Cast<UActorComponent>(TargetActor->GetClass()->FindArchetype(SourceComponent->GetClass(), SourceComponent->GetFName()));

						// If it is us, then we're done, we don't need to find this
						if (TargetComponent == SourceComponent)
						{
							TargetComponent = nullptr;
						}
					}
				}
				else
				{
					TargetComponent = CastChecked<UActorComponent>(SourceComponent->GetArchetype(), ECastCheckedType::NullAllowed);

					// If the returned target component is not from the direct class of the actor we're targeting, we need to insert an inheritable component
					if (TargetComponent && (TargetComponent->GetOuter() != TargetActor->GetClass()))
					{
						// This component doesn't exist in the hierarchy anywhere and we're not going to modify the CDO, so we'll drop it
						if (TargetComponent->HasAnyFlags(RF_ClassDefaultObject))
						{
							TargetComponent = nullptr;
						}
						else
						{
							UBlueprintGeneratedClass* BPGC = CastChecked<UBlueprintGeneratedClass>(TargetActor->GetClass());
							UBlueprint* Blueprint = CastChecked<UBlueprint>(BPGC->ClassGeneratedBy);
							UInheritableComponentHandler* InheritableComponentHandler = Blueprint->GetInheritableComponentHandler(true);
							if (InheritableComponentHandler)
							{
								FComponentKey Key;
								FName const SourceComponentName = SourceComponent->GetFName();

								BPGC = Cast<UBlueprintGeneratedClass>(BPGC->GetSuperClass());
								while (!Key.IsValid() && BPGC)
								{
									USCS_Node* SCSNode = BPGC->SimpleConstructionScript->FindSCSNode(SourceComponentName);
									if (!SCSNode)
									{
										UBlueprint* SuperBlueprint = CastChecked<UBlueprint>(BPGC->ClassGeneratedBy);
										for (UActorComponent* ComponentTemplate : BPGC->ComponentTemplates)
										{
											if (ComponentTemplate->GetFName() == SourceComponentName)
											{
												if (UEdGraph* UCSGraph = FBlueprintEditorUtils::FindUserConstructionScript(SuperBlueprint))
												{
													TArray<UK2Node_AddComponent*> ComponentNodes;
													UCSGraph->GetNodesOfClass<UK2Node_AddComponent>(ComponentNodes);

													for (UK2Node_AddComponent* UCSNode : ComponentNodes)
													{
														if (ComponentTemplate == UCSNode->GetTemplateFromNode())
														{
															Key = FComponentKey(SuperBlueprint, FUCSComponentId(UCSNode));
															break;
														}
													}
												}
												break;
											}
										}
									}
									else
									{
										Key = FComponentKey(SCSNode);
										break;
									}
									BPGC = Cast<UBlueprintGeneratedClass>(BPGC->GetSuperClass());
								}

								if (ensure(Key.IsValid()))
								{
									check(InheritableComponentHandler->GetOverridenComponentTemplate(Key) == nullptr);
									TargetComponent = InheritableComponentHandler->CreateOverridenComponentTemplate(Key);
								}
								else
								{
									TargetComponent = nullptr;
								}
							}
						}
					}
				}
			}
		}

		return TargetComponent;
	}


	UActorComponent* FindMatchingComponentInstance(UActorComponent* SourceComponent, AActor* TargetActor)
	{
		UActorComponent* MatchingComponent = NULL;
		int32 StartIndex = 0;

		if (TargetActor)
		{
			TInlineComponentArray<UActorComponent*> TargetComponents;
			TargetActor->GetComponents(TargetComponents);
			MatchingComponent = FindMatchingComponentInstance(SourceComponent, TargetActor, TargetComponents, StartIndex);
		}

		return MatchingComponent;
	}
	
	void CopySingleProperty(const UObject* const InSourceObject, UObject* const InTargetObject, FProperty* const InProperty)
	{
		// Copy the property. Go through all child properties to find UObject.
		//If the UObject is a PIE object, then replace it with a valid object.
		//If the object is replaced, then rehash the set/map that contained that object.

		TArray<FPropertyVisitorPath> PropertiesToRehash;
		auto VisitPred = [InTargetObject, &PropertiesToRehash](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
			{
				const FPropertyVisitorPath& Path = Context.Path;
				const FPropertyVisitorData& Data = Context.Data;
				const FProperty* Property = Path.Top().Property;
				const void* SourcePtr = Data.PropertyData;
				void* TargetPtr = Path.GetPropertyDataPtr(InTargetObject);
				check(TargetPtr);

				EPropertyVisitorControlFlow Result = EPropertyVisitorControlFlow::StepInto;
				bool bCopyPropertyValue = Path.Num() == 1;
				// If the property is referencing an actor or actor component in the PIE world, then we can try and fix that reference up to the equivalent
				// from the editor world; otherwise we have to skip it
				if (const FObjectPropertyBase* const ObjectProperty = CastField<FObjectPropertyBase>(Property))
				{
					bool bObjectValueChanged = false;

					UObject* const SourceObjectPropertyValue = ObjectProperty->GetObjectPropertyValue(SourcePtr);
					if (SourceObjectPropertyValue && SourceObjectPropertyValue->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
					{
						// Don't copy any PIE object.

						// REFERENCE an existing actor in the editor world from a REFERENCE in the PIE world
						if (SourceObjectPropertyValue->IsA(AActor::StaticClass()))
						{
							// We can try and fix-up an actor reference from the PIE world to instead be the version from the persistent world
							AActor* const EditorWorldActor = GetEditorWorldCounterpartActor(Cast<AActor>(SourceObjectPropertyValue));
							if (EditorWorldActor)
							{
								ObjectProperty->SetObjectPropertyValue(TargetPtr, EditorWorldActor);
								bObjectValueChanged = true;
							}
						}
						// REFERENCE an existing actor component in the editor world from a REFERENCE in the PIE world
						else if (SourceObjectPropertyValue->IsA(UActorComponent::StaticClass()) && InTargetObject->IsA(AActor::StaticClass()))
						{
							AActor* const TargetActor = Cast<AActor>(InTargetObject);
							TInlineComponentArray<UActorComponent*> TargetComponents;
							TargetActor->GetComponents(TargetComponents);

							// We can try and fix-up an actor component reference from the PIE world to instead be the version from the persistent world
							int32 TargetComponentIndex = 0;
							UActorComponent* const EditorWorldComponent = FindMatchingComponentInstance(Cast<UActorComponent>(SourceObjectPropertyValue), TargetActor, TargetComponents, TargetComponentIndex);
							if (EditorWorldComponent)
							{
								ObjectProperty->SetObjectPropertyValue(TargetPtr, EditorWorldComponent);
								bObjectValueChanged = true;
							}
						}
					}

					// TSet and TMap needs to rehash if modified.
					// TObjectPtr should not be used for keys (use FObjectKey instead).
					// TSoftObjectPtr are valid keys and could be modified above.
					if (bObjectValueChanged)
					{
						bCopyPropertyValue = false;
						const TArray<FPropertyVisitorInfo>& Infos = Path.GetPath();
						for (int32 Index = Infos.Num() - 1; Index >= 0; --Index)
						{
							if (CastField<FSetProperty>(Infos[Index].Property)
								|| CastField<FMapProperty>(Infos[Index].Property))
							{
								PropertiesToRehash.AddUnique(FPropertyVisitorPath(MakeArrayView(Infos.GetData(), Index+1)));
							}
						}
					}

					Result = EPropertyVisitorControlFlow::StepOver;
				}
				
				if (bCopyPropertyValue)
				{
					// For all the other property types. Only copy if first level.
					Property->CopyCompleteValue(TargetPtr, SourcePtr);
				}

				// As an optimization skip over container's of int/double/... only step into property that can contains UObject.
				auto CanPropertyContainsObject = [](const FProperty* Property)
					{
						const bool bPropertyCannotContainsObject = CastField<FNumericProperty>(Property)
							|| CastField<FBoolProperty>(Property)
							|| CastField<FEnumProperty>(Property)
							|| CastField<FNameProperty>(Property)
							|| CastField<FStrProperty>(Property)
							|| CastField<FTextProperty>(Property)
							|| CastField<FDelegateProperty>(Property)
							|| CastField<FMulticastDelegateProperty>(Property);
						return !bPropertyCannotContainsObject;
					};
				if (const FSetProperty* const SetProperty = CastField<FSetProperty>(Property))
				{
					if (!CanPropertyContainsObject(SetProperty->ElementProp))
					{
						Result = EPropertyVisitorControlFlow::StepOver;
					}
				}
				else if (const FMapProperty* const MapProperty = CastField<FMapProperty>(Property))
				{
					if (!CanPropertyContainsObject(MapProperty->KeyProp)
						&& !CanPropertyContainsObject(MapProperty->ValueProp))
					{
						Result = EPropertyVisitorControlFlow::StepOver;
					}
				}
				else if (const FArrayProperty* const ArrayProperty = CastField<FArrayProperty>(Property))
				{
					if (!CanPropertyContainsObject(ArrayProperty->Inner))
					{
						Result = EPropertyVisitorControlFlow::StepOver;
					}
				}

				return Result;
			};

		UStruct* PropertyOwner = InProperty->GetOwnerStruct();
		check(PropertyOwner);
		FPropertyVisitorPath Path;
		void* SourceObjectPtr = const_cast<UObject*>(InSourceObject);
		FPropertyVisitorData Data(SourceObjectPtr, /*ParentStructData*/nullptr);
		FPropertyVisitorContext Context(Path, Data);
		PropertyVisitorHelpers::VisitProperty(PropertyOwner, InProperty, Context, VisitPred);

		// Rehash TSet and TMap in reverse order in case one contains the other.
		for (int32 Index = PropertiesToRehash.Num() - 1; Index >= 0; --Index)
		{
			const FPropertyVisitorPath& PathToRehash = PropertiesToRehash[Index];
			const FProperty* Property = PathToRehash.Top().Property;
			void* TargetPtr = PathToRehash.GetPropertyDataPtr(InTargetObject);
			if (const FSetProperty* const SetProperty = CastField<FSetProperty>(Property))
			{
				FScriptSetHelper SetHelper(SetProperty, TargetPtr);
				SetHelper.Rehash();
			}
			else
			{
				const FMapProperty* const MapProperty = CastFieldChecked<FMapProperty>(Property);
				FScriptMapHelper MapHelper(MapProperty, TargetPtr);
				MapHelper.Rehash();
			}
		}
	}
	
	int32 CopyActorProperties( AActor* SourceActor, AActor* TargetActor, const FCopyOptions& Options )
	{
		check( SourceActor != nullptr && TargetActor != nullptr );

		const bool bIsPreviewing = ( Options.Flags & ECopyOptions::PreviewOnly ) != 0;

		int32 CopiedPropertyCount = 0;

		// The actor's classes should be compatible, right?
		UClass* ActorClass = SourceActor->GetClass();
		check( TargetActor->GetClass()->IsChildOf(ActorClass) );

		// Get archetype instances for propagation (if requested)
		TArray<AActor*> ArchetypeInstances;
		if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
		{
			TArray<UObject*> ObjectArchetypeInstances;
			TargetActor->GetArchetypeInstances(ObjectArchetypeInstances);

			for (UObject* ObjectArchetype : ObjectArchetypeInstances)
			{
				if (AActor* ActorArchetype = Cast<AActor>(ObjectArchetype))
				{
					ArchetypeInstances.Add(ActorArchetype);
				}
			}
		}

		bool bTransformChanged = false;

		// Copy non-component properties from the old actor to the new actor
		// @todo sequencer: Most of this block of code was borrowed (pasted) from UEditorEngine::ConvertActors().  If we end up being able to share these code bodies, that would be nice!
		TSet<UObject*> ModifiedObjects;
		for( FProperty* Property = ActorClass->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext )
		{
			const bool bIsTransient = !!( Property->PropertyFlags & CPF_Transient );
			const bool bIsComponentContainer = !!( Property->PropertyFlags & CPF_ContainsInstancedReference );
			const bool bIsComponentProp = !!( Property->PropertyFlags & ( CPF_InstancedReference | CPF_ContainsInstancedReference ) );
			const bool bIsBlueprintReadonly = !!(Options.Flags & ECopyOptions::FilterBlueprintReadOnly) && !!( Property->PropertyFlags & CPF_BlueprintReadOnly );
			const bool bIsIdentical = Property->Identical_InContainer( SourceActor, TargetActor );

			if( !bIsTransient && !bIsIdentical && !bIsComponentContainer && !bIsComponentProp && !bIsBlueprintReadonly)
			{
				const bool bIsSafeToCopy = (!( Options.Flags & ECopyOptions::OnlyCopyEditOrInterpProperties ) || ( Property->HasAnyPropertyFlags( CPF_Edit | CPF_Interp ) ))
				                        && (!( Options.Flags & ECopyOptions::SkipInstanceOnlyProperties) || ( !Property->HasAllPropertyFlags(CPF_DisableEditOnTemplate) ) );
				if( bIsSafeToCopy )
				{
					if (!Options.CanCopyProperty(*Property, *SourceActor))
					{
						continue;
					}

					if( !bIsPreviewing )
					{
						if( !ModifiedObjects.Contains(TargetActor) )
						{
							// Start modifying the target object
							TargetActor->Modify();
							ModifiedObjects.Add(TargetActor);
						}

						if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
						{
							TargetActor->PreEditChange( Property );
						}

						// Determine which archetype instances match the current property value of the target actor (before it gets changed). We only want to propagate the change to those instances.
						TArray<UObject*> ArchetypeInstancesToChange;
						if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
						{
							for( AActor* ArchetypeInstance : ArchetypeInstances )
							{
								if( ArchetypeInstance != nullptr && Property->Identical_InContainer( ArchetypeInstance, TargetActor ) )
								{
									ArchetypeInstancesToChange.Add( ArchetypeInstance );
								}
							}
						}

						CopySingleProperty(SourceActor, TargetActor, Property);

						if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
						{
							FPropertyChangedEvent PropertyChangedEvent( Property );
							TargetActor->PostEditChangeProperty( PropertyChangedEvent );
						}

						if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
						{
							for( int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstancesToChange.Num(); ++InstanceIndex )
							{
								UObject* ArchetypeInstance = ArchetypeInstancesToChange[InstanceIndex];
								if( ArchetypeInstance != nullptr )
								{
									if( !ModifiedObjects.Contains(ArchetypeInstance) )
									{
										ArchetypeInstance->Modify();
										ModifiedObjects.Add(ArchetypeInstance);
									}

									CopySingleProperty( TargetActor, ArchetypeInstance, Property );
								}
							}
						}
					}

					++CopiedPropertyCount;
				}
			}
		}

		// Copy component properties from source to target if they match. Note that the component lists may not be 1-1 due to context-specific components (e.g. editor-only sprites, etc.).

		TArray<TPair<UActorComponent*, UActorComponent*>> SourceTargetComponentPairs;

		auto BuildComponentPairs = [&SourceTargetComponentPairs, SourceActor](AActor* PrimaryActor, AActor* SecondaryActor)
		{
			TInlineComponentArray<UActorComponent*> SecondaryComponents(SecondaryActor);

			const bool bPrimaryIsSource = (PrimaryActor == SourceActor);
			int32 SecondaryComponentIndex = 0;
			for (UActorComponent* PrimaryComponent : PrimaryActor->GetComponents())
			{
				if (PrimaryComponent->CreationMethod == EComponentCreationMethod::UserConstructionScript)
				{
					continue;
				}
				if (UActorComponent* SecondaryComponent = FindMatchingComponentInstance(PrimaryComponent, SecondaryActor, SecondaryComponents, SecondaryComponentIndex))
				{
					if (bPrimaryIsSource)
					{
						SourceTargetComponentPairs.Emplace(PrimaryComponent, SecondaryComponent);
					}
					else
					{
						SourceTargetComponentPairs.Emplace(SecondaryComponent, PrimaryComponent);
					}
				}
			}
		};

		const bool bSourceActorIsCDO = SourceActor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
		const bool bTargetActorIsCDO = TargetActor->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject);
		const bool bSourceActorIsBPCDO = bSourceActorIsCDO && ActorClass->HasAllClassFlags(CLASS_CompiledFromBlueprint);

		// If the source actor is a CDO, then the target actor should drive the collection of components since FindMatchingComponentInstance
		// does work to seek out SCS and ICH components for blueprints
		if (bSourceActorIsCDO)
		{
			BuildComponentPairs(TargetActor, SourceActor);
		}
		else
		{
			BuildComponentPairs(SourceActor, TargetActor);
		}

		for (const TPair<UActorComponent*, UActorComponent*>& ComponentPair : SourceTargetComponentPairs)
		{
			UActorComponent* SourceComponent = ComponentPair.Key;
			UActorComponent* TargetComponent = ComponentPair.Value;
			if (!IsValid(TargetComponent)) // The Target Component might not be valid if the component list has been regenerated (see AActor::PostEditChangeProperty)
			{
				TargetComponent = FindMatchingComponentInstance(SourceComponent, TargetActor);
			}

			UClass* ComponentClass = SourceComponent->GetClass();
			check( ComponentClass == TargetComponent->GetClass() );

			// Build a list of matching component archetype instances for propagation (if requested)
			TArray<UActorComponent*> ComponentArchetypeInstances;
			if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
			{
				for( AActor* ArchetypeInstance : ArchetypeInstances )
				{
					if( ArchetypeInstance != nullptr )
					{
						UActorComponent* ComponentArchetypeInstance = FindMatchingComponentInstance( TargetComponent, ArchetypeInstance );
						if( ComponentArchetypeInstance != nullptr )
						{
							ComponentArchetypeInstances.AddUnique( ComponentArchetypeInstance );
						}
					}
				}
			}

			TSet<const FProperty*> SourceUCSModifiedProperties;
			SourceComponent->GetUCSModifiedProperties(SourceUCSModifiedProperties);

			TArray<UActorComponent*> ComponentInstancesToReregister;

			// Copy component properties
			for( FProperty* Property = ComponentClass->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext )
			{
				const bool bIsTransient = !!( Property->PropertyFlags & CPF_Transient );
				const bool bIsIdentical = Property->Identical_InContainer( SourceComponent, TargetComponent );
				const bool bIsComponent = !!( Property->PropertyFlags & ( CPF_InstancedReference | CPF_ContainsInstancedReference ) );
				const bool bIsTransform =
					Property->GetFName() == USceneComponent::GetRelativeScale3DPropertyName() ||
					Property->GetFName() == USceneComponent::GetRelativeLocationPropertyName() ||
					Property->GetFName() == USceneComponent::GetRelativeRotationPropertyName();

				auto SourceComponentIsRoot = [&]()
				{
					USceneComponent* RootComponent = SourceActor->GetRootComponent();
					if (SourceComponent == RootComponent)
					{
						return true;
					}
					else if (RootComponent == nullptr && bSourceActorIsBPCDO)
					{
						// If we're dealing with a BP CDO as source, then look at the target for whether this is the root component
						return (TargetComponent == TargetActor->GetRootComponent());
					}
					return false;
				};

				if( !bIsTransient && !bIsIdentical && !bIsComponent && !SourceUCSModifiedProperties.Contains(Property)
					&& ( !bIsTransform || (!bSourceActorIsCDO && !bTargetActorIsCDO) || !SourceComponentIsRoot() ) )
				{
					const bool bIsSafeToCopy = (!(Options.Flags & ECopyOptions::OnlyCopyEditOrInterpProperties) || (Property->HasAnyPropertyFlags(CPF_Edit | CPF_Interp)))
						                    && (!(Options.Flags & ECopyOptions::SkipInstanceOnlyProperties) || (!Property->HasAllPropertyFlags(CPF_DisableEditOnTemplate)));
					if( bIsSafeToCopy )
					{
						if (!Options.CanCopyProperty(*Property, *SourceActor))
						{
							continue;
						}
							
						if( !bIsPreviewing )
						{
							if( !ModifiedObjects.Contains(TargetComponent) )
							{
								TargetComponent->UnregisterComponent();
								TargetComponent->SetFlags(RF_Transactional);
								TargetComponent->Modify();
								ModifiedObjects.Add(TargetComponent);
							}

							if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
							{
								// @todo simulate: Should we be calling this on the component instead?
								TargetActor->PreEditChange( Property );
							}

							// Determine which component archetype instances match the current property value of the target component (before it gets changed). We only want to propagate the change to those instances.
							TArray<UActorComponent*> ComponentArchetypeInstancesToChange;
							if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
							{
								for (UActorComponent* ComponentArchetypeInstance : ComponentArchetypeInstances)
								{
									if( ComponentArchetypeInstance != nullptr && Property->Identical_InContainer( ComponentArchetypeInstance, TargetComponent ) )
									{
										bool bAdd = true;
										// We also need to double check that either the direct archetype of the target is also identical
										if (ComponentArchetypeInstance->GetArchetype() != TargetComponent)
										{
											UClass* TargetCompClass = TargetComponent->GetClass();
											UActorComponent* CheckComponent = CastChecked<UActorComponent>(ComponentArchetypeInstance->GetArchetype());
											while (CheckComponent != ComponentArchetypeInstance && CheckComponent->GetClass() == TargetCompClass)
											{
												if (!Property->Identical_InContainer( CheckComponent, TargetComponent ))
												{
													bAdd = false;
													break;
												}
												CheckComponent = CastChecked<UActorComponent>(CheckComponent->GetArchetype());
											}
										}
											
										if (bAdd)
										{
											ComponentArchetypeInstancesToChange.Add( ComponentArchetypeInstance );
										}
									}
								}
							}

							CopySingleProperty(SourceComponent, TargetComponent, Property);

							// Notify the target one of it's properties might have changed
							TargetComponent->PostReinitProperties();

							if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
							{
								FPropertyChangedEvent PropertyChangedEvent( Property );
								TargetActor->PostEditChangeProperty( PropertyChangedEvent );
								// When simulating, a call to AActor::PostEditChangeProperty will reset and reconstruct the Actor's component array
								// which will make the TargetComponent invalid, so we need to update the variable to point to the newly created one
								if (!IsValid(TargetComponent))
								{
									TargetComponent = FindMatchingComponentInstance(SourceComponent, TargetActor);
								}
							}

							if( Options.Flags & ECopyOptions::PropagateChangesToArchetypeInstances )
							{
								for( int32 InstanceIndex = 0; InstanceIndex < ComponentArchetypeInstancesToChange.Num(); ++InstanceIndex )
								{
									UActorComponent* ComponentArchetypeInstance = ComponentArchetypeInstancesToChange[InstanceIndex];
									if( ComponentArchetypeInstance != nullptr )
									{
										if( !ModifiedObjects.Contains(ComponentArchetypeInstance) )
										{
											// Ensure that this instance will be included in any undo/redo operations, and record it into the transaction buffer.
											// Note: We don't do this for components that originate from script, because they will be re-instanced from the template after an undo, so there is no need to record them.
											if (!ComponentArchetypeInstance->IsCreatedByConstructionScript())
											{
												ComponentArchetypeInstance->SetFlags(RF_Transactional);
												ComponentArchetypeInstance->Modify();
												ModifiedObjects.Add(ComponentArchetypeInstance);
											}

											// We must also modify the owner, because we'll need script components to be reconstructed as part of an undo operation.
											AActor* Owner = ComponentArchetypeInstance->GetOwner();
											if( Owner != nullptr && !ModifiedObjects.Contains(Owner))
											{
												Owner->Modify();
												ModifiedObjects.Add(Owner);
											}
										}

										if (ComponentArchetypeInstance->IsRegistered())
										{
											ComponentArchetypeInstance->UnregisterComponent();
											ComponentInstancesToReregister.Add(ComponentArchetypeInstance);
										}

										CopySingleProperty( TargetComponent, ComponentArchetypeInstance, Property );
									}
								}
							}
						}

						++CopiedPropertyCount;

						if( bIsTransform )
						{
							bTransformChanged = true;
						}
					}
				}
			}

			for (UActorComponent* ModifiedComponentInstance : ComponentInstancesToReregister)
			{
				ModifiedComponentInstance->RegisterComponent();
			}
		}

		if (!bIsPreviewing && CopiedPropertyCount > 0 && TargetActor->GetClass()->HasAllClassFlags(CLASS_CompiledFromBlueprint))
		{
			if (bTargetActorIsCDO)
			{
				FBlueprintEditorUtils::PostEditChangeBlueprintActors(CastChecked<UBlueprint>(TargetActor->GetClass()->ClassGeneratedBy));
			}
			else
			{
				TargetActor->RerunConstructionScripts();
			}
		}

		// If one of the changed properties was part of the actor's transformation, then we'll call PostEditMove too.
		if( !bIsPreviewing && bTransformChanged )
		{
			if( Options.Flags & ECopyOptions::CallPostEditMove )
			{
				const bool bFinishedMove = true;
				TargetActor->PostEditMove( bFinishedMove );
			}
		}

		return CopiedPropertyCount;
	}
}

//////////////////////////////////////////////////////////////////////////
// FCachedActorLabels

FCachedActorLabels::FCachedActorLabels()
{
	
}

FCachedActorLabels::FCachedActorLabels(UWorld* World, const TSet<AActor*>& IgnoredActors)
{
	Populate(World, IgnoredActors);
}

void FCachedActorLabels::Populate(UWorld* World, const TSet<AActor*>& IgnoredActors)
{
	ActorLabels.Empty();

	for (FActorIterator It(World); It; ++It)
	{
		if (!IgnoredActors.Contains(*It))
		{
			ActorLabels.Add(It->GetActorLabel());
		}
	}
	ActorLabels.Shrink();
}

//////////////////////////////////////////////////////////////////////////

void ExecuteInvalidateCachedShaders(const TArray< FString >& Args)
{
	if(Args.Num() == 0)
	{
		// todo: log error, at least one command is needed
		UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\nAs this command should not be executed accidentally it requires you to specify an extra parameter.");
		return;
	}

	FString FileName = FPaths::EngineDir() + TEXT("Shaders/Public/ShaderVersion.ush");

	FileName = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FileName);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	SourceControlProvider.Init();

	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(FileName, EStateCacheUsage::ForceUpdate);
	if(SourceControlState.IsValid())
	{
		if( SourceControlState->CanCheckout() || SourceControlState->IsCheckedOutOther() )
		{
			if(SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), FileName) == ECommandResult::Failed)
			{
				UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\nCouldn't check out \"ShaderVersion.ush\"");
				return;
			}
		}
		else if(!SourceControlState->IsSourceControlled())
		{
			UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\n\"ShaderVersion.ush\" is not under revision control.");
		}
		else if(SourceControlState->IsCheckedOutOther())
		{
			UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\n\"ShaderVersion.ush\" is already checked out by someone else\n(UE SourceControl needs to be fixed to allow multiple checkout.)");
			return;
		}
		else if(SourceControlState->IsDeleted())
		{
			UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\n\"ShaderVersion.ush\" is marked for delete");
			return;
		}
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	IFileHandle* FileHandle = PlatformFile.OpenWrite(*FileName);
	if(FileHandle)
	{
		FString Guid = FString::Printf(
			TEXT("// Copyright Epic Games, Inc. All Rights Reserved.\n")
			TEXT("// This file is automatically generated by the console command r.InvalidateCachedShaders\n")
			TEXT("// Each time the console command is executed it generates a new GUID. As this file is included\n")
			TEXT("// in Platform.ush (which should be included in any shader) it allows to invalidate the shader DDC.\n")
			TEXT("// \n")
			TEXT("#pragma message(\"UESHADERMETADATA_VERSION %s\")"), *FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphens));

		FileHandle->Write((const uint8*)TCHAR_TO_ANSI(*Guid), Guid.Len());
		delete FileHandle;

		UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders succeeded\n\"ShaderVersion.ush\" was updated.\n");
	}
	else
	{
		UE_LOGF(LogConsoleResponse, Display, "r.InvalidateCachedShaders failed\nCouldn't open \"ShaderVersion.ush\".\n");
	}
}

FAutoConsoleCommand InvalidateCachedShaders(
	TEXT("r.InvalidateCachedShaders"),
	TEXT("Invalidate shader cache by making a unique change to ShaderVersion.ush which is included in common.usf.")
	TEXT("To initiate actual the recompile of all shaders use \"recompileshaders changed\" or press \"Ctrl Shift .\".\n")
	TEXT("The ShaderVersion.ush file should be automatically checked out but  it needs to be checked in to have effect on other machines."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ExecuteInvalidateCachedShaders)
	);

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
