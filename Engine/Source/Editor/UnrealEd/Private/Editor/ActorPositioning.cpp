// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ActorPositioning.h"
#include "EngineDefines.h"
#include "CollisionQueryParams.h"
#include "PrimitiveViewRelevance.h"
#include "RenderingThread.h"
#include "PrimitiveSceneProxy.h"
#include "Components/PrimitiveComponent.h"
#include "Components/ShapeComponent.h"
#include "GameFramework/Volume.h"
#include "Components/ModelComponent.h"
#include "Editor.h"
#include "ActorFactories/ActorFactory.h"
#include "EditorViewportClient.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "LevelEditorViewport.h"
#include "SnappingUtils.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/ObjectPositioning.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Elements/Framework/EngineElementsLibrary.h"

namespace ActorPositioningLocals
{
	FActorPositionTraceResult ConvertToActorPositionTraceResult(const UE::Positioning::FObjectPositioningTraceResult& ResultIn)
	{
		using namespace UE::Positioning;

		FActorPositionTraceResult ResultOut;
		switch (ResultIn.State)
		{
		case FObjectPositioningTraceResult::HitSuccess:
			ResultOut.State = FActorPositionTraceResult::HitSuccess;
			break;
		case FObjectPositioningTraceResult::Default:
			ResultOut.State = FActorPositionTraceResult::Default;
			break;
		case FObjectPositioningTraceResult::Failed:
			ResultOut.State = FActorPositionTraceResult::Failed;
			break;
		}

		ResultOut.Location = ResultIn.Location;
		ResultOut.SurfaceNormal = ResultIn.SurfaceNormal;
		ResultOut.HitActor = Cast<AActor>(ResultIn.HitObject.Get());

		return ResultOut;
	}

	static void AddIgnoredActors(FCollisionQueryParams& OutParams, const TArray<AActor*>* IgnoreActors)
	{
		if (!IgnoreActors)
		{
			return;
		}

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		if (!Registry)
		{
			OutParams.AddIgnoredActors(*IgnoreActors);
			return;
		}

		for (AActor* IgnoreActor : *IgnoreActors)
		{
			if (!IgnoreActor)
			{
				continue;
			}

			FTypedElementHandle Element = UEngineElementsLibrary::AcquireEditorActorElementHandle(IgnoreActor);
			TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Element);
			if (WorldElement)
			{
				WorldElement.AddIgnoredElementToCollisionQueryParams(OutParams, true);
			}
			else
			{
				OutParams.AddIgnoredActor(IgnoreActor);
			}
		}
	}
}

FActorPositionTraceResult FActorPositioning::TraceWorldForPositionWithDefault(const FViewportCursorLocation& Cursor, const FSceneView& View, const TArray<AActor*>* IgnoreActors)
{
	FCollisionQueryParams Param(SCENE_QUERY_STAT(DragDropTrace), true);
	ActorPositioningLocals::AddIgnoredActors(Param, IgnoreActors);

	return ActorPositioningLocals::ConvertToActorPositionTraceResult(
		UE::Positioning::TraceWorldForPositionWithDefault(Cursor, View, IgnoreActors ? &Param : nullptr));
}

FActorPositionTraceResult FActorPositioning::TraceWorldForPosition(const FViewportCursorLocation& Cursor, const FSceneView& View, const TArray<AActor*>* IgnoreActors)
{
	FCollisionQueryParams Param(SCENE_QUERY_STAT(DragDropTrace), true);
	ActorPositioningLocals::AddIgnoredActors(Param, IgnoreActors);

	return ActorPositioningLocals::ConvertToActorPositionTraceResult(
		UE::Positioning::TraceWorldForPosition(Cursor, View, IgnoreActors ? &Param : nullptr));
}

FActorPositionTraceResult FActorPositioning::TraceWorldForPosition(const UWorld& InWorld, const FSceneView& InSceneView, const FVector& RayStart, const FVector& RayEnd, const TArray<AActor*>* IgnoreActors)
{
	FCollisionQueryParams Param(SCENE_QUERY_STAT(DragDropTrace), true);
	ActorPositioningLocals::AddIgnoredActors(Param, IgnoreActors);

	return ActorPositioningLocals::ConvertToActorPositionTraceResult(
		UE::Positioning::TraceWorldForPosition(InWorld, InSceneView, RayStart, RayEnd, IgnoreActors ? &Param : nullptr));
}

FTransform FActorPositioning::GetCurrentViewportPlacementTransform(const AActor& Actor, bool bSnap, const FViewportCursorLocation* InCursor)
{
	FTransform ActorTransform = FTransform::Identity;
	if (GCurrentLevelEditingViewportClient)
	{
		// Get cursor origin and direction in world space.
		FViewportCursorLocation CursorLocation = InCursor ? *InCursor : GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();
		const auto CursorPos = CursorLocation.GetCursorPos();

		if (CursorLocation.GetViewportType() == LVT_Perspective && !GCurrentLevelEditingViewportClient->Viewport->GetHitProxy(CursorPos.X, CursorPos.Y))
		{
			ActorTransform.SetTranslation(GetActorPositionInFrontOfCamera(Actor, CursorLocation.GetOrigin(), CursorLocation.GetDirection()));
			return ActorTransform;
		}
	}

	const FSnappedPositioningData PositioningData = FSnappedPositioningData(nullptr, GEditor->ClickLocation, GEditor->ClickPlane)
		.DrawSnapHelpers(true)
		.UseFactory(GEditor->FindActorFactoryForActorClass(Actor.GetClass()))
		.UsePlacementExtent(Actor.GetPlacementExtent());

	ActorTransform = bSnap ? GetSnappedSurfaceAlignedTransform(PositioningData) : GetSurfaceAlignedTransform(PositioningData);

	if (GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled)
	{
		// HACK: If we are aligning rotation to surfaces, we have to factor in the inverse of the actor's rotation and translation so that the resulting transform after SpawnActor is correct.

		if (auto* RootComponent = Actor.GetRootComponent())
		{
			RootComponent->UpdateComponentToWorld();
		}

		FVector OrigActorScale3D = ActorTransform.GetScale3D();
		ActorTransform = Actor.GetTransform().Inverse() * ActorTransform;
		ActorTransform.SetScale3D(OrigActorScale3D);
	}

	return ActorTransform;
}

FVector FActorPositioning::GetActorPositionInFrontOfCamera(const AActor& InActor, const FVector& InCameraOrigin, const FVector& InCameraDirection)
{
	// Get the  radius of the actors bounding cylinder.  Height is not needed.
	float CylRadius, CylHeight;
	InActor.GetComponentsBoundingCylinder(CylRadius, CylHeight);

	// a default cylinder radius if no bounding cylinder exists.  
	const float	DefaultCylinderRadius = 50.0f;

	if( CylRadius == 0.0f )
	{
		// If the actor does not have a bounding cylinder, use a default value.
		CylRadius = DefaultCylinderRadius;
	}

	// The new location the cameras origin offset by the actors bounding cylinder radius down the direction of the cameras view. 
	FVector NewLocation = InCameraOrigin + InCameraDirection * CylRadius + InCameraDirection * GetDefault<ULevelEditorViewportSettings>()->BackgroundDropDistance;

	// Snap the new location if snapping is enabled
	FSnappingUtils::SnapPointToGrid( NewLocation, FVector::ZeroVector );
	return NewLocation;
}

FTransform FActorPositioning::GetSurfaceAlignedTransform(const FPositioningData& Data)
{
	// Sort out the rotation first, then do the location
	FQuat RotatorQuat = Data.StartTransform.GetRotation();

	if (Data.ActorFactory)
	{
		RotatorQuat = Data.ActorFactory->AlignObjectToSurfaceNormal(Data.SurfaceNormal, RotatorQuat);
	}
	else
	{
		// Try to find a default alignment if we don't have a factory.
		RotatorQuat = FindActorAlignmentRotation(RotatorQuat, FVector::UpVector, Data.SurfaceNormal);
	}

	// Choose the largest location offset of the various options (global viewport settings, collision, factory offset)
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	const float SnapOffsetExtent = (ViewportSettings->SnapToSurface.bEnabled) ? (ViewportSettings->SnapToSurface.SnapOffsetExtent) : (0.0f);
	const FVector PlacementExtent = (Data.ActorFactory && !Data.ActorFactory->bUsePlacementExtent) ? FVector::ZeroVector : Data.PlacementExtent;
	const double CollisionOffsetExtent = FVector::BoxPushOut(Data.SurfaceNormal, PlacementExtent);

	FVector LocationOffset = Data.SurfaceNormal * FMath::Max(SnapOffsetExtent, CollisionOffsetExtent);
	if (Data.ActorFactory && LocationOffset.SizeSquared() < Data.ActorFactory->SpawnPositionOffset.SizeSquared())
	{
		// Rotate the Spawn Position Offset to match our rotation
		LocationOffset = RotatorQuat.RotateVector(-Data.ActorFactory->SpawnPositionOffset);
	}

	return FTransform(Data.bAlignRotation ? RotatorQuat : Data.StartTransform.GetRotation(), Data.SurfaceLocation + LocationOffset);
}

FTransform FActorPositioning::GetSnappedSurfaceAlignedTransform(const FSnappedPositioningData& Data)
{
	FVector SnappedLocation = Data.SurfaceLocation;
	FSnappingUtils::SnapPointToGrid(SnappedLocation, FVector(0.f));

	// Secondly, attempt vertex snapping
	FVector AlignToNormal;
	if (!Data.LevelViewportClient || !FSnappingUtils::SnapLocationToNearestVertex( SnappedLocation, Data.LevelViewportClient->GetDropPreviewLocation(), Data.LevelViewportClient, AlignToNormal, Data.bDrawSnapHelpers ))
	{
		AlignToNormal = Data.SurfaceNormal;
	}

	return GetSurfaceAlignedTransform(Data);
}
