// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ObjectPositioning.h"

#include "Components/ModelComponent.h"
#include "Components/ShapeComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EditorViewportClient.h" // FViewportCursorLocation
#include "GameFramework/Volume.h"
#include "LandscapeComponent.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"

namespace ObjectPositioningLocals
{
	bool bCVarAllowNonPrimitiveComponentHits = true;
	static FAutoConsoleVariableRef CVarAllowNonPrimitiveComponentHits(
		TEXT("PlacementMode.AllowNonPrimitiveComponentHits"),
		bCVarAllowNonPrimitiveComponentHits,
		TEXT("When raycasting the world in placement mode, allow hits of physics objects that are not tied to a UPrimitiveComponent (to work with non-actor workflows)."));

	struct FPrimitiveOwner
	{
		TWeakObjectPtr<UObject> Object = nullptr;
		bool bConsiderForPlacementWhenHidden = false;
	};
	
	/**
	 * Prunes list of hit results for object positioning calculations based on conditions that could be tested
	 * on the game thread and returns a list of primitives for the remaining this.
	 * @note If a non-primitive based hit is found and bCVarAllowNonPrimitiveComponentHits is true then
     * an empty weak obj ptr will be added to the result to represent the hit.
	 */
	TArray<FPrimitiveOwner> FilterHitsGameThread(TArray<FHitResult>& InOutHits)
	{
		check(IsInGameThread() || IsInParallelGameThread());
		TArray<FPrimitiveOwner> PrimitiveOwners;
		PrimitiveOwners.Reserve(InOutHits.Num());

		InOutHits.RemoveAll([&PrimitiveOwners](const FHitResult& Hit)
		{
			if (Hit.bStartPenetrating)
			{
				return true;
			}

			const FActorInstanceHandle& HitObjHandle = Hit.HitObjectHandle;

			// Try and find a primitive component for the hit
			UPrimitiveComponent* PrimitiveComponent = Hit.GetComponent();

			if (!PrimitiveComponent)
			{
				PrimitiveComponent = Cast<UPrimitiveComponent>(HitObjHandle.GetRootComponent());		
			}
			if (PrimitiveComponent && PrimitiveComponent->IsA(ULandscapeHeightfieldCollisionComponent::StaticClass()))
			{
				PrimitiveComponent = CastChecked<ULandscapeHeightfieldCollisionComponent>(PrimitiveComponent)->GetRenderComponent();
			}

			FPrimitiveOwner Owner;
			if (!PrimitiveComponent)
			{
				// If we don't have a primitive component, either ignore the hit, or pass it through if the CVar is set appropriately.
				// If we pass the hit through, we still need to add an entry to the WeakPrimitives list to make sure that
				// we have an entry for each hit index.
				if (!bCVarAllowNonPrimitiveComponentHits)
				{
					// Filter out the hit if the CVar didn't allow it.
					return true;
				}

				Owner = { Hit.PhysicsObjectOwner.Get(), false };
			}
			else
			{
				// Ignore volumes and shapes
				if (HitObjHandle.DoesRepresentClass(AVolume::StaticClass()))
				{
					return true;
				}

				if (PrimitiveComponent->IsA(UShapeComponent::StaticClass()))
				{
					return true;
				}

				Owner = { PrimitiveComponent, (bool)PrimitiveComponent->bConsiderForActorPlacementWhenHidden };
			}

			if (!Owner.Object.IsValid())
			{
				return true;
			}
				
			PrimitiveOwners.Add(MoveTemp(Owner));
			return false;
		});

		return MoveTemp(PrimitiveOwners);
	}

	
	/** Check to see if the specified hit result should be ignored from object positioning calculations for the specified scene view */
	bool IsHitIgnoredRenderingThread(const FPrimitiveOwner& InPrimitiveOwner, const FSceneView& InSceneView)
	{
		// We're using the SceneProxy and ViewRelevance here, we should execute from the render thread
		check(IsInParallelRenderingThread());

		const IPrimitiveComponent* PrimitiveComponent = TComponentInterfaceProvider<IPrimitiveComponent>::Provides(InPrimitiveOwner.Object.Get());

		// Only use this component if it is visible in the specified scene views
		bool bIsRenderedOnScreen = false;
		bool bIgnoreTranslucentPrimitive = false;
		const bool bConsiderInvisibleComponentForPlacement = PrimitiveComponent && InPrimitiveOwner.bConsiderForPlacementWhenHidden;

		if (PrimitiveComponent && PrimitiveComponent->GetSceneProxy())
		{
			// Only use this component if it is visible in the specified scene views
			const FPrimitiveViewRelevance ViewRelevance = PrimitiveComponent->GetSceneProxy()->GetViewRelevance(&InSceneView);
			// BSP is a bit special in that its bDrawRelevance is false even when drawn as wireframe because InSceneView.Family->EngineShowFlags.BSPTriangles is off
			bIsRenderedOnScreen = ViewRelevance.bDrawRelevance || (InPrimitiveOwner.Object->IsA(UModelComponent::StaticClass()) && InSceneView.Family->EngineShowFlags.BSP);
			bIgnoreTranslucentPrimitive = ViewRelevance.HasTranslucency() && !GetDefault<UEditorPerProjectUserSettings>()->bAllowSelectTranslucent;
		}

		return (!bIsRenderedOnScreen && !bConsiderInvisibleComponentForPlacement) || bIgnoreTranslucentPrimitive;
	}
}


UE::Positioning::FObjectPositioningTraceResult UE::Positioning::TraceWorldForPositionWithDefault(const FViewportCursorLocation& Cursor, const FSceneView& View, FCollisionQueryParams* CollisionQueryParams)
{
	FObjectPositioningTraceResult Results = TraceWorldForPosition(Cursor, View, CollisionQueryParams);
	if (Results.State == FObjectPositioningTraceResult::Failed)
	{
		Results.State = FObjectPositioningTraceResult::Default;

		// And put it in front of the camera
		const float DistanceMultiplier = (Cursor.GetViewportType() == LVT_Perspective) ? GetDefault<ULevelEditorViewportSettings>()->BackgroundDropDistance : 0.0f;
		Results.Location = Cursor.GetOrigin() + Cursor.GetDirection() * DistanceMultiplier;
	}
	return Results;
}

UE::Positioning::FObjectPositioningTraceResult UE::Positioning::TraceWorldForPosition(const FViewportCursorLocation& Cursor, const FSceneView& View, FCollisionQueryParams* CollisionQueryParams)
{
	const auto* ViewportClient = Cursor.GetViewportClient();
	const auto ViewportType = ViewportClient->GetViewportType();

	// Start with a ray that encapsulates the entire world
	FVector RayStart = Cursor.GetOrigin();
	if (ViewportType == LVT_OrthoXY || ViewportType == LVT_OrthoXZ || ViewportType == LVT_OrthoYZ ||
		ViewportType == LVT_OrthoNegativeXY || ViewportType == LVT_OrthoNegativeXZ || ViewportType == LVT_OrthoNegativeYZ)
	{
		RayStart -= Cursor.GetDirection() * HALF_WORLD_MAX / 2;
	}

	const FVector RayEnd = RayStart + Cursor.GetDirection() * HALF_WORLD_MAX;
	return TraceWorldForPosition(*ViewportClient->GetWorld(), View, RayStart, RayEnd, CollisionQueryParams);
}

UE::Positioning::FObjectPositioningTraceResult UE::Positioning::TraceWorldForPosition(const UWorld& InWorld, const FSceneView& InSceneView, const FVector& RayStart, const FVector& RayEnd, FCollisionQueryParams* CollisionQueryParams)
{
	using namespace ObjectPositioningLocals;

	TArray<FHitResult> Hits;

	FCollisionQueryParams Params;
	if (CollisionQueryParams)
	{
		Params = *CollisionQueryParams;
	}
	Params.TraceTag = SCENE_QUERY_STAT(DragDropTrace);
	Params.bTraceComplex = true;

	FObjectPositioningTraceResult Results;
	if (InWorld.LineTraceMultiByObjectType(Hits, RayStart, RayEnd, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects), Params))
	{
		{
			// Filter out anything that should be ignored based on information accessible on the game thread
			// and build list of remaining weak primitive components that need to be filtered on the rendering thread 
			TArray<FPrimitiveOwner> PrimitiveOwners = FilterHitsGameThread(Hits);
			ensure(Hits.Num() == PrimitiveOwners.Num());

			// Send IsHitIgnoredRenderingThread on the render thread since we're accessing view relevance
			ENQUEUE_RENDER_COMMAND(TraceWorldForPosition_FilterHitsByViewRelevance)(
				[&Hits, &PrimitiveOwners, &InSceneView](FRHICommandListImmediate& RHICmdList)
				{
					// Filter out anything that should be ignored
					int32 Index = 0;
					Hits.RemoveAll([&Index, &InSceneView, &PrimitiveOwners](const FHitResult&)
					{
						return IsHitIgnoredRenderingThread(PrimitiveOwners[Index++], InSceneView);
					});
				}
			);
			
			// We need the result to come back before continuing
			FRenderCommandFence Fence;
			Fence.BeginFence();
			Fence.Wait();
		}

		// Go through all hits and find closest
		double ClosestHitDistanceSqr = std::numeric_limits<double>::max();

		for (const FHitResult& Hit : Hits)
		{
			const double DistanceToHitSqr = (Hit.ImpactPoint - RayStart).SizeSquared();
			if (DistanceToHitSqr < ClosestHitDistanceSqr)
			{
				ClosestHitDistanceSqr = DistanceToHitSqr;
				Results.Location = Hit.Location;
				Results.SurfaceNormal = Hit.Normal.GetSafeNormal();
				Results.State = FObjectPositioningTraceResult::HitSuccess;
				Results.HitObject = Hit.HitObjectHandle.GetManagingActor();
			}
		}
	}

	return Results;
}