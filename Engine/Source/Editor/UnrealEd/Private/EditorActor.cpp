// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Misc/ITransaction.h" // GUndo
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"
#include "AI/NavigationSystemBase.h"
#include "Components/LightComponent.h"
#include "Model.h"
#include "Exporters/Exporter.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/Brush.h"
#include "UnrealEdGlobals.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/LevelFactory.h"
#include "Editor/GroupActor.h"
#include "Animation/SkeletalMeshActor.h"
#include "Particles/Emitter.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyPortFlags.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Components/ChildActorComponent.h"
#include "Engine/Polys.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Dialogs/Dialogs.h"
#include "ScopedTransaction.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "BusyCursor.h"
#include "Engine/BrushBuilder.h"
#include "BSPOps.h"
#include "EditorLevelUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "LevelEditorViewport.h"
#include "Layers/LayersSubsystem.h"
#include "ActorEditorUtils.h"
#include "Particles/ParticleSystemComponent.h"
#include "UnrealExporter.h"
#include "LevelEditor.h"
#include "Engine/LODActor.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "ActorGroupingUtils.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "AssetSelection.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "EdMode.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "Misc/ScopedSlowTask.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SceneInterface.h"
#include "SDeleteReferencedActorDialog.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#define LOCTEXT_NAMESPACE "UnrealEd.EditorActor"

DEFINE_LOG_CATEGORY_STATIC(LogEditorActor, Log, All);

namespace UE::EditorActor::Private
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	/** 
	 * Takes a query that provides readonly world and visibility columns, and sets the visibility on the returned columns 
	 *  if the world matches. Handles undo if a transaction is open.
	 */ 
	void RunSetVisibilityQuery(ICoreProvider& DataStorage, QueryHandle Query, UWorld* WorldIn, bool bVisible)
	{
		// We'll gather the rows to change in one pass and actually change them in another, to
		//  avoid trying to add a tag (FVisibilityChangedTag) from within a callback 
		FRowHandleArray RowsToChange;
		DataStorage.RunQuery(Query, CreateDirectQueryCallbackBinding(
			[WorldIn, bVisible, &RowsToChange](RowHandle Row, const FTypedElementWorldColumn& WorldColumn, const FVisibleInEditorColumn& VisibilityColumn)
		{
			if (WorldColumn.World.Get() != WorldIn)
			{
				return;
			}
			if (VisibilityColumn.bIsVisibleInEditor != bVisible)
			{
				RowsToChange.Add(Row);
			}
		}));

		// Apply the change
		for (RowHandle Row : RowsToChange.GetRows())
		{
			FVisibleInEditorColumn* VisibilityColumn = DataStorage.GetColumn<FVisibleInEditorColumn>(Row);
			if (ensure(VisibilityColumn))
			{
				VisibilityColumn->bIsVisibleInEditor = bVisible;
				DataStorage.AddColumn<FVisibilityChangedTag>(Row);
			}
		}

		// Emit an undo change if a transaction is open
		if (GUndo && WorldIn)
		{
			GUndo->StoreUndo(WorldIn, MakeUnique<FVisibilityColumnChange>(MoveTemp(RowsToChange), bVisible));
		}
	}

	/**
	 * Walks across editor object hierarchies for all selected items and calls VisitFn
	 */
	void WalkSelectedHierarchies(ICoreProvider& DataStorage, UWorld* WorldIn, 
		TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)> VisitFn)
	{
		// Gather selected rows with matching world
		QueryHandle Query = DataStorage.RegisterQuery(
			Select()
			.ReadOnly<FTypedElementWorldColumn>()
			.Where()
			.None<FTypedElementActorTag>()
			.All<FTypedElementSelectionColumn>()
			.Compile());
		ON_SCOPE_EXIT{ DataStorage.UnregisterQuery(Query); };

		TArray<RowHandle> SelectedRows;
		DataStorage.RunQuery(Query, CreateDirectQueryCallbackBinding(
			[WorldIn, &DataStorage, &SelectedRows](RowHandle Row, const FTypedElementWorldColumn& WorldColumn)
			{
				if (WorldColumn.World.Get() != WorldIn)
				{
					return;
				}
				SelectedRows.Add(Row);
			}));

		// TODO: This should match UTedsEditorHierarchyFactory::EditorObjectHierarchyName when that is not experimental
		FName EditorObjectHierarchyName("EditorObjectHierarchy");
		FHierarchyHandle Hierarchy = DataStorage.FindHierarchyByName(EditorObjectHierarchyName);
		if (!ensure(DataStorage.IsValidHierarchyHandle(Hierarchy)))
		{
			return;
		}

		// Walk the hierarchies for the selected rows
		for (RowHandle Row : SelectedRows)
		{
			DataStorage.WalkDepthFirst(Hierarchy, Row, VisitFn);
		}
	}

	void SetSelectedNonActorVisibility(UWorld* WorldIn, bool bVisible, bool bApplyToHierarchy)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!DataStorage)
		{
			return;
		}

		if (!bApplyToHierarchy)
		{
			// This is the simpler case- just change the selected items.

			// Query gets world and visibility columns for all selected non-actors
			QueryHandle Query = DataStorage->RegisterQuery(
				Select()
				.ReadOnly<FTypedElementWorldColumn, FVisibleInEditorColumn>()
				.Where()
				.None<FTypedElementActorTag>()
				.All<FTypedElementSelectionColumn>()
				.Compile());
			ON_SCOPE_EXIT{ DataStorage->UnregisterQuery(Query); };

			RunSetVisibilityQuery(*DataStorage, Query, WorldIn, bVisible);
			return;
		}

		// If we got here, we're processing the hierarchy.

		FRowHandleArray RowsToChange;
		WalkSelectedHierarchies(*DataStorage, WorldIn, [&RowsToChange, bVisible](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
		{
			if (const FVisibleInEditorColumn* VisibilityColumn = Context.GetColumn<FVisibleInEditorColumn>(Target))
			{
				if (VisibilityColumn->bIsVisibleInEditor != bVisible)
				{
					RowsToChange.Add(Target);
				}
			}
		});

		// Apply the change
		for (RowHandle Row : RowsToChange.GetRows())
		{
			FVisibleInEditorColumn* VisibilityColumn = DataStorage->GetColumn<FVisibleInEditorColumn>(Row);
			if (ensure(VisibilityColumn))
			{
				VisibilityColumn->bIsVisibleInEditor = bVisible;
				DataStorage->AddColumn<FVisibilityChangedTag>(Row);
			}
		}

		// Emit an undo change if a transaction is open
		if (GUndo && WorldIn)
		{
			GUndo->StoreUndo(WorldIn, MakeUnique<FVisibilityColumnChange>(MoveTemp(RowsToChange), bVisible));
		}
	}

	void SetAllNonActorVisiblity(UWorld* WorldIn, bool bVisible)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!DataStorage)
		{
			return;
		}
		
		QueryHandle Query = DataStorage->RegisterQuery(
			Select()
			.ReadOnly<FTypedElementWorldColumn, FVisibleInEditorColumn>()
			.Where()
			.None<FTypedElementActorTag>()
			.Compile());
		ON_SCOPE_EXIT{ DataStorage->UnregisterQuery(Query); };
		RunSetVisibilityQuery(*DataStorage, Query, WorldIn, bVisible);
	}

	void SetUnselectedNonActorVisibility(UWorld* WorldIn, bool bVisible)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!DataStorage)
		{
			return;
		}

		// Note that it is insufficient to just set the visibility of unselected items, even
		//  if the selected ones are already visible, because in some cases the visibility handling
		//  needs to propagate the visibility to some children that might otherwise remain invisible,
		//  as they are not directly selected. For this reason, we go with this two step approach.

		SetAllNonActorVisiblity(WorldIn, false);
		SetSelectedNonActorVisibility(WorldIn, true, 
			// This seems to be the current behavior for actors, so we match it
			/*bApplyToHierarchy*/ false);
	}

	bool HaveAnySelectedVisibleNonActor(UWorld* WorldIn, bool bIncludeHierarchy)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);
		if (!DataStorage)
		{
			return false;
		}

		bool bHaveVisible = false;
		if (!bIncludeHierarchy)
		{
			// Query gets world and visibility columns for all selected non-actors
			QueryHandle Query = DataStorage->RegisterQuery(
				Select()
				.ReadOnly<FTypedElementWorldColumn, FVisibleInEditorColumn>()
				.Where()
				.None<FTypedElementActorTag>()
				.All<FTypedElementSelectionColumn>()
				.Compile());
			ON_SCOPE_EXIT{ DataStorage->UnregisterQuery(Query); };

			DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding(
				[WorldIn, &bHaveVisible](RowHandle Row, const FTypedElementWorldColumn& WorldColumn, const FVisibleInEditorColumn& VisibilityColumn)
			{
				if (WorldColumn.World.Get() != WorldIn)
				{
					return;
				}

				bHaveVisible = bHaveVisible || VisibilityColumn.bIsVisibleInEditor;
			}));
		}
		else
		{
			WalkSelectedHierarchies(*DataStorage, WorldIn, [&bHaveVisible](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
			{
				if (const FVisibleInEditorColumn* VisibilityColumn = Context.GetColumn<FVisibleInEditorColumn>(Target))
				{
					bHaveVisible = bHaveVisible || VisibilityColumn->bIsVisibleInEditor;
				}
			});
		}

		return bHaveVisible;
	}
} // namespace UE::EditorActor::Private


struct FGarbageEliminationScopeGuard
{
	bool PreviousValue;

	FGarbageEliminationScopeGuard(bool bNewValue)
		: PreviousValue(UObjectBaseUtility::IsGarbageEliminationEnabled())
	{
		UObjectBaseUtility::SetGarbageEliminationEnabled(bNewValue);
	}

	~FGarbageEliminationScopeGuard()
	{
		UObjectBaseUtility::SetGarbageEliminationEnabled(PreviousValue);
	}
};

static int32 RecomputePoly( ABrush* InOwner, FPoly* Poly )
{
	// force recalculation of normal, and texture U and V coordinates in FPoly::Finalize()
	Poly->Normal = FVector3f::ZeroVector;

	return Poly->Finalize( InOwner, 0 );
}

/*-----------------------------------------------------------------------------
   Actor adding/deleting functions.
-----------------------------------------------------------------------------*/

void UUnrealEdEngine::edactCopySelected( UWorld* InWorld, FString* DestinationData )
{
	if (GetSelectedComponentCount() > 0)
	{
		// Copy selected components
		TArray<UActorComponent*> SelectedComponents;
		GetSelectedComponents()->GetSelectedObjects<UActorComponent>(SelectedComponents);
		CopyComponents(SelectedComponents, DestinationData);
	}
	else
	{
		// Copy selected actors
		TArray<AActor*> SelectedActors;
		GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
		CopyActors(SelectedActors, InWorld, DestinationData);
	}
}

void UUnrealEdEngine::CopyComponents(const TArray<UActorComponent*>& InComponentsToCopy, FString* DestinationData) const
{
	// Remove:
	// - Components that cannot be copied.
	TArray<UActorComponent*> ComponentsToCopy = InComponentsToCopy;
	ComponentsToCopy.RemoveAll([](UActorComponent* InComponent)
	{
		return !FComponentEditorUtils::CanCopyComponent(InComponent);
	});

	FComponentEditorUtils::CopyComponents(ComponentsToCopy, DestinationData);
}

void UUnrealEdEngine::CopyActors(const TArray<AActor*>& InActorsToCopy, UWorld* InWorld, FString* DestinationData) const
{
	// Remove:
	// - Actors belonging to prefabs unless all actors in the prefab are selected.
	// - Builder brushes.
	// - World Settings.
	TArray<AActor*> ActorsToCopy = InActorsToCopy;
	{
		bool bSomeSelectedActorsNotInCurrentLevel = false;
		ActorsToCopy.RemoveAll([&bSomeSelectedActorsNotInCurrentLevel](AActor* InActor)
		{
			// Remove any selected builder brushes.
			ABrush* Brush = Cast<ABrush>(InActor);
			if (Brush && FActorEditorUtils::IsABuilderBrush(Brush))
			{
				return true;
			}

			// Remove world settings
			if (InActor->IsA<AWorldSettings>())
			{
				return true;
			}

			// If any selected actors are not in the current level, warn the user that some actors will not be copied.
			if (!bSomeSelectedActorsNotInCurrentLevel && !InActor->GetLevel()->IsCurrentLevel())
			{
				bSomeSelectedActorsNotInCurrentLevel = true;
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "CopySelectedActorsInNonCurrentLevel", "Some selected actors are not in the current level and will not be copied."));
			}

			return false;
		});
	}
	// Filter out other actors.
	OnFilterCopiedActors().Broadcast(ActorsToCopy);

	// Export the actors.
	FStringOutputDevice Ar;
	const FSelectedActorExportObjectInnerContext Context(ActorsToCopy);
	const bool bExportSucceeded = UExporter::ExportToOutputDevice(&Context, InWorld, NULL, Ar, TEXT("copy"), 0, PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified);

	if (bExportSucceeded)
	{
		if (DestinationData)
		{
			*DestinationData = MoveTemp(Ar);
		}
		else
		{
			FPlatformApplicationMisc::ClipboardCopy(*Ar);
		}
	}
}

bool UUnrealEdEngine::WarnIfDestinationLevelIsHidden( UWorld* InWorld ) const
{
	bool bShouldLoadHiddenLevels = true;
	bool bShowPasteHiddenWarning = true;
	TArray<ULevel*> Levels;
	TArray<bool> bTheyShouldBeVisible;
	//prepare the warning dialog
	FSuppressableWarningDialog::FSetupInfo Info( LOCTEXT( "Warning_PasteWarningBody","You are trying to paste to a hidden level.\nSuppressing this will default to Do Not Paste" ), LOCTEXT( "Warning_PasteWarningHeader","Pasting To Hidden Level" ), "PasteHiddenWarning" );
	Info.ConfirmText = LOCTEXT( "Warning_PasteContinue","Unhide Level and paste" );
	Info.CancelText = LOCTEXT( "Warning_PasteCancel","Do not paste" );

	//check streaming levels first
	for (ULevelStreaming* StreamedLevel : InWorld->GetStreamingLevels())
	{
		//this is the active level - check if it is visible
		if (StreamedLevel && StreamedLevel->GetShouldBeVisibleInEditor() == false )
		{
			ULevel* Level = StreamedLevel->GetLoadedLevel();
			if( Level && Level->IsCurrentLevel() )
			{
				if (bShowPasteHiddenWarning)
				{
					bShowPasteHiddenWarning = false;
					//the streamed level is not visible - check what the user wants to do
					bShouldLoadHiddenLevels = (FSuppressableWarningDialog(Info).ShowModal() == FSuppressableWarningDialog::Confirm);
				}
				
				if (bShouldLoadHiddenLevels)
				{
					Levels.Add(Level);
					bTheyShouldBeVisible.Add(true);
				}
			}
		}
	}

	//now check the active level (this handles the persistent level also)
	if (bShouldLoadHiddenLevels)
	{
		if( FLevelUtils::IsLevelVisible( InWorld->GetCurrentLevel() ) == false )
		{	
			if (bShowPasteHiddenWarning)
			{
				bShowPasteHiddenWarning = false;
				//the streamed level is not visible - check what the user wants to do
				bShouldLoadHiddenLevels = (FSuppressableWarningDialog(Info).ShowModal() == FSuppressableWarningDialog::Confirm);
			}

			if (bShouldLoadHiddenLevels)
			{
				Levels.Add(InWorld->GetCurrentLevel());
				bTheyShouldBeVisible.Add(true);
			}

		}
	}
	// For efficiency, set visibility of all levels at once
	if (Levels.Num() > 0)
	{
		EditorLevelUtils::SetLevelsVisibility(Levels, bTheyShouldBeVisible, true);
	}
	return !bShouldLoadHiddenLevels;
}

void UUnrealEdEngine::edactPasteSelected(UWorld* InWorld, bool bDuplicate, bool bOffsetLocations, bool bWarnIfHidden, const FString* SourceData)
{
	if (GetSelectedComponentCount() > 0)
	{
		AActor* SelectedActor = CastChecked<AActor>(*GetSelectedActorIterator());

		TArray<UActorComponent*> PastedComponents;
		PasteComponents(PastedComponents, SelectedActor, bWarnIfHidden, SourceData);

		if (PastedComponents.Num() > 0)
		{
			// Select the new clones
			USelection* ComponentSelection = GetSelectedComponents();
			ComponentSelection->Modify();
			ComponentSelection->BeginBatchSelectOperation();
			ComponentSelection->DeselectAll();

			for (UActorComponent* PastedComponent : PastedComponents)
			{
				GEditor->SelectComponent(PastedComponent, true, false, true);
			}

			ComponentSelection->EndBatchSelectOperation(true);
		}
	}
	else
	{
		TArray<AActor*> PastedActors;
		PasteActors(PastedActors, InWorld, bOffsetLocations ? GEditor->GetGridLocationOffset(/*bUniformOffset*/true) : FVector::ZeroVector, bDuplicate, bWarnIfHidden, SourceData);

		if (PastedActors.Num() > 0)
		{
			// Select the new clones
			USelection* ActorSelection = GetSelectedActors();
			ActorSelection->Modify();
			ActorSelection->BeginBatchSelectOperation();
			ActorSelection->DeselectAll();

			for (AActor* PastedActor : PastedActors)
			{
				GEditor->SelectActor(PastedActor, true, false, true);
			}

			ActorSelection->EndBatchSelectOperation(true);

			// Note the selection change.  This will also redraw level viewports and update the pivot.
			NoteSelectionChange();
		}
	}
}

void UUnrealEdEngine::PasteComponents(TArray<UActorComponent*>& OutPastedComponents, AActor* TargetActor, const bool bWarnIfHidden, const FString* SourceData)
{
	OutPastedComponents.Reset();

	// Check and warn if the user is trying to paste to a hidden level. This will return if they wish to abort the process...
	if (bWarnIfHidden && WarnIfDestinationLevelIsHidden(TargetActor->GetWorld()))
	{
		return;
	}

	FComponentEditorUtils::PasteComponents(OutPastedComponents, TargetActor, TargetActor->GetRootComponent(), SourceData);

	if (OutPastedComponents.Num() > 0)
	{
		// Make sure all the SCS trees have a chance to update
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.BroadcastComponentsEdited();
	}
}

void UUnrealEdEngine::PasteActors(TArray<AActor*>& OutPastedActors, UWorld* InWorld, const FVector& LocationOffset, bool bDuplicate, bool bWarnIfHidden, const FString* SourceData)
{
	OutPastedActors.Reset();

	// Check and warn if the user is trying to paste to a hidden level. This will return if they wish to abort the process...
	if (bWarnIfHidden && WarnIfDestinationLevelIsHidden(InWorld))
	{
		return;
	}

	OnPasteActorsBeginDelegate.Broadcast();

	ON_SCOPE_EXIT{ OnPasteActorsEndDelegate.Broadcast(OutPastedActors); };

	const FScopedBusyCursor BusyCursor;

	FCachedActorLabels ActorLabels(InWorld);

	// Get pasted text.
	FString PasteString;
	if (SourceData)
	{
		PasteString = *SourceData;
	}
	else
	{
		FPlatformApplicationMisc::ClipboardPaste(PasteString);
	}
	const TCHAR* Paste = *PasteString;

	USelection* ActorSelection = GetSelectedActors();

	// FactoryCreateText will mutate the selection, so cache it here so we can restore it later
	TArray<uint8> OriginalSelectionState;
	FObjectWriter(ActorSelection, OriginalSelectionState);

	// Turn off automatic BSP update while pasting to save rebuilding geometry potentially multiple times
	const bool bBSPAutoUpdate = GetDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate;
	GetMutableDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate = false;

	// Import the actors.
	TStrongObjectPtr<ULevelFactory> Factory(NewObject<ULevelFactory>());
	Factory->LevelFactoryCreateText(ULevel::StaticClass(), InWorld->GetCurrentLevel(), InWorld->GetCurrentLevel()->GetFName(), RF_Transactional, NULL, bDuplicate ? TEXT("move") : TEXT("paste"), Paste, Paste + FCString::Strlen(Paste), GWarn, &OutPastedActors);

	// Reinstate old BSP update setting, and force a rebuild - any levels whose geometry has changed while pasting will be rebuilt
	GetMutableDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate = bBSPAutoUpdate;
	
	// FactoryCreateText sets the selection to the new actors, so restore the original selection
	FObjectReader(ActorSelection, OriginalSelectionState);

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied			LevelDirtyCallback;

	// Update the actors' locations and update the global list of visible layers.
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	bool bRebuildBSP = false;
	for (AActor* Actor : OutPastedActors)
	{
		if (!bRebuildBSP)
		{
			if (ABrush* Brush = Cast<ABrush>(Actor))
			{
				bRebuildBSP = Brush->IsStaticBrush();
			}
		}

		if (!LocationOffset.IsZero())
		{
			// We only want to offset the location if this actor is the root of a selected attachment hierarchy
			// Offsetting children of an attachment hierarchy would cause them to drift away from the node they're attached to
			// as the offset would effectively get applied twice
			AActor* ParentActor = Actor->GetAttachParentActor();
			if (!ParentActor)
			{
				Actor->TeleportTo(Actor->GetActorLocation() + LocationOffset, Actor->GetActorRotation(), false, true);
			}
			else if(!OutPastedActors.Contains(ParentActor))
			{
				FName SocketName = Actor->GetAttachParentSocketName();
				Actor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
				Actor->TeleportTo(Actor->GetActorLocation() + LocationOffset, Actor->GetActorRotation(), false, true);
				Actor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform, SocketName);
			}
		}

		if (!GetDefault<ULevelEditorMiscSettings>()->bAvoidRelabelOnPasteSelected)
		{
			// Re-label duplicated actors so that labels become unique
			FActorLabelUtilities::SetActorLabelUnique(Actor, Actor->GetActorLabel(), &ActorLabels);
			ActorLabels.Add(Actor->GetActorLabel());
		}
		else
		{
			// Make sure to broadcast label change after actor import as it may change between spawn and import
			FCoreDelegates::OnActorLabelChanged.Broadcast(Actor);
		}

		LayersSubsystem->InitializeNewActorLayers(Actor);

		// Ensure any layers this actor belongs to are visible
		LayersSubsystem->SetLayersVisibility(Actor->Layers, true);

		Actor->CheckDefaultSubobjects();
		Actor->InvalidateLightingCache();
		// Call PostEditMove to update components, etc.
		Actor->PostEditMove(true);
		Actor->PostDuplicate(EDuplicateMode::Normal);
		Actor->CheckDefaultSubobjects();

		// Request saves/refreshes.
		Actor->MarkPackageDirty();
		LevelDirtyCallback.Request();
	}

	if (bRebuildBSP)
	{
		RebuildAlteredBSP();
	}
}

namespace  {
/**
 * A collection of actors to duplicate and prefabs to instance that all belong to the same level.
 */
class FDuplicateJob
{
public:
	/** A list of actors to duplicate. */
	TArray<AActor*>	Actors;

	/** The source level that all actors in the Actors array come from. */
	ULevel*			SrcLevel;

	/**
	 * Duplicate the job's actors to the specified destination level.  The new actors
	 * are appended to the specified output lists of actors.
	 *
	 * @param	OutNewActors			[out] Newly created actors are appended to this list.
	 * @param	DestLevel				The level to duplicate the actors in this job to.
	 * @param	bOffsetLocations		@see PasteActors.
	 */
	void DuplicateActorsToLevel(TArray<AActor*>& OutNewActors, ULevel* DestLevel, const FVector& LocationOffset)
	{
		// Check neither level is locked
		if (FLevelUtils::IsLevelLocked(SrcLevel))
		{
			UE_LOGF(LogEditorActor, Warning, "DuplicateActorsToLevel: The requested operation could not be completed because the level is locked.");
			return;
		}

		TArray<FTransform> ActorTransforms;
		for (AActor* Actor : Actors)
		{
			ActorTransforms.Add(Actor->GetActorTransform());
		}

		if (!ActorPlacementUtils::IsLevelValidForActorPlacement(DestLevel, ActorTransforms))
		{
			return;
		}

		FString ScratchData;

		// Copy actors from src level
		{
			// Cache the current source level
			ULevel* CurrentSrcLevel = SrcLevel->OwningWorld->GetCurrentLevel();
			SrcLevel->OwningWorld->SetCurrentLevel(SrcLevel);
			GUnrealEd->CopyActors(Actors, SrcLevel->OwningWorld, &ScratchData);
			SrcLevel->OwningWorld->SetCurrentLevel(CurrentSrcLevel);
		}

		// Paste to the dest level
		{
			ULevel* CurrentDestLevel = DestLevel->OwningWorld->GetCurrentLevel();
			DestLevel->OwningWorld->SetCurrentLevel(DestLevel);

			TArray<AActor*> PastedActors;
			GUnrealEd->PasteActors(PastedActors, DestLevel->OwningWorld, LocationOffset, /*bDuplicate*/true, /*bWarnIfHidden*/true, &ScratchData);
			
			DestLevel->OwningWorld->SetCurrentLevel(CurrentDestLevel);

			// We generate new seeds when we duplicate
			for (AActor* Actor : PastedActors)
			{
				Actor->SeedAllRandomStreams();
			}

			OutNewActors.Append(MoveTemp(PastedActors));
		}
	}
};
}


void UUnrealEdEngine::edactDuplicateSelected( ULevel* InLevel, bool bOffsetLocations )
{
	if (GetSelectedComponentCount() > 0)
	{
		USelection* ComponentSelection = GetSelectedComponents();

		TArray<UActorComponent*> SelectedComponents;
		ComponentSelection->GetSelectedObjects<UActorComponent>(SelectedComponents);

		TArray<UActorComponent*> NewComponents;
		DuplicateComponents(SelectedComponents, NewComponents);

		if (NewComponents.Num() > 0)
		{
			// Select the new clones
			ComponentSelection->Modify();
			ComponentSelection->BeginBatchSelectOperation();
			ComponentSelection->DeselectAll();

			for (UActorComponent* NewComponent : NewComponents)
			{
				GEditor->SelectComponent(NewComponent, true, false);
			}

			ComponentSelection->EndBatchSelectOperation(true);
		}
	}
	else
	{
		USelection* ActorSelection = GetSelectedActors();

		TArray<AActor*> SelectedActors;
		ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

		TArray<AActor*> NewActors;
		DuplicateActors(SelectedActors, NewActors, InLevel, bOffsetLocations ? GEditor->GetGridLocationOffset(/*bUniformOffset*/false) : FVector::ZeroVector);

		if (NewActors.Num() > 0)
		{
			// Select the new clones
			ActorSelection->Modify();
			ActorSelection->BeginBatchSelectOperation();
			ActorSelection->DeselectAll();

			for (AActor* Actor : NewActors)
			{
				GEditor->SelectActor(Actor, true, false);
			}

			ActorSelection->EndBatchSelectOperation(true);

			// Note the selection change.  This will also redraw level viewports and update the pivot.
			NoteSelectionChange();

			GLevelEditorModeTools().ActorsDuplicatedNotify(SelectedActors, NewActors, bOffsetLocations);
		}
	}
}

void UUnrealEdEngine::DuplicateComponents(const TArray<UActorComponent*>& InComponentsToDuplicate, TArray<UActorComponent*>& OutNewComponents)
{
	OutNewComponents.Reset(InComponentsToDuplicate.Num());

	for (UActorComponent* Component : InComponentsToDuplicate)
	{
		if (FComponentEditorUtils::CanCopyComponent(Component))
		{
			if (UActorComponent* NewComponent = FComponentEditorUtils::DuplicateComponent(Component))
			{
				OutNewComponents.Add(NewComponent);
			}
		}
	}

	if (OutNewComponents.Num() > 0)
	{
		// Make sure all the SCS trees have a chance to update
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.BroadcastComponentsEdited();
	}
}

void UUnrealEdEngine::DuplicateActors(const TArray<AActor*>& InActorsToDuplicate, TArray<AActor*>& OutNewActors, ULevel* InLevel, const FVector& LocationOffset)
{
	OutNewActors.Reset(InActorsToDuplicate.Num());

	const FScopedBusyCursor BusyCursor;

	// Create per-level job lists.
	TMap<ULevel*, TSharedPtr<FDuplicateJob>> DuplicateJobs;
	for (AActor* Actor : InActorsToDuplicate)
	{
		ULevel* SrcLevel = Actor->GetLevel();
		TSharedPtr<FDuplicateJob> Job = DuplicateJobs.FindRef(SrcLevel);
		if (!Job)
		{
			// Allocate a new job for the level.
			Job = DuplicateJobs.Add(SrcLevel, MakeShared<FDuplicateJob>());
			Job->SrcLevel = SrcLevel;
		}
		Job->Actors.Add(Actor);
	}

	// For each level, select the actors in that level and copy-paste into the destination level.
	for (const auto& DuplicateJobPair : DuplicateJobs)
	{
		FDuplicateJob& Job = *DuplicateJobPair.Value;
		Job.DuplicateActorsToLevel(OutNewActors, InLevel, LocationOffset);
	}

	// Finally, cleanup.
	DuplicateJobs.Reset();
}

bool UUnrealEdEngine::CanDeleteSelectedActors( const UWorld* InWorld, const bool bStopAtFirst, const bool bLogUndeletable, TArray<AActor*>* OutDeletableActors ) const
{
	// Iterate over selected actors and assemble a list of actors to delete.
	bool bContainsDeletable = false;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor		= static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		bool bDeletable = false;
		FText CannotDeleteReason;
		if (CanDeleteActor(Actor, &CannotDeleteReason))
		{
			bContainsDeletable = true;
			bDeletable = true;
		}

		// Can this actor be deleted
		if ( bDeletable )
		{
			if ( OutDeletableActors )
			{
				OutDeletableActors->Add( Actor );
			}
			if ( bStopAtFirst )
			{
				break;	// Did we only want to know if ANY of the actors were deletable
			}
		}
		else if ( bLogUndeletable )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Name"), FText::FromString( Actor->GetFullName() ));

			FText LogText;
			if (CannotDeleteReason.IsEmpty())
			{
				LogText = FText::Format(LOCTEXT("CannotDeleteActor", "Cannot delete actor {Name}"), Arguments);
			}
			else
			{
				Arguments.Add(TEXT("Reason"), CannotDeleteReason);
				LogText = FText::Format(LOCTEXT("CannotDeleteActorWithReason", "Cannot delete actor {Name} - {Reason}"), Arguments);
			}
			UE_LOGF(LogEditorActor, Log, "%ls", *LogText.ToString());
		}
	}
	return bContainsDeletable;
}

bool UUnrealEdEngine::CanDeleteComponent(const UActorComponent* InComponent, FText* OutReason) const
{
	if (!InComponent->IsEditableWhenInherited())
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CanDeleteComponent_Error_ComponentNotEditableWhenInherited", "Can't delete non-editable component.");
		}
		return false;
	}

	if (!FComponentEditorUtils::CanDeleteComponent(InComponent))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CanDeleteComponent_Error_InvalidComponent", "Can't delete non-instanced components or the scene root.");
		}
		return false;
	}

	return true;
}

bool UUnrealEdEngine::CanDeleteActor(const AActor* InActor, FText* OutReason) const
{
	if (!InActor->HasAllFlags(RF_Transactional))
	{
		if (OutReason)
		{
			*OutReason = LOCTEXT("CanDeleteActor_Error_NotTransactional", "Can't delete non-transactional actors.");
		}
		return false;
	}

	{
		FText TmpReason;
		if (!InActor->CanDeleteSelectedActor(TmpReason))
		{
			if (OutReason)
			{
				*OutReason = TmpReason;
			}
			return false;
		}
	}

	return true;
}

bool UUnrealEdEngine::ShouldAbortComponentDeletion(const TArray<UActorComponent*>& InComponentsToDelete, FText* OutReason) const
{
	for (UActorComponent* Component : InComponentsToDelete)
	{
		AActor* OwnerActor = Component->GetOwner();
		if (OwnerActor && FLevelUtils::IsLevelLocked(OwnerActor))
		{
			if (OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("ShouldAbortComponentDeletion_Error_LevelLockedDuringComponentDeletion", "Cannot delete component '{0}' because its owner level is locked."), FText::FromString(Component->GetPathName()));
			}
			return true;
		}
	}

	return false;
}

bool UUnrealEdEngine::ShouldAbortActorDeletion(const TArray<AActor*>& InActorsToDelete, FText* OutReason) const
{
	for (AActor* Actor : InActorsToDelete)
	{
		if (FLevelUtils::IsLevelLocked(Actor))
		{
			if (OutReason)
			{
				*OutReason = FText::Format(LOCTEXT("ShouldAbortActorDeletion_LevelLockedDuringActorDeletion", "Cannot delete actor '{0}' because its owner level is locked."), FText::FromString(Actor->GetPathName()));
			}
			return true;
		}
	}

	return false;
}

bool UUnrealEdEngine::DeleteComponents(const TArray<UActorComponent*>& InComponentsToDelete, UTypedElementSelectionSet* InSelectionSet, const bool bVerifyDeletionCanHappen)
{
	if (bVerifyDeletionCanHappen)
	{
		FText ErrorMsg;
		if (ShouldAbortComponentDeletion(InComponentsToDelete, &ErrorMsg))
		{
			FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMsg));
			return false;
		}
	}

	const double StartSeconds = FPlatformTime::Seconds();

	FSlateApplication::Get().CancelDragDrop();

	// Get a list of all the deletable components
	TArray<UActorComponent*> EditableComponentsToDelete;
	EditableComponentsToDelete.Reserve(InComponentsToDelete.Num());
	for (UActorComponent* ComponentToDelete : InComponentsToDelete)
	{
		FText ErrorMsg;
		if (!CanDeleteComponent(ComponentToDelete, &ErrorMsg))
		{
			UE_LOGF(LogEditorActor, Log, "Cannot delete component %ls: %ls", *ComponentToDelete->GetFullName(), *ErrorMsg.ToString());
			continue;
		}

		// Modify the actor that owns the component
		if (AActor* OwnerActor = ComponentToDelete->GetOwner())
		{
			OwnerActor->Modify();
		}

		EditableComponentsToDelete.Add(ComponentToDelete);
	}

	if (EditableComponentsToDelete.Num() == 0)
	{
		return false;
	}

	TUniquePtr<FTypedElementList::FLegacySyncScopedBatch> LegacySyncBatch = MakeUnique<FTypedElementList::FLegacySyncScopedBatch>(*InSelectionSet->GetElementList());

	const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
		.SetAllowHidden(true)
		.SetAllowGroups(false)
		.SetWarnIfLocked(false)
		.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);

	// Clear the selection of any components we may delete
	for (UActorComponent* ComponentToDelete : EditableComponentsToDelete)
	{
		if (FTypedElementHandle ComponentHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(ComponentToDelete, /*bAllowCreate*/false))
		{
			InSelectionSet->DeselectElement(ComponentHandle, SelectionOptions);
		}
	}

	// Delete the components
	UActorComponent* ComponentToSelect = nullptr;
	const int32 NumDeletedComponents = FComponentEditorUtils::DeleteComponents(EditableComponentsToDelete, ComponentToSelect);

	if (NumDeletedComponents > 0)
	{
		// Make sure all the SCS trees have a chance to rebuild
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.BroadcastComponentsEdited();

		// Remove all references to destroyed components once at the end, instead of once for each component destroyed
		{
			FGarbageEliminationScopeGuard Guard(true);
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	// Update the editor component selection if possible
	if (ComponentToSelect)
	{
		InSelectionSet->SelectElement(UEngineElementsLibrary::AcquireEditorComponentElementHandle(ComponentToSelect), SelectionOptions);
	}

	// Make sure the selection changed event fires so the SCS trees can update their selection
	LegacySyncBatch->ForceDirty();
	LegacySyncBatch.Reset();
	NoteSelectionChange();

	UE_LOGF(LogEditorActor, Log, "Deleted %d Components (%3.3f secs)", NumDeletedComponents, FPlatformTime::Seconds() - StartSeconds);
	return NumDeletedComponents > 0;
}

bool UUnrealEdEngine::DeleteActors(const TArray<AActor*>& InActorsToDelete, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const bool bVerifyDeletionCanHappen, const bool bWarnAboutReferences, const bool bWarnAboutSoftReferences)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUnrealEdEngine::DeleteActors);
	if (bVerifyDeletionCanHappen)
	{
		// TODO: Bubble up error message?
		FText ErrorMsg;
		if (ShouldAbortActorDeletion(InActorsToDelete, &ErrorMsg))
		{
			FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMsg));
			return false;
		}
	}

	const double StartSeconds = FPlatformTime::Seconds();

	// Aggregate the time we waited on user input to remove it from the total time
	double DialogWaitingSeconds = 0;

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Get a list of all the deletable actors
	TArray<AActor*> ActorsToDelete;
	TArray<FSoftObjectPath> ActorsToDeletePaths;
	ActorsToDelete.Reserve(InActorsToDelete.Num());
	ActorsToDeletePaths.Reserve(InActorsToDelete.Num());
	for (AActor* ActorToDelete : InActorsToDelete)
	{
		FText ErrorMsg;
		if (!CanDeleteActor(ActorToDelete, &ErrorMsg))
		{
			// If this was a temporary editor preview actor, go ahead and delete it now
			if (ActorToDelete->bIsEditorPreviewActor)
			{
				InWorld->DestroyActor(ActorToDelete);
			}
			else
			{
				UE_LOGF(LogEditorActor, Log, "Cannot delete actor %ls: %ls", *ActorToDelete->GetFullName(), *ErrorMsg.ToString());
			}
			continue;
		}

		ActorsToDelete.Add(ActorToDelete);
		ActorsToDeletePaths.Add(ActorToDelete);
	}

	TMap<AActor*, TArray<AActor*>> ReferencingActorsMap;
	TMap<FSoftObjectPath, TArray<UObject*>> SoftReferencingObjectsMap;
	{
		TArray<UClass*> ClassTypesToIgnore;
		ClassesToIgnoreDeleteReferenceWarning.AddUnique(ALevelScriptActor::StaticClass());
		// The delete warning is meant for actor references that affect gameplay.  Group actors do not affect gameplay and should not show up as a warning.
		ClassesToIgnoreDeleteReferenceWarning.AddUnique(AGroupActor::StaticClass());

		// If we want to warn about references to the actors to be deleted, it is a lot more efficient to query
		// the world first and build a map of actors referenced by other actors. We can then quickly look this up later on in the loop.
		if (bWarnAboutReferences)
		{
			FBlueprintEditorUtils::GetActorReferenceMap(InWorld,MutableView(ClassesToIgnoreDeleteReferenceWarning), ReferencingActorsMap);

			if (bWarnAboutSoftReferences)
			{
				FScopedSlowTask SlowTask(static_cast<float>(ActorsToDeletePaths.Num()), LOCTEXT("ComputeActorSoftReferences", "Computing References"));
				SlowTask.MakeDialogDelayed(1.0f);

				FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				AssetToolsModule.Get().FindSoftReferencesToObjects(ActorsToDeletePaths, SoftReferencingObjectsMap);
			}
		}
	}

	// Maintain a list of levels that have already been Modify()'d so that each level is modified only once
	TArray<ULevel*> LevelsAlreadyModified;
	// A list of levels that will need their BSP updated after the deletion is complete
	TSet<ULevel*> LevelsToRebuildBSP;
	TSet<ULevel*> LevelsToRebuildNavigation;

	bool bRequestedDeleteAllByLevel = false;
	bool bRequestedDeleteAllByActor = false;
	bool bRequestedDeleteAllBySoftReference = false;
	bool bRequestedDeleteAllByGroup = false;
	int32 DeleteCount = 0;

	TUniquePtr<FTypedElementList::FLegacySyncScopedBatch> LegacySyncBatch = MakeUnique<FTypedElementList::FLegacySyncScopedBatch>(*InSelectionSet->GetElementList());

	const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
		.SetAllowHidden(true)
		.SetAllowGroups(false)
		.SetWarnIfLocked(false)
		.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);

	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	for (AActor* Actor : ActorsToDelete)
	{
		//If actor is referenced by script, ask user if they really want to delete
		ULevelScriptBlueprint* LSB = Actor->GetLevel()->GetLevelScriptBlueprint(true);

		// Get the array of actors that reference this actor from the cached map we built above.
		TArray<AActor*>* ReferencingActors = nullptr;
		if (bWarnAboutReferences)
		{
			ReferencingActors = ReferencingActorsMap.Find(Actor);
		}

		TArray<UK2Node*> ReferencedToActorsFromLevelScriptArray;
		FBlueprintEditorUtils::FindReferencesToActorFromLevelScript(LSB, Actor, ReferencedToActorsFromLevelScriptArray);

		bool bReferencedByLevelScript = bWarnAboutReferences && (nullptr != LSB && ReferencedToActorsFromLevelScriptArray.Num() > 0);
		bool bReferencedByActor = false;
		bool bReferencedByLODActor = false;
		bool bReferencedByGroupActor = false;
		bool bReferencedBySoftReference = false;
		TArray<UObject*>* SoftReferencingObjects = nullptr;
		const UPackage* ActorPackage = Actor->GetPackage();

		if (bWarnAboutSoftReferences)
		{
			SoftReferencingObjects = SoftReferencingObjectsMap.Find(Actor);

			if (SoftReferencingObjects)
			{
				// Remove any references from object types marked to be ignored
				for (int32 i = SoftReferencingObjects->Num() - 1; i >= 0; --i)
				{
					const UObject* SoftReferencingObject = (*SoftReferencingObjects)[i];
					if (const AActor* ReferencingActor = Cast<AActor>(SoftReferencingObject); ReferencingActor && ActorsToDelete.Contains(ReferencingActor))
					{
						SoftReferencingObjects->RemoveAt(i);
					} // Object is inside the actor being deleted, so reference should be ignored (check that they share the same package because that could mean we are in a special case where warnings are needed)
					else if (SoftReferencingObject->IsInOuter(Actor) && SoftReferencingObject->GetPackage() == ActorPackage)
					{
						SoftReferencingObjects->RemoveAt(i);
					}
					else
					{
						for (const TObjectPtr<UClass>& ClassToIgnore : ClassesToIgnoreDeleteReferenceWarning)
						{
							if (SoftReferencingObject->IsA(ClassToIgnore))
							{
								SoftReferencingObjects->RemoveAt(i);
								break;
							}
						}
					}
				}

				bReferencedBySoftReference = SoftReferencingObjects->Num() > 0;
			}
		}

		// If there are any referencing actors, make sure that they are reference types that we care about.
		if (ReferencingActors != nullptr)
		{
			for (AActor* ReferencingActor : *ReferencingActors)
			{
				// Skip to next if we are referencing ourselves
				if (ReferencingActor == Actor || ActorsToDelete.Contains(ReferencingActor))
				{
					continue;
				}
				else if (Cast<ALODActor>(ReferencingActor))
				{
					bReferencedByLODActor = true;
				}
				else if (Cast<AGroupActor>(ReferencingActor))
				{
					bReferencedByGroupActor = true;
				}
				else
				{
					// If the referencing actor is a child actor that is referencing us, do not treat it
					// as referencing for the purposes of warning about deletion
					UChildActorComponent* ParentComponent = ReferencingActor->GetParentComponent();
					if (ParentComponent == nullptr || ParentComponent->GetOwner() != Actor)
					{
						bReferencedByActor = true;

						FText ActorReferencedMessage = FText::Format(LOCTEXT("ActorDeleteReferencedMessage", "Actor {0} is referenced by {1}."),
							FText::FromString(Actor->GetActorLabel()),
							FText::FromString(ReferencingActor->GetActorLabel())
						);
						UE_LOGF(LogEditorActor, Log, "%ls", *ActorReferencedMessage.ToString());
					}
				}
			}
		}

		// We have references from one or more sources, prompt the user for feedback.
		if (bReferencedByLevelScript || bReferencedByActor || bReferencedBySoftReference || bReferencedByLODActor || bReferencedByGroupActor)
		{
			if ((bReferencedByLevelScript && !bRequestedDeleteAllByLevel) ||
				(bReferencedByActor && !bRequestedDeleteAllByActor) ||
				(bReferencedBySoftReference && !bRequestedDeleteAllBySoftReference) ||
				(bReferencedByGroupActor && !bRequestedDeleteAllByGroup))
			{
				FText ConfirmDelete;

				// We separate groups and other actors, so we can visually display them separately in the list
				TArray<TSharedPtr<FText>> GroupActorReferencers;
				TArray<TSharedPtr<FText>> OtherReferencers;

				if (ReferencingActors != nullptr)
				{
					for (AActor* ReferencingActor : *ReferencingActors)
					{
						if (ReferencingActor->IsA<ALevelScriptActor>())
						{
							continue;
						}

						const FString ActorLabel = FString::Printf(TEXT("%s"), *ReferencingActor->GetActorLabel());

						if (ReferencingActor->IsA<AGroupActor>())
						{
							GroupActorReferencers.Add(MakeShared<FText>(FText::FromString(ActorLabel)));
						}
						else if (!ReferencingActor->IsA<ALevelScriptActor>())
						{
							OtherReferencers.Add(MakeShared<FText>(FText::FromString(ActorLabel)));
						}
					}
				}

				OtherReferencers.Append(GroupActorReferencers);

				if (bReferencedBySoftReference)
				{
					for (UObject* ReferencingObject : *SoftReferencingObjects)
					{
						FString ActorLabel;

						if (AActor* ReferencingActor = Cast<AActor>(ReferencingObject))
						{
							ActorLabel = FString::Printf(TEXT("%s in %s (soft reference)"), *ReferencingActor->GetActorLabel(), *FPackageName::GetLongPackageAssetName(ReferencingActor->GetOutermost()->GetName()));
						}
						else
						{
							ActorLabel = FString::Printf(TEXT("%s (soft reference)"), *ReferencingObject->GetPathName());
						}

						OtherReferencers.Add(MakeShared<FText>(FText::FromString(ActorLabel)));
					}
				}

				EDeletedActorReferenceTypes DeletedActorReferenceTypes = EDeletedActorReferenceTypes::None;

				if (bReferencedByGroupActor)
				{
					EnumAddFlags(DeletedActorReferenceTypes, EDeletedActorReferenceTypes::Group);
				}

				if (bReferencedByActor || bReferencedBySoftReference)
				{
					EnumAddFlags(DeletedActorReferenceTypes, EDeletedActorReferenceTypes::ActorOrAsset);
				}

				if (bReferencedByLevelScript)
				{
					EnumAddFlags(DeletedActorReferenceTypes, EDeletedActorReferenceTypes::LevelBlueprint);
				}

				const double DialogStartSeconds = FPlatformTime::Seconds();

				const bool bCanShowApplyToAll = ActorsToDelete.Num() > 1;

				TSharedRef<SDeleteReferencedActorDialog> ConfirmDialog = SNew(SDeleteReferencedActorDialog)
					.Referencers(OtherReferencers)
					.ShowApplyToAll(bCanShowApplyToAll)
					.ActorToDeleteLabel(Actor->GetActorLabel())
					.ReferenceTypes(DeletedActorReferenceTypes);

				uint32 Result = ConfirmDialog->ShowModal();

				DialogWaitingSeconds += FPlatformTime::Seconds() - DialogStartSeconds;

				if (ConfirmDialog->GetApplyToAll())
				{
					if (Result == 0)
					{
						bRequestedDeleteAllByLevel |= bReferencedByLevelScript;
						bRequestedDeleteAllByActor |= bReferencedByActor;
						bRequestedDeleteAllBySoftReference |= bReferencedBySoftReference;
						bRequestedDeleteAllByGroup |= bReferencedByGroupActor;
					}
					else
					{
						break;
					}
				}
				else if (Result == 1)
				{
					continue;
				}
			}

			if (bReferencedByLevelScript)
			{
				FBlueprintEditorUtils::ModifyActorReferencedGraphNodes(LSB, Actor);
			}

			if (bReferencedByActor || bReferencedByLODActor || bReferencedByGroupActor)
			{
				check(ReferencingActors != nullptr);
				for (AActor* ReferencingActor : *ReferencingActors)
				{
					ReferencingActor->Modify();

					ALODActor* LODActor = Cast<ALODActor>(ReferencingActor);
					// it's possible other actor is referencing this
					if (LODActor)
					{
						LODActor->RemoveSubActor(Actor);

						FText SubActorRemovedMessage = FText::Format(LOCTEXT("LODActorSubActorDeletedMessage", "Sub Actor '{0}' was removed from LODActor '{1}'."),
							FText::FromString(Actor->GetActorLabel()),
							FText::FromString(ReferencingActor->GetActorLabel())
						);
						UE_LOGF(LogEditorActor, Log, "%ls", *SubActorRemovedMessage.ToString());
					}
				}
			}
		}

		bool bRebuildNavigation = false;

		ABrush* Brush = Cast< ABrush >(Actor);
		if (Brush && !FActorEditorUtils::IsABuilderBrush(Brush)) // Track whether or not a brush actor was deleted.
		{
			ULevel* BrushLevel = Actor->GetLevel();
			if (BrushLevel && !Brush->IsVolumeBrush())
			{
				BrushLevel->Model->Modify(false);
				LevelsToRebuildBSP.Add(BrushLevel);
				// Rebuilding bsp will also take care of navigation
				LevelsToRebuildNavigation.Remove(BrushLevel);
			}
			else if (BrushLevel && !LevelsToRebuildBSP.Contains(BrushLevel))
			{
				LevelsToRebuildNavigation.Add(BrushLevel);
			}
		}

		// If the actor about to be deleted is in a group, be sure to remove it from the group
		AGroupActor* ActorParentGroup = AGroupActor::GetParentForActor(Actor);
		if (ActorParentGroup)
		{
			ActorParentGroup->Remove(*Actor);
		}

		// Remove actor from all asset editors
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->RemoveAssetFromAllEditors(Actor);

		// Mark the actor's level as dirty.
		Actor->MarkPackageDirty();
		LevelDirtyCallback.Request();

		// Deselect the Actor.
		if (FTypedElementHandle ActorHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor, /*bAllowCreate*/false))
		{
			InSelectionSet->DeselectElement(ActorHandle, SelectionOptions);
		}
		UEngineElementsLibrary::UnregisterActorElement(Actor);

		// Modify the level.  Each level is modified only once.
		// @todo DB: Shouldn't this be calling UWorld::ModifyLevel?
		ULevel* Level = Actor->GetLevel();
		if (LevelsAlreadyModified.Find(Level) == INDEX_NONE)
		{
			LevelsAlreadyModified.Add(Level);
			// Don't mark the level dirty when deleting a transient actor, or an external actor when the level is in `use external actors` mode.
			bool bShouldDirty = !Actor->HasAllFlags(RF_Transient) && (!Actor->IsPackageExternal() || !Level->IsUsingExternalActors());
			Level->Modify(bShouldDirty);
		}

		UE_LOGF(LogEditorActor, Log, "Deleted Actor: %ls", *Actor->GetClass()->GetName());

		// Destroy actor and clear references.
		LayersSubsystem->DisassociateActorFromLayers(Actor);
		bool WasDestroyed = Actor->GetWorld()->EditorDestroyActor(Actor, false);
		checkf(WasDestroyed, TEXT("Failed to destroy Actor %s (%s)"), *Actor->GetClass()->GetName(), *Actor->GetActorLabel());

		DeleteCount++;
	}

	// Remove all references to destroyed actors once at the end, instead of once for each Actor destroyed
	{
		FGarbageEliminationScopeGuard Guard(true);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	LegacySyncBatch.Reset();
	NoteSelectionChange();

	// If any brush actors were modified, update the Bsp in the appropriate levels
	if (LevelsToRebuildBSP.Num())
	{
		FlushRenderingCommands();

		for (ULevel* Level : LevelsToRebuildBSP)
		{
			GEditor->RebuildLevel(*Level);
		}
	}

	if (LevelsToRebuildNavigation.Num())
	{
		for (ULevel* Level : LevelsToRebuildNavigation)
		{
			if (Level)
			{
				FNavigationSystem::UpdateLevelCollision(*Level);
			}
		}
	}

	if (LevelsToRebuildBSP.Num() || LevelsToRebuildNavigation.Num())
	{
		RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
	}

	UE_LOGF(LogEditorActor, Log, "Deleted %d Actors (%3.3f secs)", DeleteCount, (FPlatformTime::Seconds() - StartSeconds) - DialogWaitingSeconds);
	return true;
}

bool UUnrealEdEngine::edactDeleteSelected( UWorld* InWorld, bool bVerifyDeletionCanHappen, bool bWarnAboutReferences, bool bWarnAboutSoftReferences)
{
	if (GetSelectedComponentCount() > 0)
	{
		// Delete selected components
		TArray<UActorComponent*> SelectedComponents;
		GetSelectedComponents()->GetSelectedObjects<UActorComponent>(SelectedComponents);
		return DeleteComponents(SelectedComponents, GetSelectedComponents()->GetElementSelectionSet(), bVerifyDeletionCanHappen);
	}
	else
	{
		// Delete selected actors
		TArray<AActor*> SelectedActors;
		GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);
		return DeleteActors(SelectedActors, InWorld, GetSelectedActors()->GetElementSelectionSet(), bVerifyDeletionCanHappen, bWarnAboutReferences, bWarnAboutSoftReferences);
	}
}

bool UUnrealEdEngine::ShouldAbortActorDeletion() const
{
	TArray<AActor*> SelectedActors;
	GetSelectedActors()->GetSelectedObjects<AActor>(SelectedActors);

	FText ErrorMsg;
	if (ShouldAbortActorDeletion(SelectedActors, &ErrorMsg))
	{
		FSlateNotificationManager::Get().AddNotification(FNotificationInfo(ErrorMsg));
		return true;
	}

	return false;
}

void UUnrealEdEngine::edactReplaceSelectedBrush( UWorld* InWorld )
{
	// Make a list of brush actors to replace.
	ABrush* DefaultBrush = InWorld->GetDefaultBrush();

	TArray<ABrush*> BrushesToReplace;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );
		ABrush* Brush = Cast< ABrush >( Actor );
		if ( Brush && Actor->HasAnyFlags(RF_Transactional) && Actor != DefaultBrush )
		{
			BrushesToReplace.Add( Brush );
		}
	}

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied			LevelDirtyCallback;

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();

	// Replace brushes.
	ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
	for ( int32 BrushIndex = 0 ; BrushIndex < BrushesToReplace.Num() ; ++BrushIndex )
	{
		ABrush* SrcBrush = BrushesToReplace[BrushIndex];
		ABrush* NewBrush = FBSPOps::csgAddOperation( DefaultBrush, SrcBrush->PolyFlags, (EBrushType)SrcBrush->BrushType );
		if( NewBrush )
		{
			if( NewBrush->GetBrushBuilder() )
			{
				FActorLabelUtilities::SetActorLabelUnique(NewBrush, FText::Format(NSLOCTEXT("UnrealEd", "BrushName", "{0} Brush"), FText::FromString(NewBrush->GetBrushBuilder()->GetClass()->GetDescription())).ToString());
			}

			SrcBrush->MarkPackageDirty();
			NewBrush->MarkPackageDirty();

			LevelDirtyCallback.Request();

			NewBrush->Modify(false);

			NewBrush->Layers.Append( SrcBrush->Layers );

			NewBrush->CopyPosRotScaleFrom( SrcBrush );
			NewBrush->PostEditMove( true );
			SelectActor( SrcBrush, false, false );
			SelectActor( NewBrush, true, false );

			LayersSubsystem->DisassociateActorFromLayers( SrcBrush );
			InWorld->EditorDestroyActor( SrcBrush, true );
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


AActor* UUnrealEdEngine::ReplaceActor( AActor* CurrentActor, UClass* NewActorClass, UObject* Archetype, bool bNoteSelectionChange )
{
	FVector SpawnLoc = CurrentActor->GetActorLocation();
	FRotator SpawnRot = CurrentActor->GetActorRotation();
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Template = Cast<AActor>(Archetype);
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	AActor* NewActor = CurrentActor->GetWorld()->SpawnActor( NewActorClass, &SpawnLoc, &SpawnRot, SpawnInfo );
	if( NewActor )
	{
		NewActor->Modify();
		ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		LayersSubsystem->InitializeNewActorLayers( NewActor );

		const bool bCurrentActorSelected = GetSelectedActors()->IsSelected( CurrentActor );
		if ( bCurrentActorSelected )
		{
			// The source actor was selected, so deselect the old actor and select the new one.
			GetSelectedActors()->Modify();
			SelectActor( NewActor, bCurrentActorSelected, false );
			SelectActor( CurrentActor, false, false );
		}

		{
			LayersSubsystem->DisassociateActorFromLayers( NewActor );
			NewActor->Layers.Empty();

			LayersSubsystem->AddActorToLayers( NewActor, CurrentActor->Layers );

			NewActor->EditorReplacedActor( CurrentActor );
		}

		LayersSubsystem->DisassociateActorFromLayers( CurrentActor );
		CurrentActor->GetWorld()->EditorDestroyActor( CurrentActor, true );

		// Note selection change if necessary and requested.
		if ( bCurrentActorSelected && bNoteSelectionChange )
		{
			NoteSelectionChange();
		}

		//whenever selection changes, recompute whether the selection contains a locked actor
		bCheckForMovementLockedItems = true;

		//whenever selection changes, recompute whether the selection contains a world info actor
		bCheckForWorldSettingsActors = true;
	}

	return NewActor;
}


void UUnrealEdEngine::edactReplaceSelectedNonBrushWithClass(UClass* Class)
{
	// Make a list of actors to replace.
	TArray<AActor*> ActorsToReplace;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );
		ABrush* Brush = Cast< ABrush >( Actor );
		if ( !Brush && Actor->HasAnyFlags(RF_Transactional) )
		{
			ActorsToReplace.Add( Actor );
		}
	}

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied			LevelDirtyCallback;

	// Replace actors.
	for ( int32 i = 0 ; i < ActorsToReplace.Num() ; ++i )
	{
		AActor* SrcActor = ActorsToReplace[i];
		AActor* NewActor = ReplaceActor( SrcActor, Class, NULL, false );
		if ( NewActor )
		{
			NewActor->MarkPackageDirty();
			LevelDirtyCallback.Request();
		}
	}

	NoteSelectionChange();
}


void UUnrealEdEngine::edactReplaceClassWithClass(UWorld* InWorld, UClass* SrcClass, UClass* DstClass)
{
	// Make a list of actors to replace.
	TArray<AActor*> ActorsToReplace;
	for( TActorIterator<AActor> It(InWorld, SrcClass); It; ++It )
	{
		AActor* Actor = *It;
		if ( Actor->HasAnyFlags(RF_Transactional) )
		{
			ActorsToReplace.Add( Actor );
		}
	}

	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied			LevelDirtyCallback;

	// Replace actors.
	for ( int32 i = 0 ; i < ActorsToReplace.Num() ; ++i )
	{
		AActor* SrcActor = ActorsToReplace[i];
		AActor* NewActor = ReplaceActor( SrcActor, DstClass, NULL, false );
		if ( NewActor )
		{
			NewActor->MarkPackageDirty();
			LevelDirtyCallback.Request();
		}
	}

	NoteSelectionChange();
}

static void SetActorVisibility(AActor* Actor, bool bVisible)
{
	// Save the actor to the transaction buffer to support undo/redo, but do
	// not call Modify, as we do not want to dirty the actor's package and
	// we're only editing temporary, transient values
	SaveToTransactionBuffer(Actor, false);
	Actor->SetIsTemporarilyHiddenInEditor(!bVisible);
}

void UUnrealEdEngine::GetChildHierarchyOfSelectedActor(TArray<AActor*>& OutActorArray, AActor* ParentActor, bool bExcludeSelectedChildren, const TOptional<bool>& bIsFindingHidden)
{
	// If the ParentActor is a builder brush or doesn't match the find flag (if it is set), then don't
	// add it to the array or continue searching the hierarchy
	if ( !FActorEditorUtils::IsABuilderBrush(ParentActor) )
	{
		// If the actor doesn't match the bIsFindingHidden, we still want to search its children in case they match
		if( !bIsFindingHidden.IsSet() || ParentActor->IsHiddenEd() == bIsFindingHidden.GetValue() )
		{
			OutActorArray.Add( ParentActor );
		}

		TArray<AActor*> ActorChildren;
		ParentActor->GetAttachedActors(ActorChildren, false, false);
		for (AActor* Child : ActorChildren)
		{
			if (!bExcludeSelectedChildren || !GEditor->GetSelectedActors()->IsSelected(Child))
			{
				GetChildHierarchyOfSelectedActor(OutActorArray, Child, bExcludeSelectedChildren, bIsFindingHidden);
			}
		}
	}
}

void UUnrealEdEngine::Internal_HideSelected( UWorld* InWorld, bool bApplyToChildHierarchy )
{
	// Assemble a list of actors to hide.
	TArray<AActor*> ActorsToHide;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if ( bApplyToChildHierarchy )
		{
			GetChildHierarchyOfSelectedActor(ActorsToHide, Actor, true, false);
		}
		else if ( !FActorEditorUtils::IsABuilderBrush(Actor) && !Actor->IsHiddenEd() )
		{
			ActorsToHide.Add( Actor );
		}
	}
	
	// Hide the actors that were selected and deselect them in the process
	if ( ActorsToHide.Num() > 0 )
	{
		USelection* SelectedActors = GetSelectedActors();
		SelectedActors->Modify();

		for( int32 ActorIndex = 0 ; ActorIndex < ActorsToHide.Num() ; ++ActorIndex )
		{
			AActor* Actor = ActorsToHide[ ActorIndex ];
			
			SetActorVisibility(Actor, false);
			SelectedActors->Deselect( Actor );
		}

		NoteSelectionChange();
	}

	// Iterate through all of the BSP models and hide any that were selected (deselecting them in the process)
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( ( CurSurface.PolyFlags & PF_Selected ) && !CurSurface.IsHiddenEd() )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					
					// Deselect the surface and mark it as hidden to the editor
					CurSurface.PolyFlags &= ~PF_Selected;
					CurSurface.bHiddenEdTemporary = true;
				}
			}
		}
	}

	// Hide selected non-actor elements via TEDS
	UE::EditorActor::Private::SetSelectedNonActorVisibility(InWorld, false, bApplyToChildHierarchy);

	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::edactHideSelected( UWorld* InWorld )
{
	Internal_HideSelected(InWorld, false);
}

void UUnrealEdEngine::edactHideSelectedHierarchy( UWorld* InWorld )
{
	Internal_HideSelected(InWorld, true);
}

void UUnrealEdEngine::edactHideUnselected( UWorld* InWorld )
{
	// Iterate through all of the actors and hide the ones which are not selected and are not already hidden
	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		if( !FActorEditorUtils::IsABuilderBrush(Actor) && !Actor->IsActorOrSelectionParentSelected() && !Actor->IsHiddenEd() )
		{
			SetActorVisibility(Actor, false);
		}
	}

	// Iterate through all of the BSP models and hide the ones which are not selected and are not already hidden
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )		
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;

				// Only modify surfaces that aren't selected and aren't already hidden
				if ( !( CurSurface.PolyFlags & PF_Selected ) && !CurSurface.IsHiddenEd() )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.bHiddenEdTemporary = true;
				}
			}
		}
	}

	// Hide unselected non-actor elements via TEDS
	UE::EditorActor::Private::SetUnselectedNonActorVisibility(InWorld, false);

	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::Internal_ToggleSelectedVisibility(UWorld* InWorld, bool bApplyToChildHierarchy)
{
	// Assemble a list of actors to operate on.
	TArray<AActor*> ActorsToConsider;
	bool bTargetVisibility = true;

	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if (!FActorEditorUtils::IsABuilderBrush(Actor))
		{
			if ( bApplyToChildHierarchy )
			{
				GetChildHierarchyOfSelectedActor(ActorsToConsider, Actor, true);
			}
			else
			{
				ActorsToConsider.Add( Actor );
			}
			bTargetVisibility = bTargetVisibility && Actor->IsHiddenEd();
		}
	}
	
	// Check BSP to confirm their visibility
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( ( CurSurface.PolyFlags & PF_Selected ) )
				{
					bTargetVisibility = bTargetVisibility && CurSurface.IsHiddenEd();
				}
			}
		}
	}

	// Check non-actor element visibility to include in the toggle decision
	if (bTargetVisibility)
	{
		bTargetVisibility = !UE::EditorActor::Private::HaveAnySelectedVisibleNonActor(InWorld, bApplyToChildHierarchy);
	}
	
	// Apply the target visibility to the target actors
	if ( ActorsToConsider.Num() > 0 )
	{
		for( int32 ActorIndex = 0 ; ActorIndex < ActorsToConsider.Num() ; ++ActorIndex )
		{
			AActor* Actor = ActorsToConsider[ ActorIndex ];
			if (Actor->IsHiddenEd() != !bTargetVisibility)
			{
				SetActorVisibility(Actor, bTargetVisibility);
			}
		}
	}
	
	// Apply the target visibility to BSP
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( ( CurSurface.PolyFlags & PF_Selected ) )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.bHiddenEdTemporary = !bTargetVisibility;
				}
			}
		}
	}

	// Apply the target visibility to non-actor elements
	UE::EditorActor::Private::SetSelectedNonActorVisibility(InWorld, bTargetVisibility, bApplyToChildHierarchy);
	
	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::edactToggleSelectedVisibility(UWorld* InWorld)
{
	Internal_ToggleSelectedVisibility(InWorld, false);
}

void UUnrealEdEngine::edactToggleSelectedHierarchyVisibility(UWorld* InWorld)
{
	Internal_ToggleSelectedVisibility(InWorld, true);
}

void UUnrealEdEngine::edactUnHideAll( UWorld* InWorld )
{
	// Iterate through all of the actors and unhide them
	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		if( !FActorEditorUtils::IsABuilderBrush(Actor) && Actor->IsTemporarilyHiddenInEditor() )
		{
			SetActorVisibility(Actor, true);
		}
	}

	// Iterate through all of the BSP models and unhide them if they are already hidden
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( CurSurface.bHiddenEdTemporary )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.bHiddenEdTemporary = false;
				}
			}
		}
	}

	// Unhide all non-actor elements via TEDS
	UE::EditorActor::Private::SetAllNonActorVisiblity(InWorld, true);

	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::Internal_HideSelectedStartup( UWorld* InWorld, bool bApplyToChildHierarchy )
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	TArray<AActor*> ActorsToHideAtStartup;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if ( bApplyToChildHierarchy )
		{
			GetChildHierarchyOfSelectedActor(ActorsToHideAtStartup, Actor, true);
		}
		else if ( !FActorEditorUtils::IsABuilderBrush(Actor) )
		{
			ActorsToHideAtStartup.Add( Actor );
		}
	}

	for (AActor* Actor : ActorsToHideAtStartup)
	{
		// Set the actor to hide at editor startup, if it's not already set that way
		if ( !Actor->IsHiddenEdAtStartup() )
		{
			Actor->Modify();
			Actor->bHiddenEd = true;
			LevelDirtyCallback.Request();
		}
	}

	if ( InWorld )
	{
		// Iterate through all of the selected BSP surfaces
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				
				// Set the BSP surface to hide at editor startup, if it's not already set that way
				const bool bSelected = CurSurface.Actor->IsActorOrSelectionParentSelected() || (CurSurface.PolyFlags & PF_Selected);
				if (bSelected && !CurSurface.IsHiddenEdAtStartup())
				{
					CurLevelModel.Modify(false);
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.PolyFlags |= PF_HiddenEd;
					LevelDirtyCallback.Request();
				}
			}
		}
	}
	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::edactHideSelectedStartup( UWorld* InWorld )
{
	Internal_HideSelectedStartup(InWorld, false);
}

void UUnrealEdEngine::edactHideSelectedHierarchyStartup( UWorld* InWorld )
{
	Internal_HideSelectedStartup(InWorld, true);
}

void UUnrealEdEngine::edactUnHideAllStartup( UWorld* InWorld )
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Iterate over all actors
	for ( FActorIterator It(InWorld); It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		// If the actor is set to be hidden at editor startup, change it so that it will be shown at startup
		if ( !FActorEditorUtils::IsABuilderBrush(Actor) && Actor->IsHiddenEdAtStartup() )
		{
			Actor->Modify();
			Actor->bHiddenEd = false;
			LevelDirtyCallback.Request();
		}
	}

	if ( InWorld )
	{
		// Iterate over all BSP surfaces
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;

				// If the BSP surface is set to be hidden at editor startup, change it so that it will be shown at startup
				if ( CurSurface.IsHiddenEdAtStartup() )
				{
					CurLevelModel.Modify(false);
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.PolyFlags &= ~PF_HiddenEd;
					LevelDirtyCallback.Request();
				}
			}
		}
	}
	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::Internal_UnHideSelectedStartup( UWorld* InWorld, bool bApplyToChildHierarchy )
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	TArray<AActor*> ActorsToShowAtStartup;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if( bApplyToChildHierarchy )
		{
			GetChildHierarchyOfSelectedActor(ActorsToShowAtStartup, Actor, true);
		}
		else if ( !FActorEditorUtils::IsABuilderBrush(Actor) )
		{
			ActorsToShowAtStartup.Add( Actor );
		}
	}

	for (AActor* Actor : ActorsToShowAtStartup)
	{
		// Set the actor to show at editor startup, if it's not already set that way
		if ( Actor->IsHiddenEdAtStartup() )
		{
			Actor->Modify();
			Actor->bHiddenEd = false;
			LevelDirtyCallback.Request();
		}
	}

	if ( InWorld )
	{
		// Iterate over all selected BSP surfaces
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;

				// Mark the selected BSP surface as showing at editor startup if it was currently set to be hidden
				const bool bSelected = CurSurface.Actor->IsActorOrSelectionParentSelected() || (CurSurface.PolyFlags & PF_Selected);
				if (bSelected && CurSurface.IsHiddenEdAtStartup())
				{
					CurLevelModel.Modify(false);
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.PolyFlags &= ~PF_HiddenEd;
					LevelDirtyCallback.Request();
				}
			}
		}
	}
	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::edactUnHideSelectedStartup( UWorld* InWorld )
{
	Internal_UnHideSelectedStartup(InWorld, false);
}

void UUnrealEdEngine::edactUnHideSelectedHierarchyStartup( UWorld* InWorld )
{
	Internal_UnHideSelectedStartup(InWorld, true);
}

void UUnrealEdEngine::Internal_UnhideSelected(UWorld* InWorld, bool bApplyToChildHierarchy)
{
	// Assemble a list of actors to show.
	TArray<AActor*> ActorsToShow;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if( bApplyToChildHierarchy )
		{
			GetChildHierarchyOfSelectedActor(ActorsToShow, Actor, true, true);
		}
		else if ( !FActorEditorUtils::IsABuilderBrush(Actor) && Actor->IsHiddenEd() )
		{
			ActorsToShow.Add( Actor );
		}
	}
	
	// Show the actors that were selected
	if ( ActorsToShow.Num() > 0 )
	{
		USelection* SelectedActors = GetSelectedActors();
		SelectedActors->Modify();

		for( int32 ActorIndex = 0 ; ActorIndex < ActorsToShow.Num() ; ++ActorIndex )
		{
			AActor* Actor = ActorsToShow[ ActorIndex ];
			
			SetActorVisibility(Actor, true);
		}
	}

	// Iterate through all of the BSP models and show any that were selected
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( ( CurSurface.PolyFlags & PF_Selected ) && !CurSurface.IsHiddenEd() )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.bHiddenEdTemporary = false;
				}
			}
		}
	}

	// Show selected non-actor elements via TEDS
	UE::EditorActor::Private::SetSelectedNonActorVisibility(InWorld, true, bApplyToChildHierarchy);

	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::edactUnhideSelected( UWorld* InWorld )
{
	Internal_UnhideSelected(InWorld, false);
}

void UUnrealEdEngine::edactUnhideSelectedHierarchy( UWorld* InWorld )
{
	Internal_UnhideSelected(InWorld, true);
}

void UUnrealEdEngine::CreateBSPVisibilityMap(UWorld* InWorld, TMap<AActor*, TArray<int32>>& OutBSPMap, bool& bOutAllVisible  )
{
	// Start out true, we do not know otherwise.
	bOutAllVisible = true;

	// Iterate through all of the BSP models and any that are visible to the list.
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;

				// If the surface is visible, we will want to add it to the map.
				if ( CurSurface.bHiddenEdTemporary == false )
				{
					// First check if we have already added our surface's brush actor to the map.
					TArray<int32>* BrushPolyList = OutBSPMap.Find(CurSurface.Actor);
					if(BrushPolyList)
					{
						// We found the brush actor on the list, so add our polygon ID to the list.
						BrushPolyList->Add(CurSurface.iBrushPoly);
					}
					else
					{
						// The brush actor has not been added to the map, add it.
						OutBSPMap.Add(CurSurface.Actor, TArray<int32>());

						// Grab the list out and add our brush poly to it.
						BrushPolyList = OutBSPMap.Find(CurSurface.Actor);
						BrushPolyList->Add(CurSurface.iBrushPoly);

					}
				}
				else
				{
					// We found one that is not visible, so they are not ALL visible. We will continue to map out geometry to come up with a complete Visibility map.
					bOutAllVisible = false;
				}
			}
		}
	}
}

void UUnrealEdEngine::MakeBSPMapVisible(const TMap<AActor*, TArray<int32>>& InBSPMap, UWorld* InWorld )
{
	// Iterate through all of the BSP models and show any that were selected
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;

				// Check if we can find the surface's actor in the map.
				const TArray<int32>* BrushPolyList = InBSPMap.Find(CurSurface.Actor);
				if(BrushPolyList)
				{
					// We have the list of brush polygons that are visible, check if the current one is on the list.
					if(BrushPolyList->FindByKey(CurSurface.iBrushPoly))
					{
						// Make the surface visible.
						CurSurface.bHiddenEdTemporary = false;
					}
					else
					{
						// The brush poly was not in the map, so it should be hidden.
						CurSurface.bHiddenEdTemporary = true;
					}
				}
				else
				{
					// There was no brush poly list, that means no polygon on this brush was visible, make this surface hidden.
					CurSurface.bHiddenEdTemporary = true;
				}
			}
		}
	}

}

AActor* UUnrealEdEngine::GetDesiredAttachmentState(TArray<AActor*>& OutNewChildren)
{
	// Get the selection set (first one will be the new base)
	AActor* NewBase = NULL;
	OutNewChildren.Empty();
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* SelectedActor = Cast<AActor>(*It);
		if(SelectedActor)
		{
			OutNewChildren.AddUnique(SelectedActor);
		}
	}

	// Last element of the array becomes new base
	if(OutNewChildren.Num() > 0)
	{
		NewBase = OutNewChildren.Pop();
	}

	return NewBase;
}

void UUnrealEdEngine::AttachSelectedActors()
{
	const FScopedTransaction Transaction( NSLOCTEXT("Editor", "UndoAction_PerformAttachment", "Attach actors") );

	// Get what we want attachment to be
	TArray<AActor*> NewChildren;
	AActor* NewBase = GetDesiredAttachmentState(NewChildren);
	if(NewBase && NewBase->GetRootComponent() && (NewChildren.Num() > 0))
	{
	
		// Do the actual base change
		for(int32 ChildIdx=0; ChildIdx<NewChildren.Num(); ChildIdx++)
		{
			AActor* Child = NewChildren[ChildIdx];
			if( Child )
			{
				ParentActors( NewBase, Child, NAME_None );
			}
		}

		RedrawLevelEditingViewports();
	}
}

void UUnrealEdEngine::edactSelectAll( UWorld* InWorld )
{
	// If there are a lot of actors to process, pop up a warning "are you sure?" box
	int32 NumActors = InWorld->GetActorCount();
	bool bShowProgress = false;
	if( NumActors >= EditorActorSelectionDefs::MaxActorsToSelectBeforeWarning )
	{
		bShowProgress = true;

		const FText ConfirmText = FText::Format( NSLOCTEXT("UnrealEd", "Warning_ManyActorsForSelect", "There are {0} actors in the world. Are you sure you want to select them all?" ), FText::AsNumber(NumActors) );

		FSuppressableWarningDialog::FSetupInfo Info( ConfirmText, NSLOCTEXT("UnrealEd", "Warning_ManyActors", "Warning: Many Actors" ), "Warning_ManyActors" );
		Info.ConfirmText = NSLOCTEXT("ModalDialogs", "SelectAllConfirm", "Select All");
		Info.CancelText = NSLOCTEXT("ModalDialogs", "SelectAllCancel", "Cancel");

		FSuppressableWarningDialog ManyActorsWarning( Info );
		if( ManyActorsWarning.ShowModal() == FSuppressableWarningDialog::Cancel )
		{
			return;
		}
	}

	if( bShowProgress )
	{
		GWarn->BeginSlowTask( LOCTEXT("BeginSelectAllActorsTaskStatusMessage", "Selecting All Actors"), true);
	}

	// Add all selected actors' layer name to the LayerArray.
	USelection* SelectedActors = GetSelectedActors();

	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		if( !Actor->IsSelected() && !Actor->IsHiddenEd() && Actor->IsSelectable())
		{
			SelectActor( Actor, 1, 0 );
		}
	}

	// Iterate through all of the BSP models and select them if they are not hidden
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( !CurSurface.IsHiddenEd() )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.PolyFlags |= PF_Selected;
				}
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();

	NoteSelectionChange();

	if( bShowProgress )
	{
		GWarn->EndSlowTask( );
	}
}


void UUnrealEdEngine::edactSelectInvert( UWorld* InWorld )
{
	// If there are a lot of actors to process, pop up a warning "are you sure?" box
	int32 NumActors = InWorld->GetActorCount();
	bool bShowProgress = false;
	if( NumActors >= EditorActorSelectionDefs::MaxActorsToSelectBeforeWarning )
	{
		bShowProgress = true;
		const FText ConfirmText = FText::Format( NSLOCTEXT("UnrealEd", "Warning_ManyActorsForInvertSelect", "There are {0} actors in the world. Are you sure you want to invert selection on them all?" ), FText::AsNumber(NumActors) );

		FSuppressableWarningDialog::FSetupInfo Info ( ConfirmText, NSLOCTEXT("UnrealEd", "Warning_ManyActors", "Warning: Many Actors" ), "Warning_ManyActors" );
		Info.ConfirmText = NSLOCTEXT("ModalDialogs", "InvertSelectionConfirm", "Invert Selection");
		Info.CancelText = NSLOCTEXT("ModalDialogs", "InvertSelectionCancel", "Cancel");

		FSuppressableWarningDialog ManyActorsWarning( Info );
		if( ManyActorsWarning.ShowModal() == FSuppressableWarningDialog::Cancel )
		{
			return;
		}
	}

	if( bShowProgress )
	{
		GWarn->BeginSlowTask( LOCTEXT("BeginInvertingActorSelectionTaskMessage", "Inverting Selected Actors"), true);
	}

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	// Iterate through all of the actors and select them if they are not currently selected (and not hidden)
	// or deselect them if they are currently selected

	// Turn off Grouping during this process to avoid double toggling of selected actors via group selection
	const bool bGroupingActiveSaved = UActorGroupingUtils::IsGroupingActive();
	UActorGroupingUtils::SetGroupingActive(false);
	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		if( !FActorEditorUtils::IsABuilderBrush(Actor) && !Actor->IsHiddenEd() )
		{
			SelectActor( Actor, !Actor->IsSelected(), false );
		}
	}
	// Restore bGroupingActive to its original value
	UActorGroupingUtils::SetGroupingActive(bGroupingActiveSaved);

	// Iterate through all of the BSP models and select them if they are not currently selected (and not hidden)
	// or deselect them if they are currently selected
	if ( InWorld )
	{
		for ( TArray<ULevel*>::TConstIterator LevelIterator = InWorld->GetLevels().CreateConstIterator(); LevelIterator; ++LevelIterator )
		{
			UModel& CurLevelModel = *( ( *LevelIterator )->Model );
			for ( TArray<FBspSurf>::TIterator SurfaceIterator( CurLevelModel.Surfs ); SurfaceIterator; ++SurfaceIterator )
			{
				FBspSurf& CurSurface = *SurfaceIterator;
				if ( !CurSurface.IsHiddenEd() )
				{
					CurLevelModel.ModifySurf( SurfaceIterator.GetIndex(), false );
					CurSurface.PolyFlags ^= PF_Selected;
				}
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();

	NoteSelectionChange();

	if( bShowProgress )
	{
		GWarn->EndSlowTask( );
	}
}

static void GetAttachedActors( AActor* Actor, bool bRecurseChildren, TSet< AActor* >& OutActors )
{
	TArray< AActor* > ChildrenActors;
	Actor->GetAttachedActors( ChildrenActors );
	for ( AActor* ChildActor : ChildrenActors )
	{
		OutActors.Add( ChildActor );

		if ( bRecurseChildren )
		{
			GetAttachedActors( ChildActor, bRecurseChildren, OutActors );
		}
	}
}

void UUnrealEdEngine::edactSelectAllChildren( bool bRecurseChildren )
{
	USelection* CurrentSelection = GetSelectedActors();

	TArray< AActor* > SelectedActors;
	CurrentSelection->GetSelectedObjects< AActor >( SelectedActors );

	CurrentSelection->BeginBatchSelectOperation();
	CurrentSelection->Modify();

	// Turn off Grouping during this process to avoid double toggling of selected actors via group selection
	const bool bGroupingActiveSaved = UActorGroupingUtils::IsGroupingActive();
	UActorGroupingUtils::SetGroupingActive( false );

	// Iterate through all the selected actors and select their children if they are not currently selected
	TSet< AActor* > ActorsToSelect;
	for ( AActor* Actor : SelectedActors )
	{
		// Don't recurse through the same actor twice
		if ( !bRecurseChildren || !ActorsToSelect.Contains( Actor ) )
		{
			GetAttachedActors( Actor, bRecurseChildren, ActorsToSelect );
		}
	}

	for ( AActor* Actor : ActorsToSelect )
	{
		if ( !FActorEditorUtils::IsABuilderBrush( Actor ) && !Actor->IsSelected() )
		{
			// Select actor even if hidden
			SelectActor( Actor, true, false, true );
		}
	}

	// Restore bGroupingActive to its original value
	UActorGroupingUtils::SetGroupingActive( bGroupingActiveSaved );

	CurrentSelection->EndBatchSelectOperation();

	NoteSelectionChange();
}

void UUnrealEdEngine::edactSelectOfClass( UWorld* InWorld, UClass* Class )
{
	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	for( TActorIterator<AActor> It(InWorld, Class); It; ++It )
	{
		AActor* Actor = *It;
		if( Actor->GetClass()==Class && !Actor->IsSelected() && !Actor->IsHiddenEd() )
		{
			// Selection by class not permitted for actors belonging to prefabs.
			// Selection by class not permitted for builder brushes.
			if ( !FActorEditorUtils::IsABuilderBrush(Actor) )
			{
				SelectActor( Actor, 1, 0 );
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectOfClassAndArchetype( UWorld* InWorld, const TSubclassOf<AActor> InClass, const UObject* InArchetype )
{
	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	// Select all actors with of the provided class and archetype, assuming they aren't already selected, 
	// aren't hidden in the editor, aren't a member of a prefab, and aren't builder brushes
	for( TActorIterator<AActor> ActorIter(InWorld, InClass); ActorIter; ++ActorIter )
	{
		AActor* CurActor = *ActorIter;
		if ( CurActor->GetClass() == InClass && CurActor->GetArchetype() == InArchetype && !CurActor->IsSelected() 
			&& !CurActor->IsHiddenEd() && !FActorEditorUtils::IsABuilderBrush(CurActor) )
		{
			SelectActor( CurActor, true, false );
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectSubclassOf( UWorld* InWorld, UClass* Class )
{
	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	for( TActorIterator<AActor> It(InWorld, Class); It; ++It )
	{
		AActor* Actor = *It;
		if( !Actor->IsSelected() && !Actor->IsHiddenEd() )
		{
			// Selection by class not permitted for actors belonging to prefabs.
			// Selection by class not permitted for builder brushes.
			if ( !FActorEditorUtils::IsABuilderBrush(Actor) )
			{
				SelectActor( Actor, 1, 0 );
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectDeleted( UWorld* InWorld )
{
	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();

	SelectedActors->Modify();

	bool bSelectionChanged = false;
	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		if( !Actor->IsSelected() && !Actor->IsHiddenEd() )
		{
			if( !IsValid(Actor) )
			{
				bSelectionChanged = true;
				SelectActor( Actor, 1, 0 );
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();

	if ( bSelectionChanged )
	{
		NoteSelectionChange();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
//	Select matching static meshes.
//
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace {

/**
 * Information about an actor and its static mesh.
 */
class FStaticMeshActor
{
public:
	/** Non-NULL if the actor is a static mesh. */
	AStaticMeshActor* StaticMeshActor;
	/** Non-NULL if the actor has a static mesh. */
	UStaticMesh* StaticMesh;

	FStaticMeshActor()
		: StaticMeshActor(NULL)
		, StaticMesh(NULL)
	{}

	bool IsStaticMeshActor() const
	{
		return StaticMeshActor != NULL;
	}

	bool HasStaticMesh() const
	{
		return StaticMesh != NULL;
	}

	/**
	 * Extracts the static mesh information from the specified actor.
	 */
	static bool GetStaticMeshInfoFromActor(AActor* Actor, FStaticMeshActor& OutStaticMeshActor)
	{
		OutStaticMeshActor.StaticMeshActor = Cast<AStaticMeshActor>( Actor );

		if( OutStaticMeshActor.IsStaticMeshActor() )
		{
			if ( OutStaticMeshActor.StaticMeshActor->GetStaticMeshComponent() )
			{
				OutStaticMeshActor.StaticMesh = OutStaticMeshActor.StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh();
			}
		}
		return OutStaticMeshActor.HasStaticMesh();
	}
};

} // namespace


void UUnrealEdEngine::edactSelectMatchingStaticMesh( bool bAllClasses )
{
	TArray<FStaticMeshActor> StaticMeshActors;

	TArray<UWorld*> SelectedWorlds;
	// Make a list of selected actors with static meshes.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		FStaticMeshActor ActorInfo;
		if ( FStaticMeshActor::GetStaticMeshInfoFromActor( Actor, ActorInfo ) )
		{
			if ( ActorInfo.IsStaticMeshActor() )
			{
				StaticMeshActors.Add( ActorInfo );
				SelectedWorlds.AddUnique( Actor->GetWorld() );	
			}				
		}
	}
	if( SelectedWorlds.Num() == 0 )
	{
		UE_LOGF(LogEditorActor, Log, "No worlds found in edactSelectMatchingStaticMesh" );
		return;
	}
	// Make sure we have only 1 valid world 
	check(SelectedWorlds.Num() == 1);
	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();

	// Loop through all non-hidden actors in visible levels, selecting those that have one of the
	// static meshes in the list.
	for( FActorIterator It(SelectedWorlds[0]); It; ++It )
	{
		AActor* Actor = *It;
		if ( !Actor->IsHiddenEd() )
		{
			FStaticMeshActor ActorInfo;
			if ( FStaticMeshActor::GetStaticMeshInfoFromActor( Actor, ActorInfo ) )
			{
				bool bSelectActor = false;
				if ( bAllClasses || ActorInfo.IsStaticMeshActor() )
				{
					for ( int32 i = 0 ; i < StaticMeshActors.Num() ; ++i )
					{
						if ( StaticMeshActors[i].StaticMesh == ActorInfo.StaticMesh )
						{
							bSelectActor = true;
							break;
						}
					}
				}

				if ( bSelectActor )
				{
					SelectActor( Actor, true, false );
				}
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectMatchingSkeletalMesh(bool bAllClasses)
{
	TArray<USkeletalMesh*> SelectedMeshes;
	bool bSelectSkelMeshActors = false;
	bool bSelectPawns = false;

	TArray<UWorld*> SelectedWorlds;
	// Make a list of skeletal meshes of selected actors, and note what classes we have selected.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		// Look for SkelMeshActor
		ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor);
		if(SkelMeshActor && SkelMeshActor->GetSkeletalMeshComponent())
		{
			bSelectSkelMeshActors = true;
			SelectedMeshes.AddUnique(SkelMeshActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset());
			SelectedWorlds.AddUnique(Actor->GetWorld());			
		}

		// Look for Pawn
		APawn* Pawn = Cast<APawn>(Actor);
		if (Pawn)
		{
			USkeletalMeshComponent* PawnSkeletalMesh = Pawn->FindComponentByClass<USkeletalMeshComponent>();
			if (PawnSkeletalMesh)
			{
				bSelectPawns = true;
				SelectedMeshes.AddUnique(PawnSkeletalMesh->GetSkeletalMeshAsset());
				SelectedWorlds.AddUnique(Actor->GetWorld());
			}
		}
	}
	if( SelectedWorlds.Num() == 0 )
	{
		UE_LOGF(LogEditorActor, Log, "No worlds found in edactSelectMatchingSkeletalMesh" );
		return;
	}
	// Make sure we have only 1 valid world 
	check( SelectedWorlds.Num() == 1 );
	// If desired, select all class types
	if(bAllClasses)
	{
		bSelectSkelMeshActors = true;
		bSelectPawns = true;
	}

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();

	// Loop through all non-hidden actors in visible levels, selecting those that have one of the skeletal meshes in the list.
	for( FActorIterator It(SelectedWorlds[0]); It; ++It )
	{
		AActor* Actor = *It;
		if ( !Actor->IsHiddenEd() )
		{
			bool bSelectActor = false;

			if(bSelectSkelMeshActors)
			{
				ASkeletalMeshActor* SkelMeshActor = Cast<ASkeletalMeshActor>(Actor);
				if( SkelMeshActor && 
					SkelMeshActor->GetSkeletalMeshComponent() && 
					SelectedMeshes.Contains(SkelMeshActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset()) )
				{
					bSelectActor = true;
				}
			}

			if(bSelectPawns)
			{
				APawn* Pawn = Cast<APawn>(Actor);
				if (Pawn)
				{
					USkeletalMeshComponent* PawnSkeletalMesh = Pawn->FindComponentByClass<USkeletalMeshComponent>();
					if (PawnSkeletalMesh && SelectedMeshes.Contains(PawnSkeletalMesh->GetSkeletalMeshAsset()) )
					{
						bSelectActor = true;
					}
				}
			}

			if ( bSelectActor )
			{
				SelectActor( Actor, true, false );
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectMatchingMaterial()
{
	// Set for fast lookup of used materials.
	TSet<UMaterialInterface*> MaterialsInSelection;

	TArray<UWorld*> SelectedWorlds;
	// For each selected actor, find all the materials used by this actor.
	for ( FSelectionIterator ActorItr( GetSelectedActorIterator() ) ; ActorItr ; ++ActorItr )
	{
		AActor* CurrentActor = Cast<AActor>( *ActorItr );

		if( CurrentActor )
		{
			// Find the materials by iterating over every primitive component.
			for (UActorComponent* Component : CurrentActor->GetComponents())
			{
				if (UPrimitiveComponent* CurrentComponent = Cast<UPrimitiveComponent>(Component))
				{
					TArray<UMaterialInterface*> UsedMaterials;
					CurrentComponent->GetUsedMaterials(UsedMaterials);
					MaterialsInSelection.Append(UsedMaterials);
					SelectedWorlds.AddUnique(CurrentActor->GetWorld());
				}
			}
		}
	}

	if( SelectedWorlds.Num() == 0 )
	{
		UE_LOGF(LogEditorActor, Log, "No worlds found in edactSelectMatchingMaterial" );
		return;
	}
	// Make sure we have only 1 valid world 
	check( SelectedWorlds.Num() == 1 );

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();	

	// Now go over every actor and see if any of the actors are using any of the materials that 
	// we found above.
	for( FActorIterator ActorIt(SelectedWorlds[0]); ActorIt; ++ActorIt )
	{
		AActor* Actor = *ActorIt;

		// Do not bother checking hidden actors
		if( !Actor->IsHiddenEd() )
		{
			TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
			Actor->GetComponents(PrimitiveComponents);

			const int32 NumComponents = PrimitiveComponents.Num();
			for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex )
			{
				UPrimitiveComponent* CurrentComponent = PrimitiveComponents[ComponentIndex];

				TArray<UMaterialInterface*> UsedMaterials;
				CurrentComponent->GetUsedMaterials( UsedMaterials );
				const int32 NumMaterials = UsedMaterials.Num();
				// Iterate over every material we found so far and see if its in the list of materials used by selected actors.
				for( int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex )
				{
					UMaterialInterface* Material = UsedMaterials[ MatIndex ];
					// Is this material used by currently selected actors?
					if( MaterialsInSelection.Find( Material ) )
					{
						SelectActor( Actor, true, false );
						// We dont need to continue searching as this actor has already been selected
						MatIndex = NumMaterials;
						ComponentIndex = NumComponents;
					}
				}
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactSelectMatchingEmitter()
{
	TArray<UParticleSystem*> SelectedParticleSystemTemplates;

	TArray<UWorld*> SelectedWorlds;
	// Check all of the currently selected actors to find the relevant particle system templates to use to match
	for ( FSelectionIterator SelectedIterator( GetSelectedActorIterator() ) ; SelectedIterator ; ++SelectedIterator )
	{
		AActor* Actor = static_cast<AActor*>( *SelectedIterator );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		AEmitter* Emitter = Cast<AEmitter>( Actor );
		
		if ( Emitter && Emitter->GetParticleSystemComponent() && Emitter->GetParticleSystemComponent()->Template )
		{
			SelectedParticleSystemTemplates.AddUnique( Emitter->GetParticleSystemComponent()->Template );
			SelectedWorlds.AddUnique( Actor->GetWorld() );
		}
	}

	if( SelectedWorlds.Num() == 0 )
	{
		UE_LOGF(LogEditorActor, Log, "No worlds found in edactSelectMatchingEmitter" );
		return;
	}
	// Make sure we have only 1 valid world 
	check( SelectedWorlds.Num() == 1 );

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();
	// Iterate over all of the non-hidden actors, selecting those who have a particle system template that matches one from the previously-found list
	for( TActorIterator<AEmitter> ActorIterator(SelectedWorlds[0]); ActorIterator; ++ActorIterator )
	{
		AEmitter* ActorAsEmitter = *ActorIterator;
		if ( !ActorAsEmitter->IsHiddenEd() )
		{
			if ( ActorAsEmitter->GetParticleSystemComponent() && SelectedParticleSystemTemplates.Contains( ActorAsEmitter->GetParticleSystemComponent()->Template ) )
			{
				SelectActor( ActorAsEmitter, true, false );
			}
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}



void UUnrealEdEngine::edactSelectRelevantLights( UWorld* InWorld )
{
	TArray<ALight*> RelevantLightList;
	// Make a list of selected actors with static meshes.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		if (Actor->GetLevel()->IsCurrentLevel() )
		{
			// Gather static lighting info from each of the actor's components.
			for (UActorComponent* Component : Actor->GetComponents())
			{
				UPrimitiveComponent* Primitive = Cast<UPrimitiveComponent>(Component);
				if (Primitive && Primitive->IsRegistered())
				{
					TArray<const ULightComponent*> RelevantLightComponents;
					InWorld->Scene->GetRelevantLights(Primitive, &RelevantLightComponents);

					for (int32 LightComponentIndex = 0; LightComponentIndex < RelevantLightComponents.Num(); LightComponentIndex++)
					{
						const ULightComponent* LightComponent = RelevantLightComponents[LightComponentIndex];
						ALight* LightOwner = Cast<ALight>(LightComponent->GetOwner());
						if (LightOwner)
						{
							RelevantLightList.AddUnique(LightOwner);
						}
					}
				}
			}
		}
	}

	USelection* SelectedActors = GetSelectedActors();
	SelectedActors->BeginBatchSelectOperation();
	SelectedActors->Modify();

	SelectNone( false, true );

	UE_LOGF(LogEditorActor, Log, "Found %d relevant lights!", RelevantLightList.Num());
	for (int32 LightIdx = 0; LightIdx < RelevantLightList.Num(); LightIdx++)
	{
		ALight* Light = RelevantLightList[LightIdx];
		if (Light)
		{
			SelectActor(Light, true, false);
			UE_LOGF(LogEditorActor, Log, "\t%ls", *(Light->GetPathName()));
		}
	}

	SelectedActors->EndBatchSelectOperation();
	NoteSelectionChange();
}


void UUnrealEdEngine::edactAlignOrigin()
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Apply transformations to all selected brushes.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ABrush* Brush = Cast< ABrush >( Actor );
		if ( Brush )
		{
			LevelDirtyCallback.Request();

			Brush->PreEditChange(NULL);
			Brush->Modify(false);

			//Snap the location of the brush to the grid
			FVector BrushLocation = Brush->GetActorLocation();
			BrushLocation.X = FMath::RoundToFloat( BrushLocation.X  / GetGridSize() ) * GetGridSize();
			BrushLocation.Y = FMath::RoundToFloat( BrushLocation.Y  / GetGridSize() ) * GetGridSize();
			BrushLocation.Z = FMath::RoundToFloat( BrushLocation.Z  / GetGridSize() ) * GetGridSize();
			Brush->SetActorLocation(BrushLocation, false);

			//Update EditorMode locations to match the new brush location
			FEditorModeTools& Tools = GLevelEditorModeTools();
			Tools.SetPivotLocation( Brush->GetActorLocation(), true );

			Brush->Brush->BuildBound();
			Brush->PostEditChange();
		}
	}
}


void UUnrealEdEngine::edactAlignVertices()
{
	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;
	
	//Before aligning verts, align the origin with the grid
	edactAlignOrigin();

	// Apply transformations to all selected brushes.
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );
		ABrush* Brush = Cast< ABrush >( Actor );
		if ( Brush )
		{
			LevelDirtyCallback.Request();

			Brush->PreEditChange(NULL);
			Brush->Modify(false);
			FVector BrushLocation = Brush->GetActorLocation();
			const FTransform BrushTransform = Brush->GetRootComponent()->GetComponentTransform();

			// Snap each vertex in the brush to an integer grid.
			UPolys* Polys = Brush->Brush->Polys;
			for( int32 PolyIdx=0; PolyIdx<Polys->Element.Num(); PolyIdx++ )
			{
				FPoly* Poly = &Polys->Element[PolyIdx];
				for( int32 VertIdx=0; VertIdx<Poly->Vertices.Num(); VertIdx++ )
				{
					const float GridSize = GetGridSize();

					// Snap each vertex to the nearest grid.
					const FVector3f Vertex = Poly->Vertices[VertIdx];
					const FVector VertexWorld = BrushTransform.TransformPosition((FVector)Vertex);
					const FVector3f VertexSnapped(FMath::RoundToFloat(VertexWorld.X / GridSize) * GridSize,
												  FMath::RoundToFloat(VertexWorld.Y / GridSize) * GridSize,
												  FMath::RoundToFloat(VertexWorld.Z / GridSize) * GridSize);
					const FVector VertexSnappedLocal = BrushTransform.InverseTransformPosition((FVector)VertexSnapped);

					Poly->Vertices[VertIdx] = (FVector3f)VertexSnappedLocal;
				}

				// If the snapping resulted in an off plane polygon, triangulate it to compensate.
				if( !Poly->IsCoplanar() || !Poly->IsConvex() )
				{

					FPoly BadPoly = *Poly;
					// Remove the bad poly
					Polys->Element.RemoveAt( PolyIdx );

					// Triangulate the bad poly
					TArray<FPoly> Triangles;
					if ( BadPoly.Triangulate( Brush, Triangles ) > 0 )
					{
						// Add all new triangles to the brush
						for( int32 TriIdx = 0 ; TriIdx < Triangles.Num() ; ++TriIdx )
						{
							Polys->Element.Add( Triangles[TriIdx] );
						}
					}
					
					PolyIdx = -1;
				}
				else
				{
					if( RecomputePoly( Brush, &Polys->Element[PolyIdx] ) == -2 )
					{
						PolyIdx = -1;
					}

					if (UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>())
					{
						BrushSubsystem->UpdateGeometryFromBrush(Brush);
					}
				}
			}

			Brush->Brush->BuildBound();

			Brush->PostEditChange();
		}
	}
}

#undef LOCTEXT_NAMESPACE
