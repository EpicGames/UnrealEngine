// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorActorFolders.h"

#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/LazySingleton.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "EngineGlobals.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "EditorFolderUtils.h"
#include "ScopedTransaction.h"
#include "UObject/ObjectSaveContext.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "ActorFolder.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorActorFolders)

#define LOCTEXT_NAMESPACE "FActorFolders"

// Static member definitions

FOnActorFolderCreated	FActorFolders::OnFolderCreated;
FOnActorFolderMoved		FActorFolders::OnFolderMoved;
FOnActorFolderDeleted	FActorFolders::OnFolderDeleted;

FActorFolders::FActorFolders()
{
	check(GEngine);
	GEngine->OnLevelActorFolderChanged().AddRaw(this, &FActorFolders::OnActorFolderChanged);
	GEngine->OnActorFolderAdded().AddRaw(this, &FActorFolders::OnActorFolderAdded);
	
	FEditorDelegates::PostSaveWorldWithContext.AddRaw(this, &FActorFolders::OnWorldSaved);
	FEditorDelegates::PostSaveExternalActors.AddRaw(this, &FActorFolders::SaveWorldFoldersState);

	check(GEditor);
	UActorEditorContextSubsystem::Get()->RegisterClient(this);
	bAnyLevelsChanged = false;
}

FActorFolders::~FActorFolders()
{
	if (GEngine)
	{
		GEngine->OnLevelActorFolderChanged().RemoveAll(this);
		GEngine->OnActorFolderAdded().RemoveAll(this);
	}

	FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::PostSaveExternalActors.RemoveAll(this);

	if (GEditor)
	{
		if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
		{
			ActorEditorContextSubsystem->UnregisterClient(this);
		}
	}
}

void FActorFolders::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Add references for all our UObjects so they don't get collected
	Collector.AddReferencedObjects(WorldFolders);
}

FActorFolders& FActorFolders::Get()
{
	return TLazySingleton<FActorFolders>::Get();
}

void FActorFolders::Housekeeping()
{
	for (auto It = WorldFolders.CreateIterator(); It; ++It)
	{
		if (!It.Key().Get())
		{
			It.Value()->OnRemoved();
			It.RemoveCurrent();
		}
	}
}

void FActorFolders::BroadcastOnActorFolderCreated(UWorld& InWorld, const FFolder& InFolder)
{
	OnFolderCreated.Broadcast(InWorld, InFolder);
	BroadcastOnActorEditorContextClientChanged(InWorld);
}

void FActorFolders::BroadcastOnActorFolderDeleted(UWorld& InWorld, const FFolder& InFolder)
{
	OnFolderDeleted.Broadcast(InWorld, InFolder);
	BroadcastOnActorEditorContextClientChanged(InWorld);
}

void FActorFolders::BroadcastOnActorFolderMoved(UWorld& InWorld, const FFolder& InSrcFolder, const FFolder& InDstFolder)
{
	OnFolderMoved.Broadcast(InWorld, InSrcFolder, InDstFolder);
	BroadcastOnActorEditorContextClientChanged(InWorld);
}

static bool IsRunningGameOrPIE()
{
	return (GIsEditor && GEditor && GEditor->GetPIEWorldContext()) || IsRunningGame();
}

void FActorFolders::OnWorldSaved(UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	SaveWorldFoldersState(World);
}

void FActorFolders::SaveWorldFoldersState(UWorld* World)
{
	if (auto* Folders = WorldFolders.Find(World))
	{
		(*Folders)->SaveState();
	}
}

bool FActorFolders::IsInitializedForWorld(UWorld& InWorld) const
{
	return !!WorldFolders.Find(&InWorld);
}

void FActorFolders::InitializeForWorld(UWorld& InWorld)
{
	GetOrCreateWorldFolders(InWorld);
}

UWorldFolders& FActorFolders::GetOrCreateWorldFolders(UWorld& InWorld)
{
	if (auto* Folders = WorldFolders.Find(&InWorld))
	{
		return **Folders;
	}

	return CreateWorldFolders(InWorld);
}

void FActorFolders::OnActorFolderChanged(const AActor* InActor, FName OldPath)
{
	check(InActor && InActor->GetWorld());

	FScopedTransaction Transaction(LOCTEXT("UndoAction_FolderChanged", "Actor Folder Changed"));

	UWorld* World = InActor->GetWorld();
	const FFolder NewPath = InActor->GetFolder();

	if (AddFolderToWorld(*World, NewPath))
	{
		BroadcastOnActorFolderCreated(*World, NewPath);
	}
}

FActorFolderProps* FActorFolders::GetFolderProperties(UWorld& InWorld, const FFolder& InFolder)
{
	return GetOrCreateWorldFolders(InWorld).GetFolderProperties(InFolder);
}

UWorldFolders& FActorFolders::CreateWorldFolders(UWorld& InWorld)
{
	// Clean up any stale worlds
	Housekeeping();

	// We intentionally don't pass RF_Transactional to ConstructObject so that we don't record the creation of the object into the undo buffer
	// (to stop it getting deleted on undo as we manage its lifetime), but we still want it to be RF_Transactional so we can record any changes later
	UWorldFolders* Folders = NewObject<UWorldFolders>(GetTransientPackage(), NAME_None, RF_NoFlags);
	WorldFolders.Add(&InWorld, Folders);
	Folders->Initialize(&InWorld);
	return *Folders;
}

FFolder FActorFolders::GetDefaultFolderForSelection(UWorld& InWorld, TArray<FFolder>* InSelectedFolders)
{
	// Find a common parent folder, or put it at the root
	TOptional<FFolder> CommonFolder;

	auto MergeFolders = [&CommonFolder](FFolder& Folder)
	{
		if (!CommonFolder.IsSet())
		{
			CommonFolder = Folder;
		}
		else if (CommonFolder.GetValue().GetRootObject() != Folder.GetRootObject())
		{
			CommonFolder.Reset();
			return false;
		}
		else if (CommonFolder.GetValue().GetPath() != Folder.GetPath())
		{
			// Empty path and continue iterating as we need to continue validating RootObjects
			CommonFolder = FFolder(CommonFolder->GetRootObject());
		}
		return true;
	};

	bool bMergeStopped = false;
	for( FSelectionIterator SelectionIt( *GEditor->GetSelectedActors() ); SelectionIt; ++SelectionIt )
	{
		AActor* Actor = CastChecked<AActor>(*SelectionIt);

		FFolder Folder = Actor->GetFolder();
		// Special case for Level Instance, make root as level instance if editing
		if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
		{
			if (LevelInstance->IsEditing())
			{
				Folder = FFolder(FFolder::FRootObject(Actor), FFolder::GetEmptyPath());
			}
		}
		if (!MergeFolders(Folder))
		{
			bMergeStopped = true;
			break;
		}
	}
	if (!bMergeStopped && InSelectedFolders)
	{
		for (FFolder& Folder : *InSelectedFolders)
		{
			if (!MergeFolders(Folder))
			{
				break;
			}
		}
	}

	return GetDefaultFolderName(InWorld, CommonFolder.Get(FFolder::GetInvalidFolder()));
}

FFolder FActorFolders::GetFolderName(UWorld& InWorld, const FFolder& InParentFolder, const FName& InLeafName)
{
	FFolder ParentFolder = FFolder::IsRootObjectValid(InParentFolder.GetRootObject()) ? InParentFolder : FFolder(GetWorldFolderRootObject(InWorld), InParentFolder.GetPath());

	// This is potentially very slow but necessary to find a unique name
	const UWorldFolders& Folders = GetOrCreateWorldFolders(InWorld);
	const FFolder::FRootObject& RootObject = ParentFolder.GetRootObject();
	const FString LeafNameString = InLeafName.ToString();

	// Find the last non-numeric character
	int32 LastDigit = LeafNameString.FindLastCharByPredicate([](TCHAR Ch) { return !FChar::IsDigit(Ch); });
	uint32 SuffixLen = (LeafNameString.Len() - LastDigit) - 1;

	if (LastDigit == INDEX_NONE)
	{
		// Name is entirely numeric, eg. "123", so no suffix exists
		SuffixLen = 0;
	}

	// Trim any numeric suffix
	uint32 Suffix = 1;
	FString LeafNameRoot;
	if (SuffixLen > 0)
	{
		LeafNameRoot = LeafNameString.LeftChop(SuffixLen);
		FString LeafSuffix = LeafNameString.RightChop(LeafNameString.Len() - SuffixLen);
		Suffix = LeafSuffix.IsNumeric() ? FCString::Atoi(*LeafSuffix) : 1;
	}
	else
	{
		LeafNameRoot = LeafNameString;
	}

	// Create a valid base name for this folder
	FNumberFormattingOptions NumberFormat;
	NumberFormat.SetUseGrouping(false);
	NumberFormat.SetMinimumIntegralDigits(SuffixLen);

	FText LeafName = FText::Format(LOCTEXT("FolderNamePattern", "{0}{1}"), FText::FromString(LeafNameRoot), SuffixLen > 0 ? FText::AsNumber(Suffix++, &NumberFormat) : FText::GetEmpty());

	FString ParentFolderPath = ParentFolder.IsNone() ? TEXT("") : ParentFolder.ToString();
	if (!ParentFolderPath.IsEmpty())
	{
		ParentFolderPath += "/";
	}

	FName FolderName(*(ParentFolderPath + LeafName.ToString()));
	while (Folders.ContainsFolder(FFolder(RootObject, FolderName)))
	{
		LeafName = FText::Format(LOCTEXT("FolderNamePattern", "{0}{1}"), FText::FromString(LeafNameRoot), FText::AsNumber(Suffix++, &NumberFormat));
		FolderName = FName(*(ParentFolderPath + LeafName.ToString()));
		if (Suffix == 0)
		{
			// We've wrapped around a 32bit unsigned int - something must be seriously wrong!
			FolderName = NAME_None;
			break;
		}
	}

	return FFolder(RootObject, FolderName);
}

FFolder FActorFolders::GetDefaultFolderName(UWorld& InWorld, const FFolder& InParentFolder)
{
	FFolder ParentFolder = FFolder::IsRootObjectValid(InParentFolder.GetRootObject()) ? InParentFolder : FFolder(GetWorldFolderRootObject(InWorld), InParentFolder.GetPath());

	// This is potentially very slow but necessary to find a unique name
	const UWorldFolders& Folders = GetOrCreateWorldFolders(InWorld);
	const FFolder::FRootObject& RootObject = ParentFolder.GetRootObject();

	// Create a valid base name for this folder
	FNumberFormattingOptions NumberFormat;
	NumberFormat.SetUseGrouping(false);
	uint32 Suffix = 1;
	FText LeafName = FText::Format(LOCTEXT("DefaultFolderNamePattern", "NewFolder{0}"), FText::AsNumber(Suffix++, &NumberFormat));

	FString ParentFolderPath = ParentFolder.IsNone() ? TEXT("") : ParentFolder.ToString();
	if (!ParentFolderPath.IsEmpty())
	{
		ParentFolderPath += "/";
	}

	FName FolderName(*(ParentFolderPath + LeafName.ToString()));
	while (Folders.ContainsFolder(FFolder(RootObject, FolderName)))
	{
		LeafName = FText::Format(LOCTEXT("DefaultFolderNamePattern", "NewFolder{0}"), FText::AsNumber(Suffix++, &NumberFormat));
		FolderName = FName(*(ParentFolderPath + LeafName.ToString()));
		if (Suffix == 0)
		{
			// We've wrapped around a 32bit unsigned int - something must be seriously wrong!
			FolderName = NAME_None;
			break;
		}
	}

	return FFolder(RootObject, FolderName);
}

void FActorFolders::CreateFolderContainingSelection(UWorld& InWorld, const FFolder& InFolder)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));
	CreateFolder(InWorld, InFolder);
	SetSelectedFolderPath(InFolder);
}

void FActorFolders::SetSelectedFolderPath(const FFolder& InFolder) const
{
	// Move the currently selected actors into the new folder
	USelection* SelectedActors = GEditor->GetSelectedActors();

	const FFolder::FRootObject& RootObject = InFolder.GetRootObject();
	FName Path = InFolder.GetPath();

	for (FSelectionIterator SelectionIt(*SelectedActors); SelectionIt; ++SelectionIt)
	{
		AActor* Actor = CastChecked<AActor>(*SelectionIt);

		// If this actor is parented to another, which is also in the selection, skip it so that it moves when its parent does (otherwise it's orphaned)
		const AActor* ParentActor = Actor->GetAttachParentActor();
		if (ParentActor && SelectedActors->IsSelected(ParentActor))
		{
			continue;
		}

		// Currently not supported to change rootobject through this interface
		if (RootObject == Actor->GetFolderRootObject())
		{
			Actor->SetFolderPath_Recursively(Path);
		}
	}
}

bool FActorFolders::CreateFolder(UWorld& InWorld, const FFolder& InFolder)
{
	FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

	if (AddFolderToWorld(InWorld, InFolder))
	{
		BroadcastOnActorFolderCreated(InWorld, InFolder);
		return true;
	}
	return false;
}

void FActorFolders::OnActorFolderAdded(UActorFolder* InActorFolder)
{
	check(InActorFolder && InActorFolder->GetOuterULevel());
	ULevel* Level = InActorFolder->GetOuterULevel();
	check(Level->IsUsingActorFolders());

	AddFolderToWorld(*Level->GetWorld(), InActorFolder->GetFolder());
	// To avoid overriding the expanded state initialized by UWorldFolders::LoadState, only override value if folder is initially collapsed.
	if (!InActorFolder->IsInitiallyExpanded())
	{
		SetIsFolderExpanded(*Level->GetWorld(), InActorFolder->GetFolder(), false);
	}
}

void FActorFolders::OnFolderRootObjectRemoved(UWorld& InWorld, const FFolder::FRootObject& InFolderRootObject)
{
	const FFolder::FRootObject FolderRootObject = FFolder::IsRootObjectValid(InFolderRootObject) ? InFolderRootObject : GetWorldFolderRootObject(InWorld);

	TArray<FFolder> FoldersToDelete;
	ForEachFolderWithRootObject(InWorld, FolderRootObject, [&FoldersToDelete](const FFolder& Folder)
	{
		FoldersToDelete.Add(Folder);
		return true;
	});

	RemoveFoldersFromWorld(InWorld, FoldersToDelete, /*bBroadcastDelete*/ true);
}

void FActorFolders::DeleteFolder(UWorld& InWorld, const FFolder& InFolderToDelete)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_DeleteFolder", "Delete Folder"));
	if (GetOrCreateWorldFolders(InWorld).RemoveFolder(InFolderToDelete, /*bShouldDeleteFolder*/ true))
	{
		BroadcastOnActorFolderDeleted(InWorld, InFolderToDelete);
	}
}

bool FActorFolders::RenameFolderInWorld(UWorld& InWorld, const FFolder& OldPath, const FFolder& NewPath)
{
	// We currently don't support changing the root object
	check(OldPath.GetRootObject() == NewPath.GetRootObject());

	const FString OldPathString = OldPath.ToString();
	const FString NewPathString = NewPath.ToString();

	if (OldPath.IsNone() || OldPathString.Equals(NewPathString) || FEditorFolderUtils::PathIsChildOf(NewPathString, OldPathString))
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("UndoAction_RenameFolder", "Rename Folder"));
	return GetOrCreateWorldFolders(InWorld).RenameFolder(OldPath, NewPath);
}

bool FActorFolders::AddFolderToWorld(UWorld& InWorld, const FFolder& InFolder)
{
	return GetOrCreateWorldFolders(InWorld).AddFolder(InFolder);
}

void FActorFolders::RemoveFoldersFromWorld(UWorld& InWorld, const TArray<FFolder>& InFolders, bool bBroadcastDelete)
{
	if (InFolders.Num() > 0)
	{
		UWorldFolders& Folders = GetOrCreateWorldFolders(InWorld);
		Folders.Modify();
		for (const FFolder& Folder : InFolders)
		{
			if (Folders.RemoveFolder(Folder))
			{
				if (bBroadcastDelete)
				{
					BroadcastOnActorFolderDeleted(InWorld, Folder);
				}
			}
		}
	}
}

bool FActorFolders::ContainsFolder(UWorld& InWorld, const FFolder& InFolder)
{
	return GetFolderProperties(InWorld, InFolder) != nullptr;
}

bool FActorFolders::IsFolderExpanded(UWorld& InWorld, const FFolder& InFolder)
{
	return GetOrCreateWorldFolders(InWorld).IsFolderExpanded(InFolder);
}

void FActorFolders::SetIsFolderExpanded(UWorld& InWorld, const FFolder& InFolder, bool bIsExpanded)
{
	GetOrCreateWorldFolders(InWorld).SetIsFolderExpanded(InFolder, bIsExpanded);
}

FFolder FActorFolders::GetActorEditorContextFolder(UWorld& InWorld, bool bMustMatchCurrentLevel) const
{
	const UWorldFolders** Folders = (const UWorldFolders **)WorldFolders.Find(&InWorld);
	if (Folders)
	{
		return (*Folders)->GetActorEditorContextFolder(bMustMatchCurrentLevel);
	}
	return FFolder::GetWorldRootFolder(&InWorld);
}

void FActorFolders::SetActorEditorContextFolder(UWorld& InWorld, const FFolder& InFolder)
{
	if (!InWorld.IsGameWorld())
	{
		if (GetOrCreateWorldFolders(InWorld).SetActorEditorContextFolder(InFolder))
		{
			BroadcastOnActorEditorContextClientChanged(InWorld);
		}
	}
}

void FActorFolders::OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor)
{
	switch (InType)
	{
		case EActorEditorContextAction::ApplyContext:
			check(InActor && InActor->GetWorld() == InWorld);
			{
				const bool bMustMatchCurrentLevel = false;
				FFolder Folder = GetActorEditorContextFolder(*InWorld, bMustMatchCurrentLevel);
				if (!Folder.IsNone() && (Folder.GetRootObjectAssociatedLevel() == InActor->GetLevel()))
				{
					// Currently not supported to change rootobject through this interface
					if (Folder.GetRootObject() == InActor->GetFolderRootObject())
					{
						InActor->SetFolderPath_Recursively(Folder.GetPath());
					}
				}
			}
			break;
		case EActorEditorContextAction::ResetContext:
			SetActorEditorContextFolder(*InWorld, FFolder::GetWorldRootFolder(InWorld));
			break;
		case EActorEditorContextAction::PushContext:
		case EActorEditorContextAction::PushDuplicateContext:
			GetOrCreateWorldFolders(*InWorld).PushActorEditorContext(InType == EActorEditorContextAction::PushDuplicateContext);
			break;
		case EActorEditorContextAction::PopContext:
			if (UWorldFolders** Folders = (UWorldFolders**)WorldFolders.Find(InWorld))
			{
				(*Folders)->PopActorEditorContext();
			}
			break;
		case EActorEditorContextAction::InitializeContextFromActor:
			SetActorEditorContextFolder(*(InActor->GetWorld()), InActor->GetFolder());
			break;
	}
}

void FActorFolders::CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const
{
	const FFolder Folder = GetActorEditorContextFolder(*InWorld);
	if (!Folder.IsNone())
	{
		UActorEditorContextActorFolderState* State = NewObject<UActorEditorContextActorFolderState>(InStateCollection);
		State->FolderPath = Folder.GetPath();
		InStateCollection->AddState(State);
	}
}

void FActorFolders::RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection)
{
	FFolder Folder = FFolder::GetWorldRootFolder(InWorld);

	if (const UActorEditorContextActorFolderState* State = InStateCollection->GetState<UActorEditorContextActorFolderState>())
	{
		FFolder StateFolder(Folder.GetRootObject(), State->FolderPath);
		if (ContainsFolder(*InWorld, StateFolder))
		{
			Folder = StateFolder;
		}
	}
	
	SetActorEditorContextFolder(*InWorld, Folder);
}

bool FActorFolders::GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const
{
	const FFolder Folder = GetActorEditorContextFolder(*InWorld);
	if (!Folder.IsNone())
	{
		OutDiplayInfo.Title = TEXT("Actor Folder");
		OutDiplayInfo.Brush = FAppStyle::GetBrush(TEXT("SceneOutliner.FolderClosed"));
		return true;
	}
	return false;
}

TSharedRef<SWidget> FActorFolders::GetActorEditorContextWidget(UWorld* InWorld) const
{
	const FFolder Folder = GetActorEditorContextFolder(*InWorld);
	FText Text = (!Folder.IsNone()) ? FText::FromName(Folder.GetLeafName()) : FText::GetEmpty();
	return SNew(STextBlock).Text(Text);
}

void FActorFolders::BroadcastOnActorEditorContextClientChanged(UWorld& InWorld)
{
	if (!InWorld.IsGameWorld())
	{
		ActorEditorContextClientChanged.Broadcast(this);
	}
}

void FActorFolders::ForEachFolder(UWorld& InWorld, TFunctionRef<bool(const FFolder&)> Operation)
{
	GetOrCreateWorldFolders(InWorld).ForEachFolder(Operation);
}

FFolder::FRootObject FActorFolders::GetWorldFolderRootObject(UWorld& InWorld)
{
	return FFolder::GetWorldRootFolder(&InWorld).GetRootObject();
}

void FActorFolders::ForEachFolderWithRootObject(UWorld& InWorld, const FFolder::FRootObject& InFolderRootObject, TFunctionRef<bool(const FFolder&)> Operation)
{
	const FFolder::FRootObject FolderRootObject = FFolder::IsRootObjectValid(InFolderRootObject) ? InFolderRootObject : GetWorldFolderRootObject(InWorld);
	GetOrCreateWorldFolders(InWorld).ForEachFolderWithRootObject(FolderRootObject, Operation);
}

FFolder FActorFolders::GetActorDescInstanceFolder(UWorld& InWorld, const FWorldPartitionActorDescInstance* InActorDescInstance)
{
	if (InActorDescInstance)
	{
		UWorld& OuterWorld = InActorDescInstance->GetContainerInstance() ? *InActorDescInstance->GetContainerInstance()->GetTypedOuter<UWorld>() : InWorld;
		{
			ULevel* OuterLevel = OuterWorld.PersistentLevel;
			if (OuterLevel->IsUsingActorFolders())
			{
				if (UActorFolder* ActorFolder = OuterLevel->GetActorFolder(InActorDescInstance->GetFolderGuid()))
				{
					return ActorFolder->GetFolder();
				}
				return FFolder::GetWorldRootFolder(&OuterWorld).GetRootObject();
			}
			return FFolder(FFolder::GetWorldRootFolder(&OuterWorld).GetRootObject(), InActorDescInstance->GetFolderPath());
		}
	}
	return FFolder::GetInvalidFolder();
}

void FActorFolders::ForEachActorDescInstanceInFolders(UWorld& InWorld, const TSet<FName>& InPaths, TFunctionRef<bool(const FWorldPartitionActorDescInstance*)> Operation, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	if (UWorldPartition* WorldPartition = InWorld.GetWorldPartition())
	{
		FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [&](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FFolder ActorDescFolder = GetActorDescInstanceFolder(InWorld, ActorDescInstance);
			if (ActorDescFolder == FFolder::GetInvalidFolder())
			{
				return true;
			}

			FName ActorDescPath = ActorDescFolder.GetPath();
			if (ActorDescPath.IsNone() || !InPaths.Contains(ActorDescPath))
			{
				return true;
			}

			return Operation(ActorDescInstance);
		});
	}
}

void FActorFolders::ForEachActorInFolders(UWorld& InWorld, const TArray<FName>& InPaths, TFunctionRef<bool(AActor*)> Operation, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	TSet<FName> Paths;
	Paths.Append(InPaths);
	ForEachActorInFolders(InWorld, Paths, Operation, InFolderRootObject);
}

void FActorFolders::ForEachActorInFolders(UWorld& InWorld, const TSet<FName>& InPaths, TFunctionRef<bool(AActor*)> Operation, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	const FFolder::FRootObject FolderRootObject = FFolder::IsRootObjectValid(InFolderRootObject) ? InFolderRootObject : GetWorldFolderRootObject(InWorld);

	for (FActorIterator ActorIt(&InWorld); ActorIt; ++ActorIt)
	{
		if (ActorIt->GetFolderRootObject() != FolderRootObject)
		{
			continue;
		}
		FName ActorPath = ActorIt->GetFolderPath();
		if (ActorPath.IsNone() || !InPaths.Contains(ActorPath))
		{
			continue;
		}

		if (!Operation(*ActorIt))
		{
			return;
		}
	}
}

void FActorFolders::GetActorsFromFolders(UWorld& InWorld, const TArray<FName>& InPaths, TArray<AActor*>& OutActors, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	TSet<FName> Paths;
	Paths.Append(InPaths);
	GetActorsFromFolders(InWorld, Paths, OutActors, InFolderRootObject);
}

void FActorFolders::GetActorsFromFolders(UWorld& InWorld, const TSet<FName>& InPaths, TArray<AActor*>& OutActors, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	const FFolder::FRootObject FolderRootObject = FFolder::IsRootObjectValid(InFolderRootObject) ? InFolderRootObject : GetWorldFolderRootObject(InWorld);

	ForEachActorInFolders(InWorld, InPaths, [&OutActors](AActor* InActor)
	{
		OutActors.Add(InActor);
		return true;
	}, FolderRootObject);
}


void FActorFolders::GetWeakActorsFromFolders(UWorld& InWorld, const TArray<FName>& InPaths, TArray<TWeakObjectPtr<AActor>>& OutActors, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	TSet<FName> Paths;
	Paths.Append(InPaths);
	GetWeakActorsFromFolders(InWorld, Paths, OutActors, InFolderRootObject);
}

void FActorFolders::GetWeakActorsFromFolders(UWorld& InWorld, const TSet<FName>& InPaths, TArray<TWeakObjectPtr<AActor>>& OutActors, const FFolder::FRootObject& InFolderRootObject /*= FFolder::GetInvalidRootObject()*/)
{
	const FFolder::FRootObject FolderRootObject = FFolder::IsRootObjectValid(InFolderRootObject) ? InFolderRootObject : GetWorldFolderRootObject(InWorld);

	ForEachActorInFolders(InWorld, InPaths, [&OutActors](AActor* InActor)
	{
		OutActors.Add(InActor);
		return true;
	}, FolderRootObject);
}

#undef LOCTEXT_NAMESPACE