// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAnimUtils.h"
#include "Modules/ModuleManager.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Engine/SkeletalMesh.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Framework/Notifications/NotificationManager.h"

#include "ObjectEditorUtils.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "EditorReimportHandler.h"
#include "EditorFramework/AssetImportData.h"
#include "AnimationBlueprintLibrary.h"

#define LOCTEXT_NAMESPACE "EditorAnimUtils"

namespace EditorAnimUtils
{
	/** Helper archive class to find all references, used by the cycle finder **/
	class FFindAnimAssetRefs : public FArchiveUObject
	{
	public:
		/**
		* Constructor
		*
		* @param	Src		the object to serialize which may contain a references
		*/
		FFindAnimAssetRefs(UObject* Src, TArray<UAnimationAsset*>& OutAnimationAssets) : AnimationAssets(OutAnimationAssets)
		{
			// use the optimized RefLink to skip over properties which don't contain object references
			ArIsObjectReferenceCollector = true;

			ArIgnoreArchetypeRef = false;
			ArIgnoreOuterRef = true;
			ArIgnoreClassRef = false;

			Src->Serialize(*this);
		}

		virtual FString GetArchiveName() const { return TEXT("FFindAnimAssetRefs"); }

	private:
		/** Serialize a reference **/
		FArchive& operator<<(class UObject*& Obj)
		{
			if (UAnimationAsset* Anim = Cast<UAnimationAsset>(Obj))
			{
				AnimationAssets.AddUnique(Anim);
			}
			return *this;
		}

		TArray<UAnimationAsset*>& AnimationAssets;
	};

	//////////////////////////////////////////////////////////////////
	// FAnimationRetargetContext
	FAnimationRetargetContext::FAnimationRetargetContext(const TArray<FAssetData>& AssetsToRetarget, bool bRetargetReferredAssets, bool bInConvertAnimationDataInComponentSpaces, const FNameDuplicationRule& NameRule) 
		: SingleTargetObject(NULL)
		, bConvertAnimationDataInComponentSpaces(bInConvertAnimationDataInComponentSpaces)
	{
		TArray<UObject*> Objects;
		for(auto Iter = AssetsToRetarget.CreateConstIterator(); Iter; ++Iter)
		{
			Objects.Add((*Iter).GetAsset());
		}
		auto WeakObjectList = FObjectEditorUtils::GetTypedWeakObjectPtrs<UObject>(Objects);
		Initialize(WeakObjectList,bRetargetReferredAssets);
	}

	FAnimationRetargetContext::FAnimationRetargetContext(TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets, bool bInConvertAnimationDataInComponentSpaces, const FNameDuplicationRule& NameRule) 
		: SingleTargetObject(NULL)
		, bConvertAnimationDataInComponentSpaces(bInConvertAnimationDataInComponentSpaces)
	{
		Initialize(AssetsToRetarget,bRetargetReferredAssets);
	}

	void FAnimationRetargetContext::Initialize(TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets)
	{
		for(auto Iter = AssetsToRetarget.CreateConstIterator(); Iter; ++Iter)
		{
			UObject* Asset = (*Iter).Get();
			if( UAnimationAsset* AnimAsset = Cast<UAnimationAsset>(Asset) )
			{
				AnimationAssetsToRetarget.AddUnique(AnimAsset);
			}
			else if( UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Asset) )
			{
				// Add parent non-template blueprints
				UAnimBlueprint* ParentBP = Cast<UAnimBlueprint>(AnimBlueprint->ParentClass->ClassGeneratedBy);
				while (ParentBP)
				{
					// Cant transitively retarget templates
					if(!(ParentBP->bIsTemplate && ParentBP->TargetSkeleton == nullptr))
					{
						AnimBlueprintsToRetarget.AddUnique(ParentBP);
					}
					ParentBP = Cast<UAnimBlueprint>(ParentBP->ParentClass->ClassGeneratedBy);
				}
				
				AnimBlueprintsToRetarget.AddUnique(AnimBlueprint);
			}
		}
		
		if(AssetsToRetarget.Num() == 1)
		{
			//Only chose one object to retarget, keep track of it
			SingleTargetObject = AssetsToRetarget[0].Get();
		}

		if(bRetargetReferredAssets)
		{
			// Grab assets from the blueprint. Do this first as it can add complex assets to the retarget array
			// which will need to be processed next.
			for(auto Iter = AnimBlueprintsToRetarget.CreateConstIterator(); Iter; ++Iter)
			{
				GetAllAnimationSequencesReferredInBlueprint( (*Iter), AnimationAssetsToRetarget);
			}

			int32 AssetIndex = 0;
			while (AssetIndex < AnimationAssetsToRetarget.Num())
			{
				UAnimationAsset* AnimAsset = AnimationAssetsToRetarget[AssetIndex++];
				AnimAsset->HandleAnimReferenceCollection(AnimationAssetsToRetarget, true);
			}
		}
	}

	bool FAnimationRetargetContext::HasAssetsToRetarget() const
	{
		return	AnimationAssetsToRetarget.Num() > 0 ||
				AnimBlueprintsToRetarget.Num() > 0;
	}

	bool FAnimationRetargetContext::HasDuplicates() const
	{
		return	DuplicatedAnimAssets.Num() > 0     ||
				DuplicatedBlueprints.Num() > 0;
	}

	TArray<UObject*> FAnimationRetargetContext::GetAllDuplicates() const
	{
		TArray<UObject*> Duplicates;

		if (AnimationAssetsToRetarget.Num() > 0)
		{
			Duplicates.Append(AnimationAssetsToRetarget);
		}

		if(AnimBlueprintsToRetarget.Num() > 0)
		{
			Duplicates.Append(AnimBlueprintsToRetarget);
		}
		return Duplicates;
	}

	UObject* FAnimationRetargetContext::GetSingleTargetObject() const
	{
		return SingleTargetObject;
	}

	UObject* FAnimationRetargetContext::GetDuplicate(const UObject* OriginalObject) const
	{
		if(HasDuplicates())
		{
			if(const UAnimationAsset* Asset = Cast<const UAnimationAsset>(OriginalObject)) 
			{
				if(DuplicatedAnimAssets.Contains(Asset))
				{
					return DuplicatedAnimAssets.FindRef(Asset);
				}
			}
			if(const UAnimBlueprint* AnimBlueprint = Cast<const UAnimBlueprint>(OriginalObject))
			{
				if(DuplicatedBlueprints.Contains(AnimBlueprint))
				{
					return DuplicatedBlueprints.FindRef(AnimBlueprint);
				}
			}
		}
		return NULL;
	}

	void FAnimationRetargetContext::DuplicateAssetsToRetarget(UPackage* DestinationPackage, const FNameDuplicationRule* NameRule)
	{
		if(!HasDuplicates())
		{
			TArray<UAnimationAsset*> AnimationAssetsToDuplicate = AnimationAssetsToRetarget;
			TArray<UAnimBlueprint*> AnimBlueprintsToDuplicate = AnimBlueprintsToRetarget;

			// We only want to duplicate unmapped assets, so we remove mapped assets from the list we're duplicating
			for(TPair<UAnimationAsset*, UAnimationAsset*>& Pair : RemappedAnimAssets)
			{
				AnimationAssetsToDuplicate.Remove(Pair.Key);
			}

			DuplicatedAnimAssets = DuplicateAssets<UAnimationAsset>(AnimationAssetsToDuplicate, DestinationPackage, NameRule);
			DuplicatedBlueprints = DuplicateAssets<UAnimBlueprint>(AnimBlueprintsToDuplicate, DestinationPackage, NameRule);

			// If we are moving the new asset to a different directory we need to fixup the reimport path. This should only effect source FBX paths within the project.
			if (!NameRule->FolderPath.IsEmpty())
			{
				for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : DuplicatedAnimAssets)
				{
					UAnimSequence* SourceSequence = Cast<UAnimSequence>(Pair.Key);
					UAnimSequence* DestinationSequence = Cast<UAnimSequence>(Pair.Value);
					if (SourceSequence && DestinationSequence)
					{
						for (int index = 0; index < SourceSequence->AssetImportData->SourceData.SourceFiles.Num(); index++)
						{
							const FString& RelativeFilename = SourceSequence->AssetImportData->SourceData.SourceFiles[index].RelativeFilename;
							const FString OldPackagePath = FPackageName::GetLongPackagePath(SourceSequence->GetPathName()) / TEXT("");
							const FString NewPackagePath = FPackageName::GetLongPackagePath(DestinationSequence->GetPathName()) / TEXT("");

							if (NewPackagePath != OldPackagePath)
							{
								const FString AbsoluteSrcPath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(OldPackagePath));
								const FString SrcFile = AbsoluteSrcPath / RelativeFilename;

								if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*SrcFile))
								{
									FString OldSourceFilePath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(OldPackagePath), RelativeFilename);

									TArray<FString> Paths;
									Paths.Add(OldSourceFilePath);

									// Update the reimport file names
									FReimportManager::Instance()->UpdateReimportPaths(DestinationSequence, Paths);
								}
							}
						}
					}
				}
			}

			// Remapped assets needs the duplicated ones added
			RemappedAnimAssets.Append(DuplicatedAnimAssets);

			DuplicatedAnimAssets.GenerateValueArray(AnimationAssetsToRetarget);
			DuplicatedBlueprints.GenerateValueArray(AnimBlueprintsToRetarget);
		}
	}

	void FAnimationRetargetContext::RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton)
	{
		check (!bConvertAnimationDataInComponentSpaces || OldSkeleton);
		check (NewSkeleton);

		if (bConvertAnimationDataInComponentSpaces)
		{
			// we need to update reference pose before retargeting. 
			// this is to ensure the skeleton has the latest pose you're looking at. 
			USkeletalMesh * PreviewMesh = NULL;
			if (OldSkeleton != NULL)
			{
				PreviewMesh = OldSkeleton->GetPreviewMesh(true);
				if (PreviewMesh)
				{
					OldSkeleton->UpdateReferencePoseFromMesh(PreviewMesh);
				}
			}
			
			PreviewMesh = NewSkeleton->GetPreviewMesh(true);
			if (PreviewMesh)
			{
				NewSkeleton->UpdateReferencePoseFromMesh(PreviewMesh);
			}
		}

		// anim sequences will be retargeted first becauseReplaceSkeleton forces it to change skeleton
		// @todo: please note that I think we can merge two loops 
		//(without separating two loops - one for AnimSequence and one for everybody else)
		// but if you have animation asssets that does replace skeleton, it will try fix up internal asset also
		// so I think you might be doing twice - look at AnimationAsset:ReplaceSkeleton
		// for safety, I'm doing Sequence first and then everything else
		// however this can be re-investigated and fixed better in the future
		for(auto Iter = AnimationAssetsToRetarget.CreateIterator(); Iter; ++Iter)
		{
			UAnimSequence* AnimSequenceToRetarget = Cast<UAnimSequence>(*Iter);
			if (AnimSequenceToRetarget)
			{
				// Copy curve data from source asset, preserving data in the target if present.
				if (OldSkeleton)
				{
					UAnimationBlueprintLibrary::CopyAnimationCurveNamesToSkeleton(OldSkeleton, NewSkeleton, AnimSequenceToRetarget, ERawCurveTrackTypes::RCT_Float);

					// clear transform curves since those curves won't work in new skeleton
					// since we're deleting curves, mark this rebake flag off
					IAnimationDataController& Controller = AnimSequenceToRetarget->GetController();
					Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform);

					// I can't copy transform curves yet because transform curves need retargeting. 
					//EditorAnimUtils::CopyAnimCurves(OldSkeleton, NewSkeleton, AssetToRetarget, USkeleton::AnimTrackCurveMappingName, FRawCurveTracks::TransformType);
				}
			}

			UAnimationAsset* AssetToRetarget = (*Iter);
			if (HasDuplicates())
			{
				AssetToRetarget->ReplaceReferredAnimations(RemappedAnimAssets);
			}
			AssetToRetarget->ReplaceSkeleton(NewSkeleton, bConvertAnimationDataInComponentSpaces);
			AssetToRetarget->MarkPackageDirty();
		}

		// convert all Animation Blueprints and compile 
		for ( auto AnimBPIter = AnimBlueprintsToRetarget.CreateIterator(); AnimBPIter; ++AnimBPIter )
		{
			UAnimBlueprint * AnimBlueprint = (*AnimBPIter);

			AnimBlueprint->TargetSkeleton = NewSkeleton;

			// We can directly retarget templates (although not transitively via parents) so we need to make sure we clear the flag here
			AnimBlueprint->bIsTemplate = false;

			if (HasDuplicates())
			{
				// if they have parent blueprint, make sure to re-link to the new one also
				UAnimBlueprint* CurrentParentBP = Cast<UAnimBlueprint>(AnimBlueprint->ParentClass->ClassGeneratedBy);
				if (CurrentParentBP)
				{
					UAnimBlueprint* const * ParentBP = DuplicatedBlueprints.Find(CurrentParentBP);
					if (ParentBP)
					{
						AnimBlueprint->ParentClass = (*ParentBP)->GeneratedClass;
					}
				}
			}

			if(RemappedAnimAssets.Num() > 0)
			{
				ReplaceReferredAnimationsInBlueprint(AnimBlueprint, RemappedAnimAssets);
			}

			FBlueprintEditorUtils::RefreshAllNodes(AnimBlueprint);
			FKismetEditorUtilities::CompileBlueprint(AnimBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
			AnimBlueprint->PostEditChange();
			AnimBlueprint->MarkPackageDirty();
		}
	}

	void FAnimationRetargetContext::AddRemappedAsset(UAnimationAsset* OriginalAsset, UAnimationAsset* NewAsset)
	{
		RemappedAnimAssets.Add(OriginalAsset, NewAsset);
	}

	void OpenAssetFromNotify(UObject* AssetToOpen)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetToOpen);
	}

	//////////////////////////////////////////////////////////////////
	UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, TArray<TWeakObjectPtr<UObject>> AssetsToRetarget, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule, bool bConvertSpace)
	{
		FAnimationRetargetContext RetargetContext(AssetsToRetarget, bRetargetReferredAssets, bConvertSpace);
		return RetargetAnimations(OldSkeleton, NewSkeleton, RetargetContext, bRetargetReferredAssets, NameRule);
	}

	UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, const TArray<FAssetData>& AssetsToRetarget, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule, bool bConvertSpace)
	{
		FAnimationRetargetContext RetargetContext(AssetsToRetarget, bRetargetReferredAssets, bConvertSpace);
		return RetargetAnimations(OldSkeleton, NewSkeleton, RetargetContext, bRetargetReferredAssets, NameRule);
	}

	UObject* RetargetAnimations(USkeleton* OldSkeleton, USkeleton* NewSkeleton, FAnimationRetargetContext& RetargetContext, bool bRetargetReferredAssets, const FNameDuplicationRule* NameRule)
	{
		check(NewSkeleton);
		UObject* OriginalObject  = RetargetContext.GetSingleTargetObject();
		UPackage* DuplicationDestPackage = NewSkeleton->GetOutermost();

		if(	RetargetContext.HasAssetsToRetarget() )
		{
			if(NameRule)
			{
				RetargetContext.DuplicateAssetsToRetarget(DuplicationDestPackage, NameRule);
			}
			RetargetContext.RetargetAnimations(OldSkeleton, NewSkeleton);
		}

		FNotificationInfo Notification(FText::GetEmpty());
		Notification.ExpireDuration = 5.f;

		UObject* NotifyLinkObject = OriginalObject;
		if(OriginalObject && NameRule)
		{
			NotifyLinkObject = RetargetContext.GetDuplicate(OriginalObject);
		}

		if(!NameRule)
		{
			if(OriginalObject)
			{
				Notification.Text = FText::Format(LOCTEXT("SingleNonDuplicatedAsset", "'{0}' retargeted to new skeleton '{1}'"), FText::FromString(OriginalObject->GetName()), FText::FromString(NewSkeleton->GetName()));
			}
			else
			{
				Notification.Text = FText::Format(LOCTEXT("MultiNonDuplicatedAsset", "Assets retargeted to new skeleton '{0}'"), FText::FromString(NewSkeleton->GetName()));
			}
			
		}
		else
		{
			if(OriginalObject)
			{
				Notification.Text = FText::Format(LOCTEXT("SingleDuplicatedAsset", "'{0}' duplicated to '{1}' and retargeted"), FText::FromString(OriginalObject->GetName()), FText::FromString(DuplicationDestPackage->GetName()));
			}
			else
			{
				Notification.Text = FText::Format(LOCTEXT("MultiDuplicatedAsset", "Assets duplicated to '{0}' and retargeted"), FText::FromString(DuplicationDestPackage->GetName()));
			}
		}

		if(NotifyLinkObject)
		{
			Notification.Hyperlink = FSimpleDelegate::CreateStatic(&OpenAssetFromNotify, NotifyLinkObject);
			Notification.HyperlinkText = LOCTEXT("OpenAssetLink", "Open");
		}

		FSlateNotificationManager::Get().AddNotification(Notification);

		// sync newly created objects on CB
		if (NotifyLinkObject)
		{
			TArray<UObject*> NewObjects = RetargetContext.GetAllDuplicates();
			TArray<FAssetData> CurrentSelection;
			for(auto& NewObject : NewObjects)
			{
				CurrentSelection.Add(FAssetData(NewObject));
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets(CurrentSelection);
		}
		if(OriginalObject && NameRule)
		{
			return RetargetContext.GetDuplicate(OriginalObject);
		}
		return NULL;
	}

	FString CreateDesiredName(UObject* Asset, const FNameDuplicationRule* NameRule)
	{
		check(Asset);

		FString NewName = Asset->GetName();

		if(NameRule)
		{
			NewName = NameRule->Rename(Asset);
		}

		return NewName;
	}

	TMap<UObject*, UObject*> DuplicateAssetsInternal(const TArray<UObject*>& AssetsToDuplicate, UPackage* DestinationPackage, const FNameDuplicationRule* NameRule)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		TMap<UObject*, UObject*> DuplicateMap;
		for(UObject* AssetToDuplicate : AssetsToDuplicate)
		{
			if(DuplicateMap.Contains(AssetToDuplicate))
			{
				continue; // ignore duplicates (shouldn't happen though)
			}
			
			// create unique name in case of existing asset with same name
			FString PathName = (NameRule)? NameRule->FolderPath : FPackageName::GetLongPackagePath(DestinationPackage->GetName());
			FString ObjectName = CreateDesiredName(AssetToDuplicate, NameRule);
			FString PackageName;
			FString BasePackageName = PathName+"/"+ObjectName;
			AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), PackageName, ObjectName);

			// create a new asset
			if (UObject* NewAsset = AssetToolsModule.Get().DuplicateAsset(ObjectName, PathName, AssetToDuplicate))
			{
				DuplicateMap.Add(AssetToDuplicate, NewAsset);
			}
		}

		return DuplicateMap;
	}

	void GetAllAnimationSequencesReferredInBlueprint(UAnimBlueprint* AnimBlueprint, TArray<UAnimationAsset*>& AnimationAssets)
	{
		UObject* DefaultObject = AnimBlueprint->GetAnimBlueprintGeneratedClass()->GetDefaultObject();
		FFindAnimAssetRefs AnimRefFinderObject(DefaultObject, AnimationAssets);
		
		// For assets referenced in the event graph (either pin default values or variable-get nodes)
		// we need to serialize the nodes in that graph
		for(UEdGraph* GraphPage : AnimBlueprint->UbergraphPages)
		{
			for(UEdGraphNode* Node : GraphPage->Nodes)
			{
				FFindAnimAssetRefs AnimRefFinderBlueprint(Node, AnimationAssets);
			}
		}

		// Gather references in functions
		for(UEdGraph* GraphPage : AnimBlueprint->FunctionGraphs)
		{
			for(UEdGraphNode* Node : GraphPage->Nodes)
			{
				FFindAnimAssetRefs AnimRefFinderBlueprint(Node, AnimationAssets);
			}
		}
		
		// removes blendspaces that are embedded in the graph so that we don't duplicate them to make new assets
		AnimationAssets.RemoveAll([](UAnimationAsset*& AnimationAsset)
		{
			const bool bIsBlendspace = IsValid(Cast<UBlendSpace>(AnimationAsset));
			const bool bIsEmbedded = !AnimationAsset->GetSkeleton();
			return bIsBlendspace && bIsEmbedded;
		});
	}

	void ReplaceReferredAnimationsInBlueprint(UAnimBlueprint* AnimBlueprint, const TMap<UAnimationAsset*, UAnimationAsset*>& AnimAssetReplacementMap)
	{
		UObject* DefaultObject = AnimBlueprint->GetAnimBlueprintGeneratedClass()->GetDefaultObject();

		FArchiveReplaceObjectRef<UAnimationAsset> ReplaceAr(DefaultObject, AnimAssetReplacementMap);
		FArchiveReplaceObjectRef<UAnimationAsset> ReplaceAr2(AnimBlueprint, AnimAssetReplacementMap);

		// Replace event graph references
		for(UEdGraph* GraphPage : AnimBlueprint->UbergraphPages)
		{
			for(UEdGraphNode* Node : GraphPage->Nodes)
			{
				FArchiveReplaceObjectRef<UAnimationAsset> ReplaceGraphAr(Node, AnimAssetReplacementMap);
			}
		}

		// Replace references in functions
		for(UEdGraph* GraphPage : AnimBlueprint->FunctionGraphs)
		{
			for(UEdGraphNode* Node : GraphPage->Nodes)
			{
				FArchiveReplaceObjectRef<UAnimationAsset> ReplaceGraphAr(Node, AnimAssetReplacementMap);
			}
		}
	}

	void CopyAnimCurves(USkeleton* OldSkeleton, USkeleton* NewSkeleton, UAnimSequenceBase* SequenceBase, const FName ContainerName, ERawCurveTrackTypes CurveType)
	{
		// In some circumstances the asset may have already been updated during the retarget process (eg. retargeting of child assets for blendspaces, etc)
		if (NewSkeleton != SequenceBase->GetSkeleton())
		{
			UAnimationBlueprintLibrary::CopyAnimationCurveNamesToSkeleton(OldSkeleton, NewSkeleton, SequenceBase, CurveType);
		}
	}

	FString FNameDuplicationRule::Rename(const UObject* Asset) const
	{
		check(Asset);

		FString NewName = Asset->GetName();

		NewName = NewName.Replace(*ReplaceFrom, *ReplaceTo);
		return FString::Printf(TEXT("%s%s%s"), *Prefix, *NewName, *Suffix);
	}
}

#undef LOCTEXT_NAMESPACE
