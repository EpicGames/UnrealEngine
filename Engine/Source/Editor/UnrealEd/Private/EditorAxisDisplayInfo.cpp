// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAxisDisplayInfo.h"

#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "EditorAxisDisplayInfo"

namespace UE::Editor::Private
{
	static TAutoConsoleVariable<int> EditorAxisDisplayCoordinateSystem(
		TEXT("Editor.AxisDisplayCoordinateSystem"),
		0, // XYZ
		TEXT("Sets the editor's axis display coordinate system { 0 = XYZ (default), 1 = FRU (deprecated), 2 = LUF }"),
		ECVF_ReadOnly
	);
}

FEditorAxisDisplayInfo::FEditorAxisDisplayInfo()
{
	FEditorDelegates::OnEditorBoot.AddRaw(this, &FEditorAxisDisplayInfo::InitSettingsInfo);
}

EAxisList::Type FEditorAxisDisplayInfo::GetAxisDisplayCoordinateSystem() const
{
	if (!AxisDisplayCoordinateSystem.IsSet())
	{
		const int EditorAxisDisplayCoordinateSystem = UE::Editor::Private::EditorAxisDisplayCoordinateSystem.GetValueOnAnyThread();
		
		if (EditorAxisDisplayCoordinateSystem == 2)
		{
			AxisDisplayCoordinateSystem = EAxisList::LeftUpForward;
		}
		else
		{
			// There is no enumeration for the deprecated (Forward,Right,Up) (EditorAxisDisplayCoordinateSystem == 1), it uses EAxisList::XYZ

			ensureMsgf(EditorAxisDisplayCoordinateSystem == 0 || EditorAxisDisplayCoordinateSystem == 1,
				TEXT("Unsupported Editor.AxisDisplayCoordinateSystem: %d"), EditorAxisDisplayCoordinateSystem);

			AxisDisplayCoordinateSystem = EAxisList::XYZ;
		}
	}

	return AxisDisplayCoordinateSystem.Get(EAxisList::XYZ/*DefaultValue*/);
}

FText FEditorAxisDisplayInfo::GetAxisToolTip(EAxisList::Type Axis) const
{
	static const FText XToolTip = LOCTEXT("XDisplayName", "X");
	static const FText YToolTip = LOCTEXT("YDisplayName", "Y");
	static const FText ZToolTip = LOCTEXT("ZDisplayName", "Z");
	static const FText LeftToolTip = LOCTEXT("LeftToolTip", "Left (was -Y)");
	static const FText UpDisplayName = LOCTEXT("UpToolTip", "Up (was Z)");
	static const FText ForwardDisplayName = LOCTEXT("ForwardToolTip", "Forward (was X)");

	Axis = MapAxis(Axis);
	
	if (Axis == EAxisList::X)
	{
		return XToolTip;
	}
	else if (Axis == EAxisList::Y)
	{
		return YToolTip;
	}
	else if (Axis == EAxisList::Z)
	{
		return ZToolTip;
	}
	else if (Axis == EAxisList::Left)
	{
		return LeftToolTip;
	}
	else if (Axis == EAxisList::Up)
	{
		return UpDisplayName;
	}
	else if (Axis == EAxisList::Forward)
	{
		return ForwardDisplayName;
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedDisplayName", "Unsupported");
	
}

FText FEditorAxisDisplayInfo::GetAxisDisplayName(EAxisList::Type Axis)
{
	static const FText XDisplayName = LOCTEXT("XDisplayName", "X");
	static const FText YDisplayName = LOCTEXT("YDisplayName", "Y");
	static const FText ZDisplayName = LOCTEXT("ZDisplayName", "Z");
	static const FText LeftDisplayName = LOCTEXT("LeftDisplayName", "Left");
	static const FText UpDisplayName = LOCTEXT("UpDisplayName", "Up");
	static const FText ForwardDisplayName = LOCTEXT("ForwardDisplayName", "Forward");
	
	Axis = MapAxis(Axis);

	if (Axis == EAxisList::X)
	{
		return XDisplayName;
	}
	else if (Axis == EAxisList::Y)
	{
		return YDisplayName;
	}
	else if (Axis == EAxisList::Z)
	{
		return ZDisplayName;
	}
	else if (Axis == EAxisList::Left)
	{
		return LeftDisplayName;
	}
	else if (Axis == EAxisList::Up)
	{
		return UpDisplayName;
	}
	else if (Axis == EAxisList::Forward)
	{
		return ForwardDisplayName;
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedDisplayName", "Unsupported");
}

FText FEditorAxisDisplayInfo::GetAxisDisplayNameShort(EAxisList::Type Axis)
{
	static const FText XDisplayNameShort = LOCTEXT("XDisplayNameShort", "X");
	static const FText YDisplayNameShort = LOCTEXT("YDisplayNameShort", "Y");
	static const FText ZDisplayNameShort = LOCTEXT("ZDisplayNameShort", "Z");
	static const FText LeftDisplayNameShort = LOCTEXT("LeftDisplayNameShort", "L");
	static const FText UpDisplayNameShort = LOCTEXT("UpDisplayNameShort", "U");
	static const FText ForwardDisplayNameShort = LOCTEXT("ForwardDisplayNameShort", "F");

	Axis = MapAxis(Axis);

	if (Axis == EAxisList::X)
	{
		return XDisplayNameShort;
	}
	else if (Axis == EAxisList::Y)
	{
		return YDisplayNameShort;
	}
	else if (Axis == EAxisList::Z)
	{
		return ZDisplayNameShort;
	}
	else if (Axis == EAxisList::Left)
	{
		return LeftDisplayNameShort;
	}
	else if (Axis == EAxisList::Up)
	{
		return UpDisplayNameShort;
	}
	else if (Axis == EAxisList::Forward)
	{
		
		return ForwardDisplayNameShort;
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("UnsupportedDisplayNameShort", "?");
}

FLinearColor FEditorAxisDisplayInfo::GetAxisColor(EAxisList::Type Axis)
{
	Axis = MapAxis(Axis);

	if (Axis == EAxisList::X || Axis == EAxisList::Forward)
	{
		return GetDefault<UEditorStyleSettings>()->XAxisColor;
	}
	else if (Axis == EAxisList::Y || Axis == EAxisList::Left)
	{
		return GetDefault<UEditorStyleSettings>()->YAxisColor;
	}
	else if (Axis == EAxisList::Z || Axis == EAxisList::Up)
	{
		return GetDefault<UEditorStyleSettings>()->ZAxisColor;
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return FLinearColor::Black;
}

FIntVector4 FEditorAxisDisplayInfo::DefaultAxisComponentDisplaySwizzle() const
{
	if (GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
	{
		// Unreal:    -Y    Z     X
		// Semantic: Left, Up, Forward
		return FIntVector4(1, 2, 0, 3);
	}
	return FIntVector4(0, 1, 2, 3);
}

bool FEditorAxisDisplayInfo::UseForwardRightUpDisplayNames()
{
	if (!bUseForwardRightUpDisplayNames.IsSet())
	{
		const int EditorAxisDisplayCoordinateSystem = UE::Editor::Private::EditorAxisDisplayCoordinateSystem.GetValueOnAnyThread();
		bUseForwardRightUpDisplayNames = (EditorAxisDisplayCoordinateSystem == 1);
	}

	return bUseForwardRightUpDisplayNames.Get(false/*DefaultValue*/);
}

FText FEditorAxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Type Axis) const
{
	Axis = MapAxis(Axis);
	
	if (Axis == EAxisList::X)
	{
		return LOCTEXT("GetRotationAxisToolTip_Roll", "Roll");
	}
	else if (Axis == EAxisList::Y)
	{
		return LOCTEXT("GetRotationAxisToolTip_Pitch", "Pitch");
	}
	else if (Axis == EAxisList::Z)
	{
		return LOCTEXT("GetRotationAxisToolTip_Yaw", "Yaw");
	}
	else if (Axis == EAxisList::Forward)
	{
		return LOCTEXT("GetRotationAxisToolTip_Forward", "Forward (was X)");
	}
	else if (Axis == EAxisList::Left)
	{
		return LOCTEXT("GetRotationAxisToolTip_Left", "Left (was -Y)");
	}
	else if (Axis == EAxisList::Up)
	{
		return LOCTEXT("GetRotationAxisToolTip_Up", "Up (was Z)");
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("GetRotationAxisToolTip_Unsupported", "?");
}

FText FEditorAxisDisplayInfo::GetRotationAxisName(EAxisList::Type Axis)
{
	return GetRotationAxisNameShort(Axis);
}

FText FEditorAxisDisplayInfo::GetRotationAxisNameShort(EAxisList::Type Axis)
{
	Axis = MapAxis(Axis);
	
	if (Axis == EAxisList::X)
	{
		return LOCTEXT("GetRotationAxisNameShort_X", "Roll");
	}
	else if (Axis == EAxisList::Y)
	{
		return LOCTEXT("GetRotationAxisNameShort_Y", "Pitch");
	}
	else if (Axis == EAxisList::Z)
	{
		return LOCTEXT("GetRotationAxisNameShort_Z", "Yaw");
	}
	else if (Axis == EAxisList::Forward)
	{
		return LOCTEXT("GetRotationAxisNameShort_Forward", "Forward");
	}
	else if (Axis == EAxisList::Left)
	{
		return LOCTEXT("GetRotationAxisNameShort_Left", "Left");
	}
	else if (Axis == EAxisList::Up)
	{
		return LOCTEXT("GetRotationAxisNameShort_Up", "Up");
	}

	ensureMsgf(false, TEXT("Unsupported Axis: %d"), Axis);

	return LOCTEXT("GetRotationAxisNameShort_Unsupported", "?");
}

EAxisList::Type FEditorAxisDisplayInfo::MapAxis(EAxisList::Type Axis) const
{
	if (GetAxisDisplayCoordinateSystem() == EAxisList::XYZ)
	{
		if (Axis == EAxisList::Forward)
		{
			return EAxisList::X;
		}
		else if (Axis == EAxisList::Left)
		{
			return EAxisList::Y;
		}
		else if (Axis == EAxisList::Up)
		{
			return EAxisList::Z;
		}
	}

	return Axis;
}

void FEditorAxisDisplayInfo::InitSettingsInfo(double EditorStartupTime)
{
	if (FProperty* XAxisColorProperty = UEditorStyleSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, XAxisColor)))
	{
		XAxisColorProperty->SetMetaData(TEXT("DisplayName"), *FText::Format(LOCTEXT("XAxisColorDisplayName", "{0} Axis Color"), GetAxisDisplayName(EAxisList::Forward)).ToString());
		XAxisColorProperty->SetMetaData(TEXT("ToolTip"), *FText::Format(LOCTEXT("XAxisColorToolTip", "The color used for the {0} axis"), GetAxisDisplayName(EAxisList::Forward)).ToString());
	}
	if (FProperty* YAxisColorProperty = UEditorStyleSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, YAxisColor)))
	{
		YAxisColorProperty->SetMetaData(TEXT("DisplayName"), *FText::Format(LOCTEXT("YAxisColorDisplayName", "{0} Axis Color"), GetAxisDisplayName(EAxisList::Left)).ToString());
		YAxisColorProperty->SetMetaData(TEXT("ToolTip"), *FText::Format(LOCTEXT("YAxisColorToolTip", "The color used for the {0} axis"), GetAxisDisplayName(EAxisList::Left)).ToString());
	}
	if (FProperty* ZAxisColorProperty = UEditorStyleSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorStyleSettings, ZAxisColor)))
	{
		ZAxisColorProperty->SetMetaData(TEXT("DisplayName"), *FText::Format(LOCTEXT("ZAxisColorDisplayName", "{0} Axis Color"), GetAxisDisplayName(EAxisList::Up)).ToString());
		ZAxisColorProperty->SetMetaData(TEXT("ToolTip"), *FText::Format(LOCTEXT("ZAxisColorToolTip", "The color used for the {0} axis"), GetAxisDisplayName(EAxisList::Up)).ToString());
	}
}

#undef LOCTEXT_NAMESPACE
