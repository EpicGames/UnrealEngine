// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UEditorBrushBuilder.cpp: UnrealEd brush builder.
=============================================================================*/

#include "Builders/EditorBrushBuilder.h"
#include "Model.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "Builders/ConeBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Builders/CurvedStairBuilder.h"
#include "Builders/CylinderBuilder.h"
#include "Builders/LinearStairBuilder.h"
#include "Builders/SheetBuilder.h"
#include "Builders/SpiralStairBuilder.h"
#include "Builders/TetrahedronBuilder.h"
#include "Builders/VolumetricBuilder.h"
#include "Engine/Brush.h"
#include "Engine/Polys.h"
#include "Editor.h"
#include "BSPOps.h"
#include "SnappingUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/Selection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorBrushBuilder)

#define LOCTEXT_NAMESPACE "BrushBuilder"

/*-----------------------------------------------------------------------------
	UEditorBrushBuilder.
-----------------------------------------------------------------------------*/
UEditorBrushBuilder::UEditorBrushBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BitmapFilename = TEXT("BBGeneric");
	ToolTip = TEXT("BrushBuilderName_Generic");
	NotifyBadParams = true;
}

void UEditorBrushBuilder::BeginBrush(bool InMergeCoplanars, FName InLayer)
{
	Layer = InLayer;
	MergeCoplanars = InMergeCoplanars;
	Vertices.Empty();
	Polys.Empty();
}

bool UEditorBrushBuilder::EndBrush( UWorld* InWorld, ABrush* InBrush )
{
	//!!validate
	
	if (!GEditor)
	{
		return false;
	}

	ABrush* BuilderBrush = (InBrush != nullptr) ? InBrush : (InWorld != nullptr) ? InWorld->GetDefaultBrush() : nullptr;
	if (!ensure(BuilderBrush))
	{
		return false;
	}

	// Ensure the builder brush is unhidden.
	BuilderBrush->SetHidden(false);
	BuilderBrush->bHiddenEdLayer = false;

	AActor* Actor = GEditor->GetSelectedActors()->GetTop<AActor>();
	FVector Location;
	if ( InBrush == nullptr )
	{
		Location = Actor ? Actor->GetActorLocation() : BuilderBrush->GetActorLocation();
	}
	else
	{
		Location = InBrush->GetActorLocation();
	}

	UModel* Brush = BuilderBrush->Brush;
	if (Brush == nullptr)
	{
		return true;
	}

	Brush->Modify(false);
	BuilderBrush->Modify(false);

	FRotator Temp(0.0f,0.0f,0.0f);
	FSnappingUtils::SnapToBSPVertex( Location, FVector::ZeroVector, Temp );
	BuilderBrush->SetActorLocation(Location, false);
	BuilderBrush->SetPivotOffset( FVector::ZeroVector );

	// Try and maintain the materials assigned to the surfaces. 
	TArray<FPoly> CachedPolys;
	UMaterialInterface* CachedMaterial = nullptr;
	if( Brush->Polys->Element.Num() == Polys.Num() )
	{
		// If the number of polygons match we assume its the same shape.
		CachedPolys.Append(Brush->Polys->Element);
	}
	else if( Brush->Polys->Element.Num() > 0 )
	{
		// If the polygons have changed check if we only had one material before. 
		CachedMaterial = Brush->Polys->Element[0].Material;
		if (CachedMaterial != NULL)
		{
			for( auto Poly : Brush->Polys->Element )
			{
				if( CachedMaterial != Poly.Material )
				{
					CachedMaterial = NULL;
					break;
				}
			}
		}
	}

	// Clear existing polys.
	Brush->Polys->Element.Empty();

	const bool bUseCachedPolysMaterial = CachedPolys.Num() > 0;
	int32 CachedPolyIdx = 0;
	for( TArray<FBuilderPoly>::TIterator It(Polys); It; ++It )
	{
		if( It->Direction<0 )
		{
			for( int32 i=0; i<It->VertexIndices.Num()/2; i++ )
			{
				Exchange( It->VertexIndices[i], It->VertexIndices.Last(i) );
			}
		}

		FPoly Poly;
		Poly.Init();
		Poly.ItemName = It->ItemName;
		Poly.Base = (FVector3f)Vertices[It->VertexIndices[0]];
		Poly.PolyFlags = It->PolyFlags;

		// Try and maintain the polygons material where possible
		Poly.Material = ( bUseCachedPolysMaterial ) ? ToRawPtr(CachedPolys[CachedPolyIdx++].Material) : ToRawPtr(CachedMaterial);

		for( int32 j=0; j<It->VertexIndices.Num(); j++ )
		{
			Poly.Vertices.Emplace(Vertices[It->VertexIndices[j]]);
		}
		if( Poly.Finalize( BuilderBrush, 1 ) == 0 )
		{
			Brush->Polys->Element.Add(Poly);
		}
	}

	if( MergeCoplanars )
	{
		GEditor->bspMergeCoplanars( Brush, 0, 1 );
		FBSPOps::bspValidateBrush( Brush, 1, 1 );
	}

	Brush->Linked = 1;
	FBSPOps::bspValidateBrush( Brush, 0, 1 );
	Brush->BuildBound();

	GEditor->RedrawLevelEditingViewports();
	GEditor->SetPivot(BuilderBrush->GetActorLocation(), false, true);

	BuilderBrush->ReregisterAllComponents();

	return true;
}

int32 UEditorBrushBuilder::GetVertexCount() const
{
	return Vertices.Num();
}

FVector UEditorBrushBuilder::GetVertex(int32 i) const
{
	return Vertices.IsValidIndex(i) ? Vertices[i] : FVector::ZeroVector;
}

int32 UEditorBrushBuilder::GetPolyCount() const
{
	return Polys.Num();
}

bool UEditorBrushBuilder::BadParameters(const FText& Msg)
{
	if ( NotifyBadParams )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Msg"), Msg);
		FNotificationInfo Info( FText::Format( LOCTEXT( "BadParameters", "Bad parameters in brush builder\n{Msg}" ), Arguments ) );
		Info.bFireAndForget = true;
		Info.ExpireDuration = Msg.IsEmpty() ? 4.0f : 6.0f;
		Info.bUseLargeFont = Msg.IsEmpty();
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
		FSlateNotificationManager::Get().AddNotification( Info );
	}
	return 0;
}

int32 UEditorBrushBuilder::Vertexv(FVector V)
{
	int32 Result = Vertices.Num();
	Vertices.Add(V);

	return Result;
}

int32 UEditorBrushBuilder::Vertex3f(float X, float Y, float Z)
{
	int32 Result = Vertices.Num();
	Vertices.Emplace(X,Y,Z);
	return Result;
}

void UEditorBrushBuilder::Poly3i(int32 Direction, int32 i, int32 j, int32 k, FName ItemName, bool bIsTwoSidedNonSolid )
{
	FBuilderPoly& Poly = Polys.AddDefaulted_GetRef();
	Poly.Direction=Direction;
	Poly.ItemName=ItemName;
	Poly.VertexIndices.Add(i);
	Poly.VertexIndices.Add(j);
	Poly.VertexIndices.Add(k);
	Poly.PolyFlags = PF_DefaultFlags | (bIsTwoSidedNonSolid ? (PF_TwoSided|PF_NotSolid) : 0);
}

void UEditorBrushBuilder::Poly4i(int32 Direction, int32 i, int32 j, int32 k, int32 l, FName ItemName, bool bIsTwoSidedNonSolid )
{
	FBuilderPoly& Poly = Polys.AddDefaulted_GetRef();
	Poly.Direction=Direction;
	Poly.ItemName=ItemName;
	Poly.VertexIndices.Add(i);
	Poly.VertexIndices.Add(j);
	Poly.VertexIndices.Add(k);
	Poly.VertexIndices.Add(l);
	Poly.PolyFlags = PF_DefaultFlags | (bIsTwoSidedNonSolid ? (PF_TwoSided|PF_NotSolid) : 0);
}

void UEditorBrushBuilder::PolyBegin(int32 Direction, FName ItemName)
{
	FBuilderPoly& Poly = Polys.AddDefaulted_GetRef();
	Poly.ItemName=ItemName;
	Poly.Direction = Direction;
	Poly.PolyFlags = PF_DefaultFlags;
}

void UEditorBrushBuilder::Polyi(int32 i)
{
	Polys.Last().VertexIndices.Add(i);
}

void UEditorBrushBuilder::PolyEnd()
{
}

bool UEditorBrushBuilder::Build( UWorld* InWorld, ABrush* InBrush )
{
	return false;
}

void UEditorBrushBuilder::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!GIsTransacting)
	{
		// Rebuild brush on property change
		ABrush* Brush = Cast<ABrush>(GetOuter());
		UWorld* BrushWorld = Brush ? Brush->GetWorld() : nullptr;
		if (BrushWorld)
		{
			Brush->bInManipulation = PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive;
			Build(BrushWorld, Brush);
		}
	}
}

UConeBuilder::UConeBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Cone;
		FConstructorStatics()
			: NAME_Cone(TEXT("Cone"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Z = 300.0f;
	CapZ = 290.0f;
	OuterRadius = 200.0f;
	InnerRadius = 190.0f;
	Sides = 8;
	GroupName = ConstructorStatics.NAME_Cone;
	AlignToSide = true;
	Hollow = false;
	BitmapFilename = TEXT("Btn_Cone");
	ToolTip = TEXT("BrushBuilderName_Cone");
}

void UConeBuilder::BuildCone( int32 Direction, bool InAlignToSide, int32 InSides, float InZ, float Radius, FName Item )
{
	int32 n = GetVertexCount();
	int32 Ofs = 0;
	if( InAlignToSide )
	{
		Radius /= FMath::Cos(PI/InSides);
		Ofs = 1;
	}

	// Vertices.
	for( int32 i=0; i<InSides; i++ )
		Vertex3f( Radius*FMath::Sin((2*i+Ofs)*PI/InSides), Radius*FMath::Cos((2*i+Ofs)*PI/InSides), 0 );
	Vertex3f( 0, 0, InZ );

	// Polys.
	for( int32 i=0; i<InSides; i++ )
		Poly3i( Direction, n+i, n+InSides, n+((i+1)%InSides), Item );
}

void UConeBuilder::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		static FName Name_Z(GET_MEMBER_NAME_CHECKED(UConeBuilder, Z));
		static FName Name_CapZ(GET_MEMBER_NAME_CHECKED(UConeBuilder, CapZ));

		const float ZEpsilon = 0.1f;

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_Z && Z <= CapZ)
		{
			Z = CapZ + ZEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_CapZ && CapZ >= Z)
		{
			CapZ = FMath::Max(0.0f, Z - ZEpsilon);
		}

		static FName Name_OuterRadius(GET_MEMBER_NAME_CHECKED(UConeBuilder, OuterRadius));
		static FName Name_InnerRadius(GET_MEMBER_NAME_CHECKED(UConeBuilder, InnerRadius));

		const float RadiusEpsilon = 0.1f;

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_OuterRadius && OuterRadius <= InnerRadius)
		{
			OuterRadius = InnerRadius + RadiusEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_InnerRadius && InnerRadius >= OuterRadius)
		{
			InnerRadius = FMath::Max(0.0f, OuterRadius - RadiusEpsilon);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UConeBuilder::Build( UWorld* InWorld, ABrush* InBrush )
{
	if( Sides<3 )
		return BadParameters(LOCTEXT("ConeNotEnoughSides", "Not enough sides in cone brush"));
	if( Z<=0 || OuterRadius<=0 )
		return BadParameters(LOCTEXT("ConeInvalidRadius", "Invalid cone brush radius"));
	if( Hollow && (InnerRadius<=0 || InnerRadius>=OuterRadius) )
		return BadParameters(LOCTEXT("ConeInvalidRadius", "Invalid cone brush radius"));
	if( Hollow && CapZ>Z )
		return BadParameters(LOCTEXT("ConeInvalidZ", "Invalid cone brush Z value"));
	if( Hollow && (CapZ==Z && InnerRadius==OuterRadius) )
		return BadParameters(LOCTEXT("ConeInvalidRadius", "Invalid cone brush radius"));

	BeginBrush( false, GroupName );
	BuildCone( +1, AlignToSide, Sides, Z, OuterRadius, FName(TEXT("Top")) );
	if( Hollow )
	{
		BuildCone( -1, AlignToSide, Sides, CapZ, InnerRadius, FName(TEXT("Cap")) );
		if( OuterRadius!=InnerRadius )
			for( int32 i=0; i<Sides; i++ )
				Poly4i( 1, i, ((i+1)%Sides), Sides+1+((i+1)%Sides), Sides+1+i, FName(TEXT("Bottom")) );
	}
	else
	{
		PolyBegin( 1, FName(TEXT("Bottom")) );
		for( int32 i=0; i<Sides; i++ )
			Polyi( i );
		PolyEnd();
	}
	return EndBrush( InWorld, InBrush );
}

UCubeBuilder::UCubeBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Cube;
		FConstructorStatics()
			: NAME_Cube(TEXT("Cube"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	X = 200.0f;
	Y = 200.0f;
	Z = 200.0;
	WallThickness = 10.0f;
	GroupName = ConstructorStatics.NAME_Cube;
	Hollow = false;
	Tessellated = false;
	BitmapFilename = TEXT("Btn_Box");
	ToolTip = TEXT("BrushBuilderName_Cube");
}

void UCubeBuilder::BuildCube( int32 Direction, float dx, float dy, float dz, bool _tessellated )
{
	int32 n = GetVertexCount();

	for( int32 i=-1; i<2; i+=2 )
		for( int32 j=-1; j<2; j+=2 )
			for( int32 k=-1; k<2; k+=2 )
				Vertex3f( i*dx/2, j*dy/2, k*dz/2 );

	// If the user wants a Tessellated cube, create the sides out of tris instead of quads.
	if( _tessellated )
	{
		Poly3i(Direction,n+0,n+1,n+3);
		Poly3i(Direction,n+3,n+2,n+0);
		Poly3i(Direction,n+2,n+3,n+7);
		Poly3i(Direction,n+7,n+6,n+2);
		Poly3i(Direction,n+6,n+7,n+5);
		Poly3i(Direction,n+5,n+4,n+6);
		Poly3i(Direction,n+4,n+5,n+1);
		Poly3i(Direction,n+1,n+0,n+4);
		Poly3i(Direction,n+3,n+1,n+5);
		Poly3i(Direction,n+5,n+7,n+3);
		Poly3i(Direction,n+0,n+2,n+6);
		Poly3i(Direction,n+6,n+4,n+0);
	}
	else
	{
		Poly4i(Direction,n+0,n+1,n+3,n+2);
		Poly4i(Direction,n+2,n+3,n+7,n+6);
		Poly4i(Direction,n+6,n+7,n+5,n+4);
		Poly4i(Direction,n+4,n+5,n+1,n+0);
		Poly4i(Direction,n+3,n+1,n+5,n+7);
		Poly4i(Direction,n+0,n+2,n+6,n+4);
	}
}

void UCubeBuilder::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		static FName Name_X(GET_MEMBER_NAME_CHECKED(UCubeBuilder, X));
		static FName Name_Y(GET_MEMBER_NAME_CHECKED(UCubeBuilder, Y));
		static FName Name_Z(GET_MEMBER_NAME_CHECKED(UCubeBuilder, Z));
		static FName Name_WallThickness(GET_MEMBER_NAME_CHECKED(UCubeBuilder, WallThickness));

		const float ThicknessEpsilon = 0.1f;

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_X && X <= WallThickness)
		{
			X = WallThickness + ThicknessEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_Y && Y <= WallThickness)
		{
			Y = WallThickness + ThicknessEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_Z && Z <= WallThickness)
		{
			Z = WallThickness + ThicknessEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_WallThickness && WallThickness >= FMath::Min3(X, Y, Z))
		{
			WallThickness = FMath::Max(0.0f, FMath::Min3(X, Y, Z) - ThicknessEpsilon);
		}

		static FName Name_Hollow(GET_MEMBER_NAME_CHECKED(UCubeBuilder, Hollow));
		static FName Name_Tesselated(GET_MEMBER_NAME_CHECKED(UCubeBuilder, Tessellated));

		if (PropertyChangedEvent.Property->GetFName() == Name_Hollow && Hollow && Tessellated)
		{
			Hollow = false;
		}

		if (PropertyChangedEvent.Property->GetFName() == Name_Tesselated && Hollow && Tessellated)
		{
			Tessellated = false;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UCubeBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( Z<=0 || Y<=0 || X<=0 )
		return BadParameters(LOCTEXT("CubeInvalidDimensions", "Invalid cube dimensions"));
	if( Hollow && (Z<=WallThickness || Y<=WallThickness || X<=WallThickness) )
		return BadParameters(LOCTEXT("CubeInvalidWallthickness", "Invalid cube wall thickness"));
	if( Hollow && Tessellated )
		return BadParameters(LOCTEXT("TessellatedIncompatibleWithHollow", "The 'Tessellated' option can't be specified with the 'Hollow' option."));

	BeginBrush( false, GroupName );
	BuildCube( +1, X, Y, Z, Tessellated );
	if( Hollow )
		BuildCube( -1, X-WallThickness, Y-WallThickness, Z-WallThickness, Tessellated );
	return EndBrush( InWorld, InBrush );
}

UCurvedStairBuilder::UCurvedStairBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_CStair;
		FConstructorStatics()
			: NAME_CStair(TEXT("CStair"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	InnerRadius = 200;
	StepHeight = 20;
	StepWidth = 200;
	AngleOfCurve = 90;
	NumSteps = 10;
	GroupName = ConstructorStatics.NAME_CStair;
	CounterClockwise = false;
	AddToFirstStep = 0;
	BitmapFilename = TEXT("Btn_CurvedStairs");
	ToolTip = TEXT("BrushBuilderName_CurvedStair");
}

void UCurvedStairBuilder::BuildCurvedStair( int32 Direction )
{
	FRotator RotStep(ForceInit);
	RotStep.Yaw = static_cast<float>(AngleOfCurve) / NumSteps;

	if( CounterClockwise )
	{
		RotStep.Yaw *= -1;
		Direction *= -1;
	}

	// Generate the inner curve points.
	int32 InnerStart = GetVertexCount();
	int32 Adjustment;
	FVector vtx(ForceInit);
	FVector NewVtx;
	vtx.X = InnerRadius;
	for( int32 x = 0 ; x < (NumSteps + 1) ; x++ )
	{
		if( x == 0 )
			Adjustment = AddToFirstStep;
		else
			Adjustment = 0;

		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);

		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z - Adjustment );
		vtx.Z += StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
	}

	// Generate the outer curve points.
	int32 OuterStart = GetVertexCount();
	vtx.X = InnerRadius + StepWidth;
	vtx.Z = 0;
	for( int32 x = 0 ; x < (NumSteps + 1) ; x++ )
	{
		if( x == 0 )
			Adjustment = AddToFirstStep;
		else
			Adjustment = 0;

		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);

		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z - Adjustment );
		vtx.Z += StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
	}

	// Generate the bottom inner curve points.
	int32 BottomInnerStart = GetVertexCount();
	vtx.X = InnerRadius;
	vtx.Z = 0;
	for( int32 x = 0 ; x < (NumSteps + 1) ; x++ )
	{
		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z - AddToFirstStep );
	}

	// Generate the bottom outer curve points.
	int32 BottomOuterStart = GetVertexCount();
	vtx.X = InnerRadius + StepWidth;
	for( int32 x = 0 ; x < (NumSteps + 1) ; x++ )
	{
		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z - AddToFirstStep );
	}

	for( int32 x = 0 ; x < NumSteps ; x++ )
	{
		Poly4i( Direction, InnerStart + (x * 2) + 2, InnerStart + (x * 2) + 1, OuterStart + (x * 2) + 1, OuterStart + (x * 2) + 2, FName(TEXT("steptop")) );
		Poly4i( Direction, InnerStart + (x * 2) + 1, InnerStart + (x * 2), OuterStart + (x * 2), OuterStart + (x * 2) + 1, FName(TEXT("stepfront")) );
		Poly4i( Direction, BottomInnerStart + x, InnerStart + (x * 2) + 1, InnerStart + (x * 2) + 2, BottomInnerStart + x + 1, FName(TEXT("innercurve")) );
		Poly4i( Direction, OuterStart + (x * 2) + 1, BottomOuterStart + x, BottomOuterStart + x + 1, OuterStart + (x * 2) + 2, FName(TEXT("outercurve")) );
		Poly4i( Direction, BottomInnerStart + x, BottomInnerStart + x + 1, BottomOuterStart + x + 1, BottomOuterStart + x, FName(TEXT("Bottom")) );
	}

	// Back panel.
	Poly4i( Direction, BottomInnerStart + NumSteps, InnerStart + (NumSteps * 2), OuterStart + (NumSteps * 2), BottomOuterStart + NumSteps, FName(TEXT("back")) );
}

bool UCurvedStairBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( AngleOfCurve<1 || AngleOfCurve>360 )
		return BadParameters(LOCTEXT("StairAngleOutOfRange", "Angle is out of range."));
	if( InnerRadius<1 || StepWidth<1 || NumSteps<1 )
		return BadParameters(LOCTEXT("StairInvalidStepParams", "Invalid step parameters."));

	BeginBrush( false, GroupName );
	BuildCurvedStair( +1 );
	return EndBrush( InWorld, InBrush );
}

UCylinderBuilder::UCylinderBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Cylinder;
		FConstructorStatics()
			: NAME_Cylinder(TEXT("Cylinder"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Z = 200.0f;
	OuterRadius = 200.0f;
	InnerRadius = 190.0f;
	Sides = 8;
	GroupName = ConstructorStatics.NAME_Cylinder;
	AlignToSide = true;
	Hollow = false;
	BitmapFilename = TEXT("Btn_Cylinder");
	ToolTip = TEXT("BrushBuilderName_Cylinder");
}

void UCylinderBuilder::BuildCylinder( int32 Direction, bool InAlignToSide, int32 InSides, float InZ, float Radius )
{
	int32 n = GetVertexCount();
	int32 Ofs = 0;
	if( InAlignToSide )
	{
		Radius /= FMath::Cos(PI/InSides);
		Ofs = 1;
	}

	// Vertices.
	for( int32 i=0; i<InSides; i++ )
		for( int32 j=-1; j<2; j+=2 )
			Vertex3f( Radius*FMath::Sin((2*i+Ofs)*PI/InSides), Radius*FMath::Cos((2*i+Ofs)*PI/InSides), j*InZ/2 );

	// Polys.
	for( int32 i=0; i<InSides; i++ )
		Poly4i( Direction, n+i*2, n+i*2+1, n+((i*2+3)%(2*InSides)), n+((i*2+2)%(2*InSides)), FName(TEXT("Wall")) );
}

void UCylinderBuilder::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		static FName Name_OuterRadius(GET_MEMBER_NAME_CHECKED(UCylinderBuilder, OuterRadius));
		static FName Name_InnerRadius(GET_MEMBER_NAME_CHECKED(UCylinderBuilder, InnerRadius));

		const float RadiusEpsilon = 0.1f;

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_OuterRadius && OuterRadius <= InnerRadius)
		{
			OuterRadius = InnerRadius + RadiusEpsilon;
		}

		if (Hollow && PropertyChangedEvent.Property->GetFName() == Name_InnerRadius && InnerRadius >= OuterRadius)
		{
			InnerRadius = FMath::Max(0.0f, OuterRadius - RadiusEpsilon);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UCylinderBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( Sides<3 )
		return BadParameters(LOCTEXT("CylinderInvalidSides", "Not enough cylinder sides."));
	if( Z<=0 || OuterRadius<=0 )
		return BadParameters(LOCTEXT("CylinderInvalidRadius", "Invalid cylinder radius"));
	if( Hollow && (InnerRadius<=0 || InnerRadius>=OuterRadius) )
		return BadParameters(LOCTEXT("CylinderInvalidRadius", "Invalid cylinder radius"));

	BeginBrush( false, GroupName );
	BuildCylinder( +1, AlignToSide, Sides, Z, OuterRadius );
	if( Hollow )
	{
		BuildCylinder( -1, AlignToSide, Sides, Z, InnerRadius );
		for( int32 j=-1; j<2; j+=2 )
			for( int32 i=0; i<Sides; i++ )
				Poly4i( j, i*2+(1-j)/2, ((i+1)%Sides)*2+(1-j)/2, ((i+1)%Sides)*2+(1-j)/2+Sides*2, i*2+(1-j)/2+Sides*2, FName(TEXT("Cap")) );
	}
	else
	{
		for( int32 j=-1; j<2; j+=2 )
		{
			PolyBegin( j, FName(TEXT("Cap")) );
			for( int32 i=0; i<Sides; i++ )
				Polyi( i*2+(1-j)/2 );
			PolyEnd();
		}
	}
	return EndBrush( InWorld, InBrush );
}

ULinearStairBuilder::ULinearStairBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_LinearStair;
		FConstructorStatics()
			: NAME_LinearStair(TEXT("LinearStair"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	StepLength = 30;
	StepHeight = 20;
	StepWidth = 200;
	NumSteps = 10;
	AddToFirstStep = 0;
	GroupName = ConstructorStatics.NAME_LinearStair;
	BitmapFilename = TEXT("Btn_Staircase");
	ToolTip = TEXT("BrushBuilderName_LinearStair");
}

bool ULinearStairBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	// Check for bad input.
	if( StepLength<=0 || StepHeight<=0 || StepWidth<=0 )
		return BadParameters(LOCTEXT("LinearStairInvalidStepParams", "Invalid step parameters."));
	if( NumSteps<=1 || NumSteps>45 )
		return BadParameters(LOCTEXT("LinearStairNumStepsOutOfRange", "NumSteps must be greater than 1 and less than 46.f"));


	// Build the brush.

	BeginBrush( false, GroupName );

	int32 CurrentX = 0;
	int32 CurrentY = 0;
	int32 CurrentZ = 0;

	int32 LastIdx = GetVertexCount();

	// Bottom poly.
	Vertex3f( 0,						0,			-StepHeight );
	Vertex3f( 0,						StepWidth,	-StepHeight );
	Vertex3f( StepLength * NumSteps,	StepWidth,	-StepHeight );
	Vertex3f( StepLength * NumSteps,	0,			-StepHeight );
	Poly4i(1, 0, 1, 2, 3, FName(TEXT("Base")));
	LastIdx += 4;

	// Back poly.
	Vertex3f( StepLength * NumSteps,	StepWidth,	-StepHeight );
	Vertex3f( StepLength * NumSteps,	StepWidth,	(StepHeight * (NumSteps - 1)) + AddToFirstStep );
	Vertex3f( StepLength * NumSteps,	0,			(StepHeight * (NumSteps - 1)) + AddToFirstStep );
	Vertex3f( StepLength * NumSteps,	0,			-StepHeight );
	Poly4i(1, 4, 5, 6, 7, FName(TEXT("Back")));
	LastIdx += 4;

	// Tops of steps.
	for( int32 i = 0 ; i < NumSteps ; i++ )
	{
		CurrentX = (i * StepLength);
		CurrentZ = (i * StepHeight) + AddToFirstStep;

		// Top of the step
		Vertex3f( CurrentX,					CurrentY,				CurrentZ );
		Vertex3f( CurrentX,					CurrentY + StepWidth,	CurrentZ );
		Vertex3f( CurrentX + StepLength,	CurrentY + StepWidth,	CurrentZ );
		Vertex3f( CurrentX + StepLength,	CurrentY,				CurrentZ );

		Poly4i(1,
			LastIdx+(i*4)+3,
			LastIdx+(i*4)+2,
			LastIdx+(i*4)+1,
			LastIdx+(i*4), FName(TEXT("Step")));
	}
	LastIdx += (NumSteps*4);

	// Fronts of steps.
	for( int32 i = 0 ; i < NumSteps ; i++ )
	{
		CurrentX = (i * StepLength);
		CurrentZ = (i * StepHeight) + AddToFirstStep;
		int32 Adjustment = 0;
		if( i == 0 )
			Adjustment = AddToFirstStep;

		// Top of the step
		Vertex3f( CurrentX,		CurrentY,				CurrentZ );
		Vertex3f( CurrentX,		CurrentY,				CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX,		CurrentY + StepWidth,	CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX,		CurrentY + StepWidth,	CurrentZ );

		Poly4i(1,
			LastIdx+(i*12)+3,
			LastIdx+(i*12)+2,
			LastIdx+(i*12)+1,
			LastIdx+(i*12), FName(TEXT("Rise")));

		// Sides of the step
		Vertex3f( CurrentX,								CurrentY,		CurrentZ );
		Vertex3f( CurrentX,								CurrentY,		CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX + (StepLength*(NumSteps-i)),	CurrentY,		CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX + (StepLength*(NumSteps-i)),	CurrentY,		CurrentZ );

		Poly4i(1,
			LastIdx+(i*12)+4,
			LastIdx+(i*12)+5,
			LastIdx+(i*12)+6,
			LastIdx+(i*12)+7, FName(TEXT("Side")));

		Vertex3f( CurrentX,								CurrentY + StepWidth,		CurrentZ );
		Vertex3f( CurrentX,								CurrentY + StepWidth,		CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX + (StepLength*(NumSteps-i)),	CurrentY + StepWidth,		CurrentZ - StepHeight - Adjustment );
		Vertex3f( CurrentX + (StepLength*(NumSteps-i)),	CurrentY + StepWidth,		CurrentZ );

		Poly4i(1,
			LastIdx+(i*12)+11,
			LastIdx+(i*12)+10,
			LastIdx+(i*12)+9,
			LastIdx+(i*12)+8, FName(TEXT("Side")));
	}

	return EndBrush( InWorld, InBrush );
}

USheetBuilder::USheetBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Sheet;
		FConstructorStatics()
			: NAME_Sheet(TEXT("Sheet"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	X = 256;
	Y = 256;
	XSegments = 1;
	YSegments = 1;
	Axis = AX_Horizontal;
	GroupName = ConstructorStatics.NAME_Sheet;
	BitmapFilename = TEXT("Btn_Sheet");
	ToolTip = TEXT("BrushBuilderName_Sheet");
}

bool USheetBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( Y<=0 || X<=0 || XSegments<=0 || YSegments<=0 )
		return BadParameters(LOCTEXT("SheetInvalidParams", "Invalid sheet parameters."));

	BeginBrush( false, GroupName );
	int32 XStep = X/XSegments;
	int32 YStep = Y/YSegments;

	int32 count = 0;
	for( int32 i = 0 ; i < XSegments ; i++ )
	{
		for( int32 j = 0 ; j < YSegments ; j++ )
		{
			if( Axis==AX_Horizontal )
			{
				Vertex3f(  (i*XStep)-X/2, (j*YStep)-Y/2, 0 );
				Vertex3f(  (i*XStep)-X/2, ((j+1)*YStep)-Y/2, 0 );
				Vertex3f(  ((i+1)*XStep)-X/2, ((j+1)*YStep)-Y/2, 0 );
				Vertex3f(  ((i+1)*XStep)-X/2, (j*YStep)-Y/2, 0 );
			}
			else if( Axis==AX_XAxis )
			{
				Vertex3f(  0, (i*XStep)-X/2, (j*YStep)-Y/2 );
				Vertex3f(  0, (i*XStep)-X/2, ((j+1)*YStep)-Y/2 );
				Vertex3f(  0, ((i+1)*XStep)-X/2, ((j+1)*YStep)-Y/2 );
				Vertex3f(  0, ((i+1)*XStep)-X/2, (j*YStep)-Y/2 );
			}
			else
			{
				Vertex3f(  (i*XStep)-X/2, 0, (j*YStep)-Y/2 );
				Vertex3f(  (i*XStep)-X/2, 0, ((j+1)*YStep)-Y/2 );
				Vertex3f(  ((i+1)*XStep)-X/2, 0, ((j+1)*YStep)-Y/2 );
				Vertex3f(  ((i+1)*XStep)-X/2, 0, (j*YStep)-Y/2 );
			}

			Poly4i(+1,count,count+1,count+2,count+3,FName(TEXT("Sheet")),true);
			count = GetVertexCount();
		}
	}

	return EndBrush( InWorld, InBrush );
}

USpiralStairBuilder::USpiralStairBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Spiral;
		FConstructorStatics()
			: NAME_Spiral(TEXT("Spiral"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	InnerRadius = 100;
	StepWidth = 200;
	StepHeight = 20;
	StepThickness = 50;
	NumStepsPer360 = 16;
	NumSteps = 16;
	SlopedCeiling = false;
	SlopedFloor = false;
	GroupName = ConstructorStatics.NAME_Spiral;
	CounterClockwise = false;
	BitmapFilename = TEXT("Btn_SpiralStairs");
	ToolTip = TEXT("BrushBuilderName_SpiralStair");
}

void USpiralStairBuilder::BuildCurvedStair( int32 Direction )
{
	FRotator RotStep(ForceInit);
	RotStep.Yaw = 360.0f / NumStepsPer360;
	if( CounterClockwise )
	{
		RotStep.Yaw *= -1;
		Direction *= -1;
	}

	// Generate the vertices for the first stair.
	FVector Template[8];
	int32 idx = 0;
	int32 VertexStart = GetVertexCount();
	FVector vtx(ForceInit);
	vtx.X = InnerRadius;
	FVector NewVtx;
	for( int32 x = 0 ; x < 2 ; x++ )
	{
		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);
		
		vtx.Z = 0;
		if( SlopedCeiling && x == 1 )
			vtx.Z = StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
		Template[idx].X = NewVtx.X;		Template[idx].Y = NewVtx.Y;		Template[idx].Z = vtx.Z;		idx++;

		vtx.Z = StepThickness;
		if( SlopedFloor && x == 0 )
			vtx.Z -= StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
		Template[idx].X = NewVtx.X;		Template[idx].Y = NewVtx.Y;		Template[idx].Z = vtx.Z;		idx++;
	}

	vtx.X = InnerRadius + StepWidth;
	for( int32 x = 0 ; x < 2 ; x++ )
	{
		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);

		vtx.Z = 0;
		if( SlopedCeiling && x == 1 )
			vtx.Z = StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
		Template[idx].X = NewVtx.X;		Template[idx].Y = NewVtx.Y;		Template[idx].Z = vtx.Z;		idx++;

		vtx.Z = StepThickness;
		if( SlopedFloor && x == 0 )
			vtx.Z -= StepHeight;
		Vertex3f( NewVtx.X, NewVtx.Y, vtx.Z );
		Template[idx].X = NewVtx.X;		Template[idx].Y = NewVtx.Y;		Template[idx].Z = vtx.Z;		idx++;
	}

	// Create steps from the template
	for( int32 x = 0 ; x < NumSteps ; x++ )
	{
		if( SlopedFloor )
		{
			Poly3i( Direction, VertexStart + 3, VertexStart + 1, VertexStart + 5, FName(TEXT("steptop")) );
			Poly3i( Direction, VertexStart + 3, VertexStart + 5, VertexStart + 7, FName(TEXT("steptop")) );
		}
		else
			Poly4i( Direction, VertexStart + 3, VertexStart + 1, VertexStart + 5, VertexStart + 7, FName(TEXT("steptop")) );

		Poly4i( Direction, VertexStart + 0, VertexStart + 1, VertexStart + 3, VertexStart + 2, FName(TEXT("inner")) );
		Poly4i( Direction, VertexStart + 5, VertexStart + 4, VertexStart + 6, VertexStart + 7, FName(TEXT("GetOuter()")) );
		Poly4i( Direction, VertexStart + 1, VertexStart + 0, VertexStart + 4, VertexStart + 5, FName(TEXT("stepfront")) );
		Poly4i( Direction, VertexStart + 2, VertexStart + 3, VertexStart + 7, VertexStart + 6, FName(TEXT("stepback")) );

		if( SlopedCeiling )
		{
			Poly3i( Direction, VertexStart + 0, VertexStart + 2, VertexStart + 6, FName(TEXT("stepbottom")) );
			Poly3i( Direction, VertexStart + 0, VertexStart + 6, VertexStart + 4, FName(TEXT("stepbottom")) );
		}
		else
			Poly4i( Direction, VertexStart + 0, VertexStart + 2, VertexStart + 6, VertexStart + 4, FName(TEXT("stepbottom")) );

		VertexStart = GetVertexCount();
		for( int32 y = 0 ; y < 8 ; y++ )
		{
			NewVtx = FRotationMatrix(RotStep * (x + 1)).TransformVector(Template[y]);
			Vertex3f( NewVtx.X, NewVtx.Y, NewVtx.Z + (StepHeight * (x + 1)) );
		}
	}
}

bool USpiralStairBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( InnerRadius<1 || StepWidth<1 || NumSteps<1 || NumStepsPer360<3 )
		return BadParameters(LOCTEXT("SpiralStairInvalidStepParams", "Invalid step parameters."));

	BeginBrush( false, GroupName );
	BuildCurvedStair( +1 );
	return EndBrush( InWorld, InBrush );
}

UTetrahedronBuilder::UTetrahedronBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Tetrahedron;
		FConstructorStatics()
			: NAME_Tetrahedron(TEXT("Tetrahedron"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Radius = 256.0f;
	SphereExtrapolation = 2;
	GroupName = ConstructorStatics.NAME_Tetrahedron;
	BitmapFilename = TEXT("Btn_Sphere");
	ToolTip = TEXT("BrushBuilderName_Tetrahedron");
}

void UTetrahedronBuilder::Extrapolate( int32 a, int32 b, int32 c, int32 Count, float InRadius )
{
	if( Count>1 )
	{
		int32 ab=Vertexv( InRadius*(GetVertex(a)+GetVertex(b)).GetSafeNormal() );
		int32 bc=Vertexv( InRadius*(GetVertex(b)+GetVertex(c)).GetSafeNormal() );
		int32 ca=Vertexv( InRadius*(GetVertex(c)+GetVertex(a)).GetSafeNormal() );
		Extrapolate(a,ab,ca,Count-1,InRadius);
		Extrapolate(b,bc,ab,Count-1,InRadius);
		Extrapolate(c,ca,bc,Count-1,InRadius);
		Extrapolate(ab,bc,ca,Count-1,InRadius);
		//wastes shared vertices
	}
	else Poly3i(+1,a,b,c);
}

void UTetrahedronBuilder::BuildTetrahedron( float R, int32 InSphereExtrapolation )
{
	Vertex3f( R,0,0);
	Vertex3f(-R,0,0);
	Vertex3f(0, R,0);
	Vertex3f(0,-R,0);
	Vertex3f(0,0, R);
	Vertex3f(0,0,-R);

	Extrapolate(2,1,4,InSphereExtrapolation,Radius);
	Extrapolate(1,3,4,InSphereExtrapolation,Radius);
	Extrapolate(3,0,4,InSphereExtrapolation,Radius);
	Extrapolate(0,2,4,InSphereExtrapolation,Radius);
	Extrapolate(1,2,5,InSphereExtrapolation,Radius);
	Extrapolate(3,1,5,InSphereExtrapolation,Radius);
	Extrapolate(0,3,5,InSphereExtrapolation,Radius);
	Extrapolate(2,0,5,InSphereExtrapolation,Radius);
}

bool UTetrahedronBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( Radius<=0 || SphereExtrapolation<=0 )
		return BadParameters(LOCTEXT("TetrahedronInvalidParams", "Invalid tetrahedron parameters."));
	if( SphereExtrapolation > 5 )
		return BadParameters(LOCTEXT("TetrahedronSphereExtrapolationTooLarge", "Setting 'SphereExtrapolation' to more than 5 is invalid.  The resulting ABrush* will have too many polygons to be useful."));

	BeginBrush( false, GroupName );
	BuildTetrahedron( Radius, SphereExtrapolation );
	return EndBrush( InWorld, InBrush );
}


UVolumetricBuilder::UVolumetricBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FName NAME_Volumetric;
		FConstructorStatics()
			: NAME_Volumetric(TEXT("Volumetric"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	Z = 128.0f;
	Radius = 64.0f;
	NumSheets = 2;
	GroupName = ConstructorStatics.NAME_Volumetric;
	BitmapFilename = TEXT("Btn_Volumetric");
	ToolTip = TEXT("BrushBuilderName_Volumetric");
}

void UVolumetricBuilder::BuildVolumetric( int32 Direction, int32 InNumSheets, float InZ, float InRadius )
{
	int32 n = GetVertexCount();
	FRotator RotStep(ForceInit);
	RotStep.Yaw = 360.f / (InNumSheets * 2);

	// Vertices.
	FVector NewVtx;
	FVector vtx(ForceInit);
	vtx.X = Radius;
	vtx.Z = InZ / 2;
	for( int32 x = 0 ; x < (InNumSheets * 2) ; x++ )
	{
		NewVtx = FRotationMatrix(RotStep * x).TransformVector(vtx);
		Vertex3f( NewVtx.X, NewVtx.Y, NewVtx.Z );
		Vertex3f( NewVtx.X, NewVtx.Y, NewVtx.Z - InZ );
	}

	// Polys.
	for( int32 x = 0 ; x < InNumSheets ; x++ )
	{
		int32 y = (x*2) + 1;
		if( y >= (InNumSheets * 2) ) y -= (InNumSheets * 2);
		Poly4i( Direction, n+(x*2), n+y, n+y+(InNumSheets*2), n+(x*2)+(InNumSheets*2), FName(TEXT("Sheets")), true);
	}
}

bool UVolumetricBuilder::Build(UWorld* InWorld, ABrush* InBrush)
{
	if( NumSheets<2 )
		return BadParameters(LOCTEXT("VolumetricInvalidSheets", "Invalid volumetric sheet count."));
	if( Z<=0 || Radius<=0 )
		return BadParameters(LOCTEXT("VolumetricInvalidRadius", "Invalid volumetric radius parameters."));

	BeginBrush( true, GroupName );
	BuildVolumetric( +1, NumSheets, Z, Radius );
	return EndBrush( InWorld, InBrush );
}

#undef LOCTEXT_NAMESPACE
