// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Bsp.cpp: Unreal engine Bsp-related functions.
=============================================================================*/

#include "CoreMinimal.h"
#include "Materials/MaterialInterface.h"
#include "Model.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "BSPOps.h"
#include "BSPUtils.h"
#include "Engine/Selection.h"


void UEditorEngine::bspBuildFPolys( UModel* Model, bool SurfLinks, int32 iNode, TArray<FPoly>* DestArray )
{
	FBSPUtils::bspBuildFPolys(Model, SurfLinks, iNode, DestArray);
}


void UEditorEngine::bspMergeCoplanars( UModel* Model, bool RemapLinks, bool MergeDisparateTextures )
{
	FBSPUtils::bspMergeCoplanars(Model, RemapLinks, MergeDisparateTextures);
}


void UEditorEngine::bspCleanup( UModel *Model )
{
	FBSPUtils::bspCleanup(Model);
}


int UEditorEngine::bspNodeToFPoly
(
	UModel	*Model,
	int32	    iNode,
	FPoly	*EdPoly
)
{
	return FBSPUtils::bspNodeToFPoly(Model, iNode, EdPoly);
}


int UEditorEngine::bspBrushCSG
(
	ABrush*		Actor, 
	UModel*		Model, 
	uint32		PolyFlags, 
	EBrushType	BrushType,
	ECsgOper	CSGOper, 
	bool		bBuildBounds,
	bool		bMergePolys,
	bool		bReplaceNULLMaterialRefs,
	bool		bShowProgressBar/*=true*/
)
{
	UMaterialInterface* SelectedMaterialInstance = GetSelectedObjects()->GetTop<UMaterialInterface>();
	return FBSPUtils::bspBrushCSG(Actor, Model, TempModel, SelectedMaterialInstance, PolyFlags, BrushType, CSGOper, bBuildBounds, bMergePolys, bReplaceNULLMaterialRefs, bShowProgressBar);
}


void UEditorEngine::bspOptGeom( UModel *Model )
{
	if( GUndo )
		ResetTransaction( NSLOCTEXT("UnrealEd", "GeometryOptimization", "Geometry Optimization") );

	FBSPUtils::bspOptGeom(Model);
}
