// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorBuildUtils.cpp: Utilities for building in the editor
=============================================================================*/

#include "EditorBuildUtils.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeExit.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/Brush.h"
#include "Framework/Application/SlateApplication.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Materials/MaterialInterface.h"
#include "AI/NavigationSystemBase.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "EditorLevelUtils.h"
#include "BusyCursor.h"
#include "Dialogs/SBuildProgress.h"
#include "LightingBuildOptions.h"
#include "AssetToolsModule.h"
#include "Logging/MessageLog.h"
#include "HierarchicalLOD.h"
#include "ActorEditorUtils.h"
#include "MaterialUtilities.h"
#include "UnrealEngine.h"
#include "DebugViewModeHelpers.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "MaterialStatsCommon.h"
#include "Materials/MaterialInstance.h"
#include "UObject/UObjectIterator.h"
#include "VirtualTexturingEditorModule.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "LandscapeEditTypes.h"
#include "LandscapeSubsystem.h"
#include "ShaderCompilerCore.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/SWorldPartitionBuildNavigationDialog.h"
#include "WorldPartition/WorldPartitionBuildNavigationOptions.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionRuntimeVirtualTextureBuilder.h"
#include "AssetCompilingManager.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ShaderCompiler.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorBuildUtils, Log, All);

#define LOCTEXT_NAMESPACE "EditorBuildUtils"

extern UNREALED_API bool GLightmassDebugMode; 
extern UNREALED_API bool GLightmassStatsMode;
extern FSwarmDebugOptions GSwarmDebugOptions;

const FName FBuildOptions::BuildGeometry(TEXT("BuildGeometry"));
const FName FBuildOptions::BuildVisibleGeometry(TEXT("BuildVisibleGeometry"));
const FName FBuildOptions::BuildLighting(TEXT("BuildLighting"));
const FName FBuildOptions::BuildAIPaths(TEXT("BuildAIPaths"));
const FName FBuildOptions::BuildSelectedAIPaths(TEXT("BuildSelectedAIPaths"));
const FName FBuildOptions::BuildAll(TEXT("BuildAll"));
const FName FBuildOptions::BuildAllSubmit(TEXT("BuildAllSubmit"));
const FName FBuildOptions::BuildAllOnlySelectedPaths(TEXT("BuildAllOnlySelectedPaths"));
const FName FBuildOptions::BuildHierarchicalLOD(TEXT("BuildHierarchicalLOD"));
const FName FBuildOptions::BuildHierarchicalLOD_SelectedActors(TEXT("BuildHierarchicalLOD_SelectedActors"));
const FName FBuildOptions::BuildMinimap(TEXT("BuildMinimap"));
const FName FBuildOptions::BuildLandscapeSplineMeshes(TEXT("BuildLandscapeSplineMeshes"));
const FName FBuildOptions::BuildTextureStreaming(TEXT("BuildTextureStreaming"));
const FName FBuildOptions::BuildVirtualTexture(TEXT("BuildVirtualTexture"));
const FName FBuildOptions::BuildAllLandscape(TEXT("BuildAllLandscape"));

bool FEditorBuildUtils::bBuildingNavigationFromUserRequest = false;
TMap<FName, FEditorBuildUtils::FCustomBuildType> FEditorBuildUtils::CustomBuildTypes;
FName FEditorBuildUtils::InProgressBuildId;

namespace UE::EditorBuildUtils
{
	static bool bNavmeshAllowPartitionedBuildingFromEditor = false; // Experimental, not enabled by default yet.
	static FAutoConsoleVariableRef AllowPartitionedBuildingFromEditorDeprecated(TEXT("n.bNavmeshAllowPartitionedBuildingFromEditor"), bNavmeshAllowPartitionedBuildingFromEditor, TEXT("Enable experimental navmesh partition building. Deprecated 5.3: use ai.nav.bNavmeshAllowPartitionedBuildingFromEditor instead."), ECVF_Default);
	static FAutoConsoleVariableRef AllowPartitionedBuildingFromEditor(TEXT("ai.nav.bNavmeshAllowPartitionedBuildingFromEditor"), bNavmeshAllowPartitionedBuildingFromEditor, TEXT("Enable experimental navmesh partition building."), ECVF_Default);
}

/**
 * Class that handles potentially-async Build All requests.
 */
class FBuildAllHandler
{
public:
	void StartBuild(UWorld* World, FName BuildId, const TWeakPtr<SBuildProgressWidget>& BuildProgressWidget);
	void ResumeBuild();
	void AddCustomBuildStep(FName Id, FName InsertBefore);
	void RemoveCustomBuildStep(FName Id);

	static FBuildAllHandler& Get()
	{
		static FBuildAllHandler Instance;
		return Instance;
	}

private:
	FBuildAllHandler();
	FBuildAllHandler(const FBuildAllHandler&);

	void ProcessBuild(const TWeakPtr<SBuildProgressWidget>& BuildProgressWidget);
	void BuildFinished();

	TArray<FName> BuildSteps;
	int32 CurrentStep;

	UWorld* CurrentWorld;
	FName CurrentBuildId;
};

/** Constructor */
FEditorBuildUtils::FEditorAutomatedBuildSettings::FEditorAutomatedBuildSettings()
:	BuildErrorBehavior( ABB_PromptOnError ),
	UnableToCheckoutFilesBehavior( ABB_PromptOnError ),
	NewMapBehavior( ABB_PromptOnError ),
	FailedToSaveBehavior( ABB_PromptOnError ),
	bUseSCC( true ),
	bAutoAddNewFiles( true ),
	bShutdownEditorOnCompletion( false )
{}

/**
 * Start an automated build of all current maps in the editor. Upon successful conclusion of the build, the newly
 * built maps will be submitted to source control.
 *
 * @param	BuildSettings		Build settings used to dictate the behavior of the automated build
 * @param	OutErrorMessages	Error messages accumulated during the build process, if any
 *
 * @return	true if the build/submission process executed successfully; false if it did not
 */

bool FEditorBuildUtils::EditorAutomatedBuildAndSubmit( const FEditorAutomatedBuildSettings& BuildSettings, FText& OutErrorMessages )
{
	// Assume the build is successful to start
	bool bBuildSuccessful = true;
	
	// Keep a set of packages that should be submitted to source control at the end of a successful build. The build preparation and processing
	// will add and remove from the set depending on build settings, errors, etc.
	TSet<UPackage*> PackagesToSubmit;

	// Perform required preparations for the automated build process
	bBuildSuccessful = PrepForAutomatedBuild( BuildSettings, PackagesToSubmit, OutErrorMessages );

	// If the preparation went smoothly, attempt the actual map building process
	if ( bBuildSuccessful )
	{
		bBuildSuccessful = EditorBuild( GWorld, FBuildOptions::BuildAllSubmit );

		// If the map build failed, log the error
		if ( !bBuildSuccessful )
		{
			LogErrorMessage( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_BuildFailed", "The map build failed or was canceled."), OutErrorMessages );
		}

		// If we are going to shutdown after this has run (ie running from the cmdline) then we should wait for the distributed lighting build to complete.
		if (BuildSettings.bShutdownEditorOnCompletion)
		{
			while (GUnrealEd->IsLightingBuildCurrentlyRunning())
			{
				GUnrealEd->UpdateBuildLighting();
			}
		}	
	}

	// If any map errors resulted from the build, process them according to the behavior specified in the build settings
	if ( bBuildSuccessful && FMessageLog("MapCheck").NumMessages( EMessageSeverity::Warning ) > 0 )
	{
		bBuildSuccessful = ProcessAutomatedBuildBehavior( BuildSettings.BuildErrorBehavior, NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_MapErrors", "Map errors occurred while building.\n\nAttempt to continue the build?"), OutErrorMessages );
	}

	// If it's still safe to proceed, attempt to save all of the level packages that have been marked for submission
	if ( bBuildSuccessful )
	{
		UPackage* CurOutermostPkg = GWorld->PersistentLevel->GetOutermost();
		FString PackagesThatFailedToSave;

		// Try to save the p-level if it should be submitted
		if ( PackagesToSubmit.Contains( CurOutermostPkg ) && !FEditorFileUtils::SaveLevel( GWorld->PersistentLevel ) )
		{
			// If the p-level failed to save, remove it from the set of packages to submit
			PackagesThatFailedToSave += FString::Printf( TEXT("%s\n"), *CurOutermostPkg->GetName() );
			PackagesToSubmit.Remove( CurOutermostPkg );
		}
		
		// Try to save each streaming level (if they should be submitted)
		for (ULevelStreaming* CurStreamingLevel : GWorld->GetStreamingLevels())
		{
			if (CurStreamingLevel)
			{			
				if (ULevel* Level = CurStreamingLevel->GetLoadedLevel())
				{
					CurOutermostPkg = Level->GetOutermost();
					if ( PackagesToSubmit.Contains( CurOutermostPkg ) && !FEditorFileUtils::SaveLevel( Level ) )
					{
						// If a save failed, remove the streaming level from the set of packages to submit
						PackagesThatFailedToSave += FString::Printf( TEXT("%s\n"), *CurOutermostPkg->GetName() );
						PackagesToSubmit.Remove( CurOutermostPkg );
					}
				}
			}
		}

		// If any packages failed to save, process the behavior specified by the build settings to see how the process should proceed
		if ( PackagesThatFailedToSave.Len() > 0 )
		{
			bBuildSuccessful = ProcessAutomatedBuildBehavior( BuildSettings.FailedToSaveBehavior,
				FText::Format( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_FilesFailedSave", "The following assets failed to save and cannot be submitted:\n\n{0}\n\nAttempt to continue the build?"), FText::FromString(PackagesThatFailedToSave) ),
				OutErrorMessages );
		}
	}

	// If still safe to proceed, make sure there are actually packages remaining to submit
	if ( bBuildSuccessful )
	{
		bBuildSuccessful = PackagesToSubmit.Num() > 0;
		if ( !bBuildSuccessful )
		{
			LogErrorMessage( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_NoValidLevels", "None of the current levels are valid for submission; automated build aborted."), OutErrorMessages );
		}
	}

	// Finally, if everything has gone smoothly, submit the requested packages to source control
	if ( bBuildSuccessful && BuildSettings.bUseSCC )
	{
		SubmitPackagesForAutomatedBuild( PackagesToSubmit, BuildSettings );
	}

	// Check if the user requested the editor shutdown at the conclusion of the automated build
	if ( BuildSettings.bShutdownEditorOnCompletion )
	{
		FPlatformMisc::RequestExit( false );
	}

	return bBuildSuccessful;
}

static bool IsBuildCancelled()
{
	return GEditor->GetMapBuildCancelled();
}


bool FEditorBuildUtils::EditorCanBuild( UWorld* InWorld, FName Id )
{
	// Only process custom types with conditional execution.
	// Preserving legacy behavior otherwise.
	if (CustomBuildTypes.Contains(Id))
	{
		const FCustomBuildType& CustomBuild = CustomBuildTypes.FindChecked(Id);
		if (CustomBuild.CanDoBuild.IsBound())
		{
			return CustomBuild.CanDoBuild.Execute(InWorld, Id);
		}
	}

	return true;
}

/**
 * Perform an editor build with behavior dependent upon the specified id
 *
 * @param	Id	Action Id specifying what kind of build is requested
 *
 * @return	true if the build completed successfully; false if it did not (or was manually canceled)
 */
bool FEditorBuildUtils::EditorBuild( UWorld* InWorld, FName Id, const bool bAllowLightingDialog )
{
	FMessageLog("MapCheck").NewPage(LOCTEXT("MapCheckNewPage", "Map Check"));

	// Make sure to set this flag to false before ALL builds.
	GEditor->SetMapBuildCancelled( false );

	// Will be set to false if, for some reason, the build does not happen.
	bool bDoBuild = true;
	// Indicates whether the persistent level should be dirtied at the end of a build.
	bool bDirtyPersistentLevel = true;

	// Hack: These don't initialize properly and if you pick BuildAll right off the
	// bat when opening a map you will get incorrect values in them.
	GSwarmDebugOptions.Touch();

	// Show option dialog first, before showing the DlgBuildProgress window.
	FLightingBuildOptions LightingBuildOptions;
	if ( Id == FBuildOptions::BuildLighting )
	{
		// Retrieve settings from ini.
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelected"),		LightingBuildOptions.bOnlyBuildSelected,			GEditorPerProjectIni );
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildCurrentLevel"),	LightingBuildOptions.bOnlyBuildCurrentLevel,		GEditorPerProjectIni );
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelectedLevels"),LightingBuildOptions.bOnlyBuildSelectedLevels,	GEditorPerProjectIni );
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("OnlyBuildVisibility"),	LightingBuildOptions.bOnlyBuildVisibility,		GEditorPerProjectIni );
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"),		LightingBuildOptions.bUseErrorColoring,			GEditorPerProjectIni );
		GConfig->GetBool( TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"),	LightingBuildOptions.bShowLightingBuildInfo,		GEditorPerProjectIni );
		int32 QualityLevel;
		GConfig->GetInt(  TEXT("LightingBuildOptions"), TEXT("QualityLevel"),			QualityLevel,						GEditorPerProjectIni );
		QualityLevel = FMath::Clamp<int32>(QualityLevel, Quality_Preview, Quality_Production);
		LightingBuildOptions.QualityLevel = (ELightingBuildQuality)QualityLevel;
	}

	// Show the build progress dialog.
	SBuildProgressWidget::EBuildType BuildType = SBuildProgressWidget::BUILDTYPE_Geometry;

	if (Id == FBuildOptions::BuildGeometry ||
		Id == FBuildOptions::BuildVisibleGeometry ||
		Id == FBuildOptions::BuildAll ||
		Id == FBuildOptions::BuildAllOnlySelectedPaths)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_Geometry;
	}
	else if (Id == FBuildOptions::BuildLighting)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_Lighting;
	}
	else if (Id == FBuildOptions::BuildAIPaths ||
		Id == FBuildOptions::BuildSelectedAIPaths)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_Paths;
	}
	else if (Id == FBuildOptions::BuildHierarchicalLOD)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_HLODs;
	}
	else if (Id == FBuildOptions::BuildMinimap)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_Minimap;
	}
	else if (Id == FBuildOptions::BuildLandscapeSplineMeshes)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_LandscapeSplineMeshes;
	}
	else if (Id == FBuildOptions::BuildTextureStreaming)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_TextureStreaming;
	}
	else if (Id == FBuildOptions::BuildVirtualTexture)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_VirtualTexture;
	}
	else if (Id == FBuildOptions::BuildAllLandscape)
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_AllLandscape;
	}
	else
	{
		BuildType = SBuildProgressWidget::BUILDTYPE_Unknown;	
	}

	TWeakPtr<class SBuildProgressWidget> BuildProgressWidget = GWarn->ShowBuildProgressWindow();
	if ( BuildProgressWidget.IsValid() )
	{
		BuildProgressWidget.Pin()->SetBuildType(BuildType);
	}

	TSoftObjectPtr<UWorld> World = InWorld;

	bool bShouldMapCheck = !FParse::Param(FCommandLine::Get(), TEXT("SkipMapCheck"));
	if (Id == FBuildOptions::BuildGeometry)
	{
		// We can't set the busy cursor for all windows, because lighting
		// needs a cursor for the lighting options dialog.
		const FScopedBusyCursor BusyCursor;

		GUnrealEd->Exec( InWorld, TEXT("MAP REBUILD") );

		if (GetDefault<ULevelEditorMiscSettings>()->bNavigationAutoUpdate)
		{
			TriggerNavigationBuilder(InWorld, Id);
		}

		// No need to dirty the persient level if we're building BSP for a sub-level.
		bDirtyPersistentLevel = false;
	}
	else if (Id == FBuildOptions::BuildVisibleGeometry)
	{
		// If any levels are hidden, prompt the user about how to proceed
		bDoBuild = GEditor->WarnAboutHiddenLevels( InWorld, true );
		if ( bDoBuild )
		{
			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			GUnrealEd->Exec( InWorld, TEXT("MAP REBUILD ALLVISIBLE") );

			if (GetDefault<ULevelEditorMiscSettings>()->bNavigationAutoUpdate)
			{
				TriggerNavigationBuilder(InWorld, Id);
			}
		}
	}
	else if (Id == FBuildOptions::BuildLighting)
	{
		if( bDoBuild )
		{
			bool bBSPRebuildNeeded = false;
			// Only BSP brushes affect lighting.  Check if there is any BSP in the level and skip the geometry rebuild if there isn't any.
			for( TActorIterator<ABrush> ActorIt( InWorld ); ActorIt; ++ActorIt )
			{
				ABrush* Brush = *ActorIt;
				
				if( !Brush->IsVolumeBrush() && !Brush->IsBrushShape() && !FActorEditorUtils::IsABuilderBrush( Brush ) )
				{
					// brushes that aren't volumes are considered bsp
					bBSPRebuildNeeded = true;
					break;
				}
			}

			if( bBSPRebuildNeeded )
			{
				// BSP export to lightmass relies on current BSP state
				GUnrealEd->Exec( InWorld, TEXT("MAP REBUILD ALLVISIBLE") );
			}

			GUnrealEd->BuildLighting( LightingBuildOptions );
			bShouldMapCheck = false;
		}

		bDirtyPersistentLevel = false;
	}
	else if (Id == FBuildOptions::BuildAIPaths)
	{
		bDoBuild = GEditor->WarnAboutHiddenLevels( InWorld, false );
		if ( bDoBuild )
		{
			GEditor->ResetTransaction( NSLOCTEXT("UnrealEd", "RebuildNavigation", "Rebuilding Navigation") );

			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			TriggerNavigationBuilder(InWorld, Id);

			// No need to dirty the world package if it uses external actors
			if (InWorld->PersistentLevel->IsUsingExternalActors())
			{
				bDirtyPersistentLevel = false;
			}
		}
	}
	else if (CustomBuildTypes.Contains(Id))
	{
		const auto& CustomBuild = CustomBuildTypes.FindChecked(Id);
		check(CustomBuild.DoBuild.IsBound());

		// Invoke custom build.
		auto Result = CustomBuild.DoBuild.Execute(InWorld, Id);

		bDoBuild = Result != EEditorBuildResult::Skipped;
		bShouldMapCheck = Result == EEditorBuildResult::Success && !CustomBuild.bExternalProcess;
		bDirtyPersistentLevel = Result == EEditorBuildResult::Success && !CustomBuild.bExternalProcess;

		if (Result == EEditorBuildResult::InProgress)
		{
			InProgressBuildId = Id;
		}
	}
	else if (Id == FBuildOptions::BuildHierarchicalLOD)
	{
		bDoBuild = GEditor->WarnAboutHiddenLevels( InWorld, false );
		if ( bDoBuild )
		{
			GEditor->ResetTransaction( NSLOCTEXT("UnrealEd", "BuildHLODMeshes", "Building Hierarchical LOD Meshes") );

			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			if (InWorld->IsPartitionedWorld())
			{
				bShouldMapCheck = false;
				bDirtyPersistentLevel = false;
			}

			TriggerHierarchicalLODBuilder(InWorld);
		}
	}
	else if (Id == FBuildOptions::BuildMinimap)
	{
		bDoBuild = InWorld->IsPartitionedWorld();
		if ( bDoBuild )
		{
			GEditor->ResetTransaction( NSLOCTEXT("UnrealEd", "BuildMinimap", "Building Minimap") );

			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			bShouldMapCheck = false;
			bDirtyPersistentLevel = false;

			TriggerMinimapBuilder(InWorld);
		}
	}
	else if (Id == FBuildOptions::BuildLandscapeSplineMeshes)
	{
		bDoBuild = InWorld->IsPartitionedWorld();
		if ( bDoBuild )
		{
			GEditor->ResetTransaction( NSLOCTEXT("UnrealEd", "BuildLandscapeSplineMeshes", "Building Landscape Spline Meshes") );

			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			bShouldMapCheck = false;
			bDirtyPersistentLevel = false;

			TriggerLandscapeSplineMeshesBuilder(InWorld);
		}
	}
	else if (Id == FBuildOptions::BuildAllLandscape)
	{
		bDoBuild = GEditor->WarnAboutHiddenLevels(InWorld, false);
		if (bDoBuild)
		{
			GEditor->ResetTransaction(NSLOCTEXT("UnrealEd", "BuildAllLandscape", "Building Landscape"));

			// We can't set the busy cursor for all windows, because lighting
			// needs a cursor for the lighting options dialog.
			const FScopedBusyCursor BusyCursor;

			bShouldMapCheck = false;
			EditorBuildAllLandscape(InWorld);

			bDirtyPersistentLevel = false;
		}
	}
	else if (Id == FBuildOptions::BuildAll || Id == FBuildOptions::BuildAllSubmit)
	{
		// TODO: WarnIfLightingBuildIsCurrentlyRunning should check with FBuildAllHandler
		bDoBuild = GEditor->WarnAboutHiddenLevels( InWorld, true );
		bool bLightingAlreadyRunning = GUnrealEd->WarnIfLightingBuildIsCurrentlyRunning();
		if ( bDoBuild && !bLightingAlreadyRunning )
		{
			FBuildAllHandler::Get().StartBuild(InWorld, Id, BuildProgressWidget);
		}
	}
	else
	{
		UE_LOGF(LogEditorBuildUtils, Warning, "Invalid build Id: %ls", *Id.ToString());
		bDoBuild = false;
	}

	// It's possible the world was unloaded & reloaded if external commands were run.
	// To work around this, reassign the initial world from it's soft object path.
	InWorld = World.IsValid() ? World.Get() : GEditor->GetEditorWorldContext().World();

	// Check map for errors (only if build operation happened)
	if ( bShouldMapCheck && bDoBuild && !GEditor->GetMapBuildCancelled() )
	{
		GUnrealEd->Exec( InWorld, TEXT("MAP CHECK DONTDISPLAYDIALOG") );
	}

	if ( bDoBuild && BuildProgressWidget.IsValid() )
	{
		// Display elapsed build time.
		UE_LOGF(LogEditorBuildUtils, Log,  "Build time %ls", *BuildProgressWidget.Pin()->BuildElapsedTimeText().ToString() );
	}

	// Build completed, hide the build progress dialog.
	// NOTE: It's important to turn off modalness before hiding the window, otherwise a background
	//		 application may unexpectedly be promoted to the foreground, obscuring the editor.
	GWarn->CloseBuildProgressWindow();
	
	GUnrealEd->RedrawLevelEditingViewports();

	if ( bDoBuild )
	{
		if ( bDirtyPersistentLevel )
		{
			InWorld->MarkPackageDirty();
		}
		ULevel::LevelDirtiedEvent.Broadcast();
	}

	// Don't show map check if we cancelled build because it may have some bogus data
	const bool bBuildCompleted = bDoBuild && !GEditor->GetMapBuildCancelled();
	if( bBuildCompleted )
	{
		if (bShouldMapCheck)
		{
			FMessageLog("MapCheck").Open( EMessageSeverity::Warning );
		}
		FMessageLog("LightingResults").Notify(LOCTEXT("LightingErrorsNotification", "There were lighting errors."), EMessageSeverity::Error);
	}

	return bBuildCompleted;
}

/**
 * Private helper method to log an error both to GWarn and to the build's list of accumulated errors
 *
 * @param	InErrorMessage			Message to log to GWarn/add to list of errors
 * @param	OutAccumulatedErrors	List of errors accumulated during a build process so far
 */
void FEditorBuildUtils::LogErrorMessage( const FText& InErrorMessage, FText& OutAccumulatedErrors )
{
	OutAccumulatedErrors = FText::Format( LOCTEXT("AccumulateErrors", "{0}\n{1}"), OutAccumulatedErrors, InErrorMessage );
	UE_LOGF(LogEditorBuildUtils, Warning, "%ls", *InErrorMessage.ToString() );
}

/**
 * Helper method to handle automated build behavior in the event of an error. Depending on the specified behavior, one of three
 * results are possible:
 *	a) User is prompted on whether to proceed with the automated build or not,
 *	b) The error is regarded as a build-stopper and the method returns failure,
 *	or
 *	c) The error is acknowledged but not regarded as a build-stopper, and the method returns success.
 * In any event, the error is logged for the user's information.
 *
 * @param	InBehavior				Behavior to use to respond to the error
 * @param	InErrorMsg				Error to log
 * @param	OutAccumulatedErrors	List of errors accumulated from the build process so far; InErrorMsg will be added to the list
 *
 * @return	true if the build should proceed after processing the error behavior; false if it should not
 */
bool FEditorBuildUtils::ProcessAutomatedBuildBehavior( EAutomatedBuildBehavior InBehavior, const FText& InErrorMsg, FText& OutAccumulatedErrors )
{
	// Assume the behavior should result in the build being successful/proceeding to start
	bool bSuccessful = true;

	switch ( InBehavior )
	{
		// In the event the user should be prompted for the error, display a modal dialog describing the error and ask the user
		// if the build should proceed or not
	case ABB_PromptOnError:
		{
			bSuccessful = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, InErrorMsg);
		}
		break;

		// In the event that the specified error should abort the build, mark the processing as a failure
	case ABB_FailOnError:
		bSuccessful = false;
		break;
	}

	// Log the error message so the user is aware of it
	LogErrorMessage( InErrorMsg, OutAccumulatedErrors );

	// If the processing resulted in the build inevitably being aborted, write to the log about the abortion
	if ( !bSuccessful )
	{
		LogErrorMessage( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_AutomatedBuildAborted", "Automated build aborted."), OutAccumulatedErrors );
	}

	return bSuccessful;
}


/**
 * Helper method designed to perform the necessary preparations required to complete an automated editor build
 *
 * @param	BuildSettings		Build settings that will be used for the editor build
 * @param	OutPkgsToSubmit		Set of packages that need to be saved and submitted after a successful build
 * @param	OutErrorMessages	Errors that resulted from the preparation (may or may not force the build to stop, depending on build settings)
 *
 * @return	true if the preparation was successful and the build should continue; false if the preparation failed and the build should be aborted
 */
bool FEditorBuildUtils::PrepForAutomatedBuild( const FEditorAutomatedBuildSettings& BuildSettings, TSet<UPackage*>& OutPkgsToSubmit, FText& OutErrorMessages )
{
	// Assume the preparation is successful to start
	bool bBuildSuccessful = true;

	OutPkgsToSubmit.Empty();

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Source control is required for the automated build, so ensure that SCC support is compiled in and
	// that the server is enabled and available for use
	if ( BuildSettings.bUseSCC && !(ISourceControlModule::Get().IsEnabled() && SourceControlProvider.IsAvailable() ) )
	{
		bBuildSuccessful = false;
		LogErrorMessage( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_SCCError", "Cannot connect to revision control; automated build aborted."), OutErrorMessages );
	}

	TArray<UPackage*> PreviouslySavedWorldPackages;
	TArray<UPackage*> PackagesToCheckout;
	TArray<ULevel*> LevelsToSave;

	if ( bBuildSuccessful )
	{
		TArray<UWorld*> AllWorlds;
		FString UnsavedWorlds;
		EditorLevelUtils::GetWorlds( GWorld, AllWorlds, true );

		// Check all of the worlds that will be built to ensure they have been saved before and have a filename
		// associated with them. If they don't, they won't be able to be submitted to source control.
		FString CurWorldPkgFileName;
		for ( TArray<UWorld*>::TConstIterator WorldIter( AllWorlds ); WorldIter; ++WorldIter )
		{
			const UWorld* CurWorld = *WorldIter;
			check( CurWorld );

			UPackage* CurWorldPackage = CurWorld->GetOutermost();
			check( CurWorldPackage );

			if ( FPackageName::DoesPackageExist( CurWorldPackage->GetName(), &CurWorldPkgFileName ) )
			{
				PreviouslySavedWorldPackages.AddUnique( CurWorldPackage );

				// Add all packages which have a corresponding file to the set of packages to submit for now. As preparation continues
				// any packages that can't be submitted due to some error will be removed.
				OutPkgsToSubmit.Add( CurWorldPackage );
			}
			else
			{
				UnsavedWorlds += FString::Printf( TEXT("%s\n"), *CurWorldPackage->GetName() );
			}
		}

		// If any of the worlds haven't been saved before, process the build setting's behavior to see if the build
		// should proceed or not
		if ( UnsavedWorlds.Len() > 0 )
		{
			bBuildSuccessful = ProcessAutomatedBuildBehavior( BuildSettings.NewMapBehavior, 
				FText::Format( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_UnsavedMap", "The following levels have never been saved before and cannot be submitted:\n\n{0}\n\nAttempt to continue the build?"), FText::FromString(UnsavedWorlds) ),
				OutErrorMessages );
		}
	}

	// Load the asset tools module
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	if ( bBuildSuccessful && BuildSettings.bUseSCC )
	{
		// Update the source control status of any relevant world packages in order to determine which need to be
		// checked out, added to the depot, etc.
		SourceControlProvider.Execute( ISourceControlOperation::Create<FUpdateStatus>(), SourceControlHelpers::PackageFilenames(PreviouslySavedWorldPackages) );

		FString PkgsThatCantBeCheckedOut;
		for ( TArray<UPackage*>::TConstIterator PkgIter( PreviouslySavedWorldPackages ); PkgIter; ++PkgIter )
		{
			UPackage* CurPackage = *PkgIter;
			const FString CurPkgName = CurPackage->GetName();
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(CurPackage, EStateCacheUsage::ForceUpdate);

			if( !SourceControlState.IsValid() ||
				(!SourceControlState->IsSourceControlled() &&
				 !SourceControlState->IsUnknown() &&
				 !SourceControlState->IsIgnored()))
			{
				FString CurFilename;
				if ( FPackageName::DoesPackageExist( CurPkgName, &CurFilename ) )
				{
					if ( IFileManager::Get().IsReadOnly( *CurFilename ) )
					{
						PkgsThatCantBeCheckedOut += FString::Printf( TEXT("%s\n"), *CurPkgName );
						OutPkgsToSubmit.Remove( CurPackage );
					}
				}
			}
			else if(SourceControlState->IsCheckedOut())
			{
			}
			else if(SourceControlState->CanCheckout())
			{
				PackagesToCheckout.Add( CurPackage );
			}
			else
			{
				PkgsThatCantBeCheckedOut += FString::Printf( TEXT("%s\n"), *CurPkgName );
				OutPkgsToSubmit.Remove( CurPackage );
			}
		}

		// If any of the packages can't be checked out or are read-only, process the build setting's behavior to see if the build
		// should proceed or not
		if ( PkgsThatCantBeCheckedOut.Len() > 0 )
		{
			bBuildSuccessful = ProcessAutomatedBuildBehavior( BuildSettings.UnableToCheckoutFilesBehavior,
				FText::Format( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_UnsaveableFiles", "The following assets cannot be checked out of revision control (or are read-only) and cannot be submitted:\n\n{0}\n\nAttempt to continue the build?"), FText::FromString(PkgsThatCantBeCheckedOut) ),
				OutErrorMessages );
		}
	}

	if ( bBuildSuccessful )
	{
		// Check out all of the packages from source control that need to be checked out
		if ( PackagesToCheckout.Num() > 0 )
		{
			TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(PackagesToCheckout);
			SourceControlProvider.Execute( ISourceControlOperation::Create<FCheckOut>(), PackageFilenames );

			// Update the package status of the packages that were just checked out to confirm that they
			// were actually checked out correctly
			SourceControlProvider.Execute(  ISourceControlOperation::Create<FUpdateStatus>(), PackageFilenames );

			FString FilesThatFailedCheckout;
			for ( TArray<UPackage*>::TConstIterator CheckedOutIter( PackagesToCheckout ); CheckedOutIter; ++CheckedOutIter )
			{
				UPackage* CurPkg = *CheckedOutIter;
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(CurPkg, EStateCacheUsage::ForceUpdate);

				// If any of the packages failed to check out, remove them from the set of packages to submit
				if ( !SourceControlState.IsValid() || (!SourceControlState->IsCheckedOut() && !SourceControlState->IsAdded() && SourceControlState->IsSourceControlled()) )
				{
					FilesThatFailedCheckout += FString::Printf( TEXT("%s\n"), *CurPkg->GetName() );
					OutPkgsToSubmit.Remove( CurPkg );
				}
			}

			// If any of the packages failed to check out correctly, process the build setting's behavior to see if the build
			// should proceed or not
			if ( FilesThatFailedCheckout.Len() > 0 )
			{
				bBuildSuccessful = ProcessAutomatedBuildBehavior( BuildSettings.UnableToCheckoutFilesBehavior,
					FText::Format( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_FilesFailedCheckout", "The following assets failed to checkout of revision control and cannot be submitted:\n{0}\n\nAttempt to continue the build?"), FText::FromString(FilesThatFailedCheckout)),
					OutErrorMessages );
			}
		}
	}

	// Verify there are still actually any packages left to submit. If there aren't, abort the build and warn the user of the situation.
	if ( bBuildSuccessful )
	{
		bBuildSuccessful = OutPkgsToSubmit.Num() > 0;
		if ( !bBuildSuccessful )
		{
			LogErrorMessage( NSLOCTEXT("UnrealEd", "AutomatedBuild_Error_NoValidLevels", "None of the current levels are valid for submission; automated build aborted."), OutErrorMessages );
		}
	}

	// If the build is safe to commence, force all of the levels visible to make sure the build operates correctly
	if ( bBuildSuccessful )
	{
		bool bVisibilityToggled = false;
		UWorld* World = GWorld;
		if ( !FLevelUtils::IsLevelVisible( World->PersistentLevel ) )
		{
			EditorLevelUtils::SetLevelVisibility( World->PersistentLevel, true, false );
			bVisibilityToggled = true;
		}
		for (ULevelStreaming* CurStreamingLevel : World->GetStreamingLevels())
		{
			if ( CurStreamingLevel && !FLevelUtils::IsStreamingLevelVisibleInEditor( CurStreamingLevel ) )
			{
				CurStreamingLevel->SetShouldBeVisibleInEditor(true);
				bVisibilityToggled = true;
			}
		}
		if ( bVisibilityToggled )
		{
			World->FlushLevelStreaming();
		}
	}

	return bBuildSuccessful;
}


/**
 * Helper method to submit packages to source control as part of the automated build process
 *
 * @param	InPkgsToSubmit	Set of packages which should be submitted to source control
 * @param	BuildSettings	Build settings used during the automated build
 */
void FEditorBuildUtils::SubmitPackagesForAutomatedBuild( const TSet<UPackage*>& InPkgsToSubmit, const FEditorAutomatedBuildSettings& BuildSettings )
{
	TArray<FString> LevelsToAdd;
	TArray<FString> LevelsToSubmit;

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// first update the status of the packages
	SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), SourceControlHelpers::PackageFilenames(InPkgsToSubmit.Array()));

	// Iterate over the set of packages to submit, determining if they need to be checked in or
	// added to the depot for the first time
	for ( TSet<UPackage*>::TConstIterator PkgIter( InPkgsToSubmit ); PkgIter; ++PkgIter )
	{
		const UPackage* CurPkg = *PkgIter;
		const FString PkgName = CurPkg->GetName();
		const FString PkgFileName = SourceControlHelpers::PackageFilename(CurPkg);

		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(CurPkg, EStateCacheUsage::ForceUpdate);
		if(SourceControlState.IsValid())
		{
			if ( SourceControlState->IsCheckedOut() || SourceControlState->IsAdded() )
			{
				LevelsToSubmit.Add( PkgFileName );
			}
			else if ( BuildSettings.bAutoAddNewFiles && !SourceControlState->IsSourceControlled() && !SourceControlState->IsIgnored() )
			{
				LevelsToSubmit.Add( PkgFileName );
				LevelsToAdd.Add( PkgFileName );
			}
		}
	}

	// Then, if we've also opted to check in any packages, iterate over that list as well
	if(BuildSettings.bCheckInPackages)
	{
		TArray<FString> PackageNames = BuildSettings.PackagesToCheckIn;
		for ( TArray<FString>::TConstIterator PkgIterName(PackageNames); PkgIterName; PkgIterName++ )
		{
			const FString& PkgName = *PkgIterName;
			const FString PkgFileName = SourceControlHelpers::PackageFilename(PkgName);
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(PkgFileName, EStateCacheUsage::ForceUpdate);
			if(SourceControlState.IsValid())
			{
				if ( SourceControlState->IsCheckedOut() || SourceControlState->IsAdded() )
				{
					LevelsToSubmit.Add( PkgFileName );
				}
				else if ( !SourceControlState->IsSourceControlled() && !SourceControlState->IsIgnored() )
				{
					// note we add the files we need to add to the submit list as well
					LevelsToSubmit.Add( PkgFileName );
					LevelsToAdd.Add( PkgFileName );
				}
			}
		}
	}

	// first add files that need to be added
	SourceControlProvider.Execute( ISourceControlOperation::Create<FMarkForAdd>(), LevelsToAdd, EConcurrency::Synchronous );

	// Now check in all the changes, including the files we added above
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = StaticCastSharedRef<FCheckIn>(ISourceControlOperation::Create<FCheckIn>());
	if (BuildSettings.ChangeDescription.IsEmpty())
	{
		CheckInOperation->SetDescription(NSLOCTEXT("UnrealEd", "AutomatedBuild_AutomaticSubmission", "[Automatic Submission]"));
	}
	else
	{
		CheckInOperation->SetDescription(FText::FromString(BuildSettings.ChangeDescription));
	}
	SourceControlProvider.Execute( CheckInOperation, LevelsToSubmit, EConcurrency::Synchronous );
}

void FEditorBuildUtils::TriggerNavigationBuilder(UWorld*& InOutWorld, FName Id)
{
	if (InOutWorld)
	{
		if (Id == FBuildOptions::BuildAIPaths ||
			Id == FBuildOptions::BuildSelectedAIPaths ||
			Id == FBuildOptions::BuildAllOnlySelectedPaths ||
			Id == FBuildOptions::BuildAll ||
			Id == FBuildOptions::BuildAllSubmit)
		{
			bBuildingNavigationFromUserRequest = true;
		}
		else
		{
			bBuildingNavigationFromUserRequest = false;
		}

		if (UE::EditorBuildUtils::bNavmeshAllowPartitionedBuildingFromEditor && InOutWorld->IsPartitionedWorld())
		{
			// Ask user to save dirty packages before getting the package name since it might change (for untitled worlds).
			if (FEditorFileUtils::SaveDirtyPackages(
					/*bPromptUserToSave=*/true,
					/*bSaveMapPackages=*/true,
					/*bSaveContentPackages=*/false,
					/*bFastSave*/false,
					/*bNotifyNoPackagesSaved=*/false,
					/*bCanBeDeclined=*/false))
			{
				const FString& LongPackageName = GetNameSafe(InOutWorld->GetPackage());
				WorldPartitionBuildNavigation(LongPackageName);
				InOutWorld = GEditor->GetEditorWorldContext().World();
			}
		}
		else
		{
			// Invoke navmesh generator
			FNavigationSystem::Build(*InOutWorld);
		}
	}
}

bool FEditorBuildUtils::WorldPartitionBuildNavigation(const FString& InLongPackageName)
{
	UWorldPartitionBuildNavigationOptions* DefaultBuildNavigationOptions = GetMutableDefault<UWorldPartitionBuildNavigationOptions>();
	DefaultBuildNavigationOptions->bVerbose = false;
	DefaultBuildNavigationOptions->bCleanPackages = false;

	const TSharedPtr<SWindow> DlgWindow =
		SNew(SWindow)
		.Title(LOCTEXT("BuildNavigationWindowTitle", "Build Navigation Settings"))
		.ClientSize(SWorldPartitionBuildNavigationDialog::DEFAULT_WINDOW_SIZE)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::FixedSize);

	const TSharedRef<SWorldPartitionBuildNavigationDialog> Dialog =
		SNew(SWorldPartitionBuildNavigationDialog)
		.ParentWindow(DlgWindow)
		.BuildNavigationOptions(DefaultBuildNavigationOptions);

	DlgWindow->SetContent(Dialog);

	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	FSlateApplication::Get().AddModalWindow(DlgWindow.ToSharedRef(), MainFrameModule.GetParentWindow());

	if (Dialog->ClickedOk())
	{
		// Try to provide complete Path, if we can't try with project name
		const FString ProjectPath = FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : FApp::GetProjectName();

		const ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

		const FString Arguments = FString::Printf(TEXT("\"%s\" -run=WorldPartitionBuilderCommandlet %s %s -SCCProvider=%s %s %s"),
				*ProjectPath,
				*InLongPackageName,
				TEXT(" -AllowCommandletRendering -Builder=WorldPartitionNavigationDataBuilder -log=WPNavigationBuilderLog.txt"),
				*SCCProvider.GetName().ToString(),
				DefaultBuildNavigationOptions->bVerbose ? TEXT("-Verbose") : TEXT(""),
				DefaultBuildNavigationOptions->bCleanPackages ? TEXT("-CleanPackages") : TEXT(""));
		
		const bool bSuccess = RunWorldPartitionBuilder(InLongPackageName,
			LOCTEXT("WorldPartitionBuildNavigationProgress", "Building navigation..."),
			LOCTEXT("WorldPartitionBuildNavigationCancelled", "Building navigation cancelled!"),
			LOCTEXT("WorldPartitionBuildNavigationFailed", "Errors occured during the build process, please refer to the logs ('WPNavigationBuilderLog.txt')."),
			Arguments);

		return bSuccess;
	}
	
	return false;
}

bool FEditorBuildUtils::RunWorldPartitionBuilder(
	const FString& MapToLoad,
	const FText& ProgressText,
	const FText& CancelledText,
	const FText& FailureText,
	const FString& CommandLineArguments,
	TFunction<void(FScopedSlowTask&)> UpdateSlowTask
	)
{
	// Ask user to save dirty packages
	if (!FEditorFileUtils::SaveDirtyPackages(/*bPromptUserToSave=*/true, /*bSaveMapPackages=*/true, /*bSaveContentPackages=*/false))
	{
		return false;
	}

	// Unload any loaded map
	if (!UEditorLoadingAndSavingUtils::NewBlankMap(/*bSaveExistingMap*/false))
	{
		return false;
	}

	FProcHandle ProcessHandle;
	bool bCancelled = false;

	// Task scope
	{
		FScopedSlowTask SlowTask(0, ProgressText);
		SlowTask.MakeDialog(true);

		const FString CurrentExecutableName = FPlatformProcess::ExecutablePath();

		uint32 ProcessID;
		ProcessHandle = FPlatformProcess::CreateProc(
			*CurrentExecutableName,
			*CommandLineArguments,
			/*bLaunchedDetached*/true,
			/*bLaunchedHidden*/false,
			/*bLaunchedReallyHidden*/false,
			&ProcessID,
			/*PriorityModifier*/0,
			/*OptionalWorkingDirectory*/nullptr,
			/*PipeWriteChild*/nullptr);

		while (FPlatformProcess::IsProcRunning(ProcessHandle))
		{
			if (SlowTask.ShouldCancel())
			{
				bCancelled = true;
				FPlatformProcess::TerminateProc(ProcessHandle);
				break;
			}

			if (UpdateSlowTask.IsSet())
			{
				UpdateSlowTask(SlowTask);
			}

			SlowTask.EnterProgressFrame(0);
			FPlatformProcess::Sleep(0.1f);
		}
	}

	int32 Result = 0;
	if (!bCancelled && FPlatformProcess::GetProcReturnCode(ProcessHandle, &Result))
	{
		// Force a directory watcher tick for the asset registry to get notified of the changes
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		DirectoryWatcherModule.Get()->Tick(-1.0f);

		// Unload any loaded map
		if (!UEditorLoadingAndSavingUtils::NewBlankMap(/*bSaveExistingMap*/false))
		{
			return false;
		}

		// Force registry update before loading converted map
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		AssetRegistry.ScanModifiedAssetFiles({MapToLoad});
		AssetRegistry.ScanPathsSynchronous(ULevel::GetExternalObjectsPaths(MapToLoad), true);

		FEditorFileUtils::LoadMap(MapToLoad);
		return true;
	}
	
	if (bCancelled)
	{
		FMessageDialog::Open(EAppMsgType::Ok, CancelledText);
	}

	if (Result != 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FailureText);
	}

	return false;
}

/**
 * Call this when an async custom build step has completed (successfully or not).
 */
void FEditorBuildUtils::AsyncBuildCompleted()
{
	check(InProgressBuildId != NAME_None);

	// Reset in-progress id before resuming build all do we don't overwrite something that's just been set.
	auto BuildId = InProgressBuildId;
	InProgressBuildId = NAME_None;

	if (BuildId == FBuildOptions::BuildAll || BuildId == FBuildOptions::BuildAllSubmit)
	{
		FBuildAllHandler::Get().ResumeBuild();
	}
}

/**
 * Is there currently an (async) build in progress?
 */
bool FEditorBuildUtils::IsBuildCurrentlyRunning()
{
	return InProgressBuildId != NAME_None;
}

/**
 * Register a custom build type.
 */
void FEditorBuildUtils::RegisterCustomBuildType(
	const FName Id,
	const FDoEditorBuildDelegate& DoBuild,
	const FName BuildAllExtensionPoint,
	const FText& MenuEntryLabel,
	const FText& MenuSectionLabel,
	bool bExternalProcess)
{
	check(!CustomBuildTypes.Contains(Id));
	CustomBuildTypes.Add(Id, FCustomBuildType(DoBuild, BuildAllExtensionPoint, MenuEntryLabel, MenuSectionLabel, bExternalProcess));

	if (BuildAllExtensionPoint != NAME_None)
	{
		FBuildAllHandler::Get().AddCustomBuildStep(Id, BuildAllExtensionPoint);
	}
}

/**
 * Register a custom build type.
 */
void FEditorBuildUtils::RegisterCustomBuildType(
	const FName Id,
	const FCanDoEditorBuildDelegate& CanDoBuild,
	const FDoEditorBuildDelegate& DoBuild,
	const FName BuildAllExtensionPoint,
	const FText& MenuEntryLabel,
	const FText& MenuSectionLabel,
	bool bExternalProcess)
{
	check(!CustomBuildTypes.Contains(Id));
	CustomBuildTypes.Add(Id, FCustomBuildType(CanDoBuild, DoBuild, BuildAllExtensionPoint, MenuEntryLabel, MenuSectionLabel, bExternalProcess));

	if (BuildAllExtensionPoint != NAME_None)
	{
		FBuildAllHandler::Get().AddCustomBuildStep(Id, BuildAllExtensionPoint);
	}
}

/**
 * Unregister a custom build type.
 * @param Id The identifier of the build type to unregister.
 */
void FEditorBuildUtils::UnregisterCustomBuildType(FName Id)
{
	CustomBuildTypes.Remove(Id);
	FBuildAllHandler::Get().RemoveCustomBuildStep(Id);
}

void FEditorBuildUtils::GetBuildTypes(TArray<FName>& Types)
{
	CustomBuildTypes.GetKeys(Types);
}

void FEditorBuildUtils::GetBuildTypesLocalizedLabels(TArray<FText>& RegisteredBuildTypesEntryLabels, TArray<FText>& RegisteredBuildTypesSectionLabels)
{
	for (auto It = CustomBuildTypes.CreateConstIterator(); It; ++It)
	{
		RegisteredBuildTypesEntryLabels.Add(It->Value.MenuEntryLabel);
		RegisteredBuildTypesSectionLabels.Add(It->Value.MenuSectionLabel);
	}
}

/**
 * Initialise Build All handler.
 */
FBuildAllHandler::FBuildAllHandler()
	: CurrentStep(0)
{
	// Add built in build steps.
	BuildSteps.Add(FBuildOptions::BuildAllLandscape);
	BuildSteps.Add(FBuildOptions::BuildGeometry);
	BuildSteps.Add(FBuildOptions::BuildHierarchicalLOD);
	BuildSteps.Add(FBuildOptions::BuildAIPaths);
	// Texture streaming goes before lighting as lighting needs to be the last build step.
	// This is not an issue as lightmaps are not taken into consideration in the texture streaming build.
	BuildSteps.Add(FBuildOptions::BuildTextureStreaming);
	//Lighting must always be the last one when doing a build all
	BuildSteps.Add(FBuildOptions::BuildLighting);
}

/**
 * Add a custom Build All step.
 */
void FBuildAllHandler::AddCustomBuildStep(FName Id, FName InsertBefore)
{
	const int32 InsertionPoint = BuildSteps.Find(InsertBefore);
	if (InsertionPoint != INDEX_NONE)
	{
		BuildSteps.Insert(Id, InsertionPoint);
	}
}

/**
 * Remove a custom Build All step.
 */
void FBuildAllHandler::RemoveCustomBuildStep(FName Id)
{
	BuildSteps.Remove(Id);
}

/**
 * Commence a new Build All operation.
 */
void FBuildAllHandler::StartBuild(UWorld* World, FName BuildId, const TWeakPtr<SBuildProgressWidget>& BuildProgressWidget)
{
	check(CurrentStep == 0);
	check(CurrentWorld == nullptr);
	check(CurrentBuildId == NAME_None);

	CurrentWorld = World;
	CurrentBuildId = BuildId;
	ProcessBuild(BuildProgressWidget);
}

/**
 * Resume a Build All build from where it was left off.
 */
void FBuildAllHandler::ResumeBuild()
{
	// Resuming from async operation, may be about to do slow stuff again so show the progress window again.
	TWeakPtr<SBuildProgressWidget> BuildProgressWidget = GWarn->ShowBuildProgressWindow();

	// We have to increment the build step, resuming from an async build step
	CurrentStep++;

	ProcessBuild(BuildProgressWidget);

	// Synchronous part completed, hide the build progress dialog.
	GWarn->CloseBuildProgressWindow();
}

/**
 * Internal method that actual does the build.
 */
void FBuildAllHandler::ProcessBuild(const TWeakPtr<SBuildProgressWidget>& BuildProgressWidget)
{
	const FScopedBusyCursor BusyCursor;

	TSoftObjectPtr<UWorld> World = CurrentWorld;

	// Loop until we finish, or we start an async step.
	while (true)
	{
		// It's possible the world was unloaded & reloaded if external commands were run.
		// To work around this, reassign the initial world from it's soft object path.
		CurrentWorld = World.IsValid() ? World.Get() : GEditor->GetEditorWorldContext().World();

		if (GEditor->GetMapBuildCancelled())
		{
			// Build cancelled, so bail.
			BuildFinished();
			break;
		}

		check(BuildSteps.IsValidIndex(CurrentStep));
		FName StepId = BuildSteps[CurrentStep];

		if (StepId == FBuildOptions::BuildGeometry)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_Geometry);
			GUnrealEd->Exec(CurrentWorld, TEXT("MAP REBUILD ALLVISIBLE") );
		}
		else if (StepId == FBuildOptions::BuildHierarchicalLOD)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_HLODs);
			FEditorBuildUtils::TriggerHierarchicalLODBuilder(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildMinimap)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_Minimap);
			FEditorBuildUtils::TriggerMinimapBuilder(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildLandscapeSplineMeshes)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_LandscapeSplineMeshes);
			FEditorBuildUtils::TriggerLandscapeSplineMeshesBuilder(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildTextureStreaming)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_TextureStreaming);
			FEditorBuildUtils::EditorBuildTextureStreaming(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildVirtualTexture)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_VirtualTexture);
			FEditorBuildUtils::EditorBuildVirtualTexture(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildAllLandscape)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_AllLandscape);
			FEditorBuildUtils::EditorBuildAllLandscape(CurrentWorld);
		}
		else if (StepId == FBuildOptions::BuildAIPaths)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_Paths);
			FEditorBuildUtils::TriggerNavigationBuilder(CurrentWorld, CurrentBuildId);
		}
		else if (StepId == FBuildOptions::BuildLighting)
		{
			BuildProgressWidget.Pin()->SetBuildType(SBuildProgressWidget::BUILDTYPE_Lighting);

			FLightingBuildOptions LightingOptions;

			int32 QualityLevel;

			// Force automated builds to always use production lighting
			if ( CurrentBuildId == FBuildOptions::BuildAllSubmit )
			{
				QualityLevel = Quality_Production;
			}
			else
			{
				GConfig->GetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), QualityLevel, GEditorPerProjectIni);
				QualityLevel = FMath::Clamp<int32>(QualityLevel, Quality_Preview, Quality_Production);
			}
			LightingOptions.QualityLevel = (ELightingBuildQuality)QualityLevel;

			GUnrealEd->BuildLighting(LightingOptions);

			// TODO!
			//bShouldMapCheck = false;

			// Lighting is always the last step (Lightmass isn't set up to resume builds).
			BuildFinished();
			break;
		}
		else
		{
			FEditorBuildUtils::FCustomBuildType& CustomBuildType = FEditorBuildUtils::CustomBuildTypes[StepId];
			if (CustomBuildType.CanDoBuild.IsBound() == false || CustomBuildType.CanDoBuild.Execute(CurrentWorld, CurrentBuildId))
			{
				const EEditorBuildResult Result = CustomBuildType.DoBuild.Execute(CurrentWorld, CurrentBuildId);

				if (Result == EEditorBuildResult::InProgress)
				{
					// Build & Submit builds must be synchronous.
					check(CurrentBuildId != FBuildOptions::BuildAllSubmit);

					// Build step is running asynchronously, so let it run.
					FEditorBuildUtils::InProgressBuildId = CurrentBuildId;
					break;
				}
			}
		}

		// Next go around we want to do the next step.
		CurrentStep++;
	}
}

/**
 * Called when a build is finished (successfully or not).
 */
void FBuildAllHandler::BuildFinished()
{
	CurrentStep = 0;
	CurrentWorld = nullptr;
	CurrentBuildId = NAME_None;
}

void FEditorBuildUtils::TriggerHierarchicalLODBuilder(UWorld* InWorld)
{
	if (InWorld->IsPartitionedWorld())
	{
		TSubclassOf<UWorldPartitionBuilder> WorldPartitionHLODsBuilder = FindObjectChecked<UClass>(nullptr, TEXT("/Script/UnrealEd.WorldPartitionHLODsBuilder"), EFindObjectFlags::ExactClass);
		IWorldPartitionEditorModule::Get().RunBuilder(WorldPartitionHLODsBuilder, InWorld);
	}
	else
	{
		// Invoke HLOD generator, with either preview or full build
		InWorld->HierarchicalLODBuilder->BuildMeshesForLODActors(false);
	}
}

void FEditorBuildUtils::TriggerMinimapBuilder(UWorld* InWorld)
{
	if (InWorld->IsPartitionedWorld())
	{
		TSubclassOf<UWorldPartitionBuilder> WorldPartitionMiniMapBuilder = FindObjectChecked<UClass>(nullptr, TEXT("/Script/UnrealEd.WorldPartitionMiniMapBuilder"), EFindObjectFlags::ExactClass);
		IWorldPartitionEditorModule::Get().RunBuilder(WorldPartitionMiniMapBuilder, InWorld);
	}
}

void FEditorBuildUtils::TriggerLandscapeSplineMeshesBuilder(UWorld* InWorld)
{
	if (InWorld->IsPartitionedWorld())
	{
		TSubclassOf<UWorldPartitionBuilder> WorldPartitionLandscapeSplineMeshesBuilder = FindObjectChecked<UClass>(nullptr, TEXT("/Script/UnrealEd.WorldPartitionLandscapeSplineMeshesBuilder"), EFindObjectFlags::ExactClass);
		IWorldPartitionEditorModule::Get().RunBuilder(WorldPartitionLandscapeSplineMeshesBuilder, InWorld);
	}
}

EDebugViewShaderMode ViewModeIndexToDebugViewShaderMode(EViewModeIndex SelectedViewMode)
{
	switch (SelectedViewMode)
	{
	case VMI_ShaderComplexity:
		return DVSM_ShaderComplexity;
	case VMI_ShaderComplexityWithQuadOverdraw:
		return DVSM_ShaderComplexityContainedQuadOverhead;
	case VMI_QuadOverdraw:
		return DVSM_QuadComplexity;
	case VMI_PrimitiveDistanceAccuracy:
		return DVSM_PrimitiveDistanceAccuracy;
	case VMI_MeshUVDensityAccuracy:
		return DVSM_MeshUVDensityAccuracy;
	case VMI_MaterialTextureScaleAccuracy:
		return DVSM_MaterialTextureScaleAccuracy;
	case VMI_RequiredTextureResolution:
		return DVSM_RequiredTextureResolution;
	case VMI_LODColoration:
	case VMI_HLODColoration:
		return DVSM_LODColoration;
	case VMI_VisualizeGPUSkinCache:
		return DVSM_VisualizeGPUSkinCache;
	case VMI_LWCComplexity:
		return DVSM_LWCComplexity;
	case VMI_ShadowCasters:
		return DVSM_ShadowCasters;
	case VMI_StreamingTextureDeficit:
	case VMI_StreamingTextureResidency:
	case VMI_StreamingMeshLODDeficit:
	case VMI_StreamingMeshLODResidency:
		return DVSM_StreamingDeficit;
	case VMI_Unknown:
	default :
		return DVSM_None;
	}
}

void FEditorBuildUtils::UpdateTextureStreamingMaterialBindings(UWorld* InWorld)
{
	const EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::High;
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	const EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;

	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

	TSet<UMaterialInterface*> Materials;
	if (GetUsedMaterialsInWorld(InWorld, Materials, nullptr))
	{
		// Flush renderthread since we are about to update the material streaming data.
		FlushRenderingCommands();

		for (UMaterialInterface* MaterialInterface : Materials)
		{
			if (!MaterialInterface)
			{
				continue;
			}

			TArray<UTexture*> UsedTextures;
			TArray< TArray<int32> > UsedIndices;
			MaterialInterface->GetUsedTexturesAndIndices(UsedTextures, UsedIndices, QualityLevel, ShaderPlatform);
			MaterialInterface->SortTextureStreamingData(true, false);

			MaterialInterface->TextureStreamingDataMissingEntries.Empty();

			for (int32 UsedIndex = 0; UsedIndex < UsedTextures.Num(); ++UsedIndex)
			{
				if (UsedTextures[UsedIndex])
				{
					TArray<FMaterialTextureInfo>& MaterialData = MaterialInterface->GetTextureStreamingData();

					int32 LowerIndex = INDEX_NONE;
					int32 HigherIndex = INDEX_NONE;
					if (MaterialInterface->FindTextureStreamingDataIndexRange(UsedTextures[UsedIndex]->GetFName(), LowerIndex, HigherIndex))
					{
						// Here we expect every entry in UsedIndices to match one of the entry in the range.
						for (int32 SubIndex = 0; LowerIndex <= HigherIndex && SubIndex < UsedIndices[UsedIndex].Num(); ++LowerIndex, ++SubIndex)
						{
							MaterialData[LowerIndex].TextureIndex = UsedIndices[UsedIndex][SubIndex];
						}
					}
					else // If the texture is missing in the material data, add it ass missing
					{
						FMaterialTextureInfo MissingInfo;
						MissingInfo.TextureName = UsedTextures[UsedIndex]->GetFName();
						for (int32 SubIndex = 0; SubIndex < UsedIndices[UsedIndex].Num(); ++SubIndex)
						{
							MissingInfo.TextureIndex = UsedIndices[UsedIndex][SubIndex];
							MaterialInterface->TextureStreamingDataMissingEntries.Add(MissingInfo);
						}
					}
				}
			}
		}
	}
}


bool FEditorBuildUtils::EditorBuildTextureStreaming(UWorld* InWorld, EViewModeIndex SelectedViewMode)
{
	if (!InWorld) return false;

	const bool bNeedsMaterialData = SelectedViewMode == VMI_MaterialTextureScaleAccuracy || SelectedViewMode == VMI_Unknown;

	FScopedSlowTask BuildTextureStreamingTask(bNeedsMaterialData ? 5.f : 1.f, SelectedViewMode == VMI_Unknown ? LOCTEXT("TextureStreamingBuild", "Building Texture Streaming") : LOCTEXT("TextureStreamingDataUpdate", "Building Missing ViewMode Data"));
	BuildTextureStreamingTask.MakeDialog(true);

	const EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::High;
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	const EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;

	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

	if (bNeedsMaterialData)
	{
		TSet<UMaterialInterface*> Materials;
		if (!GetUsedMaterialsInWorld(InWorld, Materials, &BuildTextureStreamingTask))
		{
			return false;
		}

		if (Materials.Num())
		{
			if (!CompileDebugViewModeShaders(DVSM_OutputMaterialTextureScales, QualityLevel, FeatureLevel, Materials, &BuildTextureStreamingTask))
			{
				return false;
			}
		}
		else
		{
			BuildTextureStreamingTask.EnterProgressFrame();
		}

		// Exporting Material TexCoord Scales
		if (Materials.Num())
		{
			FScopedSlowTask SlowTask(1.f, (LOCTEXT("TextureStreamingBuild_ExportingMaterialScales", "Computing Per Texture Material Data")));
			const double StartTime = FPlatformTime::Seconds();
			const float OneOverNumMaterials = 1.f / (float)Materials.Num();

			FMaterialUtilities::FExportErrorManager ExportErrors(ShaderPlatform);

			for (UMaterialInterface* MaterialInterface : Materials)
			{
				check(MaterialInterface);

				BuildTextureStreamingTask.EnterProgressFrame(OneOverNumMaterials);
				SlowTask.EnterProgressFrame(OneOverNumMaterials);
				if (GWarn->ReceivedUserCancel()) return false;

				bool bNeedsRebuild = SelectedViewMode == VMI_Unknown || !MaterialInterface->HasTextureStreamingData();
				if (!bNeedsRebuild && SelectedViewMode == VMI_MaterialTextureScaleAccuracy)
				{
					// In that case only process material that have incomplete data
					for (FMaterialTextureInfo TextureData : MaterialInterface->GetTextureStreamingData())
					{
						if (TextureData.IsValid() && TextureData.TextureIndex == INDEX_NONE)
						{
							bNeedsRebuild = true;
							break;
						}
					}
				}
				if (bNeedsRebuild)
				{
					FMaterialUtilities::ExportMaterialUVDensities(MaterialInterface, QualityLevel, ShaderPlatform, ExportErrors);
				}
			}
			UE_LOGF(LogLevel, Display, "Export Material TexCoord Scales took %.3f seconds.", FPlatformTime::Seconds() - StartTime);
			ExportErrors.OutputToLog();
		}
		else
		{
			BuildTextureStreamingTask.EnterProgressFrame();
		}
	}

	if (!BuildTextureStreamingComponentData(InWorld, QualityLevel, FeatureLevel, SelectedViewMode == VMI_Unknown, BuildTextureStreamingTask))
	{
		return false;
	}

	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
	return true;
}

static bool AreCloseToOnePercent(float A, float B)
{
	return FMath::Abs(A - B) / FMath::Max3(FMath::Abs(A), FMath::Abs(B), 1.f) < 0.01f;
}

bool FEditorBuildUtils::EditorBuildMaterialTextureStreamingData(UPackage* Package)
{
	const EMaterialQualityLevel::Type QualityLevel = EMaterialQualityLevel::High;
	const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
	const EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;

	TSet<UMaterialInterface*> Materials;
	if (Package)
	{
		//if a package is explicitly provided, we're only interested in materials under that package.
		//there is no need to perform a prior GC on this path, as we shouldn't be about to unhash any objects in the provided package.
		TArray<UObject*> ObjectsInPackage;
		GetObjectsWithOuter(Package, ObjectsInPackage);
		for (UObject* Obj : ObjectsInPackage)
		{
			UMaterialInterface* Material = Cast<UMaterialInterface>(Obj);
			if (Material && Material->HasAnyFlags(RF_Public) && Material->UseAnyStreamingTexture())
			{
				FMaterialResource* Resource = Material->GetMaterialResource(GMaxRHIShaderPlatform);
				if (Resource)
				{
					Resource->CacheShaders();
					Materials.Add(Material);
				}
			}
		}
	}
	else
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		for (TObjectIterator<UMaterialInterface> MaterialIt; MaterialIt; ++MaterialIt)
		{
			UMaterialInterface* Material = *MaterialIt;
			if (Material && Material->GetOutermost() != GetTransientPackage() && Material->HasAnyFlags(RF_Public) && Material->UseAnyStreamingTexture())
			{
				Materials.Add(Material);
			}
		}
	}

	if (Materials.Num() == 0)
	{ //early out if there's nothing to work on.
		return false;
	}

	FScopedSlowTask SlowTask(3.f); // { Sync Pending Shader, Wait for Compilation, Export }
	SlowTask.MakeDialog(true);
	const float OneOverNumMaterials = 1.f / FMath::Max(1.f, (float)Materials.Num());

	bool bAnyPackagesDirtied = false;
	if (CompileDebugViewModeShaders(DVSM_OutputMaterialTextureScales, QualityLevel, FeatureLevel, Materials, &SlowTask))
	{
		FMaterialUtilities::FExportErrorManager ExportErrors(ShaderPlatform);
		for (UMaterialInterface* MaterialInterface : Materials)
		{
			SlowTask.EnterProgressFrame(OneOverNumMaterials);
			if (MaterialInterface)
			{
				//for the explicit package path, we also want to use the quality level from the material resource to ensure we get a hit on the shadermap.
				FMaterialResource* Resource = Package ? MaterialInterface->GetMaterialResource(GMaxRHIShaderPlatform) : nullptr;

				TArray<FMaterialTextureInfo> PreviousData = MaterialInterface->GetTextureStreamingData();
				if (FMaterialUtilities::ExportMaterialUVDensities(MaterialInterface, Resource ? Resource->GetQualityLevel() : QualityLevel, ShaderPlatform, ExportErrors))
				{
					TArray<FMaterialTextureInfo> NewData = MaterialInterface->GetTextureStreamingData();

					bool bNeedsResave = PreviousData.Num() != NewData.Num();
					if (!bNeedsResave)
					{
						for (int32 EntryIndex = 0; EntryIndex < NewData.Num(); ++EntryIndex)
						{
							if (NewData[EntryIndex].TextureName != PreviousData[EntryIndex].TextureName ||
								!AreCloseToOnePercent(NewData[EntryIndex].SamplingScale, PreviousData[EntryIndex].SamplingScale) ||
								NewData[EntryIndex].UVChannelIndex != PreviousData[EntryIndex].UVChannelIndex)
							{
								bNeedsResave = true;
								break;
							}
						}
					}

					if (bNeedsResave)
					{
						MaterialInterface->MarkPackageDirty();
						bAnyPackagesDirtied = true;
					}
				}
			}
		}
		ExportErrors.OutputToLog();
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	return bAnyPackagesDirtied;
}

bool FEditorBuildUtils::EditorBuildVirtualTexture(UWorld* InWorld)
{
	if (InWorld == nullptr)
	{
		return true;
	}

	IVirtualTexturingEditorModule* Module = FModuleManager::Get().GetModulePtr<IVirtualTexturingEditorModule>("VirtualTexturingEditor");
	if (Module == nullptr)
	{
		return false;
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	ON_SCOPE_EXIT
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	};

	FWorldPartitionHelpers::FForEachActorWithLoadingResult ForEachActorWithLoadingResult;
	if (UWorldPartition* WorldPartition = InWorld->GetWorldPartition())
	{
		FScopedSlowTask BuildTask(1.0f, LOCTEXT("VirtualTextureLoadActors", "Loading Actors"));
		BuildTask.MakeDialog();
		UWorldPartitionRuntimeVirtualTextureBuilder::LoadRuntimeVirtualTextureActors(WorldPartition, ForEachActorWithLoadingResult);
	}

	TArray<URuntimeVirtualTextureComponent*> Components = Module->GatherRuntimeVirtualTextureComponents(InWorld);
	FBuildAllStreamedMipsResult Result = Module->BuildAllStreamedMips({ .World = InWorld, .Components = Components, .bRestoreFeatureLevelAfterBuilding = true });
	return Result.bSuccess;
}

void FEditorBuildUtils::EditorBuildAllLandscape(UWorld* InWorld)
{
	if (InWorld)
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = InWorld->GetSubsystem<ULandscapeSubsystem>())
		{
			LandscapeSubsystem->BuildAll(UE::Landscape::EBuildFlags::WriteFinalLog);
		}
	}
}

#undef LOCTEXT_NAMESPACE
