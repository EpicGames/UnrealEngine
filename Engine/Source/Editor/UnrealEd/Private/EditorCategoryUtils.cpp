// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorCategoryUtils.h"

#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "HAL/PlatformMath.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "Misc/CString.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "EditorCategoryUtils"

/*******************************************************************************
 * FEditorCategoryUtils Helpers
 ******************************************************************************/

namespace FEditorCategoryUtilsImpl
{
	using namespace FEditorCategoryUtils;

	struct FCategoryInfo
	{
		FText DisplayName;
		FText Tooltip;
		FString DocLink;
		FString DocExcerpt;
	};

	typedef TMap<FString, FCategoryInfo> FCategoryInfoMap;

	/**
	 * Gets the table that tracks mappings from string keys to qualified 
	 * category paths. Inits the structure if it hadn't been before (adds 
	 * default mappings for all FCommonEditorCategory values)
	 * 
	 * @return A map from string category keys, to fully qualified category paths.
	 */
	FCategoryInfoMap& GetCategoryTable();

	/**
	 * Performs a lookup into the category key table, retrieving a fully 
	 * qualified category path for the specified key.
	 * 
	 * @param  Key	The key you want a category path for.
	 * @return The category display string associated with the specified key (an empty string if an entry wasn't found).
	 */
	const FText& GetCategory(const FString& Key);

	/**
	 * Performs a lookup into the category key table, retrieving a fully 
	 * qualified category path for the specified key.
	 * 
	 * @param  CategoryDisplayName	Display name for the category, will be used if a tooltip can not be found in the Documentation Page
	 * @param  DocLink				Path to the documentation page that contains the excerpt for this category
	 * @param  DocExcerpt			Name of the excerpt within the document page for this category
	 * @return						The tooltip (if any) stored at the doc path
	 */
	FText GetTooltipForCategory(const FString& CategoryDisplayName, const FString& DocLink, const FString& DocExcerpt);

	/** Metadata tags */
	const FName ClassHideCategoriesMetaKey(TEXT("HideCategories"));
	const FName ClassShowCategoriesMetaKey(TEXT("ShowCategories"));
}

//------------------------------------------------------------------------------
FEditorCategoryUtilsImpl::FCategoryInfoMap& FEditorCategoryUtilsImpl::GetCategoryTable()
{
	static FCategoryInfoMap CategoryLookup;

	static bool bInitialized = false;
	// this function is reentrant, so we have to guard against recursion
	if (!bInitialized)
	{
		bInitialized = true;

		RegisterCategoryKey("AI", LOCTEXT("AICategory", "AI"));
		RegisterCategoryKey("Animation", LOCTEXT("AnimationCategory", "Animation"));
		RegisterCategoryKey("Audio", LOCTEXT("AudioCategory", "Audio"));
		RegisterCategoryKey("Development", LOCTEXT("DevelopmentCategory", "Development"));
		RegisterCategoryKey("Effects", LOCTEXT("EffectsCategory", "Effects"));
		RegisterCategoryKey("Gameplay", LOCTEXT("GameplayCategory", "Game"));
		RegisterCategoryKey("Input", LOCTEXT("InputCategory", "Input"));
		RegisterCategoryKey("Math", LOCTEXT("MathCategory", "Math"));
		RegisterCategoryKey("Networking", LOCTEXT("NetworkingCategory", "Networking"));
		RegisterCategoryKey("Pawn", LOCTEXT("PawnCategory", "Pawn"));
		RegisterCategoryKey("Rendering", LOCTEXT("RenderingCategory", "Rendering"));
		RegisterCategoryKey("Utilities", LOCTEXT("UtilitiesCategory", "Utilities"));
		RegisterCategoryKey("Delegates", LOCTEXT("DelegatesCategory", "Event Dispatchers"));
		RegisterCategoryKey("Variables", LOCTEXT("VariablesCategory", "Variables"));
		RegisterCategoryKey("Class", LOCTEXT("ClassCategory", "Class"));
		RegisterCategoryKey("UserInterface", LOCTEXT("UserInterfaceCategory", "User Interface"));
		RegisterCategoryKey("AnimNotify", LOCTEXT("AnimNotifyCategory", "Add AnimNotify Event"));
		RegisterCategoryKey("BranchPoint", LOCTEXT("BranchPointCategory", "Add Montage Branching Point Event"));

		// Utilities sub categories
		RegisterCategoryKey("FlowControl", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("FlowControlCategory", "Flow Control")));
		RegisterCategoryKey("Transformation", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("TransformationCategory", "Transformation")));
		RegisterCategoryKey("String", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("StringCategory", "String")));
		RegisterCategoryKey("Text", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("TextCategory", "Text")));
		RegisterCategoryKey("Name", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("NameCategory", "Name")));
		RegisterCategoryKey("Enum", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("EnumCategory", "Enum")));
		RegisterCategoryKey("Struct", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("StructCategory", "Struct")));
		RegisterCategoryKey("Macro", BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("MacroCategory", "Macro")));
	}

	return CategoryLookup;
}

//------------------------------------------------------------------------------
const FText& FEditorCategoryUtilsImpl::GetCategory(const FString& Key)
{
	if (FEditorCategoryUtilsImpl::FCategoryInfo const* FoundCategory = GetCategoryTable().Find(Key))
	{
		return FoundCategory->DisplayName;
	}
	return FText::GetEmpty();
}

//------------------------------------------------------------------------------
FText FEditorCategoryUtilsImpl::GetTooltipForCategory(const FString& CategoryDisplayName, const FString& DocLink, const FString& DocExcerpt)
{
	FText Tooltip;

	TSharedRef<IDocumentation> Documentation = IDocumentation::Get();
	if (Documentation->PageExists(DocLink))
	{
		TSharedRef<IDocumentationPage> DocPage = Documentation->GetPage(DocLink, NULL);
	
		const FString TooltipExcerptSuffix(TEXT("__Tooltip"));

		FExcerpt Excerpt;
		if (DocPage->GetExcerpt(DocExcerpt + TooltipExcerptSuffix, Excerpt))
		{
			static const FString TooltipVarKey(TEXT("Tooltip"));
			if (FString* TooltipValue = Excerpt.Variables.Find(TooltipVarKey))
			{
				Tooltip = FText::FromString(TooltipValue->Replace(TEXT("\\n"),TEXT("\n")));
			}
		}
	}

	if (Tooltip.IsEmpty())
	{
		FString CategoryTooltip;

		if (CategoryDisplayName.Split(TEXT("|"), nullptr, &CategoryTooltip, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			Tooltip = FText::FromString(CategoryTooltip);
		}
		else
		{
			Tooltip = FText::FromString(CategoryDisplayName);
		}
	}

	return Tooltip;
}

/*******************************************************************************
 * FEditorCategoryUtils
 ******************************************************************************/

//------------------------------------------------------------------------------
void FEditorCategoryUtils::RegisterCategoryKey(const FString& Key, const FText& Category, const FText& Tooltip)
{
	FEditorCategoryUtilsImpl::FCategoryInfo& CategoryInfo = FEditorCategoryUtilsImpl::GetCategoryTable().Add(Key);

	CategoryInfo.DisplayName = GetCategoryDisplayString(Category);
	CategoryInfo.DocLink = TEXT("Shared/GraphNodes/Blueprint/NodeCategories");
	CategoryInfo.DocExcerpt = Key;
	CategoryInfo.Tooltip = (Tooltip.IsEmpty() ? FEditorCategoryUtilsImpl::GetTooltipForCategory(CategoryInfo.DisplayName.ToString(), CategoryInfo.DocLink, CategoryInfo.DocExcerpt) : Tooltip);
}

void FEditorCategoryUtils::RegisterCategoryKey(const FString& Key, const FText& Category, const FString& DocLink, const FString& DocExcerpt)
{
	FEditorCategoryUtilsImpl::FCategoryInfo& CategoryInfo = FEditorCategoryUtilsImpl::GetCategoryTable().Add(Key);

	CategoryInfo.DisplayName = GetCategoryDisplayString(Category);
	CategoryInfo.DocLink = DocLink;
	CategoryInfo.DocExcerpt = DocExcerpt;
	CategoryInfo.Tooltip = FEditorCategoryUtilsImpl::GetTooltipForCategory(CategoryInfo.DisplayName.ToString(), CategoryInfo.DocLink, CategoryInfo.DocExcerpt);
}

//------------------------------------------------------------------------------
const FText& FEditorCategoryUtils::GetCommonCategory(const FCommonEditorCategory::EValue CategoryId)
{
	static TMap<FCommonEditorCategory::EValue, FString> CommonCategoryKeys;
	if (CommonCategoryKeys.Num() == 0)
	{
		CommonCategoryKeys.Add(FCommonEditorCategory::AI, "AI");
		CommonCategoryKeys.Add(FCommonEditorCategory::Animation, "Animation");
		CommonCategoryKeys.Add(FCommonEditorCategory::Audio, "Audio");
		CommonCategoryKeys.Add(FCommonEditorCategory::Development, "Development");
		CommonCategoryKeys.Add(FCommonEditorCategory::Effects, "Effects");
		CommonCategoryKeys.Add(FCommonEditorCategory::Gameplay, "Gameplay");
		CommonCategoryKeys.Add(FCommonEditorCategory::Input, "Input");
		CommonCategoryKeys.Add(FCommonEditorCategory::Math, "Math");
		CommonCategoryKeys.Add(FCommonEditorCategory::Networking, "Networking");
		CommonCategoryKeys.Add(FCommonEditorCategory::Pawn, "Pawn");
		CommonCategoryKeys.Add(FCommonEditorCategory::Rendering, "Rendering");
		CommonCategoryKeys.Add(FCommonEditorCategory::Utilities, "Utilities");
		CommonCategoryKeys.Add(FCommonEditorCategory::Delegates, "Delegates");
		CommonCategoryKeys.Add(FCommonEditorCategory::Variables, "Variables");
		CommonCategoryKeys.Add(FCommonEditorCategory::Class, "Class");
		CommonCategoryKeys.Add(FCommonEditorCategory::UserInterface, "UserInterface");
		CommonCategoryKeys.Add(FCommonEditorCategory::AnimNotify, "AnimNotify");
		CommonCategoryKeys.Add(FCommonEditorCategory::BranchPoint, "BranchPoint");

		CommonCategoryKeys.Add(FCommonEditorCategory::FlowControl, "FlowControl");
		CommonCategoryKeys.Add(FCommonEditorCategory::Transformation, "Transformation");
		CommonCategoryKeys.Add(FCommonEditorCategory::String, "String");
		CommonCategoryKeys.Add(FCommonEditorCategory::Text, "Text");
		CommonCategoryKeys.Add(FCommonEditorCategory::Name, "Name");
		CommonCategoryKeys.Add(FCommonEditorCategory::Enum, "Enum");
		CommonCategoryKeys.Add(FCommonEditorCategory::Struct, "Struct");
		CommonCategoryKeys.Add(FCommonEditorCategory::Macro, "Macro");
	}

	if (FString* CategoryKey = CommonCategoryKeys.Find(CategoryId))
	{
		return FEditorCategoryUtilsImpl::GetCategory(*CategoryKey);
	}
	return FText::GetEmpty();
}

//------------------------------------------------------------------------------
FText FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::EValue RootId, const FText& SubCategory)
{
	FText ConstructedCategory;

	const FText& RootCategory = GetCommonCategory(RootId);
	if (RootCategory.IsEmpty())
	{
		ConstructedCategory = SubCategory;
	}
	else if (SubCategory.IsEmpty())
	{
		ConstructedCategory = RootCategory;
	}
	else
	{
		// @TODO: FText::Format() is expensive, since this is category 
		//        concatenation, we could just do FString concatenation
		ConstructedCategory = FText::Format(LOCTEXT("ConcatedCategory", "{0}|{1}"), RootCategory, SubCategory);
	}
	return ConstructedCategory;
}

//------------------------------------------------------------------------------
FText FEditorCategoryUtils::GetCategoryDisplayString(const FText& UnsanitizedCategory)
{
	return FText::FromString(GetCategoryDisplayString(UnsanitizedCategory.ToString()));
}

//------------------------------------------------------------------------------
FString FEditorCategoryUtils::GetCategoryDisplayString(const FString& UnsanitizedCategory)
{
	FString DisplayString = UnsanitizedCategory;

	int32 KeyIndex = INDEX_NONE;
	do
	{
		KeyIndex = DisplayString.Find(TEXT("{"), ESearchCase::CaseSensitive, ESearchDir::FromStart, KeyIndex);
		if (KeyIndex != INDEX_NONE)
		{
			int32 EndIndex = DisplayString.Find(TEXT("}"), ESearchCase::CaseSensitive, ESearchDir::FromStart, KeyIndex);
			if (EndIndex != INDEX_NONE)
			{
				FString ToReplaceStr = FString::ConstructFromPtrSize(*DisplayString + KeyIndex, EndIndex+1 - KeyIndex);
				FString ReplacementStr;
				
				int32 KeyLen = EndIndex - (KeyIndex + 1);
				if (KeyLen > 0)
				{
					FString Key = FString::ConstructFromPtrSize(*DisplayString + KeyIndex+1, KeyLen);
					Key.TrimStartInline();
					ReplacementStr = FEditorCategoryUtilsImpl::GetCategory(*Key).ToString();
				}
				DisplayString.ReplaceInline(*ToReplaceStr, *ReplacementStr);
			}
			KeyIndex = EndIndex;
		}

	} while (KeyIndex != INDEX_NONE);

	DisplayString = FName::NameToDisplayString(DisplayString, /*bIsBool =*/false);
	DisplayString.ReplaceInline(TEXT("| "), TEXT("|"), ESearchCase::CaseSensitive);

	return DisplayString;
}

//------------------------------------------------------------------------------
void FEditorCategoryUtils::GetClassHideCategories(const UStruct* Class, TArray<FString>& CategoriesOut, bool bHomogenize)
{
	CategoriesOut.Empty();

	using namespace FEditorCategoryUtilsImpl;
	if (Class->HasMetaData(ClassHideCategoriesMetaKey))
	{
		const FString& HideCategories = Class->GetMetaData(ClassHideCategoriesMetaKey);

		HideCategories.ParseIntoArray(CategoriesOut, TEXT(" "), /*InCullEmpty =*/true);
		
		if (bHomogenize)
		{
			for (FString& Category : CategoriesOut)
			{
				Category = GetCategoryDisplayString(Category);
			}
		}
	}
}

//------------------------------------------------------------------------------
void  FEditorCategoryUtils::GetClassShowCategories(const UStruct* Class, TArray<FString>& CategoriesOut)
{
	CategoriesOut.Empty();

	using namespace FEditorCategoryUtilsImpl;
	if (Class->HasMetaData(ClassShowCategoriesMetaKey))
	{
		const FString& ShowCategories = Class->GetMetaData(ClassShowCategoriesMetaKey);
		ShowCategories.ParseIntoArray(CategoriesOut, TEXT(" "), /*InCullEmpty =*/true);

		for (FString& Category : CategoriesOut)
		{
			Category = GetCategoryDisplayString(FText::FromString(Category)).ToString();
		}
	}
}

//------------------------------------------------------------------------------
bool FEditorCategoryUtils::IsCategoryHiddenFromClass(const UStruct* Class, FCommonEditorCategory::EValue CategoryId)
{
	return IsCategoryHiddenFromClass(Class, GetCommonCategory(CategoryId));
}

//------------------------------------------------------------------------------
bool FEditorCategoryUtils::IsCategoryHiddenFromClass(const UStruct* Class, const FText& Category)
{
	return IsCategoryHiddenFromClass(Class, Category.ToString());
}

//------------------------------------------------------------------------------
bool FEditorCategoryUtils::IsCategoryHiddenFromClass(const UStruct* Class, const FString& Category)
{
	TArray<FString> ClassHideCategories;
	GetClassHideCategories(Class, ClassHideCategories);
	return IsCategoryHiddenFromClass(ClassHideCategories, Class, Category);
}

//------------------------------------------------------------------------------
bool FEditorCategoryUtils::IsCategoryHiddenFromClass(const TArray<FString>& ClassHideCategories, const UStruct* Class, const FString& Category)
{
	bool bIsHidden = false;
	if(ClassHideCategories.Num() == 0)
	{
		return bIsHidden;
	}

	// run the category through sanitization so we can ensure compares will hit
	const FString DisplayCategory = GetCategoryDisplayString(Category);

	for (const FString& HideCategory : ClassHideCategories)
	{
		bIsHidden = (HideCategory == DisplayCategory);
		if (bIsHidden)
		{
			TArray<FString> ClassShowCategories;
			GetClassShowCategories(Class, ClassShowCategories);
			// if they hid it, and showed it... favor showing (could be a shown in a sub-class, and hid in a super)
			bIsHidden = (ClassShowCategories.Find(DisplayCategory) == INDEX_NONE);
		}
		else // see if the category's root is hidden
		{
			TArray<FString> SubCategoryList;
			DisplayCategory.ParseIntoArray(SubCategoryList, TEXT("|"), /*InCullEmpty =*/true);

			FString FullSubCategoryPath;
			for (const FString& SubCategory : SubCategoryList)
			{
				FullSubCategoryPath += SubCategory;
				if ((HideCategory == SubCategory) || (HideCategory == FullSubCategoryPath))
				{
					TArray<FString> ClassShowCategories;
					GetClassShowCategories(Class, ClassShowCategories);
					// if they hid it, and showed it... favor showing (could be a shown in a sub-class, and hid in a super)
					bIsHidden = (ClassShowCategories.Find(DisplayCategory) == INDEX_NONE);
				}
				FullSubCategoryPath += "|";
			}
		}

		if (bIsHidden)
		{
			break;
		}
	}

	return bIsHidden;
}

//------------------------------------------------------------------------------
void FEditorCategoryUtils::GetCategoryTooltipInfo(const FString& Category, FText& Tooltip, FString& DocLink, FString& DocExcerpt)
{
	if (FEditorCategoryUtilsImpl::FCategoryInfo const* FoundCategory = FEditorCategoryUtilsImpl::GetCategoryTable().Find(Category))
	{
		DocLink = FoundCategory->DocLink;
		DocExcerpt = FoundCategory->DocExcerpt;
		Tooltip = FoundCategory->Tooltip;
	}
	else
	{
		// Fall back to some defaults
		DocLink = TEXT("Shared/GraphNodes/Blueprint/NodeCategories");
		DocExcerpt = Category;
		Tooltip = FEditorCategoryUtilsImpl::GetTooltipForCategory(GetCategoryDisplayString(Category), DocLink, DocExcerpt);
	}
}

//------------------------------------------------------------------------------
TSet<FString> FEditorCategoryUtils::GetHiddenCategories(const UStruct* Class)
{
	TArray<FString> ClassHiddenCategories;
	GetClassHideCategories(Class, ClassHiddenCategories);
	TArray<FString> ClassForceVisibleCategories;
	GetClassShowCategories(Class, ClassForceVisibleCategories);

	const TSet<FString> ClassHiddenCategoriesSet(ClassHiddenCategories);
	const TSet<FString> ClassForceVisibleCategoriesSet(ClassForceVisibleCategories);

	return ClassHiddenCategoriesSet.Difference(ClassForceVisibleCategoriesSet);
}

#undef LOCTEXT_NAMESPACE
