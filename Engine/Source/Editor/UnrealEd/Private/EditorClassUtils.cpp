// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorClassUtils.h"
#include "HAL/FileManager.h"
#include "Widgets/Layout/SSpacer.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Editor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "Containers/VersePath.h"
#include "IDocumentationPage.h"
#include "IDocumentation.h"
#include "SourceCodeNavigation.h"
#include "VerseVM/VVMVerseClass.h"
#include "Widgets/Input/SHyperlink.h"
#include "BlueprintEditorSettings.h"

FString FEditorClassUtils::GetDocumentationPage(const UClass* Class)
{
	return (Class ? FString::Printf( TEXT("Shared/Types/%s%s"), Class->GetPrefixCPP(), *Class->GetName() ) : FString());
}

FString FEditorClassUtils::GetDocumentationExcerpt(const UClass* Class)
{
	return (Class ? FString::Printf( TEXT("%s%s"), Class->GetPrefixCPP(), *Class->GetName() ) : FString());
}

TSharedRef<SToolTip> FEditorClassUtils::GetTooltip(const UClass* Class)
{
	FText ToolTipText;
	if (Class)
	{
		const bool bShowShortTooltips = GetDefault<UBlueprintEditorSettings>()->bShowShortTooltips;
		if (const UVerseClass* VerseClass = Cast<const UVerseClass>(Class))
		{
			static const FName NAME_Tooltip("Tooltip");
			static const FName NAME_ShortTooltip("ShortTooltip");
			if ((bShowShortTooltips && !VerseClass->GetMetaData(NAME_ShortTooltip).IsEmpty()) || !VerseClass->GetMetaData(NAME_Tooltip).IsEmpty())
			{
				ToolTipText = Class->GetToolTipText(bShowShortTooltips);
			}
			else
			{
				// If the tooltip is empty, GetToolTipText will use the class FName, but that is the mangled package relative path, so instead use the full Verse path for Verse classes.
				ToolTipText = VerseClass->GetVersePath().AsText();
			}
		}
		else
		{
			ToolTipText = Class->GetToolTipText(bShowShortTooltips);
		}
	}
	return GetTooltip(Class, ToolTipText);
}

TSharedRef<SToolTip> FEditorClassUtils::GetTooltip(const UClass* Class, const TAttribute<FText>& OverrideText)
{
	return (Class ? IDocumentation::Get()->CreateToolTip(OverrideText, nullptr, GetDocumentationPage(Class), GetDocumentationExcerpt(Class)) : SNew(SToolTip));
}

FString FEditorClassUtils::GetDocumentationLinkFromExcerpt(const FString& DocLink, const FString DocExcerpt)
{
	FString DocumentationLink;
	TSharedRef<IDocumentation> Documentation = IDocumentation::Get();
	if (Documentation->PageExists(DocLink))
	{
		TSharedRef<IDocumentationPage> ClassDocs = Documentation->GetPage(DocLink, NULL);

		FExcerpt Excerpt;
		if (ClassDocs->GetExcerpt(DocExcerpt, Excerpt))
		{
			FString* FullDocumentationLink = Excerpt.Variables.Find(TEXT("ToolTipFullLink"));
			if (FullDocumentationLink)
			{
				DocumentationLink = *FullDocumentationLink;
			}
		}
	}

	return DocumentationLink;
}

FString FEditorClassUtils::GetDocumentationLinkBaseUrlFromExcerpt(const FString& DocLink, const FString DocExcerpt)
{
	FString DocumentationLinkBaseUrl;
	TSharedRef<IDocumentation> Documentation = IDocumentation::Get();
	if (Documentation->PageExists(DocLink))
	{
		TSharedRef<IDocumentationPage> ClassDocs = Documentation->GetPage(DocLink, NULL);

		FExcerpt Excerpt;
		if (ClassDocs->GetExcerpt(DocExcerpt, Excerpt))
		{
			FString* DocumentationLinkBaseUrlValue = Excerpt.Variables.Find(TEXT("BaseUrl"));
			if (DocumentationLinkBaseUrlValue)
			{
				DocumentationLinkBaseUrl = *DocumentationLinkBaseUrlValue;
			}
		}
	}

	return DocumentationLinkBaseUrl;
}

FString FEditorClassUtils::GetDocumentationLink(const UClass* Class, const FString& OverrideExcerpt)
{
	const FString ClassDocsPage = GetDocumentationPage(Class);
	const FString ExcerptSection = (OverrideExcerpt.IsEmpty() ? GetDocumentationExcerpt(Class) : OverrideExcerpt);

	return GetDocumentationLinkFromExcerpt(ClassDocsPage, ExcerptSection);
}

FString FEditorClassUtils::GetDocumentationLinkBaseUrl(const UClass* Class, const FString& OverrideExcerpt)
{
	const FString ClassDocsPage = GetDocumentationPage(Class);
	const FString ExcerptSection = (OverrideExcerpt.IsEmpty() ? GetDocumentationExcerpt(Class) : OverrideExcerpt);

	return GetDocumentationLinkBaseUrlFromExcerpt(ClassDocsPage, ExcerptSection);
}


TSharedRef<SWidget> FEditorClassUtils::GetDocumentationLinkWidget(const UClass* Class)
{
	TSharedRef<SWidget> DocLinkWidget = SNullWidget::NullWidget;
	const FString DocumentationLink = GetDocumentationLink(Class);
	const FString DocumentationLinkBaseUrl = GetDocumentationLinkBaseUrl(Class);

	if (!DocumentationLink.IsEmpty())
	{
		DocLinkWidget = IDocumentation::Get()->CreateAnchor(DocumentationLink, FString(), FString(), DocumentationLinkBaseUrl);
	}

	return DocLinkWidget;
}

TSharedRef<SWidget> FEditorClassUtils::GetDynamicDocumentationLinkWidget(const TAttribute<const UClass*>& ClassAttribute)
{
	auto GetLink = [ClassAttribute]()
	{
		return GetDocumentationLink(ClassAttribute.Get(nullptr));
	};
	auto GetBaseUrl = [ClassAttribute]()
	{
		return GetDocumentationLinkBaseUrl(ClassAttribute.Get(nullptr));
	};
	return IDocumentation::Get()->CreateAnchor(TAttribute<FString>::CreateLambda(GetLink), FString(), FString(), TAttribute<FString>::CreateLambda(GetBaseUrl));
}

TSharedRef<SWidget> FEditorClassUtils::GetSourceLink(const UClass* Class, const FSourceLinkParams& Params)
{
	TSharedRef<SWidget> Link = SNew(SSpacer);

	UBlueprint* Blueprint = (Class ? Cast<UBlueprint>(Class->ClassGeneratedBy) : nullptr);
	if (Blueprint)
	{
		struct Local
		{
			static void OnEditBlueprintClicked(TWeakObjectPtr<UBlueprint> InBlueprint, TWeakObjectPtr<UObject> InAsset)
			{
				if (UBlueprint* BlueprintToEdit = InBlueprint.Get())
				{
					// Set the object being debugged if given an actor reference (if we don't do this before we edit the object the editor wont know we are debugging something)
					if (UObject* Asset = InAsset.Get())
					{
						check(Asset->GetClass()->ClassGeneratedBy == BlueprintToEdit);
						BlueprintToEdit->SetObjectBeingDebugged(Asset);
					}
					// Open the blueprint
					GEditor->EditObject(BlueprintToEdit);
				}
			}
		};

		TWeakObjectPtr<UBlueprint> BlueprintPtr = Blueprint;

		auto CanEditBlueprint = [BlueprintPtr]()
		{
			if (const UBlueprint* Blueprint = BlueprintPtr.Get())
			{
				return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->IsAssetEditable(Blueprint);
			}

			return false;
		};

		FText BlueprintNameText = FText::FromName(Blueprint->GetFName());

		FText FormattedText;
		if (Params.BlueprintFormat)
		{
			FormattedText = FText::Format(*Params.BlueprintFormat, BlueprintNameText);
		}
		else if (Params.bUseDefaultFormat)
		{
			const FText DefaultBlueprintFormat = NSLOCTEXT("SourceHyperlink", "EditBlueprint", "Edit {0}");
			FormattedText = FText::Format(DefaultBlueprintFormat, BlueprintNameText);
		}
		else
		{
			FormattedText = BlueprintNameText;
		}

		TSharedRef<SWidget> NoLinkWidget = SNullWidget::NullWidget;
		if (Params.bEmptyIfNoLink)
		{
			NoLinkWidget = SNew(SSpacer)
				.Visibility_Lambda([CanEditBlueprint]() { return CanEditBlueprint() ? EVisibility::Collapsed : EVisibility::Visible; });
		}
		else
		{
			NoLinkWidget = SNew(STextBlock)
				.Text(Params.bUseFormatIfNoLink ? FormattedText : BlueprintNameText)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Visibility_Lambda([CanEditBlueprint]() { return CanEditBlueprint() ? EVisibility::Collapsed : EVisibility::Visible; });
		}

		Link = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SHyperlink)
					.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
					.OnNavigate_Static(&Local::OnEditBlueprintClicked, BlueprintPtr, Params.Object)
					.Text(FormattedText)
					.ToolTipText(NSLOCTEXT("SourceHyperlink", "EditBlueprint_ToolTip", "Click to edit the blueprint"))
					.Visibility_Lambda([CanEditBlueprint]() { return CanEditBlueprint() ? EVisibility::Visible : EVisibility::Collapsed; })
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					NoLinkWidget
				];

	}
	else if (Class)
	{
		TWeakObjectPtr<const UClass> WeakClassPtr(Class);

		auto OnNavigateToClassCode = [WeakClassPtr]()
		{
			if (const UClass* Class = WeakClassPtr.Get())
			{
				FSourceCodeNavigation::NavigateToClass(Class);
			}
		};

		auto CanNavigateToClassCode = [WeakClassPtr]()
		{
			if (const UClass* Class = WeakClassPtr.Get())
			{
				return FSourceCodeNavigation::CanNavigateToClass(Class);
			}
			return false;
		};

		const FText ClassDisplayNameText = Class->GetDisplayNameText();
		const FText ClassNameText = FText::FromName(Class->GetFName());

		FText FormattedText;
		if (Params.CodeFormat)
		{
			FormattedText = FText::Format(*Params.CodeFormat, ClassDisplayNameText);
		}
		else if (Params.bUseDefaultFormat)
		{
			const FText DefaultCodeFormat = NSLOCTEXT("SourceHyperlink", "GoToCode", "Open {0}");
			FormattedText = FText::Format(DefaultCodeFormat, ClassDisplayNameText);
		}
		else
		{
			FormattedText = ClassDisplayNameText;
		}

		TSharedRef<SWidget> NoLinkWidget = SNullWidget::NullWidget;
		if (Params.bEmptyIfNoLink)
		{
			NoLinkWidget = SNew(SSpacer)
				.Visibility_Lambda([CanNavigateToClassCode]() { return CanNavigateToClassCode() ? EVisibility::Collapsed : EVisibility::Visible; });
		}
		else
		{
			NoLinkWidget = SNew(STextBlock)
				.Text(Params.bUseFormatIfNoLink ? FormattedText : ClassDisplayNameText)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Visibility_Lambda([CanNavigateToClassCode]() { return CanNavigateToClassCode() ? EVisibility::Collapsed : EVisibility::Visible; });
		}

		Link = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate_Lambda(OnNavigateToClassCode)
				.Text(FormattedText)
				.ToolTipText(FText::Format(NSLOCTEXT("SourceHyperlink", "GoToCode_ToolTip", "Click to open the source file for the underlying class ({0}) in {1}"), 
					ClassNameText, FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
				.Visibility_Lambda([CanNavigateToClassCode]() { return CanNavigateToClassCode() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				NoLinkWidget
			];
	}

	return Link;
}

TSharedRef<SWidget> FEditorClassUtils::GetSourceLink(const UClass* Class, const TWeakObjectPtr<UObject> ObjectWeakPtr)
{
	FSourceLinkParams Params;
	Params.Object = ObjectWeakPtr;
	Params.bUseDefaultFormat = true;
	Params.bEmptyIfNoLink = true;

	return GetSourceLink(Class, Params);
}

TSharedRef<SWidget> FEditorClassUtils::GetSourceLinkFormatted(const UClass* Class, const TWeakObjectPtr<UObject> ObjectWeakPtr, const FText& BlueprintFormat, const FText& CodeFormat)
{
	FSourceLinkParams Params;
	Params.Object = ObjectWeakPtr;
	Params.BlueprintFormat = &BlueprintFormat;
	Params.CodeFormat = &CodeFormat;
	Params.bEmptyIfNoLink = true;

	return GetSourceLink(Class, Params);
}

UClass* FEditorClassUtils::GetClassFromString(const FString& ClassName)
{
	if(ClassName.IsEmpty() || ClassName == TEXT("None"))
	{
		return nullptr;
	}

	UClass* Class = nullptr;
	if (!FPackageName::IsShortPackageName(ClassName))
	{
		Class = FindObject<UClass>(nullptr, *ClassName);
	}
	else
	{
		Class = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FEditorClassUtils::GetClassFromString"));
	}
	if(!Class)
	{
		Class = LoadObject<UClass>(nullptr, *ClassName);
	}
	return Class;
}

bool FEditorClassUtils::IsBlueprintAsset(const FAssetData& InAssetData, bool* bOutIsBPGC /*= nullptr*/)
{
	bool bIsBP = (InAssetData.AssetClassPath == UBlueprint::StaticClass()->GetClassPathName());
	bool bIsBPGC = (InAssetData.AssetClassPath == UBlueprintGeneratedClass::StaticClass()->GetClassPathName());

	if (!bIsBP && !bIsBPGC)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		TArray<FTopLevelAssetPath> AncestorClassNames;
		AssetRegistry.GetAncestorClassNames(InAssetData.AssetClassPath, AncestorClassNames);

		if (AncestorClassNames.Contains(UBlueprint::StaticClass()->GetClassPathName()))
		{
			bIsBP = true;
		}
		else if (AncestorClassNames.Contains(UBlueprintGeneratedClass::StaticClass()->GetClassPathName()))
		{
			bIsBPGC = true;
		}
	}

	if (bOutIsBPGC)
	{
		*bOutIsBPGC = bIsBPGC;
	}

	return bIsBP || bIsBPGC;
}

FTopLevelAssetPath FEditorClassUtils::GetClassPathNameFromAssetTag(const FAssetData& InAssetData)
{
	const FString GeneratedClassPath = InAssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
	return FTopLevelAssetPath(FPackageName::ExportTextPathToObjectPath(FStringView(GeneratedClassPath)));
}

FTopLevelAssetPath FEditorClassUtils::GetClassPathNameFromAsset(const FAssetData& InAssetData, bool bGenerateClassPathIfMissing /*= false*/)
{
	bool bIsBPGC = false;
	const bool bIsBP = IsBlueprintAsset(InAssetData, &bIsBPGC);

	if (bIsBPGC)
	{
		return FTopLevelAssetPath(InAssetData.GetSoftObjectPath().GetAssetPath());
	}
	else if (bIsBP)
	{
		FTopLevelAssetPath ClassPath = GetClassPathNameFromAssetTag(InAssetData);
		if (bGenerateClassPathIfMissing && ClassPath.IsNull())
		{
			FString ClassPathString = InAssetData.GetObjectPathString();
			ClassPathString += TEXT("_C");
			ClassPath = FTopLevelAssetPath(ClassPathString);
		}
		return ClassPath;
	}
	return FTopLevelAssetPath();
}

void FEditorClassUtils::GetImplementedInterfaceClassPathsFromAsset(const struct FAssetData& InAssetData, TArray<FString>& OutClassPaths)
{
	if (!InAssetData.IsValid())
	{
		return;
	}

	const FString ImplementedInterfaces = InAssetData.GetTagValueRef<FString>(FBlueprintTags::ImplementedInterfaces);
	if (!ImplementedInterfaces.IsEmpty())
	{
		// Parse string like "((Interface=Class'"/Script/VPBookmark.VPBookmarkProvider"'),(Interface=Class'"/Script/VPUtilities.VPContextMenuProvider"'))"
		// We don't want to actually resolve the hard ref so do some manual parsing

		FString FullInterface;
		FString CurrentString = *ImplementedInterfaces;
		while (CurrentString.Split(TEXT("Interface="), nullptr, &FullInterface))
		{
			// Cutoff at next )
			int32 RightParen = INDEX_NONE;
			if (FullInterface.FindChar(TCHAR(')'), RightParen))
			{
				// Keep parsing
				CurrentString = FullInterface.Mid(RightParen);

				// Strip class name
				FullInterface = *FPackageName::ExportTextPathToObjectPath(FullInterface.Left(RightParen));

				// Handle quotes
				FString InterfacePath;
				const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(*FullInterface, InterfacePath, true);

				if (NewBuffer)
				{
					OutClassPaths.Add(InterfacePath);
				}
			}
		}
	}
}
