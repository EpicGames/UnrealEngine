// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorCommandLineUtils.h"

#include "AssetDefinitionRegistry.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Stats/Stats.h"
#include "Misc/App.h"
#include "Misc/AssetRegistryInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "Misc/PackageName.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "TickableEditorObject.h"
#include "Commandlets/Commandlet.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "Dialogs/Dialogs.h"
#include "Interfaces/IMainFrameModule.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "ProjectDescriptor.h"
#include "UObject/SavePackage.h"

#define LOCTEXT_NAMESPACE "EditorCommandLineUtils"

// Forward Declarations
struct FMergeAsset;

/*******************************************************************************
 * EditorCommandLineUtilsImpl Declaration
 ******************************************************************************/

/**  */
namespace EditorCommandLineUtilsImpl
{
	static const TCHAR* DebugLightmassCommandSwitch = TEXT("LIGHTMASSDEBUG");
	static const TCHAR* LightmassStatsCommandSwitch = TEXT("LIGHTMASSSTATS");

	static const TCHAR* DiffCommandSwitch  = TEXT("diff");
	static const FText  DiffCommandHelpTxt = INVTEXT("\
Usage: \n\
    -diff [options] left right                                                 \n\
    -diff [options] remote local base result                                   \n\
\n\
Options: \n\
    -echo               Prints back the command arguments and then exits.      \n\
    -help, -h, -?       Display this message and then exits.                   \n");

	/**  */
	static bool ParseCommandArgs(const TCHAR* FullEditorCmdLine, const TCHAR* CmdSwitch, FString& CmdArgsOut);

	/** */
	static FString FindProjectFile(const FString& AssetFilePath);

	/**  */
	static void RaiseEditorMessageBox(const FText& Title, const FText& BodyText, const bool bExitOnClose);

	/**  */
	static void ForceCloseEditor();
	
	/**  */
	static void RunAssetDiffCommand(TSharedPtr<SWindow> MainEditorWindow, bool bIsRunningProjBrowser, FString CommandArgs);

	/**  */
	static void RunAssetMerge(FMergeAsset const& Base, FMergeAsset const& Remote, FMergeAsset const& Local, FMergeAsset const& Result);

	/**  */
	static UObject* ExtractAssetFromPackage(UPackage* Package);
}

/*******************************************************************************
 * FCommandLineErrorReporter
 ******************************************************************************/
 
/**  */
struct FCommandLineErrorReporter
{
	FCommandLineErrorReporter(const FString& Command, const FString& CommandArgs)
		: CommandSwitch(FText::FromString(Command))
		, FullCommand(FText::FromString("-" + Command + " " + CommandArgs))
		, bHasBlockingError(false)
	{}

	/**  */
	void ReportFatalError(const FText& Title, const FText& ErrorMsg);

	/**  */
	void ReportError(const FText& Title, const FText& ErrorMsg, bool bIsFatal);

	/**  */
	bool HasBlockingError() const;

private:
	FText CommandSwitch;
	FText FullCommand;
	bool  bHasBlockingError;
};

//------------------------------------------------------------------------------
void FCommandLineErrorReporter::ReportFatalError(const FText& Title, const FText& ErrorMsg)
{
	ReportError(Title, ErrorMsg, /*bIsFatal =*/true);
}


//------------------------------------------------------------------------------
void FCommandLineErrorReporter::ReportError(const FText& Title, const FText& ErrorMsg, bool bIsFatal)
{
	if (bHasBlockingError)
	{
		return;
	}

	FText FullErrorMsg = FText::Format(LOCTEXT("CommandLineError", "Erroneous editor command: {0}\n\n{1}\n\nRun '-{2} -h' for more help."),
		FullCommand, ErrorMsg, CommandSwitch);

	bHasBlockingError = bIsFatal;
	EditorCommandLineUtilsImpl::RaiseEditorMessageBox(Title, FullErrorMsg, /*bShutdownOnOk =*/bIsFatal);
}

//------------------------------------------------------------------------------
bool FCommandLineErrorReporter::HasBlockingError() const
{
	return bHasBlockingError;
}

/*******************************************************************************
 * FFauxStandaloneToolManager Implementation
 ******************************************************************************/

/**  
 * Helps keep up the facade for tools can launch "stand-alone"... Hides the main 
 * editor window, and monitors for when all visible windows are closed (so it
 * can kill the editor process).
 */
class FFauxStandaloneToolManager : FTickableEditorObject
{
public:
	FFauxStandaloneToolManager(TSharedPtr<SWindow> MainEditorWindow);

	// FTickableEditorObject interface
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual void Tick(float DeltaTime) override;	
	// End FTickableEditorObject interface

	/**  */
	void Disable();

private:
	TWeakPtr<SWindow> MainEditorWindow;
};

//------------------------------------------------------------------------------
FFauxStandaloneToolManager::FFauxStandaloneToolManager(TSharedPtr<SWindow> InMainEditorWindow)
	: MainEditorWindow(InMainEditorWindow)
{
	// present the illusion that this is a stand-alone editor by hiding the 
	// root level editor window
	InMainEditorWindow->HideWindow();
}

//------------------------------------------------------------------------------
TStatId FFauxStandaloneToolManager::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FFauxStandaloneToolManager, STATGROUP_Tickables);
}

//------------------------------------------------------------------------------
void FFauxStandaloneToolManager::Tick(float DeltaTime)
{
	if (MainEditorWindow.IsValid())
	{	
		FSlateApplication& WindowManager = FSlateApplication::Get();
		TArray< TSharedRef<SWindow> > ActiveWindows = WindowManager.GetInteractiveTopLevelWindows();

		bool bVisibleWindowFound = false;
		for (TSharedRef<SWindow> Window : ActiveWindows)
		{
			if (Window->IsVisible())
			{
				bVisibleWindowFound = true;
				break;
			}
		}

		if (!bVisibleWindowFound)
		{
			EditorCommandLineUtilsImpl::ForceCloseEditor();
		}
	}
	else
	{
		EditorCommandLineUtilsImpl::ForceCloseEditor();
	}
}

//------------------------------------------------------------------------------
void FFauxStandaloneToolManager::Disable()
{
	if (MainEditorWindow.IsValid())
	{
		MainEditorWindow.Pin()->ShowWindow();
	}
}

/*******************************************************************************
 * FMergeAsset
 ******************************************************************************/
 
struct FMergeAsset
{
public:
	FMergeAsset(const TCHAR* DstFileName);

	/**  */
	bool SetSourceFile(const FString& SrcFilePathIn, FCommandLineErrorReporter& ErrorReporter);

	/**  */
	bool Load(FCommandLineErrorReporter& ErrorReporter);

	/**  */
	UClass* GetClass() const;

	/**  */
	UObject* GetAssetObj() const;

	/**  */
	FRevisionInfo GetRevisionInfo() const;

	/**  */
	const FString& GetSourceFilePath() const;

	/**  */
	const FString& GetAssetFilePath() const;

	UPackage* GetPackage() const;

private:
	UPackage* Package;
	UObject*  AssetObj;
	FString   DestFilePath;
	FString   SrcFilePath;
};


//------------------------------------------------------------------------------
FMergeAsset::FMergeAsset(const TCHAR* DstFileName)
	: Package(nullptr)
	, AssetObj(nullptr)
	, DestFilePath(FPaths::Combine(*FPaths::DiffDir(), DstFileName))
{
	const FString& AssetExt = FPackageName::GetAssetPackageExtension();
	if (!DestFilePath.EndsWith(AssetExt))
	{
		DestFilePath += AssetExt;
	}
}

//------------------------------------------------------------------------------
bool FMergeAsset::SetSourceFile(const FString& SrcFilePathIn, FCommandLineErrorReporter& ErrorReporter)
{
	SrcFilePath.Empty();
	if (!FPaths::FileExists(SrcFilePathIn))
	{
		ErrorReporter.ReportFatalError(LOCTEXT("BadFilePathTitle", "Bad File Path"),
			FText::Format(LOCTEXT("BadFilePathError", "'{0}' is an invalid file."), FText::FromString(SrcFilePathIn)));
	}
	else
	{
		SrcFilePath = SrcFilePathIn;
	}
	return !SrcFilePath.IsEmpty();
}

//------------------------------------------------------------------------------
bool FMergeAsset::Load(FCommandLineErrorReporter& ErrorReporter)
{
	if (SrcFilePath.IsEmpty())
	{
		// was SetSourceFile() called prior to this?
		return false;
	}

	// UE cannot open files with certain special characters in them (like 
	// the # symbol), so we make a copy of the file with a more UE digestible 
	// path (since this may be a perforce temp file)
	if (IFileManager::Get().Copy(*DestFilePath, *SrcFilePath) != COPY_OK)
	{
		ErrorReporter.ReportFatalError(LOCTEXT("LoadFailedTitle", "Unable to Copy File"),
			FText::Format(LOCTEXT("LoadFailedError", "Failed to make a local copy of the asset file: '{0}'."), FText::FromString(SrcFilePath)));
	}
	else if (UPackage* AssetPkg = LoadPackage(/*Outer =*/nullptr, *DestFilePath, LOAD_None))
	{
		if (UObject* ExtractedAsset = EditorCommandLineUtilsImpl::ExtractAssetFromPackage(AssetPkg))
		{
			Package  = AssetPkg;
			AssetObj = ExtractedAsset;
		}
		else
		{
			ErrorReporter.ReportFatalError(LOCTEXT("AssetNotFoundTitle", "Asset Not Found"),
				FText::Format(LOCTEXT("AssetNotFoundError", "Failed to find the asset object inside the package file: '{0}'."), FText::FromString(SrcFilePath)));
		}
	}

	return (AssetObj != nullptr);
}

//------------------------------------------------------------------------------
UClass* FMergeAsset::GetClass() const
{
	return (AssetObj != nullptr) ? AssetObj->GetClass() : nullptr;
}

//------------------------------------------------------------------------------
UObject* FMergeAsset::GetAssetObj() const
{
	return AssetObj;
}

//------------------------------------------------------------------------------
FRevisionInfo FMergeAsset::GetRevisionInfo() const
{
	FString SrcFileName = FPaths::GetBaseFilename(SrcFilePath);

	FRevisionInfo RevisionInfoOut = FRevisionInfo::InvalidRevision();

	FString BaseFileName, RevisionStr;
	if (SrcFileName.Split(TEXT("#"), &BaseFileName, &RevisionStr))
	{
		// @TODO: if connected to source-control, extract changelist and date info
		RevisionInfoOut.Revision = *RevisionStr;
	}

	return RevisionInfoOut;
}

//------------------------------------------------------------------------------
const FString& FMergeAsset::GetSourceFilePath() const
{
	return SrcFilePath;
}

//------------------------------------------------------------------------------
const FString& FMergeAsset::GetAssetFilePath() const
{
	return DestFilePath;
}

UPackage* FMergeAsset::GetPackage() const
{
	return Package;
}

/*******************************************************************************
 * EditorCommandLineUtilsImpl Implementation
 ******************************************************************************/

//------------------------------------------------------------------------------
static bool EditorCommandLineUtilsImpl::ParseCommandArgs(const TCHAR* FullEditorCmdLine, const TCHAR* CmdSwitch, FString& CmdArgsOut)
{
	if (FParse::Param(FullEditorCmdLine, CmdSwitch))
	{
		FString CmdPrefix;
		FString(FullEditorCmdLine).Split(FString("-") + CmdSwitch, &CmdPrefix, &CmdArgsOut);
		return true;
	}
	return false;
}

//------------------------------------------------------------------------------
static FString EditorCommandLineUtilsImpl::FindProjectFile(const FString& AssetFilePathIn)
{
	FString FoundProjectPath;

	FString AssetFilePath = AssetFilePathIn;
	FPaths::NormalizeFilename(AssetFilePath);	

	const TCHAR* const ContentDirName = TEXT("/Content/");

	FString ProjectDir, AssetSubPath;
	if (AssetFilePath.Split(ContentDirName, &ProjectDir, &AssetSubPath))
	{
		const FString UProjExt = TEXT(".") + FProjectDescriptor::GetExtension();
		const FString ProjectWildcardPath = FPaths::Combine(*ProjectDir, *FString(TEXT("*") + UProjExt));

		TArray<FString> FoundFiles;
		IFileManager::Get().FindFiles(FoundFiles, *ProjectWildcardPath, /*Files =*/true, /*Directories =*/false);

		if (FoundFiles.Num() > 0)
		{
			FoundProjectPath = FPaths::Combine(*ProjectDir, *FoundFiles[0]);

			const FString DirName = FPaths::GetBaseFilename(ProjectDir);
			for (FString FileName : FoundFiles)
			{
				// favor project files that match the directory name
				if (FPaths::GetBaseFilename(FileName) == DirName)
				{
					FoundProjectPath = FPaths::Combine(*ProjectDir, *FileName);
					break;
				}
			}
		}
		else 
		{
			// guess at what the project path would be (in case this is a  
			// perforce temp file, and its path mimics the real asset file's 
			// directory structure)
			FString GameName = FPaths::GetCleanFilename(ProjectDir);
			FoundProjectPath = FPaths::Combine(*FPaths::RootDir(), *GameName, *FString(GameName + UProjExt));
			// make sure what we're guessing at exists...
			if (!FPaths::FileExists(FoundProjectPath))
			{
				FoundProjectPath.Empty();
			}
		}
	}

	return FoundProjectPath;
}

//------------------------------------------------------------------------------
static void EditorCommandLineUtilsImpl::RaiseEditorMessageBox(const FText& Title, const FText& BodyText, const bool bExitOnClose)
{
	FOnMsgDlgResult OnDialogClosed;
	if (bExitOnClose)
	{
		auto OnDialogClosedLambda = [](const TSharedRef<SWindow>&, EAppReturnType::Type)
		{
			ForceCloseEditor();
		};
		OnDialogClosed = FOnMsgDlgResult::CreateStatic(OnDialogClosedLambda);
	}

	OpenMsgDlgInt_NonModal(EAppMsgType::Ok, BodyText, Title, OnDialogClosed)->ShowWindow();
}

//------------------------------------------------------------------------------
static void EditorCommandLineUtilsImpl::ForceCloseEditor()
{
	// We used to call IMainFrameModule::RequestCloseEditor, but that runs a lot of logic that should only be
	// run for the real project editor (notably UThumbnailManager::CaptureProjectThumbnail was causing a crash on shutdown
	// but INI serialization was running when it should not have as well). Instead, we just raise the QUIT_EDITOR command:
	GEngine->DeferredCommands.Add(TEXT("QUIT_EDITOR"));
}

//------------------------------------------------------------------------------
static void EditorCommandLineUtilsImpl::RunAssetDiffCommand(TSharedPtr<SWindow> MainEditorWindow, bool bIsRunningStartupDialog, FString CommandArgs)
{
	// if the editor is running the project browser (or another like dialog on startup), 
	// then the user has to first make a selection (and then the editor will re-launch with this command).
	if (bIsRunningStartupDialog)
	{
		// @TODO: can we run without loading a project?
		return;
	}

	// static so it exists past this function, but doesn't get instantiated  
	// until this function is called
	static FFauxStandaloneToolManager FauxStandaloneToolManager(MainEditorWindow);

	TMap<FString, FString> Params;
	TArray<FString> Tokens;
	TArray<FString> Switches;
	UCommandlet::ParseCommandLine(*CommandArgs, Tokens, Switches, Params);

	if (Switches.Contains("h") ||
		Switches.Contains("?") ||
		Switches.Contains("help"))
	{
		RaiseEditorMessageBox(LOCTEXT("DiffCommandHelp", "Diff/Merge Command-Line Help"), DiffCommandHelpTxt, /*bExitOnClose =*/true);
		return;
	}

	if (Switches.Contains("echo"))
	{
		RaiseEditorMessageBox(LOCTEXT("PassedCommandArgs", "Passed Command Arguments"), 
			FText::FromString(CommandArgs), /*bExitOnClose =*/true);
		return;
	}

	const int32 FilesNeededForDiff  = 2;
	const int32 FilesNeededForMerge = 4;
	const int32 MaxFilesNeeded = FilesNeededForMerge;

	FMergeAsset MergeAssets[MaxFilesNeeded] = {
		FMergeAsset(TEXT("MergeTool-Left")),
		FMergeAsset(TEXT("MergeTool-Right")),
		FMergeAsset(TEXT("MergeTool-Base")),
		FMergeAsset(TEXT("MergeTool-Merge")),
	};
	FMergeAsset& LeftAsset   = MergeAssets[0];
	FMergeAsset& ThierAsset  = LeftAsset;
	FMergeAsset& RightAsset  = MergeAssets[1];
	FMergeAsset& OurAsset    = RightAsset;
	FMergeAsset& BaseAsset   = MergeAssets[2];
	FMergeAsset& MergeResult = MergeAssets[3];

	//--------------------------------------
	// Parse file paths from command-line
	//--------------------------------------

	FCommandLineErrorReporter ErrorReporter(DiffCommandSwitch, CommandArgs);

	int32 ParsedFileCount = 0;
	for (int32 FileIndex = 0; FileIndex < Tokens.Num() && ParsedFileCount < MaxFilesNeeded; ++FileIndex)
	{
		FString& FilePath = Tokens[FileIndex];

		FMergeAsset& MergeAsset = MergeAssets[ParsedFileCount];
		if (MergeAsset.SetSourceFile(FilePath, ErrorReporter))
		{
			++ParsedFileCount;
		}
	}

	//--------------------------------------
	// Verify file count
	//--------------------------------------

	const bool bWantsMerge = (ParsedFileCount > FilesNeededForDiff);
	if (ParsedFileCount < FilesNeededForDiff)
	{
		ErrorReporter.ReportFatalError(LOCTEXT("TooFewParamsTitle", "Too Few Parameters"),
			LOCTEXT("TooFewParamsError", "At least two files are needed (for a diff)."));
	}
	else if (bWantsMerge && (ParsedFileCount < FilesNeededForMerge))
	{
		ErrorReporter.ReportFatalError(LOCTEXT("TooFewParamsTitle", "Too Few Parameters"),
			LOCTEXT("TooFewMergeParamsError", "To merge, at least two files are needed."));
	}
	else if (Tokens.Num() > FilesNeededForMerge)
	{
		ErrorReporter.ReportFatalError(LOCTEXT("TooManyParamsTitle", "Too Many Parameters"),
			FText::Format( LOCTEXT("TooManyParamsError", "There were too many command arguments supplied. The maximum files needed are {0} (for merging)"), FText::AsNumber(FilesNeededForMerge) ));
	}

	//--------------------------------------
	// Load diff/merge asset files
	//--------------------------------------

	bool bLoadSuccess = true;
	if (bWantsMerge)
	{
		bLoadSuccess &= ThierAsset.Load(ErrorReporter);
		bLoadSuccess &= OurAsset.Load(ErrorReporter);
		bLoadSuccess &= BaseAsset.Load(ErrorReporter);
	}
	else
	{
		bLoadSuccess &= LeftAsset.Load(ErrorReporter);
		bLoadSuccess &= RightAsset.Load(ErrorReporter);
	}

	//--------------------------------------
	// Verify asset types
	//--------------------------------------

	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	if (bLoadSuccess)
	{
		if (LeftAsset.GetClass() != RightAsset.GetClass())
		{
			ErrorReporter.ReportFatalError(LOCTEXT("TypeMismatchTitle", "Asset Type Mismatch"),
				LOCTEXT("TypeMismatchError", "Cannot compare files of different asset types."));
		}
		else if (bWantsMerge)
		{
			UClass* AssetClass = OurAsset.GetClass();
			TWeakPtr<IAssetTypeActions> AssetActions = AssetTools.GetAssetTypeActionsForClass(AssetClass);

			if (AssetClass != BaseAsset.GetClass())
			{
				ErrorReporter.ReportFatalError(LOCTEXT("TypeMismatchTitle", "Asset Type Mismatch"),
					LOCTEXT("MergeTypeMismatchError", "Cannot merge files of different asset types."));
			}
			else if(!AssetActions.IsValid() || !AssetActions.Pin()->CanMerge())
			{
				ErrorReporter.ReportFatalError(LOCTEXT("CannotMergeTitle", "Cannot Merge"),
					FText::Format(LOCTEXT("CannotMergeError", "{0} asset files can not be merged."), FText::FromName(AssetClass->GetFName())));
			}
		}
	}

	//--------------------------------------
	// Preform diff/merge
	//--------------------------------------

	if (bLoadSuccess && !ErrorReporter.HasBlockingError())
	{
		if (bWantsMerge)
		{
			// unlike with diffing, for merging we rely on asset editors for
			// merging, and those windows get childed to the main window (so it
			// needs to be visible)
			//
			// @TODO: get it so asset editor windows can be shown standalone
			FauxStandaloneToolManager.Disable();

			RunAssetMerge(BaseAsset, ThierAsset, OurAsset, MergeResult);
		}
		else
		{
			AssetTools.DiffAssets(LeftAsset.GetAssetObj(), RightAsset.GetAssetObj(), LeftAsset.GetRevisionInfo(), RightAsset.GetRevisionInfo());
		}
	}
}

//------------------------------------------------------------------------------
static void EditorCommandLineUtilsImpl::RunAssetMerge(FMergeAsset const& Base, FMergeAsset const& Remote, FMergeAsset const& Local, FMergeAsset const& Result)
{
	class FMergeResolutionHandler : public TSharedFromThis<FMergeResolutionHandler>
	{
	public:
		FMergeResolutionHandler(FMergeAsset const& LocalIn, const FString& DstFilePathIn)
			: MergingAsset(LocalIn)
			, Resolution(EAssetMergeResult::Unknown)
			, DstFilePath(DstFilePathIn)
		{
			// force the user to save the result file (so we know if they "accepted" the merge)
			MergingAsset.GetPackage()->SetDirtyFlag(true);
		}

		/** Records the user's selected resolution, and closes the editor. */
		void HandleMergeResolution(UPackage* MergedPackageIn, EAssetMergeResult ResolutionIn)
		{
			if (MergedPackageIn == MergingAsset.GetPackage())
			{
				if (ResolutionIn == EAssetMergeResult::Cancelled)
				{
					// they don't want to save any changes, so clear the flag
					MergingAsset.GetPackage()->SetDirtyFlag(false);
				}
				
				if (Resolution == EAssetMergeResult::Unknown)
				{
					Resolution = ResolutionIn;
					EditorCommandLineUtilsImpl::ForceCloseEditor();
				}

				if (ResolutionIn == EAssetMergeResult::Completed)
				{
					FSavePackageArgs SaveArgs;
					SaveArgs.TopLevelFlags = RF_Standalone;
					SaveArgs.Error = GLog;
					UPackage::SavePackage(MergingAsset.GetPackage(), MergingAsset.GetAssetObj(), *DstFilePath, SaveArgs);
				}
			}
		}

	private:
		FMergeAsset MergingAsset;
		EAssetMergeResult Resolution;
		FString DstFilePath;
	};
	
	const FString& ResultFilePath = (!Result.GetSourceFilePath().IsEmpty()) ? Result.GetSourceFilePath() : Local.GetSourceFilePath();
	TSharedRef<FMergeResolutionHandler> MergeHandler = MakeShareable(new FMergeResolutionHandler(Local, ResultFilePath));

	// we use a lambda delegate to route the call into MergeHandler (we require 
	// this intermediate to hold onto a MergeHandler ref, so it doesn't get 
	// prematurely destroyed at the end of this function)
	auto HandleMergeResolution = [MergeHandler](const FAssetMergeResults& Results)
	{
		MergeHandler->HandleMergeResolution(Results.MergedPackage, Results.Result);
	};
	const FOnAssetMergeResolved MergeResolutionDelegate = FOnAssetMergeResolved::CreateLambda(HandleMergeResolution);

	// have to mount the save directory so that the BP-editor can save
	// the merged asset packages
	FPackageName::RegisterMountPoint(TEXT("/Temp/"), FPaths::ProjectSavedDir());
	
	const UClass* AssetClass = Local.GetClass();
	check(AssetClass != nullptr);
	// bring up the merge tool...
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetClass);
	if (AssetDefinition->CanMerge())
	{
		FAssetManualMergeArgs MergeArgs;
		MergeArgs.BaseAsset = Base.GetAssetObj();
		MergeArgs.RemoteAsset = Remote.GetAssetObj();
		MergeArgs.LocalAsset = Local.GetAssetObj();
		MergeArgs.ResolutionCallback = MergeResolutionDelegate;
		MergeArgs.Flags = MF_NO_GUI;
		AssetDefinition->Merge(MergeArgs);
	}
}

//------------------------------------------------------------------------------
static UObject* EditorCommandLineUtilsImpl::ExtractAssetFromPackage(UPackage* Package)
{
	return Package->FindAssetInPackage();
}

/*******************************************************************************
 * FEditorCommandLineUtils Definition
 ******************************************************************************/

//------------------------------------------------------------------------------
bool FEditorCommandLineUtils::ParseGameProjectPath(const TCHAR* CmdLine, FString& ProjPathOut, FString& GameNameOut)
{
	using namespace EditorCommandLineUtilsImpl; // for ParseCommandArgs(), etc.

	FString DiffArgs;
	if (ParseCommandArgs(CmdLine, DiffCommandSwitch, DiffArgs))
	{
		TArray<FString> Tokens, Switches;
		UCommandlet::ParseCommandLine(*DiffArgs, Tokens, Switches);

		for (FString FilePath : Tokens)
		{
			FPaths::NormalizeFilename(FilePath);
			ProjPathOut = FindProjectFile(FilePath);

			if (!ProjPathOut.IsEmpty())
			{
				GameNameOut = FPaths::GetBaseFilename(ProjPathOut);
				// favor project files that are in the same directory tree as 
				// the supplied file
				if ( FilePath.StartsWith(FPaths::GetPath(ProjPathOut)) )
				{
					break;
				}
			}
		}
	}
	return FPaths::FileExists(ProjPathOut);
}

//------------------------------------------------------------------------------
void FEditorCommandLineUtils::ProcessEditorCommands(const TCHAR* EditorCmdLine)
{
	using namespace EditorCommandLineUtilsImpl; // for DiffCommandSwitch, etc.

	// If specified, Lightmass has to be launched manually with -debug (e.g. through a debugger).
	// This creates a job with a hard-coded GUID, and allows Lightmass to be executed multiple times (even stand-alone).
	if (FParse::Param(EditorCmdLine, DebugLightmassCommandSwitch))
	{
		extern bool GLightmassDebugMode;
		GLightmassDebugMode = true;
		UE_LOGF(LogInit, Log, "Running Engine with Lightmass Debug Mode ENABLED");
	}

	// If specified, all participating Lightmass agents will report back detailed stats to the log.
	if (FParse::Param(EditorCmdLine, LightmassStatsCommandSwitch))
	{
		extern bool GLightmassStatsMode;
		GLightmassStatsMode = true;
		UE_LOGF(LogInit, Log, "Running Engine with Lightmass Stats Mode ENABLED");
	}

	FString DiffArgs;
	if (ParseCommandArgs(EditorCmdLine, DiffCommandSwitch, DiffArgs))
	{
		IMainFrameModule& MainFrameModule = IMainFrameModule::Get();
		const bool bIsMainFramInitialized = MainFrameModule.IsWindowInitialized();

		if (bIsMainFramInitialized)
		{
			RunAssetDiffCommand(MainFrameModule.GetParentWindow(), /*bIsNewProjectWindow =*/FApp::IsProjectNameEmpty(), DiffArgs);
		}
		else
		{
			MainFrameModule.OnMainFrameCreationFinished().AddStatic(&RunAssetDiffCommand, DiffArgs);
		}
	}
}

#undef LOCTEXT_NAMESPACE
