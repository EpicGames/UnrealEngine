// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorComponents.h"
#include "EngineDefines.h"
#include "HAL/IConsoleManager.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "EngineGlobals.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/Selection.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "EditorModeManager.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/AxisDisplayInfo.h"

static TAutoConsoleVariable<int32> CVarEditorNewLevelGrid(
	TEXT("r.Editor.NewLevelGrid"),
	2,
	TEXT("Wether to show the new editor level grid\n")
	TEXT("0: off\n")
	TEXT("1: Analytical Antialiasing\n")
	TEXT("2: Texture based(default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor2DGridFade(
	TEXT("r.Editor.2DGridFade"),
	0.15f,
	TEXT("Tweak to define the grid rendering in 2D viewports."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor2DSnapFade(
	TEXT("r.Editor.2DSnapFade"),
	0.3f,
	TEXT("Tweak to define the grid rendering in 2D viewports."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor3DGridFade(
	TEXT("r.Editor.3DGridFade"),
	0.5f,
	TEXT("Tweak to define the grid rendering in 3D viewports."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor3DSnapFade(
	TEXT("r.Editor.3DSnapFade"),
	0.35f,
	TEXT("Tweak to define the grid rendering in 3D viewports."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor2DSnapMin(
	TEXT("r.Editor.2DSnapMin"),
	0.25f,
	TEXT("Tweak to define the grid rendering in 2D viewports."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarEditor2DSnapScale(
	TEXT("r.Editor.2DSnapScale"),
	10.0f,
	TEXT("Tweak to define the grid rendering in 2D viewports."),
	ECVF_RenderThreadSafe);

static bool IsEditorCompositingMSAAEnabled(ERHIFeatureLevel::Type InFeatureLevel)
{
	bool Ret = false;

	if (InFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		// only supported on SM5 yet
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAA.CompositingSampleCount"));

		Ret = CVar->GetValueOnGameThread() > 1;
	}

	return Ret;
}

/*------------------------------------------------------------------------------
FGridWidget.
------------------------------------------------------------------------------*/

FGridWidget::FGridWidget()
	: GPUBasedGridMaterial(FSoftObjectPath(TEXT("/Engine/EditorMaterials/LevelGridMaterial.LevelGridMaterial")))
	, TextureBasedLevelGridMaterial(FSoftObjectPath(TEXT("/Engine/EditorMaterials/LevelGridMaterial2.LevelGridMaterial2")))
	, LevelGridMaterialInst(nullptr)
{
}

void FGridWidget::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( LevelGridMaterialInst );
}

UMaterial* FGridWidget::GetActiveLevelGridMaterial()
{
	bool bUseTextureSolution = CVarEditorNewLevelGrid.GetValueOnGameThread() > 1;

	if (bUseTextureSolution)
	{
		return TextureBasedLevelGridMaterial.LoadSynchronous();
	}
	
	return GPUBasedGridMaterial.LoadSynchronous();
}

UMaterialInstanceDynamic* FGridWidget::GetActiveLevelGridMID()
{
	bool bUseTextureSolution = CVarEditorNewLevelGrid.GetValueOnGameThread() > 1;

	UMaterial* BaseGridMat = GetActiveLevelGridMaterial();

	if (!LevelGridMaterialInst || LevelGridMaterialInst->Parent != BaseGridMat)
	{
		LevelGridMaterialInst = UMaterialInstanceDynamic::Create(BaseGridMat, nullptr);
	}

	return LevelGridMaterialInst;
}

static void GetAxisColors(FLinearColor Out[3], bool bIsPerspective)
{
	Out[0] = AxisDisplayInfo::GetAxisColor(EAxisList::Forward);
	Out[1] = AxisDisplayInfo::GetAxisColor(EAxisList::Left);
	Out[2] = AxisDisplayInfo::GetAxisColor(EAxisList::Up);

	// less prominent axis lines for perspective views
	if (bIsPerspective)
	{
		for (int i = 0; i < 3; ++i)
		{
			Out[i] += FLinearColor(0.2f, 0.2f, 0.2f, 0);
			Out[i] *= 0.3f;
		}
	}
}

static double FmodFloor(double X, double Y)
{
	const double Div = (X / Y);
	const double IntPortion = Y * FMath::FloorToDouble(Div);
	const double Result = X - IntPortion;
	return Result;
}

void FGridWidget::DrawNewGrid(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	UMaterial* GridMaterial = GetActiveLevelGridMaterial();

	if (GridMaterial->IsCompilingOrHadCompileError(View->GetShaderPlatform()))
	{
		// The material would appear to be black (because we don't use a MaterialDomain but a UsageFlag - we should change that).
		// Here we rather want to hide it.
		return;
	}

	UMaterialInstanceDynamic* MaterialInst = GetActiveLevelGridMID();

	if(!MaterialInst)
	{
		return;
	}

	bool bMSAA = IsEditorCompositingMSAAEnabled(View->GetFeatureLevel());
	bool bIsPerspective = ( View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f );

	// in unreal units
	float SnapGridSize = GEditor->GetGridSize();

	// not used yet
	const bool bSnapEnabled = GetDefault<ULevelEditorViewportSettings>()->GridEnabled;

	float SnapAlphaMultiplier = 1.0f;

	// to get a light grid in a black level but use a high opacity value to be able to see it in a bright level
	static float Darken = 0.11f;

	static FName GridColorName("GridColor");
	static FName SnapColorName("SnapColor");
	static FName ExponentName("Exponent");
	static FName AlphaBiasName("AlphaBias");
	
	if(bIsPerspective)
	{
		MaterialInst->SetVectorParameterValue(GridColorName, FLinearColor(0.6f * Darken, 0.6f * Darken, 0.6f * Darken, CVarEditor3DGridFade.GetValueOnGameThread()));
		MaterialInst->SetVectorParameterValue(SnapColorName, FLinearColor(0.5f, 0.0f, 0.0f, SnapAlphaMultiplier * CVarEditor3DSnapFade.GetValueOnGameThread()));
	}
	else
	{
		MaterialInst->SetVectorParameterValue(GridColorName, FLinearColor(0.6f * Darken, 0.6f * Darken, 0.6f * Darken, CVarEditor2DGridFade.GetValueOnGameThread()));
		MaterialInst->SetVectorParameterValue(SnapColorName, FLinearColor(0.5f, 0.0f, 0.0f, SnapAlphaMultiplier * CVarEditor2DSnapFade.GetValueOnGameThread()));
	}

	// true:1m, false:1dm ios smallest grid size
	bool bLarger1mGrid = true;

	const int Exponent = 10;

	// 2 is the default so we need to set it
	MaterialInst->SetScalarParameterValue(ExponentName, (float)Exponent);

	// without MSAA we need the grid to be more see through so lines behind it can be recognized
	MaterialInst->SetScalarParameterValue(AlphaBiasName, bMSAA ? 0.0f : 0.05f);

	// grid for size
	float GridSplit = 0.5f;
	// red dots to visualize the snap
	float SnapSplit = 0.075f;

	float WorldToUVScale = 0.001f;

	if(bLarger1mGrid)
	{
		WorldToUVScale *= 0.1f;
		GridSplit *= 0.1f;
	}

	// in 2D all grid lines are same size in world space (they are at different scale so we need to adjust here)
	FLinearColor GridSplitTriple(GridSplit * 0.01f, GridSplit * 0.1f, GridSplit);

	if(bIsPerspective)
	{
		// largest grid lines
		GridSplitTriple.R *= 8.0f;
		// medium grid lines
		GridSplitTriple.G *= 3.0f;
		// fine grid lines
		GridSplitTriple.B *= 1.0f;
	}

	if(!bIsPerspective)
	{
		// screenspace size looks better in 2d

		float ScaleX = static_cast<float>(View->ViewMatrices.GetViewToClip().M[0][0] * View->UnscaledViewRect.Width());
		float ScaleY = static_cast<float>(View->ViewMatrices.GetViewToClip().M[1][1] * View->UnscaledViewRect.Height());

		float Scale = FMath::Min(ScaleX, ScaleY);

		float GridScale = CVarEditor2DSnapScale.GetValueOnGameThread();
		float GridMin = CVarEditor2DSnapMin.GetValueOnGameThread();

		// we need to account for a larger grids setting
		SnapSplit = 1.25f * FMath::Min(GridScale / SnapGridSize / Scale, GridMin);

		// hack test
		GridSplitTriple.R = 0.25f * FMath::Min(GridScale / 100 / Scale * 0.01f, GridMin);
		GridSplitTriple.G = 0.25f * FMath::Min(GridScale / 100 / Scale * 0.1f, GridMin);
		GridSplitTriple.B = 0.25f * FMath::Min(GridScale / 100 / Scale, GridMin);
	}

	float SnapTile = (1.0f / WorldToUVScale) / FMath::Max(1.0f, SnapGridSize);

	MaterialInst->SetVectorParameterValue("GridSplit", GridSplitTriple);
	MaterialInst->SetScalarParameterValue("SnapSplit", SnapSplit);
	MaterialInst->SetScalarParameterValue("SnapTile", SnapTile);

	FMatrix ObjectToWorld = FMatrix::Identity;

	FVector CameraPos = View->ViewMatrices.GetViewOrigin();

	FVector UVCameraPos = FVector(CameraPos.X, CameraPos.Y, 0);

	ObjectToWorld.SetOrigin(UVCameraPos);

	FLinearColor AxisColors[3];
	GetAxisColors(AxisColors, bIsPerspective);

	FLinearColor UAxisColor = AxisColors[1];
	FLinearColor VAxisColor = AxisColors[0];

	if(!bIsPerspective)
	{
		double FarZ = 100000.0;

		if(View->ViewMatrices.GetWorldToView().M[1][1] == -1.f )		// Top
		{
			ObjectToWorld.SetOrigin(FVector(CameraPos.X, CameraPos.Y, -FarZ));
		}
		if(View->ViewMatrices.GetWorldToView().M[1][2] == -1.f )		// Front
		{
			UVCameraPos = FVector(CameraPos.Z, CameraPos.X, 0);
			ObjectToWorld.SetAxis(0, FVector(0,0,1));
			ObjectToWorld.SetAxis(1, FVector(1,0,0));
			ObjectToWorld.SetAxis(2, FVector(0,1,0));
			ObjectToWorld.SetOrigin(FVector(CameraPos.X, -FarZ, CameraPos.Z));
			UAxisColor = AxisColors[0];
			VAxisColor = AxisColors[2];
		}
		else if(View->ViewMatrices.GetWorldToView().M[1][0] == 1.f )		// Side
		{
			UVCameraPos = FVector(CameraPos.Y, CameraPos.Z, 0);
			ObjectToWorld.SetAxis(0, FVector(0,1,0));
			ObjectToWorld.SetAxis(1, FVector(0,0,1));
			ObjectToWorld.SetAxis(2, FVector(1,0,0));
			ObjectToWorld.SetOrigin(FVector(FarZ, CameraPos.Y, CameraPos.Z));
			UAxisColor = AxisColors[2];
			VAxisColor = AxisColors[1];
		}
	}
	
	MaterialInst->SetVectorParameterValue("UAxisColor", UAxisColor);
	MaterialInst->SetVectorParameterValue("VAxisColor", VAxisColor);

	// We don't want to affect the mouse interaction.
	PDI->SetHitProxy(0);

	// good enough to avoid the AMD artifacts, horizon still appears to be a line
	double Radii = 100000;

	if(bIsPerspective)
	{
		// the higher we get the larger we make the geometry to give the illusion of an infinite grid while maintains the precision nearby
		Radii *= FMath::Max( 1.0, FMath::Abs(CameraPos.Z) / 1000.0 );
	}
	else
	{
		double ScaleX = View->ViewMatrices.GetViewToClip().M[0][0];
		double ScaleY = View->ViewMatrices.GetViewToClip().M[1][1];

		double Scale = FMath::Min(ScaleX, ScaleY);

		Scale *= View->UnscaledViewRect.Width();

		// We render a larger grid if we are zoomed out more (good precision at any scale)
		Radii *= 1.0 / Scale;
	}

	// The grid tiles every 10 cells, so wrap the values here
	// This is needed to keep precision when position becomes very large
	UVCameraPos.X = FmodFloor(UVCameraPos.X, (double)SnapGridSize * 10.0);
	UVCameraPos.Y = FmodFloor(UVCameraPos.Y, (double)SnapGridSize * 10.0);

	FVector UVMid = UVCameraPos * WorldToUVScale;
	double UVRadi = Radii * WorldToUVScale;

	FVector UVMin = UVMid + FVector(-UVRadi, -UVRadi, 0);
	FVector UVMax = UVMid + FVector(UVRadi, UVRadi, 0);

	// vertex pos is in -1..1 range
	DrawPlane10x10(PDI, ObjectToWorld, static_cast<float>(Radii), FVector2D(UVMin), FVector2D(UVMax), MaterialInst->GetRenderProxy(), SDPG_World );
}


/*------------------------------------------------------------------------------
FEditorCommonDrawHelper.
------------------------------------------------------------------------------*/
FEditorCommonDrawHelper::FEditorCommonDrawHelper()
	: bDrawGrid(true)
	, bDrawPivot(false)
	, bDrawBaseInfo(true)
	, bDrawWorldBox(false)
	, bDrawKillZ(false)
	, AxesLineThickness(0.0f)
	, GridColorAxis(70, 70, 70)
	, GridColorMajor(40, 40, 40)
	, GridColorMinor(20, 20, 20)
	, PerspectiveGridSize(UE_OLD_HALF_WORLD_MAX1)
	, PivotColor(FColor::Red)
	, PivotSize(0.02f)
	, NumCells(64)
	, BaseBoxColor(FColor::Green)
	, DepthPriorityGroup(SDPG_World)
	, GridDepthBias(0.000001f)
	, GridWidget(nullptr)
{
}

FEditorCommonDrawHelper::~FEditorCommonDrawHelper()
{
	delete GridWidget;
}

void FEditorCommonDrawHelper::Draw(const FSceneView* View,class FPrimitiveDrawInterface* PDI)
{
	if( !PDI->IsHitTesting() )
	{
		if(bDrawBaseInfo)
		{
			DrawBaseInfo(View, PDI);
		}

		// Only draw the pivot if an actor is selected
		if( bDrawPivot && GEditor->GetSelectedActors()->CountSelections<AActor>() > 0 && View->Family->EngineShowFlags.Pivot )
		{
			DrawPivot(View, PDI);
		}

		if( View->Family->EngineShowFlags.Grid && bDrawGrid)
		{
			bool bShouldUseNewLevelGrid = CVarEditorNewLevelGrid.GetValueOnGameThread() != 0;

			if(!View->IsPerspectiveProjection())
			{
				// 3D looks better with the old grid (no thick lines)
				bShouldUseNewLevelGrid = false;
			}

			if(bShouldUseNewLevelGrid)
			{
				if(!GridWidget)
				{
					// defer creation to avoid GC issues
					GridWidget = new FGridWidget;
				}

				GridWidget->DrawNewGrid(View, PDI);
			}
			else
			{
				DrawOldGrid(View, PDI);
			}
		}
	}
}

/** Draw green lines to indicate what the selected actor(s) are based on. */
void FEditorCommonDrawHelper::DrawBaseInfo(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
// @todo UE - reimplement with new component attachment system
}

void FEditorCommonDrawHelper::DrawOldGrid(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	ESceneDepthPriorityGroup eDPG = (ESceneDepthPriorityGroup)DepthPriorityGroup;

	bool bIsPerspective = ( View->ViewMatrices.GetViewToClip().M[3][3] < 1.0f );

	static double MaxGridExtent = 8.0 * 1024 * 1024 * 1024;

	// Draw 3D perspective grid
	if( bIsPerspective)
	{
		// @todo: Persp grid should be changed to be adaptive and use same settings as ortho grid, including grid interval!
		const int32 RangeInCells = NumCells / 2;
		const int32 MajorLineInterval = NumCells / 8;

		const int32 NumLines = NumCells + 1;
		const int32 AxesIndex = NumCells / 2;
		for( int32 LineIndex = 0; LineIndex < NumLines; ++LineIndex )
		{
			bool bIsMajorLine = ( ( LineIndex - RangeInCells ) % MajorLineInterval ) == 0;

			FVector A,B;
			A.X=(PerspectiveGridSize/4.f)*(-1.0+2.0*LineIndex/NumCells);	B.X=A.X;

			A.Y=(PerspectiveGridSize/4.f);		B.Y=-(PerspectiveGridSize/4.f);
			A.Z=0.0;							B.Z=0.0;

			FColor LineColor;
			float LineThickness = 0.f;

			if ( LineIndex==AxesIndex )
			{
				LineColor = GridColorAxis;
				LineThickness = AxesLineThickness;
			}
			else if ( bIsMajorLine )
			{
				LineColor = GridColorMajor;
			}
			else
			{
				LineColor = GridColorMinor;
			}

			PDI->DrawLine(A, B, LineColor, static_cast<uint8>(eDPG), LineThickness, GridDepthBias);

			A.Y=A.X;							B.Y=B.X;
			A.X=(PerspectiveGridSize/4.f);		B.X=-(PerspectiveGridSize/4.f);
			PDI->DrawLine(A, B, LineColor, static_cast<uint8>(eDPG), LineThickness, GridDepthBias);
		}
	}
	// Draw ortho grid.
	else
	{
		const bool bIsOrthoXY = ( FMath::Abs(View->ViewMatrices.GetWorldToView().M[2][2]) > 0 );
		const bool bIsOrthoXZ = ( FMath::Abs(View->ViewMatrices.GetWorldToView().M[1][2]) > 0 );
		const bool bIsOrthoYZ = ( FMath::Abs(View->ViewMatrices.GetWorldToView().M[0][2]) > 0 );

		FLinearColor AxisColors[3];
		GetAxisColors(AxisColors, bIsPerspective);

		if (bIsOrthoXY)
		{
			const bool bNegative = View->ViewMatrices.GetWorldToView().M[2][2] > 0;

			FVector StartY(0, +MaxGridExtent, bNegative ? MaxGridExtent : -MaxGridExtent);
			FVector EndY(0, -MaxGridExtent, bNegative ? MaxGridExtent : -MaxGridExtent);
			FVector StartX(+MaxGridExtent, 0, bNegative ? MaxGridExtent : -MaxGridExtent);
			FVector EndX(-MaxGridExtent, 0, bNegative ? MaxGridExtent : -MaxGridExtent);

			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartY, &EndY, &StartY.X, &EndY.X, 0, View, PDI);
			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartX, &EndX, &StartX.Y, &EndX.Y, 1, View, PDI);
			DrawOriginAxisLine( &StartY, &EndY, &StartY.X, &EndY.X, View, PDI, AxisColors[1] );
			DrawOriginAxisLine( &StartX, &EndX, &StartX.Y, &EndX.Y, View, PDI, AxisColors[0] );
		}
		else if( bIsOrthoXZ )
		{
			const bool bNegative = View->ViewMatrices.GetWorldToView().M[1][2] > 0;

			FVector StartZ(0, bNegative ? MaxGridExtent : -MaxGridExtent, +MaxGridExtent);
			FVector EndZ(0, bNegative ? MaxGridExtent : -MaxGridExtent, -MaxGridExtent);
			FVector StartX(+MaxGridExtent, bNegative ? MaxGridExtent : -MaxGridExtent, 0);
			FVector EndX(-MaxGridExtent, bNegative ? MaxGridExtent : -MaxGridExtent, 0);

			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartZ, &EndZ, &StartZ.X, &EndZ.X, 0, View, PDI);
			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartX, &EndX, &StartX.Z, &EndX.Z, 2, View, PDI);
			DrawOriginAxisLine( &StartZ, &EndZ, &StartZ.X, &EndZ.X, View, PDI, AxisColors[2] );
			DrawOriginAxisLine( &StartX, &EndX, &StartX.Z, &EndX.Z, View, PDI, AxisColors[0] );
		}
		else if( bIsOrthoYZ )
		{
			const bool bNegative = View->ViewMatrices.GetWorldToView().M[0][2] < 0;

			FVector StartZ(bNegative ? -MaxGridExtent : MaxGridExtent, 0, +MaxGridExtent);
			FVector EndZ(bNegative ? -MaxGridExtent : MaxGridExtent, 0, -MaxGridExtent);
			FVector StartY(bNegative ? -MaxGridExtent : MaxGridExtent, +MaxGridExtent, 0);
			FVector EndY(bNegative ? -MaxGridExtent : MaxGridExtent, -MaxGridExtent, 0);

			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartZ, &EndZ, &StartZ.Y, &EndZ.Y, 1, View, PDI);
			DrawGridSection( GEditor->GetGridSize(), MaxGridExtent, &StartY, &EndY, &StartY.Z, &EndY.Z, 2, View, PDI);
			DrawOriginAxisLine( &StartZ, &EndZ, &StartZ.Y, &EndZ.Y, View, PDI, AxisColors[2] );
			DrawOriginAxisLine( &StartY, &EndY, &StartY.Z, &EndY.Z, View, PDI, AxisColors[1] );
		}

		if( bDrawKillZ && ( bIsOrthoXZ || bIsOrthoYZ ) && GWorld->GetWorldSettings()->AreWorldBoundsChecksEnabled() )
		{
			float KillZ = GWorld->GetWorldSettings()->KillZ;

			PDI->DrawLine(FVector(-MaxGridExtent, 0, KillZ), FVector(MaxGridExtent, 0, KillZ), FColor::Red, SDPG_Foreground);
			PDI->DrawLine(FVector(0, -MaxGridExtent, KillZ), FVector(0, MaxGridExtent, KillZ), FColor::Red, SDPG_Foreground);
		}
	}

	// Draw orthogonal worldframe.
	if(bDrawWorldBox)
	{
		DrawWireBox(PDI, FBox(FVector(-MaxGridExtent, -MaxGridExtent, -MaxGridExtent), FVector(MaxGridExtent, MaxGridExtent, MaxGridExtent)), GEngine->C_WorldBox, static_cast<uint8>(eDPG));
	}
}


void FEditorCommonDrawHelper::DrawGridSection(double ViewportGridY, double MaxSize, FVector* A,FVector* B, FVector::FReal* AX, FVector::FReal* BX,int32 Axis,const FSceneView* View,FPrimitiveDrawInterface* PDI )
{
	if (FMath::IsNearlyZero(ViewportGridY))
	{
		// Don't draw zero-size grid
		return;
	}

	const double SizeX = View->UnscaledViewRect.Width();
	const double Zoom = (1.0 / View->ViewMatrices.GetViewToClip().M[0][0]) * 2.0 / SizeX;
	const double Dist = SizeX * Zoom / ViewportGridY;

	// when the grid fades
	constexpr double Tweak = 6.0;
	const int32 Exponent = GEditor->IsGridSizePowerOfTwo() ? 8 : 10;
	const double IncValue = FMath::LogX(Exponent, Dist / (double)(SizeX / Tweak));
	int32 IncScale = 1;

	for (double x = 0; x < IncValue; ++x)
	{
		IncScale *= Exponent;
	}

	if (IncScale == 0)
	{
		// Prevent divide by zero
		return;
	}

	const FMatrix InvViewProjMatrix = View->ViewMatrices.GetClipToWorld();
	int64 FirstLine = FMath::TruncToInt(InvViewProjMatrix.TransformPosition(FVector(-1, -1, 0.5f)).Component(Axis) / ViewportGridY);
	int64 LastLine = FMath::TruncToInt(InvViewProjMatrix.TransformPosition(FVector(+1, +1, 0.5f)).Component(Axis) / ViewportGridY);
	if (FirstLine > LastLine)
	{
		Exchange(FirstLine, LastLine);
	}

	// Check if we're in a mode that is not using the view background (uses lighting)
	const FLinearColor Background = View->BackgroundColor;
	bool bIsLitMode = Background.GetLuminance() > 0.5;
	if (View->Family 
		&& View->Family->EngineShowFlags.Lighting > 0 
		&& View->Family->EngineShowFlags.LightingOnlyOverride == 0)
	{
		bIsLitMode = true;
	}

	// Reduce grid density when further away by removing minor lines
	const double Fract = IncValue - FMath::FloorToDouble(IncValue);
	const float MinorLineAlpha = Fract < 0.5 || Zoom < 1.f ? 0.1 : 0;
	constexpr float MajorLineAlpha = 0.25f;

	// Use contrasting color based on lit/unlit mode
	const FLinearColor LineColor = bIsLitMode ? FLinearColor::Black : FLinearColor::White;

	// Draw major and minor grid lines
	const int32 FirstLineClamped = FMath::Max<int32>(FirstLine - 1, -MaxSize / ViewportGridY) / IncScale;
	const int32 LastLineClamped = FMath::Min<int32>(LastLine + 1, +MaxSize / ViewportGridY) / IncScale;
	const int32 MajorLineInterval = FMath::TruncToInt(GEditor->GetGridInterval());
	for (int32 LineIndex = FirstLineClamped; LineIndex <= LastLineClamped; ++LineIndex)
	{
		*AX = FPlatformMath::TruncToDouble(LineIndex * ViewportGridY) * IncScale;
		*BX = FPlatformMath::TruncToDouble(LineIndex * ViewportGridY) * IncScale;

		// Only minor lines fade out with ortho zoom distance
		const bool bIsMajorLine = (MajorLineInterval == 0) || ((LineIndex % MajorLineInterval) == 0);

		FLinearColor Color;

		if (bIsMajorLine)
		{
			Color = FMath::Lerp(FLinearColor::Transparent, LineColor, MajorLineAlpha);
		}
		else
		{
			Color = FMath::Lerp(FLinearColor::Transparent, LineColor, MinorLineAlpha);
		}

		PDI->DrawTranslucentLine(*A, *B, Color, SDPG_World);
	}
}


void FEditorCommonDrawHelper::DrawOriginAxisLine(FVector* A,FVector* B, FVector::FReal* AX, FVector::FReal* BX,const FSceneView* View,FPrimitiveDrawInterface* PDI, const FLinearColor& Color)
{
	// Draw world origin lines.  We draw these last so they appear on top of the other lines.
	*AX = 0;
	*BX = 0;

	PDI->DrawLine(*A,*B,Color,SDPG_World);
}


void FEditorCommonDrawHelper::DrawPivot(const FSceneView* View,FPrimitiveDrawInterface* PDI)
{
	const FMatrix CameraToWorld = View->ViewMatrices.GetViewToWorld();

	const FVector PivLoc = GLevelEditorModeTools().SnappedLocation;

	const FVector::FReal ZoomFactor = FMath::Min<FVector::FReal>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
	const FVector::FReal WidgetRadius = View->ViewMatrices.GetWorldToClip().TransformPosition(PivLoc).W * (PivotSize / ZoomFactor);

	const FVector CamX = CameraToWorld.TransformVector( FVector(1,0,0) );
	const FVector CamY = CameraToWorld.TransformVector( FVector(0,1,0) );


	PDI->DrawLine( PivLoc - (WidgetRadius*CamX), PivLoc + (WidgetRadius*CamX), PivotColor, SDPG_Foreground );
	PDI->DrawLine( PivLoc - (WidgetRadius*CamY), PivLoc + (WidgetRadius*CamY), PivotColor, SDPG_Foreground );
}
