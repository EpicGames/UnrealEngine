// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorConstraints.cpp: Editor movement constraints.
=============================================================================*/

#include "CoreMinimal.h"
#include "LevelEditorViewport.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor/EditorEngine.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "LevelEditorActions.h"

float UEditorEngine::GetGridSize() const
{
	const TArray<float>& PosGridSizes = GetCurrentPositionGridArray();
	const int32 CurrentPosGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentPosGridSize;
	float PosVal = 0.0001f;
	if ( PosGridSizes.IsValidIndex(CurrentPosGridSize) )
	{
		PosVal = PosGridSizes[CurrentPosGridSize];
	}
	return PosVal;
}

bool UEditorEngine::IsGridSizePowerOfTwo() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	return ViewportSettings->bUsePowerOf2SnapSize;
}

void UEditorEngine::SetGridSize( int32 InIndex )
{
	FinishAllSnaps();

	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->CurrentPosGridSize = FMath::Clamp<int32>( InIndex, 0, GetCurrentPositionGridArray().Num() - 1 );

	// If user is editing snapping values, we assume they also want snapping turned on
	if (!FLevelEditorActionCallbacks::LocationGridSnap_IsChecked())
	{
		FLevelEditorActionCallbacks::LocationGridSnap_Clicked();
	}

	ViewportSettings->PostEditChange();

	FEditorDelegates::OnGridSnappingChanged.Broadcast(GetDefault<ULevelEditorViewportSettings>()->GridEnabled, GetGridSize());
	
	RedrawLevelEditingViewports();
	FEditorSupportDelegates::UpdateUI.Broadcast();
}

void UEditorEngine::GridSizeIncrement( )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	SetGridSize(ViewportSettings->CurrentPosGridSize + 1);
}

void UEditorEngine::GridSizeDecrement( )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	SetGridSize(ViewportSettings->CurrentPosGridSize - 1);
}

void UEditorEngine::ToggleAllowConstrainedAspectRatioInPreview()
{
	bAllowConstrainedAspectRatioInPreview ^= true;
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

bool UEditorEngine::GetAllowConstrainedAspectRatioInPreview()
{
	return bAllowConstrainedAspectRatioInPreview;
}

void UEditorEngine::ToggleFlipConstrainedAspectRatioInPreview()
{
	bFlipConstrainedAspectRatioInPreview ^= true;
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

bool UEditorEngine::GetFlipConstrainedAspectRatioInPreview()
{
	return bFlipConstrainedAspectRatioInPreview;
}

const TArray<float>& UEditorEngine::GetCurrentPositionGridArray() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	return (ViewportSettings->bUsePowerOf2SnapSize) ?
			ViewportSettings->Pow2GridSizes :
			ViewportSettings->DecimalGridSizes;
}

const TArray<float>& UEditorEngine::GetCurrentIntervalGridArray() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	return (ViewportSettings->bUsePowerOf2SnapSize ?
			ViewportSettings->Pow2GridIntervals :
			ViewportSettings->DecimalGridIntervals);
}

FRotator UEditorEngine::GetRotGridSize()
{
	const TArray<float> RotGridSizes = GetCurrentRotationGridArray();
	const int32 CurrentRotGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentRotGridSize;
	float RotVal = 0.0001f;
	if ( RotGridSizes.IsValidIndex(CurrentRotGridSize) )
	{
		RotVal = RotGridSizes[CurrentRotGridSize];
	}
	return FRotator(RotVal, RotVal, RotVal);
}

void UEditorEngine::SetRotGridSize( int32 InIndex, ERotationGridMode InGridMode )
{
	FinishAllSnaps();

	const TArray<float> RotGridSizes = GetCurrentRotationGridArray();

	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->CurrentRotGridMode = InGridMode;
	ViewportSettings->CurrentRotGridSize = FMath::Clamp<int32>( InIndex, 0, RotGridSizes.Num() - 1 );

	// If user is editing snapping values, we assume they also want snapping turned on
	if (!FLevelEditorActionCallbacks::RotationGridSnap_IsChecked())
	{
		FLevelEditorActionCallbacks::RotationGridSnap_Clicked();
	}

	ViewportSettings->PostEditChange();
	
	RedrawLevelEditingViewports();
	FEditorSupportDelegates::UpdateUI.Broadcast();
}

void UEditorEngine::RotGridSizeIncrement( )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	SetRotGridSize(ViewportSettings->CurrentRotGridSize + 1, ViewportSettings->CurrentRotGridMode );
}

void UEditorEngine::RotGridSizeDecrement( )
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	SetRotGridSize(ViewportSettings->CurrentRotGridSize - 1, ViewportSettings->CurrentRotGridMode );
}

const TArray<float>& UEditorEngine::GetCurrentRotationGridArray() const
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	return (ViewportSettings->CurrentRotGridMode == GridMode_Common) ?
			ViewportSettings->CommonRotGridSizes :
			ViewportSettings->DivisionsOf360RotGridSizes;
}

float UEditorEngine::GetScaleGridSize()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	float ScaleVal = 0.0001f;
	if ( ViewportSettings->ScalingGridSizes.IsValidIndex(ViewportSettings->CurrentScalingGridSize) )
	{
		ScaleVal = ViewportSettings->ScalingGridSizes[ViewportSettings->CurrentScalingGridSize];
	}
	return ScaleVal;
}

void UEditorEngine::SetScaleGridSize( int32 InIndex )
{
	FinishAllSnaps();

	ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	ViewportSettings->CurrentScalingGridSize = FMath::Clamp<int32>( InIndex, 0, ViewportSettings->ScalingGridSizes.Num() - 1 );

	// If user is editing snapping values, we assume they also want snapping turned on
	if (!FLevelEditorActionCallbacks::ScaleGridSnap_IsChecked())
	{
		FLevelEditorActionCallbacks::ScaleGridSnap_Clicked();
	}

	ViewportSettings->PostEditChange();

	RedrawLevelEditingViewports();
	FEditorSupportDelegates::UpdateUI.Broadcast();
}

float UEditorEngine::GetGridInterval()
{
	const TArray<float>& GridIntervals = GetCurrentIntervalGridArray();

	int32 CurrentIndex = GetDefault<ULevelEditorViewportSettings>()->CurrentPosGridSize;

	if (CurrentIndex >= GridIntervals.Num())
	{
		CurrentIndex = GridIntervals.Num() - 1;
	}

	float IntervalValue = 1.0f;
	if (GridIntervals.IsValidIndex(CurrentIndex))
	{
		IntervalValue = GridIntervals[CurrentIndex];
	}
	return IntervalValue;
}

FVector UEditorEngine::GetGridLocationOffset(bool bUniformOffset) const
{
	const float Offset = GetDefault<ULevelEditorViewportSettings>()->GridEnabled ? GetGridSize() : 0.0f;

	FVector LocationOffset(Offset, Offset, Offset);
	if (!bUniformOffset && GCurrentLevelEditingViewportClient)
	{
		switch (GCurrentLevelEditingViewportClient->ViewportType)
		{
		case LVT_OrthoXZ:
			LocationOffset = FVector(Offset, 0.f, Offset);
			break;

		case LVT_OrthoYZ:
			LocationOffset = FVector(0.f, Offset, Offset);
			break;

		default:
			LocationOffset = FVector(Offset, Offset, 0.f);
			break;
		}
	}
	return LocationOffset;
}
