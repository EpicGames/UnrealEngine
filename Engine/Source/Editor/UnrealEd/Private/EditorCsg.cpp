// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "GameFramework/Actor.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Engine/Brush.h"
#include "Editor/EditorEngine.h"
#include "Engine/Polys.h"
#include "Engine/Selection.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "SurfaceIterators.h"
#include "BSPOps.h"
#include "BSPUtils.h"
#include "ActorEditorUtils.h"
#include "Misc/FeedbackContext.h"
#include "EngineUtils.h"

// Globals:
static TArray<uint8*> GFlags1;
static TArray<uint8*> GFlags2;

/*-----------------------------------------------------------------------------
	Helper classes.
-----------------------------------------------------------------------------*/

/**
 * Iterator used to iterate over all static brush actors in the current level.
 */
class FStaticBrushIterator
{
public:
	/**
	 * Default constructor, initializing all member variables and iterating to first.
	 */
	FStaticBrushIterator( UWorld* InWorld )
	:	ActorIndex( -1 ),
		ReachedEnd( false ),
		World( InWorld )
	{
		// Iterate to first.
		++(*this);
	}

	/**
	 * Iterates to next suitable actor.
	 */
	void operator++()
	{
		bool FoundSuitableActor = false;
		while( !ReachedEnd && !FoundSuitableActor )
		{
			if( ++ActorIndex >= World->GetCurrentLevel()->Actors.Num() )
			{
				ReachedEnd = true;
			}
			else
			{
				//@todo locked levels - should we skip brushes contained by locked levels?
				ABrush* Brush = Cast<ABrush>(World->GetCurrentLevel()->Actors[ActorIndex]);
				FoundSuitableActor = Brush && Brush->IsStaticBrush();
			}
		}
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	AActor* operator*()
	{
		check(ActorIndex<= World->GetCurrentLevel()->Actors.Num());
		check(!ReachedEnd);
		AActor* Actor = World->GetCurrentLevel()->Actors[ActorIndex];
		return Actor;
	}

	/**
	 * Returns the current suitable actor pointed at by the Iterator
	 *
	 * @return	Current suitable actor
	 */
	AActor* operator->()
	{
		check(ActorIndex<= World->GetCurrentLevel()->Actors.Num());
		check(!ReachedEnd);
		AActor* Actor =  World->GetCurrentLevel()->Actors[ActorIndex];
		return Actor;
	}

	/**
	 * Returns whether the iterator has reached the end and no longer points
	 * to a suitable actor.
	 *
	 * @return true if iterator points to a suitable actor, false if it has reached the end
	 */
	explicit operator bool() const
	{
		return !ReachedEnd;
	}

protected:
	/** Current index into actors array		*/
	int32 ActorIndex;
	/** Whether we already reached the end	*/
	bool ReachedEnd;
	/** Relevant world context				*/
	UWorld*	World;
};


void UEditorEngine::bspRepartition( UWorld* InWorld, int32 iNode )
{
	FBSPUtils::bspRepartition(InWorld, iNode);
}

//
// Build list of leaves.
//
static void EnlistLeaves( UModel* Model, TArray<int32>& iFronts, TArray<int32>& iBacks, int32 iNode )
{
	FBspNode& Node=Model->Nodes[iNode];

	if( Node.iFront==INDEX_NONE ) iFronts.Add(iNode);
	else EnlistLeaves( Model, iFronts, iBacks, Node.iFront );

	if( Node.iBack==INDEX_NONE ) iBacks.Add(iNode);
	else EnlistLeaves( Model, iFronts, iBacks, Node.iBack );

}


void UEditorEngine::csgRebuild( UWorld* InWorld )
{
	GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "RebuildingGeometry", "Rebuilding geometry"), false );
	FBSPOps::GFastRebuild = 1;
	ABrush::GGeometryRebuildCause = TEXT("csgRebuild");
	FinishAllSnaps();

	// Empty the model out.
	InWorld->GetModel()->Modify(false);
	InWorld->GetModel()->EmptyModel(1, 1);

	// Count brushes.
	int32 BrushTotal=0, BrushCount=0;
	for( FStaticBrushIterator It(InWorld); It; ++It )
	{
		ABrush* Brush = CastChecked<ABrush>(*It);
		if( !FActorEditorUtils::IsABuilderBrush(Brush) )
		{
			BrushTotal++;
		}
	}

	// Check for the giant cube brush that is created for subtractive levels.
	// If it's found, apply the RemoveSurfaceMaterial to its polygons so they aren't lit or drawn.
	for(FStaticBrushIterator GiantCubeBrushIt(InWorld);GiantCubeBrushIt;++GiantCubeBrushIt)
	{
		ABrush* GiantCubeBrush = CastChecked<ABrush>(*GiantCubeBrushIt);
		if(GiantCubeBrush->Brush && GiantCubeBrush->Brush->Bounds.SphereRadius > HALF_WORLD_MAX)
		{
			check(GiantCubeBrush->Brush->Polys);
			for(int32 PolyIndex = 0;PolyIndex < GiantCubeBrush->Brush->Polys->Element.Num();PolyIndex++)
			{
				FPoly& Polygon = GiantCubeBrush->Brush->Polys->Element[PolyIndex];
				const float PolygonArea = Polygon.Area();
				if(PolygonArea > WORLD_MAX * WORLD_MAX * 0.99f && PolygonArea < WORLD_MAX * WORLD_MAX * 1.01f)
				{
					Polygon.Material = GEngine->RemoveSurfaceMaterial;
				}
			}
		}
	}

	// Compose all structural brushes and portals.
	for( FStaticBrushIterator It(InWorld); It; ++It )
	{	
		ABrush* Brush = CastChecked<ABrush>(*It);
		if( !FActorEditorUtils::IsABuilderBrush(Brush) )
		{
			if
			(  !(Brush->PolyFlags&PF_Semisolid)
			||	(Brush->BrushType!=Brush_Add)
			||	(Brush->PolyFlags&PF_Portal) )
			{
				// Treat portals as solids for cutting.
				if( Brush->PolyFlags & PF_Portal )
				{
					Brush->PolyFlags = (Brush->PolyFlags & ~PF_Semisolid) | PF_NotSolid;
				}
				BrushCount++;

				FFormatNamedArguments Args;
				Args.Add( TEXT("BrushCount"), BrushCount );
				Args.Add( TEXT("BrushTotal"), BrushTotal );
				GWarn->StatusUpdate( BrushCount, BrushTotal, FText::Format( NSLOCTEXT("UnrealEd", "ApplyingStructuralBrushF", "Applying structural brush {BrushCount} of {BrushTotal}"), Args ) );

				Brush->Modify(false);
				bspBrushCSG( Brush, InWorld->GetModel(), Brush->PolyFlags, (EBrushType)Brush->BrushType, CSG_None, false, true, false );
			}
		}
	}

	// Repartition the structural BSP.
	{
		GWarn->StatusUpdate( 0, 4, NSLOCTEXT("UnrealEd", "RebuildBSPBuildingPolygons", "Rebuild BSP: Building polygons") );
		bspBuildFPolys( InWorld->GetModel(), 0, 0 );

		GWarn->StatusUpdate( 1, 4, NSLOCTEXT("UnrealEd", "RebuildBSPMergingPlanars", "Rebuild BSP: Merging planars") );
		bspMergeCoplanars( InWorld->GetModel(), 0, 0 );

		GWarn->StatusUpdate( 2, 4, NSLOCTEXT("UnrealEd", "RebuildBSPPartitioning", "Rebuild BSP: Partitioning") );
		FBSPOps::bspBuild( InWorld->GetModel(), FBSPOps::BSP_Optimal, 15, 70, 0, 0 );

		GWarn->UpdateProgress( 4, 4 );
	}

	// Remember leaves.
	TArray<int32> iFronts, iBacks;
	if( InWorld->GetModel()->Nodes.Num() )
	{
		EnlistLeaves( InWorld->GetModel(), iFronts, iBacks, 0 );
	}

	// Compose all detail brushes.
	for( FStaticBrushIterator It(InWorld); It; ++It )
	{
		ABrush* Brush = CastChecked<ABrush>(*It);
		if
		(	!FActorEditorUtils::IsABuilderBrush(Brush)
		&&	(Brush->PolyFlags&PF_Semisolid)
		&& !(Brush->PolyFlags&PF_Portal)
		&&	Brush->BrushType==Brush_Add )
		{
			BrushCount++;

			FFormatNamedArguments Args;
			Args.Add( TEXT("BrushCount"), BrushCount );
			Args.Add( TEXT("BrushTotal"), BrushTotal );
			GWarn->StatusUpdate( BrushCount, BrushTotal, FText::Format( NSLOCTEXT("UnrealEd", "ApplyingDetailBrushF", "Applying detail brush {BrushCount} of {BrushTotal}"), Args ) );

			Brush->Modify(false);
			bspBrushCSG( Brush, InWorld->GetModel(), Brush->PolyFlags, (EBrushType)Brush->BrushType, CSG_None, false, true, false );
		}
	}

	// Optimize the sub-bsp's.
	GWarn->StatusUpdate( 0, 4, NSLOCTEXT("UnrealEd", "RebuildCSGOptimizingSubBSPs", "Rebuild CSG: Optimizing Sub-BSPs") );
	int32 iNode;
	for( TArray<int32>::TIterator ItF(iFronts); ItF; ++ItF )
	{
		if( (iNode=InWorld->GetModel()->Nodes[*ItF].iFront)!=INDEX_NONE )
		{
			bspRepartition( InWorld, iNode );
		}
	}
	for( TArray<int32>::TIterator ItB(iBacks); ItB; ++ItB )
	{
		if( (iNode=InWorld->GetModel()->Nodes[*ItB].iBack)!=INDEX_NONE )
		{
			bspRepartition( InWorld, iNode );
		}
	}

	GWarn->StatusUpdate( 1, 4, NSLOCTEXT("UnrealEd", "RebuildBSPOptimizingGeometry", "Rebuild BSP: Optimizing geometry") );
	bspOptGeom( InWorld->GetModel() );

	// Build bounding volumes.
	GWarn->StatusUpdate( 2, 4,  NSLOCTEXT("UnrealEd", "RebuildCSGBuildingBoundingVolumes", "Rebuild CSG: Building Bounding Volumes") );
	FBSPOps::bspBuildBounds( InWorld->GetModel() );

	// Rebuild dynamic brush BSP's.
	GWarn->StatusUpdate( 3, 4,  NSLOCTEXT("UnrealEd", "RebuildCSGRebuildingDynamicBrushBSPs", "Rebuild CSG: Rebuilding Dynamic Brush BSPs") );

	TArray<ABrush*> DynamicBrushes;
	DynamicBrushes.Empty();
	for( TActorIterator<ABrush> It(InWorld); It; ++It )
	{
		ABrush* B=*It;
		if ( B && B->GetLevel() == InWorld->GetCurrentLevel() && B->Brush && !B->IsStaticBrush() )
		{
			DynamicBrushes.Add(B);
		}
	}

	{
		FScopedSlowTask SlowTask(static_cast<float>(DynamicBrushes.Num()), NSLOCTEXT("UnrealEd", "RebuildCSGRebuildingDynamicBrushBSPs", "Rebuild CSG: Rebuilding Dynamic Brush BSPs") );
		for ( int32 BrushIndex = 0; BrushIndex < DynamicBrushes.Num(); BrushIndex++ )
		{
			SlowTask.EnterProgressFrame();

			ABrush* B = DynamicBrushes[BrushIndex];
			FBSPOps::csgPrepMovingBrush(B);
			
			if ( GEngine->GetMapBuildCancelled() )
			{
				break;
			}
		}
	}

	GWarn->UpdateProgress( 4, 4 );

	// update static navigable geometry in current level
	RebuildStaticNavigableGeometry(InWorld->GetCurrentLevel());

	// Empty EdPolys.
	InWorld->GetModel()->Polys->Element.Empty();

	// Done.
	ABrush::GGeometryRebuildCause = nullptr;
	FBSPOps::GFastRebuild = 0;
	InWorld->GetCurrentLevel()->MarkPackageDirty();
	GWarn->EndSlowTask();
}


void UEditorEngine::polySetAndClearPolyFlags(UModel *Model, uint32 SetBits, uint32 ClearBits,bool SelectedOnly, bool UpdateBrush)
{
	FBSPUtils::polySetAndClearPolyFlags(Model, SetBits, ClearBits, SelectedOnly, UpdateBrush);
}

bool UEditorEngine::polyFindBrush(UModel* InModel, int32 iSurf, FPoly& Poly)
{
	return FBSPUtils::polyFindBrush(InModel, iSurf, Poly);
}

void UEditorEngine::polyUpdateBrush
(
	UModel*	Model,
	int32  	iSurf,
	bool	bUpdateTexCoords,
	bool	bOnlyRefreshSurfaceMaterials
)
{
	FBSPUtils::polyUpdateBrush(Model, iSurf, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
}


void UEditorEngine::polyGetLinkedPolys
(
	ABrush* InBrush,
	FPoly* InPoly,
	TArray<FPoly>* InPolyList
)
{
	FBSPUtils::polyGetLinkedPolys(InBrush, InPoly, InPolyList);
}


void UEditorEngine::polySplitOverlappingEdges( TArray<FPoly>* InPolyList, TArray<FPoly>* InResult )
{
	FBSPUtils::polySplitOverlappingEdges(InPolyList, InResult);
}


void UEditorEngine::polyGetOuterEdgeList
(
	TArray<FPoly>* InPolyList,
	TArray<FEdge>* InEdgeList
)
{
	FBSPUtils::polyGetOuterEdgeList(InPolyList, InEdgeList);
}

/*-----------------------------------------------------------------------------
   All transactional polygon selection functions
-----------------------------------------------------------------------------*/

/**
 * Generates a list of brushes corresponding to the set of selected surfaces for the specified model.
 */
static void GetListOfUniqueBrushes( TArray<ABrush*>* InBrushes, UModel *Model )
{
	InBrushes->Empty();

	// Generate a list of unique brushes.
	for( int32 i = 0 ; i < Model->Surfs.Num() ; i++ )
	{
		FBspSurf* Surf = &Model->Surfs[i];
		if( Surf->PolyFlags & PF_Selected )
		{
			if ( Surf->Actor )
			{
				// See if we've already got this brush ...
				int32 brush;
				for( brush = 0 ; brush < InBrushes->Num() ; brush++ )
				{
					if( Surf->Actor == (*InBrushes)[brush] )
					{
						break;
					}
				}

				// ... if not, add it to the list.
				if( brush == InBrushes->Num() )
				{
					(*InBrushes)[ InBrushes->AddUninitialized() ] = Surf->Actor;
				}
			}
		}
	}
}

void UEditorEngine::polySelectAll(UModel *Model)
{
	polySetAndClearPolyFlags(Model,PF_Selected,0,0,0);
}

void UEditorEngine::polySelectMatchingGroups( UModel* Model )
{
	// @hack: polySelectMatchingGroups: do nothing for now as temp fix until this can be rewritten (crashes a lot)
#if 0
	FMemory::Memzero( GFlags1, sizeof(GFlags1) );
	for( int32 i=0; i<Model->Surfs.Num(); i++ )
	{
		FBspSurf *Surf = &Model->Surfs(i);
		if( Surf->PolyFlags&PF_Selected )
		{
			FPoly Poly; polyFindBrush(Model,i,Poly);
			GFlags1[Poly.Actor->Layer.GetIndex()]=1;
		}
	}
	for( int32 i=0; i<Model->Surfs.Num(); i++ )
	{
		FBspSurf *Surf = &Model->Surfs(i);
		FPoly Poly;
		polyFindBrush(Model,i,Poly);
		if
		(	(GFlags1[Poly.Actor->Layer.GetIndex()]) 
			&&	(!(Surf->PolyFlags & PF_Selected)) )
		{
			Model->ModifySurf( i, 0 );
			GEditor->SelectBSPSurf( Model, i, true, false );
		}
	}
	NoteSelectionChange();
#endif
}

void UEditorEngine::polySelectMatchingItems(UModel *InModel)
{
#if 0
	FMemory::Memzero(GFlags1,sizeof(GFlags1));
	FMemory::Memzero(GFlags2,sizeof(GFlags2));

	for( int32 i=0; i<InModel->Surfs.Num(); i++ )
	{
		FBspSurf *Surf = &InModel->Surfs(i);
		if( Surf->Actor )
		{
			if( Surf->PolyFlags & PF_Selected )
				GFlags2[Surf->Actor->Brush->GetIndex()]=1;
		}
		if( Surf->PolyFlags&PF_Selected )
		{
			FPoly Poly; polyFindBrush(InModel,i,Poly);
			GFlags1[Poly.ItemName.GetIndex()]=1;
		}
	}
	for( int32 i=0; i<InModel->Surfs.Num(); i++ )
	{
		FBspSurf *Surf = &InModel->Surfs(i);
		if( Surf->Actor )
		{
			FPoly Poly; polyFindBrush(InModel,i,Poly);
			if ((GFlags1[Poly.ItemName.GetIndex()]) &&
				( GFlags2[Surf->Actor->Brush->GetIndex()]) &&
				(!(Surf->PolyFlags & PF_Selected)))
			{
				InModel->ModifySurf( i, 0 );
				GEditor->SelectBSPSurf( InModel, i, true, false );
			}
		}
	}
	NoteSelectionChange();
#endif
}

enum EAdjacentsType
{
	ADJACENT_ALL,		// All adjacent polys
	ADJACENT_COPLANARS,	// Adjacent coplanars only
	ADJACENT_WALLS,		// Adjacent walls
	ADJACENT_FLOORS,	// Adjacent floors or ceilings
	ADJACENT_SLANTS,	// Adjacent slants
};

/**
 * Selects all adjacent polygons (only coplanars if Coplanars==1)
 * @return		Number of polygons newly selected.
 */
static int32 TagAdjacentsType(UWorld* InWorld, EAdjacentsType AdjacentType)
{
	// Allocate GFlags1
	check( GFlags1.Num() == 0 );
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		uint8* Ptr = new uint8[MAX_uint16+1];
		FMemory::Memzero( Ptr, MAX_uint16+1 );
		GFlags1.Add( Ptr );
	}

	FVert		*VertPool;
	FVector3f		*Base,*Normal;
	uint8		b;
	int32		    i;
	int			Selected,Found;

	Selected = 0;

	// Find all points corresponding to selected vertices:
	int32 ModelIndex1 = 0;
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		uint8* Flags1 = GFlags1[ModelIndex1++];
		for (i=0; i<Model->Nodes.Num(); i++)
		{
			FBspNode &Node = Model->Nodes[i];
			FBspSurf &Poly = Model->Surfs[Node.iSurf];
			if (Poly.PolyFlags & PF_Selected)
			{
				VertPool = &Model->Verts[Node.iVertPool];
				for (b=0; b<Node.NumVertices; b++)
				{
					Flags1[(VertPool++)->pVertex] = 1;
				}
			}
		}
	}

	// Select all unselected nodes for which two or more vertices are selected:
	ModelIndex1 = 0;
	int32 ModelIndex2 = -1;
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		uint8* Flags1 = GFlags1[ModelIndex1];
		ModelIndex1++;
		ModelIndex2++;
		for( i = 0 ; i < Model->Nodes.Num() ; i++ )
		{
			FBspNode &Node = Model->Nodes[i];
			FBspSurf &Poly = Model->Surfs[Node.iSurf];
			if (!(Poly.PolyFlags & PF_Selected))
			{
				Found    = 0;
				VertPool = &Model->Verts[Node.iVertPool];
				//
				Base   = &Model->Points [Poly.pBase];
				Normal = &Model->Vectors[Poly.vNormal];
				//
				for (b=0; b<Node.NumVertices; b++) Found += Flags1[(VertPool++)->pVertex];
				//
				if (AdjacentType == ADJACENT_COPLANARS)
				{
					if (!GFlags2[ModelIndex2][Node.iSurf]) Found=0;
				}
				else if (AdjacentType == ADJACENT_FLOORS)
				{
					if (FMath::Abs(Normal->Z) <= 0.85) Found = 0;
				}
				else if (AdjacentType == ADJACENT_WALLS)
				{
					if (FMath::Abs(Normal->Z) >= 0.10) Found = 0;
				}
				else if (AdjacentType == ADJACENT_SLANTS)
				{
					if (FMath::Abs(Normal->Z) > 0.85) Found = 0;
					if (FMath::Abs(Normal->Z) < 0.10) Found = 0;
				}

				if (Found > 0)
				{
					Model->ModifySurf( Node.iSurf, 0 );
					GEditor->SelectBSPSurf( Model, Node.iSurf, true, false );
					Selected++;
				}
			}
		}
	}

	// Free GFlags1.
	for ( i = 0 ; i < GFlags1.Num() ; ++i )
	{
		delete[] GFlags1[i];
	}
	GFlags1.Empty();

	GEditor->NoteSelectionChange();
	return Selected;
}

void UEditorEngine::polySelectAdjacents(UWorld* InWorld, UModel* InModel)
{
	do {} while (TagAdjacentsType(InWorld, ADJACENT_ALL) > 0);
}

void UEditorEngine::polySelectCoplanars(UWorld* InWorld, UModel* InModel)
{
	// Allocate GFlags2
	check( GFlags2.Num() == 0 );
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		uint8* Ptr = new uint8[MAX_uint16+1];
		FMemory::Memzero( Ptr, MAX_uint16+1 );
		GFlags2.Add( Ptr );
	}

	/////////////
	// Tag coplanars.

	int32 ModelIndex = 0;
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		uint8* Flags2 = GFlags2[ModelIndex++];
		for(int32 SelectedNodeIndex = 0;SelectedNodeIndex < Model->Nodes.Num();SelectedNodeIndex++)
		{
			FBspNode&	SelectedNode = Model->Nodes[SelectedNodeIndex];
			FBspSurf&	SelectedSurf = Model->Surfs[SelectedNode.iSurf];

			if(SelectedSurf.PolyFlags & PF_Selected)
			{
				const FVector	SelectedBase = (FVector)Model->Points[Model->Verts[SelectedNode.iVertPool].pVertex];
				const FVector SelectedNormal = (FVector)Model->Vectors[SelectedSurf.vNormal];

				for(int32 NodeIndex = 0;NodeIndex < Model->Nodes.Num();NodeIndex++)
				{
					FBspNode&	Node = Model->Nodes[NodeIndex];
					FBspSurf&	Surf = Model->Surfs[Node.iSurf];
					const FVector Base = (FVector)Model->Points[Model->Verts[Node.iVertPool].pVertex];
					const FVector Normal = (FVector)Model->Vectors[Surf.vNormal];

					const float ParallelDotThreshold = 0.98f; // roughly 11.4 degrees (!), but this is the long-standing behavior.
					if (FVector::Coincident(SelectedNormal, Normal, ParallelDotThreshold) &&
						FVector::Coplanar(SelectedBase, SelectedNormal, Base, Normal, ParallelDotThreshold) && !(Surf.PolyFlags & PF_Selected))
					{
						Flags2[Node.iSurf] = 1;
					}
				}
			}
		}
	}

	do {} while (TagAdjacentsType(InWorld, ADJACENT_COPLANARS) > 0);

	// Free GFlags2
	for ( int32 i = 0 ; i < GFlags2.Num() ; ++i )
	{
		delete[] GFlags2[i];
	}
	GFlags2.Empty();
}

void UEditorEngine::polySelectMatchingBrush(UModel *InModel)
{
	TArray<ABrush*> Brushes;
	GetListOfUniqueBrushes( &Brushes, InModel );

	// Select all the faces.
	for( int32 i = 0 ; i < InModel->Surfs.Num() ; i++ )
	{
		FBspSurf* Surf = &InModel->Surfs[i];
		if ( Surf->Actor )
		{
			// Select all the polys on each brush in the unique list.
			for( int32 brush = 0 ; brush < Brushes.Num() ; brush++ )
			{
				ABrush* CurBrush = Brushes[brush];
				if( Surf->Actor == CurBrush )
				{
					for( int32 poly = 0 ; poly < CurBrush->Brush->Polys->Element.Num() ; poly++ )
					{
						if( Surf->iBrushPoly == poly )
						{
							InModel->ModifySurf( i, 0 );
							GEditor->SelectBSPSurf( InModel, i, true, false );
						}
					}
				}
			}
		}
	}
	NoteSelectionChange();
}


void UEditorEngine::polySelectMatchingMaterial(UWorld* InWorld, bool bCurrentLevelOnly)
{
	// true if at least one surface was selected.
	bool bSurfaceWasSelected = false;

	// true if default material representations have already been added to the materials list.
	bool bDefaultMaterialAdded = false;

	TArray<UMaterialInterface*> Materials;

	if ( bCurrentLevelOnly )
	{
		// Get list of unique materials that are on selected faces.
		for ( TSelectedSurfaceIterator<FCurrentLevelSurfaceLevelFilter> It(InWorld) ; It ; ++It )
		{
			if ( It->Material && It->Material != UMaterial::GetDefaultMaterial(MD_Surface) )
			{
				Materials.AddUnique( It->Material );
			}
			else if ( !bDefaultMaterialAdded )
			{
				bDefaultMaterialAdded = true;

				// Add both representations of the default material.
				Materials.AddUnique( NULL );
				Materials.AddUnique( UMaterial::GetDefaultMaterial(MD_Surface) );
			}
		}

		// Select all surfaces with matching materials.
		for ( TSurfaceIterator<FCurrentLevelSurfaceLevelFilter> It(InWorld) ; It ; ++It )
		{
			// Map the default material to NULL, so that NULL assignments match manual default material assignments.
			if( Materials.Contains( It->Material ) )
			{
				UModel* Model = It.GetModel();
				const int32 SurfaceIndex = It.GetSurfaceIndex();
				Model->ModifySurf( SurfaceIndex, 0 );
				GEditor->SelectBSPSurf( Model, SurfaceIndex, true, false );
				bSurfaceWasSelected = true;
			}
		}
	}
	else
	{
		// Get list of unique materials that are on selected faces.
		for ( TSelectedSurfaceIterator<> It(InWorld) ; It ; ++It )
		{
			if ( It->Material && It->Material != UMaterial::GetDefaultMaterial(MD_Surface) )
			{
				Materials.AddUnique( It->Material );
			}
			else if ( !bDefaultMaterialAdded )
			{
				bDefaultMaterialAdded = true;

				// Add both representations of the default material.
				Materials.AddUnique( NULL );
				Materials.AddUnique( UMaterial::GetDefaultMaterial(MD_Surface) );
			}
		}

		// Select all surfaces with matching materials.
		for ( TSurfaceIterator<> It(InWorld) ; It ; ++It )
		{
			// Map the default material to NULL, so that NULL assignments match manual default material assignments.
			if( Materials.Contains( It->Material ) )
			{
				UModel* Model = It.GetModel();
				const int32 SurfaceIndex = It.GetSurfaceIndex();
				Model->ModifySurf( SurfaceIndex, 0 );
				GEditor->SelectBSPSurf( Model, SurfaceIndex, true, false );
				bSurfaceWasSelected = true;
			}
		}
	}

	if ( bSurfaceWasSelected )
	{
		NoteSelectionChange();
	}
}


void UEditorEngine::polySelectMatchingResolution(UWorld* InWorld, bool bCurrentLevelOnly)
{
	// true if at least one surface was selected.
	bool bSurfaceWasSelected = false;

	TArray<float> SelectedResolutions;

	if (bCurrentLevelOnly == true)
	{
		for (TSelectedSurfaceIterator<FCurrentLevelSurfaceLevelFilter> It(InWorld); It; ++It)
		{
			SelectedResolutions.AddUnique(It->LightMapScale);
		}

		if (SelectedResolutions.Num() > 0)
		{
			if (SelectedResolutions.Num() > 1)
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "BSPSelect_DifferentResolutionsSelected", "Different selected resolutions.\nCan only select matching for a single resolution."));
			}
			else
			{
				// Select all surfaces with matching materials.
				for (TSurfaceIterator<FCurrentLevelSurfaceLevelFilter> It(InWorld); It; ++It)
				{
					if (SelectedResolutions.Contains(It->LightMapScale))
					{
						UModel* Model = It.GetModel();
						const int32 SurfaceIndex = It.GetSurfaceIndex();
						Model->ModifySurf( SurfaceIndex, 0 );
						GEditor->SelectBSPSurf( Model, SurfaceIndex, true, false );
						bSurfaceWasSelected = true;
					}
				}
			}
		}
	}
	else
	{
		for (TSelectedSurfaceIterator<> It(InWorld); It; ++It)
		{
			SelectedResolutions.AddUnique(It->LightMapScale);
		}

		if (SelectedResolutions.Num() > 0)
		{
			if (SelectedResolutions.Num() > 1)
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "BSPSelect_DifferentResolutionsSelected", "Different selected resolutions.\nCan only select matching for a single resolution."));
			}
			else
			{
				// Select all surfaces with matching materials.
				for (TSurfaceIterator<> It(InWorld); It; ++It)
				{
					if (SelectedResolutions.Contains(It->LightMapScale))
					{
						UModel* Model = It.GetModel();
						const int32 SurfaceIndex = It.GetSurfaceIndex();
						Model->ModifySurf( SurfaceIndex, 0 );
						GEditor->SelectBSPSurf( Model, SurfaceIndex, true, false );
						bSurfaceWasSelected = true;
					}
				}
			}
		}
	}

	if ( bSurfaceWasSelected )
	{
		NoteSelectionChange();
	}
}

void UEditorEngine::polySelectAdjacentWalls( UWorld* InWorld, UModel* InModel )
{
	do 
	{
	} 
	while (TagAdjacentsType(InWorld, ADJACENT_WALLS) > 0);
}

void UEditorEngine::polySelectAdjacentFloors( UWorld* InWorld, UModel* InModel )
{
	do
	{
	}
	while (TagAdjacentsType(InWorld, ADJACENT_FLOORS) > 0);
}

void UEditorEngine::polySelectAdjacentSlants( UWorld* InWorld, UModel* InModel )
{
	do 
	{
	} 
	while (TagAdjacentsType(InWorld, ADJACENT_SLANTS) > 0);
}

void UEditorEngine::polySelectReverse( UModel* InModel )
{
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		InModel->ModifySurf( i, 0 );
		Poly->PolyFlags ^= PF_Selected;
	}
}

void UEditorEngine::polyMemorizeSet( UModel* InModel )
{
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		if (Poly->PolyFlags & PF_Selected) 
		{
			if (!(Poly->PolyFlags & PF_Memorized))
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags |= (PF_Memorized);
			}
		}
		else
		{
			if (Poly->PolyFlags & PF_Memorized)
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags &= (~PF_Memorized);
			}
		}
	}
}

void UEditorEngine::polyRememberSet( UModel* InModel )
{
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		if (Poly->PolyFlags & PF_Memorized) 
		{
			if (!(Poly->PolyFlags & PF_Selected))
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags |= (PF_Selected);
			}
		}
		else
		{
			if (Poly->PolyFlags & PF_Selected)
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags &= (~PF_Selected);
			}
		}
	}
}

void UEditorEngine::polyXorSet( UModel* InModel )
{
	int			Flag1,Flag2;
	//
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		Flag1 = (Poly->PolyFlags & PF_Selected ) != 0;
		Flag2 = (Poly->PolyFlags & PF_Memorized) != 0;
		//
		if (Flag1 ^ Flag2)
		{
			if (!(Poly->PolyFlags & PF_Selected))
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags |= PF_Selected;
			}
		}
		else
		{
			if (Poly->PolyFlags & PF_Selected)
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags &= (~PF_Selected);
			}
		}
	}
}

void UEditorEngine::polyUnionSet( UModel* InModel )
{
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		if (!(Poly->PolyFlags & PF_Memorized))
		{
			if (Poly->PolyFlags & PF_Selected)
			{
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags &= (~PF_Selected);
			}
		}
	}
}

void UEditorEngine::polyIntersectSet( UModel* InModel )
{
	for (int32 i=0; i<InModel->Surfs.Num(); i++)
	{
		FBspSurf *Poly = &InModel->Surfs[i];
		if ((Poly->PolyFlags & PF_Memorized) && !(Poly->PolyFlags & PF_Selected))
		{
			InModel->ModifySurf( i, 0 );
			Poly->PolyFlags |= PF_Selected;
		}
	}
}

void UEditorEngine::polySelectZone( UModel* InModel )
{
	// identify the list of currently selected zones
	TArray<int32> iZoneList;
	for( int32 i = 0; i < InModel->Nodes.Num(); i++ )
	{
		FBspNode* Node = &InModel->Nodes[i];
		FBspSurf* Poly = &InModel->Surfs[ Node->iSurf ];
		if( Poly->PolyFlags & PF_Selected )
		{
			if( Node->iZone[1] != 0 )
			{
				iZoneList.AddUnique( Node->iZone[1] ); //front zone
			}

			if( Node->iZone[0] != 0 )
			{
				iZoneList.AddUnique( Node->iZone[0] ); //back zone
			}
		}
	}

	// select all polys that are match one of the zones identified above
	for( int32 i = 0; i < InModel->Nodes.Num(); i++ )
	{
		FBspNode* Node = &InModel->Nodes[i];
		for( int32 j = 0; j < iZoneList.Num(); j++ ) 
		{
			if( Node->iZone[1] == iZoneList[j] || Node->iZone[0] == iZoneList[j] )
			{
				FBspSurf* Poly = &InModel->Surfs[ Node->iSurf ];
				InModel->ModifySurf( i, 0 );
				Poly->PolyFlags |= PF_Selected;
			}
		}
	}

}

/*---------------------------------------------------------------------------------------
   Brush selection functions
---------------------------------------------------------------------------------------*/

//
// Generic selection routines
//

typedef int32 (*BRUSH_SEL_FUNC)( ABrush* Brush, int32 Tag );

static void MapSelect( UWorld* InWorld, BRUSH_SEL_FUNC Func, int32 Tag )
{
	for( FStaticBrushIterator It(InWorld); It; ++It )
	{
		ABrush* Brush = CastChecked<ABrush>(*It);
		if( Func( Brush, Tag ) )
		{
			GEditor->SelectActor( Brush, true, false );
		}
		else
		{
			GEditor->SelectActor( Brush, false, false );
		}
	}
}

/**
 * Selects no brushes.
 */
static int32 BrushSelectNoneFunc( ABrush* Actor, int32 Tag )
{
	return 0;
}

/**
 * Selects brushes by their CSG operation.
 */
static int32 BrushSelectOperationFunc( ABrush* Actor, int32 Tag )
{
	return ((EBrushType)Actor->BrushType == Tag) && !(Actor->PolyFlags & (PF_NotSolid | PF_Semisolid));
}
void UEditorEngine::MapSelectOperation( UWorld* InWorld, EBrushType BrushType)
{
	MapSelect( InWorld, BrushSelectOperationFunc, BrushType );
}

int32 BrushSelectFlagsFunc( ABrush* Actor, int32 Tag )
{
	return Actor->PolyFlags & Tag;
}
void UEditorEngine::MapSelectFlags( UWorld* InWorld, uint32 Flags)
{
	MapSelect( InWorld, BrushSelectFlagsFunc, (int)Flags );
}

void UEditorEngine::MapBrushGet(UWorld* InWorld)
{
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ABrush* BrushActor = Cast< ABrush >( Actor );
		if( BrushActor && !FActorEditorUtils::IsABuilderBrush(Actor) )
		{
			check( BrushActor->GetWorld() );			
			ABrush* WorldBrush = BrushActor->GetWorld()->GetDefaultBrush();
			check( WorldBrush );
			WorldBrush->Modify(false);
			WorldBrush->Brush->Polys->Element = BrushActor->Brush->Polys->Element;
			WorldBrush->CopyPosRotScaleFrom( BrushActor );

			WorldBrush->ReregisterAllComponents();
			break;
		}
	}

	GEditor->SelectNone( false, true );
	GEditor->SelectActor(InWorld->GetDefaultBrush(), true, true);
}


void UEditorEngine::mapBrushPut()
{
	for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		ABrush* BrushActor = Cast< ABrush >( Actor );
		if( BrushActor && !FActorEditorUtils::IsABuilderBrush(Actor) )
		{
			check( BrushActor->GetWorld() );
			ABrush* WorldBrush = BrushActor->GetWorld()->GetDefaultBrush();
			check( WorldBrush );

			BrushActor->Modify(false);
			BrushActor->Brush->Polys->Element = WorldBrush->Brush->Polys->Element;
			BrushActor->CopyPosRotScaleFrom( WorldBrush );
			BrushActor->SetNeedRebuild(BrushActor->GetLevel());

			WorldBrush->ReregisterAllComponents();

			GLevelEditorModeTools().UpdateInternalData();
		}
	}
}

//
// Generic private routine for send to front / send to back
//
static void SendTo( UWorld* InWorld, int32 bSendToFirst )
{
	ULevel*	Level = InWorld->GetCurrentLevel();
	for (AActor* Actor : Level->Actors)
	{
		if(Actor)
		{
			Actor->Modify();
		}
	}
	
	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	//@todo locked levels - do we need to skip locked levels?
	// Partition.
	TArray<AActor*> Lists[2];
	for( int32 i=2; i<Level->Actors.Num(); i++ )
	{
		if( Level->Actors[i] )
		{
			Lists[(Level->Actors[i]->IsSelected() ? 1 : 0) ^ bSendToFirst ^ 1].Add( Level->Actors[i] );
			Level->Actors[i]->MarkPackageDirty();
			LevelDirtyCallback.Request();
		}
	}

	// Refill.
	check(Level->Actors.Num()>=2);
	Level->Actors.RemoveAt(2,Level->Actors.Num()-2);
	for( int32 i=0; i<2; i++ )
	{
		for( int32 j=0; j<Lists[i].Num(); j++ )
		{
			Level->Actors.Add( Lists[i][j] );
		}
	}
}


void UEditorEngine::mapSendToFirst(UWorld* InWorld)
{
	SendTo( InWorld, 0 );
}


void UEditorEngine::mapSendToLast(UWorld* InWorld)
{
	SendTo( InWorld, 1 );
}


void UEditorEngine::mapSendToSwap(UWorld* InWorld)
{
	int32			Count	= 0;
	ULevel*		Level	= InWorld->GetCurrentLevel();
	TObjectPtr<AActor>*	Actors[2];

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	//@todo locked levels - skip for locked levels?
	for( int32 i=2; i<Level->Actors.Num() && Count < 2; i++ )
	{
		auto& Actor = Level->Actors[i];
		if( Actor && Actor->IsSelected() )
		{
			Actors[Count] = &Actor;
			Count++;
			Actor->MarkPackageDirty();
			LevelDirtyCallback.Request();
		}
	}

	if( Count == 2 )
	{
		for (AActor* Actor : InWorld->GetCurrentLevel()->Actors)
		{
			Actor->Modify();
		}
		Exchange( *Actors[0], *Actors[1] );
	}
}

void UEditorEngine::MapSetBrush( UWorld* InWorld, EMapSetBrushFlags	PropertiesMask, uint16 BrushColor, FName GroupName, uint32 SetPolyFlags, uint32 ClearPolyFlags, uint32 BrushType, int32 DrawType )
{
	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	for( FStaticBrushIterator It(InWorld); It; ++It )
	{
		ABrush* Brush = CastChecked<ABrush>(*It);
		if( !FActorEditorUtils::IsABuilderBrush(Brush) && Brush->IsSelected() )
		{
			if( PropertiesMask & MSB_PolyFlags )
			{
				Brush->Modify(false);
				Brush->PolyFlags = (Brush->PolyFlags & ~ClearPolyFlags) | SetPolyFlags;
				Brush->UpdateComponentTransforms();
				Brush->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
			if( PropertiesMask & MSB_BrushType )
			{
				Brush->Modify(false);
				Brush->BrushType = EBrushType(BrushType);
				Brush->UpdateComponentTransforms();
				Brush->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
		}
	}
}


void UEditorEngine::polyTexPan(UModel *Model,int32 PanU,int32 PanV,int32 Absolute)
{
	for(int32 SurfaceIndex = 0;SurfaceIndex < Model->Surfs.Num();SurfaceIndex++)
	{
		const FBspSurf&	Surf = Model->Surfs[SurfaceIndex];

		if(Surf.PolyFlags & PF_Selected)
		{
			if(Absolute)
			{
				Model->Points[Surf.pBase] = FVector3f::ZeroVector;
			}

			const FVector TextureU = (FVector)Model->Vectors[Surf.vTextureU];
			const FVector TextureV = (FVector)Model->Vectors[Surf.vTextureV];

			Model->Points[Surf.pBase] += FVector3f(PanU * (TextureU / TextureU.SizeSquared()));
			Model->Points[Surf.pBase] += FVector3f(PanV * (TextureV / TextureV.SizeSquared()));

			const bool bUpdateTexCoords = true;
			const bool bOnlyRefreshSurfaceMaterials = true;
			polyUpdateBrush(Model, SurfaceIndex, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
		}
	}
}


void UEditorEngine::polyTexScale( UModel* Model, float UU, float UV, float VU, float VV, bool Absolute )
{
	for( int32 i=0; i<Model->Surfs.Num(); i++ )
	{
		FBspSurf *Poly = &Model->Surfs[i];
		if (Poly->PolyFlags & PF_Selected)
		{
			FVector OriginalU = (FVector)Model->Vectors[Poly->vTextureU];
			FVector OriginalV = (FVector)Model->Vectors[Poly->vTextureV];

			if( Absolute )
			{
				OriginalU *= 1.0/OriginalU.Size();
				OriginalV *= 1.0/OriginalV.Size();
			}

			// Calc new vectors.
			Model->Vectors[Poly->vTextureU] = FVector3f(OriginalU * UU + OriginalV * UV);
			Model->Vectors[Poly->vTextureV] = FVector3f(OriginalU * VU + OriginalV * VV);

			// Update generating brush poly.
			const bool bUpdateTexCoords = true;
			const bool bOnlyRefreshSurfaceMaterials = true;
			polyUpdateBrush(Model, i, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
		}
	}
}
