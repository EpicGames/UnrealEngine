// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDirectories.h"

#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Templates/UnrealTemplate.h"

FEditorDirectories& FEditorDirectories::Get()
{
	static FEditorDirectories Directories;
	return Directories;
}

FEditorDirectories::FEditorDirectories() :
	bHasLoaded(false)
{
	ConfigFile = GEditorPerProjectIni;
	ConfigSectionName = TEXT("Directories2");
}

void FEditorDirectories::LoadLastDirectories()
{
	for( int32 CurDirectoryIndex = 0; CurDirectoryIndex < UE_ARRAY_COUNT( LastDir ); ++CurDirectoryIndex )
	{
		LastDir[CurDirectoryIndex].Reset();
	}

	// NOTE: We append a "2" to the section name to enforce backwards compatibility.  "Directories" is deprecated.
	GConfig->GetString( *ConfigSectionName, TEXT("UNR"),				LastDir[ELastDirectory::UNR],					ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("BRUSH"),			LastDir[ELastDirectory::BRUSH],				ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("FBX"),				LastDir[ELastDirectory::FBX],					ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("FBXAnim"),			LastDir[ELastDirectory::FBX_ANIM],			ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("GenericImport"),	LastDir[ELastDirectory::GENERIC_IMPORT],		ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("GenericExport"),	LastDir[ELastDirectory::GENERIC_EXPORT],		ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("GenericOpen"),		LastDir[ELastDirectory::GENERIC_OPEN],		ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("GenericSave"),		LastDir[ELastDirectory::GENERIC_SAVE],		ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("MeshImportExport"),	LastDir[ELastDirectory::MESH_IMPORT_EXPORT],	ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("WorldRoot"),		LastDir[ELastDirectory::WORLD_ROOT],			ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("Level"),			LastDir[ELastDirectory::LEVEL],					ConfigFile );
	GConfig->GetString( *ConfigSectionName, TEXT("Project"),			LastDir[ELastDirectory::PROJECT],				ConfigFile );

	// Set up some defaults if they're not defined in the ini
	if (DefaultDir.IsEmpty())
	{
		DefaultDir = FPaths::ProjectContentDir();
	}

	for( int32 CurDirectoryIndex = 0; CurDirectoryIndex < UE_ARRAY_COUNT( LastDir ); ++CurDirectoryIndex )
	{
		if (LastDir[ CurDirectoryIndex ].IsEmpty() || !FPaths::DirectoryExists(LastDir[CurDirectoryIndex]))
		{
			// Default all directories to the game content folder
			if (CurDirectoryIndex == ELastDirectory::LEVEL)
			{
				const FString DefaultMapDir = DefaultDir / TEXT("Maps");
				if( IFileManager::Get().DirectoryExists( *DefaultMapDir ) )
				{
					LastDir[CurDirectoryIndex] = DefaultMapDir;
					continue;
				}
			}
			else if (CurDirectoryIndex == ELastDirectory::PROJECT)
			{
				LastDir[CurDirectoryIndex] = FPaths::RootDir();
				continue;
			}

			// Set to the default dir
			LastDir[ CurDirectoryIndex ] = DefaultDir;
		}
	}

	bHasLoaded = true;
}

/** Writes the current "LastDir" array back out to the config files */
void FEditorDirectories::SaveLastDirectories()
{
	ensure(bHasLoaded);

	// Save out default file directories
	GConfig->SetString( *ConfigSectionName, TEXT("UNR"),				*LastDir[ELastDirectory::UNR],				ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("BRUSH"),			*LastDir[ELastDirectory::BRUSH],				ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("FBX"),				*LastDir[ELastDirectory::FBX],				ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("FBXAnim"),			*LastDir[ELastDirectory::FBX_ANIM],			ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("GenericImport"),	*LastDir[ELastDirectory::GENERIC_IMPORT],		ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("GenericExport"),	*LastDir[ELastDirectory::GENERIC_EXPORT],		ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("GenericOpen"),		*LastDir[ELastDirectory::GENERIC_OPEN],		ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("GenericSave"),		*LastDir[ELastDirectory::GENERIC_SAVE],		ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("MeshImportExport"),	*LastDir[ELastDirectory::MESH_IMPORT_EXPORT],	ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("WorldRoot"),		*LastDir[ELastDirectory::WORLD_ROOT],			ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("Level"),			*LastDir[ELastDirectory::LEVEL],				ConfigFile );
	GConfig->SetString( *ConfigSectionName, TEXT("Project"),			*LastDir[ELastDirectory::PROJECT],				ConfigFile );
}

FString FEditorDirectories::GetLastDirectory( const ELastDirectory::Type InLastDir ) const
{
	if ( InLastDir >= 0 && InLastDir < UE_ARRAY_COUNT( LastDir ) )
	{
		return LastDir[InLastDir];
	}
	return DefaultDir;
}

void FEditorDirectories::SetLastDirectory( const ELastDirectory::Type InLastDir, const FString& InLastStr )
{
	if ( InLastDir >= 0 && InLastDir < UE_ARRAY_COUNT( LastDir ) )
	{
		LastDir[InLastDir] = InLastStr;
	}
}

void FEditorDirectories::SetOverride(const FString& InConfigFile, const FString& InConfigSectionName, const FString& InDefaultDir)
{
	// Prevent saving before config has been loaded otherwise data will be lost
	if (bHasLoaded)
	{
		SaveLastDirectories();
	}

	ConfigFile = InConfigFile;
	ConfigSectionName = InConfigSectionName;
	DefaultDir = InDefaultDir;
	LoadLastDirectories();
}

void FEditorDirectories::ResetOverride()
{
	SaveLastDirectories();
	ConfigFile = GEditorPerProjectIni;
	ConfigSectionName = TEXT("Directories2");
	DefaultDir.Reset();
	LoadLastDirectories();
}
