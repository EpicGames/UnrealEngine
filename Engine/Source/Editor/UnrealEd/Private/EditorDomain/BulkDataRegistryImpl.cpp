// Copyright Epic Games, Inc. All Rights Reserved.

#include "BulkDataRegistryImpl.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DerivedDataRequestTypes.h"
#include "EditorBuildInputResolver.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/PlatformTime.h"
#include "IO/IoHash.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkDataRegistry.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/RefCounting.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "atomic"

namespace UE::BulkDataRegistry::Private
{

namespace Constants
{
	constexpr uint64 TempLoadedPayloadsSizeBudget = 1024 * 1024 * 100;
	constexpr double TempLoadedPayloadsDuration = 60.;
}

TConstArrayView<uint8> MakeArrayView(FSharedBuffer Buffer)
{
	return TConstArrayView<uint8>(reinterpret_cast<const uint8*>(Buffer.GetData()), Buffer.GetSize());
}

/** Add a hook to the BulkDataRegistry's startup delegate to use the EditorDomain as the BulkDataRegistry */
class FEditorDomainRegisterAsBulkDataRegistry
{
public:
	FEditorDomainRegisterAsBulkDataRegistry()
	{
		IBulkDataRegistry::GetSetBulkDataRegistryDelegate().BindStatic(SetBulkDataRegistry);
	}

	static IBulkDataRegistry* SetBulkDataRegistry()
	{
		return new FBulkDataRegistryImpl();
	}
} GRegisterAsBulkDataRegistry;


FBulkDataRegistryImpl::FBulkDataRegistryImpl()
{
	SharedDataLock = new FTaskSharedDataLock();
	// We piggyback on the BulkDataRegistry hook to set the pointer to tunnel in the pointer to the EditorBuildInputResolver as well
	SetGlobalBuildInputResolver(&UE::DerivedData::FEditorBuildInputResolver::Get());
	FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FBulkDataRegistryImpl::OnEndLoadPackage);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FBulkDataRegistryImpl::OnEnginePreExit);
}

void FBulkDataRegistryImpl::OnEnginePreExit()
{
	Teardown();
}

FBulkDataRegistryImpl::~FBulkDataRegistryImpl()
{
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	Teardown();
}

void FBulkDataRegistryImpl::Teardown()
{
	{
		FReadScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			return;
		}
	}

	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
	SetGlobalBuildInputResolver(nullptr);

	TMap<FGuid, FUpdatingPayload> LocalUpdatingPayloads;
	{
		FWriteScopeLock SharedDataScopeLock(SharedDataLock->ActiveLock);
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		FScopeLock PendingPackageScopeLock(&PendingPackageLock);

		// Disable all activity that might come in from other threads
		bActive = false;
		SharedDataLock->bActive = false;

		// Take custody of UpdatingPayloads
		Swap(LocalUpdatingPayloads, UpdatingPayloads);
	}

	// Since the UpdatingPayloads AsyncTasks can no longer access their Requesters, we have to call those callbacks
	for (TPair<FGuid, FUpdatingPayload>& Pair : LocalUpdatingPayloads)
	{
		for (TUniqueFunction<void(bool, const FCompressedBuffer&)>& Requester : Pair.Value.Requesters)
		{
			Requester(false, FCompressedBuffer());
		}
	}
	LocalUpdatingPayloads.Empty();

	// Clear PendingPackages
	TMap<FName, TUniquePtr<FPendingPackage>> LocalPendingPackages;
	{
		FScopeLock PendingPackageScopeLock(&PendingPackageLock);
		// Take custody of PendingPackages
		Swap(LocalPendingPackages, PendingPackages);
	}
	for (TPair<FName, TUniquePtr<FPendingPackage>>& PendingPackagePair : LocalPendingPackages)
	{
		PendingPackagePair.Value->Cancel();
	}
	LocalPendingPackages.Empty();

	// Clear PendingPackagePayloadIds. We have to take custody of PendingPayloadIds after calling
	// Cancel from all FPendingPackage, as the FPendingPackages have callbacks that may write to PendingPayloadIds
	TMap<FGuid, TRefCountPtr<FPendingPayloadId>> LocalPendingPayloadIds;
	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		// Take custody of PendingPayloadIds
		Swap(LocalPendingPayloadIds, PendingPayloadIds);
	}
	for (TPair<FGuid, TRefCountPtr<FPendingPayloadId>>& PendingPayloadIdPair : LocalPendingPayloadIds)
	{
		PendingPayloadIdPair.Value->Cancel();
	}

	// Finally wait for any FUpdatePayloadWorker jobs that are still in flight
	if (SharedDataLock->UpdatePayloadWorkerInFlight > 0)
	{
		const double StartTime = FPlatformTime::Seconds();

		UE_LOGF(LogBulkDataRegistry, Log, "Waiting on %d FUpdatePayloadWorker jobs to complete...", SharedDataLock->UpdatePayloadWorkerInFlight.load());
		
		while (SharedDataLock->UpdatePayloadWorkerInFlight > 0)
		{
			FPlatformProcess::SleepNoStats(0.0f);
		}

		UE_LOGF(LogBulkDataRegistry, Log, "All FUpdatePayloadWorker jobs finished, exit was stalled for %.1f(s)", FPlatformTime::Seconds() - StartTime);
	}
}

void FBulkDataRegistryImpl::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	TArray<TUniquePtr<FPendingPackage>> PackagesToWrite;
	{
		FScopeLock PendingPackageScopeLock(&PendingPackageLock);

		for (UPackage* LoadedPackage : Context.LoadedPackages)
		{
			FName PackageName = LoadedPackage->GetFName();
			uint32 KeyHash = GetTypeHash(PackageName);
			TUniquePtr<FPendingPackage>* PendingPackage = PendingPackages.FindByHash(KeyHash, PackageName);
			if (PendingPackage)
			{
				bool bShouldRemove;
				bool bShouldWriteCache;
				check(PendingPackage->IsValid());
				(*PendingPackage)->OnEndLoad(bShouldRemove, bShouldWriteCache);
				// We do not hold a lock when calling WriteCache, so we require that the package no longer
				// be accessible to other threads through PendingPackages if we need to write the cache,
				// so bShouldRemove must be true if bShouldWriteCache is
				check(!bShouldWriteCache || bShouldRemove);
				if (bShouldRemove)
				{
					if (bShouldWriteCache)
					{
						PackagesToWrite.Add(MoveTemp(*PendingPackage));
					}
					PendingPackages.RemoveByHash(KeyHash, PackageName);
				}
			}
		}
	}

	for (TUniquePtr<FPendingPackage>& Package : PackagesToWrite)
	{
		Package->WriteCache();
	}
}

UE::BulkDataRegistry::ERegisterResult
FBulkDataRegistryImpl::TryRegister(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData)
{
	if (!BulkData.GetIdentifier().IsValid())
	{
		return UE::BulkDataRegistry::ERegisterResult::Success;
	}

	bool bAllowedToReadWritePayloadIdFromCache = false;
	FName PackageName;
	UE::Serialization::FEditorBulkData CopyBulk(BulkData.CopyTornOff());
	UE::Serialization::FEditorBulkData PendingPackageBulk;
	if (Owner)
	{
		PackageName = Owner->GetFName();
		if (Owner->GetFileSize() // We only cache the BulkDataList for disk packages
			&& !Owner->GetHasBeenEndLoaded() // We only cache BulkDatas that are loaded before the package finishes loading
			&& CopyBulk.CanSaveForRegistry()
			)
		{
			bAllowedToReadWritePayloadIdFromCache = true;
			PendingPackageBulk = CopyBulk;
		}
	}

	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			return UE::BulkDataRegistry::ERegisterResult::Success;
		}
		UE::BulkDataRegistry::Private::FRegisteredBulk& RegisteredBulk = Registry.FindOrAdd(BulkData.GetIdentifier());
		if (RegisteredBulk.bRegistered)
		{
			bool bAllowSharing = false;
			// If the original BulkData has left memory and a new bulkdata registers with the same package owner,
			// assume that it is the same BulkData being reloaded and allow the sharing
			bAllowSharing |= (!RegisteredBulk.bInMemory && PackageName == RegisteredBulk.PackageName);
			if (!bAllowSharing)
			{
				return UE::BulkDataRegistry::ERegisterResult::AlreadyExists;
			}

			if (RegisteredBulk.PackageName.IsNone())
			{
				RegisteredBulk.PackageName = PackageName;
			}
			RegisteredBulk.bRegistered = true;
			RegisteredBulk.bInMemory = true;
			RegisteredBulk.bAllowedToWritePayloadIdToCache = false;
			RegisteredBulk.bPayloadAvailable = true;

			// For updated registrations, skip reading payloadid from cache, and call ResaveSizeTracker.Update rather than Register.
			ResaveSizeTracker.UpdateRegistrationData(Owner, BulkData);
			return UE::BulkDataRegistry::ERegisterResult::Success;
		}
		else
		{
			bool bCachedLocationMatches = false;
			UE::Serialization::FEditorBulkData& TargetBulkData = RegisteredBulk.BulkData;

			if (TargetBulkData.GetIdentifier().IsValid())
			{
				// This BulkData was added when loading the cached list of BulkData for the package, but has not been registered by an in-memory bulkdata
				check(TargetBulkData.GetIdentifier() == BulkData.GetIdentifier());
				bCachedLocationMatches = BulkData.LocationMatches(TargetBulkData);
			}

			if (!bCachedLocationMatches || !BulkData.HasPlaceholderPayloadId() || TargetBulkData.HasPlaceholderPayloadId())
			{
				// Copy the new BulkData over the value we got from the cache; the new BulkData is more authoritative
				TargetBulkData = MoveTemp(CopyBulk);
				RegisteredBulk.bAllowedToWritePayloadIdToCache = bAllowedToReadWritePayloadIdFromCache;
			}
			else
			{
				// Otherwise Keep the TargetBulkData, since it matches the location and has already calculated the PayloadId
				// Set that it is no longer allowed to write PayloadIdToCache since it has been modified after the initial load.
				RegisteredBulk.bAllowedToWritePayloadIdToCache = false;
			}

			RegisteredBulk.PackageName = PackageName;
			RegisteredBulk.bRegistered = true;
			RegisteredBulk.bInMemory = true;
			RegisteredBulk.bPayloadAvailable = true;
		}
	}
	if (bAllowedToReadWritePayloadIdFromCache)
	{
		AddPendingPackageBulkData(PackageName, MoveTemp(PendingPackageBulk), bAllowedToReadWritePayloadIdFromCache);

		// If The PendingPackage found that it is not allowed to use cache operations, then go back into the BulkData
		// in the registry and mark that it is not allowed to write its payload ids to the cache. PayloadIds are not
		// allowed to be written unless we read them first and find they do not exist, and the only place we read them
		// is from the PendingPackage.
		if (!bAllowedToReadWritePayloadIdFromCache)
		{
			FWriteScopeLock RegistryScopeLock(RegistryLock);
			if (bActive)
			{
				UE::BulkDataRegistry::Private::FRegisteredBulk* RegisteredBulk
					= Registry.Find(BulkData.GetIdentifier());
				if (RegisteredBulk)
				{
					RegisteredBulk->bAllowedToWritePayloadIdToCache = false;
				}
			}
		}
	}

	ResaveSizeTracker.Register(Owner, BulkData);
	return UE::BulkDataRegistry::ERegisterResult::Success;
}

void FBulkDataRegistryImpl::UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData)
{
	if (!BulkData.GetIdentifier().IsValid())
	{
		UE_LOGF(LogBulkDataRegistry, Warning, "UpdateRegistrationData called with invalid BulkData for Owner %ls.",
			Owner ? *Owner->GetName() : TEXT("<unknown>"));
		return;
	}

	FName PackageName = Owner ? Owner->GetFName() : NAME_None;
	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			return;
		}
		UE::BulkDataRegistry::Private::FRegisteredBulk& RegisteredBulk = Registry.FindOrAdd(BulkData.GetIdentifier());
		// Update the Owner if we have it. This is important for Package Renames, which call UpdateRegistrationData when they save
		// to the newly renamed package, with the renamed Owner package.
		if (!PackageName.IsNone())
		{
			RegisteredBulk.PackageName = PackageName;
		}
		RegisteredBulk.BulkData = BulkData.CopyTornOff();
		// Set that it is no longer allowed to write PayloadIdToCache since it has been modified after the initial load.
		RegisteredBulk.bAllowedToWritePayloadIdToCache = false;
		RegisteredBulk.bRegistered = true;
		RegisteredBulk.bInMemory = true;
		RegisteredBulk.bPayloadAvailable = true;
	}
	ResaveSizeTracker.UpdateRegistrationData(Owner, BulkData);
}

void FBulkDataRegistryImpl::Unregister(const UE::Serialization::FEditorBulkData& BulkData)
{
	const FGuid& Key = BulkData.GetIdentifier();
	FWriteScopeLock RegistryScopeLock(RegistryLock);
	if (!bActive)
	{
		return;
	}
	Registry.Remove(Key);
}

void FBulkDataRegistryImpl::OnExitMemory(const UE::Serialization::FEditorBulkData& BulkData)
{
	const FGuid& Key = BulkData.GetIdentifier();
	FWriteScopeLock RegistryScopeLock(RegistryLock);
	if (!bActive)
	{
		return;
	}
	FRegisteredBulk* Existing = Registry.Find(Key);
	if (Existing)
	{
		// Keep the FRegisteredBulk around to provide access to the BulkData in case it is requested without
		// loading the package that contains it.
		// But remove our (referenced tracked) pointer to its Payload. We do not want to force its payload
		// to remain in memory when the package is unloaded in the editor.
		// Note that we might have loaded the payload for our own purposes and set bHasTempPayload = true,
		// but that is just an optimization, and one that is not likely to occur for a BulkData in a package that
		// was loaded; bHasTempPayload is meant to provide access to the payload when the package was not loaded.
		// So clear that field as well.
		Existing->bHasTempPayload = false;
		Existing->bInMemory = false;

		// If the BulkData only existed in memory, then it is effectively gone; we clear its BulkData fields
		// and mark that we no longer have access to its payload. We keep it registered solely to keep track
		// of its package owner so we can detect duplicate Guid bugs.
		// We also do this for BulkDatas that used to have on-disk payloads, but that have been detached from their
		// linker (due e.g. to the package being unloaded or renamed) and are no longer loadable from disk.
		// For those packages, UnloadData fails to unload the Payload because it is the only accessible version
		// of the payload and UnloadData has a contract that it does not unload the data if not recoverable.
		// For our internal bulkdatas we don't need that contract; we can just let the payload become inaccessible
		// for these packages that detached their bulkdatas from disk in response to user operations.
		bool bRequiresClear = Existing->BulkData.IsMemoryOnlyPayload();
		if (!bRequiresClear)
		{
			Existing->BulkData.UnloadData();
			bRequiresClear = Existing->BulkData.IsPayloadLoaded();
		}
		if (bRequiresClear)
		{
			Existing->BulkData.Reset();
			Existing->BulkData.TearOff(); // Keep the TearOff flag after resetting
			Existing->bPayloadAvailable = false;
		}
	}
}

void FBulkDataRegistryImpl::UpdatePlaceholderPayloadId(const UE::Serialization::FEditorBulkData& BulkData)
{
	if (!BulkData.GetIdentifier().IsValid())
	{
		UE_LOGF(LogBulkDataRegistry, Warning, "UpdatePlaceholderPayloadId called with invalid BulkData.");
		return;
	}
	if (BulkData.HasPlaceholderPayloadId())
	{
		UE_LOGF(LogBulkDataRegistry, Warning, "UpdatePlaceholderPayloadId called with a BulkData that still has a PlaceholderPayloadId.");
		return;
	}
	const FGuid& Key = BulkData.GetIdentifier();

	TOptional<UE::Serialization::FEditorBulkData> WritePayloadIdBulkData;
	FName WritePayloadIdPackageName;
	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			return;
		}
		FRegisteredBulk* Existing = Registry.Find(Key);
		if (!Existing)
		{
			return;
		}
		if (!Existing->bRegistered ||
			!Existing->BulkData.GetIdentifier().IsValid() ||
			!Existing->BulkData.HasPlaceholderPayloadId() ||
			!BulkData.LocationMatches(Existing->BulkData))
		{
			return;
		}

		Existing->BulkData = BulkData.CopyTornOff();
		Existing->bAllowedToWritePayloadIdToCache = Existing->bAllowedToWritePayloadIdToCache && Existing->BulkData.CanSaveForRegistry();
		if (Existing->bAllowedToWritePayloadIdToCache)
		{
			// If there is a Get still pending the PayloadId may already be in the cache. Do not Put it until we find out whether
			// it exists. The response lambda in ReadPayloadIdsFromCache will issue the WritePayloadIdToCache if it is a miss.
			if (!PendingPayloadIds.Find(Key))
			{
				// Set that it is no longer allowed to write PayloadIdToCache since we are writing it now
				Existing->bAllowedToWritePayloadIdToCache = false;

				// Save the BulkData for writing to the cache outside of the lock
				WritePayloadIdBulkData.Emplace(Existing->BulkData);
				WritePayloadIdPackageName = Existing->PackageName;
			}
		}
	}
	if (WritePayloadIdBulkData)
	{
		FBulkDataRegistryImpl::WritePayloadIdToCache(WritePayloadIdPackageName, *WritePayloadIdBulkData);
	}
}

TFuture<UE::BulkDataRegistry::FMetaData> FBulkDataRegistryImpl::GetMeta(const FGuid& BulkDataId)
{
	bool bIsWriteLock = false;
	FRWScopeLock RegistryScopeLock(RegistryLock, SLT_ReadOnly);
	for (;;)
	{
		FRegisteredBulk* Existing = nullptr;
		if (bActive)
		{
			Existing = Registry.Find(BulkDataId);
		}
		if (!Existing || !Existing->bPayloadAvailable)
		{
			TPromise<UE::BulkDataRegistry::FMetaData> Promise;
			Promise.SetValue(UE::BulkDataRegistry::FMetaData{ FIoHash(), 0 });
			return Promise.GetFuture();
		}

		UE::Serialization::FEditorBulkData& BulkData = Existing->BulkData;
		if (!BulkData.HasPlaceholderPayloadId())
		{
			TPromise<UE::BulkDataRegistry::FMetaData> Promise;
			Promise.SetValue(UE::BulkDataRegistry::FMetaData{ BulkData.GetPayloadId(), static_cast<uint64>(BulkData.GetPayloadSize()) });
			return Promise.GetFuture();
		}

		if (!bIsWriteLock)
		{
			bIsWriteLock = true;
			RegistryScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			continue;
		}

		// The payload in the registry is missing its RawHash; start a thread to calculate it and subscribe our caller to the results
		FUpdatingPayload& UpdatingPayload = UpdatingPayloads.FindOrAdd(BulkDataId);
		if (!UpdatingPayload.AsyncTask)
		{
			UpdatingPayload.AsyncTask = new FAutoDeleteAsyncTask<FUpdatePayloadWorker>(this, BulkData, true /* bKeepTempLoadedPayload */);
			UpdatingPayload.AsyncTask->StartBackgroundTask();
		}
		TPromise<UE::BulkDataRegistry::FMetaData> Promise;
		TFuture<UE::BulkDataRegistry::FMetaData> Future = Promise.GetFuture();
		UpdatingPayload.Requesters.Add([Promise = MoveTemp(Promise)](bool bValid, const FCompressedBuffer& Buffer) mutable
			{
				Promise.SetValue(UE::BulkDataRegistry::FMetaData{ Buffer.GetRawHash(), Buffer.GetRawSize() });
			});
		return Future;
	}
}

TFuture<UE::BulkDataRegistry::FData> FBulkDataRegistryImpl::GetData(const FGuid& BulkDataId)
{
	TOptional<UE::Serialization::FEditorBulkData> CopyBulk;
	{
		bool bIsWriteLock = false;
		FRWScopeLock RegistryScopeLock(RegistryLock, SLT_ReadOnly);
		for (;;)
		{
			FRegisteredBulk* Existing = nullptr;
			if (bActive)
			{
				Existing = Registry.Find(BulkDataId);
			}
			if (!Existing || !Existing->bPayloadAvailable)
			{
				TPromise<UE::BulkDataRegistry::FData> Result;
				Result.SetValue(UE::BulkDataRegistry::FData{ FCompressedBuffer() });
				return Result.GetFuture();
			}

			if (!Existing->BulkData.HasPlaceholderPayloadId() && !Existing->bHasTempPayload)
			{
				// The contract of FEditorBulkData does not guarantee that GetCompressedPayload() is a quick operation (it may load the data
				// synchronously), so copy the BulkData into a temporary and call it outside the lock
				CopyBulk.Emplace(Existing->BulkData);
				break;
			}

			if (!bIsWriteLock)
			{
				bIsWriteLock = true;
				RegistryScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
				continue;
			}

			if (!Existing->BulkData.HasPlaceholderPayloadId())
			{
				check(Existing->bHasTempPayload);
				// We are the first GetData call after the BulkData previously loaded its Payload to calculate the RawHash.
				// Sidenote, this means GetCompressedPayload will be fast.
				// But we also have the responsibility to dump the data from memory since we have now consumed it.
				// Make sure we copy the data pointer before dumping it from the Registry version!
				CopyBulk.Emplace(Existing->BulkData);
				Existing->BulkData.UnloadData();
				Existing->bHasTempPayload = false;
				break;
			}

			// The payload in the registry is missing its RawHash, and we calculate that on demand whenever the data is requested,
			// which is now. Instead of only returning the data to our caller, we load the data and use it to update the RawHash
			// in the registry and then return the data to our caller.
			FUpdatingPayload& UpdatingPayload = UpdatingPayloads.FindOrAdd(BulkDataId);
			if (!UpdatingPayload.AsyncTask)
			{
				UpdatingPayload.AsyncTask = new FAutoDeleteAsyncTask<FUpdatePayloadWorker>(this, Existing->BulkData, false /* bKeepTempLoadedPayload */);
			}
			TPromise<UE::BulkDataRegistry::FData> Promise;
			TFuture<UE::BulkDataRegistry::FData> Future = Promise.GetFuture();
			UpdatingPayload.Requesters.Add([Promise=MoveTemp(Promise)](bool bValid, const FCompressedBuffer& Buffer) mutable
				{
					Promise.SetValue(UE::BulkDataRegistry::FData{ Buffer });
				});
			return Future;
		}
	}

	// We are calling a function that returns a TFuture on the stack-local CopyBulk, which would cause a read-after-free if the asynchronous TFuture could
	// read from the BulkData. However, the contract of FEditorBulkData guarantees that the TFuture gets a copy of all data it needs and
	// does not read from the BulkData after returning from GetCompressedPayload, so a read-after-free is not possible.
	return CopyBulk->GetCompressedPayload().Next([](FCompressedBuffer Payload)
		{
			return UE::BulkDataRegistry::FData{ Payload };
		});
}

bool FBulkDataRegistryImpl::TryGetBulkData(const FGuid& BulkDataId, UE::Serialization::FEditorBulkData* OutBulk,
	FName* OutOwner)
{
	FReadScopeLock RegistryScopeLock(RegistryLock);
	FRegisteredBulk* Existing = nullptr;
	if (bActive)
	{
		Existing = Registry.Find(BulkDataId);
	}
	if (Existing)
	{
		if (OutBulk)
		{
			*OutBulk = Existing->BulkData;
		}
		if (OutOwner)
		{
			*OutOwner = Existing->PackageName;
		}
		return true;
	}
	return false;
}

uint64 FBulkDataRegistryImpl::GetBulkDataResaveSize(FName PackageName)
{
	return ResaveSizeTracker.GetBulkDataResaveSize(PackageName);
}

void FBulkDataRegistryImpl::TickCook(float DeltaTime, bool bTickComplete)
{
	bool bWaitForCooldown = !bTickComplete;

	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			return;
		}
		PruneTempLoadedPayloads();
	}
}

void FBulkDataRegistryImpl::Tick(float DeltaTime)
{
	TickCook(DeltaTime, false /* bTickComplete */);
}

void FBulkDataRegistryImpl::AddPendingPackageBulkData(FName PackageName, UE::Serialization::FEditorBulkData&& BulkData,
	bool& bOutCacheOperationsAllowed)
{
	FScopeLock PendingPackageScopeLock(&PendingPackageLock);
	check(bActive); // Callers should early exit if !bActive
	TUniquePtr<FPendingPackage>& PendingPackage = PendingPackages.FindOrAdd(PackageName);
	if (!PendingPackage.IsValid())
	{
		PendingPackage = MakeUnique<FPendingPackage>(PackageName, this);
	}
	if (!PendingPackage->IsLoadInProgress())
	{
		bOutCacheOperationsAllowed = false;
		return;
	}
	bOutCacheOperationsAllowed = PendingPackage->AreCacheOperationsAllowed();
	PendingPackage->AddBulkData(MoveTemp(BulkData));
}

void FBulkDataRegistryImpl::AddTempLoadedPayload(const FGuid& RegistryKey, uint64 PayloadSize)
{
	// Called within RegistryLock WriteLock
	TempLoadedPayloads.Add(FTempLoadedPayload{ RegistryKey, PayloadSize, FPlatformTime::Seconds() + Constants::TempLoadedPayloadsDuration });
	TempLoadedPayloadsSize += PayloadSize;
}

void FBulkDataRegistryImpl::PruneTempLoadedPayloads()
{
	// Called within RegistryLock WriteLock
	if (!TempLoadedPayloads.IsEmpty())
	{
		double CurrentTime = FPlatformTime::Seconds();
		while (!TempLoadedPayloads.IsEmpty() &&
			(TempLoadedPayloadsSize > Constants::TempLoadedPayloadsSizeBudget
				|| TempLoadedPayloads[0].EndTime <= CurrentTime))
		{
			FTempLoadedPayload Payload = TempLoadedPayloads.PopFrontValue();
			FRegisteredBulk* Existing = Registry.Find(Payload.Guid);
			if (Existing)
			{
				// UnloadData only unloads the in-memory data, and only if the BulkData can be reloaded from disk
				Existing->BulkData.UnloadData();
				Existing->bHasTempPayload = false;
			}
			TempLoadedPayloadsSize -= Payload.PayloadSize;
		}
	}
}

void FBulkDataRegistryImpl::WritePayloadIdToCache(FName PackageName, const UE::Serialization::FEditorBulkData& BulkData)
{
	check(!PackageName.IsNone());
	TArray<uint8> Bytes;
	FMemoryWriter Writer(Bytes);
	const_cast<UE::Serialization::FEditorBulkData&>(BulkData).SerializeForRegistry(Writer);
	UE::EditorDomain::PutBulkDataPayloadId(PackageName, BulkData.GetIdentifier(), MakeSharedBufferFromArray(MoveTemp(Bytes)));
}

void FBulkDataRegistryImpl::ReadPayloadIdsFromCache(FName PackageName, TArray<TRefCountPtr<FPendingPayloadId>>&& OldPendings,
	TArray<TRefCountPtr<FPendingPayloadId>>&& NewPendings)
{
	// Cancel any old requests for the Guids in NewPendings; we are about to overwrite them
	// This cancellation has to occur outside of any lock, since the task may be in progress and
	// and to enter the lock and Cancel will wait on it
	for (TRefCountPtr<FPendingPayloadId>& OldPending : OldPendings)
	{
		OldPending->Cancel();
	}
	OldPendings.Empty();
	for (TRefCountPtr<FPendingPayloadId>& NewPending : NewPendings)
	{
		// Creation of the Request has to occur outside of any lock, because the request
		// may execute immediately on this thread and need to enter the lock; our locks are non-reentrant
		UE::DerivedData::FRequestBarrier Barrier(NewPending->GetRequestOwner());
		UE::EditorDomain::GetBulkDataPayloadId(PackageName, NewPending->GetBulkDataId(), NewPending->GetRequestOwner(),
			[this, PackageName, NewPending](FSharedBuffer Buffer)
		{
			const FGuid& BulkDataId = NewPending->GetBulkDataId();
			UE::Serialization::FEditorBulkData CachedBulkData;
			if (!Buffer.IsNull())
			{
				FMemoryReaderView Reader(MakeArrayView(Buffer));
				CachedBulkData.SerializeForRegistry(Reader);
				if (Reader.IsError() || CachedBulkData.GetIdentifier() != BulkDataId)
				{
					UE_LOGF(LogBulkDataRegistry, Warning, "Corrupt cache data for BulkDataPayloadId %ls.", WriteToString<192>(PackageName, TEXT("/"), BulkDataId).ToString());
					CachedBulkData = UE::Serialization::FEditorBulkData();
				}
			}

			TOptional<UE::Serialization::FEditorBulkData> WritePayloadIdBulkData;
			FName WritePayloadIdPackageName;
			{
				FWriteScopeLock RegistryScopeLock(RegistryLock);
				if (!bActive)
				{
					return;
				}
				TRefCountPtr<FPendingPayloadId> ExistingPending;
				if (!PendingPayloadIds.RemoveAndCopyValue(BulkDataId, ExistingPending))
				{
					return;
				}
				check(ExistingPending->GetBulkDataId() == BulkDataId);
				if (ExistingPending != NewPending)
				{
					// We removed ExistingPending because we thought it was equal to NewPending, but it's not, so put it back
					PendingPayloadIds.Add(BulkDataId, MoveTemp(ExistingPending));
					return;
				}

				FRegisteredBulk* ExistingRegisteredBulk = Registry.Find(BulkDataId);
				if (!ExistingRegisteredBulk)
				{
					return;
				}

				UE::Serialization::FEditorBulkData& ExistingBulkData = ExistingRegisteredBulk->BulkData;
				if (CachedBulkData.GetIdentifier().IsValid())
				{
					check(ExistingBulkData.GetIdentifier() == BulkDataId);
					if (ExistingBulkData.HasPlaceholderPayloadId() && CachedBulkData.LocationMatches(ExistingBulkData))
					{
						ExistingBulkData = CachedBulkData;
						// No longer allowed to write the payloadId because we have found it already exists
						ExistingRegisteredBulk->bAllowedToWritePayloadIdToCache = false;
					}
				}
				else if (ExistingRegisteredBulk->bAllowedToWritePayloadIdToCache)
				{
					// We had a missing Get, so calculate the value locally and Put the results
					if (!ExistingBulkData.HasPlaceholderPayloadId())
					{
						// In between the point where we started the cache query and we received this result,
						// The FEditorBulkData has updated its PayloadId and informed us by calling UpdatePlaceholderPayloadId
						// Put this locally computed PayloadId into the cache, since the cache is missing it.

						// Set that it is no longer allowed to write PayloadIdToCache since we are writing it now
						ExistingRegisteredBulk->bAllowedToWritePayloadIdToCache = false;

						// Save the BulkData for writing to the cache outside of the lock
						WritePayloadIdBulkData.Emplace(ExistingBulkData);
						WritePayloadIdPackageName = ExistingRegisteredBulk->PackageName;
					}
					else
					{
						// Create an FUpdatingPayload for the BulkData. Its DoWork function loads the data, sets the
						// PayloadId on the BulkData in this->Registry, and calls WritePayloadIdToCache.
						FUpdatingPayload& UpdatingPayload = UpdatingPayloads.FindOrAdd(BulkDataId);
						if (!UpdatingPayload.AsyncTask)
						{
							UpdatingPayload.AsyncTask = new FAutoDeleteAsyncTask<FUpdatePayloadWorker>(this, ExistingBulkData, false/* bKeepTempLoadedPayload */);
							UpdatingPayload.AsyncTask->StartBackgroundTask();
						}
					}
				}
			}
			if (WritePayloadIdBulkData)
			{
				FBulkDataRegistryImpl::WritePayloadIdToCache(WritePayloadIdPackageName, *WritePayloadIdBulkData);
			}
		});
	}

	// Assign the Requests we just created to the data for each guid in the map,
	// which has to be edited only within the lock
	// If for any reason (race condition, shutting down) we can not assign a Request,
	// we have to cancel the request before returning, to make sure the callback does not hold a pointer
	// to this that could become dangling.
	{
		FWriteScopeLock RegistryScopeLock(RegistryLock);
		if (!bActive)
		{
			for (TRefCountPtr<FPendingPayloadId>& NewPending : NewPendings)
			{
				OldPendings.Add(MoveTemp(NewPending));
			}
		}
		else
		{
			for (TRefCountPtr<FPendingPayloadId>& NewPending : NewPendings)
			{
				TRefCountPtr<FPendingPayloadId>* ExistingPending = PendingPayloadIds.Find(NewPending->GetBulkDataId());
				if (!ExistingPending || *ExistingPending != NewPending)
				{
					OldPendings.Add(MoveTemp(NewPending));
				}
			}
		}
	}
	NewPendings.Empty();
	for (TRefCountPtr<FPendingPayloadId>& OldPending : OldPendings)
	{
		OldPending->Cancel();
	}
	OldPendings.Empty();
}

FPendingPackage::FPendingPackage(FName InPackageName, FBulkDataRegistryImpl* InOwner)
	: PackageName(InPackageName)
	, BulkDataListCacheRequest(UE::DerivedData::EPriority::Low)
	, Owner(InOwner)
{
	PendingOperations = Flag_EndLoad | Flag_BulkDataListResults;

	// The BulkDataList for the package is keyed by the EditorDomain key for the package, which requires
	// data from the AssetRegistry and will block on the AssetRegistry's gather of the package. That blocking makes editor
	// startup too slow, and we want to skip it.
	// Therefore, for packages that load while the AssetRegistry is still scanning, we skip reading their cache value,
	// and never read/write their cache. Building DDC jobs with these packages, if they have any PlaceHolderPayloadIds,
	// will therefore always need to recalculate the real PayloadId by loading the bulk data before the DDC build starts.
	// TODO: Change this to be more nuanced, by waiting for the AssetRegistry to report the package is loaded and then
	// sending off this GetBulkDataList request, and finish off the rest of the package.
	if (!IAssetRegistry::GetChecked().IsGathering())
	{
		UE::EditorDomain::GetBulkDataList(PackageName, BulkDataListCacheRequest,
			[this](FSharedBuffer Buffer) { OnBulkDataListResults(Buffer); });
	}
	else
	{
		bCacheOperationsAllowed = false;
		OnBulkDataListResults(FSharedBuffer());
	}
}

void FPendingPackage::Cancel()
{
	// Called from outside Owner->PendingPackagesLock, so OnBulkDataList can complete on other thread while we wait
	// Called after removing this from Owner->PendingPackages under a previous cover of the lock
	// If OnBulkDataList is running on other thread its attempt to remove from PendingPackages will be a noop
	if (!BulkDataListCacheRequest.Poll())
	{
		// Optimization: prevent WriteCache from running at all if we reach here first
		PendingOperations.fetch_or(Flag_Canceled);
		BulkDataListCacheRequest.Cancel();
	}
}

void FPendingPackage::OnEndLoad(bool& bOutShouldRemove, bool& bOutShouldWriteCache)
{
	// Called from within Owner->PendingPackageLock
	bLoadInProgress = false;
	if (PendingOperations.fetch_and(~Flag_EndLoad) == Flag_EndLoad)
	{
		bOutShouldWriteCache = bCacheOperationsAllowed;
		bOutShouldRemove = true;
	}
	else
	{
		bOutShouldWriteCache = false;
		bOutShouldRemove = false;
	}
}

void FPendingPackage::OnBulkDataListResults(FSharedBuffer Buffer)
{
	if (!Buffer.IsNull())
	{
		FMemoryReaderView Reader(MakeArrayView(Buffer));
		Serialize(Reader, CachedBulkDatas);
		if (Reader.IsError())
		{
			CachedBulkDatas.Empty();
		}
	}

	bool bAbort;
	ReadCache(bAbort);

	if (PendingOperations.fetch_and(~Flag_BulkDataListResults) == Flag_BulkDataListResults)
	{
		// We do not hold a lock when writing the cache, so we need to remove this from PendingPackages 
		// before calling WriteCache to avoid other threads being able to access it.
		TUniquePtr<FPendingPackage> ThisPointer;
		{
			FScopeLock PendingPackageScopeLock(&Owner->PendingPackageLock);
			Owner->PendingPackages.RemoveAndCopyValue(PackageName, ThisPointer);
			// PackageName may have already been removed from PendingPackages if Owner is shutting down. In that case,
			// ThisPointer will be null, and Owner holds the UniquePtr to *this.
			// It will release that UniquePtr only after Cancel returns, which will be after this function returns.
		}
		if (!bAbort)
		{
			WriteCache();
		}

		// Deleting *this will destruct BulkDataListCacheRequest, which by
		// default calls Cancel, but Cancel will block on this callback we are
		// currently in. Direct the owner to keep requests alive to avoid that deadlock.
		BulkDataListCacheRequest.KeepAlive();
		ThisPointer.Reset();
		// *this may have been deleted and can no longer be accessed
	}
}

void FPendingPackage::ReadCache(bool& bOutAbort)
{
	bOutAbort = false;
	if (CachedBulkDatas.Num() == 0)
	{
		FReadScopeLock RegistryScopeLock(Owner->RegistryLock);
		bOutAbort = !Owner->bActive;
		return;
	}

	TArray<TRefCountPtr<FPendingPayloadId>> OldPendings;
	TArray<TRefCountPtr<FPendingPayloadId>> NewPendings;

	// Add each CachedBulkData to the Registry, updating RawHash if it is missing.
	// For every BulkData in this package in the Registry after the CachedBulkData has been added, 
	// if the RawHash is missing from the CachedBulkData as well, queue a read of its RawHash
	// from the separate PlaceholderPayloadId BulkTablePayloadId cache bucket.
	{
		FWriteScopeLock RegistryScopeLock(Owner->RegistryLock);
		if (!Owner->bActive)
		{
			bOutAbort = true;
			return;
		}
		for (const UE::Serialization::FEditorBulkData& BulkData : CachedBulkDatas)
		{
			FGuid BulkDataId = BulkData.GetIdentifier();
			FRegisteredBulk& TargetRegisteredBulk = Owner->Registry.FindOrAdd(BulkData.GetIdentifier());

			bool bCachedLocationMatches = true;
			UE::Serialization::FEditorBulkData& TargetBulkData = TargetRegisteredBulk.BulkData;
			if (!TargetBulkData.GetIdentifier().IsValid())
			{
				TargetBulkData = BulkData;
				TargetRegisteredBulk.PackageName = PackageName;
				TargetRegisteredBulk.bPayloadAvailable = true;
			}
			else
			{
				check(TargetBulkData.GetIdentifier() == BulkDataId);
				bCachedLocationMatches = BulkData.LocationMatches(TargetBulkData);
				if (bCachedLocationMatches && !BulkData.HasPlaceholderPayloadId() && TargetBulkData.HasPlaceholderPayloadId())
				{
					TargetBulkData = BulkData;
					TargetRegisteredBulk.PackageName = PackageName;
					TargetRegisteredBulk.bPayloadAvailable = true;
				}
			}

			if (bCachedLocationMatches && TargetBulkData.HasPlaceholderPayloadId())
			{
				NewPendings.Emplace(new FPendingPayloadId(BulkDataId));
			}
		}

		for (TRefCountPtr<FPendingPayloadId>& NewPending : NewPendings)
		{
			TRefCountPtr<FPendingPayloadId>& ExistingPending = Owner->PendingPayloadIds.FindOrAdd(NewPending->GetBulkDataId());
			if (ExistingPending.IsValid())
			{
				OldPendings.Add(MoveTemp(ExistingPending));
			}
			ExistingPending = NewPending;
		}
	}

	Owner->ReadPayloadIdsFromCache(PackageName, MoveTemp(OldPendings), MoveTemp(NewPendings));
}

void FPendingPackage::WriteCache()
{
	if (CachedBulkDatas.Num() > 0)
	{
		// If the BulkDataList cache read found some existing results, then exit; cache results are deterministic so there
		// is no need to write the list to the cache again.
		return;
	}
	if (!bCacheOperationsAllowed)
	{
		// bCacheOperationsAllowed is set to false for packages that are loaded to early to read from the cache.
		// It might still be too early to write to the cache, but also, we're not allowed to write without
		// first checking whether the value already exists by reading.
		return;
	}

	check(BulkDatas.Num() > 0); // We should have >= 1 bulkdatas, or we would not have created the FPendingPackage
	// Remove any duplicates in the runtime BulkDatas; elements later in the list override earlier elements
	{
		TSet<FGuid> BulkDataGuids;
		for (int32 Index = BulkDatas.Num() - 1; Index >= 0; --Index)
		{
			bool bAlreadyExists;
			BulkDataGuids.Add(BulkDatas[Index].GetIdentifier(), &bAlreadyExists);
			if (bAlreadyExists)
			{
				BulkDatas.RemoveAt(Index);
			}
		}
	}

	// Sort the list by guid, to avoid indeterminism in the list
	BulkDatas.Sort([](const UE::Serialization::FEditorBulkData& A, const UE::Serialization::FEditorBulkData& B)
		{
			return A.GetIdentifier() < B.GetIdentifier();
		});

	TArray<uint8> Bytes;
	FMemoryWriter Writer(Bytes);
	Serialize(Writer, BulkDatas);
	UE::EditorDomain::PutBulkDataList(PackageName, MakeSharedBufferFromArray(MoveTemp(Bytes)));
}

FUpdatePayloadWorker::FUpdatePayloadWorker(FBulkDataRegistryImpl* InBulkDataRegistry,
	const UE::Serialization::FEditorBulkData& InSourceBulk, bool bInKeepTempLoadedPayload)
	: BulkData(InSourceBulk)
	, BulkDataRegistry(InBulkDataRegistry)
	, bKeepTempLoadedPayload(bInKeepTempLoadedPayload)
{
	SharedDataLock = InBulkDataRegistry->SharedDataLock;
}

void FUpdatePayloadWorker::DoWork()
{
	FUpdatingPayload LocalUpdatingPayload;
	FCompressedBuffer Buffer;
	bool bValid = true;
	for (;;)
	{
		{
			FReadScopeLock SharedDataScopeLock(SharedDataLock->ActiveLock);
			if (!SharedDataLock->bActive)
			{
				// The BulkDataRegistry has destructed. Our list of requesters is on the BulkDataRegistry, so there's nothing we can do except exit
				return;
			}

			SharedDataLock->UpdatePayloadWorkerInFlight++;
		}

		BulkData.UpdatePayloadId();
		Buffer = BulkData.GetCompressedPayload().Get();

		TOptional<UE::Serialization::FEditorBulkData> WritePayloadIdBulkData;
		FName WritePayloadIdPackageName;
		{
			FReadScopeLock SharedDataScopeLock(SharedDataLock->ActiveLock);
			--SharedDataLock->UpdatePayloadWorkerInFlight;

			if (!SharedDataLock->bActive)
			{
				// The BulkDataRegistry has destructed. Our list of requesters is on the BulkDataRegistry, so there's nothing we can do except exit
				return;
			}
			FWriteScopeLock RegistryScopeLock(BulkDataRegistry->RegistryLock);

			if (!BulkDataRegistry->UpdatingPayloads.RemoveAndCopyValue(BulkData.GetIdentifier(), LocalUpdatingPayload))
			{
				// The updating payload might not exist in the case of the Registry shutting down; it will clear the UpdatingPayloads to cancel our action
				// Return canceled (which we treat the same as failed) to our requesters
				bValid = false;
				break;
			}
			check(BulkDataRegistry->bActive); // Only set to false at the same time as SharedDataLock->bActive

			FRegisteredBulk* RegisteredBulk = BulkDataRegistry->Registry.Find(BulkData.GetIdentifier());
			if (!RegisteredBulk)
			{
				// Some agent has deregistered the BulkData before we finished calculating its payload
				// return failure to our requesters
				bValid = false;
				break;
			}

			if (!BulkData.LocationMatches(RegisteredBulk->BulkData))
			{
				// Some caller has assigned a new BulkData. We need to abandon the BulkData we just loaded and give our callers the
				// information about the new one
				check(RegisteredBulk->BulkData.GetIdentifier() == BulkData.GetIdentifier()); // The identifier in the BulkData should match the key for that BulkData in Registry
				BulkData = RegisteredBulk->BulkData;
				// Add our LocalUpdatingPayload back to UpdatingPayloads; we removed it because we thought we were done.
				BulkDataRegistry->UpdatingPayloads.Add(BulkData.GetIdentifier(), MoveTemp(LocalUpdatingPayload));
				continue;
			}

			// Store the new PayloadId in the Registry's entry for the BulkData; new MetaData requests will no longer need to wait for it
			RegisteredBulk->BulkData = BulkData;

			// The BulkData also has the payload still loaded; keep it or remove it
			if (bKeepTempLoadedPayload)
			{
				// Keep the payload, and mark that the next GetData call should remove it
				RegisteredBulk->bHasTempPayload = true;
				BulkDataRegistry->AddTempLoadedPayload(BulkData.GetIdentifier(), BulkData.GetPayloadSize());
				BulkDataRegistry->PruneTempLoadedPayloads();
			}
			else
			{
				RegisteredBulk->BulkData.UnloadData();
			}

			if (RegisteredBulk->bAllowedToWritePayloadIdToCache)
			{
				// Set that it is no longer allowed to write PayloadIdToCache since we are writing it now
				RegisteredBulk->bAllowedToWritePayloadIdToCache = false;

				// Save the BulkData for writing to the cache outside of the lock
				WritePayloadIdBulkData.Emplace(BulkData);
				WritePayloadIdPackageName = RegisteredBulk->PackageName;
			}
		}
		if (WritePayloadIdBulkData)
		{
			FBulkDataRegistryImpl::WritePayloadIdToCache(WritePayloadIdPackageName, *WritePayloadIdBulkData);
		}
		break;
	}

	if (!bValid)
	{
		Buffer.Reset();
	}
	for (TUniqueFunction<void(bool, const FCompressedBuffer&)>& Requester : LocalUpdatingPayload.Requesters)
	{
		Requester(bValid, Buffer);
	}
}

FPendingPayloadId::FPendingPayloadId(const FGuid& InBulkDataId)
	: BulkDataId(InBulkDataId)
	, Request(UE::DerivedData::EPriority::Low)
{
	// The last reference to this can be released by the completion callback, which will deadlock
	// trying to cancel the request. KeepAlive skips cancellation in the destructor.
	Request.KeepAlive();
}

void FPendingPayloadId::Cancel()
{
	Request.Cancel();
}

void Serialize(FArchive& Ar, TArray<UE::Serialization::FEditorBulkData>& Datas)
{
	int32 Num = Datas.Num();
	Ar << Num;

	const uint32 MinSize = 4;
	if (Ar.IsLoading())
	{
		if (Ar.IsError() || Num * MinSize > Ar.TotalSize() - Ar.Tell())
		{
			Ar.SetError();
			Datas.Empty();
			return;
		}
		else
		{
			Datas.Empty(Num);
		}
		for (int32 n = 0; n < Num; ++n)
		{
			UE::Serialization::FEditorBulkData& BulkData = Datas.Emplace_GetRef();
			BulkData.SerializeForRegistry(Ar);
		}
	}
	else
	{
		for (UE::Serialization::FEditorBulkData& BulkData : Datas)
		{
			BulkData.SerializeForRegistry(Ar);
		}
	}
}

} // namespace UE::BulkDataRegistry::Private
