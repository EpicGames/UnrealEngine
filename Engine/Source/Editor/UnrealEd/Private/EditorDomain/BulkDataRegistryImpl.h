// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "DerivedDataRequestOwner.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Serialization/BulkDataRegistry.h"
#include "Serialization/EditorBulkData.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FArchive;
class FCompressedBuffer;
class FSharedBuffer;
class UPackage;
struct FEndLoadPackageContext;

namespace UE::BulkDataRegistry::Private
{

class FBulkDataRegistryImpl;

/** Struct for storage of a BulkData in the registry, including the BulkData itself and data about cache status. */
struct FRegisteredBulk
{
	FRegisteredBulk()
		: bHasTempPayload(false), bAllowedToWritePayloadIdToCache(false), bRegistered(false), bInMemory(false), bPayloadAvailable(false)
	{
	}
	UE::Serialization::FEditorBulkData BulkData;
	FName PackageName;
	/**
	 * True if this->BulkData has loaded its payload (because we wanted to update its PlaceholderPayloadId) and we kept
	 * the payload in memory because we want to use it on the GetData call that we expect to come soon.
	 */
	bool bHasTempPayload : 1;
	/**
	 * For legacy bulkdatas with a PlaceholderPayloadId, and we don't know the actual PayloadId, we will load their
	 * bulkdata and calculate the actual PayloadId when it is requested by clients such as the build system. Loading
	 * the bulkdata to calculate this is expensive so we cache the actual PayloadId in DDC.
	 * However, this caching is only valid for BulkDatas that are discovered during package loading and not modified
	 * after. bAllowedToWritePayloadIdToCache is true for those bulkdatas and is set to false if they are created too
	 * late or are modified after loading. Iff this flag is true when we load the bulkdata to calculate the PayloadId,
	 * we write the actual PayloadId to DDC. We also set this flag to false once we know it is already in the cache.
	 */
	bool bAllowedToWritePayloadIdToCache : 1;
	/**
	 * True if the key associated with this FRegisteredBulk value has been registered by an FEditorBulkData outside
	 * of the registry. Non-registered FRegisteredBulks can be created when we read the list of BulkDatas in the
	 * package from DDC and get the results back before the FEditorBulkDatas are discovered during package loading.
	 */
	bool bRegistered : 1;
	/**
	 * True iff an FEditorBulkData outside of the registry has registered the key associated with this FRegisteredBulk,
	 * AND that FEditorBulkData has not yet called OnExitMemory to indicate its destruction.
	 */
	bool bInMemory : 1;
	/**
	 * If true, then we can load the data for the BulkData to satisfy GetData requests. If false, this->BulkData is
	 * invalid and we don't know what the data is. This can happen when a memory-only FEditorBulkData calls
	 * OnExitMemory. We keep FRegisteredBulk around in that case solely to detect duplicate Guid bugs.
	 */
	bool bPayloadAvailable : 1;
};

/** Serialize an array of BulkDatas into or out of bytes saved/load from the registry's persistent cache. */
void Serialize(FArchive& Ar, TArray<UE::Serialization::FEditorBulkData>& InDatas);

/** A collection of bulkdatas that should be sent to the cache for the given package. */
class FPendingPackage
{
public:
	FPendingPackage(FName PackageName, FBulkDataRegistryImpl* InOwner);
	FPendingPackage(FPendingPackage&& Other) = delete;
	FPendingPackage(const FPendingPackage& Other) = delete;

	void Cancel();
	void AddBulkData(UE::Serialization::FEditorBulkData&& BulkData)
	{
		BulkDatas.Add(MoveTemp(BulkData));
	}
	void OnEndLoad(bool& bOutShouldRemove, bool& bOutShouldWriteCache);
	bool IsLoadInProgress() const
	{
		return bLoadInProgress;
	}
	void WriteCache();

	bool AreCacheOperationsAllowed() const
	{
		return bCacheOperationsAllowed;
	}

private:
	void OnBulkDataListResults(FSharedBuffer Buffer);
	void ReadCache(bool& bOutAbort);

	enum EFlags : int32
	{
		Flag_EndLoad = 1 << 0,
		Flag_BulkDataListResults = 1 << 1,
		Flag_Canceled = 1 << 2,
	};
	FName PackageName;
	TArray<UE::Serialization::FEditorBulkData> BulkDatas;
	TArray<UE::Serialization::FEditorBulkData> CachedBulkDatas;
	UE::DerivedData::FRequestOwner BulkDataListCacheRequest;
	FBulkDataRegistryImpl* Owner;
	/**
	 * When PendingOperations reaches zero, we can remove the FPendingPackage.
	 */
	std::atomic<int32> PendingOperations;
	/**
	 * True until the LoadPackage for *this is complete. The FPendingPackage may last longer
	 * than the initial load period while it waits for a CacheRead, but BulkDatas can only
	 * be written to its list during its initial load, to avoid non-deterministic changes
	 * to the list when operations occur on the UPackage in the editor.
	 */
	bool bLoadInProgress = true;
	/**
	 * This value is false for FPendingPackages that were not able to query the cache (e.g. because
	 * they were created too early in EngineStartup) and therefore are not allowed to write
	 * the cache either.
	 */
	bool bCacheOperationsAllowed = true;
};

/** Data about a BulkData that has loaded its payload for TryGetMeta and should drop it after GetData or a timeout. */
struct FTempLoadedPayload
{
	FGuid Guid;
	uint64 PayloadSize;
	double EndTime;
};

/** An Active flag and a lock around it for informing AutoDeleteAsyncTasks that their shared data is no longer available. */
class FTaskSharedDataLock : public FRefCountedObject
{
public:
	FRWLock ActiveLock;
	bool bActive = true;

	std::atomic<uint32> UpdatePayloadWorkerInFlight = 0;
};

/** A worker that updates the PayloadId for a BulkData that is missing its RawHash. */
class FUpdatePayloadWorker : public FNonAbandonableTask
{
public:
	FUpdatePayloadWorker(FBulkDataRegistryImpl* InBulkDataRegistry,
		const UE::Serialization::FEditorBulkData& InSourceBulk, bool bInKeepTempLoadedPayload);

	void DoWork();
	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FUpdatePayloadWorker, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	UE::Serialization::FEditorBulkData BulkData;
	TRefCountPtr<FTaskSharedDataLock> SharedDataLock;
	FBulkDataRegistryImpl* BulkDataRegistry;
	bool bKeepTempLoadedPayload;
};

/** Data storage for the FUpdatePayloadWorker that is updated while in flight for additional requesters. */
struct FUpdatingPayload
{
	/** Pointer to the task. Once set to non-null, is never modified. The task ensure this FUpdatingPayload is destroyed before the task destructs. */
	FAutoDeleteAsyncTask<FUpdatePayloadWorker>* AsyncTask = nullptr;
	TArray<TUniqueFunction<void(bool bValid, const FCompressedBuffer& Buffer)>> Requesters;
};

/** Data storage for a BulkData that is loading its PayloadId from the cache. */
class FPendingPayloadId : public FRefCountedObject
{
public:
	explicit FPendingPayloadId(const FGuid& InBulkDataId);

	FPendingPayloadId(FPendingPayloadId&& Other) = delete;
	FPendingPayloadId(const FPendingPayloadId& Other) = delete;

	void Cancel();

	UE::DerivedData::FRequestOwner& GetRequestOwner()
	{
		return Request;
	}
	const FGuid& GetBulkDataId()
	{
		return BulkDataId;
	}

private:
	FGuid BulkDataId;
	UE::DerivedData::FRequestOwner Request;
};

/** Implementation of a BulkDataRegistry that stores its persistent data in a DDC bucket. */
class FBulkDataRegistryImpl : public IBulkDataRegistry, public FTickableEditorObject, public FTickableCookObject
{
public:
	FBulkDataRegistryImpl();
	virtual ~FBulkDataRegistryImpl();

	// IBulkDataRegistry interface
	virtual UE::BulkDataRegistry::ERegisterResult
		TryRegister(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override;
	virtual void UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override;
	virtual void Unregister(const UE::Serialization::FEditorBulkData& BulkData) override;
	virtual void OnExitMemory(const UE::Serialization::FEditorBulkData& BulkData) override;
	virtual void UpdatePlaceholderPayloadId(const UE::Serialization::FEditorBulkData& BulkData) override;
	virtual TFuture<UE::BulkDataRegistry::FMetaData> GetMeta(const FGuid& BulkDataId) override;
	virtual TFuture<UE::BulkDataRegistry::FData> GetData(const FGuid& BulkDataId) override;
	virtual bool TryGetBulkData(const FGuid& BulkDataId, UE::Serialization::FEditorBulkData* OutBulk = nullptr,
		FName* OutOwner = nullptr) override;

	virtual uint64 GetBulkDataResaveSize(FName PackageName) override;

	// FTickableEditorObject/FTickableCookObject interface
	virtual void Tick(float DeltaTime) override;
	virtual void TickCook(float DeltaTime, bool bTickComplete) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override { return TStatId(); }

private:
	void AddPendingPackageBulkData(FName PackageName, UE::Serialization::FEditorBulkData&& BulkData,
		bool& bOutCacheOperationsAllowed);
	void PollPendingPackages(bool bWaitForCooldown);
	void AddTempLoadedPayload(const FGuid& RegistryKey, uint64 PayloadSize);
	void PruneTempLoadedPayloads();
	void OnEndLoadPackage(const FEndLoadPackageContext& Context);
	static void WritePayloadIdToCache(FName PackageName, const UE::Serialization::FEditorBulkData& BulkData);
	void ReadPayloadIdsFromCache(FName PackageName, TArray<TRefCountPtr<FPendingPayloadId>>&& OldPendings,
		TArray<TRefCountPtr<FPendingPayloadId>>&& NewPendings);
	void OnEnginePreExit();
	void Teardown();

	friend class FPendingPackage;
	friend class FUpdatePayloadWorker;

	// All locks can be held at the same time. They must always be entered in order: SharedDataLock, RegistryLock, PendingPackageLock
	TRefCountPtr<FTaskSharedDataLock> SharedDataLock;
	FRWLock RegistryLock;
	FCriticalSection PendingPackageLock;

	TMap<FGuid, FRegisteredBulk> Registry;
	FResaveSizeTracker ResaveSizeTracker;
	TMap<FGuid, FUpdatingPayload> UpdatingPayloads;
	/**
	 * All of the PendingPackage structs for packages that are still loading or have pending Cache Gets or Puts that
	 * were launched when they started loading. PendingPackages are deleted when the package is done loading and the
	 * cache operations are complete. @see FPendingPackage. These structs manage the DDC operations, both to list
	 * the bulkdatas present in a package and to store the actual PayloadId for legacy BulkDatas with Placeholder
	 * PayloadIds.
	 */
	TMap<FName, TUniquePtr<FPendingPackage>> PendingPackages;
	/** All of the BulkDataIds for which we have a PayloadID cache Get in progress. */
	TMap<FGuid, TRefCountPtr<FPendingPayloadId>> PendingPayloadIds;
	TRingBuffer<FTempLoadedPayload> TempLoadedPayloads;
	uint64 TempLoadedPayloadsSize = 0;
	bool bActive = true;
};

} // namespace UE::BulkDataRegistry::Private
