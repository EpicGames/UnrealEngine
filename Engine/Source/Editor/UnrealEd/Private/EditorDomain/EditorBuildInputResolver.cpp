// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorBuildInputResolver.h"

#include "Async/Future.h"
#include <atomic>
#include "Containers/ArrayView.h"
#include "Containers/SharedString.h"
#include "DerivedDataBuildAction.h"
#include "DerivedDataRequest.h"
#include "DerivedDataRequestOwner.h"
#include "HAL/Event.h"
#include "Misc/StringBuilder.h"
#include "Serialization/BulkDataRegistry.h"
#include "Templates/Tuple.h"

namespace UE::DerivedData
{

/**
 * Takes a collection of TFutures that calculate either ResolveInputMeta or ResolveInputData callback outputs,
 * and calls the Resolve callback after all the futures complete
 */
template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
class TResolveInputRequest final : public FRequestBase
{
public:
	typedef TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType> ThisType;

	TResolveInputRequest(const FBuildDefinition& InDefinition, IRequestOwner& InOwner, CallbackType&& InOnResolved);

	// IRequest interface
	virtual void SetPriority(EPriority Priority) override
	{
	}
	virtual void Cancel() override;
	virtual void Wait() override;

	/** Stores the full set of Payload Futures and completes the creation of this IRequest */
	void SetPayloads(TArrayView<TTuple<FUtf8StringView, TFuture<BulkPayloadType>>> BulkPayloads, EStatus OtherStatus);

private:
	/** Call the OnResolve function if it has not been canceled. */
	void CallOnResolved();

	/**
	 * Move BulkData's fields from the BulkDataRegistry's format into the BuildInputResolver format.
	 * Implementation varies depending on whether the template type is for MetaData or Data.
	*/
	void AssignBulkPayload(CallbackPayloadType& OutPayload, bool& bOutPayloadIsValid, BulkPayloadType& InPayload);
	/**
	 * Log that a BulkData has failed to resolve.
	 * The text of the message varies depending on whether the template type is for MetaData or Data.
	*/
	void LogBulkDataError(FUtf8StringView Key);

	/** Holds the key and TFuture handle for one of the Inputs being resolved. */
	struct FPayloadData
	{
		FUtf8StringView Key;
		TFuture<void> Future;
	};

private:
	/**
	 * The callback that was passed into the IBuildInputResolver function; called when all TFutures are complete.
	 * Can only be read/written after construction while holding the CancelLock.
	 */
	CallbackType OnResolved;

	// Data written during creation that is read-only during Cancel and CallOnResolved
	FBuildDefinition Definition;
	IRequestOwner& Owner;

	/**
	 * The array of data that is written by the TFutures and passed into the IBuildInputResolver callback function.
	 * This array is allocated during creation before any Futures are created.
	 * The elements are write-only, and only by the given Future, until RemainingInputCount is 0.
	 * After RemainingInputCount reaches 0, it is read/writable by whichever thread received the 0.
	 */
	TArray<CallbackPayloadType> Payloads;
	/**
	 * Extra data needed by this class about each of the payloads.
	 * This array is allocated during creation and is read-only afterwards.
	 * It is readable only by the public api or by any thread after RemainingInputCount reaches 0.
	 */
	TArray<FPayloadData> PayloadDatas;

	/** Event for when the callback has finished executing; some functions must not return until it is done. */
	FEventRef CallbackComplete;
	/**
	 * Tracks whether all TFutures are complete; the last one to complete calls the Resolve callback.
	 * Value starts at 1 and is decremented when creation completes, so that if all TFutures are already complete
	 * during creation, the Resolve callback will be called when creation completes instead.
	 */
	std::atomic<int32> RemainingInputCount{ 1 };
	/** Accumulated success result that is set to false if any of the TFutures fail. */
	std::atomic<bool> bAllSucceeded{ true };
	/** Atomic to allow Cancel or CallOnResolved to resolve the race to see which one will call the callback. */
	std::atomic<bool> bCallbackCalled{ false };
#if DO_CHECK
	bool bCreationComplete = false;
#endif

};
typedef TResolveInputRequest<UE::BulkDataRegistry::FMetaData, FBuildInputMetaByKey, FOnBuildInputMetaResolved> FResolveInputMetaRequest;
typedef TResolveInputRequest<UE::BulkDataRegistry::FData, FBuildInputDataByKey, FOnBuildInputDataResolved> FResolveInputDataRequest;


FEditorBuildInputResolver& FEditorBuildInputResolver::Get()
{
	static FEditorBuildInputResolver Singleton;
	return Singleton;
}

void FEditorBuildInputResolver::ResolveInputMeta(const FBuildDefinition& Definition, IRequestOwner& Owner,
	FOnBuildInputMetaResolved&& OnResolved)
{
	EStatus OtherStatus = EStatus::Ok;
	Definition.IterateInputBuilds([&OtherStatus, &Definition](FUtf8StringView Key, const FBuildValueKey& ValueKey)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: Resolving input builds is not yet implemented. "
				"Failed to resolve input build '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	/** Visits every input bulk data in order by key. */
	TArray<TTuple<FUtf8StringView, TFuture<UE::BulkDataRegistry::FMetaData>>, TInlineAllocator<4>> BulkPayloads;
	Definition.IterateInputBulkData([&BulkPayloads](FUtf8StringView Key, const FGuid& BulkDataId)
		{
			BulkPayloads.Emplace(Key, IBulkDataRegistry::Get().GetMeta(BulkDataId));
		});

	Definition.IterateInputFiles([&OtherStatus, &Definition](FUtf8StringView Key, FUtf8StringView Path)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: Resolving input files is not yet implemented. "
				"Failed to resolve input file '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	Definition.IterateInputHashes([&OtherStatus, &Definition](FUtf8StringView Key, const FIoHash& RawHash)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: resolving input hashes is not yet implemented. "
				"Failed to resolve input hash '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	TRefCountPtr Request(new FResolveInputMetaRequest(Definition, Owner, MoveTemp(OnResolved)));
	Request->SetPayloads(BulkPayloads, OtherStatus);
}

void FEditorBuildInputResolver::ResolveInputData(const FBuildDefinition& Definition, IRequestOwner& Owner,
	FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter)
{
	EStatus OtherStatus = EStatus::Ok;
	Definition.IterateInputBuilds([&OtherStatus, &Definition](FUtf8StringView Key, const FBuildValueKey& PayloadKey)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: Resolving input builds is not yet implemented. "
				"Failed to resolve input build '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	/** Visits every input bulk data in order by key. */
	TArray<TTuple<FUtf8StringView, TFuture<UE::BulkDataRegistry::FData>>, TInlineAllocator<4>> BulkPayloads;
	Definition.IterateInputBulkData([&BulkPayloads, &Filter](FUtf8StringView Key, const FGuid& BulkDataId)
		{
			if (!Filter || Filter(Key))
			{
				BulkPayloads.Emplace(Key, IBulkDataRegistry::Get().GetData(BulkDataId));
			}
		});

	Definition.IterateInputFiles([&OtherStatus, &Definition](FUtf8StringView Key, FUtf8StringView Path)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: Resolving input files is not yet implemented. "
				"Failed to resolve input build '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	Definition.IterateInputHashes([&OtherStatus, &Definition](FUtf8StringView Key, const FIoHash& RawHash)
		{
			UE_LOGF(LogCore, Error, "FEditorBuildInputResolver: Resolving input hashes is not yet implemented. "
				"Failed to resolve input build '%ls' for build of '%ls' by %ls.",
				*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
			OtherStatus = EStatus::Error;
		});

	TRefCountPtr Request(new FResolveInputDataRequest(Definition, Owner, MoveTemp(OnResolved)));
	Request->SetPayloads(BulkPayloads, OtherStatus);
}

void FEditorBuildInputResolver::ResolveInputData(const FBuildAction& Action, IRequestOwner& Owner,
	FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter)
{
	UE_LOGF(LogCore, Error, "FEditorBuildInputResolver does not implement ResolveInputData from FBuildAction. "
		"Failed to resolve input data for build of '%ls' by %ls.",
		*Action.GetName(), *WriteToString<32>(Action.GetFunction()));
	OnResolved({{}, EStatus::Error});
}

template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType>::TResolveInputRequest(
	const FBuildDefinition& InDefinition, IRequestOwner& InOwner, CallbackType&& InOnResolved)
	: OnResolved(MoveTemp(InOnResolved))
	, Definition(InDefinition)
	, Owner(InOwner)
	, CallbackComplete(EEventMode::ManualReset)
{
}

template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
void TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType>::Cancel()
{
#if DO_CHECK
	check(bCreationComplete);
#endif
	if (bCallbackCalled.exchange(true, std::memory_order_relaxed) == false)
	{
		Owner.End(this, [this]
		{
			OnResolved({{}, EStatus::Canceled});
			CallbackComplete->Trigger();
		});
	}
	else
	{
		CallbackComplete->Wait();
	}
}

template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
void TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType>::Wait()
{
#if DO_CHECK
	check(bCreationComplete);
#endif
	if (!bCallbackCalled.load(std::memory_order_relaxed))
	{
		for (FPayloadData& PayloadData : PayloadDatas)
		{
			PayloadData.Future.Wait();
		}
	}
	CallbackComplete->Wait();
}

template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
void TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType>::SetPayloads(
	TArrayView<TTuple<FUtf8StringView, TFuture<BulkPayloadType>>> InBulkPayloads, EStatus OtherStatus)
{
	RemainingInputCount.fetch_add(InBulkPayloads.Num(), std::memory_order_relaxed);

	// Preallocate the Payloads array so that it will not be reallocated while we are adding futures to it
	// This allows us to have each future write to its assigned element, potentially from another thread,
	// without the need for synchronization.
	PayloadDatas.Empty(InBulkPayloads.Num());
	Payloads.Empty(InBulkPayloads.Num());
	for (TTuple<FUtf8StringView, TFuture<BulkPayloadType>>& InPair : InBulkPayloads)
	{
		// Initialize the elements in Payloads and PayloadDatas before creating the Future that writes to the entry.
		FPayloadData& PayloadData = PayloadDatas.Emplace_GetRef();
		CallbackPayloadType& Payload = Payloads.Emplace_GetRef();
		PayloadData.Key = MoveTemp(InPair.template Get<FUtf8StringView>());
		// Payload.Key is a view of the key from the definition or action; this is safe because it will outlive the request.
		Payload.Key = PayloadData.Key;

		// The callback we define may be called and write the Payload element before we finish assigning the Future
		// and it may do so either from this thread or another thread.
		// The lambda has a TRefCountPtr to *this; if the IBuildInputResolver drops its reference to *this
		// before the final TFuture completes, *this will be deleted when the lambda completes.
		PayloadData.Future = InPair.template Get<TFuture<BulkPayloadType>>().Next(
			[pThis = TRefCountPtr<ThisType>(this), &Payload](BulkPayloadType&& Result)
			{
				bool bValid;
				pThis->AssignBulkPayload(Payload, bValid, Result);
				if (!bValid)
				{
					pThis->LogBulkDataError(Payload.Key);
					pThis->bAllSucceeded.store(false, std::memory_order_relaxed);
				}
				if (pThis->RemainingInputCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
				{
					pThis->CallOnResolved();
				}
			});
	}

#if DO_CHECK
	check(!bCreationComplete);
	bCreationComplete = true;
#endif
	const EPriority Priority = Owner.GetPriority();
	if (Priority == EPriority::Blocking)
	{
		for (FPayloadData& PayloadData : PayloadDatas)
		{
			PayloadData.Future.Wait();
		}
	}

	if (Priority == EPriority::Blocking || RemainingInputCount.load(std::memory_order_acquire) == 1)
	{
		const EStatus Status = bAllSucceeded.load(std::memory_order_relaxed) ? EStatus::Ok : EStatus::Error;
		OnResolved({Payloads, Status});
		CallbackComplete->Trigger();
	}
	else
	{
		Owner.Begin(this);
		if (RemainingInputCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
		{
			CallOnResolved();
		}
	}
}

template <class BulkPayloadType, class CallbackPayloadType, class CallbackType>
void TResolveInputRequest<BulkPayloadType, CallbackPayloadType, CallbackType>::CallOnResolved()
{
	if (bCallbackCalled.exchange(true, std::memory_order_relaxed) == false)
	{
		Owner.End(this, [this]
		{
			const EStatus Status = bAllSucceeded.load(std::memory_order_relaxed) ? EStatus::Ok : EStatus::Error;
			OnResolved({Payloads, Status});
			CallbackComplete->Trigger();
		});
	}
}

template<>
void FResolveInputMetaRequest::AssignBulkPayload(FBuildInputMetaByKey& OutPayload, bool& bOutPayloadIsValid, UE::BulkDataRegistry::FMetaData& InPayload)
{
	OutPayload.RawHash = InPayload.RawHash;
	OutPayload.RawSize = InPayload.RawSize;
	bOutPayloadIsValid = !OutPayload.RawHash.IsZero();
}

template<>
void FResolveInputDataRequest::AssignBulkPayload(FBuildInputDataByKey& OutPayload, bool& bOutPayloadIsValid, UE::BulkDataRegistry::FData& InPayload)
{
	OutPayload.Data = MoveTemp(InPayload.Buffer);
	bOutPayloadIsValid = !OutPayload.Data.IsNull();
}

template<>
void FResolveInputMetaRequest::LogBulkDataError(FUtf8StringView Key)
{
	UE_LOGF(LogCore, Error, "Failed to resolve metadata for bulk data input '%ls' for build of '%ls' by %ls.",
		*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
}

template<>
void FResolveInputDataRequest::LogBulkDataError(FUtf8StringView Key)
{
	UE_LOGF(LogCore, Error, "Failed to resolve data for bulk data input '%ls' for build of '%ls' by %ls.",
		*WriteToString<32>(Key), *Definition.GetName(), *WriteToString<32>(Definition.GetFunction()));
}

} // namespace UE::DerivedData
