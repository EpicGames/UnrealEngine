// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DerivedDataBuildInputResolver.h"

#if WITH_EDITOR

namespace UE::DerivedData
{

/** A BuildInputResolver that looks up the process-global inputs registered from package loads or a cache of the package loads. */
class FEditorBuildInputResolver : public IBuildInputResolver
{
public:
	UNREALED_API static FEditorBuildInputResolver& Get();

	virtual void ResolveInputMeta(const FBuildDefinition& Definition, IRequestOwner& Owner,
		FOnBuildInputMetaResolved&& OnResolved) override;
	virtual void ResolveInputData(const FBuildDefinition& Definition, IRequestOwner& Owner,
		FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter = FBuildInputFilter()) override;
	virtual void ResolveInputData(const FBuildAction& Action, IRequestOwner& Owner,
		FOnBuildInputDataResolved&& OnResolved, FBuildInputFilter&& Filter = FBuildInputFilter()) override;
};

} // namespace UE::DerivedData

#endif // WITH_EDITOR