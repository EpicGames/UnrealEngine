// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDomain/EditorDomain.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/AsyncFileHandle.h"
#include "Containers/ContainersFwd.h"
#include "Containers/SharedString.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DerivedDataCache.h"
#include "DerivedDataCachePolicy.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataRequestTypes.h"
#include "EditorDomain/EditorDomainArchive.h"
#include "EditorDomain/EditorDomainSave.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/LogCategory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "ModuleDescriptor.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "Templates/RefCounting.h"
#include "Templates/RemoveReference.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManagerFile.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

class IMappedFileHandle;

DEFINE_LOG_CATEGORY(LogEditorDomain);

/**
 * A pre-Main global registration struct that decides whether to create the EditorDomain, and initializes EditorDomainUtils
 * for usage by any system that needs PackageDigests, with or without FEditorDomain.
 * If the EditorDomain is enabled, also sets PackageResourceManager to use the EditorDomain as the IPackageResourceManager
 */
class FEditorDomainConstructor
{
public:
	FEditorDomainConstructor()
	{
		IPackageResourceManager::GetSetPackageResourceManagerDelegate().BindStatic(SetPackageResourceManager);
		IPackageResourceManager::GetOnClearPackageResourceManagerDelegate().AddStatic(OnClearPackageResourceManager);
	}

	static void ConditionalConstruct()
	{
		ConditionalConstruct(IsEditorDomainEnabled());
	}

	static void ConditionalConstruct(EEditorDomainEnabled Enabled)
	{
		check(FEditorDomain::SingletonEditorDomain == nullptr);
		if ((uint32)Enabled >= (uint32)EEditorDomainEnabled::Utilities)
		{
			UE::EditorDomain::UtilsInitialize();
			FEditorDomain::SingletonEditorDomain = new FEditorDomain(Enabled);
			UE::EditorDomain::IPackageDigestCache::Set(FEditorDomain::SingletonEditorDomain);
		}
		else
		{
			UE::EditorDomain::IPackageDigestCache::SetDefault();
		}
	}

	static IPackageResourceManager* SetPackageResourceManager()
	{
		EEditorDomainEnabled Enabled = IsEditorDomainEnabled();
		ConditionalConstruct(Enabled);

		if (GIsEditor)
		{
			UE_LOGF(LogEditorDomain, Display, "EditorDomain is %ls",
				(uint32)Enabled >= (uint32)EEditorDomainEnabled::PackageResourceManager ? TEXT("Enabled") : TEXT("Disabled"));
		}
		if ((uint32)Enabled >= (uint32)EEditorDomainEnabled::PackageResourceManager)
		{
			check(FEditorDomain::SingletonEditorDomain); // Should have been constructed by ConditionalConstruct
			// Set values for config settings EditorDomain read/writes depends on
			GAllowUnversionedContentInEditor = 1;
			return FEditorDomain::SingletonEditorDomain;
		}
		else
		{
			return nullptr;
		}
	}

	static void OnClearPackageResourceManager()
	{
		// SetPackageResourceManager may have already deleted the singleton, but in the case
		// where we don't register, we need to delete it ourselves.
		delete FEditorDomain::SingletonEditorDomain;
		// Destructor already sets it to null, but set it to null again to confirm and to avoid C6001: Using uninitialized memory 
		FEditorDomain::SingletonEditorDomain = nullptr;
	}
} GRegisterAsPackageResourceManager;

FEditorDomain* FEditorDomain::SingletonEditorDomain = nullptr;

FEditorDomain::FEditorDomain(EEditorDomainEnabled EnableLevel)
{
	Locks = TRefCountPtr<FLocks>(new FLocks(*this));
	Workspace.Reset(MakePackageResourceManagerFile());
	GConfig->GetBool(TEXT("CookSettings"), TEXT("EditorDomainExternalSave"), bExternalSave, GEditorIni);
	// Load / Save from EditorDomain before AssetRegistry has finished its gather can cause performance problems due to
	// blocking on AssetRegistry, and can also cause deadlocks with the AssetRegistry's gather thread, UE-281930.
	// Disable it until we have fixed those problems.
	bEnabledBeforeAssetRegistryGather = false;
	if (bExternalSave)
	{
		SaveClient.Reset(new FEditorDomainSaveClient());
	}
	bSkipSavesUntilCatalogLoaded = GIsBuildMachine;

	AssetRegistry = IAssetRegistry::Get();
	// We require calling SearchAllAssets, because we rely on being able to call WaitOnAsset
	// without needing to call ScanPathsSynchronous
	AssetRegistry->SearchAllAssets(false /* bSynchronousSearch */);

	if (EnableLevel >= EEditorDomainEnabled::PackageResourceManager)
	{
		bEditorDomainReadEnabled = !FParse::Param(FCommandLine::Get(), TEXT("noeditordomainread"))
			&& !FParse::Param(FCommandLine::Get(), TEXT("testeditordomaindeterminism"));
		bEditorDomainWriteEnabled = !FParse::Param(FCommandLine::Get(), TEXT("noeditordomainwrite"));
	}
	else
	{
		bEditorDomainReadEnabled = false;
		bEditorDomainWriteEnabled = false;
	}

	if (bEditorDomainWriteEnabled)
	{
		ELoadingPhase::Type CurrentPhase = IPluginManager::Get().GetLastCompletedLoadingPhase();
		if (CurrentPhase == ELoadingPhase::None || CurrentPhase < ELoadingPhase::PostEngineInit)
		{
			FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FEditorDomain::OnPostEngineInit);
		}
		else
		{
			OnPostEngineInit();
		}
		FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FEditorDomain::OnEndLoadPackage);
	}
	if (AssetRegistry->IsLoadingAssets() || !AssetRegistry->IsSearchAllAssets())
	{
		AssetRegistry->OnFilesLoaded().AddRaw(this, &FEditorDomain::OnAssetRegistryInitialScanComplete);
	}
	else
	{
		bAssetRegistryInitialScanComplete.store(true, std::memory_order_relaxed);
	}
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FEditorDomain::OnPackageSavedWithContext);
	AssetRegistry->OnAssetUpdatedOnDisk().AddRaw(this, &FEditorDomain::OnAssetUpdatedOnDisk);
}

FEditorDomain::~FEditorDomain()
{
	// We have a pointer to AssetRegistry, but it might be dangling due to the AssetRegistry module already shutting down.
	// Refetch it from the singleton pointer, and handle the case that it is now null.
	AssetRegistry = IAssetRegistry::Get();

	TUniquePtr<UE::DerivedData::FRequestOwner> LocalBatchDownloadOwner;
	{
		FScopeLock ScopeLock(&Locks->Lock);
		LocalBatchDownloadOwner = MoveTemp(BatchDownloadOwner);
	}
	// BatchDownloadOwner must be deleted (which calls Cancel) outside of the lock, since its callback takes the lock
	LocalBatchDownloadOwner.Reset();

	FScopeLock ScopeLock(&Locks->Lock);
	// AssetRegistry has already been destructed by this point, do not try to access it.
	// AssetRegistry->OnAssetUpdatedOnDisk().RemoveAll(this);
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	if (AssetRegistry)
	{
		AssetRegistry->OnFilesLoaded().RemoveAll(this);
	}
	Locks->Owner = nullptr;
	AssetRegistry = nullptr;
	Workspace.Reset();

	if (SingletonEditorDomain == this)
	{
		SingletonEditorDomain = nullptr;
	}
	if (UE::EditorDomain::IPackageDigestCache::Get() == this)
	{
		UE::EditorDomain::IPackageDigestCache::Set(nullptr);
	}
}

FEditorDomain* FEditorDomain::Get()
{
	return SingletonEditorDomain;
}

bool FEditorDomain::SupportsLocalOnlyPaths()
{
	// Local Only paths are supported by falling back to the WorkspaceDomain
	return true;
}

bool FEditorDomain::SupportsPackageOnlyPaths()
{
	return true;
}

bool FEditorDomain::DoesPackageExist(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	return Workspace->DoesPackageExist(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath);
}

FEditorDomain::FLocks::FLocks(FEditorDomain& InOwner)
	:Owner(&InOwner)
{
}

bool FEditorDomain::TryFindOrAddPackageSource(FScopeLock& ScopeLock, bool& bOutReenteredLock, FName PackageName,
	TRefCountPtr<FPackageSource>& OutSource, UE::EditorDomain::FPackageDigest* OutErrorDigest)
{
	// Called within &Locks->Lock, ScopeLock is locked on that Lock
	using namespace UE::EditorDomain;

	TRefCountPtr<FPackageSource>* PackageSourcePtr = PackageSources.Find(PackageName);
	if (PackageSourcePtr && *PackageSourcePtr)
	{
		bOutReenteredLock = false;
		OutSource = *PackageSourcePtr;
		return true;
	}

	// CalculatepackageDigest calls arbitrary code because of class loads and can reenter EditorDomain functions
	// We have to drop the lock around our call to it
	FPackageDigest PackageDigest;
	{
		Locks->Lock.Unlock();
		ON_SCOPE_EXIT{ Locks->Lock.Lock(); };
		PackageDigest = CalculatePackageDigest(*AssetRegistry, PackageName);
	}

	bOutReenteredLock = true;
	TRefCountPtr<FPackageSource>& PackageSource = PackageSources.FindOrAdd(PackageName);
	if (PackageSource)
	{
		OutSource = PackageSource;
		return true;
	}

	switch (PackageDigest.Status)
	{
	case FPackageDigest::EStatus::Successful:
		PackageSource = new FPackageSource();
		PackageSource->Digest = MoveTemp(PackageDigest);
		OutSource = PackageSource;
		if (!bEditorDomainReadEnabled || !EnumHasAnyFlags(PackageDigest.DomainUse, EDomainUse::LoadEnabled))
		{
			PackageSource->Source = EPackageSource::Workspace;
		}
		return true;
	case FPackageDigest::EStatus::DoesNotExistInAssetRegistry:
		OutSource.SafeRelease();
		// Remove the entry in PackageSources that we added; we added it to avoid a double lookup for new packages,
		// but for non-existent packages we want it not to be there to avoid wasting memory on it
		PackageSources.Remove(PackageName);
		if (OutErrorDigest)
		{
			*OutErrorDigest = MoveTemp(PackageDigest);
		}
		return false;
	default:
		if (bEditorDomainReadEnabled)
		{
			UE_LOGF(LogEditorDomain, Warning,
				"Could not load package %ls from EditorDomain; it will be loaded from the WorkspaceDomain: %ls",
				*WriteToString<256>(PackageName), *PackageDigest.GetStatusString());
		}
		PackageSource = new FPackageSource();
		PackageSource->Source = EPackageSource::Workspace;
		PackageSource->Digest = MoveTemp(PackageDigest);
		OutSource = PackageSource;
		return true;
	}
}

UE::EditorDomain::FPackageDigest FEditorDomain::GetPackageDigest(FName PackageName)
{
	FScopeLock ScopeLock(&Locks->Lock);
	bool bReenteredLock;
	return GetPackageDigest_WithinLock(ScopeLock, bReenteredLock, PackageName);
}

UE::EditorDomain::FPackageDigest FEditorDomain::GetPackageDigest_WithinLock(FScopeLock& ScopeLock, bool& bOutReenteredLock, FName PackageName)
{
	// Called within &Locks->Lock, ScopeLock is locked on that Lock
	using namespace UE::EditorDomain;

	TRefCountPtr<FPackageSource> PackageSource;
	FPackageDigest ErrorDigest;
	if (!TryFindOrAddPackageSource(ScopeLock, bOutReenteredLock, PackageName, PackageSource, &ErrorDigest))
	{
		return ErrorDigest;
	}
	return PackageSource->Digest;
}

void FEditorDomain::PrecachePackageDigest(FName PackageName)
{
	AssetRegistry->WaitForPackage(PackageName.ToString());
	TOptional<FAssetPackageData> PackageData = AssetRegistry->GetAssetPackageDataCopy(PackageName);
	if (PackageData)
	{
		TArray<FTopLevelAssetPath> ImportedClassPaths;
		ImportedClassPaths.Reserve(PackageData->ImportedClasses.Num());
		for (FName ClassPathName : PackageData->ImportedClasses)
		{
			FTopLevelAssetPath ClassPath(WriteToString<256>(ClassPathName).ToView());
			if (ClassPath.IsValid())
			{
				ImportedClassPaths.Add(ClassPath);
			}
		}

		UE::EditorDomain::PrecacheClassDigests(ImportedClassPaths);
	}
}

TRefCountPtr<FEditorDomain::FPackageSource> FEditorDomain::FindPackageSource(const FPackagePath& PackagePath)
{
	// Called within Locks.Lock
	using namespace UE::EditorDomain;

	FName PackageName = PackagePath.GetPackageFName();
	if (!PackageName.IsNone())
	{
		TRefCountPtr<FPackageSource>* PackageSource = PackageSources.Find(PackageName);
		if (PackageSource)
		{
			return *PackageSource;
		}
	}

	return TRefCountPtr<FPackageSource>();
}

void FEditorDomain::MarkLoadedFromWorkspaceDomain(const FPackagePath& PackagePath, TRefCountPtr<FPackageSource>& PackageSource,
	bool bHasRecordInEditorDomain)
{
	if (PackageSource->Source == FEditorDomain::EPackageSource::Workspace)
	{
		return;
	}

	PackageSource->Source = FEditorDomain::EPackageSource::Workspace;
	PackageSource->bHasRecordInEditorDomain |= bHasRecordInEditorDomain;
	if (bExternalSave)
	{
		if (PackageSource->NeedsEditorDomainSave(*this))
		{
			SaveClient->RequestSave(PackagePath);
		}
	}
	// Otherwise, we will check whether it needs to save in OnEndLoadPackage
}

void FEditorDomain::MarkLoadedFromEditorDomain(const FPackagePath& PackagePath, TRefCountPtr<FPackageSource>& PackageSource)
{
	if (PackageSource->Source == FEditorDomain::EPackageSource::Editor)
	{
		return;
	}

	PackageSource->Source = FEditorDomain::EPackageSource::Editor;
	PackageSource->bHasRecordInEditorDomain = true;
}

int64 FEditorDomain::FileSize(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	using namespace UE::EditorDomain;
	using namespace UE::DerivedData;

	checkf(CookedIndex.IsDefault(), TEXT("Editor Domain does not support bulkdata cooked indices"));

	if (PackageSegment != EPackageSegment::Header
		|| (!bEnabledBeforeAssetRegistryGather && !IsAssetRegistryInitialScanComplete()))
	{
		return Workspace->FileSize(PackagePath, FBulkDataCookedIndex::Default, PackageSegment, OutUpdatedPath);
	}

	TOptional<UE::DerivedData::FRequestOwner> Owner;
	int64 FileSize = -1;
	{
		FScopeLock ScopeLock(&Locks->Lock);
		TRefCountPtr<FPackageSource> PackageSource;
		FName PackageName = PackagePath.GetPackageFName();
		if (PackageName.IsNone())
		{
			return Workspace->FileSize(PackagePath, FBulkDataCookedIndex::Default, PackageSegment, OutUpdatedPath);
		}

		bool bReenteredLock;
		if (!TryFindOrAddPackageSource(ScopeLock, bReenteredLock, PackageName, PackageSource) || PackageSource->Source == EPackageSource::Workspace)
		{
			return Workspace->FileSize(PackagePath, FBulkDataCookedIndex::Default, PackageSegment, OutUpdatedPath);
		}
		PackageSource->SetHasLoaded();

		auto MetaDataGetComplete =
			[&FileSize, &PackageSource, &PackagePath, PackageSegment, Locks=this->Locks, OutUpdatedPath]
			(UE::DerivedData::FCacheGetResponse&& Response)
		{
			bool bLoadFromWorkspace = false;
			bool bHasRecordInEditorDomain = false;
			if (Response.Status != UE::DerivedData::EStatus::Ok)
			{
				if (PackageSource->Source == FEditorDomain::EPackageSource::Editor)
				{
					UE_LOGF(LogEditorDomain, Error, "%ls was previously loaded from the EditorDomain but now is unavailable. This may cause failures during serialization due to changed FileSize and Format.",
						*PackagePath.GetDebugName());
				}
				bLoadFromWorkspace = true;
			}
			else
			{
				bHasRecordInEditorDomain = true;
				const FCbObject& MetaData = Response.Record.GetMeta();
				bool bStorageValid = MetaData["Valid"].AsBool(false);
				if (!bStorageValid)
				{
					bLoadFromWorkspace = true;
				}
				else
				{
					FileSize = MetaData["FileSize"].AsInt64();
				}
			}

			FScopeLock ScopeLock(&Locks->Lock);
			FEditorDomain* EditorDomain = Locks->Owner;
			if (!EditorDomain)
			{
				UE_LOGF(LogEditorDomain, Warning, "%ls size read after EditorDomain shutdown. Returning -1.",
					*PackagePath.GetDebugName());
				FileSize = -1;
			}
			else if (PackageSource->Source == FEditorDomain::EPackageSource::Workspace || bLoadFromWorkspace)
			{
				EditorDomain->MarkLoadedFromWorkspaceDomain(PackagePath, PackageSource, bHasRecordInEditorDomain);
				FileSize = EditorDomain->Workspace->FileSize(PackagePath, FBulkDataCookedIndex::Default, PackageSegment, OutUpdatedPath);
			}
			else
			{
				EditorDomain->MarkLoadedFromEditorDomain(PackagePath, PackageSource);
			}
		};
		// Fetch meta-data only
		ECachePolicy SkipFlags = ECachePolicy::SkipData & ~ECachePolicy::SkipMeta;
		Owner.Emplace(EPriority::Highest);
		RequestEditorDomainPackage(PackagePath, PackageSource->Digest.Hash, SkipFlags,
			*Owner, MoveTemp(MetaDataGetComplete));
	}
	COOK_STAT(auto Timer = UE::EditorDomain::CookStats::Usage.TimeAsyncWait());
	COOK_STAT(Timer.TrackCyclesOnly());
	Owner->Wait();
	return FileSize;
}

FOpenPackageResult FEditorDomain::OpenReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	using namespace UE::EditorDomain;
	using namespace UE::DerivedData;

	checkf(CookedIndex.IsDefault(), TEXT("Editor Domain does not support bulkdata cooked indices"));

	FScopeLock ScopeLock(&Locks->Lock);
	if (PackageSegment != EPackageSegment::Header
		|| (!bEnabledBeforeAssetRegistryGather && !IsAssetRegistryInitialScanComplete()))
	{
		return Workspace->OpenReadPackage(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath);
	}
	FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		return Workspace->OpenReadPackage(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath);
	}
	TRefCountPtr<FPackageSource> PackageSource;
	bool bReenteredLock;
	if (!TryFindOrAddPackageSource(ScopeLock, bReenteredLock, PackageName, PackageSource) || (PackageSource->Source == EPackageSource::Workspace))
	{
		return Workspace->OpenReadPackage(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath);
	}
	PackageSource->SetHasLoaded();

	// TODO: Change priority to High instead of Blocking once we have removed the GetPackageFormat below
	// and don't need to block on the result before exiting this function
	EPriority Priority = EPriority::Blocking;
	FEditorDomainReadArchive* Result = new FEditorDomainReadArchive(Locks, PackagePath, PackageSource, Priority);
	const FIoHash PackageEditorHash = PackageSource->Digest.Hash;
	const bool bHasEditorSource = (PackageSource->Source == EPackageSource::Editor);

	// Unlock before requesting the package because the completion callback takes the lock.
	ScopeLock.Unlock();

	// Fetch only meta-data in the initial request
	ECachePolicy SkipFlags = ECachePolicy::SkipData & ~ECachePolicy::SkipMeta;
	RequestEditorDomainPackage(PackagePath, PackageEditorHash,
		SkipFlags, Result->GetRequestOwner(),
		[Result](FCacheGetResponse&& Response)
		{
			// Note that ~FEditorDomainReadArchive waits for this callback to be called, so Result cannot dangle
			Result->OnRecordRequestComplete(MoveTemp(Response));
		});

	// Precache the exports segment
	// EDITOR_DOMAIN_TODO: Skip doing this for OpenReadPackage calls from bulk data
	Result->Precache(0, 0);

	if (OutUpdatedPath)
	{
		*OutUpdatedPath = PackagePath;
	}

	const EPackageFormat Format = bHasEditorSource ? EPackageFormat::Binary : Result->GetPackageFormat();
	bool bNeedsEngineVersionChecks = bHasEditorSource ? false : (Result->GetPackageSource() != EPackageSource::Editor);
	return FOpenPackageResult{ TUniquePtr<FArchive>(Result), Format, bNeedsEngineVersionChecks};
}

FOpenAsyncPackageResult FEditorDomain::OpenAsyncReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex, EPackageSegment PackageSegment)
{
	using namespace UE::EditorDomain;
	using namespace UE::DerivedData;

	checkf(CookedIndex.IsDefault(), TEXT("Editor Domain does not support bulkdata cooked indices"));

	FScopeLock ScopeLock(&Locks->Lock);
	if (PackageSegment != EPackageSegment::Header
		|| (!bEnabledBeforeAssetRegistryGather && !IsAssetRegistryInitialScanComplete()))
	{
		return Workspace->OpenAsyncReadPackage(PackagePath, CookedIndex, PackageSegment);
	}

	FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		return Workspace->OpenAsyncReadPackage(PackagePath, CookedIndex, PackageSegment);
	}
	TRefCountPtr<FPackageSource> PackageSource;
	bool bReenteredLock;
	if (!TryFindOrAddPackageSource(ScopeLock, bReenteredLock, PackageName, PackageSource) ||
		(PackageSource->Source == EPackageSource::Workspace))
	{
		return Workspace->OpenAsyncReadPackage(PackagePath, CookedIndex, PackageSegment);
	}
	PackageSource->SetHasLoaded();

	// TODO: Change priority to Normal instead of Blocking once we have removed the GetPackageFormat below
	// and don't need to block on the result before exiting this function
	EPriority Priority = EPriority::Blocking;
	FEditorDomainAsyncReadFileHandle* Result =
		new FEditorDomainAsyncReadFileHandle(Locks, PackagePath, PackageSource, Priority);
	const bool bHasEditorSource = (PackageSource->Source == EPackageSource::Editor);
	FIoHash EditorDomainHash = PackageSource->Digest.Hash;

	// Unlock before requesting the package because the completion callback takes the lock.
	ScopeLock.Unlock();

	// Fetch meta-data only in the initial request
	ECachePolicy SkipFlags = ECachePolicy::SkipData & ~ECachePolicy::SkipMeta;
	RequestEditorDomainPackage(PackagePath, EditorDomainHash, SkipFlags, Result->GetRequestOwner(),
		[Result](FCacheGetResponse&& Response)
		{
			// Note that ~FEditorDomainAsyncReadFileHandle waits for this callback to be called, so Result cannot dangle
			Result->OnRecordRequestComplete(MoveTemp(Response));
		});

	const EPackageFormat Format = bHasEditorSource ? EPackageFormat::Binary : Result->GetPackageFormat();
	bool bNeedsEngineVersionChecks = bHasEditorSource ? false : (Result->GetPackageSource() != EPackageSource::Editor);
	return FOpenAsyncPackageResult{ TUniquePtr<IAsyncReadFileHandle>(Result), Format, bNeedsEngineVersionChecks };
}

IMappedFileHandle* FEditorDomain::OpenMappedHandleToPackage(const FPackagePath& PackagePath,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	// No need to implement this runtime feature in the editor domain.
	return nullptr;
}

bool FEditorDomain::TryMatchCaseOnDisk(const FPackagePath& PackagePath, FPackagePath* OutNormalizedPath)
{
	return Workspace->TryMatchCaseOnDisk(PackagePath, OutNormalizedPath);
}

TUniquePtr<FArchive> FEditorDomain::OpenReadExternalResource(EPackageExternalResource ResourceType, FStringView Identifier)
{
	return Workspace->OpenReadExternalResource(ResourceType, Identifier);
}

bool FEditorDomain::DoesExternalResourceExist(EPackageExternalResource ResourceType, FStringView Identifier)
{
	return Workspace->DoesExternalResourceExist(ResourceType, Identifier);
}

FOpenAsyncPackageResult FEditorDomain::OpenAsyncReadExternalResource(
	EPackageExternalResource ResourceType, FStringView Identifier)
{
	return Workspace->OpenAsyncReadExternalResource(ResourceType, Identifier);
}

void FEditorDomain::FindPackagesRecursive(TArray<TPair<FPackagePath, EPackageSegment>>& OutPackages,
	FStringView PackageMount, FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard)
{
	return Workspace->FindPackagesRecursive(OutPackages, PackageMount, FileMount, RootRelPath, BasenameWildcard);
}

void FEditorDomain::IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
	FPackageSegmentVisitor Callback)
{
	Workspace->IteratePackagesInPath(PackageMount, FileMount, RootRelPath, Callback);

}
void FEditorDomain::IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentVisitor Callback)
{
	Workspace->IteratePackagesInLocalOnlyDirectory(RootDir, Callback);
}

void FEditorDomain::IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount,
	FStringView RootRelPath, FPackageSegmentStatVisitor Callback)
{
	Workspace->IteratePackagesStatInPath(PackageMount, FileMount, RootRelPath, Callback);
}

void FEditorDomain::IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentStatVisitor Callback)
{
	Workspace->IteratePackagesStatInLocalOnlyDirectory(RootDir, Callback);
}

void FEditorDomain::Tick(float DeltaTime)
{
	if (bExternalSave)
	{
		SaveClient->Tick(DeltaTime);
	}
}

void FEditorDomain::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	if (bExternalSave || !bEditorDomainWriteEnabled)
	{
		return;
	}
	if (!bEnabledBeforeAssetRegistryGather && !IsAssetRegistryInitialScanComplete())
	{
		return;
	}

	TArray<UPackage*> PackagesToSave;
	{
		FScopeLock ScopeLock(&Locks->Lock);
		if (!bHasPassedPostEngineInit)
		{
			return;
		}
		PackagesToSave.Reserve(Context.LoadedPackages.Num());
		for (UPackage* Package : Context.LoadedPackages)
		{
			PackagesToSave.Add(Package);
		}
		FilterKeepPackagesToSave(PackagesToSave);
	}

	for (UPackage* Package : PackagesToSave)
	{
		UE::EditorDomain::TrySavePackage(Package);
	}
}

void FEditorDomain::OnPostEngineInit()
{
	{
		FScopeLock ScopeLock(&Locks->Lock);
		bHasPassedPostEngineInit = true;
		if (bExternalSave || !bEditorDomainWriteEnabled)
		{
			return;
		}
	}

	if (!bEnabledBeforeAssetRegistryGather && !IsAssetRegistryInitialScanComplete())
	{
		return;
	}

	TArray<UPackage*> PackagesToSave;
	FString PackageName;
	for (TObjectIterator<UPackage> It; It; ++It)
	{
		UPackage* Package = *It;
		Package->GetName(PackageName);
		if (Package->IsFullyLoaded() && !FPackageName::IsScriptPackage(PackageName))
		{
			PackagesToSave.Add(Package);
		}
	}

	{
		FScopeLock ScopeLock(&Locks->Lock);
		FilterKeepPackagesToSave(PackagesToSave);
	}

	for (UPackage* Package : PackagesToSave)
	{
		UE::EditorDomain::TrySavePackage(Package);
	}
}

void FEditorDomain::OnAssetRegistryInitialScanComplete()
{
	bAssetRegistryInitialScanComplete.store(true, std::memory_order_relaxed);
}

bool FEditorDomain::IsAssetRegistryInitialScanComplete() const
{
	return bAssetRegistryInitialScanComplete.load(std::memory_order_relaxed);
}

void FEditorDomain::FilterKeepPackagesToSave(TArray<UPackage*>& InOutPackagesToSave)
{
	FPackagePath PackagePath;
	for (int32 Index = 0; Index < InOutPackagesToSave.Num(); )
	{
		UPackage* Package = InOutPackagesToSave[Index];
		bool bKeep = false;
		if (FPackagePath::TryFromPackageName(Package->GetFName(), PackagePath))
		{
			TRefCountPtr<FPackageSource> PackageSource = FindPackageSource(PackagePath);
			if (PackageSource && PackageSource->NeedsEditorDomainSave(*this))
			{
				PackageSource->bHasSaved = true;
				bKeep = true;
			}
		}
		if (bKeep)
		{
			++Index;
		}
		else
		{
			InOutPackagesToSave.RemoveAtSwap(Index);
		}
	}
}

bool FEditorDomain::FPackageSource::NeedsEditorDomainSave(FEditorDomain& EditorDomain) const
{
	return !bHasSaved && !bHasRecordInEditorDomain &&
		(!EditorDomain.bSkipSavesUntilCatalogLoaded || bLoadedAfterCatalogLoaded);
}

void FEditorDomain::FPackageSource::SetHasLoaded()
{
	if (bHasLoaded)
	{
		return;
	}
	bHasLoaded = true;
	bLoadedAfterCatalogLoaded = bHasQueriedCatalog;
}

void FEditorDomain::BatchDownload(TArrayView<FName> PackageNames)
{
	using namespace UE::EditorDomain;
	using namespace UE::DerivedData;

	FScopeLock ScopeLock(&Locks->Lock);
	if (!BatchDownloadOwner)
	{
		BatchDownloadOwner = MakeUnique<FRequestOwner>(EPriority::Highest);
	}

	TArray<FCacheGetRequest> CacheRequests;
	CacheRequests.Reserve(PackageNames.Num());
	ECachePolicy CachePolicy = ECachePolicy::Default | ECachePolicy::SkipData;
	for (FName PackageName : PackageNames)
	{
		bool bReenteredLock;
		FPackageDigest PackageDigest = GetPackageDigest_WithinLock(ScopeLock, bReenteredLock, PackageName);
		if (PackageDigest.IsSuccessful() && EnumHasAnyFlags(PackageDigest.DomainUse, EDomainUse::LoadEnabled))
		{
			CacheRequests.Add({ { WriteToString<256>(PackageName) }, GetEditorDomainPackageKey(PackageDigest.Hash),
				CachePolicy });
		}
	}

	if (!CacheRequests.IsEmpty())
	{
		FRequestBarrier Barrier(*BatchDownloadOwner);
		GetCache().Get(CacheRequests, *BatchDownloadOwner, [this](FCacheGetResponse&& Response)
			{
				FScopeLock ScopeLock(&Locks->Lock);
				TRefCountPtr<FPackageSource> PackageSource;
				FName PackageName = FName(*Response.Name);
				bool bReenteredLock;
				if (TryFindOrAddPackageSource(ScopeLock, bReenteredLock, PackageName, PackageSource))
				{
					PackageSource->bHasQueriedCatalog = true;
				}
			});
	}
}

void FEditorDomain::OnPackageSavedWithContext(const FString& PackageFileName, UPackage* Package,
	FObjectPostSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsUpdatingLoadedPath())
	{
		return;
	}
	FName PackageName = Package->GetFName();
	FScopeLock ScopeLock(&Locks->Lock);
	PackageSources.Remove(PackageName);
}

void FEditorDomain::OnAssetUpdatedOnDisk(const FAssetData& AssetData)
{
	FName PackageName = AssetData.PackageName;
	if (PackageName.IsNone())
	{
		return;
	}
	FScopeLock ScopeLock(&Locks->Lock);
	PackageSources.Remove(PackageName);
}

namespace UE::EditorDomain
{

FPackageDigest::FPackageDigest(EStatus InStatus, FName InStatusArg)
 : Status(InStatus)
 , StatusArg(InStatusArg)
{
}

bool FPackageDigest::IsSuccessful() const
{
	return Status == EStatus::Successful;
}

FString FPackageDigest::GetStatusString() const
{
	return LexToString(Status, StatusArg);
}

}

FString LexToString(UE::EditorDomain::FPackageDigest::EStatus Status, FName StatusArg)
{
	using namespace UE::EditorDomain;

	switch (Status)
	{
	case FPackageDigest::EStatus::NotYetRequested:
		return TEXT("Has not been requested.");
	case FPackageDigest::EStatus::Successful:
		return TEXT("Successful.");
	case FPackageDigest::EStatus::InvalidPackageName:
		return TEXT("PackageName is not a valid LongPackageName.");
	case FPackageDigest::EStatus::DoesNotExistInAssetRegistry:
		return TEXT("Does not exist in AssetRegistry.");
	case FPackageDigest::EStatus::MissingClass:
		return FString::Printf(TEXT("Uses class %s that is not loaded."), *StatusArg.ToString());
	case FPackageDigest::EStatus::MissingCustomVersion:
		return FString::Printf(
			TEXT("Uses CustomVersion guid %s but that guid is not available in FCurrentCustomVersions."),
			*StatusArg.ToString());
	default:
		return TEXT("Unknown result code.");
	}
}
