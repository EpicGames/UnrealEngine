// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "IO/IoHash.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageSegment.h"
#include "Stats/Stats.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "UObject/NameTypes.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/UnrealNames.h"

#include <atomic>

class FArchive;
class FEditorDomainSaveClient;
class FObjectPostSaveContext;
class FPackagePath;
class FScopeLock;
class IAssetRegistry;
class IMappedFileHandle;
class UPackage;
struct FEndLoadPackageContext;
namespace UE::DerivedData { class FRequestOwner; }

namespace UE::EditorDomain
{

/** Flags for whether a package is allowed to be loaded/saved in the EditorDomain. */
enum class EDomainUse : uint8
{
	None = 0x0,
	/** The package can be loaded from the EditorDomain. */
	LoadEnabled = 0x1,
	/** The package can be saved to the EditorDomain. */
	SaveEnabled = 0x2,
};
ENUM_CLASS_FLAGS(EDomainUse);
FStringBuilderBase& operator<<(FStringBuilderBase& Writer, UE::EditorDomain::EDomainUse DomainUse);

/** Information about a package's loading from the EditorDomain. */
struct FPackageDigest
{
	enum class EStatus : uint8
	{
		NotYetRequested,
		Successful,
		InvalidPackageName,
		DoesNotExistInAssetRegistry,
		MissingClass,
		MissingCustomVersion,
	};

	FPackageDigest() = default;
	FPackageDigest(EStatus InStatus, FName InStatusArg = NAME_None);
	bool IsSuccessful() const;
	FString GetStatusString() const;

	/**
	 * Hash used to lookup the resaved package in the cache;
	 * created from properties that change if the saved version would change (package bytes, serialization versions).
	 */
	FIoHash Hash;
	/** List of CustomVersions used to save the package. */
	UE::AssetRegistry::FPackageCustomVersionsHandle CustomVersions;
	/** Allow flags for whether the package can be saved/loaded from EditorDomain. */
	EDomainUse DomainUse = EDomainUse::None;
	/** Status for creation of this digest. Either Success or an error code for why it couldn't be created. */
	EStatus Status = EStatus::NotYetRequested;
	/** Extended information for the status description (e.g. missing class name). */
	FName StatusArg;
};

/**
 * An interface to cache repeated calls to GetPackageDigest. This would be just a global TMap without an interface,
 * except that EditorDomain if enabled needs to store other data along with the digest so it takes over the cache duty
 * using its own storage.
 */
class IPackageDigestCache
{
public:
	virtual ~IPackageDigestCache() = default;
	/**
	 * Calculate the PackageDigest for the given PackageName.
	 * Callable from any thread, but will return !IsSuccessful digest if game-thread data is required and not cached.
	 */
	virtual FPackageDigest GetPackageDigest(FName PackageName) = 0;

	static IPackageDigestCache* Get();
	static void Set(IPackageDigestCache* Cache);
	static void SetDefault();
};

}

extern FString LexToString(UE::EditorDomain::FPackageDigest::EStatus Status, FName StatusArg);

DECLARE_LOG_CATEGORY_EXTERN(LogEditorDomain, Log, All);

/**
 * The EditorDomain is container for optimized but still editor-usable versions of WorkspaceDomain packages.
 * The WorkspaceDomain is the source data for Unreal packages; packages created by the editor or compatible importers
 * that can be read by any future build of the project's editor. This source data is converted to an optimized format
 * for the current binary and saved into the EditorDomain, for faster loads when requested again by a later invocation
 * of the editor. The optimizations include running upgrades in UObjects' PostLoad and Serialize, and saving the
 * package in Unversioned format.
 *
 * FEditorDomain is a subclass of IPackageResourceManager that handles PackagePath requests by looking up the
 * package in the EditorDomain cache, stored in the DerivedDataCache. If a version of the package matching the current
 * WorkspaceDomain package and the current binary does not exist, then the EditorDomain falls back to loading from
 * the WorkspaceDomain (through ordinary IFileManager operations on the Root/Game/Content folders) and creates
 * the EditorDomain version for next time.
 */
class FEditorDomain : public IPackageResourceManager, public FTickableEditorObject, public UE::EditorDomain::IPackageDigestCache
{
public:
	/** Different options for which domain a package comes from. */
	enum class EPackageSource
	{
		Undecided,
		Workspace,
		Editor
	};

	FEditorDomain(EEditorDomainEnabled EnableLevel);
	virtual ~FEditorDomain();

	/** Return the EditorDomain that is registered as the global PackageResourceManager, if there is one. */
	static FEditorDomain* Get();

	// IPackageResourceManager interface
	virtual bool SupportsLocalOnlyPaths() override;
	virtual bool SupportsPackageOnlyPaths() override;
	virtual bool DoesPackageExist(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual int64 FileSize(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual FOpenPackageResult OpenReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual FOpenAsyncPackageResult OpenAsyncReadPackage(const FPackagePath& PackagePath,
		FBulkDataCookedIndex CookedIndex, EPackageSegment PackageSegment) override;
	virtual IMappedFileHandle* OpenMappedHandleToPackage(const FPackagePath& PackagePath,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual bool TryMatchCaseOnDisk(const FPackagePath& PackagePath, FPackagePath* OutNormalizedPath = nullptr) override;
	virtual TUniquePtr<FArchive> OpenReadExternalResource(EPackageExternalResource ResourceType, FStringView Identifier) override;
	virtual bool DoesExternalResourceExist(EPackageExternalResource ResourceType, FStringView Identifier) override;
	virtual FOpenAsyncPackageResult OpenAsyncReadExternalResource(
		EPackageExternalResource ResourceType, FStringView Identifier) override;
	virtual void FindPackagesRecursive(TArray<TPair<FPackagePath, EPackageSegment>>& OutPackages, FStringView PackageMount,
		FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard) override;
	virtual void IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentVisitor Callback) override;
	virtual void IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentVisitor Callback) override;
	virtual void IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentStatVisitor Callback) override;
	virtual void IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir, 
		FPackageSegmentStatVisitor Callback) override;

	// FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}
	virtual TStatId GetStatId() const override
	{
		return TStatId();
	}

	// IPackageDigestCache interface
	virtual UE::EditorDomain::FPackageDigest GetPackageDigest(FName PackageDigest) override;

	// EditorDomain interface
	/** Fetch data from game-thread sources that is required to calculate the PackageDigest of the given PackageName. */
	void PrecachePackageDigest(FName PackageName);
	bool IsReadingPackages() const;
	bool IsWritingPackages() const;

	/** Request the download of the given packages from the upstream DDC server. */
	void BatchDownload(TArrayView<FName> PackageNames);

private:
	/**
	 * Reference-counted struct holding the locks used for multithreaded synchronization.
	 * Shared with Archives and other helpers that might outlive *this.
	 */
	class FLocks : public FRefCountedObject
	{
	public:
		FLocks(FEditorDomain& InOwner);
		FEditorDomain* Owner;
		FCriticalSection Lock;
	};
	/**
	 * Data about which domain a package comes from. Multiple queries of the same
	 * package have to align (for e.g. BulkData offsets) so we have to keep track of this.
	 */
	struct FPackageSource : public FRefCountedObject
	{
		FPackageSource()
			: bHasSaved(false), bHasLoaded(false), bHasQueriedCatalog(false), bLoadedAfterCatalogLoaded(false)
			, bHasRecordInEditorDomain(false)
		{
		}

		/** Return whether we should call TrySavePackage into the EditorDomain. */
		bool NeedsEditorDomainSave(FEditorDomain& EditorDomain) const;
		/** Mark that a load is being attempted, which can impact whether we need to save. */
		void SetHasLoaded();

		UE::EditorDomain::FPackageDigest Digest;
		EPackageSource Source = EPackageSource::Undecided;
		bool bHasSaved : 1;
		bool bHasLoaded : 1;
		bool bHasQueriedCatalog : 1;
		bool bLoadedAfterCatalogLoaded : 1;
		bool bHasRecordInEditorDomain : 1;
	};

	/** Disallow copy constructors */
	FEditorDomain(const FEditorDomain& Other) = delete;
	FEditorDomain(FEditorDomain&& Other) = delete;

	/**
	 * Read the PackageSource data from PackageSources, or from the asset registry if not in PackageSources.
	 * Note this function can exit and reenter the provided ScopeLock. If so it will set bOutReenteredLock=true and caller must handle
	 * retesting variables that may have changed while outside of the lock.
	*/
	bool TryFindOrAddPackageSource(FScopeLock& ScopeLock, bool& bOutReenteredLock, FName PackageName, TRefCountPtr<FPackageSource>& OutSource,
		UE::EditorDomain::FPackageDigest* OutErrorDigest=nullptr);
	/** Return the PackageSource data in PackageSources, if it exists */
	TRefCountPtr<FPackageSource> FindPackageSource(const FPackagePath& PackagePath);
	/** Mark that we had to load the Package from the workspace domain, and schedule its save into the EditorDomain if allowed. */
	void MarkLoadedFromWorkspaceDomain(const FPackagePath& PackagePath, TRefCountPtr<FPackageSource>& PackageSource, bool bHasRecordInEditorDomain);
	/** Mark that we were able to load from the EditorDomain */
	void MarkLoadedFromEditorDomain(const FPackagePath& PackagePath, TRefCountPtr<FPackageSource>& PackageSource);
	/** Callback for PostEngineInit, to handle saving of packages which we could not save before then. */
	void OnPostEngineInit();
	/** Callback for AssetRegistry's InitialScanComplete, to enable save/load if we disable before initial scan. */
	void OnAssetRegistryInitialScanComplete();
	/** Report whether we have set our flag from OnAssetRegistryInitialScanComplete. */
	bool IsAssetRegistryInitialScanComplete() const;
	/** EndLoad callback to handle saving the EditorDomain version of the package. */
	void OnEndLoadPackage(const FEndLoadPackageContext& Context);
	/** For each of the now-loaded packages, if we had to load from workspace domain, save into the editor domain. */
	void FilterKeepPackagesToSave(TArray<UPackage*>& InOutLoadedPackages);
	/** PackageSaved context to invalidate our information about where it should be loaded from. */
	void OnPackageSavedWithContext(const FString& PackageFileName, UPackage* Package,
		FObjectPostSaveContext ObjectSaveContext);
	/** AssetUpdated event to invalidate our information about where it should be loaded from. */
	void OnAssetUpdatedOnDisk(const FAssetData& AssetData);
	/** Same As GetPackageDigest, but assumes lock is already held. */
	UE::EditorDomain::FPackageDigest GetPackageDigest_WithinLock(FScopeLock& ScopeLock, bool& bOutReenteredLock, FName PackageDigest);


	/** Subsystem used to request the save of missing packages into the EditorDomain from a separate process. */
	TUniquePtr<FEditorDomainSaveClient> SaveClient;
	/** PackageResourceManagerFile to fall back to WorkspaceDomain when packages are missing from EditorDomain. */
	TUniquePtr<IPackageResourceManager> Workspace;
	/** Cached pointer to the global AssetRegistry. */
	IAssetRegistry* AssetRegistry = nullptr;
	/** Locks used by *this and its helper objects. */
	TRefCountPtr<FLocks> Locks;
	/** Digests previously found for a package. Used for optimization, but also to record loaded-from-domain. */
	TMap<FName, TRefCountPtr<FPackageSource>> PackageSources;
	/** RequestOwner used for BatchDownloads from upstream server. */
	TUniquePtr<UE::DerivedData::FRequestOwner> BatchDownloadOwner;

	/** True by default, set to false when reading is disabled for testing. */
	bool bEditorDomainReadEnabled = true;
	/** True by default, set to false when writing is disabled for testing. */
	bool bEditorDomainWriteEnabled = true;
	/** If true, use an out-of-process EditorDomainSaveServer for saves, else save in process in EndLoad */
	bool bExternalSave = false;
	/** Marker for whether our PostEngineInit callback has been called */
	bool bHasPassedPostEngineInit = false;
	/** Marker for whether AssetRegistry initial scan has finished. */
	std::atomic<bool> bAssetRegistryInitialScanComplete{ false };
	/** Configuration setting for whether save/load is allowed before the AssetRegistry initial scan has finished. */
	bool bEnabledBeforeAssetRegistryGather = true;
	/**
	 * Configuration value for whether saves into the EditorDomain should be skipped for packages loaded before
	 * the DDC server reports whether they exist in the upstream DDC.
	 */
	bool bSkipSavesUntilCatalogLoaded = false;

	static FEditorDomain* SingletonEditorDomain;

	friend class FEditorDomainConstructor;
	friend class FEditorDomainPackageSegments;
	friend class FEditorDomainReadArchive;
	friend class FEditorDomainAsyncReadFileHandle;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline bool FEditorDomain::IsReadingPackages() const
{
	return bEditorDomainReadEnabled;
}

inline bool FEditorDomain::IsWritingPackages() const
{
	return bEditorDomainWriteEnabled;
}
