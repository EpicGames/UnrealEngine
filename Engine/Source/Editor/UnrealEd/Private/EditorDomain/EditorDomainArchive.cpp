// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDomain/EditorDomainArchive.h"

#include "Algo/BinarySearch.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringView.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataCachePolicy.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataValue.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/PlatformString.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageSegment.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"

FEditorDomainPackageSegments::FEditorDomainPackageSegments(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
	const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource, UE::DerivedData::EPriority Priority)
	: EditorDomainLocks(InLocks)
	, RequestOwner(Priority)
	, PackagePath(InPackagePath)
	, PackageSource(InPackageSource)
	, EditorDomainHash(InPackageSource->Digest.Hash)
{
}

FEditorDomainPackageSegments::~FEditorDomainPackageSegments()
{
	Close();
}

FEditorDomainPackageSegments::ESource FEditorDomainPackageSegments::GetSource() const
{
	return Source;
}

std::atomic<FEditorDomainPackageSegments::ESource>& FEditorDomainPackageSegments::GetAsyncSource()
{
	return AsyncSource;
}

void FEditorDomainPackageSegments::WaitForReady() const
{
	if (Source != ESource::Uninitialized)
	{
		return;
	}
	COOK_STAT(auto Timer = UE::EditorDomain::CookStats::Usage.TimeAsyncWait());
	COOK_STAT(Timer.TrackCyclesOnly());
	const_cast<UE::DerivedData::FRequestOwner&>(RequestOwner).Wait();
	Source = AsyncSource;
}

void FEditorDomainPackageSegments::Seek(int64 InPos)
{
	Pos = InPos;
}

int64 FEditorDomainPackageSegments::Tell() const
{
	return Pos;
}

int64 FEditorDomainPackageSegments::TotalSize() const
{
	check(Source == ESource::Segments);
	return Size;
}

const FPackagePath& FEditorDomainPackageSegments::GetPackagePath() const
{
	return PackagePath;
}

void FEditorDomainPackageSegments::Close()
{
	// Called from Interface-only
	AsyncSource = ESource::Closed;
	RequestOwner.Cancel();
	for (FSegment& Segment : Segments)
	{
		if (Segment.RequestOwner)
		{
			Segment.RequestOwner->Cancel();
		}
	}

	Segments.Reset();
	Source = ESource::Closed;
}

bool FEditorDomainPackageSegments::Serialize(void* V, int64 Length, bool bIsError)
{
	// Called from Interface-only
	check(Source == ESource::Segments); // Caller should not call in any other case
	int64 End = Pos + Length;
	if (Pos < 0)
	{
		UE_LOG(LogEditorDomain, Error, TEXT("Invalid negative Pos %" INT64_FMT " (file = %s)"), Pos,
			*PackagePath.GetDebugName());
		return false;
	}
	if (End > Size)
	{
		// Log the error if this is the first error. Do not log it on subsequent errors, to prevent spam in e.g. array reading
		UE_CLOG(!bIsError, LogEditorDomain, Error,
			TEXT("Requested read of %" INT64_FMT " bytes when %" INT64_FMT " bytes remain (file = %s, size=%" INT64_FMT ")"),
			Length, Size - Pos, *PackagePath.GetDebugName(), Size);
		return false;
	}
	while (Pos < End)
	{
		if (MRUSegment)
		{
			int64 SegmentEnd = MRUSegment < &Segments.Last() ? (MRUSegment + 1)->Start : Size;
			int64 LengthInSegment = FMath::Min(SegmentEnd, End) - Pos;
			int64 SegmentPos = Pos - MRUSegment->Start;
			if (SegmentPos >= 0 && LengthInSegment > 0)
			{
				FMemory::Memcpy(V, static_cast<const uint8*>(MRUSegment->Data.GetData()) + SegmentPos, LengthInSegment);
				Pos += LengthInSegment;
				V = static_cast<void*>(static_cast<uint8*>(V) + LengthInSegment);
			}
			else
			{
				MRUSegment = nullptr;
			}
		}
		else
		{
			bool bIsReady;
			MRUSegment = EnsureSegmentRange(Pos, End, true, bIsReady);
			int64 SegmentEnd = MRUSegment < &Segments.Last() ? (MRUSegment + 1)->Start : Size;
			int64 ExpectedSize = SegmentEnd - MRUSegment->Start;
			if (MRUSegment->Data.GetSize() != ExpectedSize)
			{
				UE_CLOGF(!bIsError, LogEditorDomain, Error, "Package %ls is missing cache data in segment %d.",
					*PackagePath.GetDebugName(), (int32)(MRUSegment - &Segments[0]));
				MRUSegment = nullptr;
				return false;
			}
		}
	}
	return true;
}

void FEditorDomainPackageSegments::Precache(int64 InStart, int64 InSize, bool* bOutIsReady)
{
	switch (Source)
	{
	case ESource::Uninitialized:
		if (AsyncSource != ESource::Uninitialized)
		{
			WaitForReady();
			return Precache(InStart, InSize, bOutIsReady);
		}
		else
		{
			// Set the Initial request offset. We only support a single initial offset; overwrite any
			// previously existing offset. 
			InitialRequestOffset = InStart;
			// THe offset will not be used if the callback thread reached the callback and checked the
			// value of InitialRequestOffset in between our read of AsyncSource and our write of
			// InitialRequestOffset. Check AsyncSource again to handle that case.
			if (AsyncSource != ESource::Uninitialized)
			{
				WaitForReady();
				return Precache(InStart, InSize, bOutIsReady);
			}
			if (bOutIsReady)
			{
				*bOutIsReady = false;
			}
		}
		break;
	case ESource::Segments:
	{
		int64 End = InStart + InSize;
		End = FMath::Min(End, this->Size);
		int64 Start = FMath::Clamp<int64>(InStart, 0, End);

		bool bIsReady;
		EnsureSegmentRange(Start, End, false /* bWait */, bIsReady);
		if (bOutIsReady)
		{
			*bOutIsReady = bIsReady;
		}
	}
	default:
		break;
	}
}

int32 FEditorDomainPackageSegments::GetSegmentContainingOffset(uint64 Start) const
{
	int32 StartIndex = Algo::LowerBoundBy(Segments, Start, [](const FSegment& Segment) { return Segment.Start; });
	check(0 <= StartIndex && StartIndex <= Segments.Num());
	if (StartIndex == Segments.Num() || Segments[StartIndex].Start > Start)
	{
		check(StartIndex > 0);
		--StartIndex;
	}
	check(Segments[StartIndex].Start <= Start && (StartIndex == Segments.Num() - 1 || Start < Segments[StartIndex + 1].Start));
	return StartIndex;
}

FEditorDomainPackageSegments::FSegment* FEditorDomainPackageSegments::EnsureSegmentRange(uint64 Start, uint64 End, bool bWait, bool& bOutIsReady)
{
	// Called from Interface-only
	check(Source == ESource::Segments); // Caller should not call in any other case

	int32 StartIndex = GetSegmentContainingOffset(Start);
	FSegment* StartSegment = &Segments[StartIndex];

	bOutIsReady = true;
	bool bNeedsWait = false;
	FSegment* Segment = StartSegment;
	FSegment* FileEndSegment = &Segments[0] + Segments.Num();
	do
	{
		if (!Segment->bComplete)
		{
			if (!Segment->RequestOwner)
			{
				SendSegmentRequest(*Segment);
			}
			if (Segment->bAsyncComplete)
			{
				Segment->bComplete = true;
			}
			else
			{
				bOutIsReady = false;
			}
		}
		++Segment;
	} while (Segment < FileEndSegment && Segment->Start < End);

	if (!bOutIsReady && bWait)
	{
		Segment = StartSegment;
		do
		{
			if (!Segment->bComplete)
			{
				if (Segment->RequestOwner)
				{
					Segment->RequestOwner->Wait();
				}
				check(Segment->bAsyncComplete);
				Segment->bComplete = true;
			}
			++Segment;
		} while (Segment < FileEndSegment && Segment->Start < End);
		bOutIsReady = true;
	}
	return StartSegment;
}

void FEditorDomainPackageSegments::OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response,
	TUniqueFunction<void(bool bValid)>&& CreateSegmentData,
	TUniqueFunction<bool(FEditorDomain& EditorDomain)>&& TryCreateFallbackData)
{
	using namespace UE::DerivedData;

	{
		ESource LocalAsyncSource = AsyncSource;
		if (AsyncSource == ESource::Closed)
		{
			return;
		}
		check(LocalAsyncSource == ESource::Uninitialized);
	}

	Size = 0;

	ESource NewAsyncSource = ESource::Uninitialized;
	int32 InitialRequestIndex = -1;
	bool bExistsInEditorDomain = false;
	bool bStorageValid = false;

	if (Response.Status == EStatus::Ok)
	{
		const FCbObject& MetaData = Response.Record.GetMeta();
		bExistsInEditorDomain = true;
		bStorageValid = MetaData["Valid"].AsBool(false);
		COOK_STAT(UE::EditorDomain::CookStats::Usage.TimeSyncWork().AddHit(Response.Record.GetMeta().GetSize()));
	}

	if (bStorageValid)
	{
		const FCacheRecord& Record = Response.Record;
		const uint64 FileSize = Record.GetMeta()["FileSize"].AsUInt64();

		TConstArrayView<FValueWithId> Values = Record.GetValues();
		Segments.Reserve(Values.Num());
		uint64 CompositeSize = 0;
		for (const FValueWithId& Value : Values)
		{
			if (const uint64 ValueSize = Value.GetRawSize())
			{
				Segments.Emplace(Value.GetId(), CompositeSize);
				CompositeSize += ValueSize;
			}
		}

		if (CompositeSize != FileSize)
		{
			bool bMetaDataExists = bool(Record.GetMeta());
			UE_LOGF(LogEditorDomain, Warning,
				"Package %ls received an invalid CacheRecord from DDC: %ls. Reading from workspace domain instead.",
				*PackagePath.GetDebugName(),
				bMetaDataExists ? *WriteToString<128>(
					TEXT("size of all segments "), CompositeSize, TEXT(" is not equal to FileSize in metadata "), FileSize
					) : TEXT("metadata is empty"));
			Segments.Empty();
			bExistsInEditorDomain = false;
		}
		else
		{
			bool bRegisterSuccessful = false;
			bool bEditorDomainAvailable = false;
			{
				FScopeLock DomainScopeLock(&EditorDomainLocks->Lock);
				if (EditorDomainLocks->Owner)
				{
					bEditorDomainAvailable = true;
					if (PackageSource->Source == FEditorDomain::EPackageSource::Undecided || PackageSource->Source == FEditorDomain::EPackageSource::Editor)
					{
						EditorDomainLocks->Owner->MarkLoadedFromEditorDomain(PackagePath, PackageSource);
						bRegisterSuccessful = true;
					}
				}
			}
			if (bRegisterSuccessful)
			{
				Size = FileSize;
				NewAsyncSource = ESource::Segments;
				CreateSegmentData(true /* bValid */);
				if (0 <= InitialRequestOffset && static_cast<uint32>(InitialRequestOffset) < FileSize)
				{
					InitialRequestIndex = GetSegmentContainingOffset(InitialRequestOffset);
				}
			}
			else
			{
				Segments.Empty();
				if (!bEditorDomainAvailable)
				{
					UE_LOGF(LogEditorDomain, Warning, "%ls read after EditorDomain shutdown. Archive Set to Error.",
						*PackagePath.GetDebugName());
					NewAsyncSource = ESource::Segments;
					CreateSegmentData(false /* bValid */);
				}
			}
		}
	}

	UE_LOGF(LogEditorDomain, Verbose, "Loading from %ls: %ls.",
		NewAsyncSource == ESource::Segments ? TEXT("EditorDomain") : TEXT("WorkspaceDomain"), *PackagePath.GetDebugName());
	if (NewAsyncSource == ESource::Uninitialized)
	{
		FScopeLock DomainScopeLock(&EditorDomainLocks->Lock);
		if (EditorDomainLocks->Owner)
		{
			if (PackageSource->Source == FEditorDomain::EPackageSource::Editor)
			{
				UE_LOGF(LogEditorDomain, Error, "%ls was previously loaded from the EditorDomain but now is unavailable. This may cause failures during serialization due to changed FileSize and Format.",
					*PackagePath.GetDebugName());
			}
			FEditorDomain& EditorDomain(*EditorDomainLocks->Owner);
			bool bSucceeded = TryCreateFallbackData(EditorDomain);
			if (bSucceeded)
			{
				EditorDomain.MarkLoadedFromWorkspaceDomain(PackagePath, PackageSource, bExistsInEditorDomain);
				NewAsyncSource = ESource::Fallback;
			}
			else
			{
				NewAsyncSource = ESource::Segments;
			}
		}
		else
		{
			UE_LOGF(LogEditorDomain, Warning, "%ls read after EditorDomain shutdown. Archive Set to Error.",
				*PackagePath.GetDebugName());
			NewAsyncSource = ESource::Segments;
			CreateSegmentData(false /* bValid */);
		}
	}

	ESource ExpectedValue = ESource::Uninitialized;
	bool bSetSuccessful = AsyncSource.compare_exchange_strong(ExpectedValue, NewAsyncSource);
	if (bSetSuccessful && InitialRequestIndex >= 0)
	{
		SendSegmentRequest(Segments[InitialRequestIndex]);
	}
}

void FEditorDomainPackageSegments::SendSegmentRequest(FSegment& Segment)
{
	using namespace UE::DerivedData;

	// Called from Callback-only until AsyncSource is set, then from Interface-only
	ICache& Cache = GetCache();

	// Note that Segment.RequestOwner is Interface-only and so we can write it outside the lock
	Segment.RequestOwner.Emplace(EPriority::Normal);
	FCacheGetChunkRequest SegmentChunk{{PackagePath.GetDebugName()}, UE::EditorDomain::GetEditorDomainPackageKey(EditorDomainHash), Segment.ValueId};
	SegmentChunk.Policy = ECachePolicy::Local;
	Cache.GetChunks({SegmentChunk}, *Segment.RequestOwner,
		[this, &Segment](FCacheGetChunkResponse&& Response)
		{
			if (AsyncSource == ESource::Closed)
			{
				return;
			}
			if (Segment.bAsyncComplete)
			{
				// Race condition; another request got there before us
				return;
			}

			if (Response.Status != EStatus::Ok)
			{
				UE_LOGF(LogEditorDomain, Error, "Package %ls is missing cache data in segment %d.", *PackagePath.GetDebugName(), (int32)(&Segment - &Segments[0]));
			}
			else
			{
				Segment.Data = MoveTemp(Response.RawData);
				uint64 SegmentEnd = &Segment < &Segments.Last() ? (&Segment + 1)->Start : Size;
				uint64 SegmentSize = SegmentEnd - Segment.Start;
				if (Segment.Data.GetSize() != SegmentSize)
				{
					UE_LOGF(LogEditorDomain, Error, "Package %ls has corrupted cache data in segment %d.", *PackagePath.GetDebugName(), (int32)(&Segment - &Segments[0]));
					Segment.Data.Reset();
				}
				COOK_STAT(auto Timer = UE::EditorDomain::CookStats::Usage.TimeSyncWork());
				COOK_STAT(Timer.AddHit(SegmentSize));
				COOK_STAT(Timer.TrackCyclesOnly()); // We're using TrackCyclesOnly to track the size without counting another hit. We actually don't care about the cycles
			}
			Segment.bAsyncComplete = true;
		});
}

FEditorDomainReadArchive::FEditorDomainReadArchive(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
	const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource,
	UE::DerivedData::EPriority Priority)
	: Segments(InLocks, InPackagePath, InPackageSource, Priority)
{
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
}

void FEditorDomainReadArchive::WaitForReady() const
{
	if (Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized)
	{
		return;
	}

	Segments.WaitForReady();

	// If we had any seeks, implement the seek behavior for the last seek, now that we know what that behavior is.
	int64 Pos = Segments.Tell();
	if (Pos != 0)
	{
		switch (Segments.GetSource())
		{
		case FEditorDomainPackageSegments::ESource::Fallback:
		{
			InnerArchive->Seek(Pos);
			break;
		}
		case FEditorDomainPackageSegments::ESource::Segments:
		{
			const_cast<FEditorDomainReadArchive&>(*this).Segments.Precache(Pos, 0);
			break;
		}
		default:
			checkNoEntry();
			break;
		}
	}
}

void FEditorDomainReadArchive::OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response)
{
	Segments.OnRecordRequestComplete(MoveTemp(Response),
		[this](bool bValid) { CreateSegmentData(bValid); },
		[this](FEditorDomain& EditorDomain) { return TryCreateFallbackData(EditorDomain); }
	);
}

void FEditorDomainReadArchive::CreateSegmentData(bool bValid)
{
	PackageFormat = EPackageFormat::Binary;
	if (!bValid)
	{
		SetError();
	}
}

bool FEditorDomainReadArchive::TryCreateFallbackData(FEditorDomain& EditorDomain)
{
	IPackageResourceManager& Workspace = *EditorDomain.Workspace;
	// EDITOR_DOMAIN_TODO: Editor platforms that return false in FPlatformMisc::SupportsMultithreadedFileHandles
	// will fail here because we are creating the file handle on the callback thread and using it on the 
	// interface thread which is on the game thread. Need to make all editor platforms support that, or
	// have this fallback occur when WaitForReady is called.
	FOpenPackageResult Result = Workspace.OpenReadPackage(Segments.GetPackagePath(), FBulkDataCookedIndex::Default, EPackageSegment::Header);
	if (Result.Archive)
	{
		InnerArchive = MoveTemp(Result.Archive);
		PackageFormat = Result.Format;
		return true;
	}
	else
	{
		UE_LOGF(LogEditorDomain, Warning, "%ls could not be read from WorkspaceDomain. Archive Set to Error.",
			*Segments.GetPackagePath().GetDebugName());
		PackageFormat = EPackageFormat::Binary;
		SetError();
		return false;
	}
}

void FEditorDomainReadArchive::Seek(int64 InPos)
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		if (Segments.GetAsyncSource() != FEditorDomainPackageSegments::ESource::Uninitialized)
		{
			WaitForReady();
			check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
			return Seek(InPos);
		}
		Segments.Seek(InPos);
		break;
	case FEditorDomainPackageSegments::ESource::Segments:
		Segments.Seek(InPos);
		Segments.Precache(InPos, 0);
		break;
	case FEditorDomainPackageSegments::ESource::Fallback:
		InnerArchive->Seek(InPos);
		break;
	case FEditorDomainPackageSegments::ESource::Closed:
		break;
	default:
		checkNoEntry();
		break;
	}
}

int64 FEditorDomainReadArchive::Tell()
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Fallback:
		return InnerArchive->Tell();
	default:
		return Segments.Tell();
	}
}

int64 FEditorDomainReadArchive::TotalSize()
{
	WaitForReady();
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Fallback:
		return InnerArchive->TotalSize();
	default:
		return Segments.TotalSize();
	}
}

bool FEditorDomainReadArchive::Close()
{
	Segments.Close();
	InnerArchive.Reset();
	return true;
}

void FEditorDomainReadArchive::Serialize(void* V, int64 Length)
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		WaitForReady();
		check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
		Serialize(V, Length);
		break;
	case FEditorDomainPackageSegments::ESource::Segments:
		if (!Segments.Serialize(V, Length, IsError()))
		{
			SetError();
		}
		break;
	case FEditorDomainPackageSegments::ESource::Fallback:
		InnerArchive->Serialize(V, Length);
		break;
	case FEditorDomainPackageSegments::ESource::Closed:
		UE_LOGF(LogEditorDomain, Error, "Requested read after close (file=%ls)", *Segments.GetPackagePath().GetDebugName());
		break;
	default:
		checkNoEntry();
		break;
	}
}

FString FEditorDomainReadArchive::GetArchiveName() const
{
	return Segments.GetPackagePath().GetDebugName();
}

void FEditorDomainReadArchive::Flush()
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		WaitForReady();
		check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
		Flush();
		break;
	case FEditorDomainPackageSegments::ESource::Fallback:
		InnerArchive->Flush();
		break;
	default:
		break;
	}
}

void FEditorDomainReadArchive::FlushCache()
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		WaitForReady();
		check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
		FlushCache();
		break;
	case FEditorDomainPackageSegments::ESource::Fallback:
		InnerArchive->FlushCache();
		break;
	default:
		break;
	}
}

bool FEditorDomainReadArchive::Precache(int64 PrecacheOffset, int64 PrecacheSize)
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
	{
		bool bIsReady;
		Segments.Precache(PrecacheOffset, PrecacheSize, &bIsReady);
		return bIsReady;
	}
	case FEditorDomainPackageSegments::ESource::Segments:
	{
		bool bIsReady;
		Segments.Precache(PrecacheOffset, PrecacheSize, &bIsReady);
		return bIsReady;
	}
	case FEditorDomainPackageSegments::ESource::Fallback:
		return InnerArchive->Precache(PrecacheOffset, PrecacheSize);
	default:
		return true;
	}
}

EPackageFormat FEditorDomainReadArchive::GetPackageFormat() const
{
	WaitForReady();
	return PackageFormat;
}

FEditorDomain::EPackageSource FEditorDomainReadArchive::GetPackageSource() const
{
	WaitForReady();
	return Segments.GetSource() == FEditorDomainPackageSegments::ESource::Segments ?
		FEditorDomain::EPackageSource::Editor : FEditorDomain::EPackageSource::Workspace;
}

/** An IAsyncReadRequest SizeRequest that returns a value known at construction time. */
class FAsyncSizeRequestConstant : public IAsyncReadRequest
{
public:
	FAsyncSizeRequestConstant(int64 InSize, FAsyncFileCallBack* InCallback)
		: IAsyncReadRequest(InCallback, true /* bInSizeRequest */, nullptr /* UserSuppliedMemory */)
	{
		Size = InSize;
		SetComplete();
	}

protected:
	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};

/** An IAsyncReadRequest that reads from memory that was already populated. */
class FAsyncReadRequestConstant : public IAsyncReadRequest
{
public:
	FAsyncReadRequestConstant(uint8* Data, bool bIsUserSupplied, FAsyncFileCallBack* InCallback)
		: IAsyncReadRequest(InCallback, false /* bInSizeRequest */, bIsUserSupplied ? Data : nullptr)
	{
		if (!bIsUserSupplied)
		{
			Memory = Data;
		}
		check(Memory);
		SetComplete();
	}

	virtual ~FAsyncReadRequestConstant()
	{
		if (Memory)
		{
			// this can happen with a race on cancel, it is ok, they didn't take the memory, free it now
			if (!bUserSuppliedMemory)
			{
				FMemory::Free(Memory);
			}
			Memory = nullptr;
		}
	}

protected:
	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};


FEditorDomainAsyncReadFileHandle::FEditorDomainAsyncReadFileHandle(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
	const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource,
	UE::DerivedData::EPriority InPriority)
	: Segments(InLocks, InPackagePath, InPackageSource, InPriority)
{
}

void FEditorDomainAsyncReadFileHandle::OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response)
{
	Segments.OnRecordRequestComplete(MoveTemp(Response),
		[this](bool bValid) { CreateSegmentData(bValid); },
		[this](FEditorDomain& EditorDomain) { return TryCreateFallbackData(EditorDomain); }
	);
}

EPackageFormat FEditorDomainAsyncReadFileHandle::GetPackageFormat() const
{
	if (Segments.GetSource() == FEditorDomainPackageSegments::ESource::Uninitialized)
	{
		Segments.WaitForReady();
	}
	return PackageFormat;
}

FEditorDomain::EPackageSource FEditorDomainAsyncReadFileHandle::GetPackageSource() const
{
	if (Segments.GetSource() == FEditorDomainPackageSegments::ESource::Uninitialized)
	{
		Segments.WaitForReady();
	}
	return Segments.GetSource() == FEditorDomainPackageSegments::ESource::Segments ?
		FEditorDomain::EPackageSource::Editor : FEditorDomain::EPackageSource::Workspace;
}

void FEditorDomainAsyncReadFileHandle::CreateSegmentData(bool bValid)
{
	PackageFormat = EPackageFormat::Binary;
}

bool FEditorDomainAsyncReadFileHandle::TryCreateFallbackData(FEditorDomain& EditorDomain)
{
	IPackageResourceManager& Workspace = *EditorDomain.Workspace;
	FOpenAsyncPackageResult Result = Workspace.OpenAsyncReadPackage(Segments.GetPackagePath(), FBulkDataCookedIndex::Default, EPackageSegment::Header);
	check(Result.Handle);
	InnerArchive = MoveTemp(Result.Handle);
	PackageFormat = Result.Format;
	return true;
}

IAsyncReadRequest* FEditorDomainAsyncReadFileHandle::SizeRequest(FAsyncFileCallBack* CompleteCallback)
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		Segments.WaitForReady();
		check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
		return SizeRequest(CompleteCallback);
	case FEditorDomainPackageSegments::ESource::Segments:
		return new FAsyncSizeRequestConstant(Segments.TotalSize(), CompleteCallback);
	case FEditorDomainPackageSegments::ESource::Fallback:
		return InnerArchive->SizeRequest(CompleteCallback);
	default:
		checkNoEntry();
		return nullptr;
	}
}

IAsyncReadRequest* FEditorDomainAsyncReadFileHandle::ReadRequest(int64 Offset, int64 BytesToRead,
	EAsyncIOPriorityAndFlags PriorityAndFlags, FAsyncFileCallBack* CompleteCallback,
	uint8* UserSuppliedMemory)
{
	switch (Segments.GetSource())
	{
	case FEditorDomainPackageSegments::ESource::Uninitialized:
		Segments.WaitForReady();
		check(Segments.GetSource() != FEditorDomainPackageSegments::ESource::Uninitialized);
		return ReadRequest(Offset, BytesToRead, PriorityAndFlags, CompleteCallback, UserSuppliedMemory);
	case FEditorDomainPackageSegments::ESource::Segments:
	{
		// EDITOR_DOMAIN_TODO: Make a FAsyncReadRequest that accepts a TFuture instead of synchronously reading upfront
		Segments.Precache(Offset, 0);
		Segments.Seek(Offset);
		uint8* Memory = UserSuppliedMemory;
		if (!Memory)
		{
			Memory = (uint8*)FMemory::Malloc(BytesToRead);
		}
		if (!Segments.Serialize(Memory, BytesToRead, false /* bIsError */))
		{
			if (!UserSuppliedMemory)
			{
				FMemory::Free(Memory);
			}
			return nullptr;
		}
		return new FAsyncReadRequestConstant(Memory, UserSuppliedMemory != nullptr, CompleteCallback);
	}
	case FEditorDomainPackageSegments::ESource::Fallback:
		return InnerArchive->ReadRequest(Offset, BytesToRead, PriorityAndFlags, CompleteCallback, UserSuppliedMemory);
	default:
		checkNoEntry();
		return nullptr;
	}
}

bool FEditorDomainAsyncReadFileHandle::UsesCache()
{
	return true;
}
