// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DerivedDataRequestTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "IO/IoHash.h"
#include "Misc/Optional.h"
#include "atomic"
#include "Async/AsyncFileHandle.h"
#include "Containers/Array.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValueId.h"
#include "EditorDomain/EditorDomain.h"
#include "HAL/Platform.h"
#include "Memory/SharedBuffer.h"
#include "Misc/PackagePath.h"
#include "Serialization/Archive.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "UObject/PackageResourceManager.h"

template <typename FuncType> class TUniqueFunction;

namespace UE { namespace DerivedData { struct FCacheGetResponse; } }

/**
 * Helper class for archive classes that server either an EditorDomain version of a package
 * or the WorkspaceDomain version. Starts with a request to the Cache to load the metadata for
 * an EditorDomainPackage. If the request fails, goes dormant and the owner class falls
 * back to workspace domain. If the request succeeds, it makes all Attachments from the
 * CacheRecord available as individually fetched segments.
 */
class FEditorDomainPackageSegments
{
public:
	enum class ESource : uint8
	{
		Uninitialized,
		Segments,
		Fallback,
		Closed
	};

	FEditorDomainPackageSegments(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
		const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource,
		UE::DerivedData::EPriority Priority);
	~FEditorDomainPackageSegments();

	/** Get the request owner for requests that will feed this archive. */
	UE::DerivedData::IRequestOwner& GetRequestOwner() { return RequestOwner; }
	/** Callback from the request of the Record; set whether we're reading from EditorDomain bytes or WorkspaceDomain archive. */
	void OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response,
		TUniqueFunction<void(bool bValid)>&& CreateSegmentData,
		TUniqueFunction<bool(FEditorDomain& EditorDomain)>&& TryCreateFallbackData);

	/**
	 * Get the cache of GetAsyncSource as seen on the Interface thread.
	 * If Uninitialized, caller should early exit, WaitForReady, or call GetAsyncSource.
	 */
	ESource GetSource() const;

	/** Get the ESource that is an atomic used from all threads. */
	std::atomic<ESource>& GetAsyncSource();

	/** Wait for the handle to call OnCacheRequestComplete and make the size and list of Segments available. */
	void WaitForReady() const;

	/** Set Pos to the given value. Caller is responsible for ensuring in range. Can be called before ready. */
	void Seek(int64 InPos);

	/** Report the current value of Pos. Can be called before ready. */
	int64 Tell() const;

	/**
	 * Report the combined size of all segments.
	 * Must not be called unless GetSource is ESource::Segments.
	 */
	int64 TotalSize() const;

	/** Return the PackagePath. Can be called before ready. */
	const FPackagePath& GetPackagePath() const;

	/** Cancel any requests, wait for them to complete, release memory. */
	void Close();

	/**
	 * Copy Length bytes from current Pos into V.
	 * Must not be called unless GetSource is ESource::Segments.
	 */
	bool Serialize(void* V, int64 Length, bool bIsError);

	/**
	 * Ensure requests have been sent for segments touching the given range. Out of range requests are ignored.
	 * Must not be called unless GetSource is ESource::Segments.

	 * @param bOutIsReady Set to true iff all segments touching the range have completd their request.
	 */
	void Precache(int64 InStart, int64 InSize, bool* bOutIsReady = nullptr);

private:
	/**
	 * A segment of bytes that is sourced from an attachment in the CacheRecord and
	 * is requested on demand.
	 */
	struct FSegment
	{
		FSegment(const UE::DerivedData::FValueId& InValueId, uint64 InStart)
			: ValueId(InValueId)
			, Start(InStart)
		{}

		/** Attachment ID to request from cache for this segment. Read-only. */
		UE::DerivedData::FValueId ValueId;
		/** The request that will be (has been) sent for the bytes from cache. Interface-only. */
		TOptional<UE::DerivedData::FRequestOwner> RequestOwner;
		/** Offset from the start of the entire archive. Read-only. */
		uint64 Start;
		/** Value bytes. Callback-only before bAsyncComplete, Interface-only after. */
		FSharedBuffer Data;
		/** Interface thread cache of bAsyncComplete. Interface-only. */
		bool bComplete = false;
		/** Set to true when the callback for the cache request has finished writing Data. */
		std::atomic<bool> bAsyncComplete{ false };
	};

	/** Map offset to segment index. Start must be in range 0 <= Start < Size. Must be called after Segments assigned. */
	int32 GetSegmentContainingOffset(uint64 Start) const;

	/**
	 * Make sure requests are sent for segments touching [Start, End); optionally wait for requests to finish.
	 * Must not be called unless GetSource is ESource::Segments. Caller must ensure valid range Start <= End <= Size.
	 *
	 * @param bOutIsReady Set to true if all segments touching the range have finished their request.
	 * @return Address of the first segment touching the range.
	 */
	FSegment* EnsureSegmentRange(uint64 Start, uint64 End, bool bWait, bool& bOutIsReady);

	/**
	 * Send the request for the given segment; does not check if it has already been sent.
	 * Losing Race values are ignored.
	 */
	void SendSegmentRequest(FSegment& Segment);

	/**
	 * Lock from the EditorDomain to allow shutting down this archive if callback outlives editor domain.
	 * Pointer is read-only, *pointer has internal lock.
	 */
	TRefCountPtr<FEditorDomain::FLocks> EditorDomainLocks;
	/** Initial Request for the list of segments. Read-only. */
	UE::DerivedData::FRequestOwner RequestOwner;
	/** PackagePath for fallback to WorkspaceDomain. Read-only. */
	FPackagePath PackagePath;
	/** Most recently used Segment in Serialize. Interface-only. */
	FSegment* MRUSegment = nullptr;
	/** Offset to next use in Serialize. Interface-only. */
	int64 Pos = 0;
	/**
	 * Interface-thread cache of AsyncSource. Used for performance and to trigger copying of async data on first
	 * read after Request complete. Interface-only.
	 */
	mutable ESource Source = ESource::Uninitialized;
	/**
	 * PackageSource for read/write of whether this Package comes from editor domain or workspace domain.
	 * Read-only, *pointer requires EditorDomainLocks.Lock.
	 */
	TRefCountPtr<FEditorDomain::FPackageSource> PackageSource;
	/** EditorDomainHash for requesting values, copied from PackageSource. Read-only. */
	FIoHash EditorDomainHash;

	/**
	 * Data read from the EditorDomain; each Segment is requested individually.
	 * Array allocation is Callback-only before Request returns, read-only after.
	 * Thread properties of elements are mixed; see FSegment.
	 */
	TArray<FSegment> Segments;
	/** Size if Source is Segments, 0 otherwise. Callback-only before Request returns, read-only after. */
	int64 Size = 0;
	/** 
	 * Specifies which data to use based on whether EditorDomain data exists.
	 * Set by Close to shutdown both Callback and Interface activity.
	 * Interface thread polls this in some cases to check whether it should wait for WaitForReady.
	 */
	std::atomic<ESource> AsyncSource{ ESource::Uninitialized };
	/**
	 * Instructs the Callback thread to immediately send a SegmentRequest for the given offset.
	 * Written by Interface thread. Callback thread will use it if its set in time(shortly after AsyncSource is set),
	 * otherwise will ignore or use stale value.
	 */
	std::atomic<int64> InitialRequestOffset{ -1 }; 
};

/**
 * An Archive that asynchronously waits for the cache request to complete, and reads either from the returned cache bytes
 * or from the fallback WorkspaceDomain Archive for the given PackagePath.
 *
 * This class is a serialization archive rather than a full archive; it overrides the serialization functions used by
 * LinkerLoad and BulkData but does not override all of the functions used by general Archive use as FArchiveProxy does.
 */
class FEditorDomainReadArchive : public FArchive
{
public:
	FEditorDomainReadArchive(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
		const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource,
		UE::DerivedData::EPriority Priority);

	/** Get the request owner for requests that will feed this archive. */
	UE::DerivedData::IRequestOwner& GetRequestOwner() { return Segments.GetRequestOwner(); }
	/** Callback from the request of the Record; set whether we're reading from EditorDomain bytes or WorkspaceDomain archive. */
	void OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response);
	/** Get the PackageFormat, which depends on the domain the data is read from. */
	EPackageFormat GetPackageFormat() const;
	FEditorDomain::EPackageSource GetPackageSource() const;

	// FArchive interface
	virtual void Seek(int64 InPos) override;
	virtual int64 Tell() override;
	virtual int64 TotalSize() override;
	virtual bool Close() override;
	virtual void Serialize(void* V, int64 Length) override;
	virtual FString GetArchiveName() const override;
	virtual void Flush() override;
	virtual void FlushCache() override;
	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;

private:
	void WaitForReady() const;
	void CreateSegmentData(bool bValid);
	bool TryCreateFallbackData(FEditorDomain& EditorDomain);

	FEditorDomainPackageSegments Segments;
	TUniquePtr<FArchive> InnerArchive;
	EPackageFormat PackageFormat = EPackageFormat::Binary;
};

/**
 * An IAsyncReadFileHandle that asynchronously waits for the cache request to complete, and reads either from the
 * returned cache bytes or from the fallback WorkspaceDomain Archive for the given PackagePath.
 */
class FEditorDomainAsyncReadFileHandle : public IAsyncReadFileHandle
{
public:
	FEditorDomainAsyncReadFileHandle(const TRefCountPtr<FEditorDomain::FLocks>& InLocks,
		const FPackagePath& InPackagePath, const TRefCountPtr<FEditorDomain::FPackageSource>& InPackageSource,
		UE::DerivedData::EPriority Priority);

	/** Get the request owner for requests that will feed this archive. */
	UE::DerivedData::IRequestOwner& GetRequestOwner() { return Segments.GetRequestOwner(); }
	/** Callback from the request of the Record; set whether we're reading from EditorDomain bytes or WorkspaceDomain archive. */
	void OnRecordRequestComplete(UE::DerivedData::FCacheGetResponse&& Response);
	/** Get the PackageFormat, which depends on the domain the data is read from. */
	EPackageFormat GetPackageFormat() const;
	FEditorDomain::EPackageSource GetPackageSource() const;

	// IAsyncReadFileHandle interface
	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override;
	virtual IAsyncReadRequest* ReadRequest(int64 Offset, int64 BytesToRead,
		EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal, FAsyncFileCallBack* CompleteCallback = nullptr,
		uint8* UserSuppliedMemory = nullptr) override;
	virtual bool UsesCache();

private:
	void CreateSegmentData(bool bValid);
	bool TryCreateFallbackData(FEditorDomain& EditorDomain);

	FEditorDomainPackageSegments Segments;
	TUniquePtr<IAsyncReadFileHandle> InnerArchive;
	EPackageFormat PackageFormat = EPackageFormat::Binary;
};