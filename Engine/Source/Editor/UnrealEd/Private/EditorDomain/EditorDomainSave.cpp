// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDomainSave.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "Commandlets/EditorDomainSaveCommandlet.h"
#include "DerivedDataCacheInterface.h"
#include "Dom/JsonObject.h"
#include "EditorDomain/EditorDomainUtils.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/RunnableThread.h"
#include "IPAddress.h"
#include "Misc/DateTime.h"
#include "Misc/PackagePath.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Serialization/Archive.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "UnrealEdMisc.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogEditorDomainSave);

/** A header for network messages with magic number identifier, message enum, and payload size. */
template<typename MessageEnum, uint32 ExpectedMagic, uint32 MessageStart, uint32 MessageEnd, uint32 MaxSize>
struct TMessageHeader
{
	uint32 Magic;
	uint32 Size;
	uint32 MessageType;

	void Initialize(MessageEnum InMessageType)
	{
		Magic = ExpectedMagic;
		SetMessageType(InMessageType);
	}
	void SetMessageType(MessageEnum InMessageType)
	{
		MessageType = static_cast<uint32>(InMessageType);
	}
	MessageEnum GetMessageType() const
	{
		return static_cast<MessageEnum>(MessageType);
	}
	FString ToString() const
	{
		return FString::Printf(TEXT("(Magic=%u, MessageType=%u, Size=%u)"), Magic, MessageType, Size);
	}
	bool IsValid() const
	{
		return Magic == ExpectedMagic &&
			MessageStart <= MessageType && MessageType < MessageEnd &&
			Size <= MaxSize;
	}
	friend FArchive& operator<<(FArchive& Ar, TMessageHeader& Header)
	{
		return Ar << Header.Magic << Header.Size << Header.MessageType;
	}
};

/** A ScopeLock for FPlatformProcess::NewInterprocessSynchObject types. */
struct FProcessSemaphoreScopeLock
{
	FProcessSemaphoreScopeLock(FGenericPlatformProcess::FSemaphore* InSemaphore, float TimeOutSeconds)
		: Semaphore(InSemaphore)
	{
		if (TimeOutSeconds < 0)
		{
			Semaphore->Lock();
			bLocked = true;
		}
		else
		{
			float NanosecondsPerSecond = 1e9f;
			bLocked = Semaphore->TryLock(static_cast<uint64>(TimeOutSeconds * NanosecondsPerSecond));
		}
	}

	bool IsLocked() const
	{
		return bLocked;
	}

	void Unlock()
	{
		if (bLocked)
		{
			Semaphore->Unlock();
		}
	}

	~FProcessSemaphoreScopeLock()
	{
		Unlock();
	}

private:
	FGenericPlatformProcess::FSemaphore* Semaphore;
	bool bLocked = false;
};

namespace UE
{
namespace EditorDomainSave
{

/** MessageTypes that can be sent between EditorDomainSaveServer to EditorDomainSaveClient. */
enum class EMessageType : uint32
{
	MessageTypeMagicNumber = 0xED1104,
	MessageTypeStart = MessageTypeMagicNumber,

	Save = MessageTypeStart,
	CloseConnection,

	MessageTypeEnd,
};
constexpr uint32 MessageMagicNumber = 0x854EBA92;
constexpr int32 MaxMessageSize = 1000000 * 10;

typedef TMessageHeader<EMessageType, MessageMagicNumber,
	(uint32)EMessageType::MessageTypeStart, (uint32)EMessageType::MessageTypeEnd,
	MaxMessageSize> FMessageHeader;

/** Ids used in the JSON settings file. */
namespace SettingNames
{
	const TCHAR* const ListenPortFieldName = TEXT("ListenPort");
	const TCHAR* const ProcessIdFieldName = TEXT("ProcessId");
}

/** The string identifier for the OS to Find or FindAndCreate the interprocess synchronization object. */
FString GetLockName();
/**
 * Create the inprocess handle to the interprocess synchronization object.
 * Caller must call DeleteInterprocessSynchObject when done.
 */
FPlatformProcess::FSemaphore* CreateProcessLock();
/** Read the ServerSettings JSON file from disk into a JsonObject. */
bool TryReadServerSettings(TSharedPtr<FJsonObject>& OutRootObject, FDateTime& OutServerSettingsTimestamp);
/** Write the ServerSettings JSON file to disk from a JsonObject */
bool TryWriteServerSettings(TSharedPtr<FJsonObject>& RootObject);
/** The full path to the intermediate JSON file used to store settings for the active EditorDomainSaveServer. */
FString GetServerSettingsFilename();
/** Get the timestamp of the ServerSettings JSON file, used to respond to updates. */
void GetServerSettingsTimeStamp(bool& bOutExists, FDateTime& OutModificationTime);
/** Delete the server settings file; indicates that a new server will need to be created. */
bool TryDeleteSettingsFile();

namespace Constants
{
	/** Timeout in seconds to wait for the interprocess lock before giving up trying again on the next tick. */
	constexpr double ProcessLockTimeoutSecondsWhenPolling = .005;
	/* Timeout in seconds to wait for the interprocess lock before aborting the server. */
	constexpr double ProcessLockTimeoutSecondsWhenBlocking = 60.;

	/** Timeout in seconds to wait for the server to respond to its duties, (write port, read socket). */
	constexpr double ServerUnresponsiveTimeoutSeconds = 10 * 60.;
	/** Cooldown in seconds before CollectGarbage should be called on the server after it becomes idle. */
	constexpr double CollectGarbageIdleCooldownSeconds = 10.;
	/** Cooldown in seconds before CollectGarbage should be called again on the server when it is active. */
	constexpr double CollectGarbageActiveCooldownSeconds = 60. * 10;
	/** Duration in seconds the server should sleep before checking for messages when it is idle. */
	constexpr double ServerIdleSleepPeriodSeconds = 1.;
	/** Max duration in seconds of the server's single tick of package saving. */
	constexpr double ServerPackageLoadBudgetSeconds = .5;
	/** Cooldown in seconds before the Server will shutdown when it has no clients. */
	constexpr double ServerAbdicationCooldownSeconds = 10.;

	/** Timeout in seconds to wait for the socket to connect before rechecking whether the process exists. */
	constexpr double SocketUnresponsiveTimeoutSeconds = 5.;
	/** Max duration in seconds of the clients wait for queued messages to send when it is shutting down. */
	constexpr double ClosingFlushBudgetSeconds = 10.;
	/** Cooldown in seconds between attempts to create the Server process. */
	constexpr double ProcessCreationCooldownSeconds = 10.;
	/** Cooldown in seconds before logging another connection warning. */
	constexpr double ConnectionWarningCooldownSeconds = 60.;
	/** Duration in seconds the client should sleep in its async run before trying another Tick. */
	constexpr double RunCommunicationSleepPeriodSeconds = .001;
}

}
}


FEditorDomainSaveServer::FEditorDomainSaveServer()
{
	AssetRegistry = IAssetRegistry::Get();
}

FEditorDomainSaveServer::~FEditorDomainSaveServer()
{
	Shutdown();
}

int32 FEditorDomainSaveServer::Run()
{
	if (!TryInitialize())
	{
		Shutdown();
		return 1;
	}

	while (PollShouldRun())
	{
		bool bIsIdle = true;
		TickPendingPackages(bIsIdle);
		PollIncomingConnections(bIsIdle);
		PollConnections(bIsIdle);
		TickMaintenance(bIsIdle);
	}

	Shutdown();
	return 0;
}


bool FEditorDomainSaveServer::TryInitialize()
{
	using namespace UE::EditorDomainSave;

	const TCHAR* CommandLine = FCommandLine::Get();
	if (!FParse::Value(CommandLine, TEXT("-creatorprocessid="), CreatorProcessId))
	{
		CreatorProcessId = 0;
	}
	bIsResidentServer = FParse::Param(CommandLine, TEXT("EditorDomainSaveResident"));

	ProcessLock = CreateProcessLock();
	if (!ProcessLock)
	{
		UE_LOGF(LogEditorDomainSave, Error, "Could not create InterprocessSynchObject %ls, "
			"EditorDomain saving is not enabled.", *GetLockName());
		return false;
	}

	FProcessSemaphoreScopeLock ProcessScopeLock(ProcessLock, Constants::ProcessLockTimeoutSecondsWhenBlocking);
	if (!ProcessScopeLock.IsLocked())
	{
		UE_LOGF(LogEditorDomainSave, Error, "Could not lock InterprocessSynchObject %ls for %lf seconds, "
			"aborting creation of EditorDomainSaveServer.",
			*GetLockName(), Constants::ProcessLockTimeoutSecondsWhenBlocking);
		return false;
	}

	TSharedPtr<FJsonObject> SettingsObject;
	if (!TryRefreshAuthority(nullptr /* bOutOwnsSettingFile */, &SettingsObject, &ListenPort))
	{
		// Log statement sent by TryRefreshAuthority
		return false;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	if (!SocketSubsystem)
	{
		UE_LOGF(LogEditorDomainSave, Error, "No SocketSubsystem, aborting creation of EditorDomainServer.");
		return false;
	}
	TSharedRef<FInternetAddr> ListenAddr = SocketSubsystem->GetLocalBindAddr(*GLog);
	ListenAddr->SetPort(ListenPort);
	ListenSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FEditorDomainSaveServer tcp-listen"),
		ListenAddr->GetProtocolType());
	if (!ListenSocket)
	{
		UE_LOGF(LogEditorDomainSave, Error, "CreateSocket failed, aborting creation of EditorDomainServer.");
		return false;
	}

	ListenSocket->SetReuseAddr();
	ListenSocket->SetNonBlocking(true);
	if (!ListenSocket->Bind(*ListenAddr))
	{
		UE_LOGF(LogEditorDomainSave, Error,
			"Failed to bind listen socket %ls, aborting creation of EditorDomainServer.",
			*ListenAddr->ToString(true));
		return false;
	}
	if (!ListenSocket->Listen(16))
	{
		UE_LOGF(LogEditorDomainSave, Warning,
			"Failed to listen to socket %ls, aborting creation of EditorDomainServer.",
			*ListenAddr->ToString(true));
		return false;
	}
	ListenPort = ListenSocket->GetPortNo();
	ListenAddr->SetPort(ListenPort);
	UE_LOGF(LogEditorDomainSave, Display, "EditorDomainSaveServer is listening for client connections on %ls.",
		*ListenAddr->ToString(true));

	SettingsObject->SetNumberField(SettingNames::ListenPortFieldName, ListenPort);
	if (!TryWriteServerSettings(SettingsObject))
	{
		UE_LOGF(LogEditorDomainSave, Warning,
			"Failed to write ServerSettings file %ls, aborting creation of EditorDomainServer.",
			*GetServerSettingsFilename());
		return false;
	}

	IAssetRegistry::Get()->SearchAllAssets(false /* bSynchronousSearch */);
	LastGarbageTime = FPlatformTime::Seconds();
	return true;
}

bool FEditorDomainSaveServer::TryRefreshAuthority(bool* bOutOwnsSettingsFile, TSharedPtr<FJsonObject>* OutRootObject, 
	int32* OutListenPort)
{
	using namespace UE::EditorDomainSave;

	if (bOutOwnsSettingsFile)
	{
		*bOutOwnsSettingsFile = false;
	}

	bool bIsInitialAcquisition = ListenPort == 0;
	const TCHAR* ErrorActionDesc = bIsInitialAcquisition
		? TEXT("aborting creation")
		: TEXT("relinquishing ownership and shutting down");

	int32 ServerProcessId = 0;
	int32 LocalListenPort = 0;
	TSharedPtr<FJsonObject> RootObject;
	if (bIsInitialAcquisition && bIsResidentServer)
	{
		ServerProcessId = FPlatformProcess::GetCurrentProcessId();
		RootObject = MakeShared<FJsonObject>();
		RootObject->SetNumberField(SettingNames::ProcessIdFieldName, ServerProcessId);
	}
	else
	{
		TryReadServerSettings(RootObject, SettingsTimestamp);
		if (!RootObject)
		{
			UE_LOGF(LogEditorDomainSave, Error, "Could not read settings file %ls, %ls.",
				*GetServerSettingsFilename(), ErrorActionDesc);
			return false;
		}
		if (!RootObject->TryGetNumberField(SettingNames::ProcessIdFieldName, ServerProcessId))
		{
			UE_LOGF(LogEditorDomainSave, Error, "Could not read %ls out of settings file %ls, %ls.",
				SettingNames::ProcessIdFieldName, *GetServerSettingsFilename(), ErrorActionDesc);
				return false;
		}
		int32 Value;
		if (RootObject->TryGetNumberField(SettingNames::ListenPortFieldName, Value))
		{
			LocalListenPort = Value;
		}
	}

	if (ServerProcessId != FPlatformProcess::GetCurrentProcessId())
	{
		UE_LOGF(LogEditorDomainSave, Error, "EditorDomainSave:Server Another process %d is registered as "
			"the singleton EditorDomainSaveServer in settings file %ls, this process %d is %ls.",
			ServerProcessId, *GetServerSettingsFilename(), FPlatformProcess::GetCurrentProcessId(), ErrorActionDesc);
		return false;
	}
	if (bOutOwnsSettingsFile)
	{
		*bOutOwnsSettingsFile = true;
	}

	if (!bIsInitialAcquisition && LocalListenPort != ListenPort)
	{
		UE_LOGF(LogEditorDomainSave, Error,
			"Listen port has changed from %d to %d. Change in listen port is unsupported; %ls.",
			ListenPort, LocalListenPort, ErrorActionDesc);
		return false;
	}
	if (OutListenPort)
	{
		*OutListenPort = LocalListenPort;
	}
	if (OutRootObject)
	{
		*OutRootObject = RootObject;
	}
	return true;
}

void FEditorDomainSaveServer::Shutdown()
{
	ShutdownSocket();
	if (ProcessLock)
	{
		FPlatformProcess::DeleteInterprocessSynchObject(ProcessLock);
		ProcessLock = nullptr;
	}
	GetDerivedDataCacheRef().WaitForQuiescence(true);
}

void FEditorDomainSaveServer::ShutdownSocket()
{
	ClientConnections.Empty();
	if (ListenSocket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		check(SocketSubsystem);
		ListenSocket->Close();
		SocketSubsystem->DestroySocket(ListenSocket);
		ListenSocket = nullptr;
	}
}

bool FEditorDomainSaveServer::TryAbdicate()
{
	using namespace UE::EditorDomainSave;

	if (bIsResidentServer)
	{
		return false;
	}
	FProcessSemaphoreScopeLock ProcessScopeLock(ProcessLock, static_cast<float>(Constants::ProcessLockTimeoutSecondsWhenPolling));
	if (!ProcessScopeLock.IsLocked())
	{
		return false;
	}
	if (HasExpectedConnections())
	{
		return false;
	}

	UE_LOGF(LogEditorDomainSave, Display,
		"EditorDomainSaveServer has no remaining connections and is shutting down.");
	bAbdicated = true;
	ShutdownSocket();
	bool bOwnsSettingsFile;
	TryRefreshAuthority(&bOwnsSettingsFile);
	if (bOwnsSettingsFile && !TryDeleteSettingsFile())
	{
		UE_LOGF(LogEditorDomainSave, Warning,
			"Unable to delete server settings file %ls. Future clients will have to delete the file.",
			*GetServerSettingsFilename());
	}
	return true;
}

bool FEditorDomainSaveServer::IsAbdicated() const
{
	return bAbdicated;
}

bool FEditorDomainSaveServer::HasExpectedConnections() const
{
	if (!bHasEverConnected && CreatorProcessId)
	{
		FProcHandle Handle = FPlatformProcess::OpenProcess(CreatorProcessId);
		ON_SCOPE_EXIT{ FPlatformProcess::CloseProc(Handle); };
		if (Handle.IsValid())
		{
			// We are still waiting for the creator process to connect
			return true;
		}
	}
	return ClientConnections.Num() > 0;
}

bool FEditorDomainSaveServer::PollShouldRun()
{
	using namespace UE::EditorDomainSave;

	if (IsAbdicated() || ListenSocket == nullptr)
	{
		return false;
	}
	FDateTime LocalSettingsTimestamp;
	bool bExists;
	GetServerSettingsTimeStamp(bExists, LocalSettingsTimestamp);
	if (!bExists || SettingsTimestamp != LocalSettingsTimestamp)
	{
		FProcessSemaphoreScopeLock ProcessScopeLock(ProcessLock, static_cast<float>(Constants::ProcessLockTimeoutSecondsWhenPolling));
		if (ProcessScopeLock.IsLocked())
		{
			bool bOwnsSettingsFile;
			if (!TryRefreshAuthority(&bOwnsSettingsFile))
			{
				ShutdownSocket();
				if (bOwnsSettingsFile && !TryDeleteSettingsFile())
				{
					UE_LOGF(LogEditorDomainSave, Warning,
						"Unable to delete server settings file %ls. Future clients will have to delete the file.",
						*GetServerSettingsFilename());
				}
				return false;
			}
		}
	}
	return true;
}

void FEditorDomainSaveServer::PollIncomingConnections(bool& bInOutIsIdle)
{
	bool bReadReady;
	while (ListenSocket->WaitForPendingConnection(bReadReady, FTimespan::FromSeconds(0)) && bReadReady)
	{
		FSocket* ClientSocket = ListenSocket->Accept(TEXT("Client Connection"));
		if (!ClientSocket)
		{
			UE_LOGF(LogEditorDomainSave, Warning,
				"Listen socket received a pending connection event but ListenSocket->Accept " "failed to create a ClientSocket.");
		}
		else
		{
			ClientSocket->SetNonBlocking(true);
			bHasEverConnected = true;
			bInOutIsIdle = false;
			ClientConnections.Add(TRefCountPtr<FClientConnection>(new FClientConnection(ClientSocket)));
		}
	}
}

void FEditorDomainSaveServer::PollConnections(bool& bInOutIsIdle)
{
	for (int Index = 0; Index < ClientConnections.Num(); ++Index)
	{
		TRefCountPtr<FClientConnection>& ClientConnection = ClientConnections[Index];
		bool bStillAlive;
		bool bClientIsIdle;
		ClientConnection->Poll(*this, bClientIsIdle, bStillAlive);
		if (!bStillAlive)
		{
			bInOutIsIdle = false;
			ClientConnections.RemoveAtSwap(Index);
		}
		else if (!bClientIsIdle)
		{
			bInOutIsIdle = false;
		}
	}
}

void FEditorDomainSaveServer::TickMaintenance(bool bIsIdle)
{
	using namespace UE::EditorDomainSave;

	SetIdle(bIsIdle);

	double CurrentTime = FPlatformTime::Seconds();
	if (bIsIdle)
	{
		if (!HasExpectedConnections() && CurrentTime - IdleStartTime > Constants::ServerAbdicationCooldownSeconds)
		{
			if (TryAbdicate())
			{
				return;
			}
		}
	}

	double CollectGarbageCooldownSeconds = bIsIdle ?
		Constants::CollectGarbageIdleCooldownSeconds :
		Constants::CollectGarbageActiveCooldownSeconds;
	bool bCollectedGarbageAfterIdle = bIsIdle && LastGarbageTime >= IdleStartTime;
	if (!bCollectedGarbageAfterIdle && CurrentTime - LastGarbageTime > CollectGarbageCooldownSeconds)
	{
		CollectGarbage(RF_NoFlags);
		LastGarbageTime = FPlatformTime::Seconds();
	}

	if (bIsIdle)
	{
		FPlatformProcess::Sleep(Constants::ServerIdleSleepPeriodSeconds);
	}
}

void FEditorDomainSaveServer::SetIdle(bool bIsIdle)
{
	bool bWasIdle = IdleStartTime != 0.;
	if (bIsIdle == bWasIdle)
	{
		return;
	}
	if (bIsIdle)
	{
		IdleStartTime = FPlatformTime::Seconds();
	}
	else
	{
		IdleStartTime = 0.;
	}
}


void FEditorDomainSaveServer::ProcessMessage(FClientConnection& ClientConnection,
	UE::EditorDomainSave::EMessageType MessageType, const TArray<uint8>& MessageBuffer, bool& bOutStillAlive)
{
	using namespace UE::EditorDomainSave;

	bOutStillAlive = true;

	switch (MessageType)
	{
	case EMessageType::Save:
	{
		FMemoryReader Reader(MessageBuffer);
		int32 NumPackages = -1;
		const int32 MinSizePerPackage = sizeof(int32) + 2;
		Reader << NumPackages;
		if (Reader.IsError() || NumPackages < 0 || NumPackages > MessageBuffer.Num() / MinSizePerPackage)
		{
			UE_LOGF(LogEditorDomainSave, Warning, "EditorDomainServer received corrupt save message with invalid "
				"NumPackages %d from client, closing the connection", NumPackages);
			bOutStillAlive = false;
		}
		else
		{
			for (int n = 0; n < NumPackages; ++n)
			{
				FString PackageName;
				Reader << PackageName;
				if (Reader.IsError())
				{
					UE_LOGF(LogEditorDomainSave, Warning, "EditorDomainServer received corrupt save message with "
						"invalid PackageName instance %d from client, closing the connection", n);
					bOutStillAlive = false;
					break;
				}
				PendingPackageNames.Add(MoveTemp(PackageName));
			}
		}
		break;
	}
	case EMessageType::CloseConnection:
		UE_LOGF(LogEditorDomainSave, Display,
			"EditorDomainServer received CloseConnection message from client, closing the connection");
		bOutStillAlive = false;
		break;
	default:
		UE_LOGF(LogEditorDomainSave, Warning,
			"EditorDomainServer received invalid messagetype %u from client, closing the connection",
			static_cast<uint32>(MessageType));
		bOutStillAlive = false;
		break;
	}
}

void FEditorDomainSaveServer::TickPendingPackages(bool& bInOutIsIdle)
{
	using namespace UE::EditorDomainSave;

	if (PendingPackageNames.Num() == 0)
	{
		return;
	}
	bInOutIsIdle = false;
	double StartTime = FPlatformTime::Seconds();

	if (!bHasInitializedAssetRegistry)
	{
		IAssetRegistry::Get()->SearchAllAssets(true /* bSynchronousSearch */);
		bHasInitializedAssetRegistry = true;
	}

	while (PendingPackageNames.Num() > 0)
	{
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - StartTime > Constants::ServerPackageLoadBudgetSeconds)
		{
			break;
		}
		FString PackageName = PendingPackageNames.PopFrontValue();
		FString ErrorMessage;
		if (!TrySavePackage(PackageName, ErrorMessage))
		{
			UE_LOGF(LogEditorDomainSave, Warning, "%ls", *ErrorMessage);
		}
		else
		{
			UE_LOGF(LogEditorDomainSave, Display, "Saved package %ls into the EditorDomain", *PackageName);
		}
	}
}

bool FEditorDomainSaveServer::TrySavePackage(FStringView PackageName, FString& OutErrorMessage)
{
	FPackagePath PackagePath;
	if (!FPackagePath::TryFromPackageName(PackageName, PackagePath) || !PackagePath.IsMountedPath())
	{
		OutErrorMessage = FString::Printf(TEXT("Package %.*s is not mounted. Skipping its save."),
			PackageName.Len(), PackageName.GetData());
		return false;
	}
	return TrySavePackage(PackagePath, OutErrorMessage);
}

bool FEditorDomainSaveServer::TrySavePackage(const FPackagePath& PackagePath, FString& OutErrorMessage)
{
	using namespace UE::EditorDomain;

	FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		OutErrorMessage = FString::Printf(TEXT("Package %s is not mounted. Skipping its save."), *PackagePath.GetDebugName());
		return false;
	}

	// EDITOR_DOMAIN_TODO: Check whether the package already exists in the cache before spending time on it
	UPackage* Package = LoadPackage(nullptr, PackagePath, 0);
	if (!Package)
	{
		OutErrorMessage = FString::Printf(TEXT("Package %s could not be loaded. Skipping its save."),
			*PackagePath.GetDebugName());
		return false;
	}

	if (!UE::EditorDomain::TrySavePackage(Package))
	{
		OutErrorMessage = FString::Printf(TEXT("Could not save package %s."), *PackagePath.GetDebugName());
		return false;
	}
	return true;
}

FEditorDomainSaveServer::FClientConnection::FClientConnection(FSocket* InClientSocket)
	: ClientSocket(InClientSocket)
	, MessageType(UE::EditorDomainSave::EMessageType::Save)
{
}

FEditorDomainSaveServer::FClientConnection::~FClientConnection()
{
	Shutdown();
}

void FEditorDomainSaveServer::FClientConnection::Shutdown()
{
	if (ClientSocket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		check(SocketSubsystem);
		ClientSocket->Close();
		SocketSubsystem->DestroySocket(ClientSocket);
		ClientSocket = nullptr;
	}
}

void FEditorDomainSaveServer::FClientConnection::Poll(FEditorDomainSaveServer& Server,
	bool& bOutIsIdle, bool& bOutStillAlive)
{
	using namespace UE::EditorDomainSave;

	if (!ClientSocket)
	{
		bOutStillAlive = false;
		bOutIsIdle = true;
		return;
	}

	bOutStillAlive = true;
	bOutIsIdle = true;
	if (MessageBuffer.Num() > 0)
	{
		bOutIsIdle = false;
		int32 RemainingSize = MessageBuffer.Num() - BufferOffset;
		int32 BytesRead;
		bool bConnectionAlive = ClientSocket->Recv(MessageBuffer.GetData() + BufferOffset, RemainingSize, BytesRead);
		if (BytesRead > 0)
		{
			check(BytesRead <= RemainingSize);
			BufferOffset += BytesRead;
			if (BufferOffset == MessageBuffer.Num())
			{
				Server.ProcessMessage(*this, MessageType, MessageBuffer, bOutStillAlive);
				MessageBuffer.Empty();
			}
		}
		else if (!bConnectionAlive)
		{
			bOutStillAlive = false;
		}
	}
	else
	{
		FMessageHeader MessageHeader;
		int32 BytesRead;
		bool bConnectionAlive = ClientSocket->Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader),
			BytesRead, ESocketReceiveFlags::Peek);
		check(BytesRead <= sizeof(FMessageHeader));
		if (BytesRead > 0)
		{
			bOutIsIdle = false;
			if (BytesRead == sizeof(FMessageHeader))
			{
				ClientSocket->Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader),
					BytesRead, ESocketReceiveFlags::None);
				check(BytesRead == sizeof(FMessageHeader));
				if (!MessageHeader.IsValid())
				{
					UE_LOGF(LogEditorDomainSave, Warning,
						"EditorDomainServer received invalid message header %ls from client, closing the connection.",
						*MessageHeader.ToString());
					bOutStillAlive = false;
					return;
				}
				MessageBuffer.SetNum(MessageHeader.Size - sizeof(FMessageHeader));
				BufferOffset = 0;
				MessageType = MessageHeader.GetMessageType();

				bool bUnusedIsIdle;
				Poll(Server, bUnusedIsIdle, bOutStillAlive);
			}
		}
		else if (!bConnectionAlive)
		{
			bOutStillAlive = false;
		}
	}
}


FEditorDomainSaveClient::~FEditorDomainSaveClient()
{
	Shutdown();
}

void FEditorDomainSaveClient::RequestSave(const FPackagePath& PackagePath)
{
	FScopeLock ScopeLock(&Lock);
	if (!bEnabled)
	{
		return;
	}
	Requests.Add(PackagePath);
	if (!bAsyncActive)
	{
		KickCommunication();
	}
}

void FEditorDomainSaveClient::Tick(float DeltaTime)
{
	FScopeLock ScopeLock(&Lock);
	if (Requests.Num() == 0)
	{
		return;
	}
	if (!bAsyncActive)
	{
		KickCommunication();
		PollServerMessages();
	}
}

void FEditorDomainSaveClient::KickCommunication()
{
	// Called within ThreadingLock
	check(!bAsyncActive); // Caller should have checked
	if (!TrySendBatchRequest())
	{
		bAsyncActive = true;
		if (!TryInitializeCommunication())
		{
			Requests.Empty();
		}
		else if (!bAsyncSupported)
		{
			TickCommunication();
		}
		else
		{
			Async(EAsyncExecution::Thread, [this]() { RunCommunication(); });
		}
	}
}

bool FEditorDomainSaveClient::TrySendBatchRequest()
{
	// Called within ThreadingLock,
	// Called only on public interface thread (if !bAsyncActive) or on async thread (if bAsyncActive)
	if (Requests.Num() == 0)
	{
		return true;
	}

	if (bServerSocketReady)
	{
		TArray<FPackagePath> LocalRequests;
		const int32 MaxRequestSize = 100;
		int32 PeelCount = FMath::Min<int32>(MaxRequestSize, Requests.Num());
		LocalRequests.Reserve(PeelCount);
		for (; PeelCount > 0; --PeelCount)
		{
			LocalRequests.Add(Requests.PopFrontValue());
		}

		if (TrySendRequests(LocalRequests))
		{
			return true;
		}
		else
		{
			// Push back on front in reverse order to restore original order
			for (int Index = LocalRequests.Num() - 1; Index >= 0; --Index)
			{
				Requests.AddFront(LocalRequests[Index]);
			}
			return false;
		}
	}
	else
	{
		return false;
	}
}

bool FEditorDomainSaveClient::TryInitializeCommunication()
{
	// Called within ThreadingLock
	using namespace UE::EditorDomainSave;
	if (IsInitialized())
	{
		return true;
	}
	
	if (!ISocketSubsystem::Get())
	{
		UE_LOGF(LogEditorDomainSave, Error,
			"Platform does not support network sockets, EditorDomain saving is not enabled.");
		Shutdown();
		bEnabled = false;
		return false;
	}
	FString LockName = FString(FApp::GetProjectName()) + TEXT("EditorDomainServer");
	ProcessLock = CreateProcessLock();
	if (!ProcessLock)
	{
		UE_LOGF(LogEditorDomainSave, Error,
			"Could not create InterprocessSynchObject %ls, EditorDomain saving is not enabled.", *LockName);
		Shutdown();
		bEnabled = false;
		return false;
	}
	bAsyncSupported = FPlatformProcess::SupportsMultithreading();

	return true;
}

bool FEditorDomainSaveClient::IsInitialized() const
{
	return ProcessLock != nullptr;
}

void FEditorDomainSaveClient::RunCommunication()
{
	using namespace UE::EditorDomainSave;

	bool bIdle = false;
	double FlushStartTime = 0.;
	for (;;)
	{
		check(bAsyncActive);
		if (TryConnect())
		{
			FScopeLock ScopeLock(&Lock);
			if (TrySendBatchRequest())
			{
				if (Requests.Num() == 0)
				{
					bAsyncActive = false;
					break;
				}
			}
		}
		PollServerMessages();

		if (IsStopped)
		{
			if (FlushStartTime == 0.)
			{
				FlushStartTime = FPlatformTime::Seconds();
			}
			if (FPlatformTime::Seconds() - FlushStartTime > Constants::ClosingFlushBudgetSeconds)
			{
				UE_LOGF(LogEditorDomainSave, Warning,
					"Could not flush all results before shutting down. Failing to save %d remaining requests.",
					Requests.Num());
				FScopeLock ScopeLock(&Lock);
				bAsyncActive = false;
				break;
			}
		}

		FPlatformProcess::Sleep(static_cast<float>(Constants::RunCommunicationSleepPeriodSeconds));
	}
}

void FEditorDomainSaveClient::TickCommunication()
{
	check(!bAsyncActive);
	if (TryConnect())
	{
		FScopeLock ScopeLock(&Lock);
		TrySendBatchRequest();
	}
}

void FEditorDomainSaveClient::Shutdown()
{
	using namespace UE::EditorDomainSave;

	IsStopped = true;
	for (;;)
	{
		{
			FScopeLock ScopeLock(&Lock);
			if (!bAsyncActive)
			{
				break;
			}
		}
		FPlatformProcess::Sleep(static_cast<float>(Constants::RunCommunicationSleepPeriodSeconds));
	}
	FScopeLock ScopeLock(&Lock);
	Disconnect();
	KillDanglingProcess();
	if (ProcessLock)
	{
		FPlatformProcess::DeleteInterprocessSynchObject(ProcessLock);
		ProcessLock = nullptr;
	}
}

bool FEditorDomainSaveClient::TryConnect()
{
	// Called within either the ThreadingLock or the bAsyncActive flag
	if (!bServerSocketReady)
	{
		TickConnect();
	}
	return bServerSocketReady;
}


void FEditorDomainSaveClient::TickConnect()
{
	// Called within either the ThreadingLock or the bAsyncActive flag
	if (ConnectionAttemptStartTime == 0.)
	{
		ConnectionAttemptStartTime = FPlatformTime::Seconds();
	}
	if (!TryConnectProcess())
	{
		return;
	}
	TryConnectSocket();
}

bool FEditorDomainSaveClient::TryConnectProcess()
{
	// Called within either the ThreadingLock or the bAsyncActive flag
	using namespace UE::EditorDomainSave;

	if (ServerProcessId)
	{
		return true;
	}

	bool bSuccessful = false;
	FProcessSemaphoreScopeLock ScopeLock(ProcessLock, static_cast<float>(Constants::ProcessLockTimeoutSecondsWhenPolling));
	if (!ScopeLock.IsLocked())
	{
		return false;
	}

	uint32 LocalServerListenPort = 0;
	uint32 LocalServerProcessId = 0;
	FDateTime ServerSettingsTimestamp;
	if (TryReadConnectionData(LocalServerProcessId, LocalServerListenPort, ServerSettingsTimestamp))
	{
		return true;
	}

	if (LocalServerProcessId)
	{
		FTimespan UnresponsiveTime = FDateTime::Now() - ServerSettingsTimestamp;
		if (UnresponsiveTime.GetTotalSeconds() < Constants::ServerUnresponsiveTimeoutSeconds)
		{
			// Continue waiting for the server to post the listenport
			return false;
		}
	}

	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - LastProcessCreationAttemptTimeSeconds < Constants::ProcessCreationCooldownSeconds)
	{
		// Don't create a new process (wasting resources and potentially triggering virus scanners)
		// until our cooldown period has elapsed
		return false;
	}
	LastProcessCreationAttemptTimeSeconds = CurrentTime;

	int32 CurrentProcessId = FPlatformProcess::GetCurrentProcessId();
	FString CommandletExecutable = FUnrealEdMisc::Get().GetProjectEditorBinaryPath();
	FString Commandline = FString::Printf(TEXT("%s -run=EditorDomainSave -creatorprocessid=%d"),
		*FPaths::GetProjectFilePath(), CurrentProcessId);
	FProcHandle Handle = FPlatformProcess::CreateProc(*CommandletExecutable, *Commandline,
		true /* bLaunchDetached */, true /* bLaunchHidden */, true /* bLaunchReallyHidden */,
		&LocalServerProcessId, 0 /* PriorityModifier */, *FPaths::GetPath(CommandletExecutable),
		nullptr /* PipeWriteChild */);
	if (!Handle.IsValid())
	{
		ConnectionWarning(FString::Printf(TEXT("Could not create EditorDomainSave process %s %s"),
			*CommandletExecutable, *Commandline));
		return false;
	}

	TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject());
	RootObject->SetNumberField(TEXT("ProcessId"), (double)LocalServerProcessId);
	if (!TryWriteServerSettings(RootObject))
	{
		FPlatformProcess::TerminateProc(Handle);
		TryDeleteSettingsFile();
		ConnectionWarning(FString::Printf(TEXT("EditorDomainSaveServer is %s, and we are unable to create ")
			TEXT("a new settings file %s so we can not create a new one"),
			LocalServerProcessId != 0 ? TEXT("unresponsive") : TEXT("not yet started"), *GetServerSettingsFilename()));
		return false;

	}

	// We are not successful until we the server has written the ListenPort for us to read; retry the read again on next tick
	return false;
}

bool FEditorDomainSaveClient::TryReadConnectionData(uint32& OutLocalServerProcessId, uint32& OutLocalServerListenPort,
	FDateTime& OutServerSettingsTimestamp)
{
	using namespace UE::EditorDomainSave;

	// Called within either the ThreadingLock or the bAsyncActive flag
	// Called within ProcessLock
	OutLocalServerListenPort = 0;
	OutLocalServerProcessId = 0;

	TSharedPtr<FJsonObject> RootObject;
	if (TryReadServerSettings(RootObject, OutServerSettingsTimestamp))
	{
		uint32 PotentialProcessId;
		if (RootObject->TryGetNumberField(SettingNames::ProcessIdFieldName, PotentialProcessId))
		{
			FProcHandle ServerHandle = FPlatformProcess::OpenProcess(PotentialProcessId);
			ON_SCOPE_EXIT{ FPlatformProcess::CloseProc(ServerHandle); };
			if (ServerHandle.IsValid())
			{
				OutLocalServerProcessId = PotentialProcessId;
				uint32 PotentialListenPort;
				if (RootObject->TryGetNumberField(SettingNames::ListenPortFieldName, PotentialListenPort))
				{
					OutLocalServerListenPort = PotentialListenPort;
				}
			}
		}
	}

	if (OutLocalServerProcessId && OutLocalServerListenPort)
	{
		ServerProcessId = OutLocalServerProcessId;
		ServerListenPort = OutLocalServerListenPort;
		return true;
	}
	return false;
}

bool FEditorDomainSaveClient::TryConnectSocket()
{
	// Called within either the ThreadingLock or the bAsyncActive flag
	if (bServerSocketReady)
	{
		return true;
	}

	auto PollServerSocketNotReady = [this]()
	{
		using namespace UE::EditorDomainSave;

		if (PollKillServerProcess())
		{
			// PollKillServerProcess has taken action, don't check any further timeouts
		}
		else if (FPlatformTime::Seconds() - ConnectionAttemptSocketStartTime > Constants::SocketUnresponsiveTimeoutSeconds)
		{
			UE_LOGF(LogEditorDomainSave, Verbose,
				"EditorDomainSaveClient: Could not connect to %ls which should be opened by process %d",
				*ServerAddr->ToString(true), ServerProcessId);
			DisconnectProcess(); // Reverify that the process is running
			ConnectionAttemptSocketStartTime = 0;
		}
	};
	if (!ServerSocket)
	{
		if (ConnectionAttemptSocketStartTime == 0.)
		{
			ConnectionAttemptSocketStartTime = FPlatformTime::Seconds();
		}

		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		check(SocketSubsystem); // This should have been checked already in Initialize
		bool bCanBindAllUnused;
		ServerAddr = SocketSubsystem->GetLocalHostAddr(*GLog, bCanBindAllUnused);
		ServerAddr->SetPort(ServerListenPort);
		FSocket* LocalServerSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FEditorDomainSaveClient tcp"),
			ServerAddr->GetProtocolType());
		if (LocalServerSocket == nullptr)
		{
			ConnectionWarning(TEXT("Could not create socket"));
			DisconnectSocket();
			return false;
		}

		LocalServerSocket->SetNonBlocking(true);
		if (!LocalServerSocket->Connect(*ServerAddr))
		{
			LocalServerSocket->Close();
			SocketSubsystem->DestroySocket(LocalServerSocket);
			PollServerSocketNotReady();
			return false;
		}
		ServerSocket = LocalServerSocket;
	}
	if (!bServerSocketReady)
	{
		bServerSocketReady = ServerSocket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(0));
		if (!bServerSocketReady)
		{
			PollServerSocketNotReady();
			return false;
		}
	}

	UE_LOGF(LogEditorDomainSave, Display,
		"EditorDomainSaveClient: Connected to EditorDomainSaveServer with pid %d at %ls",
		ServerProcessId, *ServerAddr->ToString(true));
	return true;
}

bool FEditorDomainSaveClient::PollKillServerProcess()
{
	using namespace UE::EditorDomainSave;

	if (FPlatformTime::Seconds() - ConnectionAttemptStartTime <= Constants::ServerUnresponsiveTimeoutSeconds)
	{
		return false;
	}

	ConnectionWarning(FString::Printf(
		TEXT("EditorDomainSaveClient: Could not connect to %s which should be opened by process %d, for %lf seconds"),
		*ServerAddr->ToString(true), ServerProcessId, Constants::ServerUnresponsiveTimeoutSeconds),
		TEXT("Revoking the process's listen authority and starting a new one"),
		true /* bIgnoreCooldown */);
	{
		FProcessSemaphoreScopeLock ProcessScopeLock(ProcessLock, static_cast<float>(Constants::ProcessLockTimeoutSecondsWhenPolling));
		if (!ProcessScopeLock.IsLocked())
		{
			return false;
		}
		TryDeleteSettingsFile();

		// We are about to assign a new DanglingProcessId, so time is up for any previous dangling process
		KillDanglingProcess();

		// We will give the server some time to notice that its authority has been revoked and shut itself down,
		// But after a timeout we kill the process
		DanglingProcessId = ServerProcessId;
	}
	DisconnectProcess();
	ConnectionAttemptStartTime = 0.;
	return true;
}

void FEditorDomainSaveClient::KillDanglingProcess()
{
	if (!DanglingProcessId)
	{
		return;
	}
	int32 ProcessId = DanglingProcessId;
	DanglingProcessId = 0;

	if (!FPathViews::GetBaseFilenameWithPath(FPlatformProcess::GetApplicationName(ProcessId)).Equals(
		FPathViews::GetBaseFilenameWithPath(FUnrealEdMisc::Get().GetProjectEditorBinaryPath()), ESearchCase::IgnoreCase))
	{
		// The process must have exited on its own and some other process has taken over the pid it had
		return;
	}

	FProcHandle Handle = FPlatformProcess::OpenProcess(ProcessId);
	ON_SCOPE_EXIT{ FPlatformProcess::CloseProc(Handle); };
	if (Handle.IsValid())
	{
		FPlatformProcess::TerminateProc(Handle, true /* KillTree */);
	}
}

bool FEditorDomainSaveClient::TrySendRequests(TArray<FPackagePath>& LocalRequests)
{
	// Called within ThreadingLock
	// Called only on public interface thread (if !bAsyncActive) or on async thread (if bAsyncActive)
	using namespace UE::EditorDomainSave;

	TArray<uint8> MessageBytes;
	FMessageHeader MessageHeader;
	{
		FMemoryWriter Message(MessageBytes);

		MessageHeader.Initialize(EMessageType::Save);
		Message << MessageHeader;

		int32 NumRequests = LocalRequests.Num();
		Message << NumRequests;
		for (const FPackagePath& PackagePath : LocalRequests)
		{
			FString PackageName = PackagePath.GetPackageName();
			Message << PackageName;
		}
		MessageHeader.Size = IntCastChecked<uint32>(Message.Tell());
		Message.Seek(0);
		Message << MessageHeader;
		Message.Seek(MessageHeader.Size);
	}
	check(MessageBytes.Num() == MessageHeader.Size);

	int32 BytesSent;
	ServerSocket->Send(MessageBytes.GetData(), MessageBytes.Num(), BytesSent);
	if (BytesSent != MessageBytes.Num())
	{
		ConnectionWarning(TEXT("Socket::Send failed to send message to server"), TEXT("Will reconnect"));
		if (bPreviousSendFailed)
		{
			PollKillServerProcess();
		}
		else
		{
			bPreviousSendFailed = true;
			ConnectionAttemptStartTime = 0; // Start the timeout from now, the first point of failure
		}
		DisconnectProcess();
		return false;
	}
	bPreviousSendFailed = false;

	LocalRequests.Reset();
	return true;
}

void FEditorDomainSaveClient::Disconnect()
{
	DisconnectProcess();
	ConnectionAttemptStartTime = 0.;
	bPreviousSendFailed = false;
}

void FEditorDomainSaveClient::DisconnectProcess()
{
	DisconnectSocket();
	ServerProcessId = 0;
	ServerListenPort = 0;
	ConnectionAttemptSocketStartTime = 0.;
}

void FEditorDomainSaveClient::DisconnectSocket()
{
	bServerSocketReady = false;
	ServerAddr.Reset();
	if (ServerSocket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		check(SocketSubsystem);
		ServerSocket->Close();
		SocketSubsystem->DestroySocket(ServerSocket);
		ServerSocket = nullptr;
	}
}

void FEditorDomainSaveClient::ConnectionWarning(FStringView Message, FStringView Consequence, bool bIgnoreCooldown)
{
	using namespace UE::EditorDomainSave;

	double CurrentTime = FPlatformTime::Seconds();
	if (bIgnoreCooldown || CurrentTime - LastConnectionWarningTimeSeconds >= Constants::ConnectionWarningCooldownSeconds)
	{
		if (!bIgnoreCooldown)
		{
			LastConnectionWarningTimeSeconds = CurrentTime;
		}
		FStringView DefaultConsequence = TEXTVIEW("EditorDomainSave is currently unavailable");
		if (Consequence.Len() == 0)
		{
			Consequence = DefaultConsequence;
		}
		UE_LOGF(LogEditorDomainSave, Warning, "%.*ls. %.*ls.", Message.Len(), Message.GetData(), Consequence.Len(),
			Consequence.GetData());
	}
}

void FEditorDomainSaveClient::PollServerMessages()
{
	// Called within either the ThreadingLock or the bAsyncActive flag
	if (!ServerSocket)
	{
		return;
	}

	// EDITOR_DOMAIN_TODO: Read messages from the server

}

namespace UE
{
namespace EditorDomainSave
{

FString GetServerSettingsFilename()
{
	return FPaths::ProjectIntermediateDir() / TEXT("Cook") / TEXT("EditorDomainSave.json");
}

FString GetLockName()
{
	return FString(FApp::GetProjectName()) + TEXT("EditorDomainServer");

}
FPlatformProcess::FSemaphore* CreateProcessLock()
{
	// EDITOR_DOMAIN_TODO: Need to add a HasInterprocessSyncLock function to prevent the
	// use of editor saving rather than throwing a fatal
	return FPlatformProcess::NewInterprocessSynchObject(GetLockName(), true /* bCreate */);
}

bool TryReadServerSettings(TSharedPtr<FJsonObject>& OutRootObject, FDateTime& OutServerSettingsTimestamp)
{
	OutRootObject = TSharedPtr<FJsonObject>();
	OutServerSettingsTimestamp = FDateTime();

	FString ServerSettingsFilename = GetServerSettingsFilename();
	IFileManager& FileManager = IFileManager::Get();
	FFileStatData StatData = FileManager.GetStatData(*ServerSettingsFilename);
	if (!StatData.bIsValid)
	{
		return false;
	}
	TUniquePtr<FArchive> FileAr(FileManager.CreateFileReader(*ServerSettingsFilename));
	if (!FileAr)
	{
		return false;
	}

	OutRootObject = MakeShareable(new FJsonObject);
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(FileAr.Get());
	if (!FJsonSerializer::Deserialize(Reader, OutRootObject))
	{
		OutRootObject.Reset();
		return false;
	}

	OutServerSettingsTimestamp = StatData.ModificationTime;
	return true;
}

bool TryWriteServerSettings(TSharedPtr<FJsonObject>& RootObject)
{
	if (!RootObject)
	{
		return false;
	}
	IFileManager& FileManager = IFileManager::Get();
	FString ServerSettingsFilename = GetServerSettingsFilename();
	FileManager.MakeDirectory(*FPaths::GetPath(ServerSettingsFilename), true /* Tree */);
	TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileWriter(*ServerSettingsFilename));
	if (!FileAr)
	{
		return false;
	}
	TSharedRef<TJsonWriter<TCHAR>> Writer = TJsonWriterFactory<TCHAR>::Create(FileAr.Get());
	return FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer, false/* bCloseWriter */);
}

bool TryDeleteSettingsFile()
{
	FString ServerSettingsFilename = GetServerSettingsFilename();
	IFileManager& FileManager = IFileManager::Get();
	return !FileManager.FileExists(*ServerSettingsFilename) || FileManager.Delete(*ServerSettingsFilename);
}

void GetServerSettingsTimeStamp(bool& bOutExists, FDateTime& OutModificationTime)
{
	OutModificationTime = IFileManager::Get().GetTimeStamp(*GetServerSettingsFilename());
	bOutExists = OutModificationTime != FDateTime::MinValue();
}


}
}
