// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Containers/Array.h"
#include "Containers/RingBuffer.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "EditorDomain/EditorDomain.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "Misc/DateTime.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FJsonObject;
class FInternetAddr;
class FPackagePath;
class FRunnableThread;
class FSocket;
class IAssetRegistry;
class IPackageResourceManager;
namespace UE
{
namespace EditorDomainSave
{
	enum class EMessageType : uint32;
}
}

/**
 * System to handle save requests sent from editor and commandlet processes to save packages
 * into the Editor Domain.
 */
class FEditorDomainSaveServer
{
public:
	FEditorDomainSaveServer();
	~FEditorDomainSaveServer();

	int32 Run();

private:
	/** A connection to a client held by EditorDomainSaveServer. */
	struct FClientConnection : public FRefCountedObject
	{
		FClientConnection(FSocket* InClientSocket);
		~FClientConnection();
		/** Release resources and indicate this connection should no longer be used. */
		void Shutdown();
		/** Read message bytes from the socket. */
		void Poll(FEditorDomainSaveServer& Server, bool& bOutIsIdle, bool& bOutStillAlive);

		/** The socket to the client process, returned from accept on the server's ListenSocket. */
		FSocket* ClientSocket;
		/**
		 * Bytes being read from the client process, held until all the expected size from the message header is read.
		 * Array is sized to the size from the message.
		 */
		TArray<uint8> MessageBuffer;
		/** Number of bytes we have so far read into the buffer. */
		int32 BufferOffset = 0;
		/** The MessageType for the message being read, as reported in the message header. */
		UE::EditorDomainSave::EMessageType MessageType;
	};

	/**
	 * Initialize the listen socket, failing if this process is not the designated handler process
	 * or if any required features are missing in the current development environment.
	 */
	bool TryInitialize();
	/**
	 * Called on startup or when the ServerSettings file on disk changes.
	 * Read the settings file, return false if this process is not (or is no longer or can no longer be) the
	 * designated handler, and consume any new or modified settings from the file
	 * 
	 * @param bOutOwnsSettingsFile [Optional] Set to true if this process still owns the settings file
	 * and should delete it on exit. This can be true even if the process should shut down due to e.g. an error.
	 * @return Whether this FEditorDomainSaveServer should continue to handle save messages
	 */
	bool TryRefreshAuthority(bool* bOutOwnsSettingsFile = nullptr, TSharedPtr<FJsonObject>* OutRootObject = nullptr,
		int32* OutListenPort = nullptr);
	/** Free used resources; destructor code moved into separate function to make flexible when it operates. */
	void Shutdown();
	/** Shut down the Listen socket and all client sockets created from it. */
	void ShutdownSocket();
	/**
	 * Check whether this server should shutdown because no clients requesting a handler currently exist.
	 * Note this check in the ensuing shutdown have to be done inside the interprocess lock to prevent races
	 * with clients that want to create a new handler process if one is not already running
	 */
	bool TryAbdicate();
	/** Return whether TryAbdicate has returned false and this server should therefore shut down. */
	bool IsAbdicated() const;
	/**
	 * Return true if there are any active client connections, or if the expected connection from the 
	 * initial client that caused the creation of this server is still pending.
	 */
	bool HasExpectedConnections() const;
	/**
	 * Called periodically to check whether this server is still the designated handler and still has clients.
	 *
	 * @return whether the server should continue running.
	 */
	bool PollShouldRun();
	/** Check the listen socket for new client connections. */
	void PollIncomingConnections(bool& bInOutIsIdle);
	/** Check the client sockets for new messages. */
	void PollConnections(bool& bInOutIsIdle);
	/** Poll periodic maintenance such as garbage collection. */
	void TickMaintenance(bool bIsIdle);
	/**
	 * Called each tick loop when whether the server is idle is known (e.g. no pending messages, no pending loads).
	 * Perform actions that should be taken when IsIdle changes
	 */
	void SetIdle(bool bIsIdle);
	/**  Switch on messagetype and handle a message sent from a client (Save, Close, others). */
	void ProcessMessage(FClientConnection& ClientConnection, UE::EditorDomainSave::EMessageType MessageType,
		const TArray<uint8>& MessageBuffer, bool& bOutStillAlive);
	/** Time-sliced tick function to load and save packages clients have requested. */
	void TickPendingPackages(bool& bInOutIsIdle);
	/** Load/Save a given PackageName/PackagePath. */
	bool TrySavePackage(FStringView PackageName, FString& OutErrorMessage);
	bool TrySavePackage(const FPackagePath& PackagePath, FString& OutErrorMessage);

	/** Network socket, pending message process, and and other information about each connected client. */
	TArray<TRefCountPtr<FClientConnection>> ClientConnections;
	/** The list of packages that have been requested saved but not yet saved. */
	TRingBuffer<FString> PendingPackageNames;
	/** Timestamp of the settings file on disk the last time we read it, to detect settings updates. */
	FDateTime SettingsTimestamp;
	/** The socket listening for client connections. */
	FSocket* ListenSocket = nullptr;
	/** Cached pointer to the global asset registry. */
	IAssetRegistry* AssetRegistry;
	/** In-process handle to the interprocess synchronization object. */
	FPlatformProcess::FSemaphore* ProcessLock = nullptr;
	/** The time at which the current idleness started. Used to decide abdication. */
	double IdleStartTime = 0;
	/** The time at which garbage was last collected. Used for scheduling the next collection. */
	double LastGarbageTime = 0;
	/** The process id that created this process (and is expected to connect). Used to decide abdication. */
	int32 CreatorProcessId = 0;
	/** The port to listen on. */
	int32 ListenPort = 0;
	/** True if the server was idle (no pending messages, no pending packages) in its previous tick. */
	bool bIdle = false;
	/** True if any client has so-far connected. Used to decide abdication. */
	bool bHasEverConnected = false;
	/** True if the server has abdicated and should shutdown. */
	bool bAbdicated = false;
	/** Tracks whether initialize-on-demand steps on the assetregistry have been done. */
	bool bHasInitializedAssetRegistry = false;
	/** Whether this server was launched as a resident server; it takes over authority and never abdicates. */
	bool bIsResidentServer = false;
};

/** Client-side system to send SaveRequests to the EditorDomainSaveServer in another process. */
class FEditorDomainSaveClient
{
public:
	~FEditorDomainSaveClient();

	/** Send a request to the EditorDomainSaveServer to save the given PackagePath into the Editor Domain. */
	void RequestSave(const FPackagePath& PackagePath);
	/** Tick communication with the server. */
	void Tick(float DeltaTime);

private:
	/**
	 * If already connected, send message synchronously.
	 * If not yet connected or send failed, start an async task to connect and send message.
	 * Early-exit the entire function if async task is already running.
	 * If async is not supported, runs one tick of the connection attempt per call.
	 */
	void KickCommunication();
	/** If connected, send a batch of names to the server. Return true on success or empty. */
	bool TrySendBatchRequest();
	/** Construct on-demand resources and flags. Returns true iff saving is enabled. */
	bool TryInitializeCommunication();
	/** Return whether initialization has already been successfully run. */
	bool IsInitialized() const;
	/** Called on an async thread. Try to connect until successful, send messages until empty, return. */
	void RunCommunication();
	/** Called from the public interface thread. Pump one connection step, send one message, return. */
	void TickCommunication();
	/** Stop the async task if it is running, wait for the async task to complete, shutdown all resources. */
	void Shutdown();
	/** Tick the connection code if it is not already ready and return true if it is ready. */
	bool TryConnect();
	/** Pump one tick of the connection process - settings file read, process creation, opening socket. */
	void TickConnect();
	/**
	 * Read the server settings file to find the process id and port of the save server process. 
	 * Create the process and server settings file if it is not running.
	 * Return whether we now have the process id and port number.
	 */
	bool TryConnectProcess();
	/** Read the server settings file to find the process id and port of the save server process. */
	bool TryReadConnectionData(uint32& OutLocalServerProcessId, uint32& OutLocalServerListenPort,
		FDateTime& OutServerSettingsTimestamp);
	/** Connect to the server's listen port. Set bServerSocketReady if completed. Retry the connect if necessary. */
	bool TryConnectSocket();
	/**
	 * Check whether the server connection has failed for too long,
	 * and if so revoke its authority to listen by rewriting the settings file.
	 */
	bool PollKillServerProcess();
	/** Terminate the process of the process from which we earlier revoked authority */
	void KillDanglingProcess();
	/** Send the batch of requests to the server connection, and remove sent requests from the LocalRequests list. */
	bool TrySendRequests(TArray<FPackagePath>& LocalRequests);
	/** Close the current connection attempt with the server, including the connection starttime. */
	void Disconnect();
	/** Remove information about the server process; it will be reconstructed when TickConnect is called. */
	void DisconnectProcess();
	/** Close the socket communicating with the server; it will be reconstructed when TickConnect is called. */
	void DisconnectSocket();
	/** Issue a connection warning with reason and consequence; apply a cooldown to prevent spamming warnings. */
	void ConnectionWarning(FStringView Message, FStringView Consequence = FStringView(), bool bIgnoreCooldown=false);
	/** If the socket is ready, check for messages from the server and process them. */
	void PollServerMessages();

	// Shared variables - threadsafe
	/** False by default, set to true if the public interface wants to shutdown the async task. */
	std::atomic<bool> IsStopped{ false };
	/** Lock around data accessed by the public interface. */
	FCriticalSection Lock;

	// PublicInterface RestrictedWrite variables
	// These variables can only be accessed on the public interface when inside the lock.
	// They may not be written when bAsyncActive is true.
	/** True by default, set to false if the communication with the server is unavailable. */
	bool bEnabled = true;
	/** True iff async calls are supported. If false, Tick will be used instead of Run. */
	bool bAsyncSupported = false;

	// SharedVariables - read/write only inside the lock.
	/** Requests through the public interface that have not yet been sent. */
	TRingBuffer<FPackagePath> Requests;
	/**
	 * True if an async task has started and not yet finished.
	 * If true (inside the lock), the async task will send all requests before exiting.
	 */
	bool bAsyncActive = false;

	// ProcessCommunication variables. These variables can only be read or written
	// 1) In public interface when inside the Lock and !bAsyncActive
	// OR
	// 2) From the async task and bAsyncActive
	/** Socket to the server process; requests are sent on this socket. */
	FSocket* ServerSocket = nullptr;
	/** In-process handle to the interprocess synchronization object. */
	FGenericPlatformProcess::FSemaphore* ProcessLock = nullptr;
	/** The url:port the server is listening on. */
	TSharedPtr<FInternetAddr> ServerAddr;
	/** When we started the current connection attempt; used to decide when the server process is not responding. */
	double ConnectionAttemptStartTime = 0.;
	/** When we started the current socket connection attempt; used to check to see if the server process is alive. */
	double ConnectionAttemptSocketStartTime = 0.;
	/** The last time we attempted to create a process. */
	double LastProcessCreationAttemptTimeSeconds = 0.;
	/* The last time we sent a warning to the log. */
	double LastConnectionWarningTimeSeconds = 0.;
	/** The pid of the process running the server, created by us or another client. */
	int32 ServerProcessId = 0;
	/** The port the server is listening on. */
	int32 ServerListenPort = 0;
	/** A server that we have told to shutdown; kill the process if it doesn't shutdown. */
	int32 DanglingProcessId = 0;
	/** Indicator that the socket asynchronous connect has completed and we can call Send on it. */
	bool bServerSocketReady = false;
	/**
	 * Tracks that our previous attempt to send failed, even though the serversocket was ready.
	 * Used to detect connecting socket but failing send.
	 */
	bool bPreviousSendFailed = false;
};
