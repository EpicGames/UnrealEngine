// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commandlets/EditorDomainSaveCommandlet.h"

#include "EditorDomainSave.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorDomainSaveCommandlet)

UEditorDomainSaveCommandlet::UEditorDomainSaveCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

int32 UEditorDomainSaveCommandlet::Main(const FString& CmdLineParams)
{
	FEditorDomainSaveServer Server;
	return Server.Run();
}
