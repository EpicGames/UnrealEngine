// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDomain/EditorDomainUtils.h"

#include "Algo/AnyOf.h"
#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/PackageReader.h"
#include "Async/ParallelFor.h"
#include "Containers/DirectoryTree.h"
#include "Cooker/CookDependencyContext.h"
#include "Cooker/CookArtifact.h"
#include "Cooker/CookPackageArtifacts.h"
#include "DerivedDataBuildDefinition.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Hash/Blake3.h"
#include "Interfaces/IPluginManager.h"
#include "IO/IoHash.h"
#include "Memory/SharedBuffer.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackagePath.h"
#include "Misc/Parse.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "Serialization/BulkDataRegistry.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/PackageWriterToSharedBuffer.h"
#include "String/ParseTokens.h"
#include "TargetDomain/TargetDomainUtils.h"
#include "Templates/Function.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "UObject/CoreRedirects.h"
#include "UObject/ObjectResource.h"
#include "UObject/ObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"

namespace UE::EditorDomain
{

// Change to a new guid when EditorDomain needs to be invalidated. Note this will also invalidate
// every TargetDomain key since TargetDomain keys include EditorDomain keys.
FGuid EditorDomainVersion(TEXT("ED92BBE49F6F4F2E94E9D8BC10AD59A7"));

}

#if !defined(EDITORDOMAINTIMEPROFILERTRACE_ENABLED)
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
#define EDITORDOMAINTIMEPROFILERTRACE_ENABLED 1
#else
#define EDITORDOMAINTIMEPROFILERTRACE_ENABLED 0
#endif
#endif

#define CUSTOM_EDITORDOMAINTIMER_LOG Cpu

#if EDITORDOMAINTIMEPROFILERTRACE_ENABLED
#define SCOPED_EDITORDOMAINTIMER_TEXT(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(TimerName, CpuChannel)
#define SCOPED_EDITORDOMAINTIMER(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TimerName, CpuChannel)
#define SCOPED_CUSTOM_EDITORDOMAINTIMER(TimerName) UE_TRACE_LOG_SCOPED_T(CUSTOM_EDITORDOMAINTIMER_LOG, TimerName, CpuChannel)
#define ADD_CUSTOM_EDITORDOMAINTIMER_META(TimerName, Key, Value) << TimerName.Key(Value)
#define SCOPED_EDITORDOMAINTIMER_CNT(TimerName)
#else
#define SCOPED_EDITORDOMAINTIMER_TEXT(TimerName)
#define SCOPED_EDITORDOMAINTIMER(TimerName)
#define SCOPED_CUSTOM_EDITORDOMAINTIMER(TimerName)
#define ADD_CUSTOM_EDITORDOMAINTIMER_META(TimerName, Key, Value)
#define SCOPED_EDITORDOMAINTIMER_CNT(TimerName)
#endif

UE_TRACE_EVENT_BEGIN(CUSTOM_EDITORDOMAINTIMER_LOG, EditorDomain_TrySavePackage, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

/** Modify the masked bits in the output: set them to A & B. */
template<typename Enum>
static void EnumSetFlagsAnd(Enum& Output, Enum Mask, Enum A, Enum B)
{
	Output = (Output & ~Mask) | (Mask & A & B);
}

template <typename KeyType, typename ValueType, typename SetAllocator, typename KeyFuncs>
static ValueType MapFindRef(const TMap<KeyType, ValueType, SetAllocator, KeyFuncs>& Map,
	typename TMap<KeyType, ValueType, SetAllocator, KeyFuncs>::KeyInitType Key, ValueType DefaultValue)
{
	const ValueType* FoundValue = Map.Find(Key);
	return FoundValue ? *FoundValue : DefaultValue;
}

static void CoreRedirectClassPath(FTopLevelAssetPath& ClassPath, bool* bWasRedirected = nullptr)
{
	FCoreRedirectObjectName ClassRedirect(ClassPath);
	FCoreRedirectObjectName RedirectedClassRedirect = FCoreRedirects::GetRedirectedName(
		ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Struct, ClassRedirect);
	if (ClassRedirect != RedirectedClassRedirect)
	{
		if (RedirectedClassRedirect.OuterName.IsNone())
		{
			ClassPath = FTopLevelAssetPath(RedirectedClassRedirect.PackageName, RedirectedClassRedirect.ObjectName);
			if (bWasRedirected)
			{
				*bWasRedirected = true;
			}
			return;
		}
		else
		{
			UE_LOGF(LogEditorDomain, Warning, "CoreRedirect for class %ls redirects to %ls which is not a TopLevelAssetPath. Ignoring it.",
				*ClassRedirect.ToString(), *RedirectedClassRedirect.ToString())
		}
	}
	if (bWasRedirected)
	{
		*bWasRedirected = false;
	}
}

#if ENABLE_COOK_STATS
namespace UE::EditorDomain::CookStats
{

FCookStats::FDDCResourceUsageStats Usage;
void Register()
{
	FCookStatsManager::CookStatsCallbacks.AddLambda([](FCookStatsManager::AddStatFuncRef AddStat)
		{
			Usage.LogStats(AddStat, TEXT("EditorDomain.Usage"), TEXT("Package"));
		});
}

}
#endif

namespace UE::EditorDomain
{
	
// Enables/disables the parallel precalculation of class data digests (ease of use for debugging purposes).
constexpr bool bPrecalculateDataForClassDigests = true;

void PrecacheClassDigest(const FTopLevelAssetPath& ClassPath);

FStringView RemoveConfigComment(FStringView Line)
{
	int32 CommentIndex;
	Line.FindChar(';', CommentIndex);
	if (CommentIndex != INDEX_NONE)
	{
		Line = Line.Left(CommentIndex);
	}
	Line.TrimStartAndEndInline();
	return Line;
}

struct FSaveStoreResultToText
{
	ESaveStorageResult Code;
	FUtf8StringView Text;
};

const FUtf8StringView SaveStoreResultInvalidText = UTF8TEXTVIEW("InvalidCode");
const FSaveStoreResultToText SaveStoreResultToText[]
{
	{ ESaveStorageResult::Valid,					UTF8TEXTVIEW("Valid") },
	{ ESaveStorageResult::UnexpectedClass,			UTF8TEXTVIEW("UnexpectedClass") },
	{ ESaveStorageResult::UnexpectedCustomVersion,	UTF8TEXTVIEW("UnexpectedCustomVersion") },
	{ ESaveStorageResult::BulkDataTooLarge,			UTF8TEXTVIEW("BulkDataTooLarge") },
	{ ESaveStorageResult::InvalidCode,				SaveStoreResultInvalidText },
};

FUtf8StringView LexToUtf8(ESaveStorageResult Result)
{
	for (const FSaveStoreResultToText& ResultToText : SaveStoreResultToText)
	{
		if (Result == ResultToText.Code)
		{
			return ResultToText.Text;
		}
	}
	return SaveStoreResultInvalidText;
}

ESaveStorageResult SaveStorageResultFromString(FUtf8StringView Text)
{
	for (const FSaveStoreResultToText& ResultToText : SaveStoreResultToText)
	{
		if (Text == ResultToText.Text)
		{
			return ResultToText.Code;
		}
	}
	return ESaveStorageResult::InvalidCode;
}

TArray<FGuid, TInlineAllocator<1>> GetCustomVersions(UClass& Class, FCustomVersionsStripedMap* InOutCache = nullptr);
TMap<FGuid, UObject*> FindCustomVersionCulprits(TConstArrayView<FGuid> UnknownGuids, UPackage* Package);
void InitializeGlobalConstructClasses();

FClassDigestMap GClassDigests;
FClassDigestMap& GetClassDigests()
{
	return GClassDigests;
}

TMap<FTopLevelAssetPath, EDomainUse> GClassBlockedUses;
TMap<FName, EDomainUse> GPackageBlockedUses;
TMultiMap<FTopLevelAssetPath, FTopLevelAssetPath> GConstructClasses;
TSet<FTopLevelAssetPath> GTargetDomainClassBlockList;
TArray<FTopLevelAssetPath> GGlobalConstructClasses;
bool bGGlobalConstructClassesInitialized = false;
bool bGUtilsCookInitialized = false;
FBlake3Hash GGlobalConstructClassesHash;
int64 GMaxBulkDataSize = -1;

// Identifier of the CacheBuckets for EditorDomain tables
const TCHAR* EditorDomainPackageBucketName = TEXT("EditorDomainPackage");
const TCHAR* BulkDataListBucketName = TEXT("BulkDataList");
const TCHAR* BulkDataPayloadIdBucketName = TEXT("BulkDataPayloadId");

static bool GetEditorDomainSaveUnversioned()
{
	auto Initialize = []()
	{
		bool bParsedValue;
		bool bResult = GConfig->GetBool(TEXT("EditorDomain"), TEXT("SaveUnversioned"), bParsedValue, GEditorIni) ? bParsedValue : true;
		return bResult;
	};
	static bool bEditorDomainSaveUnversioned = Initialize();
	return bEditorDomainSaveUnversioned;
}

/**
 * Thread-safe cache to compress CustomVersion Guids into integer handles, to reduce the cost of removing duplicates
 * when lists of CustomVersion Guids are merged
*/
class FKnownCustomVersions
{
public:
	/** Find or if necessary add the handle for each Guid; append them to the output handles. */
	static void FindOrAddHandles(TArray<int32>& OutHandles, TConstArrayView<FGuid> InGuids);
	/** Find or if necessary add the handle for each Guid; append them to the output handles. */
	static void FindOrAddHandles(TArray<int32>& OutHandles, int32 NumGuids, TFunctionRef<const FGuid& (int32)> GetGuid);
	/** Find the guid for each handle. Handles must be values returned from a FindHandle function. */
	static void FindGuidsChecked(TArray<FGuid>& OutGuids, TConstArrayView<int32> Handles);

private:
	static FRWLock Lock;
	static TMap<FGuid, int32> GuidToHandle;
	static TArray<FGuid> Guids;
};

FRWLock FKnownCustomVersions::Lock;
TMap<FGuid, int32> FKnownCustomVersions::GuidToHandle;
TArray<FGuid> FKnownCustomVersions::Guids;

void FKnownCustomVersions::FindOrAddHandles(TArray<int32>& OutHandles, TConstArrayView<FGuid> InGuids)
{
	FindOrAddHandles(OutHandles, InGuids.Num(), [InGuids](int32 Index) -> const FGuid& { return InGuids[Index]; });
}

void FKnownCustomVersions::FindOrAddHandles(TArray<int32>& OutHandles, int32 NumGuids, TFunctionRef<const FGuid& (int32)> GetGuid)
{
	// Avoid a WriteLock in most cases by finding-only the incoming guids and writing their handle to the output
	// For any Guids that are not found, add a placeholder handle and store the missing guid and its index in
	// the output in a list to iterate over later.
	TArray<TPair<FGuid, int32>> UnknownGuids;
	{
		FReadScopeLock ReadLock(Lock);
		OutHandles.Reserve(OutHandles.Num() + NumGuids);
		for (int32 Index = 0; Index < NumGuids; ++Index)
		{
			const FGuid& Guid = GetGuid(Index);
			int32* CustomVersionHandle = GuidToHandle.Find(GetGuid(Index));
			if (CustomVersionHandle)
			{
				OutHandles.Add(*CustomVersionHandle);
			}
			else
			{
				UnknownGuids.Reserve(NumGuids);
				UnknownGuids.Add(TPair<FGuid, int32>{ Guid, OutHandles.Num() });
				OutHandles.Add(INDEX_NONE);
			}
		}
	}

	if (UnknownGuids.Num())
	{
		// Add the missing guids under the writelock and write their handle over the placeholders in the output.
		FWriteScopeLock WriteLock(Lock);
		int32 NumKnownGuids = Guids.Num();
		for (TPair<FGuid, int32>& Pair : UnknownGuids)
		{
			int32& ExistingIndex = GuidToHandle.FindOrAdd(Pair.Key, NumKnownGuids);
			if (ExistingIndex == NumKnownGuids)
			{
				Guids.Add(Pair.Key);
				++NumKnownGuids;
			}
			OutHandles[Pair.Value] = ExistingIndex;
		}
	}
}

void FKnownCustomVersions::FindGuidsChecked(TArray<FGuid>& OutGuids, TConstArrayView<int32> Handles)
{
	OutGuids.Reserve(OutGuids.Num() + Handles.Num());
	FReadScopeLock ReadLock(Lock);
	for (int32 Handle : Handles)
	{
		check(0 <= Handle && Handle < Guids.Num());
		OutGuids.Add(Guids[Handle]);
	}
}

FPackageDigest CalculatePackageDigest(IAssetRegistry& AssetRegistry, FName PackageName)
{
	AssetRegistry.WaitForPackage(PackageName.ToString());
	TOptional<FAssetPackageData> PackageData = AssetRegistry.GetAssetPackageDataCopy(PackageName);
	if (!PackageData)
	{
		return FPackageDigest(FPackageDigest::EStatus::DoesNotExistInAssetRegistry);
	}
	return CalculatePackageDigest(*PackageData, PackageName);
}

FPackageDigest CalculatePackageDigest(const FAssetPackageData& PackageData, FName PackageName)
{
	static FString PackageToDump;
	static bool bDumpPackageDigest = FParse::Value(FCommandLine::Get(), TEXT("-DumpPackageDigest="), PackageToDump);
	static FName PackageNameToDump(PackageToDump);

	bool bDumpCurrentPackage = PackageName == PackageNameToDump;

	TSharedPtr<FJsonObject> DigestDependencies = nullptr;
	if (bDumpCurrentPackage)
	{
		DigestDependencies = MakeShareable(new FJsonObject());
	}

	FPackageDigest Result;
	Result.DomainUse = EDomainUse::LoadEnabled | EDomainUse::SaveEnabled;;
	EnumSetFlagsAnd(Result.DomainUse, EDomainUse::LoadEnabled | EDomainUse::SaveEnabled,
		Result.DomainUse, ~MapFindRef(GPackageBlockedUses, PackageName, EDomainUse::None));

	FBlake3 Writer;
	FStringView ProjectName(FApp::GetProjectName());
	Writer.Update(ProjectName.GetData(), ProjectName.Len() * sizeof(ProjectName[0]));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.ProjectName"), FString(ProjectName));
	}

	Writer.Update(&EditorDomainVersion, sizeof(EditorDomainVersion));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.EditorDomainVersion"), EditorDomainVersion.ToString());
	}

	uint8 EditorDomainSaveUnversioned = GetEditorDomainSaveUnversioned() ? 1 : 0;
	Writer.Update(&EditorDomainSaveUnversioned, sizeof(EditorDomainSaveUnversioned));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.EditorDomainSaveUnversioned"), FString::FromInt(EditorDomainSaveUnversioned));
	}

	Writer.Update(&PackageData.GetPackageSavedHash().GetBytes(), sizeof(PackageData.GetPackageSavedHash().GetBytes()));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.PackageData"), LexToString(PackageData.GetPackageSavedHash()));
	}

	Writer.Update(&GPackageFileUEVersion, sizeof(GPackageFileUEVersion));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.GPackageFileUEVersion"), FString::FromInt(GPackageFileUEVersion.ToValue()));
	}

	Writer.Update(&GPackageFileLicenseeUEVersion, sizeof(GPackageFileLicenseeUEVersion));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.GPackageFileLicenseeUEVersion"), FString::FromInt(GPackageFileLicenseeUEVersion));
	}

	TArray<int32> CustomVersionHandles;
	// Reserve 10 custom versions per class times 100 classes per package times twice (once in package, once in class)
	CustomVersionHandles.Reserve(10*100*2);
	TConstArrayView<UE::AssetRegistry::FPackageCustomVersion> PackageVersions = PackageData.GetCustomVersions();
	FKnownCustomVersions::FindOrAddHandles(CustomVersionHandles,
		PackageVersions.Num(), [PackageVersions](int32 Index) -> const FGuid& { return PackageVersions[Index].Key;});

	InitializeGlobalConstructClasses();
	Writer.Update(&GGlobalConstructClassesHash, sizeof(GGlobalConstructClassesHash));
	if (bDumpCurrentPackage)
	{
		DigestDependencies->SetStringField(TEXT("PackageDigest.GGlobalConstructClassesHash"), LexToString(GGlobalConstructClassesHash));
	}

	// Add the InclusiveSchema for all classes imported by the package. The InclusiveSchema includes the schema of
	// classes that can be constructed by the given class.
	FClassDigestMap& ClassDigests = GetClassDigests();
	bool bHasTriedPrecacheClassDigests = false;
	TArray<FTopLevelAssetPath> ImportedClassPaths;
	ImportedClassPaths.Reserve(PackageData.ImportedClasses.Num());
	for (FName ClassPathName : PackageData.ImportedClasses)
	{
		FTopLevelAssetPath ClassPath(WriteToString<256>(ClassPathName).ToView());
		if (ClassPath.IsValid())
		{
			ImportedClassPaths.Add(ClassPath);
		}
	}

	int32 NextClass = 0;
	while (NextClass < ImportedClassPaths.Num())
	{
		{
			FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
			while (NextClass < ImportedClassPaths.Num())
			{
				const FTopLevelAssetPath& ClassPath = ImportedClassPaths[NextClass];
				FClassDigestData* ExistingData = ClassDigests.Map.Find(ClassPath);
				if (!ExistingData)
				{
					break;
				}
				NextClass++;

				// We only need to hash the ClosestNative class. EditorDomain packages that contain blueprint class
				// instances do not use versioned data, and the incremental recook of these packages will be triggered
				// when their class changes because they have an import dependency on their class's package.
				FClassDigestData* NativeData = ExistingData;
				if (ExistingData->ClosestNative != ClassPath)
				{
					if (bDumpCurrentPackage)
					{
						DigestDependencies->SetStringField(ClassPath.ToString(), TEXT("Non native class"));
					}
					NativeData = ClassDigests.Map.Find(ExistingData->ClosestNative);
					checkf(NativeData, TEXT("Classes are only stored in a ClosestNative field if they exist"));
				}
				check(NativeData->bNative);
				Writer.Update(&NativeData->InclusiveSchemaHash, sizeof(NativeData->InclusiveSchemaHash));
				if (bDumpCurrentPackage)
				{
					DigestDependencies->SetStringField(NativeData->ClosestNative.ToString(), LexToString(NativeData->InclusiveSchemaHash));
				}

				CustomVersionHandles.Append(NativeData->CustomVersionHandles);
				EnumSetFlagsAnd(Result.DomainUse, EDomainUse::LoadEnabled | EDomainUse::SaveEnabled,
					Result.DomainUse, NativeData->EditorDomainUse);
			}
		}

		if (NextClass < ImportedClassPaths.Num())
		{
			// EDITORDOMAIN_TODO: Remove the clauses !IsInGameThread || UE::IsSavingPackage() once FindObject no longer asserts if UE::IsSavingPackage()
			if (bHasTriedPrecacheClassDigests || !IsInGameThread() || UE::IsSavingPackage())
			{
				return FPackageDigest(FPackageDigest::EStatus::MissingClass,
					FName(WriteToString<256>(ImportedClassPaths[NextClass]).ToView()));
			}
			TConstArrayView<FTopLevelAssetPath> RemainingClasses =
				TConstArrayView<FTopLevelAssetPath>(ImportedClassPaths).RightChop(NextClass);
			PrecacheClassDigests(RemainingClasses);
			bHasTriedPrecacheClassDigests = true;
		}
	}

	CustomVersionHandles.Sort();
	CustomVersionHandles.SetNum(Algo::Unique(CustomVersionHandles));

	TArray<UE::AssetRegistry::FPackageCustomVersion> CustomVersions;
	TArray<FGuid> CustomVersionGuids;
	FKnownCustomVersions::FindGuidsChecked(CustomVersionGuids, CustomVersionHandles);
	CustomVersionGuids.Sort();
	CustomVersions.Reserve(CustomVersionGuids.Num());

	int32 CustomVersionCounter = 0;
	for (const FGuid& CustomVersionGuid : CustomVersionGuids)
	{
		Writer.Update(&CustomVersionGuid, sizeof(CustomVersionGuid));
		if (bDumpCurrentPackage)
		{
			DigestDependencies->SetStringField("PackageDigest.CustomVersionGuid_" + FString::FromInt(CustomVersionCounter), LexToString(CustomVersionGuid));
		}

		TOptional<FCustomVersion> CurrentVersion = FCurrentCustomVersions::Get(CustomVersionGuid);
		if (!CurrentVersion.IsSet())
		{
			return FPackageDigest(FPackageDigest::EStatus::MissingCustomVersion, FName(CustomVersionGuid.ToString()));
		}
		Writer.Update(&CurrentVersion->Version, sizeof(CurrentVersion->Version));
		if (bDumpCurrentPackage)
		{
			DigestDependencies->SetStringField("PackageDigest.CurrentVersion_" + FString::FromInt(CustomVersionCounter), FString::FromInt(CurrentVersion->Version));
			++CustomVersionCounter;
		}

		CustomVersions.Emplace(CustomVersionGuid, CurrentVersion->Version);
	}

	Result.Hash = Writer.Finalize();
	Result.Status = FPackageDigest::EStatus::Successful;
	Result.CustomVersions = UE::AssetRegistry::FPackageCustomVersionsHandle::FindOrAdd(CustomVersions);

	if (bDumpCurrentPackage)
	{
		FString JsonStr;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonStr);
		FJsonSerializer::Serialize(DigestDependencies.ToSharedRef(), JsonWriter);

		UE_LOGF(LogEditorDomain, Display, "---Package digest dependencies---");
		UE_LOGF(LogEditorDomain, Display, "%ls", *(PackageName.ToString()));
		UE_LOGF(LogEditorDomain, Display, "%ls", *JsonStr);
		
		FString PackageDigestHash = LexToString(Result.Hash);
		UE_LOGF(LogEditorDomain, Display, "Result Hash %ls", *PackageDigestHash);
		UE_LOGF(LogEditorDomain, Display, "---");
	}

	return Result;
}

static const FBlake3Hash* GetClassDigestInternal(FTopLevelAssetPath ClassPath, FClassDigestMap& ClassDigests)
{
	// Caller has read-locked ClassDigests.Lock.

	const FClassDigestData* ExistingData = ClassDigests.Map.Find(ClassPath);
	if (!ExistingData)
	{
		if (!IsInGameThread() || UE::IsSavingPackage())
		{
			return nullptr;
		}

		// PrecacheClassDigest will try to acquire a write lock. It can't upgrade a read lock to a write lock. That's why
		// I release the read lock here.
		ClassDigests.Lock.ReadUnlock();

		PrecacheClassDigest(ClassPath);

		ClassDigests.Lock.ReadLock();

		ExistingData = ClassDigests.Map.Find(ClassPath);
		if (!ExistingData)
		{
			return nullptr;
		}
	}

	// We only support hashes for the ClosestNative class
	const FClassDigestData* NativeData = ExistingData;
	if (ExistingData->ClosestNative != ClassPath)
	{
		NativeData = ClassDigests.Map.Find(ExistingData->ClosestNative);
		checkf(NativeData, TEXT("Classes are only stored in a ClosestNative field if they exist"));
	}
	check(NativeData->bNative);

	return &NativeData->InclusiveSchemaHash;
}

bool TryAppendClassDigests(const TArrayView<UE::Cook::FCookDependency>& Dependencies, FBlake3& Writer,
	UE::Cook::FCookDependencyContext& Context, bool bUpdateValues)
{
	FClassDigestMap& ClassDigests = GetClassDigests();
	FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);

	for (UE::Cook::FCookDependency& ClassDependency : Dependencies)
	{
		FStringView ClassPathStr = ClassDependency.GetClassPath();
		FTopLevelAssetPath ClassPath(ClassPathStr);
		if (!ClassPath.IsValid())
		{
			Context.LogError(FString::Printf(TEXT("ClassDependency failed: %.*s is not a valid TopLevelAssetPath."),
				ClassPathStr.Len(), ClassPathStr.GetData()));
			continue;
		}

		const FBlake3Hash* Hash = GetClassDigestInternal(ClassPath, ClassDigests);
		if (!Hash)
		{
			Context.LogError(FString::Printf(TEXT("%s is not a valid class."), *ClassPath.ToString()));
			return false;
		}

		if (bUpdateValues)
		{
			ClassDependency.SetValue(*Hash);
		}
		Writer.Update(&Hash->GetBytes(), sizeof(Hash->GetBytes()));
	}

	return true;
}

TOptional<FBlake3Hash> TryGetClassDigest(FTopLevelAssetPath ClassPath)
{
	if (!ClassPath.IsValid())
	{
		return TOptional<FBlake3Hash>();
	}

	FClassDigestMap& ClassDigests = GetClassDigests();
	FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);

	const FBlake3Hash* Hash = GetClassDigestInternal(ClassPath, ClassDigests);
	return Hash ? TOptional<FBlake3Hash>(*Hash) : TOptional<FBlake3Hash>();
}

/**
 * Holds context data for a call to PrecacheClassDigests, which needs to recursively
 * traverse a graph of of class parents and construction classes
 */
class FPrecacheClassDigest
{
public:
	FPrecacheClassDigest(FCustomVersionsStripedMap* CustomVersionsCache = nullptr)
		: ClassDigestsMap(GetClassDigests())
		, ClassDigests(ClassDigestsMap.Map)
		, AssetRegistry(IAssetRegistry::GetChecked())
		, CustomVersionsCache(CustomVersionsCache)
	{
		ClassDigestsMap.Lock.WriteLock();
	}

	~FPrecacheClassDigest()
	{
		ClassDigestsMap.Lock.WriteUnlock();
	}

	FClassDigestData* GetRecursive(FTopLevelAssetPath ClassPath, bool bAllowRedirects);

	struct FUnlockScope
	{
		FUnlockScope(FRWLock& InLock)
			: Lock(InLock)
		{
			Lock.WriteUnlock();
		}
		~FUnlockScope()
		{
			Lock.WriteLock();
		}
		FRWLock& Lock;
	};

private:
	FClassDigestMap& ClassDigestsMap;
	TMap<FTopLevelAssetPath, FClassDigestData>& ClassDigests;
	IAssetRegistry& AssetRegistry;
	FCustomVersionsStripedMap* CustomVersionsCache;

	// Scratch variables usable during GetRecursive; they are invalidated when a recursive call is made
	FString NameStringBuffer;
	TArray<FTopLevelAssetPath> AncestorClassPaths;
};

FClassDigestData* FPrecacheClassDigest::GetRecursive(FTopLevelAssetPath ClassPath, bool bAllowRedirects)
{
	SCOPED_EDITORDOMAINTIMER(FPrecacheClassDigest::GetRecursive);
	check(!ClassPath.IsNull());

	static FString ClassName;
	static bool bDebugNullCook = FParse::Value(FCommandLine::Get(), TEXT("-DebugNullCook="), ClassName);
	bool bPrintLogsDebugNullCook = bDebugNullCook && ClassPath.GetAssetName().ToString() == ClassName;

	if (bPrintLogsDebugNullCook)
	{
		UE_LOGF(LogEditorDomain, Display, "[DebugNullCook] Calculating class digest for class %ls", *ClassPath.ToString());
		UE_LOGF(LogEditorDomain, Display, "[DebugNullCook] bAllowRedirects=%d", bAllowRedirects);
#if UE_WITH_PACKAGE_ACCESS_TRACKING
		const PackageAccessTracking_Private::FTrackedData* TrackedData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
		if (TrackedData)
		{
			UE_LOGF(LogEditorDomain, Display, "[DebugNullCook] Package %ls", *TrackedData->PackageName.ToString());
		}
#endif

		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
	}

	// Called within ClassDigestsMap.Lock.WriteLock()
	FClassDigestData* DigestData = &ClassDigests.FindOrAdd(ClassPath);
	if (DigestData->bConstructed)
	{
		return DigestData;
	}
	DigestData->bConstructed = true;

	if (bAllowRedirects)
	{
		FTopLevelAssetPath RedirectedClassPath(ClassPath);
		bool bRedirected;
		CoreRedirectClassPath(RedirectedClassPath, &bRedirected);
		if (bRedirected)
		{ 
			FClassDigestData* RedirectedDigest = GetRecursive(RedirectedClassPath, false /* bAllowRedirects */);
			check(RedirectedDigest && RedirectedDigest->bConstructed && RedirectedDigest->bConstructionComplete);
			// The map has possibly been modified so we need to recalculate the address of ClassName's DigestData
			DigestData = &ClassDigests.FindChecked(ClassPath);
			*DigestData = *RedirectedDigest;
			DigestData->bIsRedirected = true;

			return DigestData;
		}

		if (bPrintLogsDebugNullCook)
		{
			UE_LOGF(LogEditorDomain, Display, "[DebugNullCook] Failed to find redirection for class %ls", *ClassPath.ToString());
		}

	}

	// Fill in digest data config-driven flags
	DigestData->EditorDomainUse = EDomainUse::LoadEnabled | EDomainUse::SaveEnabled;
	DigestData->EditorDomainUse &= ~MapFindRef(GClassBlockedUses, ClassPath, EDomainUse::None);

	// Fill in native-specific digest data, get the ParentName, and if non-native, get the native ancestor struct
	UStruct* Struct = nullptr;
	bool bIsNative = FPackageName::IsScriptPackage(WriteToString<256>(ClassPath.GetPackageName()));
	if (bIsNative)
	{
		Struct = FindObject<UStruct>(ClassPath);
	}
	FTopLevelAssetPath ParentClassPath;
	FBlake3Hash ExclusiveSchemaHash;
	if (bIsNative)
	{
		if (Struct)
		{
			DigestData->ClosestNative = ClassPath;
			DigestData->bNative = true;
			ExclusiveSchemaHash = Struct->GetSchemaHash(false /* bSkipEditorOnly */);
			UStruct* ParentStruct = Struct->GetSuperStruct();
			if (ParentStruct)
			{
				ParentClassPath = FTopLevelAssetPath(ParentStruct);
			}
		}
		else
		{
			UE_LOGF(LogEditorDomain, Display,
				"Class %ls is imported by a package but does not exist in memory. EditorDomain keys for packages using it will be invalid if it still exists."
				"\n\tTo clear this message, resave packages that use the deleted class, or load its module earlier than the packages that use it are referenced.",
				*ClassPath.ToString());

			// Create placeholder data that acts as if this class is a leaf off of UObject
			DigestData->ClosestNative = FTopLevelAssetPath(UObject::StaticClass());
			check(DigestData->ClosestNative.IsValid());
			DigestData->bNative = true;
			ExclusiveSchemaHash.Reset();
			ParentClassPath = DigestData->ClosestNative;
		}
	}
	else
	{
		DigestData->bNative = false;
		ExclusiveSchemaHash.Reset();
		DigestData->CustomVersionHandles.Empty();

		// TODO_EDITORDOMAIN: If the class is not yet present in the assetregistry, or
		// if its parent classes are not, then we will not be able to propagate information from the parent classes; wait on the class to be parsed
		AncestorClassPaths.Reset();
		IAssetRegistry::Get()->GetAncestorClassNames(ClassPath, AncestorClassPaths);
		if (!AncestorClassPaths.IsEmpty())
		{
			ParentClassPath = AncestorClassPaths[0];
			for (const FTopLevelAssetPath& AncestorClassPath : AncestorClassPaths)
			{
				if (UStruct* CurrentStruct = FindObject<UStruct>(AncestorClassPath))
				{
					if (CurrentStruct->IsNative())
					{
						DigestData->ClosestNative = AncestorClassPath;
						break;
					}
				}
			}
		}
		if (ParentClassPath.IsNull())
		{
			ParentClassPath = FTopLevelAssetPath(UObject::StaticClass());
		}
		if (DigestData->ClosestNative.IsNull())
		{
			DigestData->ClosestNative = FTopLevelAssetPath(UObject::StaticClass());
		}
	}
	// ParentClassPath can be null for CoreUObject.Object and for structs.
	check(!DigestData->ClosestNative.IsNull());

	// Propagate values from the parent
	TArray<FTopLevelAssetPath> ConstructClasses; // Not a class scratch variable because we use it across recursive calls

	// Get the CustomVersions used by the native class; GetCustomVersions already returns all custom versions used
	// by the parent class so we do not need to copy data from the parent
	UClass* StructAsClass = Cast<UClass>(Struct);
	if (StructAsClass)
	{
		// GetCustomVersions and CallDeclareConstructClasses can create the ClassDefaultObject, which can trigger
		// LoadPackage, which can reenter this function recursively. We have to drop the lock to prevent a deadlock.
		FUnlockScope UnlockScope(ClassDigestsMap.Lock);
		FKnownCustomVersions::FindOrAddHandles(DigestData->CustomVersionHandles, GetCustomVersions(*StructAsClass, CustomVersionsCache));
		StructAsClass->CallDeclareConstructClasses(ConstructClasses);
	}
	else
	{
		DigestData->CustomVersionHandles.Reset();
	}

	if (!ParentClassPath.IsNull())
	{
		// Set bAllowRedirects = false. ParentClassPath already has been CoreRedirected, because it is from the native
		// struct's parent class pointer or it is from GetAncestorClassNames which gives the redirected ancestors.
		FClassDigestData* ParentDigest = GetRecursive(ParentClassPath, false /* bAllowRedirects */);
		// The map has possibly been modified so we need to recalculate the address of ClassName's DigestData
		DigestData = &ClassDigests.FindChecked(ClassPath);
		if (!ParentDigest)
		{
			UE_LOGF(LogEditorDomain, Display,
				"Parent class %ls of class %ls not found. Allow flags for editordomain and incremental cooking will be invalid.",
				*ParentClassPath.ToString(), *ClassPath.ToString());
		}
		else
		{
			if (!ParentDigest->bConstructionComplete)
			{
				// Suppress the warning for MulticastDelegateProperty, which has a redirector to its own child class
				// of MulticastInlineDelegateProperty
				// We could fix this case by adding bAllowRedirects to the ClassDigestsMap lookup key, but it's not a
				// problem for MuticastDelegateProperty and we don't have any other cases where it is a problem, so we
				// avoid the performance cost of doing so.
				if (ClassPath != FTopLevelAssetPath(TEXT("/Script/CoreUObject.MulticastInlineDelegateProperty")))
				{
					UE_LOGF(LogEditorDomain, Display,
						"Cycle detected in parents of class %ls. Allow flags for editordomain and incremental cooking will be invalid.",
						*ClassPath.ToString());
				}
			}
			EnumSetFlagsAnd(DigestData->EditorDomainUse, EDomainUse::LoadEnabled | EDomainUse::SaveEnabled,
				DigestData->EditorDomainUse, ParentDigest->EditorDomainUse);
			ConstructClasses.Append(ParentDigest->ConstructClasses);
			if (!StructAsClass)
			{
				DigestData->CustomVersionHandles = ParentDigest->CustomVersionHandles;
			}
		}
	}
	// Also add ConstructClasses from ini. MultiFind appends to the out argument rather than overwriting it.
	GConstructClasses.MultiFind(ClassPath, ConstructClasses);

	// Validate the construct classes, find all the transitive constructclasses, and create the InclusiveSchema
	TArray<FTopLevelAssetPath> TransitiveConstructClasses;
	TArray<int32> TransitiveCustomVersionHandles;
	bool bHasConstructClasses = false;
	if (ConstructClasses.Num())
	{
		ConstructClasses.RemoveAll([](FTopLevelAssetPath& ClassPath) { return ClassPath.IsNull(); });
		// Sort ConstructClasses lexically so the Inclusive schema is deterministic
		Algo::Sort(ConstructClasses, [](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.Compare(B) < 0; });
		ConstructClasses.SetNum(Algo::Unique(ConstructClasses));
		FBlake3 Writer;
		Writer.Update(&ExclusiveSchemaHash, sizeof(ExclusiveSchemaHash));
		bool bCycleDetected = false;

		for (int32 Index = 0; Index < ConstructClasses.Num(); ++Index)
		{
			FTopLevelAssetPath& ConstructClass = ConstructClasses[Index];
			if (ConstructClass == ClassPath)
			{
				continue;
			}
			FClassDigestData* ConstructClassDigest = GetRecursive(ConstructClass, true /* bAllowRedirects */);
			if (!ConstructClassDigest)
			{
				UE_LOGF(LogEditorDomain, Verbose,
					"Construct class %ls specified for class %ls is not found.",
					*ConstructClass.ToString(), *ClassPath.ToString());
				ConstructClasses.RemoveAt(Index);
				continue;
			}
			Writer.Update(&ConstructClassDigest->InclusiveSchemaHash, sizeof(ConstructClassDigest->InclusiveSchemaHash));
			TransitiveCustomVersionHandles.Append(ConstructClassDigest->CustomVersionHandles);

			TransitiveConstructClasses.Add(ConstructClass);
			for (const FTopLevelAssetPath& TransitiveConstructClass : ConstructClassDigest->ConstructClasses)
			{
				if (TransitiveConstructClass == ConstructClass)
				{
					bCycleDetected = true;
				}
				else
				{
					TransitiveConstructClasses.Add(TransitiveConstructClass);
				}
			}
		}
		if (bCycleDetected)
		{
			// Handling cycles is not yet implemented. If we need to handle it, we will have to change the graph
			// algorithm to keep track of referencers of inprogress classes and set their construct classes to include
			// all transitive constructclasses of the strongly connected component.
			UE_LOGF(LogEditorDomain, Error,
				"Cycle detected in ConstructClasses of class %ls. Packages with classes that can construct %ls will sometimes fail to update during incremental cooks.",
				*ClassPath.ToString(), *ClassPath.ToString());
		}
		// The map has possibly been modified so we need to recalculate the address of ClassName's DigestData
		DigestData = &ClassDigests.FindChecked(ClassPath);

		// Store the transitive data on the Digest
		if (TransitiveConstructClasses.Num())
		{
			bHasConstructClasses = true;
			DigestData->ConstructClasses = MoveTemp(TransitiveConstructClasses);
			DigestData->CustomVersionHandles.Append(TransitiveCustomVersionHandles);

			// The order of ConstructClasses is allowed to vary between runs; anything that uses it for deterministic data like a schema
			// hash will need to resort it. We set the order to use CompareFast so we can support fast binary search lookup.
			Algo::Sort(DigestData->ConstructClasses, [](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.CompareFast(B) < 0; });
			DigestData->ConstructClasses.SetNum(Algo::Unique(DigestData->ConstructClasses));
			Algo::Sort(DigestData->CustomVersionHandles);
			DigestData->CustomVersionHandles.SetNum(Algo::Unique(DigestData->CustomVersionHandles));
			
			DigestData->InclusiveSchemaHash = Writer.Finalize();
		}
	}
	if (!bHasConstructClasses)
	{
		DigestData->ConstructClasses.Reset();
		DigestData->InclusiveSchemaHash = ExclusiveSchemaHash;
	}

	DigestData->bConstructionComplete = true;
	return DigestData;
}

/** Try to add the FClassDigestData for each given class into the GetClassDigests map */
void PrecacheClassDigests(TConstArrayView<FTopLevelAssetPath> ClassPaths, FCustomVersionsStripedMap* CustomVersionsCache)
{
	SCOPED_EDITORDOMAINTIMER(EditorDomain::PrecacheClassDigests);
	FPrecacheClassDigest Digester(CustomVersionsCache);
	for (const FTopLevelAssetPath& ClassPath : ClassPaths)
	{
		if (!ClassPath.IsNull())
		{
			Digester.GetRecursive(ClassPath, true /* bAllowRedirects */);
		}
	}
}

void PrecacheClassDigest(const FTopLevelAssetPath& ClassPath)
{
	if (ClassPath.IsNull())
		return;

	FPrecacheClassDigest Digester;
	Digester.GetRecursive(ClassPath, true /* bAllowRedirects */);
}

/** Construct GGlobalConstructClasses from the classes specified by config */
void InitializeGlobalConstructClasses()
{
	if (bGGlobalConstructClassesInitialized)
	{
		return;
	}
	bGGlobalConstructClassesInitialized = true;
	{
		TArray<FString> Lines;
		GConfig->GetArray(TEXT("EditorDomain"), TEXT("GlobalCanConstructClasses"), Lines, GEditorIni);
		GGlobalConstructClasses.Reserve(Lines.Num());
		for (const FString& Line : Lines)
		{
			FStringView ClassPathText(FStringView(Line).TrimStartAndEnd());
			FTopLevelAssetPath ClassPath(ClassPathText);
			if (ClassPath.IsValid())
			{
				CoreRedirectClassPath(ClassPath);
				GGlobalConstructClasses.Add(ClassPath);
			}
			else
			{
				UE_LOGF(LogEditorDomain, Error, "Invalid ClassPath %.*ls specified by Editor.ini:[EditorDomain]:GlobalCanConstructClasses.",
					ClassPathText.Len(), ClassPathText.GetData());
			}
		}
	}

	PrecacheClassDigests(GGlobalConstructClasses);
	FClassDigestMap& ClassDigests = GetClassDigests();
	FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
	FBlake3 Writer;

	Algo::Sort(GGlobalConstructClasses, [](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.Compare(B) < 0; });
	GGlobalConstructClasses.SetNum(Algo::Unique(GGlobalConstructClasses));
	for (int32 Index = 0; Index < GGlobalConstructClasses.Num();)
	{
		const FTopLevelAssetPath& ClassPath = GGlobalConstructClasses[Index];
		FClassDigestData* ExistingData = ClassDigests.Map.Find(ClassPath);
		if (!ExistingData)
		{
			UE_LOGF(LogEditorDomain, Display,
				"Construct class %ls specified by Editor.ini:[EditorDomain]:GlobalCanConstructClasses is not found. "
				"This is a class that can be constructed automatically by SavePackage when saving old packages. "
				"Old packages that do not yet have this class will load more slowly.",
				*ClassPath.ToString());
			GGlobalConstructClasses.RemoveAt(Index);
		}
		else
		{
			Writer.Update(&ExistingData->InclusiveSchemaHash, sizeof(ExistingData->InclusiveSchemaHash));
			++Index;
		}
	}
	GGlobalConstructClassesHash = Writer.Finalize();
}

/** An archive that just collects custom versions. */
class FCustomVersionCollectorArchive : public FArchiveUObject
{
public:
	FCustomVersionCollectorArchive()
	{
		// Use the same Archive properties that are used by FPackageHarvester, since that
		// is the authoritative way of collecting CustomVersions used in the save
		SetIsSaving(true);
		SetIsPersistent(true);
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
	}
	// The base class functionality does most of what we need:
	// ignore Serialize(void*,int), ignore Serialize(UObject*), collect customversions
	// Some classes expect Seek and Tell to work, though, so we simulate those
	virtual void Seek(int64 InPos) override
	{
		check(0 <= Pos && Pos <= Max);
		Pos = InPos;
	}
	virtual int64 Tell() override
	{
		return Pos;
	}
	virtual int64 TotalSize() override
	{
		return Max;
	}
	virtual void Serialize(void* V, int64 Length) override
	{
		Pos += Length;
		Max = FMath::Max(Pos, Max);
	}
	virtual FString GetArchiveName() const override
	{
		return TEXT("FCustomVersionCollectorArchive");
	}

	virtual FArchive& operator<<(FObjectPtr& Value) override
	{
		return *this;
	}

private:
	int64 Pos = 0;
	int64 Max = 0;
};

/** Collect the CustomVersions that can be used by the given Class when it is saved */
TArray<FGuid, TInlineAllocator<1>> GetCustomVersions(UClass& Class, FCustomVersionsStripedMap* InOutCache)
{
	SCOPED_EDITORDOMAINTIMER(EditorDomain::GetCustomVersions);
	TArray<FGuid, TInlineAllocator<1>> Guids;
	if (InOutCache && InOutCache->FindAndApply(&Class, [&Guids](const TArray<FGuid, TInlineAllocator<1>>& Cached) { Guids = Cached; }))
	{
		return Guids;
	}
	FCustomVersionCollectorArchive Ar;
	Class.CallDeclareCustomVersions(Ar);
	// Default objects of blueprint classes are serialized during SavePackage with a special call to
	// UBlueprintGeneratedClass::SerializeDefaultObject
	// All packages that include a BlueprintGeneratedClass import the UClass BlueprintGeneratedClass
	// (Note the UClass BlueprintGeneratedClass is not the same as the c++ UBlueprintGeneratedClass)
	// We therefore add on the CustomVersions used by UBlueprintGeneratedClass::SerializeDefaultObject into
	// the CustomVersions for the UClass named BlueprintGeneratedClass
	static const FName NAME_EnginePackage(TEXT("/Script/Engine"));
	static const FName NAME_BlueprintGeneratedClass(TEXT("BlueprintGeneratedClass"));
	if (Class.GetFName() == NAME_BlueprintGeneratedClass && Class.GetPackage()->GetFName() == NAME_EnginePackage)
	{
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	}
	const FCustomVersionContainer& CustomVersions = Ar.GetCustomVersions();
	Guids.Reserve(CustomVersions.GetAllVersions().Num());
	for (const FCustomVersion& CustomVersion : CustomVersions.GetAllVersions())
	{
		Guids.Add(CustomVersion.Key);
	}
	Algo::Sort(Guids);
	Guids.SetNum(Algo::Unique(Guids));
	if (InOutCache)
	{
		InOutCache->Add(&Class, Guids);
	}
	return Guids;
}

/** Serialize each object in the package to find the one using each of the given CustomVersions */
TMap<FGuid, UObject*> FindCustomVersionCulprits(TConstArrayView<FGuid> UnknownGuids, UPackage* Package)
{
	TArray<UObject*> Objects;
	GetObjectsWithPackage(Package, Objects);
	TMap<FGuid, UObject*> Culprits;
	for (UObject* Object : Objects)
	{
		FCustomVersionCollectorArchive Ar;
		Object->Serialize(Ar);
		for (const FCustomVersion& CustomVersion : Ar.GetCustomVersions().GetAllVersions())
		{
			UObject*& Existing = Culprits.FindOrAdd(CustomVersion.Key);
			if (!Existing)
			{
				Existing = Object;
			}
		}
	}
	return Culprits;
}

TMap<FTopLevelAssetPath, EDomainUse> ConstructClassBlockedUses()
{
	TMap<FTopLevelAssetPath, EDomainUse> Result;
	TArray<FString> BlockListArray;
	TArray<FString> LoadBlockListArray;
	TArray<FString> SaveBlockListArray;
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("ClassBlockList"), BlockListArray, GEditorIni);
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("ClassLoadBlockList"), LoadBlockListArray, GEditorIni);
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("ClassSaveBlockList"), SaveBlockListArray, GEditorIni);
	for (TArray<FString>* Array : { &BlockListArray, &LoadBlockListArray, &SaveBlockListArray })
	{
		EDomainUse BlockedUse = Array == &BlockListArray	?	(EDomainUse::LoadEnabled | EDomainUse::SaveEnabled) : (
			Array == &LoadBlockListArray					?	EDomainUse::LoadEnabled : (
																EDomainUse::SaveEnabled));
		for (const FString& ClassPathString : *Array)
		{
			FTopLevelAssetPath ClassPath(ClassPathString);
			if (!ClassPath.IsValid())
			{
				continue;
			}
			CoreRedirectClassPath(ClassPath);
			Result.FindOrAdd(ClassPath, EDomainUse::None) |= BlockedUse;
		}
	}
	return Result;
}

TMap<FName, EDomainUse> ConstructPackageNameBlockedUses()
{
	TMap<FName, EDomainUse> Result;
	TArray<FString> BlockListArray;
	TArray<FString> LoadBlockListArray;
	TArray<FString> SaveBlockListArray;
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("PackageBlockList"), BlockListArray, GEditorIni);
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("PackageLoadBlockList"), LoadBlockListArray, GEditorIni);
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("PackageSaveBlockList"), SaveBlockListArray, GEditorIni);
	FString PackageNameString;
	FString ErrorReason;
	for (TArray<FString>* Array : { &BlockListArray, &LoadBlockListArray, &SaveBlockListArray })
	{
		EDomainUse BlockedUse = Array == &BlockListArray	?	(EDomainUse::LoadEnabled | EDomainUse::SaveEnabled) : (
				Array == &LoadBlockListArray				?	EDomainUse::LoadEnabled : (
																EDomainUse::SaveEnabled));
		for (const FString& PackageNameOrFilename : *Array)
		{
			if (!FPackageName::TryConvertFilenameToLongPackageName(PackageNameOrFilename, PackageNameString, &ErrorReason))
			{
				UE_LOGF(LogEditorDomain, Warning, "Editor.ini:[EditorDomain]:PackageBlocklist: Could not convert %ls to a LongPackageName: %ls",
					*PackageNameOrFilename, *ErrorReason);
				continue;
			}
			FName PackageName(*PackageNameString);
			FCoreRedirectObjectName PackageRedirect(NAME_None, NAME_None, PackageName);
			FCoreRedirectObjectName RedirectedPackageRedirect = FCoreRedirects::GetRedirectedName(
				ECoreRedirectFlags::Type_Package, PackageRedirect);
			if (!RedirectedPackageRedirect.OuterName.IsNone() || !RedirectedPackageRedirect.ObjectName.IsNone())
			{
				UE_LOGF(LogEditorDomain, Warning, "CoreRedirect for package %ls redirects to %ls which is not a package path. Ignoring it.",
					*PackageRedirect.ToString(), *RedirectedPackageRedirect.ToString())
			}
			else
			{
				PackageName = RedirectedPackageRedirect.PackageName;
			}
			Result.FindOrAdd(PackageName, EDomainUse::None) |= BlockedUse;
		}
	}
	return Result;
}

TSet<FTopLevelAssetPath> ConstructTargetIncrementalClassBlockList()
{
	TSet<FTopLevelAssetPath> Result;
	TArray<FString> DenyListArray;
	GConfig->GetArray(TEXT("CookSettings"), TEXT("IncrementalClassDenyList"), DenyListArray, GEditorIni);
	for (const FString& ClassPathString : DenyListArray)
	{
		FTopLevelAssetPath ClassPath(RemoveConfigComment(ClassPathString));
		if (ClassPath.IsValid())
		{
			CoreRedirectClassPath(ClassPath);
			Result.Add(ClassPath);
		}
	}
	return Result;
}

void ConstructTargetIterativeClassAllowList()
{
	SCOPED_EDITORDOMAINTIMER(EditorDomain::ConstructTargetIterativeClassAllowList);
	TSet<FTopLevelAssetPath> BlockListClassPaths = ConstructTargetIncrementalClassBlockList();

	// AllowList elements implicitly allow all parent classes, so instead of consulting a list and propagating
	// from parent classes every time we read a new class, we have to iterate the list for all classes up front and
	// propagate _TO_ parent classes. Note that we only support allowlisting native classes, otherwise we would have
	// to wait for the AssetRegistry to finish loading to be sure we could find every specified allowed class.

	// Declare a recursive Visit function. Every class we visit is allowlisted, and we visit its superclasses.
	// To decide whether a visited class is enabled, we also have to get IsBlockListed recursively from the parent.
	TSet<FTopLevelAssetPath> EnabledClassPaths;
	TStringBuilder<256> NameStringBuffer;
	TMap<FTopLevelAssetPath, TOptional<bool>> Visited;
	auto EnableClassIfNotBlocked = [&Visited, &EnabledClassPaths, &BlockListClassPaths, &NameStringBuffer]
		(UStruct* Struct, bool& bOutIsBlocked, auto& EnableClassIfNotBlockedRef)
	{
		FTopLevelAssetPath ClassPath(Struct);
		if (!ClassPath.IsValid())
		{
			return;
		}
		int32 KeyHash = GetTypeHash(ClassPath);
		TOptional<bool>& BlockedValue = Visited.FindOrAddByHash(KeyHash, ClassPath);
		if (BlockedValue.IsSet())
		{
			bOutIsBlocked = *BlockedValue;
			return;
		}
		BlockedValue = false; // If there is a cycle in the class graph, we will encounter PathName again, so initialize to false

		bool bParentBlocked = false;
		UStruct* ParentStruct = Struct->GetSuperStruct();
		if (ParentStruct)
		{
			EnableClassIfNotBlockedRef(ParentStruct, bParentBlocked, EnableClassIfNotBlockedRef);
		}

		bOutIsBlocked = bParentBlocked || BlockListClassPaths.Contains(ClassPath);
		if (bOutIsBlocked)
		{
			// Call FindOrAdd again, since the recursive calls may have altered the map and invalidated the BlockedValue reference
			Visited.FindOrAddByHash(KeyHash, ClassPath) = bOutIsBlocked;
		}
		else
		{
			EnabledClassPaths.Add(ClassPath);
		}
	};

	TArray<FString> AllowListLeafNames;
	GConfig->GetArray(TEXT("CookSettings"), TEXT("IncrementalClassAllowList"), AllowListLeafNames, GEditorIni);
	TSet<UStruct*> AllowListClasses;
	for (const FString& ClassPathString : AllowListLeafNames)
	{
		FTopLevelAssetPath ClassPath(ClassPathString);
		if (!ClassPath.IsValid())
		{
			continue;
		}
		CoreRedirectClassPath(ClassPath);
		if (!FPackageName::IsScriptPackage(WriteToString<256>(ClassPath.GetPackageName()).ToView()))
		{
			continue;
		}
		UStruct* Struct = FindObject<UStruct>(ClassPath);
		if (!Struct)
		{
			continue;
		}
		AllowListClasses.Add(Struct);
	}

	enum class EAllowCommand
	{
		Allow,
		Deny,
		Invalid,
	};

	TDirectoryTree<EAllowCommand> ScriptPackageAllowPaths;
	TArray<FString> ScriptPackageAllowListLines;
	GConfig->GetArray(TEXT("CookSettings"), TEXT("IncrementalClassScriptPackageAllowList"), ScriptPackageAllowListLines, GEditorIni);
	for (const FString& Line : ScriptPackageAllowListLines)
	{
		TArray<FStringView, TInlineAllocator<2>> Tokens;
		UE::String::ParseTokens(Line, TEXT(","), Tokens, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);
		EAllowCommand Command = EAllowCommand::Invalid;
		FString ScriptPath;

		if (Tokens.Num() == 2)
		{
			if (Tokens[0] == TEXTVIEW("Allow"))
			{
				Command = EAllowCommand::Allow;
			}
			else if (Tokens[0] == TEXTVIEW("Deny") || Tokens[0] == TEXTVIEW("Block"))
			{
				Command = EAllowCommand::Deny;
			}
			ScriptPath = Tokens[1];
			ScriptPath = ScriptPath.Replace(TEXT("<ProjectRoot>"), *FPaths::ProjectDir());
			ScriptPath = ScriptPath.Replace(TEXT("<EngineRoot>"), *FPaths::EngineDir());
			FPaths::MakeStandardFilename(ScriptPath);
		}
		if (Command == EAllowCommand::Invalid || ScriptPath.IsEmpty())
		{
			UE_LOGF(LogEditorDomain, Error,
				"Invalid value for Editor.ini:[CookSettings]:IncrementalClassScriptPackageAllowList:"
				"\n\t%ls"
				"\n\tExpected form\n\t+IncrementalClassScriptPackageAllowList=<Allow|Deny>,../../../Engine",
				*Line);
			continue;
		}
		ScriptPackageAllowPaths.FindOrAdd(ScriptPath) = Command;
	}

	// For every known /Script/ package, test whether it is allow-listed in IncrementalClassScriptPackageAllowList,
	// and if so add all classes and structs in the package to AllowListClasses 
	FModuleManager& ModuleManager = FModuleManager::Get();
	IPluginManager& PluginManager = IPluginManager::Get();
	for (TObjectIterator<UPackage> PackageIter; PackageIter; ++PackageIter)
	{
		UPackage* Package = *PackageIter;
		TStringBuilder<256> PackageName(InPlace, Package->GetFName());
		FStringView ModuleName;
		if (!FPackageName::TryConvertScriptPackageNameToModuleName(PackageName, ModuleName))
		{
			continue;
		}

		FString ModuleSourceDir;
		FName ModuleNameFName(ModuleName);

		// Try to find the path to the script package's source.
		// All script packages come from a module, but some modules are not loaded until load/save of the package
		// (even though their script package is loaded at startup)
		// First  ask the PluginManager whether any plugin claims the module. If so, use the plugin's directory.
		TSharedPtr<IPlugin> Plugin = PluginManager.GetModuleOwnerPlugin(ModuleNameFName);
		if (Plugin)
		{
			ModuleSourceDir = Plugin->GetBaseDir();
		}
		else
		{
			// Not claimed by a plugin, so its either a module in /Engine or /Game. To find out which, first
			// try the ModuleManager and use bIsGameModule.
			FModuleStatus ModuleStatus;
			if (ModuleManager.QueryModule(ModuleNameFName, ModuleStatus))
			{
				ModuleSourceDir = FPaths::Combine(ModuleStatus.bIsGameModule ? FPaths::ProjectDir() : FPaths::EngineDir(), TEXT("Source"));
			}
			else
			{
				// Not yet known to the module manager; maybe it just hasn't been loaded
				FString ModuleFilePath;
				if (ModuleManager.ModuleExists(*WriteToString<256>(ModuleName), &ModuleFilePath))
				{
					bool bIsGameModule = false;
					// In a monolithic compile, ModuleFilePath will be empty.
					// For now, treat all of those as EnginePath. TODO: Modify IMPLEMENT_MODULE to specify whether the module is game or engine
					if (ModuleFilePath.IsEmpty())
					{
						bIsGameModule = false;
					}
					else
					{
						// ModuleFilePath is the path to the dll, and since it is not a plugin, should be either in Engine/Binaries/Win64 or <ProjectRoot>/Binaries/Win64
						FPaths::MakeStandardFilename(ModuleFilePath);
						bIsGameModule = !FPathViews::IsParentPathOf(FPaths::EngineDir(), ModuleFilePath);
					}
					ModuleSourceDir = FPaths::Combine(bIsGameModule ? FPaths::ProjectDir() : FPaths::EngineDir(), TEXT("Source"));
				}
				else
				{
					// else not a module that can ever be found in the module manager; this can happen if the module is missing IMPLEMENT_MODULE
					// e.g. CoreOnline
					UE_LOGF(LogEditorDomain, Verbose,
						"Could not find module for script package %ls. Packages with classes from this script package will not be incrementally skippable.",
						*PackageName);
					continue;
				}
			}
		}

		FPaths::MakeStandardFilename(ModuleSourceDir);
		EAllowCommand* AllowCommand = ScriptPackageAllowPaths.FindClosestValue(ModuleSourceDir);
		if (!AllowCommand || *AllowCommand != EAllowCommand::Allow)
		{
			continue;
		}

		ForEachObjectWithPackage(Package, [&AllowListClasses](UObject* Object)
			{
				UStruct* Class = Cast<UStruct>(Object);
				if (Class)
				{
					AllowListClasses.Add(Class);
				}
				return true;
			}, EGetObjectsFlags::None);
	}

	for (UStruct* Struct : AllowListClasses)
	{
		bool bUnusedIsBlocked;
		EnableClassIfNotBlocked(Struct, bUnusedIsBlocked, EnableClassIfNotBlocked);
	}

	TArray<FTopLevelAssetPath> EnabledClassPathsArray = EnabledClassPaths.Array();

	FCustomVersionsStripedMap CustomVersionsCache;
	if (bPrecalculateDataForClassDigests)
	{
		TArray<UStruct*> NativeStructs;
		{
			SCOPED_EDITORDOMAINTIMER(CreateNativeCDOs);
			// Gather the native structs on which PrecacheClassDigests operates on through GetRecursive.
			for (const FTopLevelAssetPath& ClassPath : EnabledClassPathsArray)
			{
				if (FPackageName::IsScriptPackage(WriteToString<256>(ClassPath.GetPackageName())))
				{
					if (UStruct* Struct = FindObject<UStruct>(ClassPath))
					{
						NativeStructs.Add(Struct);
						if (UClass* AsClass = Cast<UClass>(Struct))
						{
							// Create uninitialized CDOs serially that could be triggered by GetCustomVersions.
							AsClass->GetDefaultObject();
						}
					}
				}
			}
		}
		{
			SCOPED_EDITORDOMAINTIMER(CacheSchemasAndCustomVersions);
			// Precalculate schema hashes and custom versions for use during PrecacheClassDigests.
			// Both GetSchemaHash and GetCustomVersions are memoized.
			ParallelFor(NativeStructs.Num(), [&NativeStructs, &CustomVersionsCache](int32 Index)
			{
				NativeStructs[Index]->GetSchemaHash(false);
				if (UClass* AsClass = Cast<UClass>(NativeStructs[Index]))
				{
					GetCustomVersions(*AsClass, &CustomVersionsCache);
				}
			});
		}
		PrecacheClassDigests(EnabledClassPathsArray, &CustomVersionsCache);
	}
	else
	{
		PrecacheClassDigests(EnabledClassPathsArray);
	}
	
	FClassDigestMap& ClassDigests = GetClassDigests();
	{
		FWriteScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		for (const FTopLevelAssetPath& ClassPath : EnabledClassPathsArray)
		{
			FClassDigestData* DigestData = ClassDigests.Map.Find(ClassPath);
			if (DigestData)
			{
				DigestData->bTargetIterativeEnabled = true;
			}
		}

		// The previous loop only set the flag bTargetIterativeEnabled to target classes of redirection, not to the source class.
		// This is an issue in the case where class Source was renamed to class Target but a package was not resaved so it still contains references to class Source.
		// So when bTargetIterativeEnabled is set to true for class Target, I need to find class Source and set the flag for it as well.
		for (TPair<FTopLevelAssetPath, FClassDigestData>& Item : ClassDigests.Map)
		{
			FClassDigestData& OldClassDigest = Item.Value;
			if (!OldClassDigest.bIsRedirected)
			{
				continue;
			}
			
			FCoreRedirectObjectName OldName(Item.Key);
			FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Struct, OldName);
				
			if (NewName == OldName)
			{
				continue;
			}

			FTopLevelAssetPath NewNameClassPath(NewName.PackageName, NewName.ObjectName);
			FClassDigestData* NewClassDigest = ClassDigests.Map.Find(NewNameClassPath);
			if (NewClassDigest)
			{
				OldClassDigest.bTargetIterativeEnabled = NewClassDigest->bTargetIterativeEnabled;
			}
		}
	}
}

/** Construct PostLoadCanConstructClasses multimap from config settings and return it */
TMultiMap<FTopLevelAssetPath, FTopLevelAssetPath> ConstructConstructClasses()
{
	TArray<FString> Lines;
	GConfig->GetArray(TEXT("EditorDomain"), TEXT("PostLoadCanConstructClasses"), Lines, GEditorIni);
	TMultiMap<FTopLevelAssetPath, FTopLevelAssetPath> ConstructClasses;
	for (const FString& Line : Lines)
	{
		int32 NumTokens = 0;
		FStringView PostLoadClassString;
		FStringView ConstructedClassString;
		UE::String::ParseTokens(Line, TEXT(','),
			[&NumTokens, &PostLoadClassString, &ConstructedClassString](FStringView Token)
			{
				*(NumTokens == 0 ? &PostLoadClassString : &ConstructedClassString) = Token;
				++NumTokens;
			});
		if (NumTokens != 2)
		{
			UE_LOGF(LogEditorDomain, Warning, "Invalid value %ls in config setting Editor.ini:[EditorDomain]:PostLoadCanConstructClasses", *Line);
			continue;
		}
		PostLoadClassString.TrimStartAndEndInline();
		ConstructedClassString.TrimStartAndEndInline();
		FTopLevelAssetPath PostLoadClass(PostLoadClassString);
		FTopLevelAssetPath ConstructedClass(ConstructedClassString);
		CoreRedirectClassPath(PostLoadClass);
		CoreRedirectClassPath(ConstructedClass);
		ConstructClasses.Add(PostLoadClass, ConstructedClass);
	};
	return ConstructClasses;
}

/** A default implementation of IPackageDigestCache that stores the Digests in a TMap. */
class FDefaultPackageDigestCache : public IPackageDigestCache
{
public:
	virtual ~FDefaultPackageDigestCache()
	{
		if (this == IPackageDigestCache::Get())
		{
			IPackageDigestCache::Set(nullptr);
		}
	}

	virtual UE::EditorDomain::FPackageDigest GetPackageDigest(FName PackageName) override
	{
		uint32 TypeHash = GetTypeHash(PackageName);
		{
			FReadScopeLock ScopeLock(Lock);
			if (FPackageDigest* PackageDigest = PackageDigests.FindByHash(TypeHash, PackageName))
			{
				return *PackageDigest;
			}
		}
		FPackageDigest PackageDigest = CalculatePackageDigest(IAssetRegistry::GetChecked(), PackageName);
		{
			FWriteScopeLock ScopeLock(Lock);
			PackageDigests.AddByHash(TypeHash, PackageName, PackageDigest);
		}
		return PackageDigest;
	}

private:
	FRWLock Lock;
	TMap<FName, FPackageDigest> PackageDigests;
};
static TOptional<FDefaultPackageDigestCache> GDefaultPackageDigestCache;
static IPackageDigestCache* GPackageDigestCache = nullptr;
IPackageDigestCache* IPackageDigestCache::Get()
{
	return GPackageDigestCache;
}

void IPackageDigestCache::Set(IPackageDigestCache* Cache)
{
	// Set is called only before Main or after Main, during single-threaded startup/shutdown, so we do not need a lock
	GPackageDigestCache = Cache;
}

void IPackageDigestCache::SetDefault()
{
	GDefaultPackageDigestCache.Emplace();
	Set(GDefaultPackageDigestCache.GetPtrOrNull());
}

void UtilsInitialize()
{
	GClassBlockedUses = ConstructClassBlockedUses();
	GPackageBlockedUses = ConstructPackageNameBlockedUses();
	GConstructClasses = ConstructConstructClasses();

	double MaxBulkDataSize = 64 * 1024;
	GConfig->GetDouble(TEXT("EditorDomain"), TEXT("MaxBulkDataSize"), MaxBulkDataSize, GEditorIni);
	GMaxBulkDataSize = static_cast<uint64>(MaxBulkDataSize);

	COOK_STAT(UE::EditorDomain::CookStats::Register());
}

void UtilsCookInitialize()
{
	SCOPED_EDITORDOMAINTIMER(EditorDomain::UtilsCookInitialize);
	if (bGUtilsCookInitialized)
	{
		return;
	}
	bGUtilsCookInitialized = true;
	ConstructTargetIterativeClassAllowList();
	UE::TargetDomain::CookInitialize();
}

UE::DerivedData::FCacheKey GetEditorDomainPackageKey(const FIoHash& EditorDomainHash)
{
	static UE::DerivedData::FCacheBucket EditorDomainPackageCacheBucket(EditorDomainPackageBucketName);
	return UE::DerivedData::FCacheKey{EditorDomainPackageCacheBucket, EditorDomainHash };
}

static UE::DerivedData::FCacheKey GetBulkDataListKey(const FIoHash& EditorDomainHash)
{
	static UE::DerivedData::FCacheBucket BulkDataListBucket(BulkDataListBucketName);
	return UE::DerivedData::FCacheKey{ BulkDataListBucket, EditorDomainHash };
}

static UE::DerivedData::FCacheKey GetBulkDataPayloadIdKey(const FIoHash& PackageAndGuidHash)
{
	static UE::DerivedData::FCacheBucket BulkDataPayloadIdBucket(BulkDataPayloadIdBucketName);
	return UE::DerivedData::FCacheKey{ BulkDataPayloadIdBucket, PackageAndGuidHash };
}

static UE::DerivedData::FValueId GetBulkDataValueId()
{
	static UE::DerivedData::FValueId Id = UE::DerivedData::FValueId::FromName(ANSITEXTVIEW("Value"));
	return Id;
}

void RequestEditorDomainPackage(const FPackagePath& PackagePath,
	const FIoHash& EditorDomainHash, UE::DerivedData::ECachePolicy SkipFlags, UE::DerivedData::IRequestOwner& Owner,
	UE::DerivedData::FOnCacheGetComplete&& Callback)
{
	using namespace UE::DerivedData;

	ICache& Cache = GetCache();
	checkf((SkipFlags & (~ECachePolicy::SkipData)) == ECachePolicy::None,
		TEXT("SkipFlags should only contain ECachePolicy::Skip* flags"));

	// Set the CachePolicy to only query from local; we do not want to wait for download from remote.
	// Downloading from remote is done in batch  see FRequestCluster::StartAsync.
	// But set the CachePolicy to store into remote. This will cause the CacheStore to push
	// any existing local value into upstream storage and refresh the last-used time in the upstream.
	ECachePolicy CachePolicy = SkipFlags | ECachePolicy::Local | ECachePolicy::StoreRemote;
	Cache.Get({{{PackagePath.GetDebugName()}, GetEditorDomainPackageKey(EditorDomainHash), CachePolicy}}, Owner, MoveTemp(Callback));
}

/** Stores data from SavePackage in accessible fields */
class FEditorDomainPackageWriter final : public TPackageWriterToSharedBuffer<FBasePackageWriter>
{
public:
	// IPackageWriter
	virtual FCapabilities GetCapabilities() const override
	{
		FCapabilities Result;
		Result.bDeclareRegionForEachAdditionalFile = true;
		return Result;
	}

	/** Deserialize the CustomVersions out of the PackageFileSummary that was serialized into the header */
	bool TryGetClassesAndVersions(TArray<FName>& OutImportedClasses, FCustomVersionContainer& OutVersions)
	{
		FMemoryReaderView HeaderArchive(SavedRecord.Packages[0].Buffer.GetView());
		FPackageReader PackageReader;
		if (!PackageReader.OpenPackageFile(&HeaderArchive))
		{
			return false;
		}

		OutVersions = PackageReader.GetPackageFileSummary().GetCustomVersionContainer();
		if (!PackageReader.ReadImportedClasses(OutImportedClasses))
		{
			return false;
		}
		return true;
	}

	struct FAttachment
	{
		FSharedBuffer Buffer;
		UE::DerivedData::FValueId ValueId;
	};
	/** The Buffer+Id for each section making up the EditorDomain's copy of the package */
	TConstArrayView<FAttachment> GetAttachments() const
	{
		return Attachments;
	}

	uint64 GetBulkDataSize() const
	{
		return BulkDataSize;
	}

	uint64 GetFileSize() const
	{
		return FileSize;
	}

protected:
	virtual void CommitPackageInternal(FPackageWriterRecords::FPackage&& Record,
		const FCommitPackageInfo& Info) override
	{
		// CommitPackage is called below with these options
		check(Info.Attachments.Num() == 0);
		check(Info.Status == IPackageWriter::ECommitStatus::Success);
		check(Info.WriteOptions == IPackageWriter::EWriteOptions::Write);
		if (Record.AdditionalFiles.Num() > 0)
		{
			// WriteAdditionalFile is only used when saving cooked packages or for SidecarDataToAppend
			// We don't handle cooked, and SidecarDataToAppend is not yet used by anything.
			// To implement this we will need to
			// 1) Add a segment argument to IPackageWriter::FAdditionalFileInfo
			// 2) Create MetaData for the EditorDomain package
			// 3) Save the sidecar segment as a separate Attachment.
			// 4) List sidecar segment and appended-to-exportsarchive segments in the metadata.
			// 5) Change FEditorDomainPackageSegments to have a separate way to request the sidecar segment.
			// 6) Handle EPackageSegment::PayloadSidecar in FEditorDomain::OpenReadPackage by returning an archive configured to deserialize the sidecar segment.
			unimplemented();
		}
		
		// EditorDomain save does not support multi package outputs
		checkf(Record.Packages.Num() == 1, TEXT("Multioutput not supported"));
		FPackageWriterRecords::FWritePackage& Package = Record.Packages[0];

		TArray<FSharedBuffer> AttachmentBuffers;
		for (const FFileRegion& FileRegion : Package.Regions)
		{
			checkf(FileRegion.Type == EFileRegionType::None, TEXT("Does not support FileRegion types other than None."));
		}
		check(Package.Buffer.GetSize() > 0); // Header+Exports segment is non-zero in length
		AttachmentBuffers.Add(Package.Buffer);

		BulkDataSize = 0;
		for (const FBulkDataRecord& BulkRecord : Record.BulkDatas)
		{
			checkf(BulkRecord.Info.BulkDataType == IPackageWriter::FBulkDataInfo::AppendToExports ||
				BulkRecord.Info.BulkDataType == IPackageWriter::FBulkDataInfo::BulkSegment,
				TEXT("Does not support BulkData types other than AppendToExports and BulkSegment."));

			const uint8* BufferStart = reinterpret_cast<const uint8*>(BulkRecord.Buffer.GetData());
			uint64 SizeFromRegions = 0;
			for (const FFileRegion& FileRegion : BulkRecord.Regions)
			{
				checkf(FileRegion.Type == EFileRegionType::None,
					TEXT("Does not support FileRegion types other than None."));
				checkf(FileRegion.Offset + FileRegion.Length <= BulkRecord.Buffer.GetSize(),
					TEXT("FileRegions in WriteBulkData were outside of the range of the BulkData's size."));
				check(FileRegion.Length > 0); // SavePackage must not call WriteBulkData with empty bulkdatas

				AttachmentBuffers.Add(FSharedBuffer::MakeView(BufferStart + FileRegion.Offset,
					FileRegion.Length, BulkRecord.Buffer));
				SizeFromRegions += FileRegion.Length;
			}
			checkf(SizeFromRegions == BulkRecord.Buffer.GetSize(), TEXT("Expects all BulkData to be in a region."));
			BulkDataSize += BulkRecord.Buffer.GetSize();
		}
		for (const FLinkerAdditionalDataRecord& AdditionalRecord : Record.LinkerAdditionalDatas)
		{
			const uint8* BufferStart = reinterpret_cast<const uint8*>(AdditionalRecord.Buffer.GetData());
			uint64 SizeFromRegions = 0;
			for (const FFileRegion& FileRegion : AdditionalRecord.Regions)
			{
				checkf(FileRegion.Type == EFileRegionType::None,
					TEXT("Does not support FileRegion types other than None."));
				checkf(FileRegion.Offset + FileRegion.Length <= AdditionalRecord.Buffer.GetSize(),
					TEXT("FileRegions in WriteLinkerAdditionalData were outside of the range of the Data's size."));
				check(FileRegion.Length > 0); // SavePackage must not call WriteLinkerAdditionalData with empty regions

				AttachmentBuffers.Add(FSharedBuffer::MakeView(BufferStart + FileRegion.Offset,
					FileRegion.Length, AdditionalRecord.Buffer));
				SizeFromRegions += FileRegion.Length;
			}
			checkf(SizeFromRegions == AdditionalRecord.Buffer.GetSize(),
				TEXT("Expects all LinkerAdditionalData to be in a region."));
			BulkDataSize += AdditionalRecord.Buffer.GetSize();
		}
		checkf(Record.PackageTrailers.Num() <= 1, TEXT("MultiOutput not supported"));
		if (Record.PackageTrailers.Num() == 1)
		{
			FPackageTrailerRecord& PackageTrailer = Record.PackageTrailers[0];
			const uint8* BufferStart = reinterpret_cast<const uint8*>(PackageTrailer.Buffer.GetData());
			int64 BufferSize = PackageTrailer.Buffer.GetSize();
			AttachmentBuffers.Add(FSharedBuffer::MakeView(BufferStart, BufferSize));
			BulkDataSize += BufferSize;
		}

		// We use a counter for ValueIds rather than hashes of the Attachments. We do this because
		// some attachments may be identical, and Attachments are not allowed to have identical ValueIds.
		// We need to keep the duplicate copies of identical payloads because BulkDatas were written into
		// the exports with offsets that expect all attachment segments to exist in the segmented archive.
		auto IntToValueId = [](uint32 Value)
		{
			alignas(decltype(Value)) UE::DerivedData::FValueId::ByteArray Bytes{};
			static_assert(sizeof(Bytes) >= sizeof(Value), "We are storing an integer counter in the Bytes array");
			// The ValueIds are sorted as an array of bytes, so the bytes of the integer must be written big-endian
			for (int ByteIndex = 0; ByteIndex < sizeof(Value); ++ByteIndex)
			{
				Bytes[sizeof(Bytes) - 1 - ByteIndex] = static_cast<uint8>(Value & 0xff);
				Value >>= 8;
			}
			return UE::DerivedData::FValueId(Bytes);
		};

		uint32 AttachmentIndex = 1; // 0 is not a valid value for IntToValueId
		Attachments.Reserve(AttachmentBuffers.Num());
		FileSize = 0;
		for (const FSharedBuffer& Buffer : AttachmentBuffers)
		{
			Attachments.Add(FAttachment{ Buffer, IntToValueId(AttachmentIndex++) });
			FileSize += Buffer.GetSize();
		}

		// Our Attachments are views into SharedBuffers that are stored on the Record. Keep those SharedBuffers
		// alive until we are done with them. We also need access to Record.Packages[0] later.
		SavedRecord = MoveTemp(Record);
	}

private:
	TArray<FAttachment> Attachments;
	FPackageWriterRecords::FPackage SavedRecord;
	uint64 FileSize = 0;
	uint64 BulkDataSize = 0;
};

static FGuid IoHashToGuid(const FIoHash& Hash)
{
	const uint32* HashInts = reinterpret_cast<const uint32*>(Hash.GetBytes());
	return FGuid(HashInts[0], HashInts[1], HashInts[2], HashInts[3]);
}

void ValidateEditorDomainDeterminism(FName PackageName, bool bStorageResultValid,
	FEditorDomainPackageWriter* PackageWriter, UE::DerivedData::FCacheGetResponse&& GetResponse);

bool TrySavePackage(UPackage* Package)
{
	using namespace UE::DerivedData;
	FEditorDomain* EditorDomain = FEditorDomain::Get();
	if (!EditorDomain)
	{
		return false;
	}

	FName PackageName = Package->GetFName();
	FString ErrorMessage;
	FPackageDigest PackageDigest = EditorDomain->GetPackageDigest(PackageName);
	if (!PackageDigest.IsSuccessful())
	{
		UE_LOGF(LogEditorDomain, Warning, "Could not save package to EditorDomain: %ls. Reason: %ls",
			*WriteToString<256>(PackageName), *PackageDigest.GetStatusString());
		return false;
	}
	if (!EnumHasAnyFlags(PackageDigest.DomainUse, EDomainUse::SaveEnabled))
	{
		UE_LOGF(LogEditorDomain, Verbose, "Skipping save of blocked package to EditorDomain: %ls.",
			*WriteToString<256>(PackageName));
		return false;
	}
	if (GMaxBulkDataSize >= 0 &&
		IBulkDataRegistry::Get().GetBulkDataResaveSize(PackageName) > static_cast<uint64>(GMaxBulkDataSize))
	{
		UE_LOGF(LogEditorDomain, Verbose, "Skipping save of package with new BulkData to EditorDomain: %ls.",
			*WriteToString<256>(PackageName));
		return false;
	}
	TOptional<FAssetPackageData> PackageData = IAssetRegistry::Get()->GetAssetPackageDataCopy(Package->GetFName());
	if (!PackageData)
	{
		UE_LOGF(LogEditorDomain, Verbose, "Could not save %ls to EditorDomain: It does not have An AssetPackageData in the AssetRegistry.",
			*WriteToString<256>(PackageName));
		return false;
	}
	COOK_STAT(auto Timer = UE::EditorDomain::CookStats::Usage.TimeSyncWork());

	UE_LOGF(LogEditorDomain, Verbose, "Saving to EditorDomain: %ls.", *WriteToString<256>(PackageName));

	SCOPED_CUSTOM_EDITORDOMAINTIMER(EditorDomain_TrySavePackage)
		ADD_CUSTOM_EDITORDOMAINTIMER_META(EditorDomain_TrySavePackage, PackageName, *WriteToString<256>(PackageName));

	uint32 SaveFlags = SAVE_NoError // Do not crash the SaveServer on an error
		| SAVE_BulkDataByReference	// EditorDomain saves reference bulkdata from the WorkspaceDomain rather than duplicating it
		| SAVE_Async				// SavePackage support for PackageWriter is only implemented with SAVE_Async
		;

	TArray<UObject*> PackageObjects;
	auto GetPackageObjects = [&PackageObjects, Package]() -> TArrayView<UObject*>
	{
		if (PackageObjects.IsEmpty())
		{
			GetObjectsWithPackage(Package, PackageObjects);
		}
		return PackageObjects;
	};
	if (GetEditorDomainSaveUnversioned())
	{
		// With some exceptions, EditorDomain packages are saved unversioned; 
		// editors request the appropriate version of the EditorDomain package matching their serialization version
		bool bSaveUnversioned = true;
		for (UObject* Object : GetPackageObjects())
		{
			UClass* Class = Object ? Object->GetClass() : nullptr;
			if (Class && Class->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				// EDITOR_DOMAIN_TODO: Revisit this once we track package schemas
				// Packages with Blueprint class instances can not be saved unversioned,
				// as the Blueprint class's layout can change during the editor's lifetime,
				// and we don't currently have a way to keep track of the changing package schema
				bSaveUnversioned = false;
			}
		}
		SaveFlags |= bSaveUnversioned ? SAVE_Unversioned_Properties : 0;
	}

	FEditorDomainPackageWriter* PackageWriter = new FEditorDomainPackageWriter();
	IPackageWriter::FBeginPackageInfo BeginInfo;
	BeginInfo.PackageName = Package->GetFName();
	PackageWriter->BeginPackage(BeginInfo);
	FSavePackageContext SavePackageContext(nullptr /* TargetPlatform */, PackageWriter);
	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;
	SaveArgs.SaveFlags = SaveFlags;
	SaveArgs.bSlowTask = false;
	SaveArgs.SavePackageContext = &SavePackageContext;
	FSavePackageResultStruct Result = GEditor->Save(Package, nullptr, TEXT("EditorDomainPackageWriter"), SaveArgs);
	if (Result.Result != ESavePackageResult::Success)
	{
		UE_LOGF(LogEditorDomain, Warning, "Could not save %ls to EditorDomain: SavePackage returned %d.",
					 *Package->GetName(), int(Result.Result));
		return false;
	}

	ICookedPackageWriter::FCommitPackageInfo Info;
	Info.Status = IPackageWriter::ECommitStatus::Success;
	Info.PackageName = Package->GetFName();
	Info.WriteOptions = IPackageWriter::EWriteOptions::Write;
	PackageWriter->CommitPackage(MoveTemp(Info));

	TArray<FName> SavedImportedClassPathNames;
	FCustomVersionContainer SavedCustomVersions;
	if (!PackageWriter->TryGetClassesAndVersions(SavedImportedClassPathNames, SavedCustomVersions))
	{
		UE_LOGF(LogEditorDomain, Warning, "Could not save %ls to EditorDomain: Could not read the PackageFileSummary from the saved bytes.",
			*Package->GetName());
		return false;
	}
	TArray<FTopLevelAssetPath> SavedImportedClassPaths;
	SavedImportedClassPaths.Reserve(SavedImportedClassPathNames.Num());
	for (FName ClassPathName : SavedImportedClassPathNames)
	{
		FTopLevelAssetPath ClassPath(WriteToString<256>(ClassPathName).ToView());
		if (!ClassPath.IsValid())
		{
			UE_LOGF(LogEditorDomain, Warning, "Could not save %ls to EditorDomain: It uses an importclass %ls that is not a TopLevelAssetPath.",
				*Package->GetName(), *ClassPathName.ToString());
			return false;
		}
		SavedImportedClassPaths.Add(ClassPath);
	}

	// Replace any Blueprint classes in SavedImportedClasses with their native base class, since only native classes
	// have class schemas and customversions that could change the custom versions. EditorDomain packages with blueprint
	// instances do not use unversioned properties and so do not need to be keyed to changes in the BP class they depend on.
	int32 NextClass = 0;
	bool bHasTriedPrecacheClassDigests = false;
	FClassDigestMap& ClassDigests = GetClassDigests();
	while (NextClass < SavedImportedClassPaths.Num())
	{
		{
			FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
			while (NextClass < SavedImportedClassPaths.Num())
			{
				FClassDigestData* ExistingData = ClassDigests.Map.Find(SavedImportedClassPaths[NextClass]);
				if (!ExistingData)
				{
					break;
				}
				SavedImportedClassPaths[NextClass] = ExistingData->ClosestNative;
				++NextClass;
			}
		}

		if (NextClass < SavedImportedClassPaths.Num())
		{
			if (bHasTriedPrecacheClassDigests)
			{
				UE_LOGF(LogEditorDomain, Warning, "Could not save %ls to EditorDomain: It constructed an instance of class %ls which which does not exist.",
					*Package->GetName(), *SavedImportedClassPaths[NextClass].ToString());
				return false;
			}
			// EDITORDOMAIN_TODO: Remove this !IsInGameThread check once FindObject no longer asserts if UE::IsSavingPackage()
			if (!IsInGameThread())
			{
				UE_LOGF(LogEditorDomain, Warning,
					"Could not save %ls to EditorDomain: It constructed an instance of a class %ls which is not yet saved in the EditorDomain, "
					"and we are not on the gamethread so we cannot add it.",
					*Package->GetName(), *SavedImportedClassPaths[NextClass].ToString());
				return false;
			}
			TConstArrayView<FTopLevelAssetPath> RemainingClasses =
				TConstArrayView<FTopLevelAssetPath>(SavedImportedClassPaths).RightChop(NextClass);
			PrecacheClassDigests(RemainingClasses);
			bHasTriedPrecacheClassDigests = true;
		}
	}

	ESaveStorageResult StorageResult = ESaveStorageResult::Valid;

	TArray<FTopLevelAssetPath> ImportedClassPaths;
	ImportedClassPaths.Reserve(PackageData->ImportedClasses.Num());
	for (FName ClassPathName : PackageData->ImportedClasses)
	{
		FTopLevelAssetPath ClassPath(WriteToString<256>(ClassPathName).ToView());
		if (ClassPath.IsValid())
		{
			ImportedClassPaths.Add(ClassPath);
		}
	}

	TSet<FTopLevelAssetPath> KnownImportedClasses;
	TSet<FGuid> KnownGuids;
	TArray<FGuid> ClassCustomVersions;
	{
		FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		for (TArray<FTopLevelAssetPath>* ClassPathArray : { &ImportedClassPaths, &GGlobalConstructClasses })
		{
			for (const FTopLevelAssetPath ImportedClassPath : *ClassPathArray)
			{
				FClassDigestData* ExistingData = ClassDigests.Map.Find(ImportedClassPath);
				if (ExistingData)
				{
					// As above with SavedImportedClasses, we add the ClosestNative of each imported class instead of
					// the (possibly BlueprintGeneratedClass) itself
					KnownImportedClasses.Add(ExistingData->ClosestNative);
					for (const FTopLevelAssetPath& ConstructedClass : ExistingData->ConstructClasses)
					{
						KnownImportedClasses.Add(ConstructedClass);
					}
					ClassCustomVersions.Reset();
					FKnownCustomVersions::FindGuidsChecked(ClassCustomVersions, ExistingData->CustomVersionHandles);
					KnownGuids.Append(ClassCustomVersions);
				}
			}
		}
	}

	for (const FTopLevelAssetPath& ImportedClass : SavedImportedClassPaths)
	{
		if (!KnownImportedClasses.Contains(ImportedClass))
		{
			// Suggested debugging technique for this message: Add a conditional breakpoint on the packagename
			// at the start of LoadPackageInternal. After it gets hit, add a breakpoint in the constructor
			// of the ImportedClass.
			UE_LOGF(LogEditorDomain, Display, "Could not save %ls to EditorDomain: It uses an unexpected class. "
				"Optimized loading and incremental cooking will be disabled for this package.\n\t"
				"Find the class that added %ls and add it to its DeclareConstructClasses function.",
				*Package->GetName(), *ImportedClass.ToString());
			StorageResult = ESaveStorageResult::UnexpectedClass;
		}
	}

	if (StorageResult == ESaveStorageResult::Valid)
	{
		TConstArrayView<UE::AssetRegistry::FPackageCustomVersion> KnownCustomVersions = PackageDigest.CustomVersions.Get();
		for (const UE::AssetRegistry::FPackageCustomVersion& CustomVersion : KnownCustomVersions)
		{
			KnownGuids.Add(CustomVersion.Key);
		}
		TArray<FGuid> UnknownGuids;
		for (const FCustomVersion& CustomVersion : SavedCustomVersions.GetAllVersions())
		{
			if (!KnownGuids.Contains(CustomVersion.Key))
			{
				UnknownGuids.Add(CustomVersion.Key);
			}
		}
		if (!UnknownGuids.IsEmpty())
		{
			TMap<FGuid, UObject*> Culprits = FindCustomVersionCulprits(UnknownGuids, Package);
			TStringBuilder<256> VersionsText;
			for (const FGuid& CustomVersionGuid : UnknownGuids)
			{
				TOptional<FCustomVersion> CustomVersion = FCurrentCustomVersions::Get(CustomVersionGuid);
				UObject* Culprit = Culprits.FindOrAdd(CustomVersionGuid);
				VersionsText << TEXT("\n\tCustomVersion(Guid=") << CustomVersionGuid << TEXT(", Name=")
					<< (CustomVersion ? *CustomVersion->GetFriendlyName().ToString() : TEXT("<Unknown>"))
					<< TEXT("): Used by ")
					<< (Culprit ? *Culprit->GetClass()->GetPathName() : TEXT("<CulpritUnknown>"));
			}

			// Suggested debugging technique for this message: SetNextStatement back to beginning of the function,
			// add a conditional breakpoint in FArchive::UsingCustomVersion with Key.A == 0x<FirstHexWordFromGuid>
			UE_LOGF(LogEditorDomain, Display, "Could not save %ls to EditorDomain: It uses an unexpected custom version. "
				"Optimized loading and incremental cooking will be disabled for this package. "
				"Modify the classes/structs to call Ar.UsingCustomVersion(Guid) in Serialize or DeclareCustomVersions.%ls",
				*Package->GetName(), VersionsText.ToString());
			StorageResult = ESaveStorageResult::UnexpectedCustomVersion;
		}
	}

	if (StorageResult == ESaveStorageResult::Valid && 
		GMaxBulkDataSize >= 0 && PackageWriter->GetBulkDataSize() > static_cast<uint64>(GMaxBulkDataSize))
	{
		// TODO EditorDomain MeshDescription: We suppress the warning if the package includes a MeshDescription;
		// MeshDescriptions do not yet detect the need for a bulkdata resave until save serialization.
		bool bDisplayAsWarning = !Algo::AnyOf(GetPackageObjects(), [](UObject* Object)
			{
				return Object->IsA<UStaticMesh>();
			});

		FString Message = FString::Printf(TEXT("Could not save package to EditorDomain because BulkData size is too large. ")
			TEXT("Package=%s, BulkDataSize=%" UINT64_FMT ", EditorDomain.MaxBulkDataSize=%" INT64_FMT)
			TEXT("\n\tWe did not detect this until after trying to save the package, which is bad for performance. Resave the package ")
			TEXT("or debug why the size was not detected by the BulkDataRegistry."),
			*WriteToString<256>(PackageName), PackageWriter->GetBulkDataSize(), GMaxBulkDataSize);
		if (bDisplayAsWarning)
		{
			UE_LOGF(LogEditorDomain, Display, "%ls", *Message);
		}
		else
		{
			UE_LOGF(LogEditorDomain, Verbose, "%ls", *Message);
		}
		StorageResult = ESaveStorageResult::BulkDataTooLarge;
	}

	bool bStorageResultValid = StorageResult == ESaveStorageResult::Valid;
	static bool bTestEditorDomainDeterminism = FParse::Param(FCommandLine::Get(), TEXT("testeditordomaindeterminism"));;
	if (bTestEditorDomainDeterminism)
	{
		FPackagePath PackagePath;
		FRequestOwner GetOwner(EPriority::Blocking);
		TOptional<FCacheGetResponse> GetResponse;
		verify(FPackagePath::TryFromPackageName(PackageName, PackagePath));
		RequestEditorDomainPackage(PackagePath, PackageDigest.Hash, ECachePolicy::None, GetOwner,
			[&GetResponse](FCacheGetResponse&& Response) { GetResponse.Emplace(MoveTemp(Response)); });
		GetOwner.Wait();
		if (!GetResponse || GetResponse->Status != EStatus::Ok)
		{
			return true;
		}
		ValidateEditorDomainDeterminism(PackageName, bStorageResultValid, PackageWriter, MoveTemp(*GetResponse));
		return true;
	}

	TCbWriter<16> MetaData;
	MetaData.BeginObject();
	uint64 FileSize = PackageWriter->GetFileSize();
	MetaData << "Valid" << bStorageResultValid;
	if (bStorageResultValid)
	{
		MetaData << "FileSize" << FileSize;
	}
	else
	{
		MetaData << "StorageResult" << LexToUtf8(StorageResult);
	}
	MetaData.EndObject();

	FCacheRecordBuilder RecordBuilder(GetEditorDomainPackageKey(PackageDigest.Hash));
	if (bStorageResultValid)
	{
		COOK_STAT(Timer.AddMiss(FileSize + MetaData.GetSaveSize()));

		uint64 TotalAttachmentSize = 0;
		for (const FEditorDomainPackageWriter::FAttachment& Attachment : PackageWriter->GetAttachments())
		{
			RecordBuilder.AddValue(Attachment.ValueId, Attachment.Buffer);
			TotalAttachmentSize += Attachment.Buffer.GetSize();
		}
		if (TotalAttachmentSize != FileSize)
		{
			UE_LOG(LogEditorDomain, Warning,
				TEXT("Could not save %s to EditorDomain due to TrySavePackage bug: size of all segments %" UINT64_FMT " is not equal to FileSize in metadata %" UINT64_FMT "."),
				*Package->GetName(), TotalAttachmentSize, FileSize);
			return false;
		}
	}
	else
	{
		COOK_STAT(Timer.AddMiss(MetaData.GetSaveSize()));
	}
	RecordBuilder.SetMeta(MetaData.Save().AsObject());
	FRequestOwner Owner(EPriority::Normal);
	GetCache().Put({ {{Package->GetName()}, RecordBuilder.Build()} }, Owner);
	Owner.KeepAlive();

	if (bStorageResultValid)
	{
		UE::Cook::FBuildResultDependenciesMap ExistingResultDependencies;
		UE::Cook::FIncrementalCookAttachments CookAttachments =
			UE::Cook::FIncrementalCookAttachments::Collect(
				Package, nullptr /* TargetPlatform */, MoveTemp(ExistingResultDependencies),
				false /* bHasSaveResult */, TConstArrayView<FName>() /* UntrackedSoftPackageReferences */,
				nullptr /* GenerationHelper */, false /* bGenerated */, TArray<FName>(), TConstArrayView<UObject*>(),
				TConstArrayView<UObject*>(), TConstArrayView<UE::SavePackageUtilities::FPreloadDependency>(),
				TConstArrayView<UE::Cook::FReplicatedLogData>());
		UE::Cook::Artifact::FAppendPackageMetaDataContext Context;
		CookAttachments.AppendCommitAttachments(Context);
		if (Context.HasEntries())
		{
			UE::TargetDomain::CommitEditorDomainCookAttachments(Package->GetFName(), Context);
		}
	}
	return true;
}

void ValidateEditorDomainDeterminism(FName PackageName, bool bStorageResultValid,
	FEditorDomainPackageWriter* PackageWriter, UE::DerivedData::FCacheGetResponse&& GetResponse)
{
	using namespace UE::DerivedData;

	FCacheRecord& Record = GetResponse.Record;

	const FCbObject& MetaData = Record.GetMeta();
	bool bPreviousStorageResultValid = MetaData["Valid"].AsBool(false);
	auto LogHeaderWarning = [PackageName]()
	{
		UE_LOGF(LogEditorDomain, Warning, "Indeterminism detected in EditorDomain save of %ls.", *PackageName.ToString());
	};

	if (bPreviousStorageResultValid != bStorageResultValid)
	{
		LogHeaderWarning();
		UE_LOGF(LogEditorDomain, Display, "\tStorageResultValid not equal. Previous: %ls, Current: %ls",
			*LexToString(bPreviousStorageResultValid), *LexToString(bStorageResultValid));
		return;
	}
	if (!bStorageResultValid)
	{
		// Nothing further to test; we ignore the PackageWriter data and write an empty record in this case.
		return;
	}

	TConstArrayView<FValueWithId> PreviousValues = Record.GetValues();
	TConstArrayView<FEditorDomainPackageWriter::FAttachment> CurrentValues = PackageWriter->GetAttachments();
	for (int32 SegmentIndex = 0; SegmentIndex < FMath::Max(PreviousValues.Num(), CurrentValues.Num()); ++SegmentIndex)
	{
		if (PreviousValues.Num() <= SegmentIndex || CurrentValues.Num() <= SegmentIndex)
		{
			LogHeaderWarning();
			UE_LOGF(LogEditorDomain, Display, "\tNumber of segments differ. Previous: %d, Current: %d",
				PreviousValues.Num(), CurrentValues.Num());
			return;
		}
		FSharedBuffer PreviousValue = PreviousValues[SegmentIndex].GetData().Decompress();
		FSharedBuffer CurrentValue = CurrentValues[SegmentIndex].Buffer;
		const uint8* PreviousBytes = reinterpret_cast<const uint8*>(PreviousValue.GetData());
		const uint8* CurrentBytes = reinterpret_cast<const uint8*>(CurrentValue.GetData());
		uint64 MinNumBytes = FMath::Min(PreviousValue.GetSize(), CurrentValue.GetSize());
		for (uint64 ByteIndex = 0; ByteIndex < MinNumBytes; ByteIndex++)
		{
			if (PreviousBytes[ByteIndex] != CurrentBytes[ByteIndex])
			{
				LogHeaderWarning();
				UE_LOG(LogEditorDomain, Display, TEXT("\tBytes differ in Segment %d, Offset %" UINT64_FMT ". Previous: %d, Current: %d"),
					SegmentIndex, ByteIndex, PreviousBytes[ByteIndex], CurrentBytes[ByteIndex]);
				return;
			}
		}
		if (MinNumBytes < PreviousValue.GetSize() || MinNumBytes < CurrentValue.GetSize())
		{
			LogHeaderWarning();
			UE_LOG(LogEditorDomain, Display, TEXT("\tSegment sizes differ in Segment %d. Previous: %" UINT64_FMT ", Current: %" UINT64_FMT "."),
				SegmentIndex, PreviousValue.GetSize(), CurrentValue.GetSize());
			return;
		}
	}

	// All bytes identical
}

void GetBulkDataList(FName PackageName, UE::DerivedData::IRequestOwner& Owner, TUniqueFunction<void(FSharedBuffer Buffer)>&& Callback)
{
	IPackageDigestCache* DigestCache = IPackageDigestCache::Get();
	FPackageDigest PackageDigest = DigestCache
		? DigestCache->GetPackageDigest(PackageName)
		: CalculatePackageDigest(IAssetRegistry::GetChecked(), PackageName);
	if (!PackageDigest.IsSuccessful())
	{
		Callback(FSharedBuffer());
		return;
	}

	if (!EnumHasAnyFlags(PackageDigest.DomainUse, EDomainUse::LoadEnabled))
	{
		Callback(FSharedBuffer());
		return;
	}

	using namespace UE::DerivedData;
	ICache& Cache = GetCache();
	Cache.Get({{{WriteToString<128>(PackageName)}, GetBulkDataListKey(PackageDigest.Hash)}}, Owner,
		[InnerCallback = MoveTemp(Callback)](FCacheGetResponse&& Response)
		{
			bool bOk = Response.Status == EStatus::Ok;
			InnerCallback(bOk ? Response.Record.GetValue(GetBulkDataValueId()).GetData().Decompress() : FSharedBuffer());
		});
}

void PutBulkDataList(FName PackageName, FSharedBuffer Buffer)
{
	IPackageDigestCache* DigestCache = IPackageDigestCache::Get();
	FPackageDigest PackageDigest = DigestCache
		? DigestCache->GetPackageDigest(PackageName)
		: CalculatePackageDigest(IAssetRegistry::GetChecked(), PackageName);
	if (!PackageDigest.IsSuccessful())
	{
		return;
	}

	if (!EnumHasAnyFlags(PackageDigest.DomainUse, EDomainUse::SaveEnabled))
	{
		return;
	}

	using namespace UE::DerivedData;
	ICache& Cache = GetCache();
	FRequestOwner Owner(EPriority::Normal);
	FCacheRecordBuilder RecordBuilder(GetBulkDataListKey(PackageDigest.Hash));
	RecordBuilder.AddValue(GetBulkDataValueId(), Buffer);
	Cache.Put({{{WriteToString<128>(PackageName)}, RecordBuilder.Build()}}, Owner);
	Owner.KeepAlive();
}

FIoHash GetPackageAndGuidHash(const FIoHash& PackageHash, const FGuid& BulkDataId)
{
	FBlake3 Builder;
	Builder.Update(&PackageHash.GetBytes(), sizeof(PackageHash.GetBytes()));
	Builder.Update(&BulkDataId, sizeof(BulkDataId));
	return Builder.Finalize();
}

void GetBulkDataPayloadId(FName PackageName, const FGuid& BulkDataId, UE::DerivedData::IRequestOwner& Owner,
	TUniqueFunction<void(FSharedBuffer Buffer)>&& Callback)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	AssetRegistry.WaitForPackage(PackageName.ToString());
	TOptional<FAssetPackageData> PackageData = AssetRegistry.GetAssetPackageDataCopy(PackageName);
	if (!PackageData)
	{
		return;
	}
	FIoHash PackageAndGuidHash = GetPackageAndGuidHash(PackageData->GetPackageSavedHash(), BulkDataId);

	using namespace UE::DerivedData;
	ICache& Cache = GetCache();
	Cache.Get({{{WriteToString<192>(PackageName, TEXT("/"), BulkDataId)}, GetBulkDataPayloadIdKey(PackageAndGuidHash)}},
		Owner,
		[InnerCallback = MoveTemp(Callback)](FCacheGetResponse&& Response)
		{
			bool bOk = Response.Status == EStatus::Ok;
			InnerCallback(bOk ? Response.Record.GetValue(GetBulkDataValueId()).GetData().Decompress() : FSharedBuffer());
		});
}

void PutBulkDataPayloadId(FName PackageName, const FGuid& BulkDataId, FSharedBuffer Buffer)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	AssetRegistry.WaitForPackage(PackageName.ToString());
	TOptional<FAssetPackageData> PackageData = AssetRegistry.GetAssetPackageDataCopy(PackageName);
	if (!PackageData)
	{
		return;
	}
	FIoHash PackageAndGuidHash = GetPackageAndGuidHash(PackageData->GetPackageSavedHash(), BulkDataId);

	using namespace UE::DerivedData;
	ICache& Cache = GetCache();
	FRequestOwner Owner(EPriority::Normal);
	FCacheRecordBuilder RecordBuilder(GetBulkDataPayloadIdKey(PackageAndGuidHash));
	RecordBuilder.AddValue(GetBulkDataValueId(), Buffer);
	Cache.Put({{{WriteToString<128>(PackageName)}, RecordBuilder.Build()}}, Owner);
	Owner.KeepAlive();
}

FStringBuilderBase& operator<<(FStringBuilderBase& Writer, UE::EditorDomain::EDomainUse DomainUse)
{
	using namespace UE::EditorDomain;

	if (!DomainUse)
	{
		Writer << TEXT("None");
		return Writer;
	}

	int32 StartLen = Writer.Len();
	auto AddSeparator = [&Writer, StartLen]()
	{
		if (Writer.Len() > StartLen)
		{
			Writer << TEXT("|");
		}
	};
	if (EnumHasAnyFlags(DomainUse, EDomainUse::LoadEnabled))
	{
		AddSeparator();
		Writer << TEXT("LoadEnabled");
	}
	if (EnumHasAnyFlags(DomainUse, EDomainUse::SaveEnabled))
	{
		AddSeparator();
		Writer << TEXT("SaveEnabled");
	}
	if (Writer.Len() == StartLen)
	{
		Writer << TEXT("<Invalid>");
	}
	return Writer;
}

static void DumpClassDigests()
{
	using namespace UE::EditorDomain;

	TArray<FTopLevelAssetPath> ClassPaths;
	for (TObjectIterator<UClass> Iter; Iter; ++Iter)
	{
		UClass* Class = *Iter;
		UPackage* Package = Class->GetPackage();
		if (!Class->HasAnyClassFlags(CLASS_Transient) && Package && !Package->HasAnyFlags(RF_Transient))
		{
			FTopLevelAssetPath ClassPath(Class);
			if (ClassPath.IsValid())
			{
				ClassPaths.Add(ClassPath);
			}
		}
	}
	Algo::Sort(ClassPaths, [](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.Compare(B) < 0; });
	TArray<FString> ClassTexts;
	ClassTexts.Reserve(ClassPaths.Num());
	TStringBuilder<1024> ClassText;

	PrecacheClassDigests(ClassPaths);

	UE::EditorDomain::FClassDigestMap& ClassDigests = UE::EditorDomain::GetClassDigests();
	TArray<FGuid> CustomVersionGuids;
	{
		FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		for (const FTopLevelAssetPath& ClassPath : ClassPaths)
		{
			ClassText.Reset();
			UE::EditorDomain::FClassDigestData* ExistingData= ClassDigests.Map.Find(ClassPath);
			if (!ExistingData)
			{
				ClassTexts.Add(TEXT("<MissingClass>"));
				continue;
			}

			CustomVersionGuids.Reset();
			FKnownCustomVersions::FindGuidsChecked(CustomVersionGuids, ExistingData->CustomVersionHandles);
			Algo::Sort(CustomVersionGuids);

			ClassText << TEXT("(");
			{
				ClassText << TEXT("EditorDomainUse=") << ExistingData->EditorDomainUse;
				ClassText << TEXT(",") << TEXT("bNative=") << LexToString(ExistingData->bNative);
				ClassText << TEXT(",") << TEXT("bTargetIterativeEnabled=")
					<< LexToString(ExistingData->bTargetIterativeEnabled);
				ClassText << TEXT(",") << TEXT("ConstructClasses=(");
				if (!ExistingData->ConstructClasses.IsEmpty())
				{
					TArray<FTopLevelAssetPath> Sorted(ExistingData->ConstructClasses);
					Algo::Sort(Sorted, [](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.Compare(B) < 0; });
					for (const FTopLevelAssetPath& Path : Sorted)
					{
						ClassText << Path << TEXT(",");
					}
					ClassText.RemoveSuffix(1);
				}
				ClassText << TEXT(")");
				ClassText << TEXT(",") << TEXT("CustomVersions=(");
				if (CustomVersionGuids.Num() > 0)
				{
					for (const FGuid& VersionGuid : CustomVersionGuids)
					{
						ClassText << VersionGuid << TEXT(",");
					}
					ClassText.RemoveSuffix(1);
				}
				ClassText << TEXT(")");
				ClassText << TEXT(",") << TEXT("InclusiveSchemaHash=") << ExistingData->InclusiveSchemaHash;
			}
			ClassText << TEXT(")");

			ClassTexts.Add(FString(ClassText.ToView()));
		}
	}

	for (int32 Index = 0; Index < ClassPaths.Num(); ++Index)
	{
		UE_LOGF(LogEditorDomain, Display, "ClassDigest: %ls=%ls",
			*WriteToString<256>(ClassPaths[Index]), *ClassTexts[Index]);
	}
}
FAutoConsoleCommand DumpClassDigestsCommand(TEXT("EditorDomain.DumpClassDigests"),
	TEXT("Write to the log the digest information for each class."),
	FConsoleCommandDelegate::CreateStatic(&DumpClassDigests));

}

#define CONSTRUCTION_DURING_UPGRADE_TRACKER_ENABLED UE_WITH_PACKAGE_ACCESS_TRACKING && !UE_BUILD_SHIPPING

#if CONSTRUCTION_DURING_UPGRADE_TRACKER_ENABLED
/**
 * Utility that validates whether objects constructed during PostLoad have been declared as constructible by the
 * PostLoading class. This utility is not in the EditorDomain namespace because the warnings are given even when not
 * using EditorDomain and we want to avoid confusion when looking at the callstack.
 */
class FConstructionDuringUpgradeTracker : public FDelayedAutoRegisterHelper
{
public:
	FConstructionDuringUpgradeTracker();

private:
	void OnObjectSystemReady();
	void OnObjectConstructed(UObject* Constructed);
	void OnEnginePreExit();
};
FConstructionDuringUpgradeTracker ConstructionDuringUpgradeTracker;

FConstructionDuringUpgradeTracker::FConstructionDuringUpgradeTracker()
	:FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady, [this] { OnObjectSystemReady(); })
{
}

void FConstructionDuringUpgradeTracker::OnObjectSystemReady()
{
	// UtilsInitialize is called during OnPreEngineInit by PackageResourceManager, even when not running EditorDomain,
	// so we don't need to call it here.
	FCoreUObjectDelegates::OnObjectConstructed.AddRaw(this, &FConstructionDuringUpgradeTracker::OnObjectConstructed);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FConstructionDuringUpgradeTracker::OnEnginePreExit);
}

void FConstructionDuringUpgradeTracker::OnObjectConstructed(UObject* Constructed)
{
	using namespace UE::EditorDomain;

	if (LogCore.GetVerbosity() < ELogVerbosity::Verbose)
	{
		// We only log if LogCore is set to >= Verbose, so skip everything otherwise
		return;
	}

	check(Constructed);
	const PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData =
		PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (!AccumulatedScopeData)
	{
		return;
	}
	if (AccumulatedScopeData->OpName != PackageAccessTrackingOps::NAME_PostLoad)
	{
		return;
	}
	const UObject* Referencer = AccumulatedScopeData->Object;
	if (!Referencer)
	{
		return;
	}
	const UClass* ReferencerClass = AccumulatedScopeData->Object->GetClass();
	const UClass* ConstructedClass = Constructed->GetClass();
	if (!ReferencerClass || !ConstructedClass || ReferencerClass == ConstructedClass)
	{
		return;
	}
	UPackage* Package = Constructed->GetPackage();
	if (Referencer->GetPackage() != Package)
	{
		return;
	}

	FTopLevelAssetPath ReferencerClassPath(ReferencerClass);
	FTopLevelAssetPath ConstructedClassPath(ConstructedClass);
	if (!ReferencerClassPath.IsValid() || !ConstructedClassPath.IsValid())
	{
		return;
	}

	const FClassDigestData* ReferencerDigest = nullptr;
	const FClassDigestData* ConstructedDigest = nullptr;
	FClassDigestMap& ClassDigests = UE::EditorDomain::GetClassDigests();
	{
		FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		ReferencerDigest = ClassDigests.Map.Find(ReferencerClassPath);
		ConstructedDigest = ClassDigests.Map.Find(ConstructedClassPath);
	}
	if (!ReferencerDigest || !ConstructedDigest)
	{
		PrecacheClassDigests({ ReferencerClassPath, ConstructedClassPath });
		FReadScopeLock ClassDigestsScopeLock(ClassDigests.Lock);
		ReferencerDigest = ClassDigests.Map.Find(ReferencerClassPath);
		ConstructedDigest = ClassDigests.Map.Find(ConstructedClassPath);
	}
	if (!ReferencerDigest || !ConstructedDigest)
	{
		return;
	}

	bool bDeclared = INDEX_NONE != Algo::BinarySearch(ReferencerDigest->ConstructClasses, ConstructedClassPath,
		[](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.CompareFast(B) < 0; });
	if (bDeclared)
	{
		return;
	}
	if (ConstructedDigest->ClosestNative != ConstructedClassPath)
	{
		bDeclared = INDEX_NONE != Algo::BinarySearch(ReferencerDigest->ConstructClasses, ConstructedDigest->ClosestNative,
			[](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) { return A.CompareFast(B) < 0; });
		if (bDeclared)
		{
			return;
		}
	}

	InitializeGlobalConstructClasses();
	if (GGlobalConstructClasses.Contains(ConstructedClassPath))
	{
		return;
	}

	TOptional<FAssetPackageData> PackageData = IAssetRegistry::Get()->GetAssetPackageDataCopy(Package->GetFName());
	FName ConstructedClassPathName(WriteToString<256>(ConstructedClassPath).ToView());
	if (!PackageData || PackageData->ImportedClasses.Contains(ConstructedClassPathName))
	{
		return;
	}

	UE_LOGF(LogCore, Verbose, "ConstructedDuringPostLoad: %ls constructed %ls."
		"\n\tThe ability to construct other objects during PostLoad needs to be declared to support incremental cooks."
		"\n\tCreate the static override function %ls%ls::DeclareConstructClasses and add %ls to OutConstructClasses.",
		*Referencer->GetFullName(), *Constructed->GetFullName(), ReferencerClass->GetPrefixCPP(), *ReferencerClass->GetName(),
		*ConstructedClassPath.ToString());
}

void FConstructionDuringUpgradeTracker::OnEnginePreExit()
{
	FCoreUObjectDelegates::OnObjectConstructed.RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
}
#endif // CONSTRUCTION_DURING_UPGRADE_TRACKER_ENABLED

