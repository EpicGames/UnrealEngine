// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "Containers/StripedMap.h"
#include "Containers/UnrealString.h"
#include "DerivedDataCache.h"
#include "EditorDomain/EditorDomain.h"
#include "HAL/CriticalSection.h"
#include "Memory/MemoryFwd.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/CustomVersion.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

DECLARE_LOG_CATEGORY_EXTERN(LogEditorDomainSave, Log, All);

class FAssetPackageData;
class FBlake3;
class FPackagePath;
class IAssetRegistry;
class UPackage;

namespace UE::Cook { class FCookDependency; }
namespace UE::Cook { struct FCookDependencyContext; }

namespace UE::DerivedData { struct FCacheKey; }

namespace UE::EditorDomain
{

using FCustomVersionsStripedMap = TStripedMap<8, const UClass*, TArray<FGuid, TInlineAllocator<1>>>;
	
/** A UClass's data that is used in the EditorDomain Digest, and holds other information about Classes the EditorDomain needs. */
struct FClassDigestData
{
public:
	/** Inclusive schemahash for the class and all of its transitively constructible classes */
	FBlake3Hash InclusiveSchemaHash;
	/** CustomVersions used by the class or its constructed classes*/
	TArray<int32> CustomVersionHandles;
	/** Classes that can be created by the class during PostLoad/PreSave (reported by DeclareConstructClasses) */
	TArray<FTopLevelAssetPath> ConstructClasses;
	/** The closest native parent of (the possibly CoreRedirected target of) the class */
	FTopLevelAssetPath ClosestNative;
	/** EditorDomainEnabled allows everything and uses only a blocklist, so DomainUse by default is enabled. */
	EDomainUse EditorDomainUse = EDomainUse::LoadEnabled | EDomainUse::SaveEnabled;

	bool bNative : 1 = false;
	/** bTargetIterativeEnabled uses an allowlist (with a blocklist override), so defaults to false. */
	bool bTargetIterativeEnabled : 1 = false;

	bool bConstructed : 1 = false;
	bool bConstructionComplete : 1 = false;

	/** Whether or not this class is the source of a redirection.*/
	bool bIsRedirected : 1 = false;
};

/** Threadsafe cache of ClassName -> Digest data for calculating EditorDomain Digests */
struct FClassDigestMap
{
	TMap<FTopLevelAssetPath, FClassDigestData> Map;
	FRWLock Lock;
};

/**
 * A result code indicating whether a EditorDomain package could be saved into the editor domain, and if not why not
 * EditorDomain records in DDC have a Valid flag in their metadata. If the Valid flag is invalid, the record does not
 * store a EditorDomain package and it instead has a StorageResult in the metadata explaining why it failed.
 */
enum class ESaveStorageResult
{
	Valid,
	UnexpectedClass,
	UnexpectedCustomVersion,
	BulkDataTooLarge,
	InvalidCode,
	// When adding values, add them to SaveStoreResultToText as well
};
FUtf8StringView LexToUtf8(UE::EditorDomain::ESaveStorageResult Result);
ESaveStorageResult SaveStorageResultFromString(FUtf8StringView Text);

/**
 * Calculate the PackageDigest for the given packagePath.
 * Reads information from the AssetRegistry to compute the digest.
 */
FPackageDigest CalculatePackageDigest(IAssetRegistry& AssetRegistry, FName PackageName);
FPackageDigest CalculatePackageDigest(const FAssetPackageData& PackageData, FName PackageName);

/** FindOrAdd class digests for the given class dependencies, and append hashes onto the writer. Returns false if not found. */
bool TryAppendClassDigests(const TArrayView<UE::Cook::FCookDependency>& Dependencies, FBlake3& Writer,
	UE::Cook::FCookDependencyContext& Context, bool bUpdateValues);

/** FindOrAdd class digests for the given class dependencies, and return it. Returns empty optional if not found. */
TOptional<FBlake3Hash> TryGetClassDigest(FTopLevelAssetPath ClassPath);

/** For any ClassNames not already in ClassDigests, look up their UStruct and add them. An optional cache for precalculated custom versions can be used. */
void PrecacheClassDigests(TConstArrayView<FTopLevelAssetPath> ClassNames, FCustomVersionsStripedMap* CustomVersionsCache = nullptr);

/** Get the CacheRequest for the given package from the EditorDomain cache bucket. */
void RequestEditorDomainPackage(const FPackagePath& PackagePath,
	const FIoHash& EditorDomainHash, UE::DerivedData::ECachePolicy SkipFlags,
	UE::DerivedData::IRequestOwner& Owner, UE::DerivedData::FOnCacheGetComplete&& Callback);
UE::DerivedData::FCacheKey GetEditorDomainPackageKey(const FIoHash& EditorDomainHash);

/** Save the given package into the EditorDomain. */
bool TrySavePackage(UPackage* Package);

/** Get the CacheRequest for the BulkDataList of the given package. */
void GetBulkDataList(FName PackageName, UE::DerivedData::IRequestOwner& Owner,
	TUniqueFunction<void(FSharedBuffer Buffer)>&& Callback);

/** Write the data for the BulkDataList of the given package to the cache. */
void PutBulkDataList(FName PackageName, FSharedBuffer Buffer);

void GetBulkDataPayloadId(FName PackageName, const FGuid& BulkDataId, UE::DerivedData::IRequestOwner& Owner,
	TUniqueFunction<void(FSharedBuffer Buffer)> && Callback);

void PutBulkDataPayloadId(FName PackageName, const FGuid& BulkDataId, FSharedBuffer Buffer);

/** Accessor for the global ClassDigest map shared by systems needing to calculate PackageDigests. */
FClassDigestMap& GetClassDigests();

/** Initializes some global config-driven values used by the EditorDomain and TargetDomain. */
void UtilsInitialize();

/** Initializes global config-driven values that are only needed by the cooker. */
void UtilsCookInitialize();

/**
 * Utility function for stripping comment and white space from config line: "Key=Value    ; comment"
 * This is not supported in general because general key/value might have a ; in them.
 * Only usable for key/value known to disallow ; characters
 */
FStringView RemoveConfigComment(FStringView Line);

/** Parse Editor.ini:[CookSettings]:IncrementalClassDenyList */
TSet<FTopLevelAssetPath> ConstructTargetIncrementalClassBlockList();

#if ENABLE_COOK_STATS
namespace CookStats
{
	extern FCookStats::FDDCResourceUsageStats Usage;
}
#endif

} // namespace UE::EditorDomain