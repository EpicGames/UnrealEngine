// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDragTools.h"
#include "InputCoreTypes.h"
#include "EditorViewportClient.h"
#include "Editor.h"
#include "SnappingUtils.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// FDragTool
//
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDragTool::FDragTool(FEditorModeTools* InModeTools)
	: bConvertDelta( true )
	, ModeTools( InModeTools )
	, Start( FVector::ZeroVector )
	, End( FVector::ZeroVector )
	, EndWk( FVector::ZeroVector )
	, bUseSnapping( false )
	, bIsDragging( false )
{
}

void FDragTool::AddDelta( const FVector& InDelta )
{
	EndWk += InDelta;

	End = EndWk;

	// Snap to constraints.
	if( bUseSnapping )
	{
		const float GridSize = GEditor->GetGridSize();
		const FVector GridBase( GridSize, GridSize, GridSize );
		FSnappingUtils::SnapPointToGrid( End, GridBase );
	}
}

void FDragTool::StartDrag(FEditorViewportClient* InViewportClient, const FVector& InStart, const FVector2D& StartScreen)
{
	Start = InStart;
	bIsDragging = true;

	// Snap to constraints.
	if( bUseSnapping )
	{
		const float GridSize = GEditor->GetGridSize();
		const FVector GridBase( GridSize, GridSize, GridSize );
		FSnappingUtils::SnapPointToGrid( Start, GridBase );
	}
	End = EndWk = Start;

	// Store button state when the drag began.
	bAltDown = InViewportClient->IsAltPressed();
	bShiftDown = InViewportClient->IsShiftPressed();
	bControlDown = InViewportClient->IsCtrlPressed();
	bLeftMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::LeftMouseButton);
	bRightMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::RightMouseButton);
	bMiddleMouseButtonDown = InViewportClient->Viewport->KeyState(EKeys::MiddleMouseButton);
}

void FDragTool::EndDrag()
{
	Start = End = EndWk = FVector::ZeroVector;
	bIsDragging = false;
}
