// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/EditorEngine.h"

#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Modules/ModuleManager.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/SavePackage.h"
#include "Application/ThrottleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Settings/EditorStyleSettings.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "AI/NavigationSystemBase.h"
#include "Components/LightComponent.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "ActorFactories/ActorFactoryCylinderVolume.h"
#include "ActorFactories/ActorFactorySphereVolume.h"
#include "Engine/Font.h"
#include "Engine/GameViewportClient.h"
#include "Engine/BrushBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Animation/AnimBlueprint.h"
#include "Factories/LevelFactory.h"
#include "Factories/TextureRenderTargetFactoryNew.h"
#include "Editor/GroupActor.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/Texture2D.h"
#include "Animation/SkeletalMeshActor.h"
#include "Engine/NavigationObjectBase.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "GameFramework/PlayerStart.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundBase.h"
#include "GameFramework/Volume.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Components/SkyLightComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Engine/Polys.h"
#include "Engine/Selection.h"
#include "Sound/SoundCue.h"
#include "Engine/TextureRenderTarget2D.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdMisc.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "Dialogs/Dialogs.h"
#include "Dialogs/DialogsPrivate.h"
#include "UnrealEdGlobals.h"
#include "InteractiveFoliageActor.h"
#include "Engine/WorldComposition.h"
#include "EditorSupportDelegates.h"
#include "BSPOps.h"
#include "EditorCommandLineUtils.h"
#include "Engine/NetDriver.h"
#include "Engine/NetConnection.h"
#include "Net/NetworkProfiler.h"
#include "Interfaces/IPluginManager.h"
#include "UObject/PackageReload.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/ArchiveCookContext.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Templates/GuardValueAccessors.h"
#include "IMediaModule.h"
#include "Scalability.h"
#include "PlatformInfo.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/AutomationTest.h"
#include "ActorFolder.h"
#include "Materials/MaterialInterface.h"
#include "UncontrolledChangelistsModule.h"
#include "SceneView.h"
#include "StaticBoundShaderState.h"
#include "PropertyColorSettings.h"
#include "WorldPartition/ContentBundle/ContentBundleActivationScope.h"

// needed for the RemotePropagator
#include "AudioDevice.h"
#include "SurfaceIterators.h"
#include "ScopedTransaction.h"

#include "ILocalizationServiceModule.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "Layers/LayersSubsystem.h"
#include "EditorLevelUtils.h"


#include "PropertyEditorModule.h"
#include "AssetSelection.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceControlWindowsModule.h"

#include "Settings/EditorSettings.h"
#include "ProjectEditorRecords.h"

#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "SCreateAssetFromObject.h"

#include "Editor/ActorPositioning.h"

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementList.h"
//#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h" // FTypedElementWorldColumn
#include "Elements/Columns/TypedElementMiscColumns.h" // FTypedElementSyncBackToWorldTag
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementTransformColumns.h" // FTypedElementLockTransformTag
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "Elements/Component/ComponentElementEditorViewportInteractionCustomization.h"

#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"

#include "Slate/SceneViewport.h"
#include "IAssetViewport.h"

#include "ContentStreaming.h"
#include "Components/SkeletalMeshComponent.h"
#include "EngineModule.h"

#include "EditorWorldExtension.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Framework/TypedElementRegistry.h"

#if PLATFORM_WINDOWS
	#include "Windows/WindowsHWrapper.h"
// For WAVEFORMATEXTENSIBLE
	#include "Windows/AllowWindowsPlatformTypes.h"
#include <mmreg.h>
	#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/RemoteConfigIni.h"

#include "AssetToolsModule.h"
#include "ObjectTools.h"
#include "MessageLogModule.h"

#include "ActorEditorUtils.h"
#include "SnappingUtils.h"
#include "Logging/MessageLog.h"

#include "MRUFavoritesList.h"
#include "Misc/EngineBuildSettings.h"

#include "EngineAnalytics.h"

// AIMdule

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "GameFramework/GameUserSettings.h"
#include "Engine/LevelStreamingVolume.h"
#include "Engine/LocalPlayer.h"
#include "EngineStats.h"
#include "Rendering/ColorVertexBuffer.h"

#if !UE_BUILD_SHIPPING
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"
#endif

#include "PhysicsPublic.h"
#include "Engine/CoreSettings.h"
#include "ShaderCompiler.h"
#include "DistanceFieldAtlas.h"
#include "MeshCardRepresentation.h"

#include "PixelInspectorModule.h"

#include "SourceCodeNavigation.h"
#include "GameProjectUtils.h"
#include "ActorGroupingUtils.h"

#include "DesktopPlatformModule.h"

#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "Editor/EditorPerformanceSettings.h"

#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "Engine/MapBuildDataRegistry.h"

#include "DynamicResolutionState.h"

#include "IHotReload.h"
#include "EditorBuildUtils.h"
#include "MaterialStatsCommon.h"
#include "MaterialShaderQualitySettings.h"

#include "Bookmarks/IBookmarkTypeTools.h"
#include "Bookmarks/BookMarkTypeActions.h"
#include "Bookmarks/BookMark2DTypeActions.h"
#include "ComponentReregisterContext.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderTargetPool.h"
#include "RenderGraphBuilder.h"
#include "CustomResourcePool.h"
#include "ToolMenus.h"
#include "IToolMenusEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/BrowseToAssetOverrideSubsystem.h"
#include "LevelEditorSubsystem.h"
#include "Engine/LevelScriptActor.h"
#include "UObject/UnrealType.h"
#include "Factories/TextureFactory.h"
#include "Engine/TextureCube.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "DerivedDataBuildLocalExecutor.h"
#include "DerivedDataBuildRemoteExecutor.h"
#include "DerivedDataBuildWorkers.h"
#include "AssetCompilingManager.h"
#include "ChaosSolversModule.h"

#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "LevelEditorDragDropHandler.h"
#include "IProjectExternalContentInterface.h"
#include "IDocumentation.h"
#include "StereoRenderTargetManager.h"
#include "Subsystems/EditorActorSubsystem.h"

#include "Engine/EngineCustomTimeStep.h"
#include "Engine/TimecodeProvider.h"
#include "TargetReceipt.h"

#include "AudioEditorSettings.h"
#include "Settings/HomeScreenCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorEngine)

DEFINE_LOG_CATEGORY_STATIC(LogEditor, Log, All);

#define LOCTEXT_NAMESPACE "UnrealEd.Editor"

//////////////////////////////////////////////////////////////////////////
// Globals

namespace PrivateEditorSelection
{

static USelection* GActorSelection = nullptr;
static USelection* GComponentSelection = nullptr;
static USelection* GObjectSelection = nullptr;

void InitSelectionSets()
{
	// Note: The actor and component typed element selection set is set and owned by the level editor, so it is deliberately left null here
	GActorSelection = USelection::CreateActorSelection(GetTransientPackage(), TEXT("SelectedActors"), RF_Transactional);
	GActorSelection->AddToRoot();

	GComponentSelection = USelection::CreateComponentSelection(GetTransientPackage(), TEXT("SelectedComponents"), RF_Transactional);
	GComponentSelection->AddToRoot();

	GObjectSelection = USelection::CreateObjectSelection(GetTransientPackage(), TEXT("SelectedObjects"), RF_Transactional);
	GObjectSelection->AddToRoot();
	GObjectSelection->SetElementSelectionSet(NewObject<UTypedElementSelectionSet>(GObjectSelection, NAME_None, RF_Transactional));

	GIsActorSelectedInEditor = [](const AActor* InActor)
	{
		return GActorSelection->IsSelected(InActor);
	};

	GIsComponentSelectedInEditor = [](const UActorComponent* InComponent)
	{
		return GComponentSelection->IsSelected(InComponent);
	};

	GIsObjectSelectedInEditor = [](const UObject* InObject)
	{
		return GObjectSelection->IsSelected(InObject);
	};
}

void DestroySelectionSets()
{
	// We may be destroyed after the UObject system has already shutdown, 
	// which would mean that these instances will be garbage
	if (UObjectInitialized())
	{
		GActorSelection->RemoveFromRoot();
		GComponentSelection->RemoveFromRoot();
		GObjectSelection->RemoveFromRoot();

		if (!GObjectSelection->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
		{
			if (UTypedElementSelectionSet* ObjectSelectionSet = GObjectSelection->GetElementSelectionSet())
			{
				ObjectSelectionSet->ClearSelection(FTypedElementSelectionOptions());
			}
		}
	}

	GIsActorSelectedInEditor = nullptr;
	GIsComponentSelectedInEditor = nullptr;
	GIsObjectSelectedInEditor = nullptr;

	GActorSelection = nullptr;
	GComponentSelection = nullptr;
	GObjectSelection = nullptr;
}

} // namespace PrivateEditorSelection

namespace ActorAttachmentHelper
{
	static const AActor* GetTopMostAttachParentActor(AActor* InActor)
	{
		check(InActor);
		AActor* TopAttachParentActor = InActor;
		AActor* NextAttachParentActor = TopAttachParentActor->GetAttachParentActor();
		while (NextAttachParentActor)
		{
			TopAttachParentActor = NextAttachParentActor;
			NextAttachParentActor = TopAttachParentActor->GetAttachParentActor();
		}
		return TopAttachParentActor;
	}
} // namespace ActorAttachmentHelper

static FAutoConsoleVariable GInvalidateHitProxiesEachSIEFrameCVar(
	TEXT("r.Editor.Viewport.InvalidateEachSIEFrame"),
	1,
	TEXT("Invalidate the viewport on each frame when SIE is running. Disabling this cvar (setting to 0) may improve performance, but impact the ability to click on objects that are moving in the viewport."));

/**
* A mapping of all startup packages to whether or not we have warned the user about editing them
*/
static TMap<UPackage*, bool> StartupPackageToWarnState;

#if PLATFORM_WINDOWS
static TWeakPtr<SNotificationItem> MissingAdvancedRenderingRequirementsNotificationPtr;
#endif

static void CheckForMissingAdvancedRenderingRequirements()
{
#if PLATFORM_WINDOWS
	if (FSlateApplication::IsInitialized() && GDynamicRHIFailedToInitializeAdvancedPlatform)
	{
		/** Utility functions for the notification */
		struct Local
		{
			static ECheckBoxState GetDontAskAgainCheckBoxState()
			{
				bool bSuppressNotification = false;
				GConfig->GetBool(TEXT("WindowsEditor"), TEXT("SuppressMissingAdvancedRenderingRequirementsNotification"), bSuppressNotification, GEditorPerProjectIni);
				return bSuppressNotification ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
			{
				const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
				GConfig->SetBool(TEXT("WindowsEditor"), TEXT("SuppressMissingAdvancedRenderingRequirementsNotification"), bSuppressNotification, GEditorPerProjectIni);
			}

			static void OnMissingAdvancedRenderingRequirementsNotificationDismissed()
			{
				TSharedPtr<SNotificationItem> NotificationItem = MissingAdvancedRenderingRequirementsNotificationPtr.Pin();

				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
					NotificationItem->Fadeout();

					MissingAdvancedRenderingRequirementsNotificationPtr.Reset();
				}
			}
		};

		const ECheckBoxState DontAskAgainCheckBoxState = Local::GetDontAskAgainCheckBoxState();
		if (DontAskAgainCheckBoxState == ECheckBoxState::Unchecked)
		{
			const FText TitleText = LOCTEXT("MissingAdvancedRenderingRequirementsNotificationTitle", "Missing support for advanced rendering features");
			const FText MessageText = LOCTEXT("MissingAdvancedRenderingRequirementsNotificationText",
				"This project attempted to launch DirectX 12 with the SM6 shader format but it is not supported by your system. This will prevent advanced rendering features like Nanite and Virtual Shadow Maps from working.\n\nMake sure your system meets the requirements for these UE5 rendering features."
			);

			FNotificationInfo Info(TitleText);
			Info.SubText = MessageText;

			Info.HyperlinkText = LOCTEXT("UnrealSoftwareRequirements", "Unreal Software Requirements");
			Info.Hyperlink = FSimpleDelegate::CreateStatic([]() { IDocumentation::Get()->Open(TEXT("hardware-and-software-specifications-for-unreal-engine")); });

			Info.bFireAndForget = false;
			Info.FadeOutDuration = 3.0f;
			Info.ExpireDuration = 0.0f;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("OK", "OK"), FText::GetEmpty(), FSimpleDelegate::CreateStatic(&Local::OnMissingAdvancedRenderingRequirementsNotificationDismissed)));

			Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&Local::GetDontAskAgainCheckBoxState);
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&Local::OnDontAskAgainCheckBoxStateChanged);
			Info.CheckBoxText = NSLOCTEXT("ModalDialogs", "DefaultCheckBoxMessage", "Don't show this again");

			MissingAdvancedRenderingRequirementsNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
			MissingAdvancedRenderingRequirementsNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
#endif // PLATFORM_WINDOWS
}


ERHIFeatureLevel::Type FPreviewPlatformInfo::GetEffectivePreviewFeatureLevel() const
{
	return bPreviewShaderPlatformActive ? PreviewFeatureLevel : GMaxRHIFeatureLevel;
}

EShaderPlatform FPreviewPlatformInfo::GetShaderPlatform() const
{
	return PreviewShaderPlatformName != NAME_None ?
		FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(PreviewShaderPlatformName) :
		GetFeatureLevelShaderPlatform(PreviewFeatureLevel);
}

void FPreviewPlatformInfo::InternalSetFriendlyName()
{
	if (PreviewShaderPlatformFriendlyName.IsEmpty())
	{
		PreviewShaderPlatformFriendlyName = FDataDrivenShaderPlatformInfo::GetFriendlyName(GetShaderPlatform(), DeviceProfileName);
	}
}

void FAssetReferenceFilterContext::AddReferencingAsset(const UObject* InReferencingAsset, FAssetData::ECreationFlags InCreationFlags, EAssetReferenceFilterProperties InProperties)
{
	if (InReferencingAsset != nullptr)
	{
		FAssetData ReferencingObjectData(InReferencingAsset, InCreationFlags);

		ReferencingAssetInfo.Emplace(ReferencingObjectData, InProperties);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReferencingAssets.Add(InReferencingAsset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAssetReferenceFilterContext::AddReferencingAsset(const FAssetData& InReferencingAsset, EAssetReferenceFilterProperties InProperties)
{
	if (InReferencingAsset.IsValid())
	{
		ReferencingAssetInfo.Emplace(InReferencingAsset, InProperties);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReferencingAssets.Add(InReferencingAsset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FAssetReferenceFilterContext::AddReferencingAssets(TConstArrayView<UObject*> InReferencingAssets, FAssetData::ECreationFlags InCreationFlags, EAssetReferenceFilterProperties InProperties)
{
	for (const UObject* ReferencingAsset : InReferencingAssets)
	{
		AddReferencingAsset(ReferencingAsset, InCreationFlags, InProperties);
	}
}

void FAssetReferenceFilterContext::AddReferencingAssets(const TArray<FAssetData>& InReferencingAssets, EAssetReferenceFilterProperties InProperties)
{
	for (const FAssetData& ReferencingAsset : InReferencingAssets)
	{
		AddReferencingAsset(ReferencingAsset, InProperties);
	}
}

void FAssetReferenceFilterContext::AddReferencingAssetsFromPropertyHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle, FAssetData::ECreationFlags InCreationFlags)
{
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	EAssetReferenceFilterProperties ReferencerProperties = EAssetReferenceFilterProperties::None;

	// Determine if this is a soft object reference property
	if (const FSoftObjectProperty* SoftProperty = CastField<FSoftObjectProperty>(PropertyHandle->GetProperty()))
	{
		ReferencerProperties |= EAssetReferenceFilterProperties::Soft;
	}

	// Determine if this is an editor only property by following the parent property chain and checking if any are editor only
	{
		TSharedPtr<IPropertyHandle> CurrentPropertyHandle = PropertyHandle;
		while (CurrentPropertyHandle.IsValid())
		{
			const FProperty* Property = CurrentPropertyHandle->GetProperty();
			if (Property != nullptr && Property->IsEditorOnlyProperty())
			{
				ReferencerProperties |= EAssetReferenceFilterProperties::EditorOnly;
				break;
			}

			CurrentPropertyHandle = CurrentPropertyHandle->GetParentHandle();
		}
	}

	TArray<UObject*> ReferencingObjects;
	PropertyHandle->GetOuterObjects(ReferencingObjects);

	for (UObject* ReferencingObject : ReferencingObjects)
	{
		AddReferencingAsset(ReferencingObject, InCreationFlags, ReferencerProperties);
	}
}

//////////////////////////////////////////////////////////////////////////
// UEditorEngine

UEditorEngine* GEditor = nullptr;

UEditorEngine::UEditorEngine(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsRunningCommandlet() && !IsRunningDedicatedServer())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UTexture2D> BadTexture;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCubeMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorSphereMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorPlaneMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> EditorCylinderMesh;
			ConstructorHelpers::FObjectFinder<UFont> SmallFont;
			FConstructorStatics()
				: BadTexture(TEXT("/Engine/EditorResources/Bad"))
				, EditorCubeMesh(TEXT("/Engine/EditorMeshes/EditorCube"))
				, EditorSphereMesh(TEXT("/Engine/EditorMeshes/EditorSphere"))
				, EditorPlaneMesh(TEXT("/Engine/EditorMeshes/EditorPlane"))
				, EditorCylinderMesh(TEXT("/Engine/EditorMeshes/EditorCylinder"))
				, SmallFont(TEXT("/Engine/EngineFonts/Roboto"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		Bad = ConstructorStatics.BadTexture.Object;
		EditorCube = ConstructorStatics.EditorCubeMesh.Object;
		EditorSphere = ConstructorStatics.EditorSphereMesh.Object;
		EditorPlane = ConstructorStatics.EditorPlaneMesh.Object;
		EditorCylinder = ConstructorStatics.EditorCylinderMesh.Object;
		EditorFont = ConstructorStatics.SmallFont.Object;
	}

	DetailMode = DM_MAX;
	CurrentPlayWorldDestination = -1;
	bIsRebuildingAlteredBSP = false;
	bDisableDeltaModification = false;
	bAllowConstrainedAspectRatioInPreview = true;
	bAllowMultiplePIEWorlds = true;
	bIsEndingPlay = false;
	DefaultWorldFeatureLevel = GMaxRHIFeatureLevel;
	PreviewPlatform = FPreviewPlatformInfo(DefaultWorldFeatureLevel, GMaxRHIShaderPlatform);

	FCoreDelegates::OnFeatureLevelDisabled.AddLambda([this](int RHIType, const FName& PreviewPlatformName)
		{
			ERHIFeatureLevel::Type FeatureLevelTypeToDisable = (ERHIFeatureLevel::Type)RHIType;
			if (PreviewPlatform.PreviewFeatureLevel == FeatureLevelTypeToDisable)
			{
				UMaterialShaderQualitySettings* MaterialShaderQualitySettings = UMaterialShaderQualitySettings::Get();
				if (MaterialShaderQualitySettings->GetPreviewPlatform() != PreviewPlatformName)
				{
					return;
				}
				
				SetPreviewPlatform(FPreviewPlatformInfo(GMaxRHIFeatureLevel, GMaxRHIShaderPlatform), false);
			}
		});

	bNotifyUndoRedoSelectionChange = true;
	bIgnoreSelectionChange = false;
	bSuspendBroadcastPostUndoRedo = false;

	EditorWorldExtensionsManager = nullptr;

	ActorGroupingUtilsClassName = UActorGroupingUtils::StaticClass();

	bUATSuccessfullyCompiledOnce = FApp::IsEngineInstalled() || FApp::GetEngineIsPromotedBuild();

	// The AssetRegistry module is needed early in initialization functions so load it here rather than in Init
	FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// Callback to get the preview platform is used for PerPlatformConfig classes
	UObject::OnGetPreviewPlatform.BindUObject(this, &UEditorEngine::GetPreviewPlatformName);

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<UEditorEngine>(this))
	{
		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(TEXT("PropertyColor"), LOCTEXT("PropertyColor", "Property Color"), 
			[this](const UPrimitiveComponent* InPrimitiveComponent)
			{
				FColor PropertyColor(FColor::White);
				if (AActor* Actor = InPrimitiveComponent->GetOwner())
				{
					if (GetPropertyColorationMatch(Actor))
					{
						PropertyColor = GetDefault<ULevelEditorViewportSettings>()->PropertyColorationColorForMatchingObjects;
					}
				}
				return PropertyColor;
			},
			[this]()
			{
				const FString EmptyString;
				SetPropertyColorationTarget(GWorld, EmptyString, nullptr, nullptr, nullptr);
			},
			LOCTEXT("PropertyColor_ToopTip", "Colorize actor if property matches. Red means a property match, otherwise the color is White."));

		for (const FPropertyColorCustomProperty& PropertyColorCustomProperty : GetDefault<UPropertyColorSettings>()->CustomProperties)
		{
			FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(PropertyColorCustomProperty.Name, PropertyColorCustomProperty.Text,
				[this, PropertyColorCustomProperty](const UPrimitiveComponent* InPrimitiveComponent)
				{
					if (AActor* Actor = InPrimitiveComponent->GetOwner())
					{
						if (GetPropertyColorationMatch(Actor))
						{
							return PropertyColorCustomProperty.PropertyColor;
						}
					}
					return PropertyColorCustomProperty.DefaultColor;
				},
				[this, PropertyColorCustomProperty]()
				{
					TArray<FString> PropertyChainNames;
					UStruct* PropertyContainer = AActor::StaticClass();
					if (PropertyColorCustomProperty.PropertyChain.ParseIntoArray(PropertyChainNames, TEXT(".")))
					{
						TSharedRef<FEditPropertyChain> PropertyChain = MakeShared<FEditPropertyChain>();

						for (const FString& PropertyName : PropertyChainNames)
						{
							if (FProperty* Property = PropertyContainer->FindPropertyByName(*PropertyName))
							{
								PropertyChain->AddTail(Property);

								if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
								{
									PropertyContainer = ObjectProperty->PropertyClass;
								}
							}
							else
							{
								UE_LOGF(LogEditor, Warning, "Invalid custom property color %ls (%ls)", *PropertyColorCustomProperty.Name.ToString(), *PropertyColorCustomProperty.PropertyChain);
								break;
							}
						}

						if (PropertyChain->Num() == PropertyChainNames.Num())
						{
							SetPropertyColorationTarget(GWorld, PropertyColorCustomProperty.PropertyValue, PropertyChain->GetTail()->GetValue(), AActor::StaticClass(), &PropertyChain);
						}
					}
				},
				PropertyColorCustomProperty.TextToolTip);
		}
	}
#endif
}


int32 UEditorEngine::GetSelectedActorCount() const
{
	int32 NumSelectedActors = 0;
	for(FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
	{
		++NumSelectedActors;
	}

	return NumSelectedActors;
}


USelection* UEditorEngine::GetSelectedActors() const
{
	return PrivateEditorSelection::GActorSelection;
}

bool UEditorEngine::IsWorldSettingsSelected() 
{
	if( bCheckForWorldSettingsActors )
	{
		bIsWorldSettingsSelected = false;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			if ( Cast<AWorldSettings>( *It ) )
			{
				bIsWorldSettingsSelected = true;
				break;
			}
		}
		bCheckForWorldSettingsActors = false;
	}

	return bIsWorldSettingsSelected;
}

FSelectionIterator UEditorEngine::GetSelectedActorIterator() const
{
	return FSelectionIterator( *GetSelectedActors() );
};

int32 UEditorEngine::GetSelectedComponentCount() const
{
	int32 NumSelectedComponents = 0;
	for (FSelectionIterator It(GetSelectedComponentIterator()); It; ++It)
	{
		++NumSelectedComponents;
	}

	return NumSelectedComponents;
}

FSelectionIterator UEditorEngine::GetSelectedComponentIterator() const
{
	return FSelectionIterator(*GetSelectedComponents());
};

FSelectedEditableComponentIterator UEditorEngine::GetSelectedEditableComponentIterator() const
{
	return FSelectedEditableComponentIterator(*GetSelectedComponents());
}

USelection* UEditorEngine::GetSelectedComponents() const
{
	return PrivateEditorSelection::GComponentSelection;
}

USelection* UEditorEngine::GetSelectedObjects() const
{
	return PrivateEditorSelection::GObjectSelection;
}

void UEditorEngine::GetContentBrowserSelectionClasses(TArray<UClass*>& Selection) const
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssets;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);

	for ( auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		UClass* AssetClass = FindObject<UClass>((*AssetIt).AssetClassPath);

		if ( AssetClass != nullptr )
		{
			Selection.AddUnique(AssetClass);
		}
	}
}

void UEditorEngine::GetContentBrowserSelections(TArray<FAssetData>& Selection) const
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().GetSelectedAssets(Selection);
}

USelection* UEditorEngine::GetSelectedSet( const UClass* Class ) const
{
	if (Class != nullptr)
	{
		USelection* SelectedSet = GetSelectedActors();
		if (Class->IsChildOf(AActor::StaticClass()))
		{
			return SelectedSet;
		}
		else
		{
			//make sure this actor isn't derived off of an interface class
			for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
			{
				AActor* TestActor = static_cast<AActor*>(*It);
				if (TestActor->GetClass()->ImplementsInterface(Class))
				{
					return SelectedSet;
				}
			}
		}
	}

	//no actor matched the interface class
	return GetSelectedObjects();
}

const UClass* UEditorEngine::GetFirstSelectedClass( const UClass* const RequiredParentClass ) const
{
	const USelection* const SelectedObjects = GetSelectedObjects();

	for(int32 i = 0; i < SelectedObjects->Num(); ++i)
	{
		const UObject* const SelectedObject = SelectedObjects->GetSelectedObject(i);

		if(SelectedObject)
		{
			const UClass* SelectedClass = nullptr;

			if(SelectedObject->IsA(UBlueprint::StaticClass()))
			{
				// Handle selecting a blueprint
				const UBlueprint* const SelectedBlueprint = StaticCast<const UBlueprint*>(SelectedObject);
				if(SelectedBlueprint->GeneratedClass)
				{
					SelectedClass = SelectedBlueprint->GeneratedClass;
				}
			}
			else if(SelectedObject->IsA(UClass::StaticClass()))
			{
				// Handle selecting a class
				SelectedClass = StaticCast<const UClass*>(SelectedObject);
			}

			if(SelectedClass && (!RequiredParentClass || SelectedClass->IsChildOf(RequiredParentClass)))
			{
				return SelectedClass;
			}
		}
	}

	return nullptr;
}

void UEditorEngine::GetSelectionStateOfLevel(FSelectionStateOfLevel& OutSelectionStateOfLevel) const
{
	OutSelectionStateOfLevel.SelectedActors.Reset();
	for (FSelectionIterator ActorIt(GetSelectedActorIterator()); ActorIt; ++ActorIt)
	{
		OutSelectionStateOfLevel.SelectedActors.Add(ActorIt->GetPathName());
	}

	OutSelectionStateOfLevel.SelectedComponents.Reset();
	for (FSelectionIterator CompIt(GetSelectedComponentIterator()); CompIt; ++CompIt)
	{
		OutSelectionStateOfLevel.SelectedComponents.Add(CompIt->GetPathName());
	}
}

void UEditorEngine::SetSelectionStateOfLevel(const FSelectionStateOfLevel& InSelectionStateOfLevel)
{
	SelectNone(/*bNotifySelectionChanged*/true, /*bDeselectBSP*/true, /*bWarnAboutTooManyActors*/false);

	if (InSelectionStateOfLevel.SelectedActors.Num() > 0)
	{
		GetSelectedActors()->Modify();
		GetSelectedActors()->BeginBatchSelectOperation();

		for (const FString& ActorName : InSelectionStateOfLevel.SelectedActors)
		{
			AActor* Actor = FindObject<AActor>(nullptr, *ActorName);
			if (Actor)
			{
				SelectActor(Actor, true, /*bNotifySelectionChanged*/true);
			}
		}

		GetSelectedActors()->EndBatchSelectOperation();
	}

	if (InSelectionStateOfLevel.SelectedComponents.Num() > 0)
	{
		GetSelectedComponents()->Modify();
		GetSelectedComponents()->BeginBatchSelectOperation();

		for (const FString& ComponentName : InSelectionStateOfLevel.SelectedComponents)
		{
			UActorComponent* ActorComp = FindObject<UActorComponent>(nullptr, *ComponentName);
			if (ActorComp)
			{
				SelectComponent(ActorComp, true, /*bNotifySelectionChanged*/true);
			}
		}

		GetSelectedComponents()->EndBatchSelectOperation();
	}

	NoteSelectionChange();
}

void UEditorEngine::ResetAllSelectionSets()
{
	GetSelectedObjects()->DeselectAll();
	GetSelectedActors()->DeselectAll();
	GetSelectedComponents()->DeselectAll();
}

static bool GetSmallToolBarIcons()
{
	return GetDefault<UEditorStyleSettings>()->bUseSmallToolBarIcons;
}

static bool GetDisplayMultiboxHooks()
{
	return GetDefault<UEditorPerProjectUserSettings>()->bDisplayUIExtensionPoints;
}

static int GetMenuSearchFieldVisibilityThreshold()
{
	return GetDefault<UEditorStyleSettings>()->MenuSearchFieldVisibilityThreshold;
}

void UEditorEngine::InitEditor(IEngineLoop* InEngineLoop)
{
	// Allow remote execution of derived data builds from this point
	// TODO: This needs to be enabled earlier to allow early data builds to be remote executed.
	if (FParse::Param(FCommandLine::Get(), TEXT("ExecuteBuildsLocally")))
	{
		InitDerivedDataBuildLocalExecutor();
	}
	else
	{
		InitDerivedDataBuildRemoteExecutor();
	}
	InitDerivedDataBuildWorkers();

	// Startup UnrealEdMisc
	FUnrealEdMisc::Get();

	// Call base.
	UEngine::Init(InEngineLoop);

	// Create selection sets.
	PrivateEditorSelection::InitSelectionSets();

	// Set slate options
	FMultiBoxSettings::UseSmallToolBarIcons = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&GetSmallToolBarIcons));
	FMultiBoxSettings::DisplayMultiboxHooks = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateStatic(&GetDisplayMultiboxHooks));
	FMultiBoxSettings::MenuSearchFieldVisibilityThreshold = TAttribute<int>::Create(TAttribute<int>::FGetter::CreateStatic(&GetMenuSearchFieldVisibilityThreshold));

	if ( FSlateApplication::IsInitialized() )
	{
		const UEditorStyleSettings* EditorSettings = GetDefault<UEditorStyleSettings>();
		const EColorVisionDeficiency DeficiencyType = EditorSettings->ColorVisionDeficiencyPreviewType;
		const int32 Severity = EditorSettings->ColorVisionDeficiencySeverity;
		const bool bCorrectDeficiency = EditorSettings->bColorVisionDeficiencyCorrection;
		const bool bShowCorrectionWithDeficiency = EditorSettings->bColorVisionDeficiencyCorrectionPreviewWithDeficiency;
		FSlateApplication::Get().GetRenderer()->SetColorVisionDeficiencyType(DeficiencyType, Severity, bCorrectDeficiency, bShowCorrectionWithDeficiency);
	}

	UEditorStyleSettings* StyleSettings = GetMutableDefault<UEditorStyleSettings>();
	StyleSettings->Init();

	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	// Needs to be set early as materials can be cached with selected material color baked in
	GEngine->SetSelectedMaterialColor(ViewportSettings->bHighlightWithBrackets ? FLinearColor::Black : StyleSettings->SelectionColor);
	GEngine->SetSelectionOutlineColor(StyleSettings->SelectionColor);
	GEngine->SetSubduedSelectionOutlineColor(StyleSettings->GetSubduedSelectionColor());
	GEngine->SelectionHighlightIntensity = ViewportSettings->SelectionHighlightIntensity;
	GEngine->BSPSelectionHighlightIntensity = ViewportSettings->BSPSelectionHighlightIntensity;

	// Set navigation system property indicating whether navigation is supposed to rebuild automatically 
	FWorldContext &EditorContext = GetEditorWorldContext();
	FNavigationSystem::SetNavigationAutoUpdateEnabled(GetDefault<ULevelEditorMiscSettings>()->bNavigationAutoUpdate, EditorContext.World()->GetNavigationSystem());

	// Allocate temporary model.
	TempModel = NewObject<UModel>();
	TempModel->Initialize(nullptr, 1);
	ConversionTempModel = NewObject<UModel>();
	ConversionTempModel->Initialize(nullptr, 1);

	// create the timer manager
	TimerManager = MakeShareable(new FTimerManager());

	// create the editor world manager
	EditorWorldExtensionsManager = NewObject<UEditorWorldExtensionManager>();

	// Settings.
	FBSPOps::GFastRebuild = 0;

	// Setup delegate callbacks for SavePackage()
	FCoreUObjectDelegates::IsPackageOKToSaveDelegate.BindUObject(this, &UEditorEngine::IsPackageOKToSave);

	// Update recents
	UpdateRecentlyLoadedProjectFiles();

	// Update the auto-load project
	UpdateAutoLoadProject();

	// Load any modules that might be required by commandlets
	//FModuleManager::Get().LoadModule(TEXT("OnlineBlueprintSupport"));

	if ( FSlateApplication::IsInitialized() )
	{
		// Setup a delegate to handle requests for opening assets
		FSlateApplication::Get().SetWidgetReflectorAssetAccessDelegate(FAccessAsset::CreateUObject(this, &UEditorEngine::HandleOpenAsset));

		// Setup a delegate to handle requests for opening debug objects
		FSlateApplication::Get().SetWidgetReflectorDebugAccessDelegate(FAccessAsset::CreateUObject(this, &UEditorEngine::HandleOpenDebugObject));
	}

	IHotReloadModule& HotReloadModule = IHotReloadModule::Get();
	HotReloadModule.OnModuleCompilerStarted ().AddUObject(this, &UEditorEngine::OnModuleCompileStarted);
	HotReloadModule.OnModuleCompilerFinished().AddUObject(this, &UEditorEngine::OnModuleCompileFinished);

	IBookmarkTypeTools& BookmarkTools = IBookmarkTypeTools::Get();
	BookmarkTools.RegisterBookmarkTypeActions(MakeShared<FBookMark2DTypeActions>());
	BookmarkTools.RegisterBookmarkTypeActions(MakeShared<FBookMarkTypeActions>());
	
	{
		TArray<UClass*> VolumeClasses;
		TArray<UClass*> VolumeFactoryClasses;

		// Create array of ActorFactory instances.
		for (TObjectIterator<UClass> ObjectIt; ObjectIt; ++ObjectIt)
		{
			UClass* TestClass = *ObjectIt;
			if (TestClass == nullptr)
			{
				continue;
			}

			if (TestClass->IsChildOf(UActorFactory::StaticClass()))
			{
				if (!TestClass->HasAnyClassFlags(CLASS_Abstract))
				{
					// if the factory is a volume shape factory we create an instance for all volume types
					if (TestClass->IsChildOf(UActorFactoryVolume::StaticClass()))
					{
						VolumeFactoryClasses.Add(TestClass);
					}
					else if (TestClass->GetDefaultObject<UActorFactory>()->bShouldAutoRegister)
					{
						UActorFactory* NewFactory = NewObject<UActorFactory>(GetTransientPackage(), TestClass);
						check(NewFactory);
						ActorFactories.Add(NewFactory);
					}
				}
			}
			else if (TestClass->IsChildOf(AVolume::StaticClass()) && TestClass != AVolume::StaticClass())
			{
				// we want classes derived from AVolume, but not AVolume itself
				VolumeClasses.Add(TestClass);
			}
		}

		ActorFactories.Reserve(ActorFactories.Num() + (VolumeFactoryClasses.Num() * VolumeClasses.Num()));
		for (UClass* VolumeFactoryClass : VolumeFactoryClasses)
		{
			// Use NewActorClass of Factory CDO as the supported base class for VolumeClasses
			const UClass* DefaultActorClass = VolumeFactoryClass->GetDefaultObject<UActorFactory>()->NewActorClass;
			for (UClass* VolumeClass : VolumeClasses)
			{
				if (DefaultActorClass && (VolumeClass && !VolumeClass->IsChildOf(DefaultActorClass)))
				{
					continue;
				}

				UActorFactory* NewFactory = NewObject<UActorFactory>(GetTransientPackage(), VolumeFactoryClass);
				check(NewFactory);
				NewFactory->NewActorClass = VolumeClass;
				ActorFactories.Add(NewFactory);
			}
		}

		FCoreUObjectDelegates::ReloadAddedClassesDelegate.AddUObject(this, &UEditorEngine::CreateVolumeFactoriesForNewClasses);
	}

	// Used for sorting ActorFactory classes.
	struct FCompareUActorFactoryByMenuPriority
	{
		FORCEINLINE bool operator()(const UActorFactory& A, const UActorFactory& B) const
		{
			if (B.MenuPriority == A.MenuPriority)
			{
				if (A.GetClass() != UActorFactory::StaticClass() && B.IsA(A.GetClass()))
				{
					return false;
				}
				else if (B.GetClass() != UActorFactory::StaticClass() && A.IsA(B.GetClass()))
				{
					return true;
				}
				else
				{
					return A.GetClass()->GetName() < B.GetClass()->GetName();
				}
			}
			else
			{
				return B.MenuPriority < A.MenuPriority;
			}
		}
	};
	// Sort by menu priority.
	ActorFactories.Sort(FCompareUActorFactoryByMenuPriority());

	if (FSlateApplication::IsInitialized() && UToolMenus::IsToolMenuUIEnabled())
	{
		UToolMenus::Get()->EditMenuIcon = FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "MultiBox.GenericToolBarIcon.Small");
		UToolMenus::Get()->EditToolbarIcon = FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "MultiBox.GenericToolBarIcon");

		TWeakPtr<FTimerManager> WeakTimerManager = TimerManager;
		UToolMenus::Get()->AssignSetTimerForNextTickDelegate(FSimpleDelegate::CreateLambda([WeakTimerManager]()
		{
			if (WeakTimerManager.IsValid())
			{
				WeakTimerManager.Pin()->SetTimerForNextTick(UToolMenus::Get(), &UToolMenus::HandleNextTick);
			}
		}));

		static FAutoConsoleCommand ToolMenusEditMenusModeCVar = FAutoConsoleCommand(
			TEXT("ToolMenus.Edit"),
			TEXT("Show edit menu buttons"),
			FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			IToolMenusEditorModule::Get().RegisterShowEditMenusModeCheckbox();

			if (!UToolMenus::Get()->EditMenuDelegate.IsBound())
			{
				UToolMenus::Get()->EditMenuDelegate.BindLambda([](UToolMenu* InMenu)
				{
					IToolMenusEditorModule::Get().OpenEditToolMenuDialog(InMenu);
				});
			}

			bool bNewSetEditMenusMode = true;
			if (Args.Num() > 0)
			{
				bNewSetEditMenusMode = (Args[0] == TEXT("1")) || FCString::ToBool(*Args[0]);
			}

			UE_LOGF(LogEditor, Log, "%ls menu editing", bNewSetEditMenusMode ? TEXT("Enable") : TEXT("Disable"));
			UToolMenus::Get()->SetEditMenusMode(bNewSetEditMenusMode);
		}));

		bool bEnableEditToolMenusUI = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.EditorExperimentalSettings"), TEXT("bEnableEditToolMenusUI"), bEnableEditToolMenusUI, GEditorPerProjectIni);
		if (bEnableEditToolMenusUI)
		{
			IToolMenusEditorModule::Get().RegisterShowEditMenusModeCheckbox();

			UToolMenus::Get()->EditMenuDelegate.BindLambda([](UToolMenu* InMenu)
			{
				IToolMenusEditorModule::Get().OpenEditToolMenuDialog(InMenu);
			});
		}

		UToolMenus::Get()->ShouldDisplayExtensionPoints.BindStatic(&GetDisplayMultiboxHooks);

		UToolMenus::Get()->RegisterStringCommandHandler("Command", FToolMenuExecuteString::CreateLambda([](const FString& InString, const FToolMenuContext& InContext)
		{
			GEditor->Exec(nullptr, *InString);
		}));
	}

	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddUObject(this, &UEditorEngine::OnAssetPostCompile);
	
	WorldAddExtraDeletionObjectsHandle = FEditorDelegates::OnAddExtraObjectsToDelete.AddStatic(&UWorld::OnAddExtraObjectsToDelete);

	OnTimecodeProviderChanged().AddUObject(this, &UEditorEngine::RegisterTimecodeProviderCompiledDelegate);
	OnCustomTimeStepChanged().AddUObject(this, &UEditorEngine::RegisterCustomTimeStepCompiledDelegate);

	RegisterTimecodeProviderCompiledDelegate();
	RegisterCustomTimeStepCompiledDelegate();
}

bool UEditorEngine::HandleOpenAsset(UObject* Asset)
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
}

bool UEditorEngine::HandleOpenDebugObject(UObject* Object)
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenDebugObjectInEditor(Object);
}

void UEditorEngine::HandleSettingChanged( FName Name )
{
	// When settings are reset to default, the property name will be "None" so make sure that case is handled.
	if (Name == FName(TEXT("ColorVisionDeficiencyPreviewType")) || 
		Name == FName(TEXT("bColorVisionDeficiencyCorrection")) ||
		Name == FName(TEXT("bColorVisionDeficiencyCorrectionPreviewWithDeficiency")) ||
		Name == FName(TEXT("ColorVisionDeficiencySeverity")) ||
		Name == NAME_None)
	{
		const UEditorStyleSettings* EditorSettings = GetDefault<UEditorStyleSettings>();
		const EColorVisionDeficiency DeficiencyType = EditorSettings->ColorVisionDeficiencyPreviewType;
		const int32 Severity = EditorSettings->ColorVisionDeficiencySeverity;
		const bool bCorrectDeficiency = EditorSettings->bColorVisionDeficiencyCorrection;
		const bool bShowCorrectionWithDeficiency = EditorSettings->bColorVisionDeficiencyCorrectionPreviewWithDeficiency;
		FSlateApplication::Get().GetRenderer()->SetColorVisionDeficiencyType(DeficiencyType, Severity, bCorrectDeficiency, bShowCorrectionWithDeficiency);
	}

	if (Name == FName("SelectionColor") || Name == NAME_None)
	{
		// Selection outline color and material color use the same color but sometimes the selected material color can be overidden so these need to be set independently
		GEngine->SetSelectedMaterialColor(GetDefault<UEditorStyleSettings>()->SelectionColor);
		GEngine->SetSelectionOutlineColor(GetDefault<UEditorStyleSettings>()->SelectionColor);
		GEngine->SetSubduedSelectionOutlineColor(GetDefault<UEditorStyleSettings>()->GetSubduedSelectionColor());
	}
}

void UEditorEngine::InitializeObjectReferences()
{
	EditorSubsystemCollection.Initialize(this);

	Super::InitializeObjectReferences();

	if ( PlayFromHerePlayerStartClass == NULL )
	{
		PlayFromHerePlayerStartClass = LoadClass<ANavigationObjectBase>(NULL, *GetDefault<ULevelEditorPlaySettings>()->PlayFromHerePlayerStartClassName, NULL, LOAD_None, NULL);
	}

#if WITH_AUTOMATION_TESTS
	if (!AutomationCommon::OnEditorAutomationMapLoadDelegate().IsBound())
	{
		AutomationCommon::OnEditorAutomationMapLoadDelegate().AddUObject(this, &UEditorEngine::AutomationLoadMap);
	}
#endif
}

bool UEditorEngine::ShouldDrawBrushWireframe( AActor* InActor )
{
	return !IsRunningCommandlet() ? GLevelEditorModeTools().ShouldDrawBrushWireframe(InActor) : false;
}

//
// Init the editor.
//

extern void StripUnusedPackagesFromList(TArray<FString>& PackageList, const FString& ScriptSourcePath);

void UEditorEngine::Init(IEngineLoop* InEngineLoop)
{
	FScopedSlowTask SlowTask(100);

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Editor Engine Initialized"), STAT_EditorEngineStartup, STATGROUP_LoadTime);

	check(!HasAnyFlags(RF_ClassDefaultObject));

	FCoreDelegates::ModalMessageDialog.BindUObject(this, &UEditorEngine::OnModalMessageDialog);
	FCoreUObjectDelegates::ShouldLoadOnTop.BindUObject(this, &UEditorEngine::OnShouldLoadOnTop);
	FCoreDelegates::PreWorldOriginOffset.AddUObject(this, &UEditorEngine::PreWorldOriginOffset);
	FCoreUObjectDelegates::OnAssetLoaded.AddUObject(this, &UEditorEngine::OnAssetLoaded);
	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UEditorEngine::OnLevelAddedToWorld);
	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UEditorEngine::OnLevelRemovedFromWorld);
	FWorldDelegates::OnSeamlessTravelTransition.AddUObject(this, &UEditorEngine::OnSeamlessTravelTransition);

	IAssetRegistry::GetChecked().OnInMemoryAssetCreated().AddUObject(this, &UEditorEngine::OnAssetCreated);

	FEditorDelegates::BeginPIE.AddLambda([](bool)
	{
		FTextLocalizationManager::Get().PushAutoEnableGameLocalizationPreview();
		FTextLocalizationManager::Get().EnableGameLocalizationPreview();

		// Always make sure dynamic resolution starts with a clean history.
		GEngine->GetDynamicResolutionState()->ResetHistory();
	});

	FEditorDelegates::PrePIEEnded.AddLambda([this](bool)
	{
		if (GetPIEWorldContext() != nullptr && GetPIEWorldContext()->World() != nullptr)
		{
			GetPIEWorldContext()->World()->DestroyDemoNetDriver();
		}
	});

	FEditorDelegates::EndPIE.AddLambda([](bool)
	{
		FTextLocalizationManager::Get().PopAutoEnableGameLocalizationPreview();
		FTextLocalizationManager::Get().DisableGameLocalizationPreview();

		// Always resume the dynamic resolution state to ensure it is same state as in game builds when starting PIE.
		GEngine->ResumeDynamicResolution();

		if (FSlateApplication::IsInitialized())
		{
			//Reset color deficiency settings in case they have been modified during PIE
			const UEditorStyleSettings* EditorSettings = GetDefault<UEditorStyleSettings>();
			const EColorVisionDeficiency DeficiencyType = EditorSettings->ColorVisionDeficiencyPreviewType;
			const int32 Severity = EditorSettings->ColorVisionDeficiencySeverity;
			const bool bCorrectDeficiency = EditorSettings->bColorVisionDeficiencyCorrection;
			const bool bShowCorrectionWithDeficiency = EditorSettings->bColorVisionDeficiencyCorrectionPreviewWithDeficiency;
			FSlateApplication::Get().GetRenderer()->SetColorVisionDeficiencyType(DeficiencyType, Severity, bCorrectDeficiency, bShowCorrectionWithDeficiency);
		}

	});

	// Initialize vanilla status before other systems that consume its status are started inside InitEditor()
	UpdateIsVanillaProduct();
	FSourceCodeNavigation::AccessOnNewModuleAdded().AddLambda([this](FName InModuleName)
	{
		UpdateIsVanillaProduct();
	});

	// Init editor.
	SlowTask.EnterProgressFrame(40);
	GEditor = this;
	InitEditor(InEngineLoop);

	LoadEditorShaderPlatformPreview();


	// Init transactioning.
	Trans = CreateTrans();

	SlowTask.EnterProgressFrame(50);

	// Load all editor modules here
	LoadDefaultEditorModules();

	SlowTask.EnterProgressFrame(10);

	float BSPTexelScale = 100.0f;
	if( GetDefault<ULevelEditorViewportSettings>()->bUsePowerOf2SnapSize )
	{
		BSPTexelScale=128.0f;
	}
	UModel::SetGlobalBSPTexelScale(BSPTexelScale);

	// Load game user settings and apply
	UGameUserSettings* MyGameUserSettings = GetGameUserSettings();
	if (MyGameUserSettings)
	{
		MyGameUserSettings->LoadSettings();
		MyGameUserSettings->ApplySettings(true);
	}

	UEditorStyleSettings* Settings = GetMutableDefault<UEditorStyleSettings>();
	Settings->OnSettingChanged().AddUObject(this, &UEditorEngine::HandleSettingChanged);

	// Purge garbage.
	Cleanse( false, 0, NSLOCTEXT("UnrealEd", "Startup", "Startup") );

	FEditorCommandLineUtils::ProcessEditorCommands(FCommandLine::Get());

	CheckForMissingAdvancedRenderingRequirements();

	// for IsInitialized()
	bIsInitialized = true;
};

void UEditorEngine::CreateVolumeFactoriesForNewClasses(const TArray<UClass*>& NewClasses)
{
	TArray<UClass*> NewVolumeClasses;
	for (UClass* NewClass : NewClasses)
	{
		if (NewClass && NewClass->IsChildOf(AVolume::StaticClass()))
		{
			NewVolumeClasses.Add(NewClass);
		}
	}

	if (NewVolumeClasses.Num() > 0)
	{
		for (TObjectIterator<UClass> ObjectIt; ObjectIt; ++ObjectIt)
		{
			UClass* TestClass = *ObjectIt;
			if (TestClass == nullptr)
			{
				continue;
			}
		
		if (!TestClass->HasAnyClassFlags(CLASS_Abstract) && TestClass->IsChildOf(UActorFactoryVolume::StaticClass()))
			{
				ActorFactories.Reserve(ActorFactories.Num() + NewVolumeClasses.Num());
				for (UClass* NewVolumeClass : NewVolumeClasses)
				{
					UActorFactory* NewFactory = NewObject<UActorFactory>(GetTransientPackage(), TestClass);
					check(NewFactory);
					NewFactory->NewActorClass = NewVolumeClass;
					ActorFactories.Add(NewFactory);
				}
			}
		}
	}
}

void UEditorEngine::InitBuilderBrush( UWorld* InWorld )
{
	check( InWorld );
	const bool bOldDirtyState = InWorld->GetCurrentLevel()->GetOutermost()->IsDirty();

	// For additive geometry mode, make the builder brush a small 256x256x256 cube so its visible.
	const int32 CubeSize = 256;
	UCubeBuilder* CubeBuilder = NewObject<UCubeBuilder>();
	CubeBuilder->X = CubeSize;
	CubeBuilder->Y = CubeSize;
	CubeBuilder->Z = CubeSize;
	CubeBuilder->Build( InWorld );

	// Restore the level's dirty state, so that setting a builder brush won't mark the map as dirty.
	if (!bOldDirtyState)
	{
		InWorld->GetCurrentLevel()->GetOutermost()->SetDirtyFlag( bOldDirtyState );
	}
}

int32 UEditorEngine::AddViewportClients(FEditorViewportClient* ViewportClient)
{
	int32 Result = AllViewportClients.Add(ViewportClient);
	ViewportClientListChangedEvent.Broadcast();
	return Result;
}

void UEditorEngine::RemoveViewportClients(FEditorViewportClient* ViewportClient)
{
	AllViewportClients.Remove(ViewportClient);

	// fix up the other viewport indices
	for (int32 ViewportIndex = ViewportClient->ViewIndex; ViewportIndex < AllViewportClients.Num(); ViewportIndex++)
	{
		AllViewportClients[ViewportIndex]->ViewIndex = ViewportIndex;
	}
	ViewportClientListChangedEvent.Broadcast();
}

int32 UEditorEngine::AddLevelViewportClients(FLevelEditorViewportClient* ViewportClient)
{
	int32 Result = LevelViewportClients.Add(ViewportClient);
	LevelViewportClientListChangedEvent.Broadcast();
	return Result;
}

void UEditorEngine::RemoveLevelViewportClients(FLevelEditorViewportClient* ViewportClient)
{
	LevelViewportClients.Remove(ViewportClient);
	LevelViewportClientListChangedEvent.Broadcast();
}

void UEditorEngine::BroadcastObjectReimported(UObject* InObject)
{
	GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(InObject);
}

void UEditorEngine::LoadDefaultEditorModules()
{
	static const TCHAR* ModuleNames[] =
		{
			TEXT("Documentation"),
			TEXT("WorkspaceMenuStructure"),
			TEXT("MainFrame"),
			TEXT("OutputLog"),
			TEXT("SourceControl"),
			TEXT("SourceControlWindows"),
			TEXT("SourceControlWindowExtender"),
			TEXT("UncontrolledChangelists"),
			TEXT("TextureCompressor"),
			TEXT("MeshUtilities"),
			TEXT("MovieSceneTools"),
			TEXT("ClassViewer"),
			TEXT("StructViewer"),
			TEXT("ContentBrowser"),
			TEXT("AssetTools"),
			TEXT("GraphEditor"),
			TEXT("KismetCompiler"),
			TEXT("Kismet"),
			TEXT("Persona"),
			TEXT("AnimationBlueprintEditor"),
			TEXT("LevelEditor"),
			TEXT("MainFrame"),
			TEXT("PropertyEditor"),
			TEXT("PackagesDialog"),
			// TEXT("AssetRegistry"), // Loaded in constructor
			TEXT("DetailCustomizations"),
			TEXT("ComponentVisualizers"),
			TEXT("Layers"),
			TEXT("AutomationWindow"),
			TEXT("AutomationController"),
			TEXT("DeviceManager"),
			TEXT("SessionFrontend"),
			TEXT("LegacyProjectLauncher"),
			TEXT("SettingsEditor"),
			TEXT("EditorSettingsViewer"),
			TEXT("InputBindingSettingsViewer"),
			TEXT("ProjectSettingsViewer"),
			TEXT("Blutility"),
			TEXT("ScriptableEditorWidgets"),
			TEXT("XmlParser"),
			TEXT("UndoHistory"),
			TEXT("DeviceProfileEditor"),
			TEXT("SourceCodeAccess"),
			TEXT("BehaviorTreeEditor"),
			TEXT("HardwareTargeting"),
			TEXT("LocalizationDashboard"),
			TEXT("MergeActors"),
			TEXT("InputBindingEditor"),
			TEXT("AudioEditor"),
			TEXT("EditorInteractiveToolsFramework"),
			TEXT("TraceInsights"),
			TEXT("StaticMeshEditor"),
			TEXT("EditorFramework"),
			TEXT("WorldBookmark"),
			TEXT("WorldPartitionEditor"),
			TEXT("EditorConfig"),
			TEXT("DerivedDataEditor"),
			TEXT("ZenEditor"),
			TEXT("CSVtoSVG"),
			TEXT("GeometryFramework"),
			TEXT("VirtualizationEditor"),
			TEXT("AnimationSettings"),
			TEXT("GameplayDebuggerEditor"),
			TEXT("RenderResourceViewer"),
			TEXT("UniversalObjectLocatorEditor"),
			TEXT("StructUtilsEditor"),
			TEXT("StructUtilsTestSuite"),
			TEXT("MassEntityEditor"),
			TEXT("SVGDistanceField"),
			TEXT("DataHierarchyEditor"),
			TEXT("ToolWidgets"),
			TEXT("UserAssetTagsEditor")
		};

	FScopedSlowTask ModuleSlowTask((float)UE_ARRAY_COUNT(ModuleNames));
	for (const TCHAR* ModuleName : ModuleNames)
	{
		ModuleSlowTask.EnterProgressFrame(1);
		FModuleManager::Get().LoadModule(ModuleName);
	}

	{
		// Load platform runtime settings modules
		TArray<FName> Modules;
		FModuleManager::Get().FindModules( TEXT( "*RuntimeSettings" ), Modules );

		for( int32 Index = 0; Index < Modules.Num(); Index++ )
		{
			FModuleManager::Get().LoadModule( Modules[Index] );
		}
	}

	{
		// Load platform editor modules
		TArray<FName> Modules;
		FModuleManager::Get().FindModules( TEXT( "*PlatformEditor" ), Modules );

		for( int32 Index = 0; Index < Modules.Num(); Index++ )
		{
			if( Modules[Index] != TEXT("ProjectTargetPlatformEditor") )
			{
				FModuleManager::Get().LoadModule( Modules[Index] );
			}
		}
	}

	if( FParse::Param( FCommandLine::Get(),TEXT( "PListEditor" ) ) )
	{
		FModuleManager::Get().LoadModule(TEXT("PListEditor"));
	}

	FModuleManager::Get().LoadModule(TEXT("LogVisualizer"));
	FModuleManager::Get().LoadModule(TEXT("WidgetRegistration"));
	FModuleManager::Get().LoadModule(TEXT("HotReload"));

	FModuleManager::Get().LoadModuleChecked(TEXT("ClothPainter"));

	// Load VR Editor support
	FModuleManager::Get().LoadModuleChecked( TEXT( "ViewportInteraction" ) );
	FModuleManager::Get().LoadModuleChecked( TEXT( "VREditor" ) );
}

void UEditorEngine::PreExit()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Cleanup GWorld before final GC Purge
		if (UWorld* World = GWorld)
		{
			World->ClearWorldComponents();
			World->CleanupWorld();
		}
		
		// Cleanup worlds that were initialized through UEditorEngine::InitializeNewlyCreatedInactiveWorld before final GC Purge
		for (TObjectIterator<UWorld> It; It; ++It)
		{
			if (UWorld* World = *It; World && World->WorldType == EWorldType::Inactive && World->IsInitialized())
			{
				// GWorld shouldn't be an Inactive World
				check(World != GWorld);
				World->ClearWorldComponents();
				World->CleanupWorld();
			}
		}

		EditorSubsystemCollection.Deinitialize();

		UnregisterTEDSQueries();
	}

	FProjectEditorRecord::TearDown();

	Super::PreExit();
}

void UEditorEngine::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		if (PlayWorld)
		{
			// this needs to be already cleaned up
			UE_LOGF(LogEditor, Warning, "Warning: Play world is active");
		}

		if (UToolMenus* ToolMenus = UToolMenus::TryGet())
		{
			ToolMenus->ShouldDisplayExtensionPoints.Unbind();
			ToolMenus->UnregisterStringCommandHandler("Command");
		}

		// Unregister events
		UObject::OnGetPreviewPlatform.Unbind();
		FEditorDelegates::MapChange.RemoveAll(this);
		FCoreDelegates::ModalMessageDialog.Unbind();
		FCoreUObjectDelegates::ShouldLoadOnTop.Unbind();
		FCoreDelegates::PreWorldOriginOffset.RemoveAll(this);
		FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
		FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
		FWorldDelegates::OnSeamlessTravelTransition.RemoveAll(this);
		GetMutableDefault<UEditorStyleSettings>()->OnSettingChanged().RemoveAll(this);

		FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule)
		{
			IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
			if (AssetRegistry)
			{
				AssetRegistry->OnInMemoryAssetCreated().RemoveAll(this);
			}
		}

		OnTimecodeProviderChanged().RemoveAll(this);
		OnCustomTimeStepChanged().RemoveAll(this);

		FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);

		FEditorDelegates::OnAddExtraObjectsToDelete.Remove(WorldAddExtraDeletionObjectsHandle);

		// Shut down transaction tracking system.
		if( Trans )
		{
			if( GUndo )
			{
				UE_LOGF(LogEditor, Warning, "Warning: A transaction is active" );
			}
			ResetTransaction( NSLOCTEXT("UnrealEd", "Shutdown", "Shutdown") );
		}

		// Destroy selection sets.
		PrivateEditorSelection::DestroySelectionSets();

		// Remove editor array from root.
		UE_LOGF(LogExit, Log, "Editor shut down" );

		// Any access of GEditor after finish destroy is invalid
		// Null out GEditor so that potential module shutdown that happens after can check for nullptr
		if (GEditor == this)
		{
			GEditor = nullptr;
		}
	}

	ShutdownDerivedDataBuildWorkers();

	Super::FinishDestroy();
}

void UEditorEngine::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UEditorEngine* This = CastChecked<UEditorEngine>(InThis);
	// Serialize viewport clients.
	for(FEditorViewportClient* ViewportClient : This->AllViewportClients)
	{
		ViewportClient->AddReferencedObjects( Collector );
	}

	// Serialize ActorFactories
	for( int32 Index = 0; Index < This->ActorFactories.Num(); Index++ )
	{
		Collector.AddReferencedObject( This->ActorFactories[ Index ], This );
	}

	// If a PIE session is about to start, keep the settings object alive.
	if (This->PlaySessionRequest.IsSet() && This->PlaySessionRequest->EditorPlaySettings)
	{
		Collector.AddReferencedObject(This->PlaySessionRequest->EditorPlaySettings, This);
	}

	// If we're in a PIE session, ensure we keep the current settings object alive.
	if (This->PlayInEditorSessionInfo.IsSet() && This->PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings)
	{
		Collector.AddReferencedObject(This->PlayInEditorSessionInfo->OriginalRequestParams.EditorPlaySettings, This);
	}

	// Keep Editor subsystems alive
	This->EditorSubsystemCollection.AddReferencedObjects(This, Collector);

	Super::AddReferencedObjects( This, Collector );
}

void UEditorEngine::Tick( float DeltaSeconds, bool bIdleMode )
{
	NETWORK_PROFILER(GNetworkProfiler.TrackFrameBegin());

	UWorld* CurrentGWorld = GWorld;
	check( CurrentGWorld );
	check( CurrentGWorld != PlayWorld || bIsSimulatingInEditor );

	// Clear out the list of objects modified this frame, used for OnObjectModified notification.
	FCoreUObjectDelegates::ObjectsModifiedThisFrame.Empty();

	// Always ensure we've got adequate slack for any worlds that are going to get created in this frame so that
	// our EditorContext reference doesn't get invalidated
	WorldList.Reserve(WorldList.Num() + 10);

	FWorldContext& EditorContext = GetEditorWorldContext();
	check( CurrentGWorld == EditorContext.World() );

	// early in the Tick() to get the callbacks for cvar changes called
	IConsoleManager::Get().CallAllConsoleVariableSinks();

	// Tick the remote config IO manager
	FRemoteConfigAsyncTaskManager::Get()->Tick();

	// Clean up the game viewports that have been closed.
	CleanupGameViewport();

	// Potentially rebuilds the streaming data.
	EditorContext.World()->ConditionallyBuildStreamingData();

	// Update the timer manager
	TimerManager->Tick(DeltaSeconds);

	if (!GUseUnifiedTimeBudgetForStreaming)
	{
		// This assumes that UObject::StaticTick only calls ProcessAsyncLoading.
		StaticTick(DeltaSeconds, !!GAsyncLoadingUseFullTimeLimit, GAsyncLoadingTimeLimit / 1000.f);
	}

	FEngineAnalytics::Tick(DeltaSeconds);
	
	// Look for realtime flags.
	bool IsRealtime = false;

	// True if a viewport has realtime audio	// If any realtime audio is enabled in the editor
	bool bAudioIsRealtime = GetDefault<ULevelEditorMiscSettings>()->bEnableRealTimeAudio;

	// By default we tick the editor world.  
	// When in PIE if we are in immersive we do not tick the editor world unless there is a visible editor viewport.
	bool bShouldTickEditorWorld = true;

	// Conditionally disable all viewport rendering when the editor is in the background.
	// This aims to improve GPU performance of other applications when the editor is not actively used.
	{
		const UEditorPerformanceSettings* PerformanceSettings = GetDefault<UEditorPerformanceSettings>();
		const bool bShouldDisableRendering = !FApp::HasFocus() && PerformanceSettings->bThrottleCPUWhenNotForeground;
		const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_BackgroundProcess", "Background Process");
		for (FEditorViewportClient* const ViewportClient : AllViewportClients)
		{
			ViewportClient->RemoveRealtimeOverride(SystemDisplayName, false /*bCheckMissingOverride*/);
			if (bShouldDisableRendering)
			{
				ViewportClient->AddRealtimeOverride(false /*bShouldBeRealtime*/, SystemDisplayName);
			}
		}
	}

	//@todo Multiple Worlds: Do we need to consider what world we are in here?

	// Find which viewport has audio focus, i.e. gets to set the listener location
	// Priorities are:
	//  Active perspective realtime view
	//	> Any realtime perspective view (first encountered)
	//	> Active perspective view
	//	> Any perspective view (first encountered)
	FEditorViewportClient* AudioFocusViewportClient = NULL;
	{
		FEditorViewportClient* BestRealtimePerspViewport = NULL;
		FEditorViewportClient* BestPerspViewport = NULL;

		for(FEditorViewportClient* const ViewportClient : AllViewportClients)
		{
			if (ViewportClient->IsPerspective())
			{
				if (ViewportClient->IsRealtime())
				{
					if (ViewportClient->Viewport && ViewportClient->Viewport->HasFocus())
					{
						// active realtime perspective -- use this and be finished
						BestRealtimePerspViewport = ViewportClient;
						break;
					}
					else if (BestRealtimePerspViewport == NULL)
					{
						// save this
						BestRealtimePerspViewport = ViewportClient;
					}
				}
				else 
				{
					if (ViewportClient->Viewport && ViewportClient->Viewport->HasFocus())
					{
						// active non-realtime perspective -- use this
						BestPerspViewport = ViewportClient;
					}
					else if (BestPerspViewport == NULL)
					{
						// save this
						BestPerspViewport = ViewportClient;
					}

				}
			}
		}

		// choose realtime if set.  note this could still be null.
		AudioFocusViewportClient = BestRealtimePerspViewport ? BestRealtimePerspViewport : BestPerspViewport;
	}
	// tell viewportclient it has audio focus
	if (AudioFocusViewportClient)
	{
		// override realtime setting if viewport chooses (i.e. for cinematic preview)
		if (AudioFocusViewportClient->IsForcedRealtimeAudio())
		{
			bAudioIsRealtime = true;
		}
	}

	// Find realtime and visibility settings on all viewport clients
	for(FEditorViewportClient* const ViewportClient : AllViewportClients)
	{
		if( PlayWorld && ViewportClient->IsVisible() )
		{
			if( ViewportClient->IsInImmersiveViewport() )
			{
				// if a viewport client is immersive then by default we do not tick the editor world during PIE unless there is a visible editor world viewport
				bShouldTickEditorWorld = false;
			}
			else
			{
				// If the viewport is not immersive but still visible while we have a play world then we need to tick the editor world
				bShouldTickEditorWorld = true;
			}
		}

		if( ViewportClient->GetScene() == EditorContext.World()->Scene )
		{
			if( ViewportClient->IsRealtime() )
			{
				IsRealtime = true;
			}
		}
	}

	// Find out if the editor has focus. Audio should only play if the editor has focus unless "Allow background audio" is set.
	const bool bHasFocus = FApp::HasFocus();

	if (bHasFocus || GetDefault<ULevelEditorMiscSettings>()->bAllowBackgroundAudio)
	{
		if (!PlayWorld)
		{
			// Adjust the global volume multiplier if the window has focus and there is no pie world or no viewport overriding audio.
			FApp::SetVolumeMultiplier( GetDefault<ULevelEditorMiscSettings>()->EditorVolumeLevel );
		}
		else
		{
			// If there is currently a pie world a viewport is overriding audio settings do not adjust the volume.
			FApp::SetVolumeMultiplier( 1.0f );
		}
	}
	else
	{
		FApp::SetVolumeMultiplier(0.0f);
	}

	if (!bHasFocus)
	{
		SetPreviewMeshMode(false);
	}

	// Tick any editor FTickableEditorObject dervived classes
	FTickableEditorObject::TickObjects( DeltaSeconds );

	// Tick the asset registry
	FAssetRegistryModule::TickAssetRegistry(DeltaSeconds);

	static FName SourceCodeAccessName("SourceCodeAccess");
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(SourceCodeAccessName);
	SourceCodeAccessModule.GetAccessor().Tick(DeltaSeconds);

	// tick the directory watcher
	// @todo: Put me into an FTSTicker that is created when the DW module is loaded
	if( !FApp::IsProjectNameEmpty() )
	{
		static FName DirectoryWatcherName("DirectoryWatcher");
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(DirectoryWatcherName);
		DirectoryWatcherModule.Get()->Tick(DeltaSeconds);
	}

#if !UE_SERVER
	static const FName MediaModuleName(TEXT("Media"));
	IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>(MediaModuleName);
#endif

	bool bAWorldTicked = false;
	bool bMediaModulePreEngineTickDone = false;
	ELevelTick TickType = IsRealtime ? LEVELTICK_ViewportsOnly : LEVELTICK_TimeOnly;

	if( bShouldTickEditorWorld )
	{ 
		//EditorContext.World()->FXSystem->Resume();
		// Note: Still allowing the FX system to tick so particle systems dont restart after entering/leaving responsive mode
		if( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() )
		{
			FKismetDebugUtilities::NotifyDebuggerOfStartOfGameFrame(EditorContext.World());
			EditorContext.World()->Tick(TickType, DeltaSeconds);
			bAWorldTicked = true;
			FKismetDebugUtilities::NotifyDebuggerOfEndOfGameFrame(EditorContext.World());

			// Track if the editor context has an active movie sequence tick associated -> if so it will have executed the MediaModule pre-engine tick already
			bMediaModulePreEngineTickDone = (MediaModule != nullptr) && EditorContext.World()->IsMovieSceneSequenceTickHandlerBound();
		}
	}

	// Perform editor level streaming previs if no PIE session is currently in progress.
	if( !PlayWorld )
	{
		for(FLevelEditorViewportClient* ViewportClient : LevelViewportClients)
		{
			// Previs level streaming volumes in the Editor.
			if ( ViewportClient->IsPerspective() && GetDefault<ULevelEditorViewportSettings>()->bLevelStreamingVolumePrevis )
			{
				const FVector& ViewLocation = ViewportClient->GetViewLocation();

				// Iterate over streaming levels and compute whether the ViewLocation is in their associated volumes.
				TMap<ALevelStreamingVolume*, bool> VolumeMap;

				for (ULevelStreaming* StreamingLevel : EditorContext.World()->GetStreamingLevels())
				{
					if( StreamingLevel )
					{
						// Assume the streaming level is invisible until we find otherwise.
						bool bStreamingLevelShouldBeVisible = false;

						// We're not going to change level visibility unless we encounter at least one
						// volume associated with the level.
						bool bFoundValidVolume = false;

						// For each streaming volume associated with this level . . .
						for ( int32 VolumeIndex = 0 ; VolumeIndex < StreamingLevel->EditorStreamingVolumes.Num() ; ++VolumeIndex )
						{
							ALevelStreamingVolume* StreamingVolume = StreamingLevel->EditorStreamingVolumes[VolumeIndex];
							if ( StreamingVolume && !StreamingVolume->bDisabled )
							{
								bFoundValidVolume = true;

								bool bViewpointInVolume;
								bool* bResult = VolumeMap.Find(StreamingVolume);
								if ( bResult )
								{
									// This volume has already been considered for another level.
									bViewpointInVolume = *bResult;
								}
								else
								{
									// Compute whether the viewpoint is inside the volume and cache the result.
									bViewpointInVolume = StreamingVolume->EncompassesPoint( ViewLocation );							

								
									VolumeMap.Add( StreamingVolume, bViewpointInVolume );
								}

								// Halt when we find a volume associated with the level that the viewpoint is in.
								if ( bViewpointInVolume )
								{
									bStreamingLevelShouldBeVisible = true;
									break;
								}
							}
						}

						// Set the streaming level visibility status if we encountered at least one volume.
						if ( bFoundValidVolume && StreamingLevel->GetShouldBeVisibleInEditor() != bStreamingLevelShouldBeVisible )
						{
							StreamingLevel->SetShouldBeVisibleInEditor(bStreamingLevelShouldBeVisible);
						}
					}
				}
				
				// Simulate world composition streaming while in editor world
				if (EditorContext.World()->WorldComposition)
				{
					EditorContext.World()->WorldComposition->UpdateEditorStreamingState(ViewLocation);
				}

				break;
			}
		}

		// Call UpdateLevelStreaming if some streaming levels needs consideration. i.e Visibility has changed
		if (EditorContext.World()->HasStreamingLevelsToConsider())
		{
			EditorContext.World()->UpdateLevelStreaming();
			FEditorDelegates::RefreshPrimitiveStatsBrowser.Broadcast();
		}
	}

	bool bToggledBetweenPIEandSIE = bIsToggleBetweenPIEandSIEQueued;

	// Kick off a Play Session request if one was queued up during the last frame.
	if (PlaySessionRequest.IsSet())
	{
		StartQueuedPlaySessionRequest();
	}
	else if( bIsToggleBetweenPIEandSIEQueued )
	{
		ToggleBetweenPIEandSIE();
	}

	// Deferred until here so it doesn't happen mid iteration of worlds.
	if (PlayInEditorSessionInfo.IsSet() && PlayInEditorSessionInfo->bLateJoinRequested)
	{
		AddPendingLateJoinClient();
	}

	static bool bFirstTick = true;
	const bool bInsideTick = true;

	// Skip updating reflection captures on the first update as the level will not be ready to display
	if (!bFirstTick)
	{
		// Update sky light first because sky diffuse will be visible in reflection capture indirect specular
		USkyLightComponent::UpdateSkyCaptureContents(EditorContext.World());
		UReflectionCaptureComponent::UpdateReflectionCaptureContents(EditorContext.World(), nullptr, false, false, bInsideTick);
	}

	EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::BeginFrame);

	// if we have the side-by-side world for "Play From Here", tick it unless we are ensuring slate is responsive
	bool bHasPIEViewport = false;
	if( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() )
	{
		// Determine number of PIE worlds that should tick and if they feature an active movie sequence tick.
		bool bMovieSequenceTickWillBeHandled = false;
		TArray<FWorldContext*> LocalPieContextPtrs;
		for (FWorldContext& PieContext : WorldList)
		{
			if (PieContext.WorldType == EWorldType::PIE && PieContext.World() != nullptr && PieContext.World()->ShouldTick())
			{
				LocalPieContextPtrs.Add(&PieContext);
				bMovieSequenceTickWillBeHandled |= (MediaModule != nullptr) && PieContext.World()->IsMovieSceneSequenceTickHandlerBound();
			}
		}

#if !UE_SERVER
		// If we do not have any active movie sequence ticking, we will issue the pre-engine tick for the MediaModule as soon as we can
		if ((MediaModule != nullptr) && !bMediaModulePreEngineTickDone && !bMovieSequenceTickWillBeHandled)
		{
			MediaModule->TickPreEngine();
		}
#endif

		// Note: WorldList can change size within this loop during PIE when stopped at a breakpoint. In that case, we are
		// running in a nested tick loop within this loop, where a new editor window with a preview viewport can be opened.
		// So we iterate on a local list here instead.
		for (FWorldContext* PieContextPtr : LocalPieContextPtrs)
		{
			FWorldContext& PieContext = *PieContextPtr;

			PlayWorld = PieContext.World();
			GameViewport = PieContext.GameViewport;

			// Switch worlds and set the play world ID
			UWorld* OldGWorld = SetPlayInEditorWorld(PlayWorld);

			// Transfer debug references to ensure debugging ref's are valid for this tick in case of multiple game instances.
			if (OldGWorld && OldGWorld != PlayWorld)
			{
				OldGWorld->TransferBlueprintDebugReferences(PlayWorld);
			}
			
			float TickDeltaSeconds; // How much time to use per tick
			
			if (PieContext.PIEFixedTickSeconds > 0.f)
			{
				PieContext.PIEAccumulatedTickSeconds += DeltaSeconds;
				TickDeltaSeconds = PieContext.PIEFixedTickSeconds;
			}
			else
			{
				PieContext.PIEAccumulatedTickSeconds = DeltaSeconds;
				TickDeltaSeconds = DeltaSeconds;
			}
			
			for ( ; PieContext.PIEAccumulatedTickSeconds >= TickDeltaSeconds; PieContext.PIEAccumulatedTickSeconds -= TickDeltaSeconds)
			{
				// Tick all travel and Pending NetGames (Seamless, server, client)
				TickWorldTravel(PieContext, TickDeltaSeconds);

				// Updates 'connecting' message in PIE network games
				UpdateTransitionType(PlayWorld);

				// Update streaming for dedicated servers in PIE
				if (PieContext.RunAsDedicated && !GUseUnifiedTimeBudgetForStreaming)
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateLevelStreaming);
					PlayWorld->UpdateLevelStreaming();
				}

				// Update the level.
				{
					// So that hierarchical stats work in PIE
					SCOPE_CYCLE_COUNTER(STAT_FrameTime);

					FKismetDebugUtilities::NotifyDebuggerOfStartOfGameFrame(PieContext.World());

					// tick the level
					PieContext.World()->Tick( LEVELTICK_All, TickDeltaSeconds );
#if WITH_EDITOR
					PieContext.World()->bToggledBetweenPIEandSIEThisFrame = bToggledBetweenPIEandSIE;
#endif
					bAWorldTicked = true;
					TickType = LEVELTICK_All;

					// Block on async loading if requested.
					if (PlayWorld->bRequestedBlockOnAsyncLoading)
					{
						BlockTillLevelStreamingCompleted(PlayWorld);
						PlayWorld->bRequestedBlockOnAsyncLoading = false;
					}

					if (!bFirstTick)
					{
						// Update sky light first because sky diffuse will be visible in reflection capture indirect specular
						USkyLightComponent::UpdateSkyCaptureContents(PlayWorld);
						UReflectionCaptureComponent::UpdateReflectionCaptureContents(PlayWorld, nullptr, false, false, bInsideTick);
					}

					FKismetDebugUtilities::NotifyDebuggerOfEndOfGameFrame(PieContext.World());
				}

				// Tick the viewports.
				if ( GameViewport != NULL )
				{
					GameViewport->Tick(TickDeltaSeconds);
					bHasPIEViewport = true;
				}
			}

			// Pop the world
			RestoreEditorWorld( OldGWorld );
		}
	}
	else
	{
#if !UE_SERVER
	// If we do not tick anything of the above, we need to still make sure to issue the pre-tick to MediaModule
		// (unless the Editor context had a movie sequence tick active and hence already did so)
		if ((MediaModule != nullptr) && !bMediaModulePreEngineTickDone)
		{
			MediaModule->TickPreEngine();
		}
#endif
	}

	if (bAWorldTicked)
	{
		FTickableGameObject::TickObjects(nullptr, TickType, false, DeltaSeconds);
	}

#if !UE_SERVER
	// tick media framework post engine ticks
	if (MediaModule != nullptr)
	{
		MediaModule->TickPostEngine();
	}
#endif

	if (bFirstTick)
	{
		bFirstTick = false;
	}

	ensure(UE::GetPlayInEditorID() == INDEX_NONE);
	UE::SetPlayInEditorID(INDEX_NONE);

	// Clean up any game viewports that may have been closed during the level tick (eg by Kismet).
	CleanupGameViewport();

	// Updates all the extensions for all the editor worlds
	EditorWorldExtensionsManager->Tick( DeltaSeconds );

	// Update viewports.
	bool bRunDrawWithEditorHidden = false;
	for (int32 ViewportIndex = AllViewportClients.Num()-1; ViewportIndex >= 0; ViewportIndex--)
	{
		FEditorViewportClient* ViewportClient = AllViewportClients[ ViewportIndex ];

		// Propagate pending tools-context invalidations before the below "Allowing Expensive Tasks || bNeedsRedraw" gate,
		// since those invalidations are what can set bNeedsRedraw to true.
		if (FEditorModeTools* ModeTools = ViewportClient->GetModeTools())
		{
			ModeTools->PropagateInvalidations(ViewportClient);
		}

		// When throttling tick only viewports which need to be redrawn (they have been manually invalidated)
		if( ( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() || ViewportClient->bNeedsRedraw ) && ViewportClient->IsVisible() )
		{
			// Switch to the correct world for the client before it ticks
			FScopedConditionalWorldSwitcher WorldSwitcher( ViewportClient );

			ViewportClient->Tick(DeltaSeconds);
			bRunDrawWithEditorHidden |= ViewportClient->WantsDrawWhenAppIsHidden();
		}
	}

	bool bIsMouseOverAnyLevelViewport = false;

	//Do this check separate to the above loop as the ViewportClient may no longer be valid after we have ticked it
	for(FLevelEditorViewportClient* ViewportClient : LevelViewportClients)
	{
		FViewport* Viewport = ViewportClient->Viewport;

		// Keep track of whether the mouse cursor is over any level viewports
		if( Viewport != NULL )
		{
			const int32 MouseX = Viewport->GetMouseX();
			const int32 MouseY = Viewport->GetMouseY();
			if( MouseX >= 0 && MouseY >= 0 && MouseX < (int32)Viewport->GetSizeXY().X && MouseY < (int32)Viewport->GetSizeXY().Y )
			{
				bIsMouseOverAnyLevelViewport = true;
				break;
			}
		}
	}

	// If the cursor is outside all level viewports, then clear the hover effect
	if( !bIsMouseOverAnyLevelViewport )
	{
		FLevelEditorViewportClient::ClearHoverFromObjects();
	}

	

	// Commit changes to the BSP model.
	EditorContext.World()->CommitModelSurfaces();	

	bool bUpdateLinkedOrthoViewports = false;
	
	/////////////////////////////
	// Redraw viewports.
	{
		// Gather worlds that need EOF updates
		// This must be done in two steps as the object hash table is locked during ForEachObjectOfClass so any NewObject calls would fail
		TArray<UWorld*, TInlineAllocator<4>> WorldsToEOFUpdate;
		ForEachObjectOfClass(UWorld::StaticClass(), [&WorldsToEOFUpdate](UObject* WorldObj)
		{
			UWorld* World = CastChecked<UWorld>(WorldObj);
			if (World->HasEndOfFrameUpdates())
			{
				WorldsToEOFUpdate.Add(World);
			}
		});

		// Make sure deferred component updates have been sent to the rendering thread.
		for (UWorld* World : WorldsToEOFUpdate)
		{
			World->SendAllEndOfFrameUpdates();
		}
	}

	// Do not redraw if the application is hidden
	const bool bAllWindowsHidden = !bHasFocus && AreAllWindowsHidden();
	bool bAnyLevelEditorsDrawn = false;
	if (!bAllWindowsHidden || bRunDrawWithEditorHidden)
	{
		FPixelInspectorModule& PixelInspectorModule = FModuleManager::LoadModuleChecked<FPixelInspectorModule>(TEXT("PixelInspectorModule"));
		if (!bAllWindowsHidden && PixelInspectorModule.IsPixelInspectorEnable())
		{
			PixelInspectorModule.ReadBackSync();
		}

		// Render view parents, then view children.
		bool bEditorFrameNonRealtimeViewportDrawn = false;
		if (GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->IsVisible())
		{
			if (!bAllWindowsHidden || GCurrentLevelEditingViewportClient->WantsDrawWhenAppIsHidden())
			{
				bool bAllowNonRealtimeViewports = true;
				GCurrentLevelEditingViewportClient->SetIsCurrentLevelEditingFocus(true);
				bool bViewportDrawn;
				bool bWasNonRealtimeViewportDrawn = UpdateSingleViewportClient(GCurrentLevelEditingViewportClient, bAllowNonRealtimeViewports, bUpdateLinkedOrthoViewports, &bViewportDrawn);
				if (GCurrentLevelEditingViewportClient->IsLevelEditorClient())
				{
					bEditorFrameNonRealtimeViewportDrawn |= bWasNonRealtimeViewportDrawn;
					bAnyLevelEditorsDrawn |= bViewportDrawn;
				}
			}
		}
		for (int32 bRenderingChildren = 0; bRenderingChildren < 2; bRenderingChildren++)
		{
			for (FEditorViewportClient* ViewportClient : AllViewportClients)
			{
				if (ViewportClient == GCurrentLevelEditingViewportClient)
				{
					//already given this window a chance to update
					continue;
				}

				if (ViewportClient->IsVisible() && (!bAllWindowsHidden || ViewportClient->WantsDrawWhenAppIsHidden()) && ViewportClient->GetScene())
				{
					// Only update ortho viewports if that mode is turned on, the viewport client we are about to update is orthographic and the current editing viewport is orthographic and tracking mouse movement.
					bUpdateLinkedOrthoViewports = GetDefault<ULevelEditorViewportSettings>()->bUseLinkedOrthographicViewports && ViewportClient->IsOrtho() && GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->IsOrtho() && GCurrentLevelEditingViewportClient->IsTracking();

					if (bRenderingChildren || bUpdateLinkedOrthoViewports)
					{
						//if we haven't drawn a non-realtime viewport OR not one of the main viewports
						bool bAllowNonRealtimeViewports = (!bEditorFrameNonRealtimeViewportDrawn) || !(ViewportClient->IsLevelEditorClient());
						ViewportClient->SetIsCurrentLevelEditingFocus(true);
						bool bViewportDrawn;
						bool bWasNonRealtimeViewportDrawn = UpdateSingleViewportClient(ViewportClient, bAllowNonRealtimeViewports, bUpdateLinkedOrthoViewports, &bViewportDrawn);
						if (ViewportClient->IsLevelEditorClient())
						{
							bEditorFrameNonRealtimeViewportDrawn |= bWasNonRealtimeViewportDrawn;
							bAnyLevelEditorsDrawn |= bViewportDrawn;
						}
					}
				}
			}
		}
	}

	if (GUseUnifiedTimeBudgetForStreaming)
	{
		HandleUnifiedStreaming(DeltaSeconds);
	}

	// Rendering resources are normally flushed when a 3D viewport is drawn. If no viewports are updated (because the editor is hidden, or no realtime viewports are visible),
	// we need to force-flush resources here, since nothing else will. Note that this condition checks if any *level* viewports have been drawn in the block above; other
	// editors can contain 3D viewports and refreshing them will also flush resources, but it's difficult to know when 3D rendering has happened in general. Instead, we
	// optimize for the level editor case, since that's performance-sensitive. If there's no level editor, but there are other 3D editors, this will do an unnecessary
	// flush, but the small performance impact in that case should not affect users.
	if (!bAnyLevelEditorsDrawn && !bHasPIEViewport && IsRunningRHIInSeparateThread())
	{
		ENQUEUE_RENDER_COMMAND(FlushPendingDeleteRHIResources_NonRealtime)(
			[](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
			});
	}

	if (!bAllWindowsHidden || bRunDrawWithEditorHidden)
	{
		// Some tasks can only be done once we finish all scenes/viewports
		GetRendererModule().PostRenderAllViewports();
	}

	ISourceControlModule::Get().Tick();
	ILocalizationServiceModule::Get().Tick();

	if( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() )
	{
		{
			SCOPE_CYCLE_COUNTER(STAT_RedrawViewports);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ViewportMisc);
			for (auto ContextIt = WorldList.CreateIterator(); ContextIt; ++ContextIt)
			{
				FWorldContext &PieContext = *ContextIt;
				if (PieContext.WorldType != EWorldType::PIE)
				{
					continue;
				}

				PlayWorld = PieContext.World();
				GameViewport = PieContext.GameViewport;

				// Render playworld. This needs to happen after the other viewports for screenshots to work correctly in PIE.
				if (PlayWorld && GameViewport && !bIsSimulatingInEditor)
				{
					// Use the PlayWorld as the GWorld, because who knows what will happen in the Tick.
					UWorld* OldGWorld = SetPlayInEditorWorld( PlayWorld );

					// Render everything.
					GameViewport->LayoutPlayers();
					check(GameViewport->Viewport);
					GameViewport->Viewport->Draw();

					// Pop the world
					RestoreEditorWorld( OldGWorld );
				}
			}
			RedrawViewports();
		}
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateLevelStreaming);
			for (FWorldContext& Context : WorldList)
			{
				if (!GUseUnifiedTimeBudgetForStreaming && Context.World() &&
					(Context.WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Game) && !Context.RunAsDedicated)
				{
					Context.World()->UpdateLevelStreaming();
				}
			}
		}
	}

	// Update resource streaming after both regular Editor viewports and PIE had a chance to add viewers.
	IStreamingManager::Get().Tick(DeltaSeconds);

	// Determine whether or not we should end the current PIE session. In some cases the client context may not be fully
	// initialized until player login is complete, so make sure we have a valid world before actually handling the request.
	const bool bEndPlayMapThisFrame = PlayWorld && bRequestEndPlayMapQueued;

	// Update Audio. This needs to occur after rendering as the rendering code updates the listener position.
	if (AudioDeviceManager)
	{
		UWorld* OldGWorld = NULL;
		if (PlayWorld)
		{
			// Use the PlayWorld as the GWorld if we're using PIE.
			OldGWorld = SetPlayInEditorWorld(PlayWorld);
		}

		// Update audio device.
		AudioDeviceManager->UpdateActiveAudioDevices((!PlayWorld && bAudioIsRealtime) || (PlayWorld && !PlayWorld->IsPaused()));
		if (bEndPlayMapThisFrame)
		{
			// Shutdown all audio devices if we've requested end playmap now to avoid issues with GC running
			TArray<FAudioDevice*> AudioDevices = AudioDeviceManager->GetAudioDevices();
			for (FAudioDevice* AudioDevice : AudioDevices)
			{
					AudioDevice->Flush(nullptr);
				}
			}

		if (PlayWorld)
		{
			// Pop the world.
			RestoreEditorWorld(OldGWorld);
		}
	}

	// Update constraints if dirtied.
	EditorContext.World()->UpdateConstraintActors();

	ENQUEUE_RENDER_COMMAND(TickPoolElements)([](FRHICommandListImmediate& RHICmdList)
	{
		GRenderTargetPool.TickPoolElements();
		FRDGBuilder::TickPoolElements();
		ICustomResourcePool::TickPoolElements(RHICmdList);
	});

	// After the play world has ticked, see if a request was made to end pie
	if (bEndPlayMapThisFrame)
	{
		EndPlayMap();
	}

	FUnrealEdMisc::Get().TickAssetAnalytics();

	FUnrealEdMisc::Get().TickPerformanceAnalytics();

	BroadcastPostEditorTick(DeltaSeconds);

	// If the fadeout animation has completed for the undo/redo notification item, allow it to be deleted
	if(UndoRedoNotificationItem.IsValid() && UndoRedoNotificationItem->GetCompletionState() == SNotificationItem::CS_None)
	{
		UndoRedoNotificationItem.Reset();
	}
}

float UEditorEngine::GetMaxTickRate( float DeltaTime, bool bAllowFrameRateSmoothing ) const
{
	float MaxTickRate = 0.0f;
	if( !ShouldThrottleCPUUsage() )
	{
		// do not limit fps in VR Preview mode or VR editor mode
		if (IsVRPreviewActive() || GEnableVREditorHacks)
		{
			return 0.0f;
		}
		const float SuperMaxTickRate = Super::GetMaxTickRate( DeltaTime, bAllowFrameRateSmoothing );
		if( SuperMaxTickRate != 0.0f )
		{
			return SuperMaxTickRate;
		}

		// Clamp editor frame rate, even if smoothing is disabled
		if( !bSmoothFrameRate && GIsEditor && !GIsPlayInEditorWorld )
		{
			MaxTickRate = 1.0f / DeltaTime;
			if (SmoothedFrameRateRange.HasLowerBound())
			{
				MaxTickRate = FMath::Max(MaxTickRate, SmoothedFrameRateRange.GetLowerBoundValue());
			}
			if (SmoothedFrameRateRange.HasUpperBound())
			{
				MaxTickRate = FMath::Min(MaxTickRate, SmoothedFrameRateRange.GetUpperBoundValue());
			}
		}

		// Laptops should throttle to 60 hz in editor to reduce battery drain
		static const auto CVarDontLimitOnBattery = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DontLimitOnBattery"));
		const bool bLimitOnBattery = (FPlatformMisc::IsRunningOnBattery() && CVarDontLimitOnBattery->GetValueOnGameThread() == 0);
		if( bLimitOnBattery )
		{
			MaxTickRate = 60.0f;
		}
	}
	else
	{
		MaxTickRate = 3.0f;
	}

	return MaxTickRate;
}

bool UEditorEngine::IsRealTimeAudioMuted() const
{
	return GetDefault<ULevelEditorMiscSettings>()->bEnableRealTimeAudio ? false : true;
}

void UEditorEngine::MuteRealTimeAudio(bool bMute)
{
	ULevelEditorMiscSettings* LevelEditorMiscSettings = GetMutableDefault<ULevelEditorMiscSettings>();

	LevelEditorMiscSettings->bEnableRealTimeAudio = bMute ? false : true;
	LevelEditorMiscSettings->PostEditChange();
}

float UEditorEngine::GetRealTimeAudioVolume() const
{
	return GetDefault<ULevelEditorMiscSettings>()->EditorVolumeLevel;
}

void UEditorEngine::SetRealTimeAudioVolume(float VolumeLevel)
{
	ULevelEditorMiscSettings* LevelEditorMiscSettings = GetMutableDefault<ULevelEditorMiscSettings>();

	LevelEditorMiscSettings->EditorVolumeLevel = VolumeLevel;
	LevelEditorMiscSettings->PostEditChange();
}

bool UEditorEngine::UpdateSingleViewportClient(FEditorViewportClient* InViewportClient, const bool bInAllowNonRealtimeViewportToDraw, bool bLinkedOrthoMovement, bool* bOutViewportDrawn /*= nullptr*/)
{
	bool bUpdatedNonRealtimeViewport = false;
	bool bViewportDrawn = false;

	if (InViewportClient->Viewport->IsSlateViewport())
	{
		// When rendering the viewport we need to know whether the final result will be shown on a HDR display. This affects the final post processing step
		FSceneViewport *SceneViewport = static_cast<FSceneViewport*>(InViewportClient->Viewport);
		TSharedPtr<SWindow> Window = SceneViewport->FindWindow();
		if (Window)
		{
			InViewportClient->Viewport->SetHDRMode(Window->GetIsHDR());
		}
	}

	// Always submit view information for content streaming 
	// otherwise content for editor view can be streamed out if there are other views (ex: thumbnails)
	if (InViewportClient->IsPerspective())
	{
		float XSize = static_cast<float>(InViewportClient->Viewport->GetSizeXY().X);

		IStreamingManager::Get().AddViewInformation( InViewportClient->GetViewLocation(), XSize, XSize / FMath::Tan(FMath::DegreesToRadians(InViewportClient->ViewFOV * 0.5f)), 1.0f, false, 0.0f, nullptr, InViewportClient->GetWorld());
	}
	
	// Only allow viewports to be drawn if we are not throttling for slate UI responsiveness or if the viewport client requested a redraw
	// Note about bNeedsRedraw: Redraws can happen during some Slate events like checking a checkbox in a menu to toggle a view mode in the viewport.  In those cases we need to show the user the results immediately
	if( FSlateThrottleManager::Get().IsAllowingExpensiveTasks() || InViewportClient->bNeedsRedraw )
	{
		// Switch to the world used by the viewport before its drawn
		FScopedConditionalWorldSwitcher WorldSwitcher( InViewportClient );
	
		// Add view information for perspective viewports.
		if( InViewportClient->IsPerspective() )
		{
			if (UWorld* ViewportClientWorld = InViewportClient->GetWorld())
			{
				ViewportClientWorld->ViewLocationsRenderedLastFrame.Add(InViewportClient->GetViewLocation());
			}
	
			// If we're currently simulating in editor, then we'll need to make sure that sub-levels are streamed in.
			// When using PIE, this normally happens by UGameViewportClient::Draw().  But for SIE, we need to do
			// this ourselves!
			if( PlayWorld != NULL && bIsSimulatingInEditor && InViewportClient->IsSimulateInEditorViewport() )
			{
				if (!GUseUnifiedTimeBudgetForStreaming)
				{
					// Update level streaming.
					InViewportClient->GetWorld()->UpdateLevelStreaming();
				}

				// Also make sure hit proxies are refreshed for SIE viewports, as the user may be trying to grab an object or widget manipulator that's moving!
				if( InViewportClient->IsRealtime() && (GInvalidateHitProxiesEachSIEFrameCVar->GetInt() != 0))
				{
					// @todo simulate: This may cause simulate performance to be worse in cases where you aren't needing to interact with gizmos.  Consider making this optional.
					InViewportClient->RequestInvalidateHitProxy( InViewportClient->Viewport );
				}
			}
		}
	
		// Redraw the viewport if it's realtime.
		if( InViewportClient->IsRealtime() )
		{
			InViewportClient->Viewport->Draw();
			InViewportClient->bNeedsRedraw = false;
			InViewportClient->bNeedsLinkedRedraw = false;
			bViewportDrawn = true;
		}
		// Redraw any linked ortho viewports that need to be updated this frame.
		else if( InViewportClient->IsOrtho() && bLinkedOrthoMovement && InViewportClient->IsVisible() )
		{
			if( InViewportClient->bNeedsLinkedRedraw || InViewportClient->bNeedsRedraw )
			{
				// Redraw this viewport
				InViewportClient->Viewport->Draw();
				InViewportClient->bNeedsLinkedRedraw = false;
				InViewportClient->bNeedsRedraw = false;
				bViewportDrawn = true;
			}
			else
			{
				// This viewport doesn't need to be redrawn.  Skip this frame and increment the number of frames we skipped.
				InViewportClient->FramesSinceLastDraw++;
			}
		}
		// Redraw the viewport if there are pending redraw, and we haven't already drawn one viewport this frame.
		else if (InViewportClient->bNeedsRedraw && bInAllowNonRealtimeViewportToDraw)
		{
			InViewportClient->Viewport->Draw();
			InViewportClient->bNeedsRedraw = false;
			bViewportDrawn = true;
			bUpdatedNonRealtimeViewport = true;
		}
		else if(UWorld* World = GetWorld())
		{
			// We're not rendering but calculate the view anyway so that we can cache the last "rendered" view info in the UWorld.
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InViewportClient->Viewport, InViewportClient->GetScene(), InViewportClient->EngineShowFlags));
			FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);

			FWorldCachedViewInfo& WorldViewInfo = World->CachedViewInfoRenderedLastFrame.AddDefaulted_GetRef();
			WorldViewInfo.ViewMatrix = View->ViewMatrices.GetWorldToView();
			WorldViewInfo.ProjectionMatrix = View->ViewMatrices.GetViewToClip();
			WorldViewInfo.ViewProjectionMatrix = View->ViewMatrices.GetWorldToClip();
			WorldViewInfo.ViewToWorld = View->ViewMatrices.GetViewToWorld();
			World->LastRenderTime = World->GetTimeSeconds();
		}

		if (InViewportClient->bNeedsInvalidateHitProxy)
		{
			InViewportClient->Viewport->InvalidateHitProxy();
			InViewportClient->bNeedsInvalidateHitProxy = false;
		}
	}

	if (bOutViewportDrawn)
	{
		*bOutViewportDrawn = bViewportDrawn;
	}

	return bUpdatedNonRealtimeViewport;
}

void UEditorEngine::InvalidateAllViewportsAndHitProxies()
{
	for (FEditorViewportClient* ViewportClient : AllViewportClients)
	{
		ViewportClient->Invalidate();
	}
}

void UEditorEngine::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Propagate the callback up to the superclass.
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, MaximumLoopIterationCount))
	{
		// Clamp to a reasonable range and feed the new value to the script core
		MaximumLoopIterationCount = FMath::Clamp( MaximumLoopIterationCount, 100, 10000000 );
		FBlueprintCoreDelegates::SetScriptMaximumLoopIterations( MaximumLoopIterationCount );
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, bCanBlueprintsTickByDefault))
	{
		FScopedSlowTask SlowTask(100, LOCTEXT("DirtyingBlueprintsDueToTickChange", "InvalidatingAllBlueprints"));

		// Flag all Blueprints as out of date (this doesn't dirty the package as needs saving but will force a recompile during PIE)
		for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
		{
			UBlueprint* Blueprint = *BlueprintIt;
			Blueprint->Status = BS_Dirty;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, bOptimizeAnimBlueprintMemberVariableAccess) ||
			 PropertyName == GET_MEMBER_NAME_CHECKED(UEngine, bAllowMultiThreadedAnimationUpdate))
	{
		FScopedSlowTask SlowTask(100, LOCTEXT("DirtyingAnimBlueprintsDueToOptimizationChange", "Invalidating All Anim Blueprints"));

		// Flag all Blueprints as out of date (this doesn't dirty the package as needs saving but will force a recompile during PIE)
		for (TObjectIterator<UAnimBlueprint> AnimBlueprintIt; AnimBlueprintIt; ++AnimBlueprintIt)
		{
			UAnimBlueprint* AnimBlueprint = *AnimBlueprintIt;
			AnimBlueprint->Status = BS_Dirty;
		}
	}
}

void UEditorEngine::Cleanse( bool ClearSelection, bool Redraw, const FText& TransReset, bool bTransReset )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::Cleanse);

	check( !TransReset.IsEmpty() );

	if (GIsRunning || IsRunningCommandlet())
	{
		if( ClearSelection )
		{
			// Clear selection sets.
			GetSelectedActors()->DeselectAll();
			GetSelectedObjects()->DeselectAll();
		}

		if (bTransReset)
		{
			// Reset the transaction tracking system.
			ResetTransaction(TransReset);
		}

		// Notify any handlers of the cleanse.
		FEditorSupportDelegates::CleanseEditor.Broadcast();

		// Redraw the levels.
		if( Redraw )
		{
			RedrawLevelEditingViewports();
		}

		// Attempt to unload any loaded redirectors. Redirectors should not
		// be referenced in memory and are only used to forward references
		// at load time.
		//
		// We also have to remove packages that redirectors were contained
		// in if those were from redirector-only package, so they can be
		// loaded again in the future. If we don't do it loading failure
		// will occur next time someone tries to use it. This is caused by
		// the fact that the loading routing will check that already
		// existed, but the object was missing in cache.
		const EObjectFlags FlagsToClear = RF_Standalone | RF_Transactional;
		TSet<UPackage*> PackagesToUnload;
		for (TObjectIterator<UObjectRedirector> RedirIt; RedirIt; ++RedirIt)
		{
			UPackage* RedirectorPackage = RedirIt->GetOutermost();

			if (PackagesToUnload.Find(RedirectorPackage))
			{
				// Package was already marked to unload
				continue;
			}

			if (RedirectorPackage == GetTransientPackage())
			{
				RedirIt->ClearFlags(FlagsToClear);
				RedirIt->RemoveFromRoot();

				continue;
			}

			TArray<UObject*> PackageObjects;
			GetObjectsWithPackage(RedirectorPackage, PackageObjects);

			if (!PackageObjects.ContainsByPredicate(
					[](UObject* Object)
					{
						// Look for any standalone objects that are not a redirector or metadata, if found this is not a redirector-only package
						return !Object->IsA<UObjectRedirector>() && Object->HasAnyFlags(RF_Standalone);
					})
				)
			{
				PackagesToUnload.Add(RedirectorPackage);
			}
			else
			{
				// In case this isn't redirector-only package, clear just the redirector.
				RedirIt->ClearFlags(FlagsToClear);
				RedirIt->RemoveFromRoot();
			}
		}

		for (UPackage* PackageToUnload : PackagesToUnload)
		{
			TArray<UObject*> PackageObjects;
			GetObjectsWithPackage(PackageToUnload, PackageObjects);
			for (UObject* Object : PackageObjects)
			{
				Object->ClearFlags(FlagsToClear);
				Object->RemoveFromRoot();
			}

			PackageToUnload->ClearFlags(FlagsToClear);
			PackageToUnload->RemoveFromRoot();
		}

		// Collect garbage.
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

		// Remaining redirectors are probably referenced by editor tools. Keep them in memory for now.
		for (TObjectIterator<UObjectRedirector> RedirIt; RedirIt; ++RedirIt)
		{
			if ( RedirIt->IsAsset() )
			{
				RedirIt->SetFlags(RF_Standalone);
			}
		}
	}
}

void UEditorEngine::EditorUpdateComponents()
{
	GWorld->UpdateWorldComponents( true, false );
}

UAudioComponent* UEditorEngine::GetPreviewAudioComponent()
{
	return PreviewAudioComponent;
}

UAudioComponent* UEditorEngine::ResetPreviewAudioComponent(USoundBase* Sound, USoundNode* SoundNode)
{
	if (PreviewSoundCue)
	{
		PreviewSoundCue->FirstNode = nullptr;
		PreviewSoundCue = nullptr;
	}

	// The audio engine may be disabled
	if (!UseSound())
	{
		return nullptr;
	}

	// 1. Initialize & overwrite new component for previewing
	if (PreviewAudioComponent)
	{
		PreviewAudioComponent->Stop();
	}

	if (Sound)
	{
		PreviewAudioComponent = FAudioDevice::CreateComponent(Sound);
	}
	else if (SoundNode)
	{
		PreviewSoundCue = NewObject<USoundCue>();
		PreviewAudioComponent = FAudioDevice::CreateComponent(PreviewSoundCue);
	}
	else
	{
		PreviewAudioComponent = NewObject<UAudioComponent>();
	}

	check(PreviewAudioComponent);

	// Mark as a preview component so the distance calculations is ignored
	PreviewAudioComponent->bPreviewComponent = true;
	PreviewAudioComponent->bIsPreviewSound = true;
	PreviewAudioComponent->bAutoDestroy = false;
	PreviewAudioComponent->bIsUISound = true;
	PreviewAudioComponent->bAllowSpatialization = false;
	PreviewAudioComponent->bReverb = false;
	PreviewAudioComponent->bCenterChannelOnly = false;

	PreviewAudioComponent->AudioDeviceID = INDEX_NONE;
	if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
	{
		if (FAudioDeviceHandle ActiveDevice = DeviceManager->GetActiveAudioDevice(); ActiveDevice.IsValid())
		{
			PreviewAudioComponent->AudioDeviceID = ActiveDevice->DeviceID;
		}
	}

	if (Sound)
	{
		PreviewAudioComponent->Sound = Sound;
	}
	else if (SoundNode)
	{
		check(PreviewSoundCue);
		PreviewSoundCue->FirstNode = SoundNode;
		PreviewAudioComponent->Sound = PreviewSoundCue;
		PreviewSoundCue->CacheAggregateValues();
	}

	// 2. Determine whether or not to apply submix behavior
	{
		const UAudioEditorSettings* AudioEditorSettings = GetDefault<UAudioEditorSettings>();
		check(AudioEditorSettings);
		PreviewAudioComponent->bIsDryPreviewPlayback = !AudioEditorSettings->bUseSubmixesForPreviewSound;
		if (const USoundBase* PreviewSound = PreviewAudioComponent->Sound)
		{
			if (PreviewSound->EditorData.OverrideUseSubmixesOnPreview != EUseSubmixOnPreviewMode::UseEditorPreference)
			{
				PreviewAudioComponent->bIsDryPreviewPlayback = PreviewSound->EditorData.OverrideUseSubmixesOnPreview == EUseSubmixOnPreviewMode::PlayPreviewsWithoutSubmixes;
			}
		}
	}

	return PreviewAudioComponent;
}

UAudioComponent* UEditorEngine::PlayPreviewSound(USoundBase* Sound,  USoundNode* SoundNode)
{
	if(UAudioComponent* AudioComponent = ResetPreviewAudioComponent(Sound, SoundNode))
	{
		AudioComponent->Play();
		OnSoundBasePreview.Broadcast(Sound);

		return AudioComponent;
	}

	return nullptr;
}

void UEditorEngine::PlayEditorSound( const FString& SoundAssetName )
{
	// Only play sounds if the user has that feature enabled
	if( !UE::IsSavingPackage() && IsInGameThread() && GetDefault<ULevelEditorMiscSettings>()->bEnableEditorSounds )
	{
		USoundBase* Sound = Cast<USoundBase>( StaticFindObject( USoundBase::StaticClass(), NULL, *SoundAssetName ) );
		if( Sound == NULL )
		{
			Sound = Cast<USoundBase>( StaticLoadObject( USoundBase::StaticClass(), NULL, *SoundAssetName ) );
		}

		if( Sound != NULL )
		{
			PlayPreviewSound( Sound );
		}
	}
}

void UEditorEngine::PlayEditorSound( USoundBase* InSound )
{
	// Only play sounds if the user has that feature enabled
	if (!UE::IsSavingPackage() && CanPlayEditorSound())
	{
		if (InSound != nullptr)
		{
			PlayPreviewSound(InSound);
		}
	}
}

bool UEditorEngine::CanPlayEditorSound() const
{
	return IsInGameThread() && GetDefault<ULevelEditorMiscSettings>()->bEnableEditorSounds;
}

void UEditorEngine::ClearPreviewComponents()
{
	if (PreviewAudioComponent)
	{
		PreviewAudioComponent->Stop();

		if (PreviewSoundCue)
		{
			PreviewSoundCue->FirstNode = nullptr;
			PreviewSoundCue = nullptr;
		}

		PreviewAudioComponent->Sound = nullptr;
		PreviewAudioComponent = nullptr;
	}

	if (PreviewMeshComp)
	{
		PreviewMeshComp->UnregisterComponent();
		PreviewMeshComp = nullptr;
	}
}

void UEditorEngine::CloseEditedWorldAssets(UWorld* InWorld)
{
	if (!InWorld)
	{
		return;
	}

	// Find all assets being edited
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	TArray<UObject*> AllAssets = AssetEditorSubsystem->GetAllEditedAssets();

	TSet<UWorld*> ClosingWorlds;

	ClosingWorlds.Add(InWorld);

	for (ULevelStreaming* LevelStreaming : InWorld->GetStreamingLevels())
	{
		if (LevelStreaming && LevelStreaming->GetLoadedLevel())
		{
			ClosingWorlds.Add(CastChecked<UWorld>(LevelStreaming->GetLoadedLevel()->GetOuter()));
		}
	}

	for(int32 i=0; i<AllAssets.Num(); i++)
	{
		UObject* Asset = AllAssets[i];
		UWorld* AssetWorld = Asset->GetTypedOuter<UWorld>();

		if ( !AssetWorld )
		{
			// This might be a world, itself
			AssetWorld = Cast<UWorld>(Asset);
		}

		if (AssetWorld && ClosingWorlds.Contains(AssetWorld))
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(Asset);
		}
	}
}

UTextureRenderTarget2D* UEditorEngine::GetScratchRenderTarget( uint32 MinSize )
{
	UTextureRenderTarget2D* ScratchRenderTarget = nullptr;

	// We never allow render targets greater than 2048
	check(MinSize <= 2048);

	// 256x256
	if(MinSize <= 256)
	{
		if(ScratchRenderTarget256 == nullptr)
		{
			ScratchRenderTarget256 = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);

			ScratchRenderTarget256->TargetGamma = GEngine->DisplayGamma;
			ScratchRenderTarget256->RenderTargetFormat = RTF_RGBA8;

			ScratchRenderTarget256->InitAutoFormat(256, 256);
		}
		ScratchRenderTarget = ScratchRenderTarget256;
	}
	// 512x512
	else if(MinSize <= 512)
	{
		if( ScratchRenderTarget512 == nullptr)
		{
			ScratchRenderTarget512 = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);

			ScratchRenderTarget512->TargetGamma = GEngine->DisplayGamma;
			ScratchRenderTarget512->RenderTargetFormat = RTF_RGBA8;

			ScratchRenderTarget512->InitAutoFormat(512, 512);
		}
		ScratchRenderTarget = ScratchRenderTarget512;
	}
	// 1024x1024
	else if(MinSize <= 1024)
	{
		if( ScratchRenderTarget1024 == nullptr)
		{
			ScratchRenderTarget1024 = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);

			ScratchRenderTarget1024->TargetGamma = GEngine->DisplayGamma;
			ScratchRenderTarget1024->RenderTargetFormat = RTF_RGBA8;

			ScratchRenderTarget1024->InitAutoFormat(1024, 1024);
		}
		ScratchRenderTarget = ScratchRenderTarget1024;
	}
	// 2048x2048
	else if(MinSize <= 2048)
	{
		if(ScratchRenderTarget2048 == nullptr)
		{
			ScratchRenderTarget2048 = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), NAME_None, RF_Transient);

			ScratchRenderTarget2048->TargetGamma = GEngine->DisplayGamma;
			ScratchRenderTarget2048->RenderTargetFormat = RTF_RGBA8;

			ScratchRenderTarget2048->InitAutoFormat(2048, 2048);
		}
		ScratchRenderTarget = ScratchRenderTarget2048;
	}

	check(ScratchRenderTarget != nullptr);

	return ScratchRenderTarget;
}


bool UEditorEngine::WarnAboutHiddenLevels( UWorld* InWorld, bool bIncludePersistentLvl) const
{
	bool bResult = true;

	const bool bPersistentLvlHidden = !FLevelUtils::IsLevelVisible( InWorld->PersistentLevel );

	// Make a list of all hidden streaming levels.
	TArray< ULevelStreaming* > HiddenLevels;
	for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
	{
		if( StreamingLevel && !FLevelUtils::IsStreamingLevelVisibleInEditor( StreamingLevel ) )
		{
			HiddenLevels.Add( StreamingLevel );
		}
	}

	// Warn the user that some levels are hidden and prompt for continue.
	if ( ( bIncludePersistentLvl && bPersistentLvlHidden ) || HiddenLevels.Num() > 0 )
	{
		FText Message;
		if ( !bIncludePersistentLvl )
		{
			Message = LOCTEXT("TheFollowingStreamingLevelsAreHidden_Additional", "The following streaming levels are hidden:\n{HiddenLevelNameList}\n\n{ContinueMessage}");
		}
		else if ( bPersistentLvlHidden )
		{
			Message = LOCTEXT("TheFollowingLevelsAreHidden_Persistent", "The following levels are hidden:\n\n    Persistent Level{HiddenLevelNameList}\n\n{ContinueMessage}");
		}
		else
		{
			Message = LOCTEXT("TheFollowingLevelsAreHidden_Additional", "The following levels are hidden:\n{HiddenLevelNameList}\n\n{ContinueMessage}");
		}

		FString HiddenLevelNames;
		for ( int32 LevelIndex = 0 ; LevelIndex < HiddenLevels.Num() ; ++LevelIndex )
		{
			HiddenLevelNames += FString::Printf( TEXT("\n    %s"), *HiddenLevels[LevelIndex]->GetWorldAssetPackageName() );
		}

		FFormatNamedArguments Args;
		Args.Add( TEXT("HiddenLevelNameList"), FText::FromString( HiddenLevelNames ) );
		Args.Add( TEXT("ContinueMessage"), LOCTEXT("HiddenLevelsContinueWithBuildQ", "These levels will not be rebuilt. Leaving them hidden may invalidate what is built in other levels.\n\nContinue with build?\n(Yes All will show all hidden levels and continue with the build)") );

		const FText MessageBoxText = FText::Format( Message, Args );

		// Create and show the user the dialog.
		const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::YesNoYesAll, MessageBoxText);

		if( Choice == EAppReturnType::YesAll )
		{
			if ( bIncludePersistentLvl && bPersistentLvlHidden )
			{
				EditorLevelUtils::SetLevelVisibility( InWorld->PersistentLevel, true, false );
			}

			// The code below should technically also make use of FLevelUtils::SetLevelVisibility, but doing
			// so would be much more inefficient, resulting in several calls to UpdateLevelStreaming
			for( int32 HiddenLevelIdx = 0; HiddenLevelIdx < HiddenLevels.Num(); ++HiddenLevelIdx )
			{
				HiddenLevels[ HiddenLevelIdx ]->SetShouldBeVisibleInEditor(true);
			}

			InWorld->FlushLevelStreaming();

			// follow up using SetLevelVisibility - streaming should now be completed so we can show actors, layers, 
			// BSPs etc. without too big a performance hit.
			TArray<ULevel*> LoadedLevels;
			TArray<bool> bTheyShouldBeVisible;
			for( int32 HiddenLevelIdx = 0; HiddenLevelIdx < HiddenLevels.Num(); ++HiddenLevelIdx )
			{
				check(HiddenLevels[ HiddenLevelIdx ]->GetLoadedLevel());
				ULevel* LoadedLevel = HiddenLevels[ HiddenLevelIdx ]->GetLoadedLevel();
				LoadedLevels.Add(LoadedLevel);
				bTheyShouldBeVisible.Add(true);
			}
			// For efficiency, set visibility of all levels at once
			if (LoadedLevels.Num() > 0)
			{
				EditorLevelUtils::SetLevelsVisibility(LoadedLevels, bTheyShouldBeVisible, false);
			}

			FEditorSupportDelegates::RedrawAllViewports.Broadcast();
		}

		// return true if the user pressed make all visible or yes.
		bResult = (Choice != EAppReturnType::No);
	}

	return bResult;
}

void UEditorEngine::ApplyDeltaToActor(AActor* InActor,
									  bool bDelta,
									  const FVector* InTrans,
									  const FRotator* InRot,
									  const FVector* InScale,
									  bool bAltDown,
									  bool bShiftDown,
									  bool bControlDown) const
{
	FInputDeviceState InputState;
	InputState.SetModifierKeyStates(bShiftDown, bAltDown, bControlDown, false);

	FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(InActor, bDelta, InTrans, InRot, InScale, GLevelEditorModeTools().PivotLocation, InputState);
}

void UEditorEngine::ApplyDeltaToComponent(USceneComponent* InComponent,
	bool bDelta,
	const FVector* InTrans,
	const FRotator* InRot,
	const FVector* InScale,
	const FVector& PivotLocation ) const
{
	FComponentElementEditorViewportInteractionCustomization::ApplyDeltaToComponent(InComponent, bDelta, InTrans, InRot, InScale, PivotLocation, FInputDeviceState());
}


void UEditorEngine::ProcessToggleFreezeCommand( UWorld* InWorld )
{
	if (InWorld->IsPlayInEditor())
	{
		ULocalPlayer* Player = PlayWorld->GetFirstLocalPlayerFromController();
		if( Player )
		{
			Player->ViewportClient->Viewport->ProcessToggleFreezeCommand();
		}
	}
	else
	{
		// pass along the freeze command to all perspective viewports
		for(FLevelEditorViewportClient* ViewportClient : LevelViewportClients)
		{
			if (ViewportClient->IsPerspective())
			{
				ViewportClient->Viewport->ProcessToggleFreezeCommand();
			}
		}
	}

	// tell editor to update views
	RedrawAllViewports();
}


void UEditorEngine::ProcessToggleFreezeStreamingCommand(UWorld* InWorld)
{
	// freeze vis in PIE
	if (InWorld && InWorld->WorldType == EWorldType::PIE)
	{
		InWorld->bIsLevelStreamingFrozen = !InWorld->bIsLevelStreamingFrozen;
	}
}


void UEditorEngine::ParseMapSectionIni(const TCHAR* InCmdParams, TArray<FString>& OutMapList)
{
	FString SectionStr;
	if (FParse::Value(InCmdParams, TEXT("MAPINISECTION="), SectionStr))
	{
		if (SectionStr.Contains(TEXT("+")))
		{
			TArray<FString> Sections;
			SectionStr.ParseIntoArray(Sections,TEXT("+"),true);
			for (int32 Index = 0; Index < Sections.Num(); Index++)
			{
				LoadMapListFromIni(Sections[Index], OutMapList);
			}
		}
		else
		{
			LoadMapListFromIni(SectionStr, OutMapList);
		}
	}
}


void UEditorEngine::LoadMapListFromIni(const FString& InSectionName, TArray<FString>& OutMapList)
{
	// 
	const FConfigSection* MapListList = GConfig->GetSection(*InSectionName, false, GEditorIni);
	if (MapListList)
	{
		for (FConfigSectionMap::TConstIterator It(*MapListList) ; It ; ++It)
		{
			FName EntryType = It.Key();
			const FString& EntryValue = It.Value().GetValue();

			if (EntryType == NAME_Map)
			{
				// Add it to the list
				OutMapList.AddUnique(EntryValue);
			}
			else if (EntryType == FName(TEXT("Section")))
			{
				// Recurse...
				LoadMapListFromIni(EntryValue, OutMapList);
			}
			else
			{
				UE_LOGF(LogEditor, Warning, "Invalid entry in map ini list: %ls, %ls=%ls",
					*InSectionName, *(EntryType.ToString()), *EntryValue);
			}
		}
	}
}

void UEditorEngine::SyncBrowserToObjects( const TArray<UObject*>& InObjectsToSync, bool bFocusContentBrowser )
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets( InObjectsToSync, false, bFocusContentBrowser );

}

void UEditorEngine::SyncBrowserToObjects( const TArray<struct FAssetData>& InAssetsToSync, bool bFocusContentBrowser )
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets( InAssetsToSync, false, bFocusContentBrowser );
}

void UEditorEngine::SyncBrowserToObject( const UObject* InObjectToSync, bool bFocusContentBrowser )
{
	SyncBrowserToObjects({ InObjectToSync }, bFocusContentBrowser);
}

void UEditorEngine::SyncBrowserToObject( const FAssetData& InAssetToSync, bool bFocusContentBrowser )
{
	SyncBrowserToObjects({ InAssetToSync }, bFocusContentBrowser);
}


bool UEditorEngine::CanSyncToContentBrowser()
{
	TArray<FAssetData> Assets;
	GetAssetsToSyncToContentBrowser(Assets);
	return Assets.Num() > 0;
}

void UEditorEngine::GetAssetsToSyncToContentBrowser(TArray<FAssetData>& Assets, bool bAllowBrowseToAssetOverride)
{
	// If the user has any BSP surfaces selected, sync to the materials on them.
	bool bFoundSurfaceMaterial = false;

	for (TSelectedSurfaceIterator<> It(GWorld); It; ++It)
	{
		FBspSurf* Surf = *It;
		UMaterialInterface* Material = Surf->Material;
		if (Material)
		{
			Assets.AddUnique(FAssetData(Material));
			bFoundSurfaceMaterial = true;
		}
	}

	// Otherwise, assemble a list of resources from selected actors.
	if (!bFoundSurfaceMaterial)
	{
		if (UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
		{
			UBrowseToAssetOverrideSubsystem* BrowseToAssetOverrideSubsystem = UBrowseToAssetOverrideSubsystem::Get();
			SelectionSet->ForEachSelectedElementHandle([&Assets, bAllowBrowseToAssetOverride, BrowseToAssetOverrideSubsystem](const FTypedElementHandle& SelectedHandle)
			{
				if (bAllowBrowseToAssetOverride)
				{
					if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(SelectedHandle))
					{
						if (AActor* Actor = ObjectInterface.GetObjectAs<AActor>())
						{
							// If BrowseToAssetOverride is set, then use the asset it points to instead of the selected asset
							FAssetData AssetData;
							if (BrowseToAssetOverrideSubsystem->TryGetBrowseToAssetOverride(Actor, AssetData))
							{
								Assets.Add(MoveTemp(AssetData));
								return true;
							}
						}
					}
				}

				if (TTypedElement<ITypedElementAssetDataInterface> AssetDataInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementAssetDataInterface>(SelectedHandle))
				{
					Assets.Append(AssetDataInterface.GetAllReferencedAssetDatas(FTypedElementAssetDataReferencedOptions().SetOnlyTopLevelAsset(true)));
				}

				return true;
			});

			// Only attempt to select the world if no elements are selected (a light might be selected, but it won't produce AssetData)
			if (!SelectionSet->HasSelectedElements() && Assets.IsEmpty())
			{
				// If nothing is selected, sync to the current level (if it's not temp/untitled)
				if (const UWorld* World = GEditor->GetEditorWorldContext().World())
				{
					if (const ULevel* Level = World->GetCurrentLevel();
						Level && !FPackageName::IsTempPackage(Level->GetPackage()->GetName()))
					{
						Assets.Add(FAssetData(Level));
					}
				}
			}
		}
	}
}

void UEditorEngine::SyncToContentBrowser(bool bAllowOverrideMetadata)
{
	TArray<FAssetData> Assets;
	GetAssetsToSyncToContentBrowser(Assets, bAllowOverrideMetadata);
	SyncBrowserToObjects(Assets);
}

void UEditorEngine::GetLevelsToSyncToContentBrowser(TArray<UObject*>& Objects)
{
	for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = CastChecked<AActor>(*It);
		ULevel* ActorLevel = Actor->GetLevel();
		if (ActorLevel)
		{
			// Get the outer World as this is the actual asset we need to find
			UObject* ActorWorld = ActorLevel->GetOuter();
			if (ActorWorld)
			{
				Objects.AddUnique(ActorWorld);
			}
		}
	}
}

void UEditorEngine::SyncActorLevelsToContentBrowser()
{
	TArray<UObject*> Objects;
	GetLevelsToSyncToContentBrowser(Objects);

	SyncBrowserToObjects(Objects);
}

bool UEditorEngine::CanSyncActorLevelsToContentBrowser()
{
	TArray<UObject*> Objects;
	GetLevelsToSyncToContentBrowser(Objects);

	return Objects.Num() > 0;
}

void UEditorEngine::GetReferencedAssetsForEditorSelection(TArray<UObject*>& Objects, const bool bIgnoreOtherAssetsIfBPReferenced)
{
	for ( TSelectedSurfaceIterator<> It(GWorld) ; It ; ++It )
	{
		FBspSurf* Surf = *It;
		UMaterialInterface* Material = Surf->Material;
		if( Material )
		{
			Objects.AddUnique( Material );
		}
	}

	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		TArray<UObject*> ActorObjects;
		Actor->GetReferencedContentObjects(ActorObjects);

		// If Blueprint assets should take precedence over any other referenced asset, check if there are any blueprints in this actor's list
		// and if so, add only those.
		if (bIgnoreOtherAssetsIfBPReferenced && ActorObjects.ContainsByPredicate([](UObject* Obj) { return Obj && Obj->IsA(UBlueprint::StaticClass()); }))
		{
			for (UObject* Object : ActorObjects)
			{
				if (Object && Object->IsA(UBlueprint::StaticClass()))
				{
					Objects.Add(Object);
				}
			}
		}
		else
		{
			Objects.Append(ActorObjects);
		}
	}
}

void UEditorEngine::GetSoftReferencedAssetsForEditorSelection(TArray<FSoftObjectPath>& SoftObjects)
{
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = static_cast<AActor*>( *It );
		checkSlow( Actor->IsA(AActor::StaticClass()) );

		Actor->GetSoftReferencedContentObjects(SoftObjects);
	}
}

// Note about the TEDS query getters: For now we lazily register the queries ourselves. However,
//  we may want to use queries in some shared library if the same ones are used elsewhere in the editor.
UE::Editor::DataStorage::QueryHandle UEditorEngine::GetSelectedMovementLockedItemsQuery() const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	// Lazy registration
	if (SelectedMovementLockedItemsQuery == InvalidQueryHandle)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (ensure(DataStorage))
		{
			SelectedMovementLockedItemsQuery = DataStorage->RegisterQuery(
				Select()
				.ReadOnly<FTypedElementWorldColumn>()
				.Where()
				.All<FTypedElementSelectionColumn, FTypedElementLockTransformTag>()
				.Compile());
		}
	}
	return SelectedMovementLockedItemsQuery;
}
UE::Editor::DataStorage::QueryHandle UEditorEngine::GetSelectedItemsQuery() const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	// Lazy registration
	if (SelectedItemsQuery == InvalidQueryHandle)
	{
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (ensure(DataStorage))
		{
			SelectedItemsQuery = DataStorage->RegisterQuery(
				Select()
				.ReadOnly<FTypedElementWorldColumn>()
				.Where()
				.All<FTypedElementSelectionColumn>()
				.Compile());
		}
	}
	return SelectedItemsQuery;
}
void UEditorEngine::UnregisterTEDSQueries()
{
	using namespace UE::Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		if (SelectedMovementLockedItemsQuery != UE::Editor::DataStorage::InvalidQueryHandle)
		{
			DataStorage->UnregisterQuery(SelectedMovementLockedItemsQuery);
			SelectedMovementLockedItemsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
		}
		if (SelectedItemsQuery != UE::Editor::DataStorage::InvalidQueryHandle)
		{
			DataStorage->UnregisterQuery(SelectedItemsQuery);
			SelectedItemsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
		}
	}
}

void UEditorEngine::ToggleSelectedActorMovementLock()
{
	// First figure out if any selected actor is already locked.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bFoundLockedActor = HasLockedActors();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Fires ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied		LevelDirtyCallback;

	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = Cast<AActor>( *It );
		checkSlow( Actor );

		Actor->Modify();

		// If nothing is locked then we'll turn on locked for all selected actors
		// Otherwise, we'll turn off locking for any actors that are locked
		Actor->SetLockLocation(!bFoundLockedActor);

		LevelDirtyCallback.Request();
	}

	// Update the editability status in the active viewport, which will update the gizmos
	if (GCurrentLevelEditingViewportClient)
	{
		constexpr bool bForceCachedElementRefresh = true;
		GCurrentLevelEditingViewportClient->GetElementsToManipulate(bForceCachedElementRefresh);
	}
	RedrawLevelEditingViewports(false);

	bCheckForMovementLockedItems = true;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCheckForLockActors = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UEditorEngine::HasLockedActors()
{
	// We don't replace the bHasLockedActors usage here with bHasSelectedMovementLockedItems
	//  because we don't want users of the deprecated function to incorrectly set the state
	//  of the new one with an actor-only result.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bCheckForLockActors)
	{
		bHasLockedActors = false;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = Cast<AActor>( *It );
			checkSlow( Actor );

			if( Actor->IsLockLocation() )
			{
				bHasLockedActors = true;
				break;
			}
		}
		bCheckForLockActors = false;
	}

	return bHasLockedActors;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UEditorEngine::HasSelectedMovementLockedItems()
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	if (!bCheckForMovementLockedItems)
	{
		return bHasSelectedMovementLockedItems;
	}
	ON_SCOPE_EXIT{ bCheckForMovementLockedItems = false; };

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!DataStorage)
	{
		// Legacy fallback if TEDS is not enabled
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bCheckForLockActors = true;
		bHasSelectedMovementLockedItems = HasLockedActors();
		return bHasSelectedMovementLockedItems;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bHasSelectedMovementLockedItems = false;
	DataStorage->RunQuery(GetSelectedMovementLockedItemsQuery(), CreateDirectQueryCallbackBinding(
		[this, LevelEditorWorld = GetEditorWorldContext().World()](RowHandle Row, const FTypedElementWorldColumn& WorldColumn)
		{
			if (WorldColumn.World.Get() == LevelEditorWorld)
			{
				bHasSelectedMovementLockedItems = true;
			}
		}));

	return bHasSelectedMovementLockedItems;
}

void UEditorEngine::ToggleSelectedItemMovementLock()
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!DataStorage)
	{
		// Legacy fallback if TEDS is not enabled
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ToggleSelectedActorMovementLock();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return;
	}

	// Gather selected rows to toggle
	TArray<RowHandle> RowsToToggle;
	DataStorage->RunQuery(GetSelectedItemsQuery(), CreateDirectQueryCallbackBinding(
		[&RowsToToggle, LevelEditorWorld = GetEditorWorldContext().World()](RowHandle Row, const FTypedElementWorldColumn& WorldColumn)
		{
			if (WorldColumn.World.Get() == LevelEditorWorld)
			{
				RowsToToggle.Add(Row);
			}
		}));

	// Could have gotten this while gathering the rows, but clearer this way
	const bool bShouldLock = !HasSelectedMovementLockedItems();

	FScopedLevelDirtied LevelDirtyCallback;

	for (RowHandle Row : RowsToToggle)
	{
		if (bShouldLock)
		{
			DataStorage->AddColumn<FTypedElementLockTransformTag>(Row);
		}
		else
		{
			DataStorage->RemoveColumn<FTypedElementLockTransformTag>(Row);
		}
		DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(Row);
		
		LevelDirtyCallback.Request();
	}

	// Defer viewport and gizmo update until later in the tick, when TEDS processors have had a
	//  chance to deal with the column additions/removals. Else the GetElementsToManipulate
	//  call for actors may get unupdated lock state.
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = TWeakObjectPtr<UEditorEngine>(this)]()
	{
		if (!WeakThis.IsValid())
		{
			return;
		}
		// Update viewport and gizmos (modeled on ToggleSelectedActorMovementLock)
		if (GCurrentLevelEditingViewportClient)
		{
			constexpr bool bForceCachedElementRefresh = true;
			GCurrentLevelEditingViewportClient->GetElementsToManipulate(bForceCachedElementRefresh);
		}
		WeakThis->RedrawLevelEditingViewports(false);
	});

	bCheckForMovementLockedItems = true;
}

void UEditorEngine::EditObject( UObject* ObjectToEdit )
{
	// @todo toolkit minor: Needs world-centric support?
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectToEdit);
}

void UEditorEngine::SelectLevelInLevelBrowser( bool bDeselectOthers )
{
	if( bDeselectOthers )
	{
		AActor* Actor = Cast<AActor>( *FSelectionIterator(*GetSelectedActors()) );
		if(Actor)
		{
			TArray<class ULevel*> EmptyLevelsList;
			Actor->GetWorld()->SetSelectedLevels(EmptyLevelsList);
		}
	}

	for ( FSelectionIterator Itor(*GetSelectedActors()) ; Itor ; ++Itor )
	{
		AActor* Actor = Cast<AActor>( *Itor);
		if ( Actor )
		{
			Actor->GetWorld()->SelectLevel( Actor->GetLevel() );
		}
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.SummonWorldBrowserHierarchy();
}

void UEditorEngine::DeselectLevelInLevelBrowser()
{
	for ( FSelectionIterator Itor(*GetSelectedActors()) ; Itor ; ++Itor )
	{
		AActor* Actor = Cast<AActor>( *Itor);
		if ( Actor )
		{
			Actor->GetWorld()->DeSelectLevel( Actor->GetLevel() );
		}
	}
	
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.SummonWorldBrowserHierarchy();
}

void UEditorEngine::SelectAllActorsWithClass( bool bArchetype )
{
	if( !bArchetype )
	{
		TArray<UClass*> SelectedClasses;
		for (auto It = GetSelectedActorIterator(); It; ++It)
		{
			SelectedClasses.AddUnique(It->GetClass());
		}

		UWorld* CurrentEditorWorld = GetEditorWorldContext().World();
		for (UClass* Class : SelectedClasses)
		{
			Exec(CurrentEditorWorld, *FString::Printf(TEXT("ACTOR SELECT OFCLASS CLASS=%s"), *Class->GetPathName()));
		}
	}
	else
	{
		// For this function to have been called in the first place, all of the selected actors should be of the same type
		// and with the same archetype; however, it's safest to confirm the assumption first
		bool bAllSameClassAndArchetype = false;
		TSubclassOf<AActor> FirstClass;
		UObject* FirstArchetype = NULL;

		// Find the class and archetype of the first selected actor; they will be used to check that all selected actors
		// share the same class and archetype
		UWorld* IteratorWorld = GWorld;
		FSelectedActorIterator SelectedActorIter(IteratorWorld);
		if ( SelectedActorIter )
		{
			AActor* FirstActor = *SelectedActorIter;
			check( FirstActor );
			FirstClass = FirstActor->GetClass();
			FirstArchetype = FirstActor->GetArchetype();

			// If the archetype of the first actor is NULL, then do not allow the selection to proceed
			bAllSameClassAndArchetype = FirstArchetype ? true : false;

			// Increment the iterator so the search begins on the second selected actor
			++SelectedActorIter;
		}
		// Check all the other selected actors
		for ( ; SelectedActorIter && bAllSameClassAndArchetype; ++SelectedActorIter )
		{
			AActor* CurActor = *SelectedActorIter;
			if ( CurActor->GetClass() != FirstClass || CurActor->GetArchetype() != FirstArchetype )
			{
				bAllSameClassAndArchetype = false;
				break;
			}
		}

		// If all the selected actors have the same class and archetype, then go ahead and select all other actors
		// matching the same class and archetype
		if ( GUnrealEd && bAllSameClassAndArchetype )
		{
			FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectOfClassAndArchetype", "Select of Class and Archetype") );
			GUnrealEd->edactSelectOfClassAndArchetype( IteratorWorld, FirstClass, FirstArchetype );
		}
	}
}


void UEditorEngine::FindSelectedActorsInLevelScript()
{
	AActor* Actor = GetSelectedActors()->GetTop<AActor>();
	if(Actor != NULL)
	{
		if (PlayWorld)
		{
			// Redirect to editor world counterpart if PIE is active. We don't index cloned PIE LSBPs for searching.
			AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(Actor);
			if (EditorActor)
			{
				Actor = EditorActor;
			}
		}

		FKismetEditorUtilities::ShowActorReferencesInLevelScript(Actor);
	}
}

bool UEditorEngine::AreAnySelectedActorsInLevelScript()
{
	AActor* Actor = GetSelectedActors()->GetTop<AActor>();
	if(Actor != NULL)
	{
		ULevelScriptBlueprint* LSB = Actor->GetLevel()->GetLevelScriptBlueprint(true);
		if( LSB != NULL )
		{
			TArray<UK2Node*> ReferencedToActors;
			if(FBlueprintEditorUtils::FindReferencesToActorFromLevelScript(LSB, Actor, ReferencedToActors))
			{
				return true;
			}
		}
	}

	return false;
}

void UEditorEngine::ConvertSelectedBrushesToVolumes( UClass* VolumeClass )
{
	UEditorActorSubsystem::ConvertSelectedBrushesToVolumes(VolumeClass);
}

void UEditorEngine::ConvertActorsFromClass( UClass* FromClass, UClass* ToClass )
{
	UEditorActorSubsystem::ConvertActorsFromClass(FromClass, ToClass);
}

void UEditorEngine::BuildReflectionCaptures(UWorld* World)
{
	// Note: Lighting and reflection build operations should only dirty BuildData packages, not ULevel packages

	FText StatusText = FText(LOCTEXT("BuildReflectionCaptures", "Building Reflection Captures..."));
	GWarn->BeginSlowTask(StatusText, true);
	GWarn->StatusUpdate(0, 1, StatusText);

	// Wait for shader compiling to finish so we don't capture the default material
	if (GShaderCompilingManager != nullptr)
	{
		UMaterialInterface::SubmitRemainingJobsForWorld(World);
		FAssetCompilingManager::Get().FinishAllCompilation();
	}

	// Process any outstanding captures before we start operating on scenarios
	UReflectionCaptureComponent::UpdateReflectionCaptureContents(World);

	// Only the cubemap array path supports reading back from the GPU
	// Calling code should not allow building reflection captures on lower feature levels
	check(World->GetFeatureLevel() >= ERHIFeatureLevel::SM5);

	// Only reset reflection captures if we had hit an OOM condition, and there is a chance we could fit in memory if we rebuild from scratch
	const bool bOnlyIfOOM = true;
	World->Scene->ResetReflectionCaptures(bOnlyIfOOM);

	// Update sky light first because it's considered direct lighting, sky diffuse will be visible in reflection capture indirect specular
	World->UpdateAllSkyCaptures();

	TArray<ULevel*> LightingScenarios;
	
	// The list of scene capture component from hidden levels to keep.
	TSet<FGuid> ResourcesToKeep; 
	for (ULevel* Level : World->GetLevels())
	{
		check(Level);
		// If the level is hidden and not a lighting scenario, scene capture data from this level should be preserved.
		if (!Level->bIsVisible && !Level->bIsLightingScenario)
		{
			for (const AActor* Actor : Level->Actors)
			{
				if (Actor)
				{
					for (const UActorComponent* Component : Actor->GetComponents())
					{
						const UReflectionCaptureComponent* ReflectionCaptureComponent = Cast<UReflectionCaptureComponent>(Component);
						if (ReflectionCaptureComponent)
						{
							ResourcesToKeep.Add(ReflectionCaptureComponent->MapBuildDataId);
						}
					}
				}
			}
		}
	}

	{
		FGlobalComponentRecreateRenderStateContext Context;

		for (ULevel* Level : World->GetLevels())
		{
			if (Level->bIsVisible)
			{
				if (Level->MapBuildData)
				{
					// Remove all existing reflection capture data from visible levels before the build
					Level->MapBuildData->InvalidateReflectionCaptures(Level->bIsLightingScenario ? &ResourcesToKeep : nullptr);
				}

				if (Level->bIsLightingScenario)
				{
					LightingScenarios.Add(Level);
				}
			}
		}
	}

	if (LightingScenarios.Num() == 0)
	{
		// No lighting scenario levels present, add a null entry to represent the default case
		LightingScenarios.Add(nullptr);
		ResourcesToKeep.Empty();
	}

	// All but the first scenario start hidden
	for (int32 LevelIndex = 1; LevelIndex < LightingScenarios.Num(); LevelIndex++)
	{
		ULevel* LightingScenario = LightingScenarios[LevelIndex];

		if (LightingScenario)
		{
			EditorLevelUtils::SetLevelVisibilityTemporarily(LightingScenario, false);
		}
	}

	for (int32 LevelIndex = 0; LevelIndex < LightingScenarios.Num(); LevelIndex++)
	{
		ULevel* LightingScenario = LightingScenarios[LevelIndex];

		if (LightingScenario && LevelIndex > 0)
		{
			// Set current scenario visible
			EditorLevelUtils::SetLevelVisibilityTemporarily(LightingScenario, true);	
		}

		TArray<UReflectionCaptureComponent*> ReflectionCapturesToBuild;

		for (TObjectIterator<UReflectionCaptureComponent> It; It; ++It)
		{
			UReflectionCaptureComponent* CaptureComponent = *It;

			if (CaptureComponent->GetOwner()
				&& World->ContainsActor(CaptureComponent->GetOwner()) 
				&& !CaptureComponent->GetOwner()->bHiddenEdLevel
				&& IsValidChecked(CaptureComponent)
				&& !ResourcesToKeep.Contains(CaptureComponent->MapBuildDataId))
			{
				// Queue an update
				// Note InvalidateReflectionCaptures will guarantee this is a recapture, we don't want old data to persist
				// We cannot modify MapBuildDataId to force a recapture as that would modify CaptureComponent's package, build operations should only modify the BuildData package
				CaptureComponent->MarkDirtyForRecaptureOrUpload();	
				ReflectionCapturesToBuild.Add(CaptureComponent);
			}
		}

		FString UpdateReason = LightingScenario ? LightingScenario->GetOuter()->GetName() : TEXT("all levels");

		// Passing in flag to verify all recaptures, no uploads
		const bool bVerifyOnlyCapturing = true;
		
		// First capture data we will use to generate endcoded data for a mobile renderer
		bool bCapturingForMobile = true;
		TArray<TArray<uint8>> EncodedCaptures;
		EncodedCaptures.AddDefaulted(ReflectionCapturesToBuild.Num());
		{
			UReflectionCaptureComponent::UpdateReflectionCaptureContents(World, *UpdateReason, bVerifyOnlyCapturing, bCapturingForMobile);
			for (int32 CaptureIndex = 0; CaptureIndex < ReflectionCapturesToBuild.Num(); CaptureIndex++)
			{ 
				UReflectionCaptureComponent* CaptureComponent = ReflectionCapturesToBuild[CaptureIndex];
				FReflectionCaptureData ReadbackCaptureData;
				World->Scene->GetReflectionCaptureData(CaptureComponent, ReadbackCaptureData);
				// Capture can fail if there are more than GMaxNumReflectionCaptures captures
				if (ReadbackCaptureData.CubemapSize > 0)
				{
					// Capture should also fail if memory limitations prevent full resolution captures from being generated.
					// We report an error message for this case below in the non-mobile capture code path (no need for two errors).
					int32 DesiredCaptureSize = UReflectionCaptureComponent::GetReflectionCaptureSize();
					if (ReadbackCaptureData.CubemapSize == DesiredCaptureSize)
					{
						ULevel* StorageLevel = LightingScenarios[LevelIndex] ? LightingScenarios[LevelIndex] : CaptureComponent->GetOwner()->GetLevel();
						UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();

						GenerateEncodedHDRData(ReadbackCaptureData.FullHDRCapturedData, ReadbackCaptureData.CubemapSize, EncodedCaptures[CaptureIndex]);
					}
				}
			}
		}

		// Capture reflection data for a general use
		bCapturingForMobile = false;
		for (UReflectionCaptureComponent* CaptureComponent : ReflectionCapturesToBuild)
		{
			CaptureComponent->MarkDirtyForRecaptureOrUpload();	
		}
		UReflectionCaptureComponent::UpdateReflectionCaptureContents(World, *UpdateReason, bVerifyOnlyCapturing, bCapturingForMobile);
		for (int32 CaptureIndex = 0; CaptureIndex < ReflectionCapturesToBuild.Num(); CaptureIndex++)
		{ 
			UReflectionCaptureComponent* CaptureComponent = ReflectionCapturesToBuild[CaptureIndex];
			FReflectionCaptureData ReadbackCaptureData;
			World->Scene->GetReflectionCaptureData(CaptureComponent, ReadbackCaptureData);

			// Capture can fail if there are more than GMaxNumReflectionCaptures captures
			if (ReadbackCaptureData.CubemapSize > 0)
			{
				// Capture should also fail if memory limitations prevent full resolution captures from being generated.
				int32 DesiredCaptureSize = UReflectionCaptureComponent::GetReflectionCaptureSize();
				if (ReadbackCaptureData.CubemapSize == DesiredCaptureSize)
				{
					ULevel* StorageLevel = LightingScenarios[LevelIndex] ? LightingScenarios[LevelIndex] : CaptureComponent->GetOwner()->GetLevel();
					UMapBuildDataRegistry* Registry = StorageLevel->GetOrCreateMapBuildData();
					FReflectionCaptureMapBuildData& CaptureBuildData = Registry->AllocateReflectionCaptureBuildData(CaptureComponent->MapBuildDataId, true);
					(FReflectionCaptureData&)CaptureBuildData = ReadbackCaptureData;
					CaptureBuildData.EncodedHDRCapturedData = MoveTemp(EncodedCaptures[CaptureIndex]);
					CaptureBuildData.FinalizeLoad();
					// Recreate capture render state now that we have valid BuildData
					CaptureComponent->MarkRenderStateDirty();
				}
				else
				{
					UE_LOGF(LogEditor, Error, "Unable to build Reflection Capture %ls, requested reflection capture cube size of %d didn't fit in memory on host machine (size clamped to %d)",
						*CaptureComponent->GetPathName(), DesiredCaptureSize, ReadbackCaptureData.CubemapSize);
				}
			}
			else
			{
				UE_LOGF(LogEditor, Warning, "Unable to build Reflection Capture %ls, max number of reflection captures exceeded", *CaptureComponent->GetPathName());
			}
		}
		// Queue an update
		// Update sky light first because it's considered direct lighting, sky diffuse will be visible in reflection capture indirect specular
		if (LightingScenario)
		{
			// Hide current scenario now that we are done capturing it
			EditorLevelUtils::SetLevelVisibilityTemporarily(LightingScenario, false);	
		}
	}

		// Passing in flag to verify all recaptures, no uploads
	// Restore initial visibility
	for (int32 LevelIndex = 0; LevelIndex < LightingScenarios.Num(); LevelIndex++)
	{
		ULevel* LightingScenario = LightingScenarios[LevelIndex];

			// Recreate capture render state now that we have valid BuildData


		if (LightingScenario)
		{
			// Hide current scenario now that we are done capturing it

	// Restore initial visibility
			EditorLevelUtils::SetLevelVisibilityTemporarily(LightingScenario, true);
		}
	}

	GWarn->EndSlowTask();
}

void UEditorEngine::EditorAddModalWindow( TSharedRef<SWindow> InModalWindow ) const
{
	// If there is already a window active, parent this new modal window to the existing window so that it doesnt fall behind
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();

	if( !ParentWindow.IsValid() )
	{
		// Parent to the main frame window
		if(FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}
	}

	FSlateApplication::Get().AddModalWindow( InModalWindow, ParentWindow );
}

UBrushBuilder* UEditorEngine::FindBrushBuilder( UClass* BrushBuilderClass )
{
	TArray< UBrushBuilder* > FoundBuilders;
	UBrushBuilder* Builder = NULL;
	// Search for the existing brush builder
	if( ContainsObjectOfClass<UBrushBuilder>( BrushBuilders, BrushBuilderClass, EFindObjectFlags::ExactClass, &FoundBuilders ) )
	{
		// Should not be more than one of the same type
		check( FoundBuilders.Num() == 1 );
		Builder = FoundBuilders[0];
	}
	else
	{
		// An existing builder does not exist so create one now
		Builder = NewObject<UBrushBuilder>(GetTransientPackage(), BrushBuilderClass);
		BrushBuilders.Add( Builder );
	}

	return Builder;
}

void UEditorEngine::ParentActors( AActor* ParentActor, AActor* ChildActor, const FName SocketName, USceneComponent* Component)
{
	if (CanParentActors(ParentActor, ChildActor))
	{
		USceneComponent* ChildRoot = ChildActor->GetRootComponent();
		USceneComponent* ParentRoot = ParentActor->GetDefaultAttachComponent();

		check(ChildRoot);	// CanParentActors() call should ensure this
		check(ParentRoot);	// CanParentActors() call should ensure this

		// modify parent and child
		const FScopedTransaction Transaction( NSLOCTEXT("Editor", "UndoAction_PerformAttachment", "Attach actors") );
		// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
		ChildActor->Modify();
		ParentActor->Modify(/*bAlwaysMarkDirty=*/false);

		// If child is already attached to something, modify the old parent and detach
		if(ChildRoot->GetAttachParent() != nullptr)
		{
			AActor* OldParentActor = ChildRoot->GetAttachParent()->GetOwner();
			OldParentActor->Modify(/*bAlwaysMarkDirty=*/false);
			ChildRoot->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

			GEngine->BroadcastLevelActorDetached(ChildActor, OldParentActor);
		}

		// If the parent is already attached to this child, modify its parent and detach so we can allow the attachment
		if(ParentRoot->IsAttachedTo(ChildRoot))
		{
			// Here its ok to mark the parent package dirty as both Parent & Child need to be saved.
			ParentRoot->GetAttachParent()->GetOwner()->Modify();
			ParentRoot->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		}

		// Snap to socket if a valid socket name was provided, otherwise attach without changing the relative transform
		ChildRoot->AttachToComponent(Component ? Component : ParentRoot, FAttachmentTransformRules::KeepWorldTransform, SocketName);

		// Update recursively attached actor folder using top most parent folder
		const AActor* TopMostAttachParentActor = ActorAttachmentHelper::GetTopMostAttachParentActor(ParentActor);
		ChildActor->SetFolderPath_Recursively(TopMostAttachParentActor->GetFolderPath());

		// Refresh editor in case child was translated after snapping to socket
		RedrawLevelEditingViewports();
	}
}

bool UEditorEngine::DetachSelectedActors()
{
	FScopedTransaction Transaction( NSLOCTEXT("Editor", "UndoAction_PerformDetach", "Detach actors") );

	bool bDetachOccurred = false;
	for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
	{
		AActor* Actor = Cast<AActor>( *It );
		checkSlow( Actor );

		USceneComponent* RootComp = Actor->GetRootComponent();
		if( RootComp != nullptr && RootComp->GetAttachParent() != nullptr)
		{
			AActor* OldParentActor = RootComp->GetAttachParent()->GetOwner();
			OldParentActor->Modify(false);
			RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
			bDetachOccurred = true;

			// Update recursively detached actor folder using top most parent folder
			const AActor* TopMostAttachParentActor = ActorAttachmentHelper::GetTopMostAttachParentActor(OldParentActor);
			Actor->SetFolderPath_Recursively(TopMostAttachParentActor->GetFolderPath());
		}
	}
	return bDetachOccurred;
}

bool UEditorEngine::CanParentActors( const AActor* ParentActor, const AActor* ChildActor, FText* ReasonText)
{
	if(ChildActor == NULL || ParentActor == NULL)
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "Null_ActorAttachmentError", "Cannot attach NULL actors.");
		}
		return false;
	}

	if(ChildActor == ParentActor)
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "Self_ActorAttachmentError", "Cannot attach actor to self.");
		}
		return false;
	}

	USceneComponent* ChildRoot = ChildActor->GetRootComponent();
	USceneComponent* ParentRoot = ParentActor->GetDefaultAttachComponent();
	if(ChildRoot == NULL || ParentRoot == NULL)
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "MissingComponent_ActorAttachmentError", "Cannot attach actors without root components.");
		}
		return false;
	}

	const ABrush* ParentBrush = Cast<const ABrush>( ParentActor );
	const ABrush* ChildBrush = Cast<const ABrush>( ChildActor );
	if( (ParentBrush && !ParentBrush->IsVolumeBrush() ) || ( ChildBrush && !ChildBrush->IsVolumeBrush() ) )
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "Brush_ActorAttachmentError", "BSP Brushes cannot be attached");
		}
		return false;
	}

	{
		FText Reason;
		if (!ParentActor->EditorCanAttachFrom(ChildActor, Reason) || !ChildActor->EditorCanAttachTo(ParentActor, Reason))
		{
			if (ReasonText)
			{
				if (Reason.IsEmpty())
				{
					*ReasonText = FText::Format(NSLOCTEXT("ActorAttachmentError", "CannotBeAttached_ActorAttachmentError", "{0} cannot be attached to {1}"), FText::FromString(ChildActor->GetActorLabel()), FText::FromString(ParentActor->GetActorLabel()));
				}
				else
				{
					*ReasonText = MoveTemp(Reason);
				}
			}
			return false;
		}
	}

	if (ChildRoot->Mobility == EComponentMobility::Static && ParentRoot->Mobility != EComponentMobility::Static)
	{
		if (ReasonText)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("StaticActor"), FText::FromString(ChildActor->GetActorLabel()));
			Arguments.Add(TEXT("DynamicActor"), FText::FromString(ParentActor->GetActorLabel()));
			
			Arguments.Add(TEXT("DynamicActorMobility"),
				ParentRoot->Mobility == EComponentMobility::Stationary
				? NSLOCTEXT("ActorAttachmentError", "StationaryMobility", "Stationary")
				: NSLOCTEXT("ActorAttachmentError", "MovableMobility", "Movable"));
				
			*ReasonText = FText::Format( NSLOCTEXT("ActorAttachmentError", "StaticDynamic_ActorAttachmentError", "Cannot attach actor with Static mobility ({StaticActor}) to actor with {DynamicActorMobility} mobility ({DynamicActor})."), Arguments);
		}
		return false;
	}

	if(ChildActor->GetLevel() != ParentActor->GetLevel())
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "WrongLevel_AttachmentError", "Actors need to be in the same level!");
		}
		return false;
	}

	if (ChildActor->GetContentBundleGuid() != ParentActor->GetContentBundleGuid())
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "WrongContentBundle_AttachmentError", "Actors need to be in the same content bundle!");
		}
		return false;
	}

	if (ChildActor->GetExternalDataLayerAsset() != ParentActor->GetExternalDataLayerAsset())
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "WrongExternalDataLayer_AttachmentError", "Actors need to be in the same external data layer");
		}
		return false;
	}

	if(ParentRoot->IsAttachedTo( ChildRoot ))
	{
		if (ReasonText)
		{
			*ReasonText = NSLOCTEXT("ActorAttachmentError", "CircularInsest_ActorAttachmentError", "Parent cannot become the child of their descendant");
		}
		return false;
	}

	return true;
}


bool UEditorEngine::IsPackageValidForAutoAdding(UPackage* InPackage, const FString& InFilename)
{
	bool bPackageIsValid = false;

	// Ensure the package exists, the user is running the editor (and not a commandlet or cooking), and that source control
	// is enabled and expecting new files to be auto-added before attempting to test the validity of the package
	if (InPackage && GIsEditor && !IsRunningCommandlet() 
		&& (ISourceControlModule::Get().IsEnabled() || FUncontrolledChangelistsModule::Get().IsEnabled())
		&& GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles)
	{
		const FString CleanFilename = FPaths::GetCleanFilename(InFilename);

		// Determine if the package has been saved before or not; if it has, it's not valid for auto-adding
		bPackageIsValid = !FPaths::FileExists(InFilename);

		// If the package is still considered valid up to this point, ensure that it is not a script or PIE package
		// and that the editor is not auto-saving.
		if ( bPackageIsValid )
		{
			const bool bIsPIEOrScriptPackage = InPackage->RootPackageHasAnyFlags( PKG_ContainsScript | PKG_PlayInEditor );
			const bool bIsAutosave = GUnrealEd && GUnrealEd->GetPackageAutoSaver().IsAutoSaving();

			if ( bIsPIEOrScriptPackage || bIsAutosave || GIsAutomationTesting )
			{
				bPackageIsValid = false;
			}
		}
	}
	return bPackageIsValid;
}

bool UEditorEngine::IsPackageOKToSave(UPackage* InPackage, const FString& InFilename, FOutputDevice* Error)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	if (InPackage && !AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(InPackage->GetName()))
	{
		return false;
	}

	return true;
}

void UEditorEngine::AddDeferredMarkForAddFiles(TArrayView<const FString> Files)
{
	DeferredFilesToAddToSourceControl.Append(Files);
}

void UEditorEngine::RunDeferredMarkForAddFiles(bool)
{
	if (DeferredFilesToAddToSourceControl.IsEmpty())
	{
		return;
	}

	bool bAddToUncontrolled = true;

	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if(SourceControlProvider.IsAvailable())
		{
			ECommandResult::Type Result = SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SourceControlHelpers::PackageFilenames(DeferredFilesToAddToSourceControl));

			bAddToUncontrolled = (Result == ECommandResult::Failed);

			if (Result == ECommandResult::Failed && !SourceControlFailureNotificationPtr.Pin().IsValid())
			{
				FTextBuilder InfoTextBuilder;
				InfoTextBuilder.AppendLine(
					FText::Format(LOCTEXT("Error_MarkForAddFailed", "{0} Mark for Add failed."),
					FText::FromName(SourceControlProvider.GetName())));

				if (FUncontrolledChangelistsModule::Get().IsEnabled())
				{
					InfoTextBuilder.AppendLine(); // Empty line
					InfoTextBuilder.AppendLine(LOCTEXT("Error_MarkForAddFailed_Uncontrolled", "Adding files to the Uncontrolled changelist instead."));
				}

				FNotificationInfo Info(InfoTextBuilder.ToText());

				Info.bFireAndForget = true;
				Info.bUseSuccessFailIcons = true;

				if (SourceControlProvider.UsesChangelists() || SourceControlProvider.UsesUncontrolledChangelists())
				{
					Info.Hyperlink = FSimpleDelegate::CreateLambda([]() { ISourceControlWindowsModule::Get().ShowChangelistsTab(); });
					Info.HyperlinkText = LOCTEXT("ViewChangesHyperlink", "View Changes");
				}

				Info.ButtonDetails.Emplace(
					LOCTEXT("Dismiss", "Dismiss"),
					LOCTEXT("DismissTooltip", "Dismiss this notification"),
					FSimpleDelegate::CreateUObject(this, &UEditorEngine::OnDismissSourceControlFailureNotification),
					SNotificationItem::CS_Fail);

				if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info))
				{
					Notification->SetExpireDuration(30.f);
					Notification->SetCompletionState(SNotificationItem::CS_Fail);
					SourceControlFailureNotificationPtr = Notification;
				}
			}
		}
	}

	if (FUncontrolledChangelistsModule::Get().IsEnabled() && bAddToUncontrolled)
	{
		FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();
		UncontrolledChangelistsModule.OnNewFilesAdded(DeferredFilesToAddToSourceControl);
	}

	// Clear the list when this run whether source control is active or not, since we do not want to accumulate those if the user is running without source control
	DeferredFilesToAddToSourceControl.Empty();
}

void UEditorEngine::OnDismissSourceControlFailureNotification()
{
	if (TSharedPtr<SNotificationItem> Notification = SourceControlFailureNotificationPtr.Pin())
	{
		Notification->SetExpireDuration(0.f);
		Notification->ExpireAndFadeout();
		SourceControlFailureNotificationPtr.Reset();
	}
}

struct UEditorEngine::FWorldScopedPackageDirtyGuard final
{
	explicit FWorldScopedPackageDirtyGuard(const UWorld* InWorld)
		: World(InWorld)
	{
		const UPackage* const WorldPackage = World->GetPackage();

		bWorldPackageDirty = WorldPackage->IsDirty();

		for (const UPackage* ExternalPackage : WorldPackage->GetExternalPackages())
		{
			if (ExternalPackage->IsDirty())
			{
				DirtyExternalPackages.Add(ExternalPackage);
			}
		}
	}

	~FWorldScopedPackageDirtyGuard()
	{
		UPackage* const WorldPackage = World->GetPackage();

		for (UPackage* ExternalPackage : WorldPackage->GetExternalPackages())
		{
			if (ExternalPackage->IsDirty() && !DirtyExternalPackages.Contains(ExternalPackage))
			{
				ExternalPackage->SetDirtyFlag(false);
			}
		}

		if (WorldPackage->IsDirty() && !bWorldPackageDirty)
		{
			WorldPackage->SetDirtyFlag(false);
		}
	}

private:
	const UWorld* const World;
	bool bWorldPackageDirty;
	TSet<const UPackage*> DirtyExternalPackages;
};

bool UEditorEngine::InitializePhysicsSceneForSaveIfNecessary(UWorld* World, bool &bOutForceInitialized)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::InitializePhysicsSceneForSaveIfNecessary);

	// We need a physics scene at save time in case code does traces during onsave events.
	bool bHasPhysicsScene = false;

	// First check if our owning world has a physics scene
	if (World->PersistentLevel && World->PersistentLevel->OwningWorld)
	{
		bHasPhysicsScene = (World->PersistentLevel->OwningWorld->GetPhysicsScene() != nullptr);
	}

	// If we didn't already find a physics scene in our owning world, maybe we personally have our own.
	if (!bHasPhysicsScene)
	{
		bHasPhysicsScene = (World->GetPhysicsScene() != nullptr);
	}


	// If we didn't find any physics scene we will synthesize one and remove it after save
	if (!bHasPhysicsScene)
	{
		// Capture Dirty packages so we can reset unwanted dirtyness 
		FWorldScopedPackageDirtyGuard WorldScopedPackageDirtyGuard(World);

		// Clear world components first so that UpdateWorldComponents below properly adds them all to the physics scene
		World->ClearWorldComponents();

		if (World->bIsWorldInitialized)
		{
			// If we don't have a physics scene and the world was initialized without one (i.e. an inactive world) then we should create one here. We will remove it down below after the save
			World->CreatePhysicsScene();

			// Keep track of the force initialization so we can use the proper cleanup
			bOutForceInitialized = false;
		}
		else
		{
			// We allow multiple reinit for the world in this case, so we need to reset the cleanup flag to avoid tripping the checks in IsCleanedUp
			World->ResetCleanedUpState();

			// If we aren't already initialized, initialize now and create a physics scene. Don't create an FX system because it uses too much video memory for bulk operations
			World->InitWorld(
				GetEditorWorldInitializationValues()
				.CreateFXSystem(false)
				.CreateAISystem(false)
				.CreateNavigation(false)
				.CreatePhysicsScene(true));
			bOutForceInitialized = true;
		}

		// Update components now that a physics scene exists.
		World->UpdateWorldComponents(true, true);

		// Set this to true so we can clean up what we just did down below
		return true;
	}

	return false;
}

void UEditorEngine::CleanupPhysicsSceneThatWasInitializedForSave(UWorld* World, bool bForceInitialized)
{
	// Capture Dirty packages so we can reset unwanted dirtyness 
	FWorldScopedPackageDirtyGuard WorldScopedPackageDirtyGuard(World);

	// Make sure we clean up the physics scene here. If we leave too many scenes in memory, undefined behavior occurs when locking a scene for read/write.
	World->ClearWorldComponents();

	if (bForceInitialized)
	{
		World->CleanupWorld(true, true, World);
	}

	World->SetPhysicsScene(nullptr);

	if (World->IsInitialized())
	{
		// Update components again in case it was a world without a physics scene but did have rendered components.
		World->UpdateWorldComponents(true, true);
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, EditorEngine_Save, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

struct FEditorSaveContext
{
	UPackage*                                  Package = nullptr;
	UObject*                                   Asset = nullptr;
	UWorld*                                    World = nullptr;
	UWorld*                                    OriginalOwningWorld = nullptr;
	FString                                    Filename;
	FSavePackageArgs                           SaveArgs;
	FObjectSaveContextData                     ObjectSaveContext;
	TUniquePtr<FSavePackageContext>            UniqueContext;
	FSavePackageResultStruct                   SavePackageResult;
	bool                                       bAutoAddPkgToSCC = false;
	bool                                       bInitializedPhysicsSceneForSave = false;
	bool                                       bForceInitializedWorld = false;
	UE::Tasks::TTask<FSavePackageResultStruct> InnerTask;
};

void UEditorEngine::BeginEditorSave(FEditorSaveContext& Context, FSlowTask* SlowTask)
{
	bool bIsCooking = Context.SaveArgs.ArchiveCookData != nullptr;
	Context.SaveArgs.TopLevelFlags = UE::SavePackageUtilities::NormalizeTopLevelFlags(Context.SaveArgs.TopLevelFlags, bIsCooking);

	if (!Context.Asset && Context.Package)
	{
		// Check if the package contains a map and set the world object as the asset
		if (Context.Package->HasAnyPackageFlags(PKG_ContainsMap))
		{
			Context.Asset = UWorld::FindWorldInPackage(Context.Package);
		}
		else
		{
			// Otherwise find the main asset of the package
			Context.Asset = Context.Package->FindAssetInPackage();
		}
	}

	// if no save package context was passed in and the default settings were modified, install a context for the save
	if (Context.SaveArgs.SavePackageContext == nullptr && !FSavePackageSettings::GetDefaultSettings().IsDefault())
	{
		Context.UniqueContext = MakeUnique<FSavePackageContext>(nullptr, nullptr, FSavePackageSettings::GetDefaultSettings());
		Context.SaveArgs.SavePackageContext = Context.UniqueContext.Get();
	}

	if (SlowTask)
	{
		SlowTask->EnterProgressFrame(10);
	}

	if (Context.SaveArgs.ArchiveCookData)
	{
		Context.ObjectSaveContext.CookType = Context.SaveArgs.ArchiveCookData->CookContext.GetCookType();
		Context.ObjectSaveContext.CookingDLC = Context.SaveArgs.ArchiveCookData->CookContext.GetCookingDLC();
		Context.ObjectSaveContext.CookInfo = Context.SaveArgs.ArchiveCookData->CookContext.GetCookInfo();
	}

	if (UWorld* World = Cast<UWorld>(Context.Asset))
	{
		Context.World = World;

		const bool bSavingConcurrent = !!(Context.SaveArgs.SaveFlags & ESaveFlags::SAVE_Concurrent);
		if (!bSavingConcurrent)
		{
			Context.bInitializedPhysicsSceneForSave = InitializePhysicsSceneForSaveIfNecessary(World, Context.bForceInitializedWorld);

			// bForceInitialized=true marks that We Called InitWorld and need to call CleanupWorld
			// but if we are saving during Load and the caller of LoadPackage requested that we keep it initialized, we should keep it initialized.
			if (Context.bForceInitializedWorld)
			{
				// TODO: If we ever add a way to set bInitialized=false on a UWorld, then in future saves after setting it back to false
				// we will still parse the KeepInitializedDuringLoadTag here and not CleanupWorld. If we set bInitialized=false on
				// a UWorld, we need to clear KeepInitializedDuringLoadTag from its InstancingContext.
				FLinkerLoad* LinkerLoad = World->GetPackage()->GetLinker();
				if (Context.bForceInitializedWorld && LinkerLoad && LinkerLoad->GetInstancingContext().HasTag(UWorld::KeepInitializedDuringLoadTag))
				{
					Context.bForceInitializedWorld = false;
				}
			}

			OnPreSaveWorld(World, FObjectPreSaveContext(Context.ObjectSaveContext));
		}

		Context.OriginalOwningWorld = World->PersistentLevel->OwningWorld;
		World->PersistentLevel->OwningWorld = World;
	}

	// See if the package is a valid candidate for being auto-added to the default changelist.
	// Only allows the addition of newly created packages while in the editor and then only if the user has the option enabled.
	Context.bAutoAddPkgToSCC = false;
	if (!Context.ObjectSaveContext.bProceduralSave)
	{
		Context.bAutoAddPkgToSCC = IsPackageValidForAutoAdding(Context.Package, Context.Filename);
	}

	if (SlowTask)
	{
		SlowTask->EnterProgressFrame(70);
	}

	UPackage::PreSavePackageWithContextEvent.Broadcast(Context.Package, FObjectPreSaveContext(Context.ObjectSaveContext));
}

void UEditorEngine::EndEditorSave(FEditorSaveContext& Context, FSlowTask* SlowTask)
{
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame(10);
	}

	Context.ObjectSaveContext.bSaveSucceeded = Context.SavePackageResult == ESavePackageResult::Success;

	// If the package is a valid candidate for being automatically-added to source control, go ahead and add it
	// to the default changelist
	if (Context.SavePackageResult == ESavePackageResult::Success && Context.bAutoAddPkgToSCC)
	{
		const bool bIsUncontrolledChangelistEnabled = FUncontrolledChangelistsModule::Get().IsEnabled();

		// IsPackageValidForAutoAdding should not return true if SCC is disabled
		check(ISourceControlModule::Get().IsEnabled() || bIsUncontrolledChangelistEnabled);

		if (!ISourceControlModule::Get().GetProvider().IsAvailable() && !bIsUncontrolledChangelistEnabled)
		{
			// Show the login window here & store the file we are trying to add.
			// We defer the add operation until we have a valid source control connection.
			ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed::CreateUObject(this, &UEditorEngine::RunDeferredMarkForAddFiles), ELoginWindowMode::Modeless);
		}
		DeferredFilesToAddToSourceControl.Add(Context.Filename);
	}

	if (SlowTask)
	{
		SlowTask->EnterProgressFrame(10);
	}

	if (Context.World)
	{
		if (Context.OriginalOwningWorld)
		{
			Context.World->PersistentLevel->OwningWorld = Context.OriginalOwningWorld;
		}

		const bool bSavingConcurrent = !!(Context.SaveArgs.SaveFlags & ESaveFlags::SAVE_Concurrent);
		if (!bSavingConcurrent)
		{
			OnPostSaveWorld(Context.World, FObjectPostSaveContext(Context.ObjectSaveContext));

			if (Context.bInitializedPhysicsSceneForSave)
			{
				CleanupPhysicsSceneThatWasInitializedForSave(Context.World, Context.bForceInitializedWorld);
			}

			if (Context.SavePackageResult == ESavePackageResult::Success) // Package saved successfully?
			{
				// Rerunning construction scripts may have made it dirty again
				Context.Package->SetDirtyFlag(false);
			}
		}
	}

	if (Context.ObjectSaveContext.bUpdatingLoadedPath)
	{
		// Notify the asset registry
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		AssetRegistry.AssetsSaved(MoveTemp(Context.SavePackageResult.SavedAssets));
	}
}

FSavePackageResultStruct UEditorEngine::Save(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
	const FSavePackageArgs& InSaveArgs)
{
	UE_TRACE_LOG_SCOPED_T(Cpu, EditorEngine_Save, CpuChannel)
		<< EditorEngine_Save.PackageName(InOuter ? *(InOuter->GetName()) : TEXT("None"));

	// Needs to be here in addition to UPackage::Save so that InitializePhysicsSceneForSaveIfNecessary and OnPreSaveWorld have appropriate referencing package info
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(InOuter, PackageAccessTrackingOps::NAME_Save); 

	FScopedSlowTask SlowTask(100, FText(), InSaveArgs.bSlowTask);

	FEditorSaveContext Context
	{
		.Package = InOuter,
		.Asset = InAsset,
		.Filename = Filename,
		.SaveArgs = InSaveArgs,
		.ObjectSaveContext = FObjectSaveContextData(InOuter, InSaveArgs.GetTargetPlatform(), Filename, InSaveArgs.SaveFlags),
	};

	BeginEditorSave(Context, &SlowTask);
	Context.SavePackageResult = UPackage::Save(Context.Package, Context.Asset, *Context.Filename, Context.SaveArgs);
	EndEditorSave(Context, &SlowTask);

	return MoveTemp(Context.SavePackageResult);
}

UE::Tasks::TTask<FSavePackageResultStruct> UEditorEngine::SaveAsync(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
		const FSavePackageArgs& InSaveArgs)
{
	FEditorSaveContext LocalContext
	{
		.Package = InOuter,
		.Asset = InAsset,
		.Filename = Filename,
		.SaveArgs = InSaveArgs,
		.ObjectSaveContext = FObjectSaveContextData(InOuter, InSaveArgs.GetTargetPlatform(), Filename, InSaveArgs.SaveFlags)
	};

	auto Context = MakeShared<FEditorSaveContext>(MoveTemp(LocalContext));

	UE::Tasks::FTaskEvent TaskEvent(TEXT("EditorSaveEvent"));
	auto BeginEditorSaveTask =
		UE::Tasks::Launch(
			TEXT("BeginEditorSave"),
			[this, Context, TaskEvent]() mutable
			{
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(Context->Package, PackageAccessTrackingOps::NAME_Save);

				BeginEditorSave(*Context, nullptr);

				Context->InnerTask = UPackage::SaveAsync(Context->Package, Context->Asset, *Context->Filename, Context->SaveArgs);

				// Once the result of the task is available, this will allow the endeditorsave to run
				TaskEvent.AddPrerequisites(Context->InnerTask);
				TaskEvent.Trigger();
			},
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadHiPri
		);

	auto EndEditorSaveTask =
		UE::Tasks::Launch(
			TEXT("EndEditorSave"),
			[this, Context]()
			{
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(Context->Package, PackageAccessTrackingOps::NAME_Save);
				Context->SavePackageResult = MoveTemp(Context->InnerTask.GetResult());
				EndEditorSave(*Context, nullptr);
				return MoveTemp(Context->SavePackageResult);
			},
			UE::Tasks::Prerequisites(BeginEditorSaveTask, TaskEvent),
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadHiPri
		);

	return EndEditorSaveTask;
}

bool UEditorEngine::SavePackage(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
	const FSavePackageArgs& SaveArgs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::SavePackage);

	// Workaround to avoid function signature change while keeping both bool and ESavePackageResult versions of SavePackage
	const FSavePackageResultStruct Result = Save(InOuter, InAsset, Filename, SaveArgs);
	return Result == ESavePackageResult::Success;
}

void UEditorEngine::OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsFirstConcurrentSave())
	{
		return;
	}
	if ( !ensure(World) )
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine_OnPreSaveWorld);

	check(World->PersistentLevel);

	// Pre save world event
	FEditorDelegates::PreSaveWorldWithContext.Broadcast(World, ObjectSaveContext);

	// Update cull distance volumes (and associated primitives).
	World->UpdateCullDistanceVolumes();

	const bool bAutosave = ObjectSaveContext.IsFromAutoSave();
	if (bAutosave)
	{
		// Temporarily flag packages saved under a PIE filename as PKG_PlayInEditor for serialization so loading
		// them will have the flag set. We need to undo this as the object flagged isn't actually the PIE package, 
		// but rather only the loaded one will be.
		// PIE prefix detected, mark package.
		if (World->GetName().StartsWith(PLAYWORLD_PACKAGE_PREFIX))
		{
			World->GetOutermost()->SetPackageFlags(PKG_PlayInEditor);
		}
	}
	else if ( !IsRunningCommandlet() && !ObjectSaveContext.IsProceduralSave())
	{
		// A user-initiated save in the editor
		FWorldContext &EditorContext = GetEditorWorldContext();

		// Check that this world is GWorld to avoid stomping on the saved views of sub-levels.
		if ( World == EditorContext.World() )
		{
			if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
			{
				FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

				// Notify slate level editors of the map change
				LevelEditor.BroadcastMapChanged( World, EMapChangeType::SaveMap );
			}
		}

		// Shrink model and clean up deleted actors.
		// Don't do this when autosaving or PIE saving so that actor adds can still undo.
		World->ShrinkLevel();

		{
			FScopedSlowTask SlowTask(0, FText::Format(NSLOCTEXT("UnrealEd", "SavingMapStatus_CollectingGarbage", "Saving map: {0}... (Collecting garbage)"), FText::FromString(World->GetName())));
			// NULL empty or "invalid" entries (e.g. IsPendingKill()) in actors array.
			CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
		}
			
		// Compact and sort actors array to remove empty entries.
		// Don't do this when autosaving or PIE saving so that actor adds can still undo.
		World->PersistentLevel->SortActorList();
	}

	// Move level position closer to world origin
	UWorld* OwningWorld = World->PersistentLevel->OwningWorld;
	if (OwningWorld->WorldComposition)
	{
		OwningWorld->WorldComposition->OnLevelPreSave(World->PersistentLevel);
	}

	// If we can get the streaming level, we should remove the editor transform before saving
	ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel( World->PersistentLevel );
	if ( LevelStreaming && World->PersistentLevel->bAlreadyMovedActors )
	{
		FLevelUtils::RemoveEditorTransform(LevelStreaming);
	}

	// Make sure the public and standalone flags are set on this world to allow it to work properly with the editor
	World->SetFlags(RF_Public | RF_Standalone);
}

void UEditorEngine::OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsLastConcurrentSave())
	{
		return;
	}
	if ( !ensure(World) )
	{
		return;
	}
	uint32 OriginalPackageFlags = ObjectSaveContext.GetOriginalPackageFlags();
	bool bSuccess = ObjectSaveContext.SaveSucceeded();

	if ( !IsRunningCommandlet() )
	{
		UPackage* WorldPackage = World->GetOutermost();
		const bool bAutosave = ObjectSaveContext.IsFromAutoSave();
		if ( bAutosave )
		{
			// Restore original value of PKG_PlayInEditor if we changed it during PIE saving
			const bool bOriginallyPIE = (OriginalPackageFlags & PKG_PlayInEditor) != 0;
			const bool bCurrentlyPIE = (WorldPackage->HasAnyPackageFlags(PKG_PlayInEditor));
			if ( !bOriginallyPIE && bCurrentlyPIE )
			{
				WorldPackage->ClearPackageFlags(PKG_PlayInEditor);
			}
		}
		else
		{
			// Normal non-pie and non-autosave codepath
			FWorldContext &EditorContext = GetEditorWorldContext();

			const bool bIsPersistentLevel = (World == EditorContext.World());
			if ( bSuccess )
			{
				// Put the map into the MRU and mark it as not dirty.

				if ( bIsPersistentLevel && FPackageName::IsValidLongPackageName(WorldPackage->GetName()))
				{
					// Set the map filename.
					const FString Filename = FPackageName::LongPackageNameToFilename(WorldPackage->GetName(), FPackageName::GetMapPackageExtension());
					FEditorFileUtils::RegisterLevelFilename( World, Filename );

					WorldPackage->SetDirtyFlag(false);

					// Update the editor's MRU level list if we were asked to do that for this level
					IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );

					if ( MainFrameModule.GetMRUFavoritesList() )
					{
						MainFrameModule.GetMRUFavoritesList()->AddMRUItem(WorldPackage->GetName());
					}

					FEditorDirectories::Get().SetLastDirectory(ELastDirectory::UNR, FPaths::GetPath(Filename)); // Save path as default for next time.
				}

				// We saved the map, so unless there are any other dirty levels, go ahead and reset the autosave timer
				if( GUnrealEd && !GUnrealEd->AnyWorldsAreDirty( World ) )
				{
					GUnrealEd->GetPackageAutoSaver().ResetAutoSaveTimer();
				}
			}

			if ( bIsPersistentLevel )
			{
				FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();
			}
		}
	}

	// Restore level original position
	UWorld* OwningWorld = World->PersistentLevel->OwningWorld;
	if (OwningWorld->WorldComposition)
	{
		OwningWorld->WorldComposition->OnLevelPostSave(World->PersistentLevel);
	}

	// If got the streaming level, we should re-apply the editor transform after saving
	ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel( World->PersistentLevel );
	if ( LevelStreaming && World->PersistentLevel->bAlreadyMovedActors )
	{
		FLevelUtils::ApplyEditorTransform(LevelStreaming);
	}

	// Post save world event
	FEditorDelegates::PostSaveWorldWithContext.Broadcast(World, ObjectSaveContext);
}

APlayerStart* UEditorEngine::CheckForPlayerStart()
{
	UWorld* IteratorWorld = GWorld;
	if ( TActorIterator<APlayerStart> It(IteratorWorld); It)
	{
		// Return the found start.
		return *It;
	}

	// No player start was found, return NULL.
	return nullptr;
}

void UEditorEngine::CloseEntryPopupWindow()
{
	if (PopupWindow.IsValid())
	{
		PopupWindow.Pin()->RequestDestroyWindow();
	}
}

EAppReturnType::Type UEditorEngine::OnModalMessageDialog(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessage, const FText& InText, const FText& InTitle)
{
	if( IsInGameThread() && FSlateApplication::IsInitialized() && FSlateApplication::Get().CanAddModalWindow() )
	{
		return OpenMessageDialog_Internal(InMessageCategory, InMessage, InText, InTitle);
	}
	else
	{
		return FPlatformMisc::MessageBoxExt(InMessage, *InText.ToString(), *InTitle.ToString());
	}
}

bool UEditorEngine::OnShouldLoadOnTop( const FString& Filename )
{
	 return FPaths::GetBaseFilename(Filename) == FPaths::GetBaseFilename(UserOpenedFile);
}

TSharedPtr<SViewport> UEditorEngine::GetGameViewportWidget() const
{
	for (auto It = SlatePlayInEditorMap.CreateConstIterator(); It; ++It)
	{
		if (It.Value().SlatePlayInEditorWindowViewport.IsValid())
		{
			return It.Value().SlatePlayInEditorWindowViewport->GetViewportWidget().Pin();
		}

		TSharedPtr<IAssetViewport> DestinationLevelViewport = It.Value().DestinationSlateViewport.Pin();
		if (DestinationLevelViewport.IsValid())
		{
			return DestinationLevelViewport->GetViewportWidget().Pin();
		}
	}

	/*
	if(SlatePlayInEditorSession.SlatePlayInEditorWindowViewport.IsValid())
	{
		return SlatePlayInEditorSession.SlatePlayInEditorWindowViewport->GetViewportWidget().Pin();
	}
	*/

	return NULL;
}

FString UEditorEngine::GetFriendlyName( const FProperty* Property, UStruct* OwnerStruct/* = NULL*/ )
{
	// first, try to pull the friendly name from the loc file
	check( Property );
	UStruct* RealOwnerStruct = Property->GetOwnerStruct();
	if ( OwnerStruct == NULL)
	{
		OwnerStruct = RealOwnerStruct;
	}
	checkSlow(OwnerStruct);

	FText FoundText;
	bool DidFindText = false;
	UStruct* CurrentStruct = OwnerStruct;
	do 
	{
		FString PropertyPathName = Property->GetPathName(CurrentStruct);

		DidFindText = FText::FindTextInLiveTable_Advanced(*CurrentStruct->GetName(), *(PropertyPathName + TEXT(".FriendlyName")), /*OUT*/FoundText );
		CurrentStruct = CurrentStruct->GetSuperStruct();
	} while( CurrentStruct != NULL && CurrentStruct->IsChildOf(RealOwnerStruct) && !DidFindText );

	if ( !DidFindText )
	{
		const FString& DefaultFriendlyName = Property->GetMetaData(TEXT("DisplayName"));
		if ( DefaultFriendlyName.IsEmpty() )
		{
			const bool bIsBool = CastField<const FBoolProperty>(Property) != NULL;
			return FName::NameToDisplayString( Property->GetName(), bIsBool );
		}
		return DefaultFriendlyName;
	}

	return FoundText.ToString();
}

UActorGroupingUtils* UEditorEngine::GetActorGroupingUtils()
{
	if (ActorGroupingUtils == nullptr)
	{
		UClass* ActorGroupingUtilsClass = ActorGroupingUtilsClassName.ResolveClass();
		if (!ActorGroupingUtilsClass)
		{
			ActorGroupingUtilsClass = UActorGroupingUtils::StaticClass();
		}

		ActorGroupingUtils = NewObject<UActorGroupingUtils>(this, ActorGroupingUtilsClass);
	}

	return ActorGroupingUtils;
}

AActor* UEditorEngine::UseActorFactoryOnCurrentSelection( UActorFactory* Factory, const FTransform* InActorTransform, EObjectFlags InObjectFlags )
{
	// ensure that all selected assets are loaded
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
	return UseActorFactory(Factory, FAssetData( GetSelectedObjects()->GetTop<UObject>() ), InActorTransform, InObjectFlags );
}

AActor* UEditorEngine::UseActorFactory( UActorFactory* Factory, const FAssetData& AssetData, const FTransform* InActorTransform, EObjectFlags InObjectFlags )
{
	AActor* NewActor = nullptr;

	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		if (ULevel* DesiredLevel = LevelEditorSubsystem->GetCurrentLevel())
		{
			if (UObject* LoadedAsset = AssetData.GetAsset())
			{
				TArray<AActor*> Actors = FLevelEditorViewportClient::TryPlacingActorFromObject(DesiredLevel, LoadedAsset, true, RF_Transactional, Factory);
				if (Actors.Num() && (Actors[0] != nullptr))
				{
					NewActor = Actors[0];
					if (InActorTransform)
					{
						NewActor->SetActorTransform(*InActorTransform, false, nullptr, ETeleportType::TeleportPhysics);
					}
				}
			}
		}
	}

	return NewActor;
}

void UEditorEngine::ReplaceSelectedActors(UActorFactory* Factory, const FAssetData& AssetData, bool bCopySourceProperties)
{
	UEditorActorSubsystem::ReplaceSelectedActors(Factory, AssetData, bCopySourceProperties);
}

void UEditorEngine::ReplaceActors(UActorFactory* Factory, const FAssetData& AssetData, const TArray<AActor*>& ActorsToReplace, TArray<AActor*>* OutNewActors, bool bCopySourceProperties)
{
	UEditorActorSubsystem::ReplaceActors(Factory, AssetData, ActorsToReplace, OutNewActors, bCopySourceProperties);
}

AActor* UEditorEngine::ConvertBrushesToStaticMesh(const FString& InStaticMeshPackageName, TArray<ABrush*>& InBrushesToConvert, const FVector& InPivotLocation)
{
	return UEditorActorSubsystem::ConvertBrushesToStaticMesh(InStaticMeshPackageName, InBrushesToConvert, InPivotLocation);
}

void UEditorEngine::ConvertLightActors( UClass* ConvertToClass )
{
	UEditorActorSubsystem::ConvertLightActors(ConvertToClass);
}

void UEditorEngine::ConvertActors( const TArray<AActor*>& ActorsToConvert, UClass* ConvertToClass, const TSet<FString>& ComponentsToConsider, bool bUseSpecialCases )
{
	UEditorActorSubsystem::ConvertActors(ActorsToConvert, ConvertToClass, ComponentsToConsider, bUseSpecialCases);
}

void UEditorEngine::DoConvertActors( const TArray<AActor*>& ActorsToConvert, UClass* ConvertToClass, const TSet<FString>& ComponentsToConsider, bool bUseSpecialCases, const FString& InStaticMeshPackageName )
{
	UEditorActorSubsystem::DoConvertActors(ActorsToConvert, ConvertToClass, ComponentsToConsider, bUseSpecialCases, InStaticMeshPackageName);
}

void UEditorEngine::NotifyToolsOfObjectReplacement(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	// Allow any other observers to act upon the object replacement
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);
	FCoreUObjectDelegates::OnObjectsReplaced.Broadcast(OldToNewInstanceMap);
}

void UEditorEngine::SetViewportsRealtimeOverride(bool bShouldBeRealtime, FText SystemDisplayName)
{
	for (FEditorViewportClient* VC : AllViewportClients)
	{
		if (VC)
		{
			VC->AddRealtimeOverride(bShouldBeRealtime, SystemDisplayName);
		}
	}

	RedrawAllViewports();

	FEditorSupportDelegates::UpdateUI.Broadcast();
}

void UEditorEngine::RemoveViewportsRealtimeOverride(FText SystemDisplayName)
{
	// We don't check that we had an override on all the viewport clients because since the caller added their override, there could have
	// been new viewport clients added to the list by someone else. It's probably that the caller just wants to make sure no viewport has
	// their override anymore so it's a sensible default to ignore those who don't have it.
	const bool bCheckMissingOverride = false;

	for (FEditorViewportClient* VC : AllViewportClients)
	{
		if (VC)
		{
			VC->RemoveRealtimeOverride(SystemDisplayName, bCheckMissingOverride);
		}
	}

	RedrawAllViewports();

	FEditorSupportDelegates::UpdateUI.Broadcast();
}


bool UEditorEngine::IsAnyViewportRealtime()
{
	for(FEditorViewportClient* VC : AllViewportClients)
	{
		if( VC )
		{
			if( VC->IsRealtime() )
			{
				return true;
			}
		}
	}
	return false;
}

bool UEditorEngine::ShouldThrottleCPUUsage() const
{
	// Don't throttle here. Benchmarking, automation, and terminal shouldn't throttle 
	if (IsRunningCommandlet() 
		|| FApp::IsBenchmarking()
		|| FApp::IsUnattended()
		|| !FApp::CanEverRender())
	{
		return false;
	}

	// There might be systems where throttling would cause issues (such as data transfer over the network) - give them
	// an opportunity to force us to not throttle.
	for (auto It = ShouldDisableCPUThrottlingDelegates.CreateConstIterator(); It; ++It)
	{
		if (It->IsBound() && It->Execute())
		{
			return false;
		}
	}
	

	bool bShouldThrottle = false;
	
	// Always check if in foreground, since VR apps will only have focus when running (PIE, etc).
	const bool bIsForeground = FPlatformApplicationMisc::IsThisApplicationForeground();
	
	if (!bIsForeground &&!FApp::HasFocus() && !IsRunningCommandlet() && !GIsAutomationTesting && !FApp::IsBenchmarking())
	{
		const UEditorPerformanceSettings* Settings = GetDefault<UEditorPerformanceSettings>();
		bShouldThrottle = Settings->bThrottleCPUWhenNotForeground;

		// Check if we should throttle due to all windows being minimized
		if (FSlateApplication::IsInitialized())
		{
			if (!bShouldThrottle)
			{
				bShouldThrottle = AreAllWindowsHidden();
			}

			// Do not throttle during drag and drop
			if (bShouldThrottle && FSlateApplication::Get().IsDragDropping())
			{
				bShouldThrottle = false;
			}

			if (bShouldThrottle)
			{
				static const FName AssetRegistryName(TEXT("AssetRegistry"));
				IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
				// Don't throttle during amortized export, greatly increases export time
				if (IsLightingBuildCurrentlyExporting() || FStaticLightingSystemInterface::IsStaticLightingSystemRunning() ||
					GShaderCompilingManager->IsCompiling() || (AssetRegistry && AssetRegistry->IsLoadingAssets()))
				{
					bShouldThrottle = false;
				}
			}
		}

		if (bShouldThrottle)
		{
			// postloading and asset compilation needs to tick on GT and throttling them
			// can make editor loading a lot longer.
			if (IsAsyncLoading() || FAssetCompilingManager::Get().GetNumRemainingAssets() > 0)
			{
				bShouldThrottle = false;
			}
		}
	}

	return bShouldThrottle;
}

AActor* UEditorEngine::AddActor(ULevel* InLevel, UClass* Class, const FTransform& Transform, bool bSilent, EObjectFlags InObjectFlags, bool bSelectActor)
{
	check( Class );

	if( !bSilent )
	{
		const auto Location = Transform.GetLocation();
		UE_LOGF(LogEditor, Log,
			"Attempting to add actor of class '%ls' to level at %0.2f,%0.2f,%0.2f",
			*Class->GetName(), Location.X, Location.Y, Location.Z );
	}

	///////////////////////////////
	// Validate class flags.

	if( Class->HasAnyClassFlags(CLASS_Abstract) )
	{
		UE_LOGF(LogEditor, Error, "Class %ls is abstract.  You can't add actors of this class to the world.", *Class->GetName() );
		return NULL;
	}
	if( Class->HasAnyClassFlags(CLASS_NotPlaceable) )
	{
		UE_LOGF(LogEditor, Error, "Class %ls isn't placeable.  You can't add actors of this class to the world.", *Class->GetName() );
		return NULL;
	}
	if( Class->HasAnyClassFlags(CLASS_Transient) )
	{
		UE_LOGF(LogEditor, Error, "Class %ls is transient.  You can't add actors of this class in UnrealEd.", *Class->GetName() );
		return NULL;
	}


	UWorld* World = InLevel->OwningWorld;
	ULevel* DesiredLevel = InLevel;

	// Don't spawn the actor if the current level is locked.
	if ( FLevelUtils::IsLevelLocked(DesiredLevel) )
	{
		FNotificationInfo Info( NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevel", "The requested operation could not be completed because the level is locked.") );
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return NULL;
	}

	// Transactionally add the actor.
	AActor* Actor = NULL;
	{
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "AddActor", "Add Actor") );
		if (!(InObjectFlags & RF_Transactional))
		{
			// Don't attempt a transaction if the actor we are spawning isn't transactional
			Transaction.Cancel();
		}

		if (bSelectActor)
		{
			SelectNone(false, true);
		}

		AActor* Default = Class->GetDefaultObject<AActor>();

		FActorSpawnParameters SpawnInfo;
		SpawnInfo.OverrideLevel = DesiredLevel;
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		SpawnInfo.ObjectFlags = InObjectFlags;
		const auto Location = Transform.GetLocation();
		const auto Rotation = Transform.GetRotation().Rotator();
		Actor = World->SpawnActor( Class, &Location, &Rotation, SpawnInfo );

		if (Actor)
		{
			FActorLabelUtilities::SetActorLabelUnique(Actor, Actor->GetDefaultActorLabel());

			if (bSelectActor)
			{
				SelectActor(Actor, 1, 0);
			}

			Actor->InvalidateLightingCache();
			Actor->PostEditMove( true );
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_Couldn'tSpawnActor", "Couldn't spawn actor. Please check the log.") );
		}
	}

	if( Actor )
	{
		// If this actor is part of any layers (set in its default properties), add them into the visible layers list.
		GetEditorSubsystem<ULayersSubsystem>()->SetLayersVisibility(Actor->Layers, true);

		// Clean up.
		Actor->MarkPackageDirty();
		ULevel::LevelDirtiedEvent.Broadcast();
	}

	if( bSelectActor )
	{
		NoteSelectionChange();
	}

	return Actor;
}

TArray<AActor*> UEditorEngine::AddExportTextActors(const FString& ExportText, bool bSilent, EObjectFlags InObjectFlags)
{
	TArray<AActor*> NewActors;

	// Don't spawn the actor if the current level is locked.
	ULevel* CurrentLevel = GWorld->GetCurrentLevel();
	if ( FLevelUtils::IsLevelLocked( CurrentLevel ) )
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelAddExportTextActors", "AddExportTextActors: The requested operation could not be completed because the level is locked."));
		return NewActors;
	}

	// Use a level factory to spawn all the actors using the ExportText
	auto Factory = NewObject<ULevelFactory>();
	FVector Location;
	{
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "AddActor", "Add Actor") );
		if ( !(InObjectFlags & RF_Transactional) )
		{
			// Don't attempt a transaction if the actor we are spawning isn't transactional
			Transaction.Cancel();
		}
		// Remove the selection to detect the actors that were created during FactoryCreateText. They will be selected when the operation in complete
		SelectNone( false, true );
		const TCHAR* Text = *ExportText;
		if ( Factory->FactoryCreateText( ULevel::StaticClass(), CurrentLevel, CurrentLevel->GetFName(), InObjectFlags, nullptr, TEXT("paste"), Text, Text + FCString::Strlen(Text), GWarn ) != nullptr )
		{
			// Now get the selected actors and calculate a center point between all their locations.
			USelection* ActorSelection = GetSelectedActors();
			FVector Origin = FVector::ZeroVector;
			for ( int32 ActorIdx = 0; ActorIdx < ActorSelection->Num(); ++ActorIdx )
			{
				AActor* Actor = CastChecked<AActor>(ActorSelection->GetSelectedObject(ActorIdx));
				NewActors.Add(Actor);
				Origin += Actor->GetActorLocation();
			}

			if ( NewActors.Num() > 0 )
			{
				// Finish the Origin calculations now that we know we are not going to divide by zero
				Origin /= NewActors.Num();

				// Set up the spawn location
				FSnappingUtils::SnapPointToGrid( ClickLocation, FVector(0, 0, 0) );
				Location = ClickLocation;
				FVector Collision = NewActors[0]->GetPlacementExtent();
				Location += ClickPlane * (FVector::BoxPushOut(ClickPlane, Collision) + 0.1f);
				FSnappingUtils::SnapPointToGrid( Location, FVector(0, 0, 0) );

				// For every spawned actor, teleport to the target loction, preserving the relative translation to the other spawned actors.
				ULayersSubsystem* LayersSubsystem = GetEditorSubsystem<ULayersSubsystem>();
				for ( int32 ActorIdx = 0; ActorIdx < NewActors.Num(); ++ActorIdx )
				{
					AActor* Actor = NewActors[ActorIdx];
					FVector OffsetToOrigin = Actor->GetActorLocation() - Origin;

					Actor->TeleportTo(Location + OffsetToOrigin, Actor->GetActorRotation(), false, true );
					Actor->InvalidateLightingCache();
					Actor->PostEditMove( true );

					LayersSubsystem->SetLayersVisibility( Actor->Layers, true );

					Actor->MarkPackageDirty();
				}

				// Send notification about a new actor being created
				ULevel::LevelDirtiedEvent.Broadcast();
				NoteSelectionChange();

	            if( !bSilent )
	            {
		            UE_LOGF(LogEditor, Log,
			            "Added '%d' actor(s) to level at %0.2f,%0.2f,%0.2f",
			            NewActors.Num(), Location.X, Location.Y, Location.Z );
	            }
			}
		}
	}

	return NewActors;
}

UActorFactory* UEditorEngine::FindActorFactoryForActorClass( const UClass* InClass )
{
	for( int32 i = 0 ; i < ActorFactories.Num() ; ++i )
	{
		UActorFactory* Factory = ActorFactories[i];

		// force NewActorClass update
		const UObject* const ActorCDO = Factory->GetDefaultActor( FAssetData() );
		if( ActorCDO != NULL && ActorCDO->GetClass() == InClass )
		{
			return Factory;
		}
	}

	return NULL;
}

UActorFactory* UEditorEngine::FindActorFactoryByClass( const UClass* InClass ) const
{
	for( int32 i = 0 ; i < ActorFactories.Num() ; ++i )
	{
		UActorFactory* Factory = ActorFactories[i];

		if( Factory != NULL && Factory->GetClass() == InClass )
		{
			return Factory;
		}
	}

	return NULL;
}

UActorFactory* UEditorEngine::FindActorFactoryByClassForActorClass( const UClass* InFactoryClass, const UClass* InActorClass )
{
	for ( int32 i = 0; i < ActorFactories.Num(); ++i )
	{
		UActorFactory* Factory = ActorFactories[i];

		if ( Factory != NULL && Factory->GetClass() == InFactoryClass )
		{
			// force NewActorClass update
			const UObject* const ActorCDO = Factory->GetDefaultActor( FAssetData(InActorClass) );
			if ( ActorCDO != NULL && ActorCDO->GetClass() == InActorClass )
			{
				return Factory;
			}
		}
	}

	return NULL;
}

void UEditorEngine::PreWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin)
{
	// In case we simulating world in the editor, 
	// we need to shift current viewport as well, 
	// so the streaming procedure will receive correct view location
	if (bIsSimulatingInEditor && 
		GCurrentLevelEditingViewportClient &&
		GCurrentLevelEditingViewportClient->IsVisible())
	{
		FVector ViewLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
		GCurrentLevelEditingViewportClient->SetViewLocation(ViewLocation - FVector(InDstOrigin - InSrcOrigin));
	}
}

void UEditorEngine::SetPreviewMeshMode( bool bState )
{
	// Only change the state if it's different than the current.
	if( bShowPreviewMesh != bState )
	{
		// Set the preview mesh mode state. 
		bShowPreviewMesh = !bShowPreviewMesh;

		bool bHavePreviewMesh = (PreviewMeshComp != NULL);

		// It's possible that the preview mesh hasn't been loaded yet,
		// such as on first-use for the preview mesh mode or there 
		// could be valid mesh names provided in the INI. 
		if( !bHavePreviewMesh )
		{
			bHavePreviewMesh = LoadPreviewMesh( PreviewMeshIndex );
		}

		// If we have a	preview mesh, change it's visibility based on the preview state. 
		if( bHavePreviewMesh )
		{
			PreviewMeshComp->SetVisibility( bShowPreviewMesh );
			PreviewMeshComp->SetCastShadow( bShowPreviewMesh );
			RedrawLevelEditingViewports();
		}
		else
		{
			// Without a preview mesh, we can't really use the preview mesh mode. 
			// So, disable it even if the caller wants to enable it. 
			bShowPreviewMesh = false;
		}
	}
}


void UEditorEngine::UpdatePreviewMesh()
{
	if( bShowPreviewMesh )
	{
		// The component should exist by now. Is the bPlayerHeight state 
		// manually changed instead of calling SetPreviewMeshMode()?
		check(PreviewMeshComp);

		// Use the cursor world location as the starting location for the line check. 
		FViewportCursorLocation CursorLocation = GCurrentLevelEditingViewportClient->GetCursorWorldLocationFromMousePos();
		FVector LineCheckStart = CursorLocation.GetOrigin();
		FVector LineCheckEnd = CursorLocation.GetOrigin() + CursorLocation.GetDirection() * HALF_WORLD_MAX;

		// Perform a line check from the camera eye to the surface to place the preview mesh. 
		FHitResult Hit(ForceInit);
		FCollisionQueryParams LineParams(SCENE_QUERY_STAT(UpdatePreviewMeshTrace), true);
		LineParams.bTraceComplex = false;
		if ( GWorld->LineTraceSingleByObjectType(Hit, LineCheckStart, LineCheckEnd, FCollisionObjectQueryParams(ECC_WorldStatic), LineParams) ) 
		{
			// Dirty the transform so UpdateComponent will actually update the transforms. 
			PreviewMeshComp->SetRelativeLocation(Hit.Location);
		}

		// Redraw the viewports because the mesh won't 
		// be shown or hidden until that happens. 
		RedrawLevelEditingViewports();
	}
}


void UEditorEngine::CyclePreviewMesh()
{
	const ULevelEditorViewportSettings& ViewportSettings = *GetDefault<ULevelEditorViewportSettings>();
	if( !ViewportSettings.PreviewMeshes.Num() )
	{
		return;
	}

	const int32 StartingPreviewMeshIndex = FMath::Min(PreviewMeshIndex, ViewportSettings.PreviewMeshes.Num() - 1);
	int32 CurrentPreviewMeshIndex = StartingPreviewMeshIndex;
	bool bPreviewMeshFound = false;

	do
	{
		// Cycle to the next preview mesh. 
		CurrentPreviewMeshIndex++;

		// If we reached the max index, start at index zero.
		if( CurrentPreviewMeshIndex == ViewportSettings.PreviewMeshes.Num() )
		{
			CurrentPreviewMeshIndex = 0;
		}

		// Load the mesh (if not already) onto the mesh actor. 
		bPreviewMeshFound = LoadPreviewMesh(CurrentPreviewMeshIndex);

		if( bPreviewMeshFound )
		{
			// Save off the index so we can reference it later when toggling the preview mesh mode. 
			PreviewMeshIndex = CurrentPreviewMeshIndex;
		}

		// Keep doing this until we found another valid mesh, or we cycled through all possible preview meshes. 
	} while( !bPreviewMeshFound && (StartingPreviewMeshIndex != CurrentPreviewMeshIndex) );
}

bool UEditorEngine::LoadPreviewMesh( int32 Index )
{
	bool bMeshLoaded = false;

	// Don't register the preview mesh into the PIE world!
	if(GWorld->IsPlayInEditor())
	{
		UE_LOGF(LogEditorViewport, Warning, "LoadPreviewMesh called while PIE world is GWorld.");
		return false;
	}

	const ULevelEditorViewportSettings& ViewportSettings = *GetDefault<ULevelEditorViewportSettings>();
	if( ViewportSettings.PreviewMeshes.IsValidIndex(Index) )
	{
		const FSoftObjectPath& MeshName = ViewportSettings.PreviewMeshes[Index];

		// If we don't have a preview mesh component in the world yet, create one. 
		if( !PreviewMeshComp )
		{
			PreviewMeshComp = NewObject<UStaticMeshComponent>();
			check(PreviewMeshComp);

			// Attach the component to the scene even if the preview mesh doesn't load.
			PreviewMeshComp->RegisterComponentWithWorld(GWorld);
		}

		// Load the new mesh, if not already loaded. 
		UStaticMesh* PreviewMesh = LoadObject<UStaticMesh>(NULL, *MeshName.ToString(), NULL, LOAD_None, NULL);

		// Swap out the meshes if we loaded or found the given static mesh. 
		if( PreviewMesh )
		{
			bMeshLoaded = true;
			PreviewMeshComp->SetStaticMesh(PreviewMesh);
		}
		else
		{
			UE_LOGF(LogEditorViewport, Warning, "Couldn't load the PreviewMeshNames for the player at index, %d, with the name, %ls.", Index, *MeshName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogEditorViewport, Log,  "Invalid array index, %d, provided for PreviewMeshNames in UEditorEngine::LoadPreviewMesh", Index );
	}

	return bMeshLoaded;
}

void UEditorEngine::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel)
	{
		// Update the editorworld list, and make sure this actor is selected if it was when we began pie/sie
		for (int32 ActorIdx = 0; ActorIdx < InLevel->Actors.Num(); ActorIdx++)
		{
			AActor* LevelActor = InLevel->Actors[ActorIdx];
			if ( LevelActor )
			{
				ObjectsThatExistInEditorWorld.Set(LevelActor);

				if ( ActorsThatWereSelected.Num() > 0 )
				{
					AActor* EditorActor = EditorUtilities::GetEditorWorldCounterpartActor( LevelActor );
					if ( EditorActor && ActorsThatWereSelected.Contains( EditorActor ) )
					{
						SelectActor( LevelActor, true, false );
					}
				}
			}
		}
	}
}

void UEditorEngine::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel)
	{
		// Update the editorworld list and deselect actors belonging to removed level
		for (int32 ActorIdx = 0; ActorIdx < InLevel->Actors.Num(); ActorIdx++)
		{
			AActor* LevelActor = InLevel->Actors[ActorIdx];
			if ( LevelActor )
			{
				ObjectsThatExistInEditorWorld.Clear(LevelActor);

				SelectActor(LevelActor, false, false);
			}
		}
	}
	// UEngine::LoadMap broadcast this event with InLevel==NULL, before cleaning up the world during travel in Multiplayer
	else 
	{
		// Clear the editor selection if it is the edited world.
		if (InWorld->IsPlayInEditor() || InWorld->WorldType == EWorldType::Editor)
		{
			SelectNone(true, true, false);
		}

		if (Trans)
		{
			if (InWorld->IsPlayInEditor())
			{
				if (Trans->ContainsPieObjects())
				{
					ResetTransaction(NSLOCTEXT("UnrealEd", "LevelRemovedFromWorldEditorCallbackPIE", "Level removed from PIE/SIE world"));
				}

				// Each additional instance of PIE in a multiplayer game will add another barrier, so if the event is triggered then this is the case and we need to lift it
				// Otherwise there will be an imbalance between barriers set and barriers removed and we won't be able to undo when we return.
				Trans->RemoveUndoBarrier();
			}
			else
			{
				// If we're in editor mode, reset transactions buffer, to ensure that there are no references to a world which is about to be destroyed
				ResetTransaction(NSLOCTEXT("UnrealEd", "LevelRemovedFromWorldEditorCallback", "Level removed from world"));
			}
		}
	}
}

void UEditorEngine::OnSeamlessTravelTransition(UWorld* InWorld)
{
	if (InWorld->IsPlayInEditor())
	{
		EndTransaction();
		SelectNone(true, true, false);
	}
}

void UEditorEngine::UpdateRecentlyLoadedProjectFiles()
{
	if (FPaths::IsProjectFilePathSet())
	{
		FDateTime CurrentTime = FDateTime::UtcNow();

		const FString AbsoluteProjectPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FPaths::GetProjectFilePath());
		// Update the recently loaded project files. Move this project file to the front of the list
		TArray<FRecentProjectFile>& RecentlyOpenedProjectFiles = GetMutableDefault<UEditorSettings>()->RecentlyOpenedProjectFiles;

		FRecentProjectFile MostRecentProject(AbsoluteProjectPath, CurrentTime);
		RecentlyOpenedProjectFiles.Remove(MostRecentProject);
		RecentlyOpenedProjectFiles.Insert(MostRecentProject, 0);

		// Trim any project files that do not have the current game project file extension
		for ( int32 FileIdx = RecentlyOpenedProjectFiles.Num() - 1; FileIdx >= 0; --FileIdx )
		{
			const FString FileExtension = FPaths::GetExtension(RecentlyOpenedProjectFiles[FileIdx].ProjectName);
			if ( FileExtension != FProjectDescriptor::GetExtension() )
			{
				RecentlyOpenedProjectFiles.RemoveAt(FileIdx, 1);
			}
		}

		// Trim the list in case we have more than the max
		const int32 MaxRecentProjectFiles = 10;
		if ( RecentlyOpenedProjectFiles.Num() > MaxRecentProjectFiles )
		{
			RecentlyOpenedProjectFiles.RemoveAt(MaxRecentProjectFiles, RecentlyOpenedProjectFiles.Num() - MaxRecentProjectFiles);
		}

		GetMutableDefault<UEditorSettings>()->PostEditChange();
		
		if (GIsEditor && !IsRunningCommandlet())
		{
			FProjectEditorRecord::QueueUpdate(
			[AbsoluteProjectPath](FProjectEditorRecord& InEditorRecords)
			{
				const TSharedPtr<FJsonObject> Projects = InEditorRecords.FindOrAddProperty(FProjectEditorRecord::ProjectsProperty);

				if(Projects.IsValid())
				{
					Projects->SetObjectField(AbsoluteProjectPath, FProjectEditorRecord::MakeDefaultProperties());
				}
			});
		}
	}
}

#if PLATFORM_MAC
static TWeakPtr<SNotificationItem> GXcodeWarningNotificationPtr;
#endif

void UEditorEngine::UpdateAutoLoadProject()
{
	// If the recent project file exists and is non-empty, update the contents with the currently loaded .uproject
	// If the recent project file exists and is empty, recent project files should not be auto-loaded
	// If the recent project file does not exist, auto-populate it with the currently loaded project in installed builds and auto-populate empty in non-installed
	//		In installed builds we default to auto-loading, in non-installed we default to opting out of auto loading
	const FString& AutoLoadProjectFileName = IProjectManager::Get().GetAutoLoadProjectFileName();
	FString RecentProjectFileContents;
	bool bShouldLoadRecentProjects = false;
	if ( FFileHelper::LoadFileToString(RecentProjectFileContents, *AutoLoadProjectFileName) )
	{
		// Update to the most recently loaded project and continue auto-loading
		if ( FPaths::IsProjectFilePathSet() )
		{
			FFileHelper::SaveStringToFile(FPaths::GetProjectFilePath(), *AutoLoadProjectFileName);
		}

		bShouldLoadRecentProjects = true;
	}
	else
	{
		// We do not default to auto-loading project files.
		bShouldLoadRecentProjects = false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GetMutableDefault<UEditorSettings>()->bLoadTheMostRecentlyLoadedProjectAtStartup = bShouldLoadRecentProjects;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Sync up the new LoadAtStartup setting with the old one
	if (UClass* SettingsClass = FindObject<UClass>(nullptr, TEXT("/Script/MainFrame.HomeScreenSettings")))
	{
		UObject* DefaultObj = SettingsClass->GetDefaultObject();
		FProperty* Prop = SettingsClass->FindPropertyByName("LoadAtStartup");
		if (Prop && DefaultObj && Prop->IsA<FEnumProperty>())
		{
			FEnumProperty* AutoLoadProperty = CastFieldChecked<FEnumProperty>(Prop);
			EAutoLoadProject Value = bShouldLoadRecentProjects ? EAutoLoadProject::LastProject : EAutoLoadProject::HomeScreen;
			AutoLoadProperty->SetValue_InContainer(DefaultObj, &Value);
		}
	}

#if PLATFORM_MAC
	if ( !GIsBuildMachine )
	{
		if(FPlatformMisc::MacOSXVersionCompare(12, 0, 0) < 0)
		{
			if(FSlateApplication::IsInitialized())
			{
				FString SuppressSettingName(FString(TEXT("UpdateMacOSX_")) + VERSION_STRINGIFY(ENGINE_MAJOR_VERSION) + TEXT("_") + VERSION_STRINGIFY(ENGINE_MINOR_VERSION) + TEXT("_") + VERSION_STRINGIFY(ENGINE_PATCH_VERSION));
				FSuppressableWarningDialog::FSetupInfo Info( LOCTEXT("UpdateMacOSX_Body","Please update to the latest version of macOS for best performance and stability."), LOCTEXT("UpdateMacOSX_Title","Update macOS"), *SuppressSettingName, GEditorSettingsIni );
				Info.ConfirmText = LOCTEXT( "OK", "OK");
				Info.bDefaultToSuppressInTheFuture = false;
				FSuppressableWarningDialog OSUpdateWarning( Info );
				OSUpdateWarning.ShowModal();
			}
			else
			{
				UE_LOGF(LogEditor, Warning, "Please update to the latest version of macOS for best performance and stability.");
			}
		}
		
		// Warn about low-memory configurations as they harm performance in the Editor
		if(FPlatformMemory::GetPhysicalGBRam() < 8)
		{
			if(FSlateApplication::IsInitialized())
			{
				FSuppressableWarningDialog::FSetupInfo Info( LOCTEXT("LowRAMWarning_Body","For best performance install at least 8GB of RAM."), LOCTEXT("LowRAMWarning_Title","Low RAM"), TEXT("LowRAMWarning"), GEditorSettingsIni );
				Info.ConfirmText = LOCTEXT( "OK", "OK");
				Info.bDefaultToSuppressInTheFuture = true;
				FSuppressableWarningDialog OSUpdateWarning( Info );
				OSUpdateWarning.ShowModal();
			}
			else
			{
				UE_LOGF(LogEditor, Warning, "For best performance install at least 8GB of RAM.");
			}
		}
		
		// And also warn about machines with fewer than 4 cores as they will also struggle
		if(FPlatformMisc::NumberOfCores() < 4)
		{
			if(FSlateApplication::IsInitialized())
			{
				FSuppressableWarningDialog::FSetupInfo Info( LOCTEXT("SlowCPUWarning_Body","For best performance a Quad-core Intel or AMD processor, 2.5 GHz or faster is recommended."), LOCTEXT("SlowCPUWarning_Title","CPU Performance Warning"), TEXT("SlowCPUWarning"), GEditorSettingsIni );
				Info.ConfirmText = LOCTEXT( "OK", "OK");
				Info.bDefaultToSuppressInTheFuture = true;
				FSuppressableWarningDialog OSUpdateWarning( Info );
				OSUpdateWarning.ShowModal();
			}
			else
			{
				UE_LOGF(LogEditor, Warning, "For best performance a Quad-core Intel or AMD processor, 2.5 GHz or faster is recommended.");
			}
		}
	}

	if (FSlateApplication::IsInitialized() && !FPlatformMisc::IsSupportedXcodeVersionInstalled())
	{
		/** Utility functions for the notification */
		struct Local
		{
			static ECheckBoxState GetDontAskAgainCheckBoxState()
			{
				bool bSuppressNotification = false;
				GConfig->GetBool(TEXT("MacEditor"), TEXT("SuppressXcodeVersionWarningNotification"), bSuppressNotification, GEditorPerProjectIni);
				return bSuppressNotification ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
			{
				const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
				GConfig->SetBool(TEXT("MacEditor"), TEXT("SuppressXcodeVersionWarningNotification"), bSuppressNotification, GEditorPerProjectIni);
			}

			static void OnXcodeWarningNotificationDismissed()
			{
				TSharedPtr<SNotificationItem> NotificationItem = GXcodeWarningNotificationPtr.Pin();

				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
					NotificationItem->Fadeout();

					GXcodeWarningNotificationPtr.Reset();
				}
			}
		};

		const bool bIsXcodeInstalled = FPlatformMisc::GetXcodePath().Len() > 0;

		const ECheckBoxState DontAskAgainCheckBoxState = Local::GetDontAskAgainCheckBoxState();
		if (DontAskAgainCheckBoxState == ECheckBoxState::Unchecked)
		{
			const FText NoXcodeMessageText = LOCTEXT("XcodeNotInstalledWarningNotification", "Xcode was not detected on this Mac.\nMetal shader compilation will fall back to runtime compiled text shaders, which are slower.\nPlease install latest version of Xcode for best performance\nand make sure it's set as default using xcode-select tool.");
			const FText OldXcodeMessageText = LOCTEXT("OldXcodeVersionWarningNotification", "Xcode installed on this Mac is too old to be used for Metal shader compilation.\nFalling back to runtime compiled text shaders, which are slower.\nPlease update to latest version of Xcode for best performance\nand make sure it's set as default using xcode-select tool.");

			FNotificationInfo Info(bIsXcodeInstalled ? OldXcodeMessageText : NoXcodeMessageText);
			Info.bFireAndForget = false;
			Info.FadeOutDuration = 3.0f;
			Info.ExpireDuration = 0.0f;
			Info.bUseLargeFont = false;
			Info.bUseThrobber = false;

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("OK", "OK"), FText::GetEmpty(), FSimpleDelegate::CreateStatic(&Local::OnXcodeWarningNotificationDismissed)));

			Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&Local::GetDontAskAgainCheckBoxState);
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&Local::OnDontAskAgainCheckBoxStateChanged);
			Info.CheckBoxText = NSLOCTEXT("ModalDialogs", "DefaultCheckBoxMessage", "Don't show this again");

			GXcodeWarningNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
			GXcodeWarningNotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
#endif

	// Clean up the auto-load-in-progress file, if it exists. This file prevents auto-loading of projects and must be deleted here to indicate the load was successful
	const FString AutoLoadInProgressFilename = AutoLoadProjectFileName + TEXT(".InProgress");
	const bool bRequireExists = false;
	const bool bEvenIfReadOnly = true;
	const bool bQuiet = true;
	IFileManager::Get().Delete(*AutoLoadInProgressFilename, bRequireExists, bEvenIfReadOnly, bQuiet);
}

FString NetworkRemapPath_TestLevelScriptActor(const ALevelScriptActor* LevelScriptActor, const FString& AssetName, const FString& LevelPackageName, const FString& PathName, const FString& PrefixedPathName)
{
	FString ResultStr;

	UClass* LSAClass = LevelScriptActor ? LevelScriptActor->GetClass() : nullptr;

	if (LSAClass && LSAClass->GetName() == AssetName && LSAClass->GetOutermost()->GetName() != LevelPackageName)
	{
		ResultStr = PathName;
	}
	else
	{
		ResultStr = PrefixedPathName;
	}

	return ResultStr;
}

FORCEINLINE bool NetworkRemapPath_local(const FWorldContext* Context, FString& Str, bool bReading, bool bIsReplay)
{
	if (bReading)
	{
		CA_ASSUME(Context != nullptr);
		const UWorld* World = Context->World();
		if (World == nullptr)
		{
			return false;
		}

		if (bIsReplay && World->RemapCompiledScriptActor(Str))
		{
			return true;
		}
		
		if (FPackageName::IsShortPackageName(Str))
		{
			return false;
		}

		// First strip any source prefix, then add the appropriate prefix for this context
		FSoftObjectPath Path = UWorld::RemovePIEPrefix(Str);
		const int32 PIEInstance = Context->PIEInstance;
		if (bIsReplay)
		{
			const FString AssetName = Path.GetAssetName();

			FString PackageNameOnly = Path.GetLongPackageName();
			FPackageName::TryConvertFilenameToLongPackageName(PackageNameOnly, PackageNameOnly);
			const FString ShortName = FPackageName::GetShortName(PackageNameOnly);

			const FString PrefixedFullName = UWorld::ConvertToPIEPackageName(Str, PIEInstance);
			const FString PrefixedPackageName = UWorld::ConvertToPIEPackageName(PackageNameOnly, PIEInstance);
			const FString WorldPackageName = World->GetOutermost()->GetName();

			if (WorldPackageName == PrefixedPackageName)
			{
				Str = NetworkRemapPath_TestLevelScriptActor(World->GetLevelScriptActor(), AssetName, WorldPackageName, Path.ToString(), PrefixedFullName);
				return true;
			}

			for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
			{
				if (StreamingLevel != nullptr)
				{
					const FString StreamingLevelName = StreamingLevel->GetWorldAsset().GetLongPackageName();
					const FString LevelPackageName = StreamingLevel->GetWorldAssetPackageName();

					if (StreamingLevelName == PrefixedPackageName)
					{
						Str = NetworkRemapPath_TestLevelScriptActor(StreamingLevel->GetLevelScriptActor(), AssetName, LevelPackageName, Path.ToString(), PrefixedFullName);
						return true;
					}
				}
			}
			
			const bool bActorClass = FPackageName::IsValidObjectPath(Path.ToString()) && !AssetName.IsEmpty() && !ShortName.IsEmpty() && (AssetName == (ShortName + TEXT("_C")));
			if (!bActorClass)
			{
				Path.FixupForPIE(PIEInstance);
			}
		}
		else
		{
			Path.FixupForPIE(PIEInstance);
		}

		FString Remapped = Path.ToString();
		if (!Remapped.Equals(Str, ESearchCase::CaseSensitive))
		{
			Str = Remapped;
			return true;
		}
	}
	else
	{
		// When sending, strip prefix
		FString Remapped = UWorld::RemovePIEPrefix(Str);
		if (!Remapped.Equals(Str, ESearchCase::CaseSensitive))
		{
			Str = Remapped;
			return true;
		}
	}
	return false;
}

bool UEditorEngine::NetworkRemapPath(UNetConnection* Connection, FString& Str, bool bReading)
{
	if (bReading)
	{
		// Reading requires a valid world which we retrieve from the supplied connection
		if (Connection == nullptr || Connection->GetWorld() == nullptr)
		{
			return false;
		}

		const bool bIsReplay = Connection->IsReplay();
		const FWorldContext& Context = GetWorldContextFromWorldChecked(Connection->GetWorld());
		return NetworkRemapPath_local(&Context, Str, bReading, bIsReplay);
	}
	else
	{
		constexpr bool bIsReplay = false;
		const FWorldContext* Context = nullptr;
		return NetworkRemapPath_local(Context, Str, bReading, bIsReplay);
	}
}

bool UEditorEngine::NetworkRemapPath( UPendingNetGame *PendingNetGame, FString& Str, bool bReading)
{
	const FWorldContext& Context = GetWorldContextFromPendingNetGameChecked(PendingNetGame);
	return NetworkRemapPath_local(&Context, Str, bReading, PendingNetGame->GetDemoNetDriver() != nullptr);
}

void UEditorEngine::CheckAndHandleStaleWorldObjectReferences(FWorldContext* InWorldContext)
{
	// This does the same as UEngine::CheckAndHandleStaleWorldObjectReferences except it also allows Editor Worlds as a valid world.

	// All worlds at this point should be the CurrentWorld of some context or preview worlds.
	
	for( TObjectIterator<UWorld> It; It; ++It )
	{
		UWorld* World = *It;
		if (World->WorldType != EWorldType::EditorPreview && World->WorldType != EWorldType::Editor && World->WorldType != EWorldType::Inactive && World->WorldType != EWorldType::GamePreview)
		{
			TArray<UWorld*> OtherEditorWorlds;
			EditorLevelUtils::GetWorlds(EditorWorld, OtherEditorWorlds, true, false);
			if (OtherEditorWorlds.Contains(World))
				continue;

			bool ValidWorld = false;
			for (int32 idx=0; idx < WorldList.Num(); ++idx)
			{
				FWorldContext& WorldContext = WorldList[idx];

				if (const UWorld* TravelWorld = WorldContext.SeamlessTravelHandler.GetLoadedWorld())
				{
					TArray<UWorld*> TravelWorlds;
					EditorLevelUtils::GetWorlds((UWorld*)TravelWorld, TravelWorlds, true, false);

					if (TravelWorlds.Contains(World))
					{
						// World valid, but not loaded yet
						ValidWorld = true;
						break;
					}
				}
				
				if (WorldContext.World())
				{
					TArray<UWorld*> OtherWorlds;
					EditorLevelUtils::GetWorlds(WorldContext.World(), OtherWorlds, true, false);

					if (OtherWorlds.Contains(World))
					{
						// Some other context is referencing this 
						ValidWorld = true;
						break;
					}
				}
			}

			if (!ValidWorld)
			{
				UE_LOGF(LogLoad, Error, "Previously active world %ls not cleaned up by garbage collection!", *World->GetPathName());
				UE_LOGF(LogLoad, Error, "Once a world has become active, it cannot be reused and must be destroyed and reloaded. World referenced by:");
			
				// Only ensure because it could be an incorrectly setup preview world
				FReferenceChainSearch::FindAndPrintStaleReferencesToObject(World, (EPrintStaleReferencesOptions::Error | EPrintStaleReferencesOptions::Ensure));
			}
		}
	}
	
	if (InWorldContext)
	{
		for (FObjectKey Key : InWorldContext->GarbageObjectsToVerify)
		{
			if (UObject* Object = Key.ResolveObjectPtrEvenIfGarbage())
			{
				UE_LOGF(LogLoad, Error, "Object %ls not cleaned up by garbage collection!", *Object->GetPathName());
			
				FReferenceChainSearch::FindAndPrintStaleReferencesToObject(Object,
					UObjectBaseUtility::IsGarbageEliminationEnabled() ? EPrintStaleReferencesOptions::Fatal : (EPrintStaleReferencesOptions::Error | EPrintStaleReferencesOptions::Ensure));
			}
		}
		InWorldContext->GarbageObjectsToVerify.Reset();
	}
}

void UEditorEngine::UpdateIsVanillaProduct()
{
	// Check that we're running a content-only project through an installed build of the engine
	bool bResult = false;
	if (FApp::IsEngineInstalled() && !GameProjectUtils::ProjectHasCodeFiles())
	{
		// Check the build was installed by the launcher
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		FString Identifier = DesktopPlatform->GetCurrentEngineIdentifier();
		if (Identifier.Len() > 0)
		{
			TMap<FString, FString> Installations;
			DesktopPlatform->EnumerateLauncherEngineInstallations(Installations);
			if (Installations.Contains(Identifier))
			{
				bool bHasThirdPartyPlugin = false;

				FString TargetReceiptFilePath = FTargetReceipt::GetDefaultPath(*FPaths::EngineDir(), TEXT("UnrealEditor"),
													FPlatformProcess::GetBinariesSubdirectory(),
													FApp::GetBuildConfiguration(),
													FPlatformProcess::GetArchitectureSuffix());
				
				FTargetReceipt Receipt;
				if (Receipt.Read(*TargetReceiptFilePath))
				{
					// Check if we have any plugins that weren't part of the build
					for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
					{	
						if (Plugin->GetDescriptor().SupportsTargetPlatform(FPlatformMisc::GetUBTPlatform()) && Plugin->GetDescriptor().Modules.Num() > 0)
						{
							if (!Receipt.BuildPlugins.Contains(Plugin->GetName()))
							{
								// Check that the plugin has modules that supports the current target.
								// The plugin descriptor might say that it supports the current target but the module(s) might say otherwise.
								// In that case, we wouldn't find them in the BuildProducts but it doesn't mean that they are 3rd party plugins and they wouldn't impact the editor experience.
								bool bHasModuleInCurrentConfiguration = false;

								for (const FModuleDescriptor& ModuleDescriptor : Plugin->GetDescriptor().Modules)
								{
									bHasModuleInCurrentConfiguration = ModuleDescriptor.IsLoadedInCurrentConfiguration();
									if (bHasModuleInCurrentConfiguration)
									{
										break;
									}
								}

								// Check if we have build products for that plugin. Mainly for plugins that aren't enabled by default but would have had their binaries built.
								if (bHasModuleInCurrentConfiguration &&
									Algo::FindByPredicate(Receipt.BuildProducts, 
									[&Plugin](const FBuildProduct& BuildProduct)
									{
										// Look for the plugin .modules file
										return BuildProduct.Type == TEXT("RequiredResource") && FPaths::IsUnderDirectory(BuildProduct.Path, Plugin->GetBaseDir());
									}) == nullptr)
								{
									bHasThirdPartyPlugin = true;
									break;
								}
							}
						}
					}
				}
				else
				{
					// Can't validate the set of plugins, assume that this isn't a vanilla build
					bHasThirdPartyPlugin = true;
				}

				// If not, we're running Epic-only code.
				if (!bHasThirdPartyPlugin)
				{
					bResult = true;
				}
			}
		}
	}

	SetIsVanillaProduct(bResult);
}

void UEditorEngine::HandleBrowseToDefaultMapFailure(FWorldContext& Context, const FString& TextURL, const FString& Error)
{
	Super::HandleBrowseToDefaultMapFailure(Context, TextURL, Error);
	RequestEndPlayMap();
}

void UEditorEngine::TriggerStreamingDataRebuild()
{
	for (int32 WorldIndex = 0; WorldIndex < WorldList.Num(); ++WorldIndex)
	{
		UWorld* World = WorldList[WorldIndex].World();
		if (World && World->WorldType == EWorldType::Editor)
		{
			// Recalculate in a few seconds.
			World->TriggerStreamingDataRebuild();
		}
	}
}

FWorldContext& UEditorEngine::GetEditorWorldContext(bool bEnsureIsGWorld)
{
	for (int32 i=0; i < WorldList.Num(); ++i)
	{
		if (WorldList[i].WorldType == EWorldType::Editor)
		{
			ensure(!bEnsureIsGWorld || WorldList[i].World() == GWorld);
			return WorldList[i];
		}
	}

	check(false); // There should have already been one created in UEngine::Init
	return CreateNewWorldContext(EWorldType::Editor);
}

FWorldContext* UEditorEngine::GetPIEWorldContext(int32 WorldPIEInstance)
{
	for (FWorldContext& WorldContext : WorldList)
	{
		if (WorldContext.WorldType == EWorldType::PIE && WorldContext.PIEInstance == WorldPIEInstance)
		{
			return &WorldContext;
		}
	}

	return nullptr;
}

void UEditorEngine::OnAssetLoaded(UObject* Asset)
{
	UWorld* World = Cast<UWorld>(Asset);
	if (World)
	{
		// Init inactive worlds here instead of UWorld::PostLoad because it is illegal to call UpdateWorldComponents while IsRoutingPostLoad
		InitializeNewlyCreatedInactiveWorld(World);
	}
}

void UEditorEngine::OnAssetCreated(UObject* Asset)
{
	UWorld* World = Cast<UWorld>(Asset);
	if (World)
	{
		// Init inactive worlds here instead of UWorld::PostLoad because it is illegal to call UpdateWorldComponents while IsRoutingPostLoad
		InitializeNewlyCreatedInactiveWorld(World);
	}
}

void UEditorEngine::OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets)
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (AssetRegistry)
	{
		for (const FAssetCompileData& CompileData : CompiledAssets)
		{
			if (CompileData.Asset.IsValid())
			{
				AssetRegistry->AssetTagsFinalized(*CompileData.Asset);
			}
		}
	}
}

void UEditorEngine::HandleTimecodeProviderCompiled(UBlueprint* InBlueprint)
{
	ReinitializeTimecodeProvider();
}

void UEditorEngine::HandleCustomTimeStepCompiled(UBlueprint* InBlueprint)
{
	ReinitializeCustomTimeStep();
}

void UEditorEngine::RegisterTimecodeProviderCompiledDelegate()
{
	if (UTimecodeProvider* NewProvider = GetTimecodeProvider())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(NewProvider->GetClass()))
		{
			Blueprint->OnCompiled().Remove(TimecodeProviderCompiledDelegateHandle);
			TimecodeProviderCompiledDelegateHandle = Blueprint->OnCompiled().AddUObject(this, &UEditorEngine::HandleTimecodeProviderCompiled);
		}
	}
}

void UEditorEngine::RegisterCustomTimeStepCompiledDelegate()
{
	if (UEngineCustomTimeStep* NewCustomTimeStep = GetCustomTimeStep())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(NewCustomTimeStep->GetClass()))
		{
			Blueprint->OnCompiled().Remove(CustomTimeStepCompiledDelegateHandle);
			CustomTimeStepCompiledDelegateHandle = Blueprint->OnCompiled().AddUObject(this, &UEditorEngine::HandleCustomTimeStepCompiled);
		}
	}
}

void UEditorEngine::InitializeNewlyCreatedInactiveWorld(UWorld* World)
{
	check(World);

	if (!World->bIsWorldInitialized && World->WorldType == EWorldType::Inactive && !World->IsInstanced())
	{
		// Guard against dirtying packages while initializing the map
		TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);
		// This is probably no longer needed with the EditorLoadingPackage guard but doesn't hurt to keep for safety.
		const bool bOldDirtyState = World->GetOutermost()->IsDirty();

		// Make sure we have a navigation system if we are cooking the asset.
		// Typically nav bounds are added when AddNavigationSystemToWorld() is called from UEditorEngine::Map_Load().
		const bool bCooking = (IsCookByTheBookInEditorFinished() == false);

		// Create the world without a physics scene because creating too many physics scenes causes deadlock issues in PhysX. The scene will be created when it is opened in the level editor.
		// Also, don't create an FXSystem because it consumes too much video memory. This is also created when the level editor opens this world.
		// Do not create AISystem/Navigation for inactive world. These ones will also be created when the level editor opens this world. if required.
		World->InitWorld(GetEditorWorldInitializationValues()
			.CreatePhysicsScene(false)
			.CreateFXSystem(false)
			.CreateAISystem(false)
			.CreateNavigation(bCooking)
			);

		// Update components so the scene is populated
		World->UpdateWorldComponents(true, true);

		if (bCooking)
		{
			// When calling World->InitWorld() with bCreateNavigation=true (just above), 
			// it calls internally FNavigationSystem::AddNavigationSystemToWorld() with bInitializeForWorld=false.
			// That does not gather nav bounds. When cooking, the nav system and nav bounds are needed on the navmesh serialize-save for tiles to be added to the archive.
			// Also this call needs to occur after World->UpdateWorldComponents() else no bounds are found.
			FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::InferFromWorldMode);
		}

		// Need to restore the dirty state as registering components dirties the world
		if (!bOldDirtyState)
		{
			World->GetOutermost()->SetDirtyFlag(bOldDirtyState);
		}
	}
}

UWorld::InitializationValues UEditorEngine::GetEditorWorldInitializationValues() const
{
	return UWorld::InitializationValues()
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(true);
}

void UEditorEngine::HandleNetworkFailure(UWorld *World, UNetDriver *NetDriver, ENetworkFailure::Type FailureType, const FString& ErrorString)
{
	// If the failure occurred during PIE while connected to another process, simply end the PIE session before
	// trying to travel anywhere.
	if (PlayOnLocalPCSessions.Num() > 0)
	{
		for (const FWorldContext& WorldContext : WorldList)
		{
			if (WorldContext.WorldType == EWorldType::PIE && WorldContext.World() == World)
			{
				RequestEndPlayMap();
				return;
			}
		}
	}

	// Otherwise, perform normal engine failure handling.
	Super::HandleNetworkFailure(World, NetDriver, FailureType, ErrorString);
}

//////////////////////////////////////////////////////////////////////////
// FActorLabelUtilities

bool FActorLabelUtilities::SplitActorLabel(FString& InOutLabel, int32& OutIdx)
{
	// Look at the label and see if it ends in a number and separate them
	const TArray<TCHAR, FString::AllocatorType>& LabelCharArray = InOutLabel.GetCharArray();
	for (int32 CharIdx = LabelCharArray.Num() - 1; CharIdx >= 0; CharIdx--)
	{
		if (CharIdx == 0 || !FChar::IsDigit(LabelCharArray[CharIdx - 1]))
		{
			FString Idx = InOutLabel.RightChop(CharIdx);
			if (Idx.Len() > 0)
			{
				InOutLabel.LeftInline(CharIdx);
				OutIdx = FCString::Atoi(*Idx);
				return true;
			}
			break;
		}
	}
	return false;
}

void FActorLabelUtilities::SetActorLabelUnique(AActor* Actor, const FString& NewActorLabel, const FCachedActorLabels* InExistingActorLabels)
{
	check(Actor);

	FString Prefix = NewActorLabel;
	FString ModifiedActorLabel = NewActorLabel;
	int32   LabelIdx = 0;

	FCachedActorLabels ActorLabels;
	if (!InExistingActorLabels)
	{
		InExistingActorLabels = &ActorLabels;

		TSet<AActor*> IgnoreActors;
		IgnoreActors.Add(Actor);
		ActorLabels.Populate(Actor->GetWorld(), IgnoreActors);
	}


	if (InExistingActorLabels->Contains(ModifiedActorLabel))
	{
		// See if the current label ends in a number, and try to create a new label based on that
		if (!FActorLabelUtilities::SplitActorLabel(Prefix, LabelIdx))
		{
			// If there wasn't a number on there, append a number, starting from 2 (1 before incrementing below)
			LabelIdx = 1;
		}

		// Update the actor label until we find one that doesn't already exist
		while (InExistingActorLabels->Contains(ModifiedActorLabel))
		{
			++LabelIdx;
			ModifiedActorLabel = FString::Printf(TEXT("%s%d"), *Prefix, LabelIdx);
		}
	}

	Actor->SetActorLabel(ModifiedActorLabel);
}

void FActorLabelUtilities::RenameExistingActor(AActor* Actor, const FString& NewActorLabel, bool bMakeUnique)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	FSoftObjectPath OldPath = FSoftObjectPath(Actor);
	if (bMakeUnique)
	{
		SetActorLabelUnique(Actor, NewActorLabel, nullptr);
	}
	else
	{
		Actor->SetActorLabel(NewActorLabel);
	}
	FSoftObjectPath NewPath = FSoftObjectPath(Actor);

	if (OldPath != NewPath)
	{
		TArray<FAssetRenameData> RenameData;
		RenameData.Add(FAssetRenameData(OldPath, NewPath, true));
		AssetToolsModule.Get().RenameAssetsWithDialog(RenameData);
	}
}

void UEditorEngine::AutomationLoadMap(const FString& MapName, bool bForceReload, FString* OutError)
{
#if WITH_AUTOMATION_TESTS
	bool bLoadAsTemplate = false;
	bool bShowProgress = false;

	bool bNeedLoadEditorMap = true;
	bool bNeedPieStart = true;
	bool bPieRunning = false;

	//check existing worlds
	const TIndirectArray<FWorldContext> WorldContexts = GEngine->GetWorldContexts();
	for (auto& Context : WorldContexts)
	{
		if (Context.World())
		{
			FString WorldPackage = Context.World()->GetOutermost()->GetName();

			if (Context.WorldType == EWorldType::PIE)
			{
				//don't quit!  This was triggered while pie was already running!
				bNeedPieStart = MapName != UWorld::StripPIEPrefixFromPackageName(WorldPackage, Context.World()->StreamingLevelsPrefix);
				bPieRunning = true;
				break;
			}
			else if (Context.WorldType == EWorldType::Editor)
			{
				bNeedLoadEditorMap = MapName != WorldPackage;
			}
		}
	}

	if (bNeedLoadEditorMap || bForceReload)
	{
		if (bPieRunning)
		{
			EndPlayMap();
		}
		FEditorFileUtils::LoadMap(*MapName, bLoadAsTemplate, bShowProgress);
		bNeedPieStart = true;
	}

	if (bNeedPieStart)
	{
		UE_LOGF(LogEditor, Log, "Starting PIE for the automation tests for world, %ls", *GWorld->GetMapName());

		FRequestPlaySessionParams RequestParams;
		ULevelEditorPlaySettings* EditorPlaySettings = NewObject<ULevelEditorPlaySettings>();
		EditorPlaySettings->SetPlayNumberOfClients(1);
		EditorPlaySettings->bLaunchSeparateServer = false;
		RequestParams.EditorPlaySettings = EditorPlaySettings;
		// Keep EditorPlaySettings alive across the deferred latent command boundary - it lives inside
		// a non-UObject struct with no UPROPERTY, so the GC cannot reach it. Balanced by
		// RemoveFromRoot() in ~FStartPIEForAutomationCommand(), which always runs.
		EditorPlaySettings->AddToRoot();

		// Make sure the player start location is a valid location.
		if (CheckForPlayerStart() == nullptr)
		{
			FAutomationEditorCommonUtils::SetPlaySessionStartToActiveViewport(RequestParams);
		}

		// Wait for the editor world's navmesh to finish building before duplicating into PIE,
		// then start PIE and wait for the map to load (FStartPIEForAutomationCommand does both).
		ADD_LATENT_AUTOMATION_COMMAND(FWaitForEditorNavMeshBuildCommand);
		ADD_LATENT_AUTOMATION_COMMAND(FStartPIEForAutomationCommand(RequestParams));
	}
#endif
	return;
}

bool UEditorEngine::IsHMDTrackingAllowed() const
{
	// @todo vreditor: Added GEnableVREditorHacks check below to allow head movement in non-PIE editor; needs revisit
	return GEnableVREditorHacks || (PlayWorld && (IsVRPreviewActive() || GetDefault<ULevelEditorPlaySettings>()->ViewportGetsHMDControl));
}

void UEditorEngine::OnModuleCompileStarted(bool bIsAsyncCompile)
{
	bIsCompiling = true;
}

void UEditorEngine::OnModuleCompileFinished(const FString& CompilationOutput, ECompilationResult::Type CompilationResult, bool bShowLog)
{
	bIsCompiling = false;
}

bool UEditorEngine::IsEditorShaderPlatformEmulated(UWorld* World)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(World->GetFeatureLevel());

	bool bIsSimulated = IsSimulatedPlatform(ShaderPlatform);

	return bIsSimulated;
}

bool UEditorEngine::IsOfflineShaderCompilerAvailable(UWorld* World)
{
	const auto ShaderPlatform = GetFeatureLevelShaderPlatform(World->GetFeatureLevel());

	const auto RealPlatform = GetSimulatedPlatform(ShaderPlatform);

	return FMaterialStatsUtils::IsPlatformOfflineCompilerAvailable(RealPlatform);
}

void UEditorEngine::OnSceneMaterialsModified()
{
	SceneMaterialsModifiedEvent.Broadcast();
}

void UEditorEngine::OnEffectivePreviewShaderPlatformChange()
{
	if (XRSystem.IsValid() && StereoRenderingDevice.IsValid())
	{
		IStereoRenderTargetManager* StereoRenderTargetManager = StereoRenderingDevice->GetRenderTargetManager();
		if (StereoRenderTargetManager)
		{
			StereoRenderTargetManager->ReconfigureForShaderPlatform(
				PreviewPlatform.bPreviewShaderPlatformActive ? PreviewPlatform.ShaderPlatform : GMaxRHIShaderPlatform);
		}
	}
}

static void SaveShaderPlatformAsDisabled(FPreviewPlatformInfo& PreviewPlatform)
{
	auto* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();

	Settings->PreviewFeatureLevel = 0;
	Settings->PreviewPlatformName = NAME_None;
	Settings->PreviewShaderFormatName = NAME_None;
	Settings->bPreviewShaderPlatformActive = false;
	Settings->bPreviewFeatureLevelWasDefault = true;
	Settings->PreviewDeviceProfileName = NAME_None;
	Settings->PreviewShaderPlatformName = NAME_None;
	Settings->PreviewShaderPlatformFriendlyName = FText();

	Settings->SaveConfig();

	Settings->PreviewFeatureLevel = (int32)PreviewPlatform.PreviewFeatureLevel;
	Settings->PreviewPlatformName = PreviewPlatform.PreviewPlatformName;
	Settings->PreviewShaderFormatName = PreviewPlatform.PreviewShaderFormatName;
	Settings->bPreviewShaderPlatformActive = PreviewPlatform.bPreviewShaderPlatformActive;
	Settings->bPreviewFeatureLevelWasDefault = (PreviewPlatform.PreviewFeatureLevel == GMaxRHIFeatureLevel);
	Settings->PreviewDeviceProfileName = PreviewPlatform.DeviceProfileName;
	Settings->PreviewShaderPlatformName = PreviewPlatform.PreviewShaderPlatformName;
	Settings->PreviewShaderPlatformFriendlyName = PreviewPlatform.GetFriendlyName();
}

static void PreviewPlatformChangePre()
{
	// We're changing global platform config; make sure async ops are not running

	// Wait for pending load requests.
	FlushAsyncLoading();

	// Need to block on all in-progress asset builds as we may be changing their config	
	FAssetCompilingManager::Get().FinishAllCompilation();
	
	// Stop any texture streaming in progress :
	UTexture::CancelPendingTextureStreaming();

	// Clear the Slate renderer's cached scene pointers before we recreate scenes.
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReleaseAccessedResources(false /* bImmediatelyFlush */);
	}

	FlushRenderingCommands();
}

extern ENGINE_API bool GDisableAutomaticTextureMaterialUpdateDependencies;
static void PreviewPlatformChangePost()
{
	// scope for RegisterTextureLODSettings ; @todo CBTLDP : remove when possible
	{
		// Need to update the cached TextureLODSettings in TargetPlatform with RegisterTextureLODSettings

		UDeviceProfile * ADP = UDeviceProfileManager::Get().GetActiveProfile();
		const UTextureLODSettings * TLS_ADP = ADP->GetTextureLODSettings();

		ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
		ITargetPlatform* TargetPlatform = TPM->GetRunningTargetPlatform();
		
		UDeviceProfile * CDP = UDeviceProfileManager::Get().FindProfile(TargetPlatform->CookingDeviceProfileName());
		UTextureLODSettings * TLS_CDP = CDP->GetTextureLODSettings();

		// typically ADP == CDP , but not always
		// when different, TextureLOD should come from CDP
		// if PreviewPlatform is enabled, then BOTH ADP and CDP should return the override

		TargetPlatform->RegisterTextureLODSettings(TLS_CDP);

		// after Register, TP->GetTextureLODSettings is equal to the CDP way
	}

	for (TObjectIterator<UWorld> It; It; ++It)
	{
		if (It->Scene)
		{
			It->Scene->UpdateEarlyZPassMode();
		}
	}

	{
		// Materials are already updated by SetPreviewPlatform, so disable the automatic per-texture material dependency update
		TGuardValue<bool> Guard(GDisableAutomaticTextureMaterialUpdateDependencies, true);

		// TextureLODSettings and platform cvars may have changed
		//	need to notify all textures
		//	don't do this until all changes are done
		// We didn't do a PreEdit on all textures; rather we did a system-wide Wait at the start of this function
	
		for( TObjectIterator<UTexture> It; It; ++It )
		{
			UTexture* Texture = *It;
			if ( Texture->GetTextureClass() == ETextureClass::RenderTarget )
			{
				// @todo CBTLDP : why?
				continue;
			}

			Texture->PostEditChange();
		}
	}
}

void UEditorEngine::SetPreviewPlatform(const FPreviewPlatformInfo& NewPreviewPlatform, bool bSaveSettings, bool bCompileShaders)
{
	// Get the requested preview platform, make sure it is valid.
	EShaderPlatform ShaderPlatform = NewPreviewPlatform.ShaderPlatform;
	if (ShaderPlatform == GMaxRHIShaderPlatform)
	{
		UE_LOGF(LogEditor, Display, "Disable Preview Platform");
	}
	else
	{
		UE_LOGF(LogEditor, Display, "Set Preview Platform: %ls (CompileShaders=%ls)",
			*NewPreviewPlatform.GetFriendlyName().ToString(),
			bCompileShaders ? TEXT("true") : TEXT("false"));
	}

	{
		FScopedSlowTask SlowTask(100.f, NSLOCTEXT("Engine", "ChangingPreviewPlatform", "Changing Preview Platform"), true);
		SlowTask.Visibility = ESlowTaskVisibility::ForceVisible;
		SlowTask.MakeDialog(/*bShowCancelButton=*/false, /*bAllowInPIE=*/true);

		PreviewPlatformChangePre();

		check(FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform));
		ERHIFeatureLevel::Type MaxFeatureLevel = NewPreviewPlatform.PreviewShaderFormatName != NAME_None ? (ERHIFeatureLevel::Type)GetMaxSupportedFeatureLevel(ShaderPlatform) : ERHIFeatureLevel::SM5;
		check(NewPreviewPlatform.PreviewShaderFormatName.IsNone() || MaxFeatureLevel == NewPreviewPlatform.PreviewFeatureLevel);

		const bool bChangedPreviewShaderPlatform = NewPreviewPlatform.ShaderPlatform != PreviewPlatform.ShaderPlatform;
		const bool bChangedEffectiveShaderPlatform = bChangedPreviewShaderPlatform
			? (PreviewPlatform.bPreviewShaderPlatformActive || NewPreviewPlatform.bPreviewShaderPlatformActive)
			: (NewPreviewPlatform.bPreviewShaderPlatformActive != PreviewPlatform.bPreviewShaderPlatformActive);
		const ERHIFeatureLevel::Type EffectiveFeatureLevel = NewPreviewPlatform.GetEffectivePreviewFeatureLevel();

		if (NewPreviewPlatform.PreviewShaderFormatName != NAME_None)
		{
			// Force generation of the autogen files if they don't already exist
			FShaderCompileUtilities::GenerateBrdfHeaders(ShaderPlatform);
		}

		// Record the new preview platform
		PreviewPlatform = NewPreviewPlatform;

		// Initially set preview as disabled in case shader compilation crashes
		if (bCompileShaders)
		{
			SaveShaderPlatformAsDisabled(PreviewPlatform);
		}

		// If we changed the preview platform, we need to update the material quality settings
		if (bChangedPreviewShaderPlatform)
		{
			UMaterialShaderQualitySettings* MaterialShaderQualitySettings = UMaterialShaderQualitySettings::Get();
			MaterialShaderQualitySettings->SetPreviewPlatform(PreviewPlatform.PreviewShaderPlatformName);
		}

		// Update any PerPlatformConfig class defaults or instances
		for (FThreadSafeObjectIterator ObjIterator(UObject::StaticClass(), RF_NoFlags); ObjIterator; ++ObjIterator)
		{
			if ((*ObjIterator) && ObjIterator->GetClass()->HasAnyClassFlags(CLASS_PerPlatformConfig))
			{
				ObjIterator->LoadConfig();
			}
		}

		// Restore device profile to defaults, then apply the preview profile
		// bRestore=true resets CVars first (needed for compile path to get a clean state).
		// bRestore=false skips the reset (toggle path avoids double CVar sink flush).
		auto UpdateDeviceProfile = [&](bool bRestore)
		{
			// Always restore if no preview profile will be set, or if explicitly requested (compile path)
			bRestore = bRestore || !PreviewPlatform.bPreviewShaderPlatformActive || PreviewPlatform.DeviceProfileName == NAME_None;

			if (bRestore)
			{
				UDeviceProfileManager::Get().RestorePreviewDeviceProfile();
			}

			if (PreviewPlatform.bPreviewShaderPlatformActive && PreviewPlatform.DeviceProfileName != NAME_None)
			{
				if (UDeviceProfile* DP = UDeviceProfileManager::Get().FindProfile(PreviewPlatform.DeviceProfileName.ToString(), false))
				{
					UDeviceProfileManager::Get().SetPreviewDeviceProfile(DP);
				}
			}
		};

		// Full platform change: re-register components, compile shaders
		if (bCompileShaders)
		{
			constexpr bool bUpdateProgressDialog = true;
			constexpr bool bCacheAllRemainingShaders = false;

			//invalidate global bound shader states so they will be created with the new shaders the next time they are set (in SetGlobalBoundShaderState)
			for (TLinkedList<FGlobalBoundShaderStateResource*>::TIterator It(FGlobalBoundShaderStateResource::GetGlobalBoundShaderStateList()); It; It.Next())
			{
				BeginUpdateResourceRHI(*It);
			}

			FGlobalComponentReregisterContext RecreateComponents;
			FlushRenderingCommands();

			// Set only require the preview feature level and the max feature level. The Max feature level is required for the toggle feature.
			for (uint32 i = (uint32)ERHIFeatureLevel::ES3_1; i < (uint32)ERHIFeatureLevel::Num; i++)
			{
				ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)i;
				UMaterialInterface::SetGlobalRequiredFeatureLevel(FeatureLevel, FeatureLevel == PreviewPlatform.PreviewFeatureLevel || FeatureLevel == GMaxRHIFeatureLevel);
			}

			// Restore editor's shader platform in case a previous SM6 preview left it stale
			GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel] = GMaxRHIShaderPlatform;
			if (PreviewPlatform.bPreviewShaderPlatformActive)
			{
				GShaderPlatformForFeatureLevel[PreviewPlatform.PreviewFeatureLevel] = ShaderPlatform;
			}

			SlowTask.EnterProgressFrame(35.0f);
			UMaterial::AllMaterialsCacheResourceShadersForRendering(bUpdateProgressDialog, bCacheAllRemainingShaders);

			SlowTask.EnterProgressFrame(35.0f);
			UMaterialInstance::AllMaterialsCacheResourceShadersForRendering(bUpdateProgressDialog, bCacheAllRemainingShaders);

			SlowTask.EnterProgressFrame(15.0f, NSLOCTEXT("Engine", "SlowTaskGlobalShaderMapMessage", "Compiling global shaders"));
			CompileGlobalShaderMap(PreviewPlatform.PreviewFeatureLevel);

			SlowTask.EnterProgressFrame(15.0f, NSLOCTEXT("Engine", "SlowTaskFinalizingMessage", "Finalizing"));
			GShaderCompilingManager->ProcessAsyncResults(false, true);

			// Update device profiles after shaders are compiled 
			UpdateDeviceProfile(/*bRestore=*/true);
		}
		else
		{
			UpdateDeviceProfile(/*bRestore=*/false);
		}

		DefaultWorldFeatureLevel = EffectiveFeatureLevel;
		PreviewShaderPlatformChanged.Broadcast(GShaderPlatformForFeatureLevel[EffectiveFeatureLevel]);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PreviewFeatureLevelChanged.Broadcast(EffectiveFeatureLevel);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Scalability::ChangeScalabilityPreviewPlatform(PreviewPlatform.GetEffectivePreviewPlatformName(), GetActiveShaderPlatform());
		Scalability::ApplyCachedQualityLevels();

		if (bChangedEffectiveShaderPlatform)
		{
			OnEffectivePreviewShaderPlatformChange();
		}

		UStaticMesh::OnLodStrippingQualityLevelChanged(nullptr);

		PreviewPlatformChangePost();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PreviewPlatformChanged.Broadcast();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		RedrawAllViewports();

		if (bSaveSettings)
		{
			SaveEditorShaderPlatformPreview();
		}
	}
	UE_LOGF(LogEditor, Display, "Done with Preview Platform");
}

void UEditorEngine::TogglePreviewPlatform()
{
	FPreviewPlatformInfo NewInfo = PreviewPlatform;
	NewInfo.bPreviewShaderPlatformActive ^= 1;

	// When the preview shares the editor's feature level, toggling changes GShaderPlatformForFeatureLevel which requires the compile path.
	const bool bNeedsCompile = (NewInfo.PreviewFeatureLevel == GMaxRHIFeatureLevel);
	if (bNeedsCompile)
	{
		UE_LOGF(LogEditor, Display, "Toggle requires shader compilation: preview feature level matches editor feature level (GMaxRHIFeatureLevel)");
	}

	SetPreviewPlatform(NewInfo, /*bSaveSettings=*/true, /*bCompileShaders=*/bNeedsCompile);
}

bool UEditorEngine::IsPreviewPlatformEnabled() const
{
	return PreviewPlatform.PreviewShaderFormatName != NAME_None || PreviewPlatform.PreviewShaderPlatformName != NAME_None;
}

bool UEditorEngine::IsPreviewPlatformActive() const
{
 	return PreviewPlatform.bPreviewShaderPlatformActive;
}

EShaderPlatform UEditorEngine::GetActiveShaderPlatform() const
{
	if (PreviewPlatform.bPreviewShaderPlatformActive)
	{
		return GShaderPlatformForFeatureLevel[PreviewPlatform.PreviewFeatureLevel];
	}
	else
	{
		return GMaxRHIShaderPlatform;
	}
}

FText UEditorEngine::GetActiveShaderPlatformFriendlyName() const
{
	if (PreviewPlatform.bPreviewShaderPlatformActive)
	{
		return PreviewPlatform.GetFriendlyName();
	}
	else
	{
		return FDataDrivenShaderPlatformInfo::GetFriendlyName(GMaxRHIShaderPlatform);
	}
}

ERHIFeatureLevel::Type UEditorEngine::GetActiveFeatureLevelPreviewType() const
{
	return PreviewPlatform.bPreviewShaderPlatformActive ? PreviewPlatform.PreviewFeatureLevel : GMaxRHIFeatureLevel;
}

// Deprecated in 5.7
void UEditorEngine::LoadEditorFeatureLevel()
{
	LoadEditorShaderPlatformPreview();
}

void UEditorEngine::LoadEditorShaderPlatformPreview()
{
	auto* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();

	EShaderPlatform ShaderPlatformToPreview = FDataDrivenShaderPlatformInfo::GetShaderPlatformFromName(Settings->PreviewShaderPlatformName);

	if (Settings->PreviewFeatureLevel >= 0 && Settings->PreviewFeatureLevel < (int32)ERHIFeatureLevel::Num && ShaderPlatformToPreview < EShaderPlatform::SP_NumPlatforms)
	{
		// Try to map a saved ShaderFormatName to the PreviewPlatformName using ITargetPlatform if we don't have one. 
		// We now store the PreviewPlatformName explicitly to support preview for platforms we don't have an ITargetPlatform of.
		if (Settings->PreviewPlatformName == NAME_None && Settings->PreviewShaderFormatName != NAME_None)
		{
			const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatformWithSupport(TEXT("ShaderFormat"), Settings->PreviewShaderFormatName);
			if (TargetPlatform)
			{
				Settings->PreviewPlatformName = FName(*TargetPlatform->IniPlatformName());
			}
		}

		FName PreviewShaderPlatformName = Settings->PreviewShaderPlatformName;
		// If we have an old .ini file in Saved we just will use what is the SP of the current FeatureLevel
		if (Settings->PreviewShaderPlatformName == NAME_None)
		{
			PreviewShaderPlatformName = *(LexToString(GShaderPlatformForFeatureLevel[Settings->PreviewFeatureLevel]));
		}

		UE_LOGF(LogExit, Display, "LoadEditorShaderPlatformPreview: %ls %ls", *PreviewShaderPlatformName.ToString(), *Settings->PreviewPlatformName.ToString() );

		SetPreviewPlatform(FPreviewPlatformInfo(
			(ERHIFeatureLevel::Type)Settings->PreviewFeatureLevel, 
			ShaderPlatformToPreview, 
			Settings->PreviewPlatformName, 
			Settings->PreviewShaderFormatName, 
			Settings->PreviewDeviceProfileName, 
			Settings->bPreviewShaderPlatformActive, 
			PreviewShaderPlatformName, 
			Settings->PreviewShaderPlatformFriendlyName,
			Settings->PreviewConstrainedAspectRatio,
			Settings->PreviewSafeZones),
			false);
	}
}

void UEditorEngine::SaveEditorShaderPlatformPreview()
{
	auto* Settings = GetMutableDefault<UEditorPerProjectUserSettings>();
	Settings->PreviewFeatureLevel = (int32)PreviewPlatform.PreviewFeatureLevel;
	Settings->PreviewPlatformName = PreviewPlatform.PreviewPlatformName;
	Settings->PreviewShaderFormatName = PreviewPlatform.PreviewShaderFormatName;
	Settings->bPreviewShaderPlatformActive = PreviewPlatform.bPreviewShaderPlatformActive;
	Settings->bPreviewFeatureLevelWasDefault = (PreviewPlatform.PreviewFeatureLevel == GMaxRHIFeatureLevel);
	Settings->PreviewDeviceProfileName = PreviewPlatform.DeviceProfileName;
	Settings->PreviewShaderPlatformName = PreviewPlatform.PreviewShaderPlatformName;
	Settings->PreviewShaderPlatformFriendlyName = PreviewPlatform.GetFriendlyName();
	Settings->PreviewConstrainedAspectRatio = PreviewPlatform.GetConstrainedAspectRatio();
	Settings->PreviewSafeZones = PreviewPlatform.GetSafeZones();
	Settings->PostEditChange();
}

bool UEditorEngine::GetPreviewPlatformName(FName& PlatformName) const
{
	FName PreviewPlatformName = PreviewPlatform.GetEffectivePreviewPlatformName();
	if (PreviewPlatformName != NAME_None)
	{
		PlatformName = PreviewPlatformName;
		return true;
	}

	return false;
}

ULevelEditorDragDropHandler* UEditorEngine::GetLevelEditorDragDropHandler() const
{
	if (DragDropHandler == nullptr)
	{
		if (OnCreateLevelEditorDragDropHandlerDelegate.IsBound())
		{
			DragDropHandler = OnCreateLevelEditorDragDropHandlerDelegate.Execute();
		}
		else
		{
			DragDropHandler = NewObject<ULevelEditorDragDropHandler>(const_cast<UEditorEngine*>(this));
		}
	}

	return DragDropHandler;
}

namespace
{
	class FProjectExternalContentDefault : public IProjectExternalContentInterface
	{
	private:
		bool IsEnabled() const override { return false; }
		void AddExternalContent(const UE::Core::FVersePath& VersePath, TSharedPtr<IPlugin> ReferencingPlugin = {}, FAddExternalContentComplete CompleteCallback = {}) override
		{
			CompleteCallback.ExecuteIfBound(false, /*Plugins=*/{});
		}
	};

	FProjectExternalContentDefault ProjectExternalContentDefault;
}

IProjectExternalContentInterface* UEditorEngine::GetProjectExternalContentInterface()
{
	IProjectExternalContentInterface* ProjectExternalContentInterface = ProjectExternalContentInterfaceGetter.IsBound() ? ProjectExternalContentInterfaceGetter.Execute() : &ProjectExternalContentDefault;
	check(ProjectExternalContentInterface);
	return ProjectExternalContentInterface;
}

#undef LOCTEXT_NAMESPACE
