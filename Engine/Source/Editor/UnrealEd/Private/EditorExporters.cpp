// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorExporters.cpp: Editor exporters.
=============================================================================*/

#include "CoreMinimal.h"
#include "Animation/Skeleton.h"
#include "Audio.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceFile.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/TextBuffer.h"
#include "UObject/Package.h"
#include "Engine/EngineTypes.h"
#include "Engine/MaterialMerging.h"
#include "GameFramework/Actor.h"
#include "SceneTypes.h"
#include "RawIndexBuffer.h"
#include "RenderingThread.h"
#include "AI/NavigationSystemBase.h"
#include "Model.h"
#include "Exporters/Exporter.h"
#include "Exporters/AnimSequenceExporterFBX.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimSequence.h"
#include "Editor/EditorEngine.h"
#include "Exporters/ExportTextContainer.h"
#include "Editor/GroupActor.h"
#include "Exporters/LevelExporterFBX.h"
#include "Exporters/LevelExporterLOD.h"
#include "Exporters/LevelExporterOBJ.h"
#include "Exporters/LevelExporterSTL.h"
#include "Exporters/LevelExporterT3D.h"
#include "Exporters/ModelExporterT3D.h"
#include "Exporters/ObjectExporterT3D.h"
#include "Exporters/PolysExporterOBJ.h"
#include "Exporters/PolysExporterT3D.h"
#include "Exporters/SequenceExporterT3D.h"
#include "Exporters/SkeletalMeshExporterFBX.h"
#include "Exporters/SoundExporterOGG.h"
#include "Exporters/SoundExporterWAV.h"
#include "Exporters/SoundSurroundExporterWAV.h"
#include "Exporters/StaticMeshExporterFBX.h"
#include "Exporters/StaticMeshExporterOBJ.h"
#include "Exporters/TextBufferExporterTXT.h"
#include "Exporters/FbxExportOption.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProcedural.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Polys.h"
#include "Misc/FeedbackContext.h"
#include "UObject/PropertyPortFlags.h"
#include "GameFramework/PhysicsVolume.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "FbxExporter.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "MaterialUtilities.h"
#include "MaterialShared.h"
#include "InstancedFoliageActor.h"
#include "LandscapeProxy.h"
#include "Landscape.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "UnrealExporter.h"
#include "InstancedFoliage.h"
#include "Engine/Selection.h"
#include "AssetExportTask.h"
#include "IMaterialBakingModule.h"
#include "MaterialBakingStructures.h"
#include "MaterialOptions.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "Components/SkeletalMeshComponent.h"
#include "Camera/CameraComponent.h"
#include "Components/LightComponent.h"
#include "Particles/Emitter.h"
#include "Engine/Brush.h"
#include "Components/BrushComponent.h"
#include "Exporters/ActorExporterT3D.h"

#include "Misc/App.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorExporters, Log, All);

namespace Private
{
	bool bLevelExportT3D_UseActorExporters = false;
	static FAutoConsoleVariableRef CVarDumpHandlerTick(
		TEXT("UnrealEd.T3DUseActorExporters"),
		bLevelExportT3D_UseActorExporters,
		TEXT(" \n"),
		ECVF_Default
	);
}

struct FScopedFbxExporterInstance
{
	FScopedFbxExporterInstance()
	{
		Exporter = UnFbx::FFbxExporter::GetInstance();
	}
	~FScopedFbxExporterInstance()
	{
		UnFbx::FFbxExporter::DeleteInstance();
		Exporter = nullptr;
	}

	UnFbx::FFbxExporter* GetExporter() { return Exporter; }
private:
	UnFbx::FFbxExporter* Exporter = nullptr;
};

/*------------------------------------------------------------------------------
	UTextBufferExporterTXT implementation.
------------------------------------------------------------------------------*/
UTextBufferExporterTXT::UTextBufferExporterTXT(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTextBuffer::StaticClass();
	FormatExtension.Add(TEXT("TXT"));
	PreferredFormatIndex = 0;
	FormatDescription.Add(TEXT("Text file"));
	bText = true;
}

bool UTextBufferExporterTXT::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	UTextBuffer* TextBuffer = CastChecked<UTextBuffer>( Object );
	FString Str( TextBuffer->GetText() );

	TCHAR* Start = const_cast<TCHAR*>(*Str);
	TCHAR* End   = Start + Str.Len();
	while( Start<End && (Start[0]=='\r' || Start[0]=='\n' || Start[0]==' ') )
		Start++;
	while( End>Start && (End [-1]=='\r' || End [-1]=='\n' || End [-1]==' ') )
		End--;
	*End = 0;

	Ar.Log( Start );

	return 1;
}

/*------------------------------------------------------------------------------
	USoundExporterWAV implementation.
------------------------------------------------------------------------------*/
USoundExporterWAV::USoundExporterWAV(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = USoundWave::StaticClass();
	bText = false;
	FormatDescription.Add(TEXT("Sound"));
	FormatExtension.Add(TEXT("WAV"));

}

bool USoundExporterWAV::SupportsObject(UObject* Object) const
{
	bool bSupportsObject = false;
	if (Super::SupportsObject(Object))
	{
		USoundWave* SoundWave = CastChecked<USoundWave>(Object);
		if (!SoundWave->IsA<USoundWaveProcedural>())
		{
			bSupportsObject = (SoundWave->NumChannels <= 2);
		}
	}
	return bSupportsObject;
}

bool USoundExporterWAV::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	USoundWave* Sound = CastChecked<USoundWave>( Object );

	// We a non-const pointer for the sake of calling serialize, but this is on writing FArchive so it's never modified
	// Assert that's the case
	if(ensure(Ar.IsSaving()))
	{	
		TFuture<FSharedBuffer> FutureBuffer = Sound->RawData.GetPayload(USoundWave::FEditorAudioBulkData::EPayloadFlags::ReturnCopyWithTransformationData); // This will block, but for now that's ok.
		void* RawWaveData = (void*)FutureBuffer.Get().GetData();
		Ar.Serialize(RawWaveData, FutureBuffer.Get().GetSize());
	}
	return true;
}

/*------------------------------------------------------------------------------
	USoundExporterOGG implementation.
------------------------------------------------------------------------------*/
USoundExporterOGG::USoundExporterOGG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = USoundWave::StaticClass();
	bText = false;
	FormatDescription.Add(TEXT("Sound"));

	FormatExtension.Add(TEXT("OGG"));

}

bool USoundExporterOGG::SupportsObject(UObject* Object) const
{
	bool bSupportsObject = false;
	if (Super::SupportsObject(Object))
	{
		USoundWave* SoundWave = CastChecked<USoundWave>(Object);
		bSupportsObject = (SoundWave->GetCompressedData("OGG") != NULL);
	}
	return bSupportsObject;
}


bool USoundExporterOGG::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	USoundWave* Sound = CastChecked<USoundWave>( Object );

	FByteBulkData* Bulk = Sound->GetCompressedData("OGG");
	if (Bulk)
	{
		Ar.Serialize(Bulk->Lock(LOCK_READ_ONLY), Bulk->GetBulkDataSize());
		Bulk->Unlock();
		return true;
	}

	return false;
}

/*------------------------------------------------------------------------------
	USoundSurroundExporterWAV implementation.
------------------------------------------------------------------------------*/
USoundSurroundExporterWAV::USoundSurroundExporterWAV(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	
	SupportedClass = USoundWave::StaticClass();
	bText = false;
	FormatExtension.Add(TEXT("WAV"));
	FormatDescription.Add(TEXT("Multichannel Sound"));

}

bool USoundSurroundExporterWAV::SupportsObject(UObject* Object) const
{
	bool bSupportsObject = false;
	if (Super::SupportsObject(Object))
	{
		USoundWave* SoundWave = CastChecked<USoundWave>(Object);
		bSupportsObject = (SoundWave->NumChannels > 2);
	}
	return bSupportsObject;
}

int32 USoundSurroundExporterWAV::GetFileCount( UObject* Object ) const
{
	return( SPEAKER_Count );
}

FString USoundSurroundExporterWAV::GetUniqueFilename( UObject* Object, const TCHAR* Filename, int32 FileIndex, int32 FileCount ) const
{
	static FString SpeakerLocations[SPEAKER_Count] =
	{
		TEXT( "_fl" ),			// SPEAKER_FrontLeft
		TEXT( "_fr" ),			// SPEAKER_FrontRight
		TEXT( "_fc" ),			// SPEAKER_FrontCenter
		TEXT( "_lf" ),			// SPEAKER_LowFrequency
		TEXT( "_sl" ),			// SPEAKER_SideLeft
		TEXT( "_sr" ),			// SPEAKER_SideRight
		TEXT( "_bl" ),			// SPEAKER_BackLeft
		TEXT( "_br" )			// SPEAKER_BackRight
	};

	FString ReturnName = FPaths::GetBaseFilename( Filename, false ) + SpeakerLocations[FileIndex] + FString( ".WAV" );

	return( ReturnName );
}

bool USoundSurroundExporterWAV::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	bool bResult = false;

	USoundWave* Sound = CastChecked<USoundWave>( Object );
	if ( Sound->ChannelSizes.Num() > 0 )
	{		
		if (ensure(Ar.IsSaving()))
		{
			// Non-const pointer for the sake of calling serialize, so assert that we are writing an archive.
			TFuture<FSharedBuffer> FutureBuffer = Sound->RawData.GetPayload(USoundWave::FEditorAudioBulkData::EPayloadFlags::ReturnCopyWithTransformationData);
			uint8* RawWaveData = (uint8*) FutureBuffer.Get().GetData();

			if( Sound->ChannelSizes[ FileIndex ] )
			{
				Ar.Serialize( RawWaveData + Sound->ChannelOffsets[ FileIndex ], Sound->ChannelSizes[ FileIndex ] );
			}
		}

		bResult = Sound->ChannelSizes[ FileIndex ] != 0;
	}

	return bResult;
}

/*------------------------------------------------------------------------------
	UObjectExporterT3D implementation.
------------------------------------------------------------------------------*/
UObjectExporterT3D::UObjectExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UObject::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("T3D"));
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal object text"));
	FormatDescription.Add(TEXT("Unreal object text"));
}

bool UObjectExporterT3D::ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags )
{
	EmitBeginObject(Ar, Object, PortFlags);
		ExportObjectInner( Context, Object, Ar, PortFlags);
	EmitEndObject(Ar);

	return true;
}

/*------------------------------------------------------------------------------
	UPolysExporterT3D implementation.
------------------------------------------------------------------------------*/
UPolysExporterT3D::UPolysExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UPolys::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("T3D"));
	FormatDescription.Add(TEXT("Unreal poly text"));

}

bool UPolysExporterT3D::ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags )
{
	UPolys* Polys = CastChecked<UPolys>( Object );

	Ar.Logf( TEXT("%sBegin PolyList\r\n"), FCString::Spc(TextIndent) );
	for( int32 i=0; i<Polys->Element.Num(); i++ )
	{
		FPoly* Poly = &Polys->Element[i];
		TCHAR TempStr[MAX_SPRINTF]=TEXT("");

		// Start of polygon plus group/item name if applicable.
		// The default values need to jive FPoly::Init().
		Ar.Logf( TEXT("%s   Begin Polygon"), FCString::Spc(TextIndent) );
		if( Poly->ItemName != NAME_None )
		{
			Ar.Logf( TEXT(" Item=%s"), *Poly->ItemName.ToString() );
		}
		if( Poly->Material )
		{
			Ar.Logf( TEXT(" Texture=%s"), *Poly->Material->GetPathName() );
		}
		if( Poly->PolyFlags != 0 )
		{
			Ar.Logf( TEXT(" Flags=%i"), Poly->PolyFlags );
		}
		if( Poly->iLink != INDEX_NONE )
		{
			Ar.Logf( TEXT(" Link=%i"), Poly->iLink );
		}
		if ( Poly->LightMapScale != 32.0f )
		{
			Ar.Logf( TEXT(" LightMapScale=%f"), Poly->LightMapScale );
		}
		Ar.Logf( TEXT("\r\n") );

		// All coordinates.
		FVector Base(Poly->Base), Normal(Poly->Normal), TextureU(Poly->TextureU), TextureV(Poly->TextureV);
		Ar.Logf( TEXT("%s      Origin   %s\r\n"), FCString::Spc(TextIndent), SetFVECTOR(TempStr,&Base) );
		Ar.Logf( TEXT("%s      Normal   %s\r\n"), FCString::Spc(TextIndent), SetFVECTOR(TempStr,&Normal) );
		Ar.Logf( TEXT("%s      TextureU %s\r\n"), FCString::Spc(TextIndent), SetFVECTOR(TempStr,&TextureU) );
		Ar.Logf( TEXT("%s      TextureV %s\r\n"), FCString::Spc(TextIndent), SetFVECTOR(TempStr,&TextureV) );
		for( int32 j=0; j<Poly->Vertices.Num(); j++ )
		{
			FVector Vertex(Poly->Vertices[j]);
			Ar.Logf( TEXT("%s      Vertex   %s\r\n"), FCString::Spc(TextIndent), SetFVECTOR(TempStr,&Vertex) );
		}
		Ar.Logf( TEXT("%s   End Polygon\r\n"), FCString::Spc(TextIndent) );
	}
	Ar.Logf( TEXT("%sEnd PolyList\r\n"), FCString::Spc(TextIndent) );

	return 1;
}

/*------------------------------------------------------------------------------
	UModelExporterT3D implementation.
------------------------------------------------------------------------------*/
UModelExporterT3D::UModelExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UModel::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("T3D"));
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal model text"));
	FormatDescription.Add(TEXT("Unreal model text"));
}

bool UModelExporterT3D::ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags )
{
	UModel* Model = CastChecked<UModel>( Object );

	Ar.Logf( TEXT("%sBegin Brush Name=%s\r\n"), FCString::Spc(TextIndent), *Model->GetName() );
	if (!(PortFlags & PPF_SeparateDeclare))
	{
		UExporter::ExportToOutputDevice(Context, Model->Polys, NULL, Ar, Type, TextIndent + 3, PortFlags);
	}
	Ar.Logf( TEXT("%sEnd Brush\r\n"), FCString::Spc(TextIndent) );

	return 1;
}

/*------------------------------------------------------------------------------
	ULevelExporterT3D implementation.
------------------------------------------------------------------------------*/

void ExporterHelper_DumpPackageInners(const FExportObjectInnerContext* Context, UPackage* InPackage, int32 TabCount)
{
	const TArray<UObject*>* Inners = Context->GetObjectInners(InPackage);
	if (Inners)
	{
		for (int32 InnerIndex = 0; InnerIndex < Inners->Num(); InnerIndex++)
		{
			UObject* InnerObj = (*Inners)[InnerIndex];

			FString TabString;
			for (int32 TabOutIndex = 0; TabOutIndex < TabCount; TabOutIndex++)
			{
				TabString += TEXT("\t");
			}

			UE_LOGF(LogEditorExporters, Log, "%ls%ls : %ls (%ls)", *TabString,
				InnerObj ? *InnerObj->GetClass()->GetName()	: TEXT("*NULL*"),
				InnerObj ? *InnerObj->GetName()				: TEXT("*NULL*"),
				InnerObj ? *InnerObj->GetPathName()			: TEXT("*NULL*"));

			UPackage* InnerPackage = Cast<UPackage>(InnerObj);
			if (InnerPackage)
			{
				TabCount++;
				ExporterHelper_DumpPackageInners(Context, InnerPackage, TabCount);
				TabCount--;
			}
		}
	}
}

UActorExporterT3D::UActorExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = AActor::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("T3D"));
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal Actor text"));
	FormatDescription.Add(TEXT("Unreal Actor text"));
}

bool UActorExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	AActor* Actor = Cast<AActor>(Object);
	if (Actor->ShouldExport())
	{
		// Temporarily unbind dynamic delegates so we don't export the bindings.
		UBlueprintGeneratedClass::UnbindDynamicDelegates(Actor->GetClass(), Actor);

		AActor* ParentActor = Actor->GetAttachParentActor();
		FName SocketName = Actor->GetAttachParentSocketName();
		FDetachmentTransformRules DetachmentTransformRules = FDetachmentTransformRules::KeepWorldTransform;
		DetachmentTransformRules.bCallModify = false;
		Actor->DetachFromActor(DetachmentTransformRules);

		const bool IsUsingActorFolders = Actor->GetLevel() && Actor->GetLevel()->IsUsingActorFolders();

		FString ParentActorString = ( ParentActor ? FString::Printf(TEXT(" ParentActor=%s"), *ParentActor->GetName() ) : TEXT(""));
		FString SocketNameString = ( (ParentActor && SocketName != NAME_None) ? FString::Printf(TEXT(" SocketName=%s"), *SocketName.ToString() ) : TEXT(""));
		FString GroupActor = (Actor->GroupActor ? FString::Printf(TEXT(" GroupActor=%s"), *Actor->GroupActor->GetName() ) : TEXT(""));
		FString GroupFolder = (Actor->GroupActor ? FString::Printf(TEXT(" GroupFolder=%s"), *Actor->GroupActor->GetFolderPath().ToString() ) : TEXT(""));
		FString ActorFolderPath = (IsUsingActorFolders ? FString::Printf(TEXT(" ActorFolderPath=\"%s\""), *Actor->GetFolderPath().ToString()) : TEXT(""));
		FString CopyPasteId = (Actor->CopyPasteId != INDEX_NONE) ? FString::Printf(TEXT(" CopyPasteId=%d"), Actor->CopyPasteId ) : TEXT("");
		FString ContentBundleGuid = (Actor->GetContentBundleGuid().IsValid() ? FString::Printf(TEXT(" ActorContentBundleGuid=%s"), *Actor->GetContentBundleGuid().ToString()) : TEXT(""));
		FString ExternalDataLayer = (Actor->GetExternalDataLayerAsset() ? FString::Printf(TEXT(" ExternalDataLayerAsset=%s"), *FSoftObjectPath(Actor->GetExternalDataLayerAsset()).ToString()) : TEXT(""));

		Ar.Logf( TEXT("%sBegin Actor Class=%s Name=%s Archetype=%s%s%s%s%s%s%s%s%s"), 
			FCString::Spc(TextIndent), *Actor->GetClass()->GetPathName(), *Actor->GetName(),
			*FObjectPropertyBase::GetExportPath(Actor->GetArchetype(), nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified), 
			*ParentActorString, *SocketNameString, *GroupActor, *GroupFolder, *ActorFolderPath, *CopyPasteId, *ContentBundleGuid, *ExternalDataLayer);

		// When exporting for diffs, export paths can cause false positives. since diff files don't get imported, we can
		// skip adding this info the file.
		if (!(PortFlags & PPF_ForDiff))
		{
			// Emit the actor path
			Ar.Logf(TEXT(" ExportPath=%s"), *FObjectPropertyBase::GetExportPath(Actor, nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified));
		}

		Ar.Logf(LINE_TERMINATOR);

		ExportRootScope = Actor;
		ExportObjectInner( Context, Actor, Ar, PortFlags | PPF_ExportsNotFullyQualified );
		ExportRootScope = nullptr;

		Ar.Logf( TEXT("%sEnd Actor\r\n"), FCString::Spc(TextIndent) );
		Actor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform, SocketName);

		// Restore dynamic delegate bindings.
		UBlueprintGeneratedClass::BindDynamicDelegates(Actor->GetClass(), Actor);
	}
	else if (GEditor)
	{
		GEditor->GetSelectedActors()->Deselect(Actor);
	}
	return true;
}

bool UActorExporterT3D::SupportsObject(UObject* Object) const
{
	if (Private::bLevelExportT3D_UseActorExporters)
	{
		return Super::SupportsObject(Object);
	}
	return false;
}

UGroupActorExporterT3D::UGroupActorExporterT3D(const FObjectInitializer& ObjectInitializer)
{
	SupportedClass = AGroupActor::StaticClass();
}

bool UGroupActorExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar,
	FFeedbackContext* Warn, uint32 PortFlags)
{
	// Don't export group actors
	return true;
}

UPhysicsVolumeExporterT3D::UPhysicsVolumeExporterT3D(const FObjectInitializer& ObjectInitializer)
{
	SupportedClass = APhysicsVolume::StaticClass();
}

bool UPhysicsVolumeExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar,
                                           FFeedbackContext* Warn, uint32 PortFlags)
{
	APhysicsVolume* PhysicsVolume = Cast<APhysicsVolume>(Object);
	if (UWorld* World = PhysicsVolume->GetWorld())
	{
		if (World->GetDefaultPhysicsVolume() == PhysicsVolume)
		{
			// Don't export the default physics volume, as it doesn't have a UModel associated with it
			// and thus will not import properly.
			return true;
		}
		return Super::ExportText(Context, Object, Type, Ar, Warn, PortFlags);
	}
	return true;
}

ULevelExporterT3D::ULevelExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("T3D"));
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal world text"));
	FormatDescription.Add(TEXT("Unreal world text"));
}

bool ULevelExporterT3D::ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags )
{
	UWorld* World = CastChecked<UWorld>( Object );
	APhysicsVolume* DefaultPhysicsVolume = World->GetDefaultPhysicsVolume();

	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	UPackage* MapPackage = NULL;
	if ((PortFlags & PPF_Copy) == 0)
	{
		// If we are not copying to clipboard, then export objects contained in the map package itself...
		MapPackage = Object->GetOutermost();
	}

	// this is the top level in the .t3d file
	if (MapPackage)
	{
		Ar.Logf(TEXT("%sBegin Map Name=%s\r\n"), FCString::Spc(TextIndent),  *(MapPackage->GetName()));
	}
	else
	{
		Ar.Logf(TEXT("%sBegin Map\r\n"), FCString::Spc(TextIndent));
	}

	// are we exporting all actors or just selected actors?
	bool bAllActors = FCString::Stricmp(Type,TEXT("COPY"))!=0 && !bSelectedOnly;

	TextIndent += 3;

	ULevel* Level;

	// start a new level section
	if (FCString::Stricmp(Type, TEXT("COPY")) == 0)
	{
		// for copy and paste, we want to select actors in the current level
		Level = World->GetCurrentLevel();

		// if we are copy/pasting, then we don't name the level - we paste into the current level
		Ar.Logf(TEXT("%sBegin Level\r\n"), FCString::Spc(TextIndent));

		// mark that we are doing a clipboard copy
		PortFlags |= PPF_Copy;
	}
	else
	{
		// for export, we only want the persistent level
		Level = World->PersistentLevel;

		//@todo seamless if we are exporting only selected, should we export from all levels? or maybe from the current level?

		// if we aren't copy/pasting, then we name the level so that when we import, we get the same level structure
		Ar.Logf(TEXT("%sBegin Level NAME=%s\r\n"), FCString::Spc(TextIndent), *Level->GetName());
	}

	TextIndent += 3;

	const int32 ActorNumberToCopy = bAllActors ? Level->Actors.Num() : Context->GetObjectNumber();
	FScopedSlowTask SlowTask(ActorNumberToCopy, NSLOCTEXT("UnrealEd", "ExportingActors", "Exporting Actors"));
	const bool bShowCancelButton = true;
	SlowTask.MakeDialogDelayed(1.f, bShowCancelButton);

	// Allow for a pre-specified (and pre-sorted) list of level inners
	TArray<TObjectPtr<AActor>> SpecifiedActors;
	const TArray<UObject*>* LevelInners = Context->GetObjectInners(Level);
	if (LevelInners)
	{
		SpecifiedActors.Reserve(LevelInners->Num());
		for (UObject* Inner : *LevelInners)
		{
			if (AActor* Actor = Cast<AActor>(Inner))
			{
				if (ensureMsgf(Actor->GetLevel() == Level, TEXT("Specified actors must actually be inners of the level!")))
				{
					SpecifiedActors.Add(Actor);
				}
			}
		}
	}

	// loop through all of the actors just in this level
	const TArray<TObjectPtr<AActor>>& Actors = LevelInners ? SpecifiedActors : Level->Actors;
	
	for (AActor* Actor : Actors)
	{
		if (SlowTask.ShouldCancel())
		{
			break;
		}
		
		if (Private::bLevelExportT3D_UseActorExporters)
		{
			// Ensure the actor is currently selected
			if( Actor && ( bAllActors || IsObjectSelectedForExport(Context, Actor) ) )
			{
				UExporter* Exporter = UExporter::FindExporter(Actor, Type);
				if (Exporter)
				{
					Exporter->TextIndent = TextIndent;
					Exporter->bSelectedOnly = bSelectedOnly;
					bool Result = Exporter->ExportText(Context, Actor, Type, Ar, Warn, PortFlags);
					if (Result == false)
					{
						return Result;
					}
				}

				SlowTask.EnterProgressFrame();
			}
		}
		else
		{
			if ( Actor == DefaultPhysicsVolume )
			{
				continue;
			}
			// Ensure actor is not a group if grouping is disabled and that the actor is currently selected
			if( Actor && !Actor->IsA(AGroupActor::StaticClass()) &&
				( bAllActors || IsObjectSelectedForExport(Context, Actor) ) )
			{
				if (Actor->ShouldExport())
				{
					// Temporarily unbind dynamic delegates so we don't export the bindings.
					UBlueprintGeneratedClass::UnbindDynamicDelegates(Actor->GetClass(), Actor);

					AActor* ParentActor = Actor->GetAttachParentActor();
					FName SocketName = Actor->GetAttachParentSocketName();
					FDetachmentTransformRules DetachmentTransformRules = FDetachmentTransformRules::KeepWorldTransform;
					DetachmentTransformRules.bCallModify = false;
					Actor->DetachFromActor(DetachmentTransformRules);

					FString ParentActorString = ( ParentActor ? FString::Printf(TEXT(" ParentActor=%s"), *ParentActor->GetName() ) : TEXT(""));
					FString SocketNameString = ( (ParentActor && SocketName != NAME_None) ? FString::Printf(TEXT(" SocketName=%s"), *SocketName.ToString() ) : TEXT(""));
					FString GroupActor = (Actor->GroupActor ? FString::Printf(TEXT(" GroupActor=%s"), *Actor->GroupActor->GetName() ) : TEXT(""));
					FString GroupFolder = (Actor->GroupActor ? FString::Printf(TEXT(" GroupFolder=%s"), *Actor->GroupActor->GetFolderPath().ToString() ) : TEXT(""));
					FString ActorFolderPath = (Level->IsUsingActorFolders() ? FString::Printf(TEXT(" ActorFolderPath=\"%s\""), *Actor->GetFolderPath().ToString()) : TEXT(""));
					FString CopyPasteId = (Actor->CopyPasteId != INDEX_NONE) ? FString::Printf(TEXT(" CopyPasteId=%d"), Actor->CopyPasteId ) : TEXT("");
					FString ContentBundleGuid = (Actor->GetContentBundleGuid().IsValid() ? FString::Printf(TEXT(" ActorContentBundleGuid=%s"), *Actor->GetContentBundleGuid().ToString()) : TEXT(""));
					FString ExternalDataLayer = (Actor->GetExternalDataLayerAsset() ? FString::Printf(TEXT(" ExternalDataLayerAsset=%s"), *FSoftObjectPath(Actor->GetExternalDataLayerAsset()).ToString()) : TEXT(""));

					Ar.Logf( TEXT("%sBegin Actor Class=%s Name=%s Archetype=%s%s%s%s%s%s%s%s%s"), 
						FCString::Spc(TextIndent), *Actor->GetClass()->GetPathName(), *Actor->GetName(),
						*FObjectPropertyBase::GetExportPath(Actor->GetArchetype(), nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified), 
						*ParentActorString, *SocketNameString, *GroupActor, *GroupFolder, *ActorFolderPath, *CopyPasteId, *ContentBundleGuid, *ExternalDataLayer);

					// When exporting for diffs, export paths can cause false positives. since diff files don't get imported, we can
					// skip adding this info the file.
					if (!(PortFlags & PPF_ForDiff))
					{
						// Emit the actor path
						Ar.Logf(TEXT(" ExportPath=%s"), *FObjectPropertyBase::GetExportPath(Actor, nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified));
					}

					Ar.Logf(LINE_TERMINATOR);

					ExportRootScope = Actor;
					ExportObjectInner( Context, Actor, Ar, PortFlags | PPF_ExportsNotFullyQualified );
					ExportRootScope = nullptr;

					Ar.Logf( TEXT("%sEnd Actor\r\n"), FCString::Spc(TextIndent) );
					Actor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform, SocketName);

					// Restore dynamic delegate bindings.
					UBlueprintGeneratedClass::BindDynamicDelegates(Actor->GetClass(), Actor);
				}
				else if (GEditor)
				{
					GEditor->GetSelectedActors()->Deselect(Actor);
				}

				SlowTask.EnterProgressFrame();
			}
		}
	}

	TextIndent -= 3;

	Ar.Logf(TEXT("%sEnd Level\r\n"), FCString::Spc(TextIndent));

	TextIndent -= 3;

	// Export information about the first selected surface in the map.  Used for copying/pasting
	// information from poly to poly.
	Ar.Logf( TEXT("%sBegin Surface\r\n"), FCString::Spc(TextIndent) );
	TCHAR TempStr[256];
	const UModel* Model = World->GetModel();
	for (const FBspSurf& Poly : Model->Surfs)
	{
		if (Poly.PolyFlags & PF_Selected )
		{
			FVector pBase = (FVector)Model->Points[Poly.pBase];
			FVector vTextureU = (FVector)Model->Vectors[Poly.vTextureU];
			FVector vTextureV = (FVector)Model->Vectors[Poly.vTextureV];
			FVector vNormal = (FVector)Model->Vectors[Poly.vNormal];

			Ar.Logf(TEXT("%sTEXTURE=%s\r\n"),   FCString::Spc(TextIndent + 3), *Poly.Material->GetPathName());
			Ar.Logf(TEXT("%sBASE      %s\r\n"), FCString::Spc(TextIndent + 3), SetFVECTOR(TempStr, &pBase));
			Ar.Logf(TEXT("%sTEXTUREU  %s\r\n"), FCString::Spc(TextIndent + 3), SetFVECTOR(TempStr, &vTextureU));
			Ar.Logf(TEXT("%sTEXTUREV  %s\r\n"), FCString::Spc(TextIndent + 3), SetFVECTOR(TempStr, &vTextureV));
			Ar.Logf(TEXT("%sNORMAL    %s\r\n"), FCString::Spc(TextIndent + 3), SetFVECTOR(TempStr, &vNormal));
			Ar.Logf(TEXT("%sPOLYFLAGS=%d\r\n"), FCString::Spc(TextIndent + 3), Poly.PolyFlags);
			break;
		}
	}
	Ar.Logf( TEXT("%sEnd Surface\r\n"), FCString::Spc(TextIndent) );

	Ar.Logf( TEXT("%sEnd Map\r\n"), FCString::Spc(TextIndent) );


	return !SlowTask.ShouldCancel();
}

void ULevelExporterT3D::ExportComponentExtra(const FExportObjectInnerContext* Context, const TArray<UActorComponent*>& Components, FOutputDevice& Ar, uint32 PortFlags)
{
	for (UActorComponent* ActorComponent : Components)
	{
		if (ActorComponent != nullptr && ActorComponent->GetWorld() != nullptr)
		{
			// Go through all FoliageActors in the world, since we support cross-level bases
			for (TActorIterator<AInstancedFoliageActor> It(ActorComponent->GetWorld()); It; ++It)
			{
				AInstancedFoliageActor* IFA = *It;
				if (IsValid(IFA))
				{
					auto FoliageInstanceMap = IFA->GetInstancesForComponent(ActorComponent);
					for (const auto& MapEntry : FoliageInstanceMap)
					{
						Ar.Logf(TEXT("%sBegin Foliage FoliageType=%s Component=%s%s"), FCString::Spc(TextIndent), *MapEntry.Key->GetPathName(), *ActorComponent->GetName(), LINE_TERMINATOR);
						for (const FFoliageInstancePlacementInfo* Inst : MapEntry.Value)
						{
							Ar.Logf(TEXT("%sLocation=%f,%f,%f Rotation=%f,%f,%f PreAlignRotation=%f,%f,%f DrawScale3D=%f,%f,%f Flags=%u%s"), FCString::Spc(TextIndent+3),
								Inst->Location.X, Inst->Location.Y, Inst->Location.Z,
								Inst->Rotation.Pitch, Inst->Rotation.Yaw, Inst->Rotation.Roll,
								Inst->PreAlignRotation.Pitch, Inst->PreAlignRotation.Yaw, Inst->PreAlignRotation.Roll,
								Inst->DrawScale3D.X, Inst->DrawScale3D.Y, Inst->DrawScale3D.Z,
								Inst->Flags,
								LINE_TERMINATOR);
						}

						Ar.Logf(TEXT("%sEnd Foliage%s"), FCString::Spc(TextIndent), LINE_TERMINATOR);
					}
				}
			}
		}
	}
}

void ULevelExporterT3D::ExportPackageObject(FExportPackageParams& ExpPackageParams){}
void ULevelExporterT3D::ExportPackageInners(FExportPackageParams& ExpPackageParams){}

/*------------------------------------------------------------------------------
	ULevelExporterSTL implementation.
------------------------------------------------------------------------------*/
ULevelExporterSTL::ULevelExporterSTL(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("STL"));
	FormatDescription.Add(TEXT("Stereolithography"));
}

bool ULevelExporterSTL::ExportText( const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags )
{
	//@todo seamless - this needs to be world, like the t3d version above
	UWorld* World = CastChecked<UWorld>(Object);
	ULevel* Level = World->PersistentLevel;

	for (FThreadSafeObjectIterator It; It; ++It)
	{
		It->UnMark(EObjectMark(OBJECTMARK_TagImp | OBJECTMARK_TagExp));
	}

	//
	// GATHER TRIANGLES
	//

	TArray<FVector> Triangles;

	for( int32 iActor=0; iActor<Level->Actors.Num(); iActor++ )
	{
		// Landscape
		ALandscape* Landscape = Cast<ALandscape>(Level->Actors[iActor]);
		if( Landscape && ( !bSelectedOnly || IsObjectSelectedForExport(Context, Landscape) ) )
		{
			ULandscapeInfo* LandscapeInfo = Landscape ? Landscape->GetLandscapeInfo() : NULL;
			if( Landscape && LandscapeInfo )
			{
				auto SelectedComponents = LandscapeInfo->GetSelectedComponents();
				
				// Export data for each component
				for( auto It = LandscapeInfo->XYtoComponentMap.CreateIterator(); It; ++It )
				{
					if (bSelectedOnly && SelectedComponents.Num() && !SelectedComponents.Contains(It.Value()))
					{
						continue;
					}
					ULandscapeComponent* Component = It.Value();
					FLandscapeComponentDataInterface CDI(Component);

					for( int32 y=0;y<Component->ComponentSizeQuads;y++ )
					{
						for( int32 x=0;x<Component->ComponentSizeQuads;x++ )
						{
							FVector P00	= CDI.GetWorldVertex(x,y);
							FVector P01	= CDI.GetWorldVertex(x,y+1);
							FVector P11	= CDI.GetWorldVertex(x+1,y+1);
							FVector P10	= CDI.GetWorldVertex(x+1,y);

							// triangulation matches FLandscapeIndexBuffer constructor
							Triangles.Add(P00);
							Triangles.Add(P11);
							Triangles.Add(P10);

							Triangles.Add(P00);
							Triangles.Add(P01);
							Triangles.Add(P11);
						}	
					}
				}		
			}
		}

		// Static meshes

		AStaticMeshActor* Actor = Cast<AStaticMeshActor>(Level->Actors[iActor]);
		if( Actor && ( !bSelectedOnly || IsObjectSelectedForExport(Context, Actor) ) && Actor->GetStaticMeshComponent()->GetStaticMesh() && Actor->GetStaticMeshComponent()->GetStaticMesh()->HasValidRenderData() )
		{
			FStaticMeshLODResources& LODModel = Actor->GetStaticMeshComponent()->GetStaticMesh()->GetRenderData()->LODResources[0];
			FIndexArrayView Indices = LODModel.IndexBuffer.GetArrayView();
			int32 NumSections = LODModel.Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				const FStaticMeshSection& Section = LODModel.Sections[SectionIndex];
				for (int32 TriIndex = 0; TriIndex < (int32)Section.NumTriangles; ++TriIndex)
				{
					int32 BaseIndex = Section.FirstIndex + TriIndex * 3;
					for( int32 v = 2 ; v > -1 ; v-- )
					{
						int32 i = Indices[BaseIndex + v];
						FVector vtx = Actor->ActorToWorld().TransformPosition((FVector)LODModel.VertexBuffers.PositionVertexBuffer.VertexPosition(i));
						Triangles.Add(vtx);
					}
				}
			}
		}
	}

	// BSP Surfaces
	for( int32 i=0;i<World->GetModel()->Nodes.Num();i++ )
	{
		FBspNode* Node = &World->GetModel()->Nodes[i];
		if( !bSelectedOnly || World->GetModel()->Surfs[Node->iSurf].PolyFlags&PF_Selected )
		{
			if( Node->NumVertices > 2 )
			{
				FVector vtx1(World->GetModel()->Points[World->GetModel()->Verts[Node->iVertPool+0].pVertex]),
					vtx2(World->GetModel()->Points[World->GetModel()->Verts[Node->iVertPool+1].pVertex]),
					vtx3;

				for( int32 v = 2 ; v < Node->NumVertices ; v++ )
				{
					vtx3 = (FVector)World->GetModel()->Points[World->GetModel()->Verts[Node->iVertPool+v].pVertex];

					Triangles.Add( vtx1 );
					Triangles.Add( vtx2 );
					Triangles.Add( vtx3 );

					vtx2 = vtx3;
				}
			}
		}
	}

	//
	// WRITE THE FILE
	//

	Ar.Logf( TEXT("%ssolid LevelBSP\r\n"), FCString::Spc(TextIndent) );

	for( int32 tri = 0 ; tri < Triangles.Num() ; tri += 3 )
	{
		FVector vtx[3];
		vtx[0] = Triangles[tri] * FVector(1,-1,1);
		vtx[1] = Triangles[tri+1] * FVector(1,-1,1);
		vtx[2] = Triangles[tri+2] * FVector(1,-1,1);

		FPlane Normal( vtx[0], vtx[1], vtx[2] );

		Ar.Logf( TEXT("%sfacet normal %1.6f %1.6f %1.6f\r\n"), FCString::Spc(TextIndent+2), Normal.X, Normal.Y, Normal.Z );
		Ar.Logf( TEXT("%souter loop\r\n"), FCString::Spc(TextIndent+4) );

		Ar.Logf( TEXT("%svertex %1.6f %1.6f %1.6f\r\n"), FCString::Spc(TextIndent+6), vtx[0].X, vtx[0].Y, vtx[0].Z );
		Ar.Logf( TEXT("%svertex %1.6f %1.6f %1.6f\r\n"), FCString::Spc(TextIndent+6), vtx[1].X, vtx[1].Y, vtx[1].Z );
		Ar.Logf( TEXT("%svertex %1.6f %1.6f %1.6f\r\n"), FCString::Spc(TextIndent+6), vtx[2].X, vtx[2].Y, vtx[2].Z );

		Ar.Logf( TEXT("%sendloop\r\n"), FCString::Spc(TextIndent+4) );
		Ar.Logf( TEXT("%sendfacet\r\n"), FCString::Spc(TextIndent+2) );
	}

	Ar.Logf( TEXT("%sendsolid LevelBSP\r\n"), FCString::Spc(TextIndent) );

	Triangles.Empty();

	return 1;
}

/*------------------------------------------------------------------------------
	Helper classes for the OBJ exporters.
------------------------------------------------------------------------------*/

// An individual face.

class FOBJFace
{
public:
	// index into FOBJGeom::VertexData (local within FOBJGeom)
	uint32 VertexIndex[3];
	/** List of vertices that make up this face. */

	/** The material that was applied to this face. */
	UMaterialInterface* Material;
};

class FOBJVertex
{
public:
	// position
	FVector Vert;
	// texture coordiante
	FVector2D UV;
	// normal
	FVector Normal;
	//	FLinearColor Colors[3];
};

// A geometric object.  This will show up as a separate object when imported into a modeling program.
class FOBJGeom
{
public:
	/** List of faces that make up this object. */
	TArray<FOBJFace> Faces;

	/** Vertex positions that make up this object. */
	TArray<FOBJVertex> VertexData;

	/** Name used when writing this object to the OBJ file. */
	FString Name;

	// Constructors.
	FORCEINLINE FOBJGeom( const FString& InName )
		:	Name( InName )
	{}
};

inline FString FixupMaterialName(UMaterialInterface* Material)
{
	return Material->GetName().Replace(TEXT("."), TEXT("_")).Replace(TEXT(":"), TEXT("_"));
}


/**
 * Adds the given actor's mesh to the GOBJObjects array if possible
 * 
 * @param Actor The actor to export
 * @param Objects The array that contains cached exportable object data
 * @param Materials Optional set of materials to gather all used materials by the objects (currently only StaticMesh materials are supported)
 */
static void AddActorToOBJs(AActor* Actor, TArray<FOBJGeom*>& Objects, TSet<UMaterialInterface*>* Materials, bool bSelectedOnly)
{
	FMatrix LocalToWorld = Actor->ActorToWorld().ToMatrixWithScale();

	// Landscape
	ALandscape* Landscape = Cast<ALandscape>( Actor );
	ULandscapeInfo* LandscapeInfo = Landscape ? Landscape->GetLandscapeInfo() : NULL;
	if( Landscape && LandscapeInfo )
	{
		auto SelectedComponents =  LandscapeInfo->GetSelectedComponents();

		// Export data for each component
		for( auto It = Landscape->GetLandscapeInfo()->XYtoComponentMap.CreateIterator(); It; ++It )
		{
			if (bSelectedOnly && SelectedComponents.Num() && !SelectedComponents.Contains(It.Value()))
			{
				continue;
			}
			ULandscapeComponent* Component = It.Value();
			FLandscapeComponentDataInterface CDI(Component, Landscape->ExportLOD);
			const int32 ComponentSizeQuads = ((Component->ComponentSizeQuads+1)>>Landscape->ExportLOD)-1;
			const int32 SubsectionSizeQuads = ((Component->SubsectionSizeQuads+1)>>Landscape->ExportLOD)-1;
			const float ScaleFactor = (float)Component->ComponentSizeQuads / (float)ComponentSizeQuads;

			FOBJGeom* OBJGeom = new FOBJGeom( Component->GetName() );
			OBJGeom->VertexData.AddZeroed( FMath::Square(ComponentSizeQuads + 1) );
			OBJGeom->Faces.AddZeroed( FMath::Square(ComponentSizeQuads) * 2 );

			// Check if there are any holes
			TArray64<uint8> RawVisData;
			uint8* VisDataMap = NULL;
			int32 TexIndex = INDEX_NONE;
			int32 WeightMapSize = (SubsectionSizeQuads + 1) * Component->NumSubsections;
			int32 ChannelOffsets[4] = {(int32)STRUCT_OFFSET(FColor,R),(int32)STRUCT_OFFSET(FColor,G),(int32)STRUCT_OFFSET(FColor,B),(int32)STRUCT_OFFSET(FColor,A)};

			const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations();
			const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures();

			for( int32 AllocIdx=0;AllocIdx < ComponentWeightmapLayerAllocations.Num(); AllocIdx++ )
			{
				const FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[AllocIdx];
				if( AllocInfo.LayerInfo == ALandscapeProxy::VisibilityLayer )
				{
					TexIndex = AllocInfo.WeightmapTextureIndex;

					ComponentWeightmapTextures[TexIndex]->Source.GetMipData(RawVisData, 0);
					VisDataMap = RawVisData.GetData() + ChannelOffsets[AllocInfo.WeightmapTextureChannel];
				}
			}

			// Export verts
			FOBJVertex* Vert = OBJGeom->VertexData.GetData();
			for( int32 y=0;y<ComponentSizeQuads+1;y++ )
			{
				for( int32 x=0;x<ComponentSizeQuads+1;x++ )
				{
					FVector WorldPos, WorldTangentX, WorldTangentY, WorldTangentZ;

					CDI.GetWorldPositionTangents( x, y, WorldPos, WorldTangentX, WorldTangentY, WorldTangentZ );

					Vert->Vert = WorldPos;
					Vert->UV = FVector2D(Component->GetSectionBase().X + x * ScaleFactor, Component->GetSectionBase().Y + y * ScaleFactor);
					Vert->Normal = WorldTangentZ;
					Vert++;
				}
			}

			int32 VisThreshold = 170;
			int32 SubNumX, SubNumY, SubX, SubY;

			FOBJFace* Face = OBJGeom->Faces.GetData();
			for( int32 y=0;y<ComponentSizeQuads;y++ )
			{
				for( int32 x=0;x<ComponentSizeQuads;x++ )
				{
					CDI.ComponentXYToSubsectionXY(x, y, SubNumX, SubNumY, SubX, SubY );
					int32 WeightIndex = SubX + SubNumX*(SubsectionSizeQuads + 1) + (SubY+SubNumY*(SubsectionSizeQuads + 1))*WeightMapSize;

					bool bInvisible = VisDataMap && VisDataMap[WeightIndex * sizeof(FColor)] >= VisThreshold;
					// triangulation matches FLandscapeIndexBuffer constructor
					Face->VertexIndex[0] = (x+0) + (y+0) * (ComponentSizeQuads+1);
					Face->VertexIndex[1] = bInvisible ? Face->VertexIndex[0] : (x+1) + (y+1) * (ComponentSizeQuads+1);
					Face->VertexIndex[2] = bInvisible ? Face->VertexIndex[0] : (x+1) + (y+0) * (ComponentSizeQuads+1);
					Face++;

					Face->VertexIndex[0] = (x+0) + (y+0) * (ComponentSizeQuads+1);
					Face->VertexIndex[1] = bInvisible ? Face->VertexIndex[0] : (x+0) + (y+1) * (ComponentSizeQuads+1);
					Face->VertexIndex[2] = bInvisible ? Face->VertexIndex[0] : (x+1) + (y+1) * (ComponentSizeQuads+1);
					Face++;
				}	
			}

			Objects.Add( OBJGeom );
		}
	}


	// Static mesh components

	UStaticMeshComponent* StaticMeshComponent = NULL;
	UStaticMesh* StaticMesh = NULL;

	TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents;
	Actor->GetComponents(StaticMeshComponents);

	for( int32 j=0; j<StaticMeshComponents.Num(); j++ )
	{
		// If its a static mesh component, with a static mesh
		StaticMeshComponent = StaticMeshComponents[j];
		if( StaticMeshComponent->IsRegistered() && StaticMeshComponent->GetStaticMesh()
			&& StaticMeshComponent->GetStaticMesh()->HasValidRenderData() )
		{
			LocalToWorld = StaticMeshComponent->GetComponentTransform().ToMatrixWithScale();
			StaticMesh = StaticMeshComponent->GetStaticMesh();
			if (StaticMesh)
			{
				// make room for the faces
				FOBJGeom* OBJGeom = new FOBJGeom(StaticMeshComponents.Num() > 1 ? StaticMesh->GetName() : Actor->GetName());

				FStaticMeshLODResources* RenderData = &StaticMesh->GetRenderData()->LODResources[0];
				FIndexArrayView Indices = RenderData->IndexBuffer.GetArrayView();
				uint32 NumIndices = Indices.Num();

				// 3 indices for each triangle
				check(NumIndices % 3 == 0);
				uint32 TriangleCount = NumIndices / 3;
				OBJGeom->Faces.AddUninitialized(TriangleCount);

				uint32 VertexCount = RenderData->VertexBuffers.PositionVertexBuffer.GetNumVertices();
				OBJGeom->VertexData.AddUninitialized(VertexCount);
				FOBJVertex* VerticesOut = OBJGeom->VertexData.GetData();

				check(VertexCount == RenderData->VertexBuffers.StaticMeshVertexBuffer.GetNumVertices());

				FMatrix LocalToWorldInverseTranspose = LocalToWorld.InverseFast().GetTransposed();
				for (uint32 i = 0; i < VertexCount; i++)
				{
					// Vertices
					VerticesOut[i].Vert = LocalToWorld.TransformPosition((FVector)RenderData->VertexBuffers.PositionVertexBuffer.VertexPosition(i));
					// UVs from channel 0
					VerticesOut[i].UV = FVector2D(RenderData->VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 0));
					// Normal
					VerticesOut[i].Normal = LocalToWorldInverseTranspose.TransformVector((FVector4)RenderData->VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(i));
				}

				bool bFlipCullMode = LocalToWorld.RotDeterminant() < 0.0f;

				uint32 CurrentTriangleId = 0;
				for (int32 SectionIndex = 0; SectionIndex < RenderData->Sections.Num(); ++SectionIndex)
				{
					FStaticMeshSection& Section = RenderData->Sections[SectionIndex];
					UMaterialInterface* Material = 0;

					// Get the material for this triangle by first looking at the material overrides array and if that is NULL by looking at the material array in the original static mesh
					Material = StaticMeshComponent->GetMaterial(Section.MaterialIndex);

					// cache the set of needed materials if desired
					if (Materials && Material)
					{
						Materials->Add(Material);
					}

					for (uint32 i = 0; i < Section.NumTriangles; i++)
					{
						FOBJFace& OBJFace = OBJGeom->Faces[CurrentTriangleId++];

						uint32 a = Indices[Section.FirstIndex + i * 3 + 0];
						uint32 b = Indices[Section.FirstIndex + i * 3 + 1];
						uint32 c = Indices[Section.FirstIndex + i * 3 + 2];

						if (bFlipCullMode)
						{
							Swap(a, c);
						}

						OBJFace.VertexIndex[0] = a;
						OBJFace.VertexIndex[1] = b;
						OBJFace.VertexIndex[2] = c;

						// Material
						OBJFace.Material = Material;
					}
				}

				Objects.Add(OBJGeom);
			}
		}
	}
}

// @param Material must not be 0
// @param MatProp e.g. MP_DiffuseColor
static void ExportMaterialPropertyTexture(const FString& BMPFilename, UMaterialInterface* Material, const EMaterialProperty MatProp)
{
	check(Material);

	// make the BMP for the diffuse channel
	TArray<FColor> OutputBMP;
	FIntPoint OutSize{};

	bool bIsValidProperty = FMaterialUtilities::SupportsExport(IsOpaqueBlendMode(*Material), MatProp);
	if (bIsValidProperty)
	{
		FMeshData MeshSettings;
		MeshSettings.MeshDescription = nullptr;
		MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
		MeshSettings.TextureCoordinateIndex = 0;

		FMaterialData MaterialSettings;
		MaterialSettings.Material = Material;

		// Add all user defined properties for baking out
		FPropertyEntry Entry(MatProp);
		if (!Entry.bUseConstantValue && Material->IsPropertyActive(Entry.Property) && Entry.Property != MP_MAX)
		{
			FIntPoint TextureSize = FMaterialUtilities::FindMaxTextureSize(Material, MatProp);
			MaterialSettings.PropertySizes.Add(Entry.Property, TextureSize);
		}

		TArray<FMeshData*> MeshSettingPtrs{ &MeshSettings };
		TArray<FMaterialData*> MaterialSettingPtrs{ &MaterialSettings };

		IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

		bool bSetData = false;
		bool bSetSize = false;
		TArray<FBakeOutput> BakeOutputs;
		Module.BakeMaterials(MaterialSettingPtrs, MeshSettingPtrs, BakeOutputs);
		if (BakeOutputs.Num() > 0)
		{
			FBakeOutput& Output = BakeOutputs[0];
			if (TArray<FColor>* PropertyData = Output.PropertyData.Find(MatProp))
			{
				OutputBMP = MoveTemp(*PropertyData);
				bSetData = true;
			}

			if (FIntPoint* PropertySize = Output.PropertySizes.Find(MatProp))
			{
				OutSize = *PropertySize;
				bSetSize = true;
			}
		}

		bIsValidProperty = bSetData && bSetSize;
	}

	// make invalid textures a solid red
	if (!bIsValidProperty)
	{
		OutSize = FIntPoint(1, 1);
		OutputBMP.Empty();
		OutputBMP.Add(FColor(255, 0, 0, 255));
	}

	// export the diffuse channel bmp
	FFileHelper::CreateBitmap(*BMPFilename, OutSize.X, OutSize.Y, OutputBMP.GetData());
}

/**
 * Exports the GOBJObjects array to the given Archive
 *
 * @param FileAr The main archive to output device. However, if MemAr exists, it will write to that until and flush it out for each object
 * @param MemAr Optional string output device for caching writes to
 * @param Warn Feedback context for updating status
 * @param OBJFilename Name of the main OBJ file to export to, used for tagalong files (.mtl, etc)
 * @param Objects The list of objects to export
 * @param Materials Optional list of materials to export
 */
void ExportOBJs(FOutputDevice& FileAr, FStringOutputDevice* MemAr, FFeedbackContext* Warn, const FString& OBJFilename, TArray<FOBJGeom*>& Objects, const TSet<UMaterialInterface*>* Materials, uint32 &IndexOffset)
{
	// write to the memory archive if it exists, otherwise use the FileAr
	FOutputDevice& Ar = MemAr ? *MemAr : FileAr;

	//Make sure we don't corrupt the obj file with terminator line
	FileAr.SetAutoEmitLineTerminator(false);

	// export extra material info if we added any
	if (Materials)
	{
		// make a .MTL file next to the .obj file that contains the materials
		FString MaterialLibFilename = FPaths::GetBaseFilename(OBJFilename, false) + TEXT(".mtl");

		// use the output device file, just like the Exporter makes for the .obj, no backup
		TUniquePtr<FOutputDeviceFile> MaterialLib = MakeUnique<FOutputDeviceFile>(*MaterialLibFilename, true);
		MaterialLib->SetSuppressEventTag(true);
		MaterialLib->SetAutoEmitLineTerminator(false);

		// export the material set to a mtllib
		int32 MaterialIndex = 0;
		for (TSet<UMaterialInterface*>::TConstIterator It(*Materials); It; ++It, ++MaterialIndex)
		{
			UMaterialInterface* Material = *It;
			FString MaterialName = FixupMaterialName(Material);

			// export the material info
			MaterialLib->Logf(TEXT("newmtl %s\r\n"), *MaterialName);

			{
				FString BMPFilename = FPaths::Combine(FPaths::GetPath(MaterialLibFilename), MaterialName + TEXT("_D.bmp"));
				ExportMaterialPropertyTexture(BMPFilename, Material, MP_BaseColor);
				MaterialLib->Logf(TEXT("\tmap_Kd %s\r\n"), *FPaths::GetCleanFilename(BMPFilename));
			}

			{
				FString BMPFilename = FPaths::Combine(FPaths::GetPath(MaterialLibFilename), MaterialName + TEXT("_S.bmp"));
				ExportMaterialPropertyTexture(BMPFilename, Material, MP_Specular);
				MaterialLib->Logf(TEXT("\tmap_Ks %s\r\n"), *FPaths::GetCleanFilename(BMPFilename));
			}

			{
				FString BMPFilename = FPaths::Combine(FPaths::GetPath(MaterialLibFilename), MaterialName + TEXT("_N.bmp"));
				ExportMaterialPropertyTexture(BMPFilename, Material, MP_Normal);
				MaterialLib->Logf(TEXT("\tbump %s\r\n"), *FPaths::GetCleanFilename(BMPFilename));
			}

			MaterialLib->Logf(TEXT("\r\n"));
		}

		Ar.Logf(TEXT("mtllib %s\n"), *FPaths::GetCleanFilename(MaterialLibFilename));
	}

	for( int32 o = 0 ; o < Objects.Num() ; ++o )
	{
		FOBJGeom* object = Objects[o];
		UMaterialInterface* CurrentMaterial = NULL;

		// Object header

		Ar.Logf( TEXT("g %s\n"), *object->Name );
		Ar.Logf( TEXT("\n") );

		// Verts

		for( int32 f = 0 ; f < object->VertexData.Num() ; ++f )
		{
			const FOBJVertex& vertex = object->VertexData[f];
			const FVector& vtx = vertex.Vert;

			Ar.Logf( TEXT("v %.4f %.4f %.4f\n"), vtx.X, vtx.Z, vtx.Y );
		}

		Ar.Logf( TEXT("\n") );

		// Texture coordinates

		for( int32 f = 0 ; f < object->VertexData.Num() ; ++f )
		{
			const FOBJVertex& face = object->VertexData[f];
			const FVector2D& uv = face.UV;

			Ar.Logf( TEXT("vt %.4f %.4f\n"), uv.X, 1.0f - uv.Y );
		}

		Ar.Logf( TEXT("\n") );

		// Normals

		for( int32 f = 0 ; f < object->VertexData.Num() ; ++f )
		{
			const FOBJVertex& face = object->VertexData[f];
			const FVector& Normal = face.Normal;

			Ar.Logf( TEXT("vn %.3f %.3f %.3f\n"), Normal.X, Normal.Z, Normal.Y );
		}

		Ar.Logf( TEXT("\n") );
		
		// Faces

		for( int32 f = 0 ; f < object->Faces.Num() ; ++f )
		{
			const FOBJFace& face = object->Faces[f];

			if( face.Material != CurrentMaterial )
			{
				CurrentMaterial = face.Material;
				Ar.Logf( TEXT("usemtl %s\n"), *FixupMaterialName(face.Material));
			}

			Ar.Logf( TEXT("f ") );

			for( int32 v = 0 ; v < 3 ; ++v )
			{
				// +1 as Wavefront files are 1 index based
				uint32 VertexIndex = IndexOffset + face.VertexIndex[v] + 1; 
				Ar.Logf( TEXT("%d/%d/%d "), VertexIndex, VertexIndex, VertexIndex);
			}

			Ar.Logf( TEXT("\n") );
		}

		IndexOffset += object->VertexData.Num();

		Ar.Logf( TEXT("\n") );

		// dump to disk so we don't run out of memory ganging up all objects
		if (MemAr)
		{
			FileAr.Log(*MemAr);
			FileAr.Flush();
			MemAr->Empty();
		}

		// we are now done with the object, free it
		delete object;
		Objects[o] = NULL;
	}
}

/**
 * Compiles the selection order array by putting every geometry object
 * with a valid selection index into the array, and then sorting it.
 */
struct FCompareMaterial
{
	FORCEINLINE bool operator()( const FOBJFace& InA, const FOBJFace& InB ) const
	{
		PTRINT A = (PTRINT)(InA.Material);
		PTRINT B = (PTRINT)(InB.Material);

		return (A < B) ? true : false;
	}
};

/*------------------------------------------------------------------------------
	ULevelExporterLOD implementation.
------------------------------------------------------------------------------*/
ULevelExporterLOD::ULevelExporterLOD(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UWorld::StaticClass();
	bText = true;
	bForceFileOperations = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("LOD.OBJ"));
	FormatDescription.Add(TEXT("Object File for LOD"));
}

bool ULevelExporterLOD::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& FileAr, FFeedbackContext* Warn, uint32 PortFlags)
{
	GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "ExportingLevelToLOD OBJ", "Exporting Level To LOD OBJ"), true );

	// containers to hold exportable objects and their materials
	TArray<FOBJGeom*> Objects;
	TSet<UMaterialInterface*> Materials;

	UWorld* World = CastChecked<UWorld>(Object);

	// write to memory to buffer file writes
	FStringOutputDevice Ar;

	// OBJ file header
	Ar.Logf (TEXT("# LOD OBJ File Generated by UnrealEd\n"));
	Ar.Logf( TEXT("\n") );

	TArray<AActor*> ActorsToExport;
	for( FActorIterator It(World); It; ++It )
	{
		AActor* Actor = *It;
		// only export selected actors if the flag is set
		if( !Actor || (bSelectedOnly && !IsObjectSelectedForExport(Context, Actor)))
		{
			continue;
		}

		ActorsToExport.Add(Actor);
	}

	// Export actors
	uint32 IndexOffset = 0;
	for (int32 Index = 0; Index < ActorsToExport.Num(); ++Index)
	{
		AActor* Actor = ActorsToExport[Index];
		Warn->StatusUpdate( Index, ActorsToExport.Num(), NSLOCTEXT("UnrealEd", "ExportingLevelToOBJ", "Exporting Level To OBJ") );

		// for now, only export static mesh actors
		if (Cast<AStaticMeshActor>(Actor) == NULL)
		{
			continue;
		}

		// export any actor that passes the tests
		AddActorToOBJs(Actor, Objects, &Materials, bSelectedOnly);

		for( int32 o = 0 ; o < Objects.Num() ; ++o )
		{
			FOBJGeom* object = Objects[o];
			object->Faces.Sort(FCompareMaterial());
		}

		// Export to the OBJ file
		ExportOBJs(FileAr, &Ar, Warn, CurrentFilename, Objects, &Materials, IndexOffset);
		Objects.Reset();
	}

	// OBJ file footer
	Ar.Logf (TEXT("# dElaernU yb detareneG eliF JBO DOL\n"));

	GWarn->EndSlowTask();

	// dump the rest to the file
	FileAr.Log(*Ar);

	return true;
}


/*------------------------------------------------------------------------------
	ULevelExporterOBJ implementation.
------------------------------------------------------------------------------*/

static void ExportPolys( UPolys* Polys, int32 &PolyNum, int32 TotalPolys, FFeedbackContext* Warn, bool bSelectedOnly, UModel* Model, TArray<FOBJGeom*>& Objects )
{
	FOBJGeom* OBJGeom = new FOBJGeom( TEXT("BSP") );

	for( int32 i=0; i<Model->Nodes.Num(); i++ )
	{
		FBspNode* Node = &Model->Nodes[i];
		FBspSurf& Surf = Model->Surfs[Node->iSurf];

		if( (Surf.PolyFlags & PF_Selected) || !bSelectedOnly )
		{
			const FVector& TextureBase = (FVector)Model->Points[Surf.pBase];
			const FVector& TextureX = (FVector)Model->Vectors[Surf.vTextureU];
			const FVector& TextureY = (FVector)Model->Vectors[Surf.vTextureV];
			const FVector& Normal = (FVector)Model->Vectors[Surf.vNormal];

			FPoly Poly;
			GEditor->polyFindBrush( Model, Node->iSurf, Poly );

			// Triangulate this node and generate an OBJ face from the vertices.
			for(int32 StartVertexIndex = 1;StartVertexIndex < Node->NumVertices-1;StartVertexIndex++)
			{
				int32 TriangleIndex = OBJGeom->Faces.AddZeroed();
				FOBJFace& OBJFace = OBJGeom->Faces[TriangleIndex];
				int32 VertexIndex = OBJGeom->VertexData.AddZeroed(3);
				FOBJVertex* Vertices = &OBJGeom->VertexData[VertexIndex];

				OBJFace.VertexIndex[0] = VertexIndex;
				OBJFace.VertexIndex[1] = VertexIndex + 1;
				OBJFace.VertexIndex[2] = VertexIndex + 2;

				// These map the node's vertices to the 3 triangle indices to triangulate the convex polygon.
				int32 TriVertIndices[3] = { Node->iVertPool,
										  Node->iVertPool + StartVertexIndex,
										  Node->iVertPool + StartVertexIndex + 1 };

				for(uint32 TriVertexIndex = 0; TriVertexIndex < 3; TriVertexIndex++)
				{
					const FVert& Vert = Model->Verts[TriVertIndices[TriVertexIndex]];
					const FVector& Vertex = (FVector)Model->Points[Vert.pVertex];

					double U = ((Vertex - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
					double V = ((Vertex - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();

					Vertices[TriVertexIndex].Vert = Vertex;
					Vertices[TriVertexIndex].UV = FVector2D( U, V );
					Vertices[TriVertexIndex].Normal = Normal;
				}
			}
		}
	}

	// Save the object representing the BSP into the OBJ pool
	if( OBJGeom->Faces.Num() > 0 )
	{
		Objects.Add( OBJGeom );
	}
	else
	{
		delete OBJGeom;
	}
}

ULevelExporterOBJ::ULevelExporterOBJ(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
	bText = true;
	bForceFileOperations = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("OBJ"));
	FormatDescription.Add(TEXT("Object File"));
}

bool ULevelExporterOBJ::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& FileAr, FFeedbackContext* Warn, uint32 PortFlags)
{
	TSet<UMaterialInterface*> GlobalMaterials;
	TSet<UMaterialInterface*> *Materials = 0;

	int32 YesNoCancelReply = EAppReturnType::Yes;
	if (!(GIsAutomationTesting || FApp::IsUnattended() || ExportTask->bAutomated))
	{
		YesNoCancelReply = FMessageDialog::Open( EAppMsgType::YesNoCancel, NSLOCTEXT("UnrealEd", "Prompt_OBJExportWithBMP", "Would you like to export the materials as images (slower)?"));
	}
	else
	{
		UE_LOGF(LogEditorExporters, Display, "Executing OBJ automated export, materials will be exported as images by default.");
	}

	switch (YesNoCancelReply)
	{
	case EAppReturnType::Yes:
			Materials = &GlobalMaterials;
			break;

	case EAppReturnType::No:
			break;

	case EAppReturnType::Cancel:
			return 1;
	}


	GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "ExportingLevelToOBJ", "Exporting Level To OBJ"), true );

	// container to hold all exportable objects
	TArray<FOBJGeom*> Objects;

	UWorld* World = CastChecked<UWorld>(Object);

	GEditor->bspBuildFPolys(World->GetModel(), 0, 0 );
	UPolys* Polys = World->GetModel()->Polys;

	// write to memory to buffer file writes
	FStringOutputDevice Ar;

	// OBJ file header

	Ar.Logf (TEXT("# OBJ File Generated by UnrealEd\n"));
	Ar.Logf( TEXT("\n") );

	uint32 IndexOffset = 0;
	// Export the BSP

	int32 Dummy;
	ExportPolys( Polys, Dummy, 0, Warn, bSelectedOnly, World->GetModel(), Objects );
	// Export polys to the OBJ file
	ExportOBJs(FileAr, &Ar, Warn, CurrentFilename, Objects, NULL, IndexOffset);
	Objects.Reset();
	// Export actors
	
	TArray<AActor*> ActorsToExport;
	for( FActorIterator It(World); It; ++It )
	{
		AActor* Actor = *It;
		// only export selected actors if the flag is set
		if( !Actor || (bSelectedOnly && !IsObjectSelectedForExport(Context, Actor)))
		{
			continue;
		}

		ActorsToExport.Add(Actor);
	}

	for( int32 Index = 0; Index < ActorsToExport.Num(); ++Index )
	{
		AActor* Actor = ActorsToExport[Index];
		Warn->StatusUpdate( Index, ActorsToExport.Num(), NSLOCTEXT("UnrealEd", "ExportingLevelToOBJ", "Exporting Level To OBJ") );

		// try to export every object
		AddActorToOBJs(Actor, Objects, Materials, bSelectedOnly);

		for( int32 o = 0 ; o < Objects.Num() ; ++o )
		{
			FOBJGeom* object = Objects[o];
			object->Faces.Sort(FCompareMaterial());
		}
	}

	// Export to the OBJ file
	ExportOBJs(FileAr, &Ar, Warn, CurrentFilename, Objects, Materials, IndexOffset);
	Objects.Reset();

	// OBJ file footer
	Ar.Logf (TEXT("# dElaernU yb detareneG eliF JBO\n"));

	GWarn->EndSlowTask();

	// write anything left in the memory Ar to disk
	FileAr.Log(*Ar);

	return 1;
}

/*------------------------------------------------------------------------------
	ULevelExporterFBX implementation.
------------------------------------------------------------------------------*/
ULevelExporterFBX::ULevelExporterFBX(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UWorld::StaticClass();
	bText = false;
	bForceFileOperations = false;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("FBX"));
	FormatDescription.Add(TEXT("FBX File"));
}

bool IsSomethingToExport(AActor* Actor, TArray<FString>& OutReasons)
{
	check(Actor);

	if (ALevelInstance* LevelInstance = Cast<ALevelInstance>(Actor))
	{
		OutReasons.Add(FString::Printf(TEXT("%s: Exporting Level Instances to FBX is not supported."), *Actor->GetActorLabel()));
		return false;
	}

	TInlineComponentArray<USceneComponent*> ComponentsToExport;
	for (UActorComponent* ActorComp : Actor->GetComponents())
	{
		USceneComponent* Component = Cast<USceneComponent>(ActorComp);

		if (Component == nullptr || Component->bHiddenInGame)
		{
			//Skip hidden component like camera mesh or other editor helper
			continue;
		}

		UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(Component);
		USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(Component);
		UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>(Component);

		if ((StaticMeshComp && StaticMeshComp->GetStaticMesh()) ||
			(SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset()) ||
			Component->IsA(UCameraComponent::StaticClass()) ||
			Component->IsA(ULightComponent::StaticClass()) ||
			(ChildActorComp && ChildActorComp->GetChildActor() && IsSomethingToExport(ChildActorComp->GetChildActor(), OutReasons))
			)
		{
			return true;
		}
	}
	return false;
}

bool IsSomethingToExport(ULevel* InLevel, bool bSelectedOnly, TArray<FString>& OutReasons)
{
	TArray<AActor*> ActorToExport;
	int32 ActorCount = InLevel->Actors.Num();
	for (int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex)
	{
		AActor* Actor = InLevel->Actors[ActorIndex];
		if (Actor != nullptr && (!bSelectedOnly || (bSelectedOnly && Actor->IsSelected())))
		{
			if (IsSomethingToExport(Actor, OutReasons))
			{
				return true;
			}
			else if(Actor->IsA(ALandscapeProxy::StaticClass()) || Actor->IsA(AEmitter::StaticClass()))
			{
				return true;
			}
			else if (Actor->IsA(ABrush::StaticClass()))
			{
				ABrush* BrushActor = Cast<ABrush>(Actor);
				UModel* Model = BrushActor->GetBrushComponent()->Brush;
				if (Model != NULL && Model->VertexBuffer.Vertices.Num() >= 3 && Model->MaterialIndexBuffers.Num() != 0)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool IsSomethingToExport(UWorld* World, bool bSelectedOnly, TArray<FString>& OutReasons)
{
	if (bSelectedOnly && World->GetModel()->Nodes.Num())
	{
		return true;
	}
	ULevel* Level = World->PersistentLevel;
	if (IsSomethingToExport(Level, bSelectedOnly, OutReasons))
	{
		return true;
	}
	// Export streaming levels and actors
	for (int32 CurLevelIndex = 0; CurLevelIndex < World->GetNumLevels(); ++CurLevelIndex)
	{
		ULevel* CurLevel = World->GetLevel(CurLevelIndex);
		if (CurLevel != NULL && CurLevel != Level)
		{
			if (IsSomethingToExport(CurLevel, bSelectedOnly, OutReasons))
			{
				return true;
			}
		}
	}
	return false;
}

void OpenOutputLogTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog"));
}

bool ULevelExporterFBX::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	UWorld* World = CastChecked<UWorld>(Object);
	TArray<FString> GeneratedWarnings;
	//Check if there is something to export
	if (!IsSomethingToExport(World, bSelectedOnly, GeneratedWarnings))
	{
		if (GeneratedWarnings.Num() > 0)
		{
			for (const FString& WarningMessage : GeneratedWarnings)
			{
				UE_LOGF(LogEditorExporters, Warning, "%ls", *WarningMessage);
			}
		}
		else
		{
			UE_LOGF(LogEditorExporters, Warning, "There is nothing to export to a fbx file.");
		}

		if (!GIsAutomationTesting && !FApp::IsUnattended())
		{
			FNotificationInfo* NotificationInfo = new FNotificationInfo(FText::GetEmpty());

			if (GeneratedWarnings.Num() > 0)
			{
				NotificationInfo->Text = FText(NSLOCTEXT("UnrealEd", "ExportingLevelToFBX_GeneratedWarnings", "Some warnings were generated while exporting to FBX!"));
				NotificationInfo->Hyperlink = FSimpleDelegate::CreateStatic(OpenOutputLogTab);
				NotificationInfo->HyperlinkText = NSLOCTEXT("UnrealEd", "ShowOutputLogButton", "Show Output Log");
				NotificationInfo->ExpireDuration = 8.0f;
			}
			else
			{
				NotificationInfo->Text = bSelectedOnly
					? FText(NSLOCTEXT("UnrealEd", "ExportingLevelToFBX_Selection", "The selection has nothing that can be exported to FBX!"))
					: FText(NSLOCTEXT("UnrealEd", "ExportingLevelToFBX_World", "The world has nothing that can be exported to FBX!"));
				NotificationInfo->ExpireDuration = 5.0f;
			}
			//QueueNotification will add the pointer to a list and the FSlateNotificationManager::Tick will delete the pointer when adding the notification to the active list.
			FSlateNotificationManager::Get().QueueNotification(NotificationInfo);
		}
		return false;
	}

	GWarn->BeginSlowTask(NSLOCTEXT("UnrealEd", "ExportingLevelToFBX", "Exporting Level To FBX"), true);

	FScopedFbxExporterInstance ScopedExporterInstance;
	bool ExportCancel = false;
	if (UFbxExportOption* AutomatedExportOptions = GetAutomatedExportOptionsFbx())
	{
		ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(AutomatedExportOptions);
	}
	else
	{
		//Show the fbx export dialog options
		bool ExportAll = false;
		ScopedExporterInstance.GetExporter()->FillExportOptions(false, true, UExporter::CurrentFilename, ExportCancel, ExportAll);
	}

	if (!ExportCancel)
	{
		ScopedExporterInstance.GetExporter()->CreateDocument();

		GWarn->StatusUpdate(0, 1, NSLOCTEXT("UnrealEd", "ExportingLevelToFBX", "Exporting Level To FBX"));
		{
			ULevel* Level = World->PersistentLevel;

			if (bSelectedOnly)
			{
				ScopedExporterInstance.GetExporter()->ExportBSP(World->GetModel(), true);
			}

			class FLevelExporterFBXNodeNameAdapter : public INodeNameAdapter
			{
			public:
				virtual FString GetActorNodeName(const AActor* InActor) override
				{
					if (ActorNames.Contains(InActor))
					{
						return ActorNames[InActor].ToString();
					}

					FString ActorNodeName = InActor->GetActorLabel();

					if (UniqueActorNames.Contains(ActorNodeName))
					{
						FString ActorNodeNamePrefix = ActorNodeName;
						int32 ActorNodeNameIndex = 1;
						FActorLabelUtilities::SplitActorLabel(ActorNodeNamePrefix, ActorNodeNameIndex);

						do
						{
							ActorNodeName = FString::Printf(TEXT("%s%d"), *ActorNodeNamePrefix, ++ActorNodeNameIndex);
						}
						while (UniqueActorNames.Contains(ActorNodeName));
					}

					UniqueActorNames.Add(ActorNodeName);
					ActorNames.Add(InActor) = FName(ActorNodeName);

					return ActorNodeName;
				}

			private:
				TSet<FString> UniqueActorNames;
				TMap<const AActor*, FName> ActorNames;
			};

			FLevelExporterFBXNodeNameAdapter NodeNameAdapter;

			ScopedExporterInstance.GetExporter()->ExportLevelMesh(Level, bSelectedOnly, NodeNameAdapter);

			// Export streaming levels and actors
			for (int32 CurLevelIndex = 0; CurLevelIndex < World->GetNumLevels(); ++CurLevelIndex)
			{
				ULevel* CurLevel = World->GetLevel(CurLevelIndex);
				if (CurLevel != NULL && CurLevel != Level)
				{
					ScopedExporterInstance.GetExporter()->ExportLevelMesh(CurLevel, bSelectedOnly, NodeNameAdapter);
				}
			}
		}
		ScopedExporterInstance.GetExporter()->WriteToFile(*UExporter::CurrentFilename);
	}

	ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(nullptr);

	GWarn->EndSlowTask();

	return true;
}

/*------------------------------------------------------------------------------
	UPolysExporterOBJ implementation.
------------------------------------------------------------------------------*/
UPolysExporterOBJ::UPolysExporterOBJ(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UPolys::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("OBJ"));
	FormatDescription.Add(TEXT("Object File"));

}

bool UPolysExporterOBJ::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	TArray<FOBJGeom*> Objects;

	UPolys* Polys = CastChecked<UPolys> (Object);

	int32 PolyNum = 0;
	int32 TotalPolys = Polys->Element.Num();

	Ar.Logf (TEXT("# OBJ File Generated by UnrealEd\n"));

	UWorld* World = CastChecked<UWorld>(Object);
	ExportPolys( Polys, PolyNum, TotalPolys, Warn, false, World->GetModel(), Objects );

	for( int32 o = 0 ; o < Objects.Num() ; ++o )
	{
		FOBJGeom* object = Objects[o];
		object->Faces.Sort(FCompareMaterial());
	}

	uint32 IndexOffset = 0;
	// Export to the OBJ file
	ExportOBJs(Ar, NULL, Warn, CurrentFilename, Objects, NULL, IndexOffset);

	Ar.Logf (TEXT("# dElaernU yb detareneG eliF JBO\n"));

	return 1;
}


/*------------------------------------------------------------------------------
	USequenceExporterT3D implementation.
------------------------------------------------------------------------------*/
USequenceExporterT3D::USequenceExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool USequenceExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	return true;
}

/*------------------------------------------------------------------------------
UStaticMeshExporterOBJ implementation.
------------------------------------------------------------------------------*/
UStaticMeshExporterOBJ::UStaticMeshExporterOBJ(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UStaticMesh::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("OBJ"));
	FormatDescription.Add(TEXT("Object File"));

}

bool UStaticMeshExporterOBJ::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	UStaticMesh* StaticMesh = CastChecked<UStaticMesh>( Object );

	{
		// Currently, we only export LOD 0 of the static mesh. In the future, we could potentially export all available LODs
		const FStaticMeshLODResources& RenderData = StaticMesh->GetLODForExport(0);
		uint32 Count = RenderData.GetNumTriangles();

		bool bHasUVLightMap = RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() > 1;

		// Create a new filename for the lightmap coordinate OBJ file (by adding "_UV1" to the end of the filename)
		FString Filename = UExporter::CurrentFilename.Left( UExporter::CurrentFilename.Len() - 4 ) + "_UV1." + UExporter::CurrentFilename.Right( 3 );

		// Open a second archive here so we can export lightmap coordinates at the same time we export the regular mesh
		FArchive* UV1File = bHasUVLightMap ? IFileManager::Get().CreateFileWriter( *Filename ) : nullptr;

		TArray<FVector> Verts;				// The verts in the mesh
		TArray<FVector2D> UVs;				// UV coords from channel 0
		TArray<FVector2D> UVLMs;			// Lightmap UVs from channel 1
		TArray<FVector> Normals;			// Normals
		TArray<uint32> SmoothingMasks;		// Complete collection of the smoothing groups from all triangles
		TArray<uint32> UniqueSmoothingMasks;	// Collection of the unique smoothing groups (used when writing out the face info into the OBJ file so we can group by smoothing group)

		if (bHasUVLightMap)
		{
			UV1File->Logf(TEXT("# UnrealEd OBJ exporter\r\n"));
		}
		Ar.Log( TEXT("# UnrealEd OBJ exporter\r\n") );

		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);
		if (MeshDescription != nullptr)
		{
			uint32 TriangleCount = MeshDescription->Triangles().Num();
			if (Count == TriangleCount)
			{
				SmoothingMasks.AddZeroed(TriangleCount);

				FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(*MeshDescription, SmoothingMasks);
				for (uint32 SmoothValue : SmoothingMasks)
				{
					UniqueSmoothingMasks.AddUnique(SmoothValue);
				}
			}
		}

		//This can happen in case the base LOD is reduce or the recompute normals has generate a different number of cases
		//the count will be lower and in such a case the smooth group of the original mesh description cannot be used since it is out of sync with the render data.
		if (SmoothingMasks.Num() != Count)
		{
			//Create one SmoothGroup with 0 values
			SmoothingMasks.Empty(Count);
			SmoothingMasks.AddZeroed(Count);
			UniqueSmoothingMasks.Empty(1);
			UniqueSmoothingMasks.Add(0);
		}

		// Collect all the data about the mesh
		Verts.Reserve(3 * Count);
		UVs.Reserve(3 * Count);
		UVLMs.Reserve(3 * Count);
		Normals.Reserve(3 * Count);

		FIndexArrayView Indices = RenderData.IndexBuffer.GetArrayView();
		
		for( uint32 tri = 0 ; tri < Count ; tri++ )
		{
			uint32 Index1 = Indices[(tri * 3) + 0];
			uint32 Index2 = Indices[(tri * 3) + 1];
			uint32 Index3 = Indices[(tri * 3) + 2];

			FVector Vertex1 = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(Index1);		//(FStaticMeshFullVertex*)(RawVertexData + Index1 * VertexStride);
			FVector Vertex2 = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(Index2);
			FVector Vertex3 = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(Index3);
			
			// Vertices
			Verts.Add( Vertex1 );
			Verts.Add( Vertex2 );
			Verts.Add( Vertex3 );

			// UVs from channel 0
			UVs.Add( FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, 0)) );
			UVs.Add( FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, 0)) );
			UVs.Add( FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index3, 0)) );

			// UVs from channel 1 (lightmap coords)
			if (bHasUVLightMap)
			{
				UVLMs.Add(FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index1, 1)));
				UVLMs.Add(FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index2, 1)));
				UVLMs.Add(FVector2D(RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(Index3, 1)));
			}

			// Normals
			Normals.Add( FVector4(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index1)) );
			Normals.Add( FVector4(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index2)) );
			Normals.Add( FVector4(RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(Index3)) );
		}

		// Write out the vertex data
		if (bHasUVLightMap)
		{
			UV1File->Logf(TEXT("\r\n"));
		}
		Ar.Log( TEXT("\r\n") );
		for( int32 v = 0 ; v < Verts.Num() ; ++v )
		{
			// Transform to Lightwave's coordinate system
			if (bHasUVLightMap)
			{
				UV1File->Logf(TEXT("v %f %f %f\r\n"), Verts[v].X, Verts[v].Z, Verts[v].Y);
			}
			Ar.Logf( TEXT("v %f %f %f\r\n"), Verts[v].X, Verts[v].Z, Verts[v].Y );
		}

		// Write out the UV data (the lightmap file differs here in that it writes from the UVLMs array instead of UVs)
		if (bHasUVLightMap)
		{
			UV1File->Logf(TEXT("\r\n"));
		}
		Ar.Log( TEXT("\r\n") );
		for( int32 uv = 0 ; uv < UVs.Num() ; ++uv )
		{
			// Invert the y-coordinate (Lightwave has their bitmaps upside-down from us).
			if (bHasUVLightMap)
			{
				UV1File->Logf(TEXT("vt %f %f\r\n"), UVLMs[uv].X, 1.0f - UVLMs[uv].Y);
			}
			Ar.Logf( TEXT("vt %f %f\r\n"), UVs[uv].X, 1.0f - UVs[uv].Y );
		}

		// Write object header
		if (bHasUVLightMap)
		{
			UV1File->Logf(TEXT("\r\n"));
			UV1File->Logf(TEXT("g UnrealEdObject\r\n"));
			UV1File->Logf(TEXT("\r\n"));
		}
		Ar.Log( TEXT("\r\n") );
		Ar.Log( TEXT("g UnrealEdObject\r\n") );
		Ar.Log( TEXT("\r\n") );

		// Write out the face windings, sectioned by unique smoothing groups

		int32 SmoothingGroup = 0;

		for( int32 sm = 0 ; sm < UniqueSmoothingMasks.Num() ; ++sm )
		{
			if (bHasUVLightMap)
			{
				UV1File->Logf(TEXT("s %i\r\n"), SmoothingGroup);
			}
			Ar.Logf( TEXT("s %i\r\n"), SmoothingGroup );
			SmoothingGroup++;

			for( int32 tri = 0 ; tri < RenderData.GetNumTriangles() ; tri++ )
			{
				if( SmoothingMasks[tri] == UniqueSmoothingMasks[sm]  )
				{
					int idx = 1 + (tri * 3);
					if (bHasUVLightMap)
					{
						UV1File->Logf(TEXT("f %d/%d %d/%d %d/%d\r\n"), idx, idx, idx + 1, idx + 1, idx + 2, idx + 2);
					}
					Ar.Logf( TEXT("f %d/%d %d/%d %d/%d\r\n"), idx, idx, idx+1, idx+1, idx+2, idx+2 );
				}
			}
		}

		// Write out footer
		if (bHasUVLightMap)
		{
			UV1File->Logf(TEXT("\r\n"));
			UV1File->Logf(TEXT("g\r\n"));
		}
		Ar.Log( TEXT("\r\n") );
		Ar.Log( TEXT("g\r\n") );

		if (bHasUVLightMap)
		{
			// Clean up and finish
			delete UV1File;
		}
	}

	// ------------------------------------------------------

	// 
	{
		// Create a new filename for the internal OBJ file (by adding "_Internal" to the end of the filename)
		FString Filename = UExporter::CurrentFilename.Left( UExporter::CurrentFilename.Len() - 4 ) + "_Internal." + UExporter::CurrentFilename.Right( 3 );

		// Open another archive
		FArchive* File = IFileManager::Get().CreateFileWriter( *Filename );

		File->Logf( TEXT("# UnrealEd OBJ exporter (_Internal)\r\n") );

		// Currently, we only export LOD 0 of the static mesh. In the future, we could potentially export all available LODs
		const FStaticMeshLODResources& RenderData = StaticMesh->GetLODForExport(0);
		uint32 VertexCount = RenderData.GetNumVertices();

		check(VertexCount == RenderData.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices());

		File->Logf( TEXT("\r\n") );
		for(uint32 i = 0; i < VertexCount; i++)
		{
			const FVector& OSPos = (FVector)RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition( i );
//			const FVector WSPos = StaticMeshComponent->LocalToWorld.TransformPosition( OSPos );
			const FVector WSPos = OSPos;

			// Transform to Lightwave's coordinate system
			File->Logf( TEXT("v %f %f %f\r\n"), WSPos.X, WSPos.Z, WSPos.Y );
		}

		File->Logf( TEXT("\r\n") );
		for(uint32 i = 0 ; i < VertexCount; ++i)
		{
			// takes the first UV
			const FVector2f UV = RenderData.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 0);

			// Invert the y-coordinate (Lightwave has their bitmaps upside-down from us).
			File->Logf( TEXT("vt %f %f\r\n"), UV.X, 1.0f - UV.Y );
		}

		File->Logf( TEXT("\r\n") );

		for(uint32 i = 0 ; i < VertexCount; ++i)
		{
			const FVector3f& OSNormal = RenderData.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ( i ); 
			const FVector WSNormal = (FVector)OSNormal;

			// Transform to Lightwave's coordinate system
			File->Logf( TEXT("vn %f %f %f\r\n"), WSNormal.X, WSNormal.Z, WSNormal.Y );
		}

		{
			FIndexArrayView Indices = RenderData.IndexBuffer.GetArrayView();
			uint32 NumIndices = Indices.Num();

			check(NumIndices % 3 == 0);
			for(uint32 i = 0; i < NumIndices / 3; i++)
			{
				// Wavefront indices are 1 based
				uint32 a = Indices[3 * i + 0] + 1;
				uint32 b = Indices[3 * i + 1] + 1;
				uint32 c = Indices[3 * i + 2] + 1;

				File->Logf( TEXT("f %d/%d/%d %d/%d/%d %d/%d/%d\r\n"), 
					a,a,a,
					b,b,b,
					c,c,c);
			}
		}

		delete File;
	}

	return true;
}

/*------------------------------------------------------------------------------
UExporterFBX implementation.
------------------------------------------------------------------------------*/
UFbxExportOption* UExporterFBX::GetAutomatedExportOptionsFbx()
{
	if (ExportTask && ExportTask->bAutomated)
	{
		return Cast<UFbxExportOption>(ExportTask->Options);
	}

	return nullptr;
}

namespace UE::EditorExporters::Private
{
	bool SetupExporterForAutomatedFBXExport(FScopedFbxExporterInstance& ScopedExporterInstance, UExporterFBX& CurrentExporter)
	{
		if (CurrentExporter.ExportTask && CurrentExporter.ExportTask->bAutomated)
		{
			if (UFbxExportOption* AutomatedExportOptions = CurrentExporter.GetAutomatedExportOptionsFbx())
			{
				ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(AutomatedExportOptions);
			}

			CurrentExporter.SetShowExportOption(false);
			return true;
		}

		return false;
	}
}

/*------------------------------------------------------------------------------
UStaticMeshExporterFBX implementation.
------------------------------------------------------------------------------*/
UStaticMeshExporterFBX::UStaticMeshExporterFBX(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{


	SupportedClass = UStaticMesh::StaticClass();
	bText = false;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("FBX"));
	FormatDescription.Add(TEXT("FBX File"));

}

bool UStaticMeshExporterFBX::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	UStaticMesh* StaticMesh = CastChecked<UStaticMesh>( Object );
	FScopedFbxExporterInstance ScopedExporterInstance;
	if (!UE::EditorExporters::Private::SetupExporterForAutomatedFBXExport(ScopedExporterInstance, *this))
	{
		//Show the fbx export dialog options
		bool ExportAll = GetBatchMode() && !GetShowExportOption();
		bool ExportCancel = false;
		ScopedExporterInstance.GetExporter()->FillExportOptions(GetBatchMode(), GetShowExportOption(), UExporter::CurrentFilename, ExportCancel, ExportAll);
		if (ExportCancel)
		{
			SetCancelBatch(GetBatchMode());
			//User cancel the FBX export
			return false;
		}
		SetShowExportOption(!ExportAll);
	}

	ScopedExporterInstance.GetExporter()->CreateDocument();
	ScopedExporterInstance.GetExporter()->ExportStaticMesh(StaticMesh);
	ScopedExporterInstance.GetExporter()->WriteToFile(*UExporter::CurrentFilename);
	ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(nullptr);

	return true;
}


/*------------------------------------------------------------------------------
USkeletalMeshExporterFBX implementation.
------------------------------------------------------------------------------*/
USkeletalMeshExporterFBX::USkeletalMeshExporterFBX(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = USkeletalMesh::StaticClass();
	bText = false;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("FBX"));
	FormatDescription.Add(TEXT("FBX File"));
}

bool USkeletalMeshExporterFBX::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>( Object );
	FScopedFbxExporterInstance ScopedExporterInstance;
	if (!UE::EditorExporters::Private::SetupExporterForAutomatedFBXExport(ScopedExporterInstance, *this))
	{
		//Show the fbx export dialog options
		bool ExportAll = GetBatchMode() && !GetShowExportOption();
		bool ExportCancel = false;
		ScopedExporterInstance.GetExporter()->FillExportOptions(GetBatchMode(), GetShowExportOption(), UExporter::CurrentFilename, ExportCancel, ExportAll);
		if (ExportCancel)
		{
			SetCancelBatch(GetBatchMode());
			//User cancel the FBX export
			return false;
		}
		SetShowExportOption(!ExportAll);
	}

	ScopedExporterInstance.GetExporter()->CreateDocument();
	ScopedExporterInstance.GetExporter()->ExportSkeletalMesh(SkeletalMesh);
	ScopedExporterInstance.GetExporter()->WriteToFile(*UExporter::CurrentFilename);
	ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(nullptr);

	return true;
}

/*------------------------------------------------------------------------------
UAnimSequenceExporterFBX implementation.
------------------------------------------------------------------------------*/
UAnimSequenceExporterFBX::UAnimSequenceExporterFBX(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAnimSequence::StaticClass();
	bText = false;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("FBX"));
	FormatDescription.Add(TEXT("FBX File"));

}

bool UAnimSequenceExporterFBX::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	UAnimSequence* AnimSequence = CastChecked<UAnimSequence>( Object );
	USkeleton* AnimSkeleton = AnimSequence->GetSkeleton();
	USkeletalMesh* PreviewMesh = nullptr;
	if (AnimSkeleton)
	{
		// first try to get already set preview mesh
		PreviewMesh = AnimSkeleton->GetAssetPreviewMesh(AnimSequence);
		if (!PreviewMesh)
		{
			// find compatible one (as used in persona toolkit)
			PreviewMesh = AnimSkeleton->FindCompatibleMesh();
		}
	}

	if (AnimSkeleton && PreviewMesh)
	{
		FScopedFbxExporterInstance ScopedExporterInstance;
		if (!UE::EditorExporters::Private::SetupExporterForAutomatedFBXExport(ScopedExporterInstance, *this))
		{
			//Show the fbx export dialog options
			bool ExportAll = GetBatchMode() && !GetShowExportOption();
			bool ExportCancel = false;
			ScopedExporterInstance.GetExporter()->FillExportOptions(GetBatchMode(), GetShowExportOption(), UExporter::CurrentFilename, ExportCancel, ExportAll);
			if (ExportCancel)
			{
				SetCancelBatch(GetBatchMode());
				//User cancel the FBX export
				return false;
			}
			SetShowExportOption(!ExportAll);
		}

		ScopedExporterInstance.GetExporter()->CreateDocument();
		ScopedExporterInstance.GetExporter()->ExportAnimSequence(AnimSequence, PreviewMesh, ScopedExporterInstance.GetExporter()->GetExportOptions()->bExportPreviewMesh);
		ScopedExporterInstance.GetExporter()->WriteToFile( *UExporter::CurrentFilename );
		ScopedExporterInstance.GetExporter()->SetExportOptionsOverride(nullptr);

		return true;
	}
	
	if(!AnimSkeleton)
	{
		UE_LOGF(LogEditorExporters, Warning, "Cannot export animation sequence [%ls] because the skeleton is not set.", *AnimSequence->GetName());
	}
	else
	{
		UE_LOGF(LogEditorExporters, Warning, "Cannot export animation sequence [%ls] because the preview mesh is not set.", *AnimSequence->GetName());
	}
	
	return false;
}

void UEditorEngine::RebuildStaticNavigableGeometry(ULevel* Level)
{
	// iterate through all BSPs and gather it's geometry, without any filtering - filtering will be done while building
	// NOTE: any other game-time static geometry can (and should) be added here
	if (Level != NULL)
	{
		Level->StaticNavigableGeometry.Reset();

		if (Level->Model != NULL)
		{
			UModel* Model =  Level->Model;
			int32 TotalPolys = 0;

			TArray<FPoly> TempPolys;

			bspBuildFPolys(Model, 0, 0, &TempPolys);
			UPolys* Polys = Model->Polys;
			int32 PolyNum = TempPolys.Num();

			TotalPolys = TotalPolys + PolyNum;

			for( int32 i = 0; i < Model->Nodes.Num(); ++i )
			{
				FBspNode* Node = &Model->Nodes[i];
				FBspSurf& Surf = Model->Surfs[Node->iSurf];
				
				if (!(Surf.Actor && Surf.Actor->ShouldExportStaticNavigableGeometry()))
				{
					continue;
				}

				const FVector& TextureBase = (FVector)Model->Points[Surf.pBase];
				const FVector& TextureX = (FVector)Model->Vectors[Surf.vTextureU];
				const FVector& TextureY = (FVector)Model->Vectors[Surf.vTextureV];
				const FVector& Normal = (FVector)Model->Vectors[Surf.vNormal];

				FPoly Poly;
				polyFindBrush( Model, Node->iSurf, Poly );

				// Triangulate this node and generate an OBJ face from the vertices.
				for(int32 StartVertexIndex = 1;StartVertexIndex < Node->NumVertices-1;StartVertexIndex++)
				{
					// These map the node's vertices to the 3 triangle indices to triangulate the convex polygon.
					int32 TriVertIndices[3] = { Node->iVertPool,
						Node->iVertPool + StartVertexIndex,
						Node->iVertPool + StartVertexIndex + 1 };

					for(uint32 TriVertexIndex = 0; TriVertexIndex < 3; TriVertexIndex++)
					{
						const FVert& Vert = Model->Verts[TriVertIndices[TriVertexIndex]];
						Level->StaticNavigableGeometry.Add( (FVector)Model->Points[Vert.pVertex] );
					}
				}
			}
		}

		FNavigationSystem::UpdateLevelCollision(*Level);
	}
}

/*-----------------------------------------------------------------------------
	UExportTextContainer
-----------------------------------------------------------------------------*/
UExportTextContainer::UExportTextContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}
