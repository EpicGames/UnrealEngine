// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorFolderUtils.h"

#include "EditorActorFolders.h"

FName FEditorFolderUtils::GetLeafName(const FName& InPath)
{
	FString PathString = InPath.ToString();
	int32 LeafIndex = 0;
	if (PathString.FindLastChar('/', LeafIndex))
	{
		return FName(*PathString.RightChop(LeafIndex + 1));
	}
	else
	{
		return InPath;
	}
}

bool FEditorFolderUtils::PathIsChildOf(const FString& PotentialChild, const FString& Parent)
{
	const int32 ParentLen = Parent.Len();
	return
		PotentialChild.Len() > ParentLen&&
		PotentialChild[ParentLen] == '/' &&
		PotentialChild.Left(ParentLen) == Parent;
}

bool FEditorFolderUtils::PathIsChildOf(const FName& PotentialChild, const FName& Parent)
{
	return PathIsChildOf(PotentialChild.ToString(), Parent.ToString());
}

void FEditorFolderUtils::RenameFolder(const FFolder& InFolder, const FText& NewFolderName, UWorld* World)
{
	if (!World)
	{
		return;
	}
		
	FName NewPath = InFolder.GetParent().GetPath();
	if (NewPath.IsNone())
	{
		NewPath = FName(*NewFolderName.ToString());
	}
	else
	{
		NewPath = FName(*(NewPath.ToString() / NewFolderName.ToString()));
	}
		
	const FFolder TreeItemNewFolder(InFolder.GetRootObject(), NewPath);
	FActorFolders::Get().RenameFolderInWorld(*World, InFolder, TreeItemNewFolder);
}