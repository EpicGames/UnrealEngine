// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/Actor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EdMode.h"
#include "LevelEditor.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

void UUnrealEdEngine::NotifyPreChange(FProperty* PropertyAboutToChange)
{
}

void UUnrealEdEngine::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	// Notify all active modes of actor property changes.
	GLevelEditorModeTools().ActorPropChangeNotify();
}

void UUnrealEdEngine::UpdateFloatingPropertyWindows(bool bForceRefresh, bool bNotifyActorSelectionChanged)
{
	if (const UTypedElementSelectionSet* SelectionSet = GetSelectedActors()->GetElementSelectionSet())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelEditor.BroadcastElementSelectionChanged(SelectionSet, bForceRefresh);

		if (bNotifyActorSelectionChanged)
		{
			// Assemble a set of valid selected actors.
			TArray<UObject*> SelectedActors;
			SelectionSet->ForEachSelectedObject<AActor>([&SelectedActors](AActor* InActor)
			{
				if (IsValidChecked(InActor))
				{
					SelectedActors.Add(InActor);
				}
				return true;
			});
			LevelEditor.BroadcastActorSelectionChanged(SelectedActors, bForceRefresh);
		}
	}
}

void UUnrealEdEngine::UpdateFloatingPropertyWindowsFromActorList(const TArray<AActor*>& ActorList, bool bForceRefresh)
{
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditor.BroadcastOverridePropertyEditorSelection(ActorList, bForceRefresh);
}
