// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
EditorLevelUtils.cpp: Editor-specific level management routines
=============================================================================*/

#include "EditorLevelUtils.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Model.h"
#include "Engine/Brush.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/WorldFactory.h"
#include "Editor/GroupActor.h"
#include "EngineGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ReferenceChainSearch.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "EditorSupportDelegates.h"

#include "BusyCursor.h"
#include "LevelUtils.h"
#include "Layers/LayersSubsystem.h"

#include "ScopedTransaction.h"
#include "ActorEditorUtils.h"
#include "ContentStreaming.h"
#include "PackageTools.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/LevelStreamingVolume.h"
#include "Components/ModelComponent.h"
#include "Misc/RuntimeErrors.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/IConsoleManager.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Dialogs/Dialogs.h"

#include "Algo/AnyOf.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorLevelUtils)

DEFINE_LOG_CATEGORY(LogLevelTools);

#define LOCTEXT_NAMESPACE "EditorLevelUtils"

UEditorLevelUtils::FCanMoveActorToLevelDelegate UEditorLevelUtils::CanMoveActorToLevelDelegate;
UEditorLevelUtils::FOnMoveActorsToLevelEvent UEditorLevelUtils::OnMoveActorsToLevelEvent;

int32 UEditorLevelUtils::MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevelStreaming* DestStreamingLevel, bool bWarnAboutReferences, bool bWarnAboutRenaming)
{
	return MoveActorsToLevel(ActorsToMove, DestStreamingLevel ? DestStreamingLevel->GetLoadedLevel() : nullptr, bWarnAboutReferences, bWarnAboutRenaming, /*bMoveAllOrFail*/false);
}

int32 UEditorLevelUtils::MoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bWarnAboutReferences, bool bWarnAboutRenaming, bool bMoveAllOrFail, TArray<AActor*>* OutActors /*=nullptr*/)
{
	const bool bMoveActors = true;
	return CopyOrMoveActorsToLevel(ActorsToMove, DestLevel, bMoveActors, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, OutActors);
}

int32 UEditorLevelUtils::CopyActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bWarnAboutReferences, bool bWarnAboutRenaming, bool bMoveAllOrFail, TArray<AActor*>* OutActors /*=nullptr*/)
{
	const bool bMoveActors = false;
	return CopyOrMoveActorsToLevel(ActorsToMove, DestLevel, bMoveActors, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, OutActors);
}

int32 UEditorLevelUtils::CopyOrMoveActorsToLevel(const TArray<AActor*>& ActorsToMove, ULevel* DestLevel, bool bMoveActors, bool bWarnAboutReferences /*= true*/, bool bWarnAboutRenaming /*= true*/, bool bMoveAllOrFail /*= false*/, TArray<AActor*>* OutActors /*=nullptr*/)
{
	int32 NumMovedActors = 0;

	if (DestLevel)
	{
		UWorld* OwningWorld = DestLevel->OwningWorld;

		// Backup the current contents of the clipboard string as we'll be using cut/paste features to move actors
		// between levels and this will trample over the clipboard data.
		FString OriginalClipboardContent;
		FPlatformApplicationMisc::ClipboardPaste(OriginalClipboardContent);

		// The final list of actors to move after invalid actors were removed
		TArray<AActor*> FinalMoveList;
		TArray<TWeakObjectPtr<AActor>> FinalWeakMoveList;
		FinalMoveList.Reserve(ActorsToMove.Num());

		bool bIsDestLevelLocked = FLevelUtils::IsLevelLocked(DestLevel);
		int32 ActorCount = 0;
		if (!bIsDestLevelLocked)
		{
			for (AActor* CurActor : ActorsToMove)
			{
				if (!CurActor)
				{
					continue;
				}
				ActorCount++;
				bool bIsSourceLevelLocked = FLevelUtils::IsLevelLocked(CurActor);

				if (!bIsSourceLevelLocked)
				{
					if (CurActor->GetLevel() != DestLevel)
					{
						bool bCanMove = true;
						CanMoveActorToLevelDelegate.Broadcast(CurActor, DestLevel, bCanMove);
						if (bCanMove)
						{
							FinalMoveList.Add(CurActor);
						}
					}
					else
					{
						UE_LOGF(LogLevelTools, Warning, "%ls is already in the destination level so it was ignored", *CurActor->GetName());
					}
				}
				else
				{
					UE_LOGF(LogLevelTools, Error, "The source level '%ls' is locked so actors could not be moved", *CurActor->GetLevel()->GetName());
				}
			}
		}
		else
		{
			UE_LOGF(LogLevelTools, Error, "The destination level '%ls' is locked so actors could not be moved", *DestLevel->GetName());
		}


		if (FinalMoveList.Num() > 0 && (!bMoveAllOrFail || FinalMoveList.Num() == ActorCount))
		{
			TArray<TTuple<FSoftObjectPath, FSoftObjectPath>> ActorPathMapping;
			GEditor->SelectNone(false, true, false);

			USelection* ActorSelection = GEditor->GetSelectedActors();
			ActorSelection->BeginBatchSelectOperation();
			FinalWeakMoveList.Reserve(FinalMoveList.Num());
			for (AActor* Actor : FinalMoveList)
			{
				FinalWeakMoveList.Add(Actor);
				check(Actor->CopyPasteId == INDEX_NONE);
				Actor->CopyPasteId = ActorPathMapping.Add(TTuple<FSoftObjectPath, FSoftObjectPath>(FSoftObjectPath(Actor), FSoftObjectPath()));
				GEditor->SelectActor(Actor, true, false);
			}
			ActorSelection->EndBatchSelectOperation(false);

			if (GEditor->GetSelectedActorCount() > 0)
			{
				// Start the transaction
				FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveSelectedActorsToSelectedLevel", "Move Actors To Level"));

				// Broadcast event
				if(bMoveActors)
				{
					OnMoveActorsToLevelEvent.Broadcast(ActorsToMove, DestLevel);
				}

				// Cache the old level
				ULevel* OldCurrentLevel = OwningWorld->GetCurrentLevel();

				FString DestinationData;
				GEditor->CopySelectedActorsToClipboard(OwningWorld, bMoveActors, bMoveActors, bWarnAboutReferences, &DestinationData);

				if (!DestinationData.IsEmpty())
				{
					// Set the new level and force it visible while we do the paste
					const bool bLevelVisible = DestLevel->bIsVisible;
					if (!bLevelVisible)
					{
						UEditorLevelUtils::SetLevelVisibility(DestLevel, true, false);
					}
				
					OwningWorld->SetCurrentLevel(DestLevel);

					bool bSelectionChanged = false;
					FDelegateHandle SelectionChangedEvent = USelection::SelectionChangedEvent.AddLambda([&bSelectionChanged](UObject* Object) { bSelectionChanged = true; });

					const bool bDuplicate = false;
					const bool bOffsetLocations = false;
					const bool bWarnIfHidden = false;
					GEditor->edactPasteSelected(OwningWorld, bDuplicate, bOffsetLocations, bWarnIfHidden, &DestinationData);

					USelection::SelectionChangedEvent.Remove(SelectionChangedEvent);

					// Restore the original current level
					OwningWorld->SetCurrentLevel(OldCurrentLevel);

					if (bSelectionChanged)
					{
						// Build a remapping of old to new names so we can do a fixup
						for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
						{
							AActor* Actor = static_cast<AActor*>(*It);

							if (ActorPathMapping.IsValidIndex(Actor->CopyPasteId))
							{
								TTuple<FSoftObjectPath, FSoftObjectPath>& Tuple = ActorPathMapping[Actor->CopyPasteId];
								check(Tuple.Value.IsNull());

								Tuple.Value = FSoftObjectPath(Actor);
								if (OutActors)
								{
									OutActors->Add(Actor);
								}
							}
							else
							{
								UE_LOGF(LogLevelTools, Error, "Cannot find remapping for moved actor ID %ls, any soft references pointing to it will be broken!", *Actor->GetPathName());
							}
							// Reset CopyPasteId on new actors
							Actor->CopyPasteId = INDEX_NONE;
						}

						// Only do Asset Rename on Move (Copy should not affect existing references)
						if (bMoveActors)
						{
							FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
							TArray<FAssetRenameData> RenameData;

							for (TTuple<FSoftObjectPath, FSoftObjectPath>& Pair : ActorPathMapping)
							{
								if (Pair.Value.IsValid())
								{
									RenameData.Add(FAssetRenameData(Pair.Key, Pair.Value, true));
								}
							}

							if (RenameData.Num() > 0)
							{
								if (bWarnAboutRenaming)
								{
									AssetToolsModule.Get().RenameAssetsWithDialog(RenameData);
								}
								else
								{
									AssetToolsModule.Get().RenameAssets(RenameData);
								}
							}
						}

						// Restore new level visibility to previous state
						if (!bLevelVisible)
						{
							UEditorLevelUtils::SetLevelVisibility(DestLevel, false, false);
						}
					}
				}

				// The moved (pasted) actors will now be selected
				NumMovedActors += FinalMoveList.Num();
			}

			for (TWeakObjectPtr<AActor> ActorPtr : FinalWeakMoveList)
			{
				// It is possible a GC happens because of RenameAssets being called so here we want to update the CopyPasteId only on reachable actors
				if (AActor* Actor = ActorPtr.Get(/*bEvenIfPendingKill=*/true))
				{
					check(Actor->CopyPasteId != INDEX_NONE);
					// Reset CopyPasteId on source actors 
					Actor->CopyPasteId = INDEX_NONE;
				}
			}
		}

		// Restore the original clipboard contents
		FPlatformApplicationMisc::ClipboardCopy(*OriginalClipboardContent);
	}

	return NumMovedActors;
}

int32 UEditorLevelUtils::MoveSelectedActorsToLevel(ULevelStreaming* DestStreamingLevel, bool bWarnAboutReferences)
{
	ensureAsRuntimeWarning(DestStreamingLevel != nullptr);
	return DestStreamingLevel ? MoveSelectedActorsToLevel(DestStreamingLevel->GetLoadedLevel(), bWarnAboutReferences) : 0;
}

int32 UEditorLevelUtils::MoveSelectedActorsToLevel(ULevel* DestLevel, bool bWarnAboutReferences)
{
	if (ensureAsRuntimeWarning(DestLevel != nullptr))
	{
		TArray<AActor*> ActorsToMove;
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It))
			{
				ActorsToMove.Add(Actor);
			}
		}

		return MoveActorsToLevel(ActorsToMove, DestLevel, bWarnAboutReferences);
	}

	return 0;
}

ULevel* UEditorLevelUtils::AddLevelsToWorld(UWorld* InWorld, TArray<FString> PackageNames, TSubclassOf<ULevelStreaming> LevelStreamingClass)
{
	if (!ensure(InWorld))
	{
		return nullptr;
	}

	FScopedSlowTask SlowTask(static_cast<float>(PackageNames.Num()), LOCTEXT("AddLevelsToWorldTask", "Adding Levels to World"));
	SlowTask.MakeDialog();

	// Sort the level packages alphabetically by name.
	PackageNames.Sort();

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Try to add the levels that were specified in the dialog.
	ULevel* NewLevel = nullptr;
	for (const FString& PackageName : PackageNames)
	{
		SlowTask.EnterProgressFrame();

		if (ULevelStreaming* NewStreamingLevel = AddLevelToWorld_Internal(InWorld, FAddLevelToWorldParams(LevelStreamingClass, *PackageName)))
		{
			NewLevel = NewStreamingLevel->GetLoadedLevel();
			if (NewLevel)
			{
				LevelDirtyCallback.Request();
			}
		}
	} // for each file

	  // Set the last loaded level to be the current level
	if (NewLevel)
	{
		if (InWorld->SetCurrentLevel(NewLevel))
		{
			FEditorDelegates::NewCurrentLevel.Broadcast();
		}
	}

	if (!IsRunningCommandlet())
	{
		// For safety
		if (GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Landscape))
		{
			GLevelEditorModeTools().ActivateDefaultMode();
		}
	}

	// Broadcast the levels have changed (new style)
	InWorld->BroadcastLevelsChanged();
	FEditorDelegates::RefreshLevelBrowser.Broadcast();

	if (GUnrealEd)
	{
		// Update volume actor visibility for each viewport since we loaded a level which could potentially contain volumes
		GUnrealEd->UpdateVolumeActorVisibility(nullptr);
	}

	return NewLevel;
}

ULevelStreaming* UEditorLevelUtils::AddLevelToWorld(UWorld* InWorld, const TCHAR* LevelPackageName, TSubclassOf<ULevelStreaming> LevelStreamingClass, const FTransform& LevelTransform)
{
	FAddLevelToWorldParams Params(LevelStreamingClass, LevelPackageName);
	Params.Transform = LevelTransform;
	return AddLevelToWorld(InWorld, Params);
}

ULevelStreaming* UEditorLevelUtils::AddLevelToWorld(UWorld* InWorld, const FAddLevelToWorldParams& InParams)
{
	if (!ensure(InWorld))
	{
		return nullptr;
	}

	FScopedSlowTask SlowTask(0, LOCTEXT("AddLevelToWorldTask", "Adding Level to World"));
	SlowTask.MakeDialog();

	// Fire ULevel::LevelDirtiedEvent when falling out of scope.
	FScopedLevelDirtied LevelDirtyCallback;

	// Try to add the levels that were specified in the dialog.
	ULevel* NewLevel = nullptr;

	ULevelStreaming* NewStreamingLevel = AddLevelToWorld_Internal(InWorld, InParams);

	// Broadcast the levels have changed (new style)
	InWorld->BroadcastLevelsChanged();
	FEditorDelegates::RefreshLevelBrowser.Broadcast();

	if (NewStreamingLevel)
	{
		NewLevel = NewStreamingLevel->GetLoadedLevel();
		if (NewLevel)
		{
			LevelDirtyCallback.Request();

			// Set the loaded level to be the current level
			if (InWorld->SetCurrentLevel(NewLevel))
			{
				FEditorDelegates::NewCurrentLevel.Broadcast();
			}
		}
	}

	if (!IsRunningCommandlet())
	{
		// For safety
		if (GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Landscape))
		{
			GLevelEditorModeTools().ActivateDefaultMode();
		}
	}

	// Update volume actor visibility for each viewport since we loaded a level which could potentially contain volumes
	if (GUnrealEd)
	{
		GUnrealEd->UpdateVolumeActorVisibility(nullptr);
	}

	return NewStreamingLevel;
}

ULevelStreaming* UEditorLevelUtils::AddLevelToWorld_Internal(UWorld* InWorld, const FAddLevelToWorldParams& InParams)
{
	ULevel* NewLevel = nullptr;
	ULevelStreaming* StreamingLevel = nullptr;
	bool bIsPersistentLevel = (InWorld->PersistentLevel->GetOutermost()->GetFName() == InParams.PackageName);

	if (bIsPersistentLevel || FLevelUtils::FindStreamingLevel(InWorld, InParams.PackageName))
	{
		// Do nothing if the level already exists in the world.
		const FText MessageText = FText::Format(NSLOCTEXT("UnrealEd", "LevelAlreadyExistsInWorld", "A level with that name ({0}) already exists in the world."), FText::FromString(InParams.PackageName.ToString()));

		FSuppressableWarningDialog::FSetupInfo Info(MessageText, LOCTEXT("AddLevelToWorld_Title", "Add Level"), "LevelAlreadyExistsInWorldWarning");
		Info.ConfirmText = LOCTEXT("AlreadyExist_Ok", "Ok");
		FSuppressableWarningDialog RemoveLevelWarning(Info);
		RemoveLevelWarning.ShowModal();
	}
	else
	{
		// If the selected class is still NULL or the selected class is abstract, abort the operation.
		if (InParams.LevelStreamingClass == nullptr || InParams.LevelStreamingClass->HasAnyClassFlags(CLASS_Abstract))
		{
			return nullptr;
		}

		const FScopedBusyCursor BusyCursor;

		StreamingLevel = NewObject<ULevelStreaming>(InWorld, InParams.LevelStreamingClass, NAME_None, RF_NoFlags, NULL);

		// Associate a package name.
		StreamingLevel->SetWorldAssetByPackageName(InParams.PackageName);

		StreamingLevel->LevelTransform = InParams.Transform;

		// Seed the level's draw color.
		StreamingLevel->LevelColor = FLinearColor::MakeRandomColor();

		// Callback to allow initialization
		if (InParams.LevelStreamingCreatedCallback)
		{
			InParams.LevelStreamingCreatedCallback(StreamingLevel);
		}

		// Add the new level to world.
		InWorld->AddStreamingLevel(StreamingLevel);

		// Refresh just the newly created level.
		TArray<ULevelStreaming*> LevelsForRefresh;
		LevelsForRefresh.Add(StreamingLevel);
		InWorld->RefreshStreamingLevels(LevelsForRefresh);

		if (!StreamingLevel->HasAnyFlags(RF_Transient))
		{
			InWorld->MarkPackageDirty();
		}

		NewLevel = StreamingLevel->GetLoadedLevel();
		if (NewLevel != nullptr)
		{
			EditorLevelUtils::SetLevelVisibility(NewLevel, true, true);

			// Levels migrated from other projects may fail to load their world settings
			// If so we create a new AWorldSettings actor here.
			if (NewLevel->GetWorldSettings(false) == nullptr)
			{
				UWorld* SubLevelWorld = CastChecked<UWorld>(NewLevel->GetOuter());

				FActorSpawnParameters SpawnInfo;
				SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				SpawnInfo.Name = GEngine->WorldSettingsClass->GetFName();
				AWorldSettings* NewWorldSettings = SubLevelWorld->SpawnActor<AWorldSettings>(GEngine->WorldSettingsClass, SpawnInfo);

				NewLevel->SetWorldSettings(NewWorldSettings);
			}
		}
	}

	if (NewLevel) // if the level was successfully added
	{
		FEditorDelegates::OnAddLevelToWorld.Broadcast(NewLevel);
	}

	return StreamingLevel;
}

ULevelStreaming* UEditorLevelUtils::SetStreamingClassForLevel(ULevelStreaming* InLevel, TSubclassOf<ULevelStreaming> LevelStreamingClass)
{
	check(InLevel);

	const FScopedBusyCursor BusyCursor;

	// Cache off the package name, as it will be lost when unloading the level
	const FName CachedPackageName = InLevel->GetWorldAssetPackageFName();

	// First hide and remove the level if it exists
	ULevel* Level = InLevel->GetLoadedLevel();
	check(Level);
	SetLevelVisibility(Level, false, false);
	check(Level->OwningWorld);
	UWorld* World = Level->OwningWorld;

	World->RemoveStreamingLevel(InLevel);

	// re-add the level with the desired streaming class
	AddLevelToWorld(World, *(CachedPackageName.ToString()), LevelStreamingClass);

	// Transfer level streaming settings
	ULevelStreaming* NewStreamingLevel = FLevelUtils::FindStreamingLevel(Level);
	if (NewStreamingLevel)
	{
		NewStreamingLevel->LevelTransform = InLevel->LevelTransform;
		NewStreamingLevel->EditorStreamingVolumes = InLevel->EditorStreamingVolumes;
		NewStreamingLevel->MinTimeBetweenVolumeUnloadRequests = InLevel->MinTimeBetweenVolumeUnloadRequests;
		NewStreamingLevel->LevelColor = InLevel->LevelColor;
		NewStreamingLevel->Keywords = InLevel->Keywords;
		NewStreamingLevel->SetFolderPath(InLevel->GetFolderPath());
	}

	return NewStreamingLevel;
}

void UEditorLevelUtils::MakeLevelCurrent(ULevel* InLevel, bool bEvenIfLocked)
{
	if (ensureAsRuntimeWarning(InLevel != nullptr))
	{
		// Locked levels can't be made current.
		if (bEvenIfLocked || !FLevelUtils::IsLevelLocked(InLevel))
		{
			// Make current broadcast if it changed
			if (InLevel->OwningWorld->SetCurrentLevel(InLevel))
			{
				FEditorDelegates::NewCurrentLevel.Broadcast();
			}

			// Deselect all selected builder brushes.
			bool bDeselectedSomething = false;
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				AActor* Actor = static_cast<AActor*>(*It);
				checkSlow(Actor->IsA(AActor::StaticClass()));
				ABrush* Brush = Cast< ABrush >(Actor);
				if (Brush && FActorEditorUtils::IsABuilderBrush(Actor))
				{
					GEditor->SelectActor(Actor, /*bInSelected=*/ false, /*bNotify=*/ false);
					bDeselectedSomething = true;
				}
			}

			// Send a selection change callback if necessary.
			if (bDeselectedSomething)
			{
				GEditor->NoteSelectionChange();
			}

			// Force the current level to be visible.
			SetLevelVisibility(InLevel, true, false);
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelMakeLevelCurrent", "MakeLevelCurrent: The requested operation could not be completed because the level is locked."));
		}
	}
}

void UEditorLevelUtils::MakeLevelCurrent(ULevelStreaming* InStreamingLevel)
{
	if (ensureAsRuntimeWarning(InStreamingLevel != nullptr))
	{
		MakeLevelCurrent(InStreamingLevel->GetLoadedLevel());
	}
}

bool UEditorLevelUtils::PrivateRemoveInvalidLevelFromWorld(ULevelStreaming* InLevelStreaming)
{
	bool bRemovedLevelStreaming = false;
	if (InLevelStreaming)
	{
		check(InLevelStreaming->GetLoadedLevel() == NULL); // This method is designed to be used to remove left over references to null levels 

		InLevelStreaming->Modify();

		// Disassociate the level from the volume.
		for (ALevelStreamingVolume* LevelStreamingVolume : InLevelStreaming->EditorStreamingVolumes)
		{
			if (LevelStreamingVolume)
			{
				LevelStreamingVolume->Modify(true);
				LevelStreamingVolume->StreamingLevelNames.Remove(InLevelStreaming->GetWorldAssetPackageFName());
			}
		}

		// Disassociate the volumes from the level.
		InLevelStreaming->EditorStreamingVolumes.Empty();

		if (UWorld* OwningWorld = InLevelStreaming->GetWorld())
		{
			OwningWorld->RemoveStreamingLevel(InLevelStreaming);
			OwningWorld->RefreshStreamingLevels({});
			bRemovedLevelStreaming = true;
		}
	}
	return bRemovedLevelStreaming;
}

bool UEditorLevelUtils::RemoveInvalidLevelFromWorld(ULevelStreaming* InLevelStreaming)
{
	bool bRemoveSuccessful = PrivateRemoveInvalidLevelFromWorld(InLevelStreaming);
	if (bRemoveSuccessful)
	{
		// Redraw the main editor viewports.
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();

		// Broadcast the levels have changed (new style)
		InLevelStreaming->GetWorld()->BroadcastLevelsChanged();
		FEditorDelegates::RefreshLevelBrowser.Broadcast();

		// Update selection for any selected actors that were in the level and are no longer valid
		GEditor->NoteSelectionChange();

		// Collect garbage to clear out the destroyed level
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
	return bRemoveSuccessful;
}


ULevelStreaming* UEditorLevelUtils::CreateNewStreamingLevel(TSubclassOf<ULevelStreaming> LevelStreamingClass, const FString& PackagePath /*= TEXT("")*/, bool bMoveSelectedActorsIntoNewLevel /*= false*/)
{
	FString Filename;
	if (PackagePath.IsEmpty() || FPackageName::TryConvertLongPackageNameToFilename(PackagePath, Filename, FPackageName::GetMapPackageExtension()))
	{
		if (ensureAsRuntimeWarning(LevelStreamingClass.Get() != nullptr))
		{
			bool bUseSaveAs = PackagePath.IsEmpty();
			return CreateNewStreamingLevelForWorld(*GEditor->GetEditorWorldContext().World(), LevelStreamingClass, Filename, bMoveSelectedActorsIntoNewLevel, nullptr, bUseSaveAs);
		}
	}

	return nullptr;
}

namespace UE::EditorLevelUtils::Private
{

ULevel* GetPersistentLevelForNewStreamingLevel(UWorld* NewLevelWorld, UWorld* InTemplateWorld)
{
	check(NewLevelWorld || InTemplateWorld);
	return NewLevelWorld ? NewLevelWorld->PersistentLevel : InTemplateWorld->PersistentLevel;
}

UWorld* GetWorldForNewStreamingLevel(UWorld* InTemplateWorld, bool bCreateWorldPartition, bool bEnableWorldPartitionStreaming)
{
	if (!InTemplateWorld)
	{
		// Create a new world
		UWorldFactory* Factory = NewObject<UWorldFactory>();
		Factory->bCreateWorldPartition = bCreateWorldPartition;
		Factory->bEnableWorldPartitionStreaming = bEnableWorldPartitionStreaming;
		Factory->WorldType = EWorldType::Inactive;
		UPackage* Pkg = CreatePackage( NULL);
		FName WorldName(TEXT("Untitled"));
		EObjectFlags Flags = RF_Public | RF_Standalone;
		UWorld* NewLevelWorld = CastChecked<UWorld>(Factory->FactoryCreateNew(UWorld::StaticClass(), Pkg, WorldName, Flags, NULL, GWarn));
		if (NewLevelWorld)
		{
			FAssetRegistryModule::AssetCreated(NewLevelWorld);
		}
		return NewLevelWorld;
	}
	return nullptr;
}
};

ULevelStreaming* UEditorLevelUtils::CreateNewStreamingLevelForWorld(UWorld& InWorld, TSubclassOf<ULevelStreaming> LevelStreamingClass, const FString& DefaultFilename /* = TEXT( "" ) */, bool bMoveSelectedActorsIntoNewLevel /* = false */, UWorld* InTemplateWorld /* = nullptr */, bool bInUseSaveAs /*= true*/, TFunction<void(ULevel*)> InPreSaveLevelOperation /* = TFunction<void(ULevel*)>()*/, const FTransform& InTransform /* = FTransform::Identity */)
{
	TArray<AActor*> ActorsToMove;
	if (bMoveSelectedActorsIntoNewLevel)
	{
		ActorsToMove.Reserve(GEditor->GetSelectedActorCount());
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It))
			{
				ActorsToMove.Add(Actor);
			}
		}
	}

	return CreateNewStreamingLevelForWorld(InWorld, LevelStreamingClass, /*bUseExternalActors=*/false, DefaultFilename, &ActorsToMove, InTemplateWorld, bInUseSaveAs, /*bIsPartitioned=*/false, InPreSaveLevelOperation, InTransform);
}

ULevelStreaming* UEditorLevelUtils::CreateNewStreamingLevelForWorld(UWorld& InWorld, TSubclassOf<ULevelStreaming> LevelStreamingClass, bool bUseExternalActors, const FString& DefaultFilename /* = TEXT("") */, const TArray<AActor*>* ActorsToMove /* = nullptr */, UWorld* InTemplateWorld /* = nullptr */, bool bInUseSaveAs /*= true*/, bool bIsPartitioned /*= false*/, TFunction<void(ULevel*)> InPreSaveLevelOperation /* = TFunction<void(ULevel*)>()*/, const FTransform& InTransform /* = FTransform::Identity */)
{
	FCreateNewStreamingLevelForWorldParams CreateParams(LevelStreamingClass, DefaultFilename);
	CreateParams.bUseExternalActors = bUseExternalActors;
	CreateParams.ActorsToMove = ActorsToMove;
	CreateParams.TemplateWorld = InTemplateWorld;
	CreateParams.bUseSaveAs = bInUseSaveAs;
	CreateParams.bCreateWorldPartition = bIsPartitioned;
	CreateParams.PreSaveLevelCallback = InPreSaveLevelOperation;
	CreateParams.Transform = InTransform;
	return CreateNewStreamingLevelForWorld(InWorld, CreateParams);
}

ULevelStreaming* UEditorLevelUtils::CreateNewStreamingLevelForWorld(UWorld& InWorld,  const FCreateNewStreamingLevelForWorldParams& InCreateParams)
{
	// Editor modes cannot be active when any level saving occurs.
	if (!IsRunningCommandlet())
	{
		GLevelEditorModeTools().DeactivateAllModes();
	}
	using namespace UE::EditorLevelUtils::Private;

	// Make sure we reenable the default mode on exit
	ON_SCOPE_EXIT
	{
		if (!IsRunningCommandlet())
		{
			GLevelEditorModeTools().ActivateDefaultMode();
		}
	};

	// This is the world we are adding the new level to
	UWorld* WorldToAddLevelTo = &InWorld;

	// This is the new streaming level's world not the persistent level world
	UWorld* NewLevelWorld = GetWorldForNewStreamingLevel(InCreateParams.TemplateWorld, InCreateParams.bCreateWorldPartition, InCreateParams.bEnableWorldPartitionStreaming);
	ULevel* PersistentLevel = GetPersistentLevelForNewStreamingLevel(NewLevelWorld, InCreateParams.TemplateWorld);
	// No need to convert actors in partitioned worlds as they are already external
	if (InCreateParams.bUseExternalActors && !InCreateParams.bCreateWorldPartition)
	{
		PersistentLevel->ConvertAllActorsToPackaging(true);
		PersistentLevel->bUseExternalActors = true;
	}
	check(InCreateParams.bCreateWorldPartition == PersistentLevel->bIsPartitioned);

	if (InCreateParams.PreSaveLevelCallback)
	{
		// Call lambda before saving level
		InCreateParams.PreSaveLevelCallback(PersistentLevel);
	}

	bool bNewWorldSaved = false;
	FString NewPackageName = InCreateParams.DefaultFilename;

	if (InCreateParams.bUseSaveAs)
	{
		bNewWorldSaved = FEditorFileUtils::SaveLevelAs(PersistentLevel, &NewPackageName);
	}
	else
	{
		bNewWorldSaved = FEditorFileUtils::SaveLevel(PersistentLevel, InCreateParams.DefaultFilename, &NewPackageName);
	}
	
	if (bNewWorldSaved && !NewPackageName.IsEmpty())
	{
		NewPackageName = FPackageName::FilenameToLongPackageName(NewPackageName);

		// Find or Load package and re-assign NewLevelWorld in case it was duplicated by Save
		UPackage* NewPackage = LoadPackage(nullptr, *NewPackageName, LOAD_None);
		if (NewPackage)
		{
			NewLevelWorld = UWorld::FindWorldInPackage(NewPackage);
		}
	}

	// If the new world was saved successfully, import it as a streaming level.
	ULevelStreaming* NewStreamingLevel = nullptr;
	ULevel* NewLevel = nullptr;
	if (bNewWorldSaved)
	{
		// Make sure to uninitialize the world since the level will be used as a streaming level
		// This will make sure that the initialization order will be respected.
		// One example is world partition initialization done inside ULevel::OnLevelLoaded.
		if (NewLevelWorld && NewLevelWorld->bIsWorldInitialized)
		{
			NewLevelWorld->CleanupWorld();
		}

		FAddLevelToWorldParams AddLevelToWorldParams(InCreateParams.LevelStreamingClass, *NewPackageName);
		AddLevelToWorldParams.Transform = InCreateParams.Transform;
		AddLevelToWorldParams.LevelStreamingCreatedCallback = InCreateParams.LevelStreamingCreatedCallback;

		NewStreamingLevel = AddLevelToWorld(WorldToAddLevelTo, AddLevelToWorldParams);
		if (NewStreamingLevel != nullptr)
		{
			NewLevel = NewStreamingLevel->GetLoadedLevel();
			// If we are moving the selected actors to the new level move them now
			if (InCreateParams.ActorsToMove)
			{
				MoveActorsToLevel(*InCreateParams.ActorsToMove, NewLevel);
			}

			// Finally make the new level the current one
			if (WorldToAddLevelTo->SetCurrentLevel(NewLevel))
			{
				FEditorDelegates::NewCurrentLevel.Broadcast();
			}
		}
	}

	// Broadcast the levels have changed (new style)
	WorldToAddLevelTo->BroadcastLevelsChanged();
	FEditorDelegates::RefreshLevelBrowser.Broadcast();

	return NewStreamingLevel;
}

bool UEditorLevelUtils::RemoveLevelsFromWorld(TArray<ULevel*> InLevels, bool bClearSelection, bool bResetTransBuffer)
{
	// Check that all levels can be removed first
	if (!InLevels.Num() || Algo::AnyOf(InLevels, [](ULevel* LevelToRemove)
	{
		if (!LevelToRemove || LevelToRemove->IsPersistentLevel())
		{
			return true;
		}

		if (FLevelUtils::IsLevelLocked(LevelToRemove))
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelRemoveLevelFromWorld", "RemoveLevelFromWorld: The requested operation could not be completed because the level is locked."));
			return true;
		}

		return false;
	}))
	{
		return false;
	}
	
	TSet<UWorld*> ChangedWorlds;

	TArray<UPackage*> PackagesToUnload;
	PackagesToUnload.Reserve(InLevels.Num());
	TArray<FName> PackageNames;
	PackageNames.Reserve(InLevels.Num());

	ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();

	// Mark all Levels as being removed to prevent iteration on them while we are removing all of them.
	// PrivateRemoveLevelFromWorld will end up calling UWorld::UpdateStreamingState.
	// This can broadcast levels change events to listeners. By setting bIsBeingRemoved = true we make sure those levels do not get iterated and cause issues.
	for (ULevel* Level : InLevels)
	{
		Level->bIsBeingRemoved = true;
	}

	for (ULevel* Level : InLevels)
	{
		Layers->RemoveLevelLayerInformation(Level);
		GEditor->CloseEditedWorldAssets(CastChecked<UWorld>(Level->GetOuter()));

		UWorld* OwningWorld = Level->OwningWorld;
		const bool bRemovingCurrentLevel = Level->IsCurrentLevel();
		PrivateRemoveLevelFromWorld(Level);
		if (bRemovingCurrentLevel)
		{
			// we must set a new level.  It must succeed
			const bool bEvenIfLocked = true;
			MakeLevelCurrent(OwningWorld->PersistentLevel, bEvenIfLocked);
		}

		FEditorSupportDelegates::PrepareToCleanseEditorObject.Broadcast(Level);

		PrivateDestroyLevel(Level);

		PackagesToUnload.Add(Level->GetOutermost());
		// Keep Names in other list because unload of package will make the PackagesToUnload invalid.
		PackageNames.Add(Level->GetOutermost()->GetFName());
		ChangedWorlds.Add(OwningWorld);
		Level->bIsBeingRemoved = false;
	}

	FText TransResetText(LOCTEXT("RemoveLevelTransReset", "Removing Levels from World"));
	if (bResetTransBuffer && GEditor->Trans)
	{
		GEditor->Trans->Reset(TransResetText);
	}

	UPackageTools::FUnloadPackageParams UnloadParams(PackagesToUnload);
	UnloadParams.bResetTransBuffer = bResetTransBuffer;
	const bool bUnloadResult = UPackageTools::UnloadPackages(UnloadParams);
	if (!bUnloadResult && !UnloadParams.OutErrorMessage.IsEmpty())
	{
		FMessageDialog::Open(EAppMsgType::Ok, UnloadParams.OutErrorMessage);
	}

	// Redraw the main editor viewports.
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	for (UWorld* ChangedWorld : ChangedWorlds)
	{
		// Broadcast the levels have changed (new style)
		ChangedWorld->BroadcastLevelsChanged();
	}
	FEditorDelegates::RefreshLevelBrowser.Broadcast();
	
	// Reset transaction buffer and run GC to clear out the destroyed level
	GEditor->Cleanse(bClearSelection, false, TransResetText, bResetTransBuffer);

	auto CheckPackage = [](const TArray<FName>& PackageNames, EPrintStaleReferencesOptions Options)
	{
		// Check Package no longer exists
		for (const FName& LevelPackageName : PackageNames)
		{
			// Ensure that world was removed
			UPackage* LevelPackage = FindObjectFast<UPackage>(NULL, LevelPackageName);
			if (LevelPackage != nullptr)
			{
				UWorld* TheWorld = UWorld::FindWorldInPackage(LevelPackage->GetOutermost());
				if (TheWorld != nullptr)
				{
					FReferenceChainSearch::FindAndPrintStaleReferencesToObject(TheWorld, Options);
					return false;
				}
			}
		}
		return true;
	};

	// Check that packages no longer exist
	EPrintStaleReferencesOptions PrintStaleReferencesOptions = EPrintStaleReferencesOptions::Log;
	if (bResetTransBuffer)
	{
		PrintStaleReferencesOptions = UObjectBaseUtility::IsGarbageEliminationEnabled() ? EPrintStaleReferencesOptions::Fatal : (EPrintStaleReferencesOptions::Error | EPrintStaleReferencesOptions::Ensure);
	}
	bool bFailed = !CheckPackage(PackageNames, PrintStaleReferencesOptions);
	if (bFailed && ((PrintStaleReferencesOptions & EPrintStaleReferencesOptions::Fatal) != EPrintStaleReferencesOptions::Fatal))
	{
		// We tried avoiding clearing the Transaction buffer but it failed. Plan B.
		GEditor->Cleanse(bClearSelection, false, TransResetText, true);
		CheckPackage(PackageNames, PrintStaleReferencesOptions);
	}

	return true;
}


bool UEditorLevelUtils::RemoveLevelFromWorld(ULevel* InLevel, bool bClearSelection, bool bResetTransBuffer)
{
	return RemoveLevelsFromWorld({ InLevel }, bClearSelection, bResetTransBuffer);
}


void UEditorLevelUtils::PrivateRemoveLevelFromWorld(ULevel* InLevel)
{
	bool bIsTransientLevelStreaming = false;
	if (ULevelStreaming* StreamingLevel = ULevelStreaming::FindStreamingLevel(InLevel))
	{
		bIsTransientLevelStreaming = StreamingLevel->HasAnyFlags(RF_Transient);
		StreamingLevel->MarkAsGarbage();
		InLevel->OwningWorld->RemoveStreamingLevel(StreamingLevel);
		InLevel->OwningWorld->RefreshStreamingLevels({});
	}
	else if (InLevel->bIsVisible)
	{
		InLevel->OwningWorld->RemoveFromWorld(InLevel);
		check(InLevel->bIsVisible == false);
	}

	if (FLevelCollection* LC = InLevel->GetCachedLevelCollection())
	{
		LC->RemoveLevel(InLevel);
	}

	InLevel->ReleaseRenderingResources();

	IStreamingManager::Get().RemoveLevel(InLevel);
	UWorld* World = InLevel->OwningWorld;
	if (World->ContainsLevel(InLevel))
	{
		// Manually call level removal world delegates PreLevelRemovedFromWorld/LevelRemovedFromWorld to simulate what UWorld::RemoveFromWorld does.
		FWorldDelegates::PreLevelRemovedFromWorld.Broadcast(InLevel, World);
		if (World->RemoveLevel(InLevel))
		{
			FWorldDelegates::LevelRemovedFromWorld.Broadcast(InLevel, World);
		}
	}

	if (InLevel->bIsLightingScenario)
	{
		World->PropagateLightingScenarioChange();
	}
	InLevel->ClearLevelComponents();

	// remove all group actors from the world in the level we are removing
	// otherwise, this will cause group actors to not be garbage collected
	for (int32 GroupIndex = World->ActiveGroupActors.Num() - 1; GroupIndex >= 0; --GroupIndex)
	{
		AGroupActor* GroupActor = Cast<AGroupActor>(World->ActiveGroupActors[GroupIndex]);
		if (GroupActor && GroupActor->IsInLevel(InLevel))
		{
			World->ActiveGroupActors.RemoveAt(GroupIndex);
		}
	}

	// Mark all model components as pending kill so GC deletes references to them.
	for (UModelComponent* ModelComponent : InLevel->ModelComponents)
	{
		if (ModelComponent != nullptr)
		{
			ModelComponent->MarkAsGarbage();
		}
	}

	// Mark all actors and their components as pending kill so GC will delete references to them.
	for (AActor* Actor : InLevel->Actors)
	{
		if (Actor != nullptr)
		{
			const bool bModify = false;
			Actor->MarkComponentsAsGarbage(bModify);
			Actor->MarkAsGarbage();
		}
	}

	if (!bIsTransientLevelStreaming)
	{
		World->MarkPackageDirty();
	}
	World->BroadcastLevelsChanged();
}

void UEditorLevelUtils::PrivateDestroyLevel(ULevel* InLevel)
{
	UWorld* World = InLevel->OwningWorld;

	UObject* Outer = InLevel->GetOuter();

	// Call cleanup on the outer world of the level so external hooks can be properly released, so that unloading the package isn't prevented.
	UWorld* OuterWorld = Cast<UWorld>(Outer);
	if (OuterWorld && OuterWorld != World)
	{
		OuterWorld->CleanupWorld();
	}

	Outer->MarkAsGarbage();
	InLevel->MarkAsGarbage();
	Outer->ClearFlags(RF_Public | RF_Standalone);

	UPackage* Package = InLevel->GetOutermost();
	// We want to unconditionally destroy the level, so clear the dirty flag here so it can be unloaded successfully
	if (Package->IsDirty())
	{
		Package->SetDirtyFlag(false);
	}
}

void UEditorLevelUtils::DeselectAllSurfacesInLevel(ULevel* InLevel)
{
	if (InLevel)
	{
		UModel* Model = InLevel->Model;
		for (int32 SurfaceIndex = 0; SurfaceIndex < Model->Surfs.Num(); ++SurfaceIndex)
		{
			FBspSurf& Surf = Model->Surfs[SurfaceIndex];
			if (Surf.PolyFlags & PF_Selected)
			{
				Model->ModifySurf(SurfaceIndex, false);
				Surf.PolyFlags &= ~PF_Selected;
			}
		}
	}
}

void UEditorLevelUtils::SetLevelVisibilityTemporarily(ULevel* Level, bool bShouldBeVisible)
{
	// Nothing to do
	if (Level == NULL)
	{
		return;
	}

	// Set the visibility of each actor in the p-level
	for (decltype(Level->Actors)::TIterator ActorIter(Level->Actors); ActorIter; ++ActorIter)
	{
		AActor* CurActor = *ActorIter;
		if (CurActor && !FActorEditorUtils::IsABuilderBrush(CurActor) && CurActor->bHiddenEdLevel == bShouldBeVisible)
		{
			CurActor->bHiddenEdLevel = !bShouldBeVisible;
			CurActor->MarkComponentsRenderStateDirty();
		}
	}

	// Set the visibility of each BSP surface in the p-level
	UModel* CurLevelModel = Level->Model;
	if (CurLevelModel)
	{
		for (TArray<FBspSurf>::TIterator SurfaceIterator(CurLevelModel->Surfs); SurfaceIterator; ++SurfaceIterator)
		{
			FBspSurf& CurSurf = *SurfaceIterator;
			CurSurf.bHiddenEdLevel = !bShouldBeVisible;
		}
	}

	// Add/remove model components from the scene
	for (int32 ComponentIndex = 0; ComponentIndex < Level->ModelComponents.Num(); ComponentIndex++)
	{
		UModelComponent* CurLevelModelCmp = Level->ModelComponents[ComponentIndex];
		if (CurLevelModelCmp)
		{
			CurLevelModelCmp->MarkRenderStateDirty();
		}
	}

	Level->GetWorld()->SendAllEndOfFrameUpdates();

	Level->bIsVisible = bShouldBeVisible;

	if (Level->bIsLightingScenario)
	{
		Level->OwningWorld->PropagateLightingScenarioChange();
	}
}

void SetLevelVisibilityNoGlobalUpdateInternal(ULevel* Level, const bool bShouldBeVisible, const bool bForceLayersVisible, const ELevelVisibilityDirtyMode ModifyMode)
{
	// Nothing to do
	if (Level == NULL)
	{
		return;
	}

	// Handle the case of the p-level
	// The p-level can't be unloaded, so its actors/BSP should just be temporarily hidden/unhidden
	// Also, intentionally do not force layers visible for the p-level
	if (Level->IsPersistentLevel())
	{
		// Create a transaction so we can undo the visibility toggle
		const FScopedTransaction Transaction(LOCTEXT("ToggleLevelVisibility", "Toggle Level Visibility"));
		if (Level->bIsVisible != bShouldBeVisible && ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
		{
			Level->Modify();
		}
		// Set the visibility of each actor in the p-level
		for (decltype(Level->Actors)::TIterator PLevelActorIter(Level->Actors); PLevelActorIter; ++PLevelActorIter)
		{
			AActor* CurActor = *PLevelActorIter;
			if (CurActor && !FActorEditorUtils::IsABuilderBrush(CurActor) && CurActor->bHiddenEdLevel == bShouldBeVisible)
			{
				if (ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
				{
					CurActor->Modify(false);
				}
				
				CurActor->bHiddenEdLevel = !bShouldBeVisible;
				CurActor->RegisterAllComponents();
				CurActor->MarkComponentsRenderStateDirty();
			}
		}

		// Set the visibility of each BSP surface in the p-level
		UModel* CurLevelModel = Level->Model;
		if (CurLevelModel)
		{
			if (ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
			{
				CurLevelModel->Modify(false);
			}

			for (TArray<FBspSurf>::TIterator SurfaceIterator(CurLevelModel->Surfs); SurfaceIterator; ++SurfaceIterator)
			{
				FBspSurf& CurSurf = *SurfaceIterator;
				CurSurf.bHiddenEdLevel = !bShouldBeVisible;
			}
		}

		// Add/remove model components from the scene
		for (int32 ComponentIndex = 0; ComponentIndex < Level->ModelComponents.Num(); ComponentIndex++)
		{
			UModelComponent* CurLevelModelCmp = Level->ModelComponents[ComponentIndex];
			if (CurLevelModelCmp)
			{
				if (bShouldBeVisible)
				{
					CurLevelModelCmp->RegisterComponentWithWorld(Level->OwningWorld);
				}
				else if (!bShouldBeVisible && CurLevelModelCmp->IsRegistered())
				{
					CurLevelModelCmp->UnregisterComponent();
				}
			}
		}

		Level->GetWorld()->OnLevelsChanged().Broadcast();
	}
	else
	{
		ULevelStreaming* StreamingLevel = NULL;
		if (Level->OwningWorld == NULL || Level->OwningWorld->PersistentLevel != Level)
		{
			StreamingLevel = FLevelUtils::FindStreamingLevel(Level);
		}

		// Create a transaction so we can undo the visibility toggle
		const FScopedTransaction Transaction(LOCTEXT("ToggleLevelVisibility", "Toggle Level Visibility"));

		// Handle the case of a streaming level
		if (StreamingLevel)
		{
			if (ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
			{
				// We need to set the RF_Transactional to make a streaming level serialize itself. so store the original ones, set the flag, and put the original flags back when done
				EObjectFlags cachedFlags = StreamingLevel->GetFlags();
				StreamingLevel->SetFlags(RF_Transactional);
				StreamingLevel->Modify();
				StreamingLevel->SetFlags(cachedFlags);
			}

			// Set the visibility state for this streaming level.
			StreamingLevel->SetShouldBeVisibleInEditor(bShouldBeVisible);
		}

		ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
		if (!bShouldBeVisible)
		{
			Layers->RemoveLevelLayerInformation(Level);
		}

		// UpdateLevelStreaming sets Level->bIsVisible directly, so we need to make sure it gets saved to the transaction buffer.
		if (Level->bIsVisible != bShouldBeVisible && ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
		{
			Level->Modify();
		}

		if (StreamingLevel)
		{
			Level->OwningWorld->FlushLevelStreaming();

			// In the Editor we expect this operation will complete in a single call
			check(Level->bIsVisible == bShouldBeVisible);
		}
		else if (Level->OwningWorld != NULL)
		{
			// In case we level has no associated StreamingLevel, remove or add to world directly
			if (bShouldBeVisible)
			{
				if (!Level->bIsVisible)
				{
					Level->OwningWorld->AddToWorld(Level);
				}
			}
			else
			{
				Level->OwningWorld->RemoveFromWorld(Level);
			}

			// In the Editor we expect this operation will complete in a single call
			check(Level->bIsVisible == bShouldBeVisible);
		}

		if (bShouldBeVisible)
		{
			Layers->AddLevelLayerInformation(Level);
		}

		// Force the level's layers to be visible, if desired
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();

		// Iterate over the level's actors, making a list of their layers and unhiding the layers.
		auto& Actors = Level->Actors;
		for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex)
		{
			AActor* Actor = Actors[ActorIndex];
			if (Actor)
			{
				bool bModified = false;
				if (bShouldBeVisible && bForceLayersVisible &&
					Layers->IsActorValidForLayer(Actor))
				{
					// Make the actor layer visible, if it's not already.
					if (Actor->bHiddenEdLayer)
					{
						if (ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
						{
							bModified = Actor->Modify(false);
						}
						Actor->bHiddenEdLayer = false;
					}

					const bool bIsVisible = true;
					Layers->SetLayersVisibility(Actor->Layers, bIsVisible);
				}

				// Set the visibility of each actor in the streaming level
				if (!FActorEditorUtils::IsABuilderBrush(Actor) && Actor->bHiddenEdLevel == bShouldBeVisible)
				{
					if (!bModified && ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
					{
						bModified = Actor->Modify(false);
					}
					Actor->bHiddenEdLevel = !bShouldBeVisible;

					if (bShouldBeVisible)
					{
						Actor->ReregisterAllComponents();
					}
					else
					{
						Actor->UnregisterAllComponents();
					}
				}
			}
		}
	}

	Level->bIsVisible = bShouldBeVisible;

	// If the level is being hidden, deselect actors and surfaces that belong to this level. (Part 1/2)
	if (!bShouldBeVisible && ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
	{
		if (UTypedElementSelectionSet* ActorSelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet())
		{
			TArray<FTypedElementHandle> LevelElementHandles;
			
			ActorSelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>(
				[Level, &LevelElementHandles](const TTypedElement<ITypedElementWorldInterface>& Element)
			{
				if (Element.GetOwnerLevel() == Level)
				{
					LevelElementHandles.Add(Element);
				}
				return true;
			});
			ActorSelectionSet->DeselectElements(LevelElementHandles, FTypedElementSelectionOptions());
		}
		
		UEditorLevelUtils::DeselectAllSurfacesInLevel(Level);
	}

	if (Level->bIsLightingScenario)
	{
		Level->OwningWorld->PropagateLightingScenarioChange();
	}
}

void UEditorLevelUtils::SetLevelVisibility(ULevel* Level, const bool bShouldBeVisible, const bool bForceLayersVisible, const ELevelVisibilityDirtyMode ModifyMode)
{
	TArray<ULevel*> Levels({ Level });
	TArray<bool> bTheyShouldBeVisible({ bShouldBeVisible });
	SetLevelsVisibility(Levels, bTheyShouldBeVisible, bForceLayersVisible, ModifyMode);
}

void UEditorLevelUtils::SetLevelsVisibility(const TArray<ULevel*>& Levels, const TArray<bool>& bTheyShouldBeVisible, const bool bForceLayersVisible, const ELevelVisibilityDirtyMode ModifyMode)
{
	// Nothing to do
	if (Levels.Num() == 0 || Levels.Num() != bTheyShouldBeVisible.Num())
	{
		return;
	}

	// Perform SetLevelVisibilityNoGlobalUpdateInternal for each Level
	for (int32 LevelIndex = 0; LevelIndex < Levels.Num(); ++LevelIndex)
	{
		ULevel* Level = Levels[LevelIndex];
		if (Level)
		{
			SetLevelVisibilityNoGlobalUpdateInternal(Level, bTheyShouldBeVisible[LevelIndex], bForceLayersVisible, ModifyMode);
		}
	}

	// If at least 1 persistent level, then RedrawAllViewports.Broadcast
	for (int32 LevelIndex = 0; LevelIndex < Levels.Num(); ++LevelIndex)
	{
		ULevel* Level = Levels[LevelIndex];
		if (Level && Level->IsPersistentLevel())
		{
			FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			break;
		}
	}

	// If at least 1 level becomes visible, force layers to update their actor status
	// Otherwise, changes made on the layers for actors belonging to a non-visible level would not work
	{
		for (int32 LevelIndex = 0; LevelIndex < bTheyShouldBeVisible.Num(); ++LevelIndex)
		{
			if (bTheyShouldBeVisible[LevelIndex])
			{
				// Equivalent to GEditor->GetEditorSubsystem<ULayersSubsystem>()->UpdateAllActorsVisibilityDefault();
				FEditorDelegates::RefreshLayerBrowser.Broadcast();
				break;
			}
		}
	}

	// Notify the Scene Outliner, as new Actors may be present in the world.
	GEngine->BroadcastLevelActorListChanged();

	// If the level is being hidden, deselect actors and surfaces that belong to this level. (Part 2/2)
	if (ModifyMode == ELevelVisibilityDirtyMode::ModifyOnChange)
	{
		for (int32 LevelIndex = 0; LevelIndex < bTheyShouldBeVisible.Num(); ++LevelIndex)
		{
			if (!bTheyShouldBeVisible[LevelIndex])
			{
				// Tell the editor selection status was changed.
				GEditor->NoteSelectionChange();
				break;
			}
		}
	}
}

void UEditorLevelUtils::ForEachWorlds(UWorld* InWorld, TFunctionRef<bool(UWorld*)> Operation, bool bIncludeInWorld, bool bOnlyEditorVisible)
{
	if (!InWorld)
	{
		return;
	}

	if (bIncludeInWorld)
	{
		if (!Operation(InWorld))
		{
			return;
		}
	}

	// Iterate over the world's level array to find referenced levels ("worlds"). We don't 
	for (ULevelStreaming* StreamingLevel : InWorld->GetStreamingLevels())
	{
		if (StreamingLevel)
		{
			// If we asked for only sub-levels that are editor-visible, then limit our results appropriately
			bool bShouldAlwaysBeLoaded = false; // Cast< ULevelStreamingAlwaysLoaded >( StreamingLevel ) != NULL;
			if (!bOnlyEditorVisible || bShouldAlwaysBeLoaded || StreamingLevel->GetShouldBeVisibleInEditor())
			{
				const ULevel* Level = StreamingLevel->GetLoadedLevel();

				// This should always be the case for valid level names as the Editor preloads all packages.
				if (Level)
				{
					// Newer levels have their packages' world as the outer.
					UWorld* World = Cast<UWorld>(Level->GetOuter());
					if (World)
					{
						if (!Operation(World))
						{
							return;
						}
					}
				}
			}
		}
	}

	// Levels can be loaded directly without StreamingLevel facilities
	for (ULevel* Level : InWorld->GetLevels())
	{
		if (Level)
		{
			// Newer levels have their packages' world as the outer.
			UWorld* World = Cast<UWorld>(Level->GetOuter());
			if (World)
			{
				if (!Operation(World))
				{
					return;
				}
			}
		}
	}
}

void UEditorLevelUtils::GetWorlds(UWorld* InWorld, TArray<UWorld*>& OutWorlds, bool bIncludeInWorld, bool bOnlyEditorVisible)
{
	OutWorlds.Empty();
	ForEachWorlds(InWorld, [&OutWorlds](UWorld* World) { OutWorlds.AddUnique(World); return true; }, bIncludeInWorld, bOnlyEditorVisible);
}

const TArray<ULevel*> UEditorLevelUtils::GetLevels(UWorld* World)
{
	if (!World)
	{
		return TArray<ULevel*>();
	}
	return World->GetLevels();
}

#undef LOCTEXT_NAMESPACE
