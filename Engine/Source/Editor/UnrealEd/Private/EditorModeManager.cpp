// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeManager.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Selection.h"
#include "Misc/MessageDialog.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/WorldSettings.h"
#include "LevelEditorViewport.h"
#include "EditorModeRegistry.h"
#include "EditorModes.h"
#include "EditorSupportDelegates.h"
#include "EdMode.h"
#include "Rendering/PositionVertexBuffer.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "Toolkits/IToolkitHost.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Commands/UICommandList.h"
#include "Toolkits/BaseToolkit.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "Tools/DefaultEdMode.h"
#include "Tools/UEdMode.h"
#include "InputRouter.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Tools/LegacyEdModeInterfaces.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"
#include "Tools/AssetEditorContextObject.h"
#include "ContextObjectStore.h"
#include "EditorInteractiveGizmoManager.h"
#include "GizmoEdModeInterface.h"
#include "UObject/GCObjectScopeGuard.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Subsystems/EditorElementSubsystem.h"

#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "TextureResource.h"
#include "EditorGizmos/EditorGizmoStateTarget.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/GizmoRotationUtil.h"
#include "Toolkits/ToolkitManager.h"

/*------------------------------------------------------------------------------
	FEditorModeTools.

	The master class that handles tracking of the current mode.
------------------------------------------------------------------------------*/

TAutoConsoleVariable<int32> FEditorModeTools::CVarEnableITFCursorOverrideSupport(
	TEXT("Editor.EnableITFCursorOverrideSupport"),
	0,
	TEXT("Enable support for the EditorModeManager to query the InteractiveToolsContext for tool customized cursor overrides."));

FEditorModeTools::FEditorModeTools()
	: PivotShown(false)
	, Snapping(false)
	, SnappedActor(false)
	, CachedLocation(ForceInitToZero)
	, PivotLocation(ForceInitToZero)
	, SnappedLocation(ForceInitToZero)
	, GridBase(ForceInitToZero)
	, TranslateRotateXAxisAngle(0.0f)
	, TranslateRotate2DAngle(0.0f)
	, DefaultModeIDs({ FAssetEdModes::EM_AssetDefault })
	, WidgetMode(UE::Widget::WM_None)
	, OverrideWidgetMode(UE::Widget::WM_None)
	, bShowWidget(true)
	, bHideViewportUI(false)
	, bSelectionHasSceneComponent(false)
	, WidgetScale(1.0f)
	, CoordSystem(COORD_World)
	, bIsTracking(false)
{
	InteractiveToolsContext = NewObject<UModeManagerInteractiveToolsContext>(GetTransientPackage(), UModeManagerInteractiveToolsContext::StaticClass(), NAME_None, RF_Transient);
	InteractiveToolsContext->InitializeContextWithEditorModeManager(this);
	InteractiveToolsContext->Activate();

	// Load the last used settings
	LoadConfig();

	if( GEditor )
	{
		// Register our callback for undo/redo
		GEditor->RegisterForUndo(this);

		// This binding ensures the mode is destroyed if the type is unregistered outside of normal shutdown process
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnEditorModeUnregistered().AddRaw(this, &FEditorModeTools::OnModeUnregistered);
		GEditor->OnViewportClientListChanged().AddRaw(this, &FEditorModeTools::OnViewportListChanged);
	}

	UE::EditorTransformGizmoUtil::RegisterTransformGizmoContextObject(this);

	FWorldDelegates::OnWorldCleanup.AddRaw(this, &FEditorModeTools::OnWorldCleanup);
}

FEditorModeTools::~FEditorModeTools()
{
	SetDefaultMode( FAssetEdModes::EM_AssetDefault );
	DeactivateAllModes();

	RemoveAllDelegateHandlers();

	ExitAllModesPendingDeactivate();
	RecycledScriptableModes.Empty();

	// We may be destroyed after the UObject system has already shutdown, 
	// which would mean that this instances will be garbage
	if (UObjectInitialized())
	{
		UE::EditorTransformGizmoUtil::UnregisterTransformGizmoContextObject(this);
		InteractiveToolsContext->Deactivate();
		InteractiveToolsContext->ShutdownContext();
		InteractiveToolsContext = nullptr;
		WeakGizmoStateTarget = nullptr;
		WeakGizmoContext = nullptr;
	}
}

void FEditorModeTools::LoadConfig(void)
{
	GConfig->GetBool(TEXT("FEditorModeTools"),TEXT("ShowWidget"),bShowWidget,
		GEditorPerProjectIni);

	static constexpr bool bGetRawValue = true;
	int32 CoordSystemAsInt = (int32)GetCoordSystem(bGetRawValue);
	GConfig->GetInt(TEXT("FEditorModeTools"),TEXT("CoordSystem"), CoordSystemAsInt,
		GEditorPerProjectIni);

	const ECoordSystem CoordinateSystem = static_cast<ECoordSystem>(CoordSystemAsInt);
	if (CoordinateSystem == COORD_Parent || CoordinateSystem == COORD_Explicit)
	{
		// parent mode is only supported with new trs gizmos for now
		if (!(bViewportITF && UEditorInteractiveGizmoManager::UsesNewTRSGizmos()))
		{
			CoordSystemAsInt = static_cast<int32>(COORD_Local);
		}
	}
	
	SetCoordSystem((ECoordSystem)CoordSystemAsInt);

	LoadWidgetSettings();
}

void FEditorModeTools::SaveConfig(void)
{
	GConfig->SetBool(TEXT("FEditorModeTools"), TEXT("ShowWidget"), bShowWidget, GEditorPerProjectIni);

	static constexpr bool bGetRawValue = true;
	GConfig->SetInt(TEXT("FEditorModeTools"), TEXT("CoordSystem"), (int32)GetCoordSystem(bGetRawValue), GEditorPerProjectIni);

	SaveWidgetSettings();
}

TSharedPtr<class IToolkitHost> FEditorModeTools::GetToolkitHost() const
{
	TSharedPtr<class IToolkitHost> Result = ToolkitHost.Pin();
	check(ToolkitHost.IsValid());
	return Result;
}

bool FEditorModeTools::HasToolkitHost() const
{
	return ToolkitHost.Pin().IsValid();
}

void FEditorModeTools::SetToolkitHost(TSharedRef<class IToolkitHost> InHost)
{
	checkf(!ToolkitHost.IsValid(), TEXT("SetToolkitHost can only be called once"));
	ToolkitHost = InHost;

	// Remove the existing context, if it exists
	InteractiveToolsContext->ContextObjectStore->RemoveContextObjectsOfType<UAssetEditorContextObject>();

	if (HasToolkitHost())
	{
		UAssetEditorContextObject* AssetEditorContextObject = NewObject<UAssetEditorContextObject>(InteractiveToolsContext->ToolManager);
		AssetEditorContextObject->SetToolkitHost(GetToolkitHost().Get());
		InteractiveToolsContext->ContextObjectStore->AddContextObject(AssetEditorContextObject);
	}
}

USelection* FEditorModeTools::GetSelectedActors() const
{
	return GEditor->GetSelectedActors();
}

USelection* FEditorModeTools::GetSelectedObjects() const
{
	return GEditor->GetSelectedObjects();
}

USelection* FEditorModeTools::GetSelectedComponents() const
{
	return GEditor->GetSelectedComponents();
}

UTypedElementSelectionSet* FEditorModeTools::GetEditorSelectionSet() const
{
	if (USelection* SelectedActorsSet = GetSelectedActors())
	{
		return SelectedActorsSet->GetElementSelectionSet();
	}
	return nullptr;
}

void FEditorModeTools::StoreSelection(FName SelectionStoreKey, bool bClearSelection)
{
	if (UTypedElementSelectionSet* SelectionSet = GetEditorSelectionSet())
	{
		StoredSelectionSets.Emplace(SelectionStoreKey, SelectionSet->GetCurrentSelectionState());

		if (bClearSelection)
		{
			SelectionSet->ClearSelection(FTypedElementSelectionOptions().SetAllowHidden(true));
		}
	}
}

void FEditorModeTools::RestoreSelection(FName SelectionStoreKey)
{
	if (UTypedElementSelectionSet* SelectionSet = GetEditorSelectionSet())
	{
		if (FTypedElementSelectionSetState* StoredState = StoredSelectionSets.Find(SelectionStoreKey))
		{
			SelectionSet->RestoreSelectionState(*StoredState);
		}
	}
}

UWorld* FEditorModeTools::GetWorld() const
{
	// When in 'Simulate' mode, the editor mode tools will actually interact with the PIE world
	if( GEditor->bIsSimulatingInEditor && GEditor->GetPIEWorldContext() )
	{
		return GEditor->GetPIEWorldContext()->World();
	}
	else
	{
		return GEditor->GetEditorWorldContext().World();
	}
}

FEditorViewportClient* FEditorModeTools::GetHoveredViewportClient() const
{
	// Note: as per the comment in MouseLeave, this currently acts as LastHoveredViewportClient.
	return HoveredViewportClient;
}

FEditorViewportClient* FEditorModeTools::GetFocusedViewportClient() const
{
	// Note: as per the comment in LostFocus, this actually currently acts as LastFocusedViewportClient.
	return FocusedViewportClient;
}

bool FEditorModeTools::SelectionHasSceneComponent() const
{
	return bSelectionHasSceneComponent;
}

void FEditorModeTools::SetSelectionHasSceneComponent(bool bHasSceneComponent)
{
	bSelectionHasSceneComponent = bHasSceneComponent;
}

bool FEditorModeTools::IsEditingDisallowed(AActor* InActor) const
{
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		if (Mode->IsEditingDisallowed(InActor))
		{
			return true;
		}
	}
	return false;
}

bool FEditorModeTools::IsSelectionAllowed(AActor* InActor, const bool bInSelected) const
{
	bool bSelectionAllowed = (ActiveScriptableModes.Num() == 0);

	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		// Exclusive ability for a mode to disable selection
		if (Mode->IsSelectionDisallowed(InActor, bInSelected))
		{
			return false;
		}
				
		bSelectionAllowed |= Mode->IsSelectionAllowed(InActor, bInSelected);		
	}

	return bSelectionAllowed;
}

bool FEditorModeTools::IsSelectionHandled(AActor* InActor, const bool bInSelected) const
{
	bool bSelectionHandled = false;
	ForEachEdMode([&bSelectionHandled, bInSelected, InActor](UEdMode* Mode)
		{
			bSelectionHandled |= Mode->Select(InActor, bInSelected);
			return true;
		});

	return bSelectionHandled;
}

bool FEditorModeTools::ProcessEditDuplicate()
{
	bool bHandled = false;
	ForEachEdMode([&bHandled](UEdMode* Mode)
		{
			bHandled |= Mode->ProcessEditDuplicate();
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::ProcessEditDelete()
{
	bool bHandled = InteractiveToolsContext->ProcessEditDelete();
	ForEachEdMode([&bHandled](UEdMode* Mode)
		{
			bHandled |= Mode->ProcessEditDelete();
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::ProcessEditCut()
{
	bool bHandled = false;
	ForEachEdMode([&bHandled](UEdMode* Mode)
		{
			bHandled = Mode->ProcessEditCut();
			return !bHandled;
		});

	return bHandled;
}

bool FEditorModeTools::ProcessEditCopy()
{
	bool bHandled = false;
	ForEachEdMode([&bHandled](UEdMode* Mode)
		{
			bHandled = Mode->ProcessEditCopy();
			return !bHandled;
		});

	return bHandled;
}

bool FEditorModeTools::ProcessEditPaste()
{
	bool bHandled = false;
	ForEachEdMode([&bHandled](UEdMode* Mode)
		{
			bHandled = Mode->ProcessEditPaste();
			return !bHandled;
		});

	return bHandled;
}

EEditAction::Type FEditorModeTools::GetActionEditDuplicate()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
		{
			const EEditAction::Type EditAction = Mode->GetActionEditDuplicate();
			if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
			{
				ReturnedAction = EditAction;
				return false;
			}

			return true;
		});

	return ReturnedAction;
}

EEditAction::Type FEditorModeTools::GetActionDragDuplicate()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
		{
			const EEditAction::Type EditAction = Mode->GetActionDragDuplicate();
			if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
			{
				ReturnedAction = EditAction;
				return false;
			}

			return true;
		});

	return ReturnedAction;
}

EEditAction::Type FEditorModeTools::GetActionEditDelete()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
	{
		const EEditAction::Type EditAction = Mode->GetActionEditDelete();
		if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
		{
			ReturnedAction = EditAction;
			return false;
		}

		return true;
	});

	return ReturnedAction;
}

EEditAction::Type FEditorModeTools::GetActionEditCut()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
		{
			const EEditAction::Type EditAction = Mode->GetActionEditCut();
			if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
			{
				ReturnedAction = EditAction;
				return false;
			}

			return true;
		});

	return ReturnedAction;
}

EEditAction::Type FEditorModeTools::GetActionEditCopy()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
		{
			const EEditAction::Type EditAction = Mode->GetActionEditCopy();
			if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
			{
				ReturnedAction = EditAction;
				return false;
			}

			return true;
		});

	return ReturnedAction;
}

EEditAction::Type FEditorModeTools::GetActionEditPaste()
{
	EEditAction::Type ReturnedAction = EEditAction::Skip;
	ForEachEdMode([&ReturnedAction](UEdMode* Mode)
		{
			const EEditAction::Type EditAction = Mode->GetActionEditPaste();
			if (EditAction == EEditAction::Process || EditAction == EEditAction::Halt)
			{
				ReturnedAction = EditAction;
				return false;
			}

			return true;
		});

	return ReturnedAction;
}

void FEditorModeTools::DeactivateOtherVisibleModes(FEditorModeID InMode)
{
	ForEachEdMode([this, InMode](UEdMode* Mode)
		{
			if (Mode->GetID() != InMode && Mode->GetModeInfo().IsVisible())
			{
				DeactivateMode(Mode->GetID());
			}
			return true;
		});
}

bool FEditorModeTools::IsSnapRotationEnabled() const
{
	bool bRetVal = false;
	ForEachEdMode([&bRetVal](UEdMode* Mode)
		{
			bRetVal = Mode->IsSnapRotationEnabled();
			return !bRetVal;
		});

	return bRetVal;
}

bool FEditorModeTools::SnapRotatorToGridOverride(FRotator& InRotation) const
{
	bool bRetVal = false;
	ForEachEdMode([&bRetVal, &InRotation](UEdMode* Mode)
		{
			bRetVal = Mode->SnapRotatorToGridOverride(InRotation);
			return !bRetVal;
		});

	return bRetVal;
}

void FEditorModeTools::ActorsDuplicatedNotify(TArray<AActor*>& InPreDuplicateSelection, TArray<AActor*>& InPostDuplicateSelection, const bool bOffsetLocations)
{
	ForEachEdMode([&InPreDuplicateSelection, &InPostDuplicateSelection, bOffsetLocations](UEdMode* Mode)
	{
		// Tell the tools about the duplication
		Mode->ActorsDuplicatedNotify(InPreDuplicateSelection, InPostDuplicateSelection, bOffsetLocations);
		return true;
	});
}

void FEditorModeTools::ActorMoveNotify()
{
	ForEachEdMode([](UEdMode* Mode)
	{
		// Also notify the current editing modes if they are interested.
		Mode->ActorMoveNotify();
		return true;
	});
}

void FEditorModeTools::ActorSelectionChangeNotify()
{
	ForEachEdMode([](UEdMode* Mode)
	{
		Mode->ActorSelectionChangeNotify();
		return true;
	});
}

void FEditorModeTools::ElementSelectionChangeNotify()
{
	ForEachEdMode([](UEdMode* Mode)
	{
		Mode->ElementSelectionChangeNotify();
		return true;
	});
}

void FEditorModeTools::ActorPropChangeNotify()
{
	ForEachEdMode([](UEdMode* Mode)
	{
		Mode->ActorPropChangeNotify();
		return true;
	});
}

void FEditorModeTools::UpdateInternalData()
{
	ForEachEdMode([](UEdMode* Mode)
	{
		Mode->UpdateInternalData();
		return true;
	});
}

bool FEditorModeTools::IsOnlyVisibleActiveMode(FEditorModeID InMode) const
{
	// Only return true if this is the *only* active mode
	bool bFoundAnotherVisibleMode = false;
	ForEachEdMode([&bFoundAnotherVisibleMode, InMode](UEdMode* Mode)
	{
		bFoundAnotherVisibleMode = (Mode->GetModeInfo().IsVisible() && Mode->GetID() != InMode);
		return !bFoundAnotherVisibleMode;
	});
	return !bFoundAnotherVisibleMode;
}

bool FEditorModeTools::IsOnlyActiveMode(FEditorModeID InMode) const
{
	return ActiveScriptableModes.Num() == 1 && ActiveScriptableModes[0]->GetID() == InMode;
}

void FEditorModeTools::DrawBrackets(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	if (!ViewportClient->IsPerspective() || !GetDefault<ULevelEditorViewportSettings>()->bHighlightWithBrackets)
	{
		return;
	}

	if (UTypedElementSelectionSet* CurrentSelection = GetEditorSelectionSet())
	{
		CurrentSelection->ForEachSelectedObject<AActor>([Canvas, View, Viewport, ViewportClient](AActor* Actor)
			{
				const FLinearColor SelectedActorBoxColor(0.6f, 0.6f, 1.0f);
				const bool bDrawBracket = Actor->IsA<AStaticMeshActor>();
				ViewportClient->DrawActorScreenSpaceBoundingBox(Canvas, View, Viewport, Actor, SelectedActorBoxColor, bDrawBracket);
				return true;
			});
	}
}

void FEditorModeTools::ForEachEdMode(TFunctionRef<bool(UEdMode*)> InCalllback) const
{
	// Copy Array in case callback deactivates a mode
	auto ActiveModes = ActiveScriptableModes;
	for (UEdMode* Mode : ActiveModes)
	{
		if (Mode)
		{
			if (!InCalllback(Mode))
			{
				break;
			}
		}
	}
}

bool FEditorModeTools::TestAllModes(TFunctionRef<bool(UEdMode*)> InCalllback, bool bExpected) const
{
	for (UEdMode* Mode : ActiveScriptableModes)
	{
		if (Mode)
		{
			if (InCalllback(Mode) != bExpected)
			{
				return false;
			}
		}
	}

	return true;
}

void FEditorModeTools::ExitAllModesPendingDeactivate()
{
	bIsExitingModesDuringTick = true;

	// Loop so that exiting one mode can queue the exit of another and have that be processed in the same frame.
	constexpr int32 MaxCycles = 32; 
	int32 RemainingCycles = MaxCycles;
	while (PendingDeactivateModes.Num() && RemainingCycles > 0)
	{
		// Make a copy so we can modify the pending deactivate modes map during ExitMode
		TMap<FEditorModeID, UEdMode*> PendingDeactivateModesCopy(ObjectPtrDecay(PendingDeactivateModes));
		for (auto& Pair : PendingDeactivateModesCopy)
		{
			ExitMode(Pair.Value);
			PendingDeactivateModes.Remove(Pair.Key);
		}
		
		--RemainingCycles;
	}

	bIsExitingModesDuringTick = false;

	if (PendingDeactivateModes.Num() > 0)
	{
		UE_LOGF(LogEditorModes, Error, "Flushing PendingDeactivateModes had %i items in the list after %i cycles. There may be a loop where modes remove each other.", PendingDeactivateModes.Num(), MaxCycles);
		for (const auto& Pair : PendingDeactivateModes)
		{
			FString ObjectName;
			if (Pair.Value)
			{
				Pair.Value->GetName(ObjectName);
			}
			UE_LOGF(LogEditorModes, Log, "  Mode Id: %ls; Mode: %ls", *Pair.Key.ToString(), *ObjectName);
		}
	}
}

void FEditorModeTools::SetPivotLocation( const FVector& Location, const bool bIncGridBase )
{
	CachedLocation = PivotLocation = SnappedLocation = Location;
	if ( bIncGridBase )
	{
		GridBase = Location;
	}
}

ECoordSystem FEditorModeTools::GetCoordSystem(bool bGetRawValue) const
{
	if (!bGetRawValue && (GetWidgetMode() == UE::Widget::WM_Scale))
	{
		return COORD_Local;
	}
	
	return CoordSystem;
}

void FEditorModeTools::SetCoordSystem(ECoordSystem NewCoordSystem)
{
	CoordSystem = NewCoordSystem;
	BroadcastCoordSystemChanged(NewCoordSystem);
}

void FEditorModeTools::SetDefaultMode( const FEditorModeID DefaultModeID )
{
	DefaultModeIDs.Reset();
	DefaultModeIDs.Add( DefaultModeID );
}

void FEditorModeTools::AddDefaultMode( const FEditorModeID DefaultModeID )
{
	DefaultModeIDs.AddUnique( DefaultModeID );
}

void FEditorModeTools::RemoveDefaultMode( const FEditorModeID DefaultModeID )
{
	DefaultModeIDs.RemoveSingle( DefaultModeID );
}

void FEditorModeTools::ActivateDefaultMode()
{
	// NOTE: Activating EM_Default will cause ALL default editor modes to be activated (handled specially in ActivateMode())
	ActivateMode( FBuiltinEditorModes::EM_Default );
}

void FEditorModeTools::ExitMode(UEdMode* InMode)
{
	if (InMode)
	{
		InMode->Exit();

		const FEditorModeID EditorModeID = InMode->GetID();
		PendingDeactivateModes.Remove(EditorModeID);
		RecycledScriptableModes.Add(EditorModeID, InMode);
	}
}

void FEditorModeTools::OnModeUnregistered(FEditorModeID ModeID)
{
	DestroyMode(ModeID);
}




void FEditorModeTools::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	UWorld* World = GetWorld();
	if (InWorld == World)
	{
		ExitAllModesPendingDeactivate();
	}
}

void FEditorModeTools::OnViewportListChanged()
{
	bool bFoundHovered = false;
	bool bFoundFocused = false;
	
	if (GEditor)
	{
		for (const FEditorViewportClient* Client : GEditor->GetAllViewportClients())
		{
			if (Client == HoveredViewportClient)
			{
				bFoundHovered = true;
			}
			if (Client == FocusedViewportClient)
			{
				bFoundFocused = true;
			}
		}
	}
	
	// If the hovered or focused clients are not found in the list, they have been removed.
	// Their pointers are now pointing at garbage and need to be reset.
	if (!bFoundHovered)
	{
		HoveredViewportClient = nullptr;
	}
	if (!bFoundFocused)
	{
		FocusedViewportClient = nullptr;
	}
}

void FEditorModeTools::RemoveAllDelegateHandlers()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnEditorModeUnregistered().RemoveAll(this);
		}
		
		GEditor->OnViewportClientListChanged().RemoveAll(this);
	}

	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

	OnEditorModeIDChanged().Clear();
	OnWidgetModeChanged().Clear();
	OnCoordSystemChanged().Clear();
}

void FEditorModeTools::DeactivateModeAtIndex(int32 Index)
{
	UEdMode* Mode = ActiveScriptableModes[Index];
	const FEditorModeID ModeID = Mode->GetID();
	PendingDeactivateModes.Emplace(ModeID, Mode);
	ActiveScriptableModes.RemoveAt(Index);
		
	if (const TSharedPtr<FModeToolkit> Toolkit = Mode->GetToolkit().Pin())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
	}

	constexpr bool bIsEnteringMode = false;
	BroadcastEditorModeIDChanged(ModeID, bIsEnteringMode);
}

void FEditorModeTools::DeactivateMode( FEditorModeID InID )
{
	// Find the mode from the ID and exit it.
	for (int32 Index = ActiveScriptableModes.Num() - 1; Index >= 0; --Index)
	{
		UEdMode* Mode = ActiveScriptableModes[Index];
		if (Mode->GetID() == InID)
		{
			DeactivateModeAtIndex(Index);
			break;
		}
	}
}

void FEditorModeTools::DeactivateAllModes()
{
	TArray<FEditorModeID> EditorModesToRemove;
	for (int32 Index = ActiveScriptableModes.Num() - 1; Index >= 0; --Index)
	{
		UEdMode* Mode = ActiveScriptableModes[Index];
		EditorModesToRemove.Add(Mode->GetID());
	}

	for (FEditorModeID EditorModeToRemove : EditorModesToRemove)
	{
		DeactivateMode(EditorModeToRemove);
	}
}

void FEditorModeTools::DestroyMode( FEditorModeID InID )
{
	// Since deactivating the last active mode will cause the default modes to be activated, make sure this mode is removed from defaults.
	RemoveDefaultMode( InID );
	
	// Add back the default default mode if we just removed the last valid default.
	if ( DefaultModeIDs.Num() == 0 )
	{
		AddDefaultMode( FAssetEdModes::EM_AssetDefault );
	}

	// Find the mode from the ID and exit it.
	DeactivateMode(InID);
	if (UEdMode* DeactivatedMode = PendingDeactivateModes.FindRef(InID))
	{
		ExitMode(DeactivatedMode);
	}

	RecycledScriptableModes.Remove(InID);
}



bool FEditorModeTools::ShouldShowModeToolbox() const
{
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		if (Mode->GetModeInfo().IsVisible() && Mode->UsesToolkits())
		{
			return true;
		}
	}

	return false;
}

void FEditorModeTools::ActivateMode(FEditorModeID InID, bool bToggle)
{
	static bool bReentrant = false;
	if( !bReentrant )
	{
		if (InID == FBuiltinEditorModes::EM_Default)
		{
			bReentrant = true;

			for( const FEditorModeID& ModeID : DefaultModeIDs )
			{
				ActivateMode( ModeID );
			}

			for( const FEditorModeID& ModeID : DefaultModeIDs )
			{
				check( IsModeActive( ModeID ) );
			}

			bReentrant = false;
			return;
		}
	}

	// Check to see if the mode is already active
	if (IsModeActive(InID))
	{
		// The mode is already active toggle it off if we should toggle off already active modes.
		if (bToggle)
		{
			DeactivateMode(InID);
		}
		// Nothing more to do
		return;
	}

	// Recycle a mode or factory a new one
	UEdMode* ScriptableMode = RecycledScriptableModes.FindRef(InID);
	bool bNeedsEnter = true;
	if (!ScriptableMode)
	{
		ScriptableMode = PendingDeactivateModes.FindRef(InID);

		if (ScriptableMode)
		{
			// If we are actively exiting modes, don't re-activate the mode
			if (bIsExitingModesDuringTick)
			{
				return;
			}

			bNeedsEnter = false;
		}
	}

	if (!ScriptableMode)
	{
		ScriptableMode = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CreateEditorModeWithToolsOwner(InID, *this);
	}

	if (!ScriptableMode)
	{
		UE_LOGF(LogEditorModes, Log, "FEditorModeTools::ActivateMode : Couldn't find mode '%ls'.", *InID.ToString());
		// Just return and leave the mode list unmodified
		return;
	}
	
	{
		// Make sure ScriptableMode doesn't get GCed while Deactivating modes
		FGCObjectScopeGuard ScriptModeGuard(ScriptableMode);

		// Remove anything that isn't compatible with this mode
		TArray<FEditorModeID> EditorModesToRemove;
		const bool bIsVisibleMode = ScriptableMode->GetModeInfo().IsVisible();
		for (int32 ModeIndex = ActiveScriptableModes.Num() - 1; ModeIndex >= 0; ModeIndex--)
		{
			UEdMode* Mode = ActiveScriptableModes[ModeIndex];
			const bool bModesAreCompatible = ScriptableMode->IsCompatibleWith(Mode->GetID()) || Mode->IsCompatibleWith(ScriptableMode->GetID());
			if (!bModesAreCompatible || (bIsVisibleMode && Mode->GetModeInfo().IsVisible()))
			{
				EditorModesToRemove.Add(Mode->GetID());
			}
		}

		for (FEditorModeID EditorModeToRemove : EditorModesToRemove)
		{
			DeactivateMode(EditorModeToRemove);
		}
	}

	ActiveScriptableModes.Add(ScriptableMode);

	// Enter the new mode
	if (bNeedsEnter)
	{
		ScriptableMode->Enter();
	}

	const bool bIsEnteringMode = true;
	BroadcastEditorModeIDChanged(InID, bIsEnteringMode);

	PendingDeactivateModes.Remove(InID);
	RecycledScriptableModes.Remove(InID);

	// Update the editor UI
	FEditorSupportDelegates::UpdateUI.Broadcast();
}

bool FEditorModeTools::EnsureNotInMode(FEditorModeID ModeID, const FText& ErrorMsg, bool bNotifyUser) const
{
	// We're in a 'safe' mode if we're not in the specified mode.
	const bool bInASafeMode = !IsModeActive(ModeID);
	if( !bInASafeMode && !ErrorMsg.IsEmpty() )
	{
		// Do we want to display this as a notification or a dialog to the user
		if ( bNotifyUser )
		{
			FNotificationInfo Info( ErrorMsg );
			FSlateNotificationManager::Get().AddNotification( Info );
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, ErrorMsg );
		}		
	}
	return bInASafeMode;
}

UEdMode* FEditorModeTools::GetActiveScriptableMode(FEditorModeID InID) const
{
	if (auto* FoundMode = ActiveScriptableModes.FindByPredicate([InID](UEdMode* Mode) { return (Mode->GetID() == InID); }))
	{
		return *FoundMode;
	}

	return nullptr;
}

UTexture2D* FEditorModeTools::GetVertexTexture() const
{
	return GEngine->DefaultBSPVertexTexture;
}

FMatrix FEditorModeTools::GetCustomDrawingCoordinateSystem() const
{
	FMatrix Matrix = FMatrix::Identity;

	switch (GetCoordSystem())
	{
		case COORD_World:
		{
			if (UEditorTransformGizmoContextObject* GizmoContextObject = WeakGizmoContext.Get())
			{
				GizmoContextObject->RotationContext = FRotationContext();
			}
		}
		break;
		
		case COORD_Local:
			Matrix = GetLocalCoordinateSystem();
		break;
		
		case COORD_Parent:
			Matrix = GetParentSpaceCoordinateSystem();
		break;

		case COORD_Explicit:
			Matrix = GetExplicitCoordinateSystem();
		break;

		default:
			break;
	}

	return Matrix;
}

FMatrix FEditorModeTools::GetCustomInputCoordinateSystem() const
{
	return GetCustomDrawingCoordinateSystem();
}

FMatrix FEditorModeTools::GetLocalCoordinateSystem() const
{
	return GetCustomCoordinateSystem([](const TTypedElement<ITypedElementWorldInterface>& InElement, FTransform& OutTransform)
	{
		InElement.GetWorldTransform(OutTransform);
	});
}

FMatrix FEditorModeTools::GetParentSpaceCoordinateSystem() const
{
	return GetCustomCoordinateSystem([](const TTypedElement<ITypedElementWorldInterface>& InElement, FTransform& OutTransform)
	{
		if (InElement.GetWorldTransform(OutTransform))
		{
			FTransform RelativeTransform;
			if (InElement.GetRelativeTransform(RelativeTransform))
			{
				const FTransform ParentWorld = RelativeTransform.Inverse() * OutTransform;
				OutTransform.SetRotation(ParentWorld.GetRotation());
			}
			return true;
		}
		return false;
	});
}

FMatrix FEditorModeTools::GetExplicitCoordinateSystem() const
{
	return GetCustomCoordinateSystem([this](const TTypedElement<ITypedElementWorldInterface>& InElement, FTransform& OutTransform)
	{
		if (UEditorTransformGizmoContextObject* GizmoContext = GetGizmoContext())
		{
			FRotationContext& RotationContext = GizmoContext->RotationContext;
			
			RotationContext = FRotationContext();
			RotationContext.bUseExplicitRotator = GetWidgetMode() == UE::Widget::EWidgetMode::WM_Rotate;

			if (UE::GizmoRotationUtil::GetRelativeTransform(InElement, OutTransform, RotationContext))
			{
				return true;
			}
		}
		return false;
	});
}

FMatrix FEditorModeTools::GetCustomCoordinateSystem(TUniqueFunction<void(const TTypedElement<ITypedElementWorldInterface>&, FTransform&)>&& InGetTransformFunc) const
{
	FMatrix Matrix = FMatrix::Identity;
	// Let the current mode have a shot at setting the local coordinate system.

	bool CustomCoordinateSystemProvided = false;
	ForEachEdMode<ILegacyEdModeWidgetInterface>([&Matrix, &CustomCoordinateSystemProvided](ILegacyEdModeWidgetInterface* LegacyMode)
		{
			CustomCoordinateSystemProvided = LegacyMode->GetCustomDrawingCoordinateSystem(Matrix, nullptr);
			return !CustomCoordinateSystemProvided;
		});

	// If there isn't an active mode overriding the local coordinate system, create it by looking at the current selection.
	if (!CustomCoordinateSystemProvided)
	{
		if (UEditorTransformGizmoContextObject* GizmoContextObject = WeakGizmoContext.Get())
		{
			GizmoContextObject->RotationContext = FRotationContext();
		}
		
		TTypedElement<ITypedElementWorldInterface> LastSelected;
		if ((this == &GLevelEditorModeTools()) && GCurrentLevelEditingViewportClient)
		{
			// Use the cache from the viewport when available
			LastSelected = GCurrentLevelEditingViewportClient->GetElementsToManipulate()->GetBottomElement<ITypedElementWorldInterface>();
		}
		else
		{
			LastSelected = UEditorElementSubsystem::GetLastSelectedEditorManipulableElement(UEditorElementSubsystem::GetEditorNormalizedSelectionSet(*GetEditorSelectionSet()));
		}

		
		if (LastSelected)
		{
			FTransform CustomToWorldTransform;
			InGetTransformFunc(LastSelected, CustomToWorldTransform);
			Matrix = FQuatRotationMatrix(CustomToWorldTransform.GetRotation());
		}
	}

	if (!Matrix.Equals(FMatrix::Identity))
	{
		Matrix.RemoveScaling();
	}

	return Matrix;
}

/** Gets the widget axis to be drawn */
EAxisList::Type FEditorModeTools::GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const
{
	EAxisList::Type OutAxis = EAxisList::All;
	for( int Index = ActiveScriptableModes.Num() - 1; Index >= 0 ; Index-- )
	{
		ILegacyEdModeWidgetInterface* Mode = Cast<ILegacyEdModeWidgetInterface>(ActiveScriptableModes[Index]);
		if ( Mode && Mode->ShouldDrawWidget() )
		{
			OutAxis = Mode->GetWidgetAxisToDraw( InWidgetMode );
			break;
		}
	}

	return OutAxis;
}

/** Mouse tracking interface.  Passes tracking messages to all active modes */
bool FEditorModeTools::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bIsTracking = true;
	CachedLocation = PivotLocation;	// Cache the pivot location

	bool bTrackingHandled = InteractiveToolsContext->StartTracking(InViewportClient, InViewport);

	// no need to go further if bHasOngoingTransform is true
	if (!bHasOngoingTransform)
	{
		ForEachEdMode<ILegacyEdModeViewportInterface>([&bTrackingHandled, InViewportClient, InViewport](ILegacyEdModeViewportInterface* ViewportInterface)
		{
			bTrackingHandled |= ViewportInterface->StartTracking(InViewportClient, InViewport);
			return true;
		});
	}

	return bTrackingHandled;
}

/** Mouse tracking interface.  Passes tracking messages to all active modes */
bool FEditorModeTools::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	bIsTracking = false;

	bool bTrackingHandled = InteractiveToolsContext->EndTracking(InViewportClient, InViewport);
	// no need to go further if bHasOngoingTransform is true
	if (!bHasOngoingTransform)
	{
		ForEachEdMode<ILegacyEdModeViewportInterface>([&bTrackingHandled, InViewportClient, InViewport](ILegacyEdModeViewportInterface* ViewportInterface)
		   {
			   bTrackingHandled |= ViewportInterface->EndTracking(InViewportClient, InViewport);
			   return true;
		   });
	}
	
	CachedLocation = PivotLocation;	// Clear the pivot location
	bHasOngoingTransform = false;
	
	return bTrackingHandled;
}

bool FEditorModeTools::AllowsViewportDragTool() const
{
	if (bHasOngoingTransform)
	{
		return false;
	}
	
	bool bCanUseDragTool = false;
	ForEachEdMode<const ILegacyEdModeViewportInterface>([&bCanUseDragTool](const ILegacyEdModeViewportInterface* LegacyMode)
		{
			bCanUseDragTool |= LegacyMode->AllowsViewportDragTool();
			return true;
		});

	return bCanUseDragTool;
}

/** Notifies all active modes that a map change has occured */
void FEditorModeTools::MapChangeNotify()
{
	ForEachEdMode([](UEdMode* Mode)
		{
			Mode->MapChangeNotify();
			return true;
		});
}

/** Notifies all active modes to empty their selections */
void FEditorModeTools::SelectNone()
{
	ForEachEdMode([](UEdMode* Mode)
		{
			Mode->SelectNone();
			return true;
		});
}

/** Notifies all active modes of box selection attempts */
bool FEditorModeTools::BoxSelect( FBox& InBox, bool InSelect )
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeSelectInterface>([&bHandled, &InBox, InSelect](ILegacyEdModeSelectInterface* LegacyMode)
		{
			bHandled |= LegacyMode->BoxSelect(InBox, InSelect);
			return true;
		});
	return bHandled;
}

/** Notifies all active modes of frustum selection attempts */
bool FEditorModeTools::FrustumSelect( const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect )
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeSelectInterface>([&bHandled, InFrustum, InViewportClient, InSelect](ILegacyEdModeSelectInterface* LegacyMode)
		{
			bHandled |= LegacyMode->FrustumSelect(InFrustum, InViewportClient, InSelect);
			return true;
		});
	return bHandled;
}


/** true if any active mode uses a transform widget */
bool FEditorModeTools::UsesTransformWidget() const
{
	bool bUsesTransformWidget = false;
	ForEachEdMode<const ILegacyEdModeWidgetInterface>([&bUsesTransformWidget](const ILegacyEdModeWidgetInterface* LegacyMode)
		{
			bUsesTransformWidget |= LegacyMode->UsesTransformWidget();
			return true;
		});

	return bUsesTransformWidget;
}

/** true if any active mode uses the passed in transform widget */
bool FEditorModeTools::UsesTransformWidget( UE::Widget::EWidgetMode CheckMode ) const
{
	bool bUsesTransformWidget = false;
	ForEachEdMode<const ILegacyEdModeWidgetInterface>([&bUsesTransformWidget, CheckMode](const ILegacyEdModeWidgetInterface* LegacyMode)
		{
			bUsesTransformWidget |= LegacyMode->UsesTransformWidget(CheckMode);
			return true;
		});

	return bUsesTransformWidget;
}

/** Sets the current widget axis */
void FEditorModeTools::SetCurrentWidgetAxis( EAxisList::Type NewAxis )
{
	ForEachEdMode<ILegacyEdModeWidgetInterface>([NewAxis](ILegacyEdModeWidgetInterface* LegacyMode)
		{
			LegacyMode->SetCurrentWidgetAxis(NewAxis);
			return true;
		});
}

/** Notifies all active modes of mouse click messages. */
bool FEditorModeTools::HandleClick(FEditorViewportClient* InViewportClient,  HHitProxy *HitProxy, const FViewportClick& Click )
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, HitProxy, Click](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->HandleClick(InViewportClient, HitProxy, Click);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox)
{
	bool bHandled = false;
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		bHandled |= Mode->ComputeBoundingBoxForViewportFocus(Actor, PrimitiveComponent, InOutBox);
	}

	return bHandled;
}

/** true if the passed in brush actor should be drawn in wireframe */	
bool FEditorModeTools::ShouldDrawBrushWireframe( AActor* InActor ) const
{
	bool bShouldDraw = false;

	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		bShouldDraw |= Mode->ShouldDrawBrushWireframe(InActor);
	}

	if((ActiveScriptableModes.Num() == 0))
	{
		// We can get into a state where there are no active modes at editor startup if the builder brush is created before the default mode is activated.
		// Ensure we can see the builder brush when no modes are active.
		bShouldDraw = true;
	}
	return bShouldDraw;
}

/** true if brush vertices should be drawn */
bool FEditorModeTools::ShouldDrawBrushVertices() const
{
	if(UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>())
	{
		// Currently only geometry mode being active prevents vertices from being drawn.
		return !BrushSubsystem->IsGeometryEditorModeActive();
	}

	return true;
}

void FEditorModeTools::PropagateInvalidations( FEditorViewportClient* ViewportClient )
{
	if (InteractiveToolsContext)
	{
		InteractiveToolsContext->PropagateInvalidations(ViewportClient);
	}
}

/** Ticks all active modes */
void FEditorModeTools::Tick( FEditorViewportClient* ViewportClient, float DeltaTime )
{	
	// Remove anything pending destruction
	ExitAllModesPendingDeactivate();

	if (ActiveScriptableModes.Num() == 0)
	{
		// Ensure the default mode is active if there are no active modes.
		ActivateDefaultMode();
	}

	InteractiveToolsContext->Tick(ViewportClient, DeltaTime);
	ForEachEdMode([ViewportClient, DeltaTime](UEdMode* Mode)
		{
			if (ILegacyEdModeViewportInterface* ViewportInterface = Cast<ILegacyEdModeViewportInterface>(Mode))
			{
				ViewportInterface->Tick(ViewportClient, DeltaTime);
			}

			Mode->ModeTick(DeltaTime);
			return true;
		});
}

/** Notifies all active modes of any change in mouse movement */
bool FEditorModeTools::InputDelta( FEditorViewportClient* InViewportClient,FViewport* InViewport,FVector& InDrag,FRotator& InRot,FVector& InScale )
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, InViewport, &InDrag, &InRot, &InScale](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
			return true;
		});
	return bHandled;
}

/** Notifies all active modes of captured mouse movement */	
bool FEditorModeTools::CapturedMouseMove( FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY )
{
	bool bHandled = InteractiveToolsContext->CapturedMouseMove(InViewportClient, InViewport, InMouseX, InMouseY);
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, InViewport, InMouseX, InMouseY](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->CapturedMouseMove(InViewportClient, InViewport, InMouseX, InMouseY);
			return true;
		});
	return bHandled;
}

/** Notifies all active modes of all captured mouse movement */	
bool FEditorModeTools::ProcessCapturedMouseMoves( FEditorViewportClient* InViewportClient, FViewport* InViewport, const TArrayView<FIntPoint>& CapturedMouseMoves )
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, InViewport, &CapturedMouseMoves](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->ProcessCapturedMouseMoves(InViewportClient, InViewport, CapturedMouseMoves);
			return true;
		});
	return bHandled;
}

/** Notifies all active modes of keyboard input via a viewport client */
bool FEditorModeTools::InputKey(FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event, bool bRouteToToolsContext)
{
	const bool bHadOngoingTransform = bHasOngoingTransform;

	bool bWasHandledByToolsContext = false;
	if (bRouteToToolsContext)
	{
		bWasHandledByToolsContext = InteractiveToolsContext->InputKey(InViewportClient, Viewport, Key, Event);
	}
	else
	{
		// If we're not routing to the tools context, we still need to let it look at the event so that it can update
		// its internal memory of which mouse keys are down, to pass the correct mouse state later.
		InteractiveToolsContext->UpdateStateWithoutRoutingInputKey(InViewportClient, Viewport, Key, Event);
	}

	if (bWasHandledByToolsContext && !bIsTracking && GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		StartTracking(InViewportClient, Viewport);
	}
	else if (bRouteToToolsContext && bIsTracking && !GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		EndTracking(InViewportClient, Viewport);
	}

	// no need to go further if bHasOngoingTransform state has changed 
	// NOTE, this should probably be done comparing HasActiveMouseCapture changes instead as it means that the ITF handled the event
	// however, as with StartTracking/EndTracking and bTrackingHandled, testing bWasHandledByToolsContext is not reliable as
	// InteractiveToolsContext->InputKey will return false when the mouse is released, even if there was an ongoing capture before the release event ended it.
	if (bHasOngoingTransform != bHadOngoingTransform)
	{
		return true;
	}
	
	// If the toolkit should process the command, it should not have been handled by ITF, or be tracked elsewhere.
	const bool bPassToToolkitCommands = bRouteToToolsContext && !bWasHandledByToolsContext;
	bool bHandled = bWasHandledByToolsContext;
	ForEachEdMode([&bHandled, bPassToToolkitCommands, Event, Key, InViewportClient, Viewport](UEdMode* Mode)
	{
		// First, always give the legacy viewport interface a chance to process they key press. This is to support any of the FModeTools that may still exist.
		if (ILegacyEdModeViewportInterface* ViewportInterface = Cast<ILegacyEdModeViewportInterface>(Mode))
		{
			if (ViewportInterface->InputKey(InViewportClient, Viewport, Key, Event))
			{
				bHandled |= true;
				return true;  // Skip passing to the mode's toolkit if the legacy mode interface handled the input.
			}
		}
		
		// Next, give the toolkit commands a chance to process the key press if the tools context did not handle the key press.
		if (bPassToToolkitCommands && (Event != IE_Released) && Mode->UsesToolkits() && Mode->GetToolkit().IsValid())
		{
			bHandled |= Mode->GetToolkit().Pin()->GetToolkitCommands()->ProcessCommandBindings(Key, FSlateApplication::Get().GetModifierKeys(), (Event == EInputEvent::IE_Repeat));
			return true;
		}

		return true;
		});

	return bHandled;
}

/** Notifies all active modes of axis movement */
bool FEditorModeTools::InputAxis(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime)
{
	bool bHandled = false;

	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport, ControllerId, Key, Delta, DeltaTime](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->InputAxis(InViewportClient, Viewport, ControllerId, Key, Delta, DeltaTime);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::GetPivotForOrbit( FVector& Pivot ) const
{
	bool bHandled = false;
	// Just return the first pivot point specified by a mode
	ForEachEdMode([&Pivot, &bHandled](const UEdMode* Mode)
		{
			bHandled = Mode->GetPivotForOrbit(Pivot);
			return !bHandled;
		});
		
	return bHandled;
}

bool FEditorModeTools::MouseEnter( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 X, int32 Y )
{
	HoveredViewportClient = InViewportClient;
	bool bHandled = InteractiveToolsContext->MouseEnter(InViewportClient, Viewport, X, Y);

	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport, X, Y](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->MouseEnter(InViewportClient, Viewport, X, Y);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::MouseLeave( FEditorViewportClient* InViewportClient, FViewport* Viewport )
{
	// TODO: HoveredViewportClient should be reset here, but there is currently a bug (UE-119516) 
	// that makes it so that flying in viewports can create mismatches between MouseEnter and MouseLeave.
	// For this reason, we currently use HoveredViewportClient as if it were LastHoveredViewportClient,
	// which works for the purposes that we use it for.
	// If we never fix the bug, we should probably just rename it.
	//HoveredViewportClient = nullptr;
	
	bool bHandled = InteractiveToolsContext->MouseLeave(InViewportClient, Viewport);

	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->MouseLeave(InViewportClient, Viewport);
			return true;
		});

	return bHandled;
}

/** Notifies all active modes that the mouse has moved */
bool FEditorModeTools::MouseMove( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 X, int32 Y )
{
	bool bHandled = InteractiveToolsContext->MouseMove(InViewportClient, Viewport, X, Y);

	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport, X, Y](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->MouseMove(InViewportClient, Viewport, X, Y);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::ReceivedFocus( FEditorViewportClient* InViewportClient, FViewport* Viewport )
{
	FocusedViewportClient = InViewportClient;
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->ReceivedFocus(InViewportClient, Viewport);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::LostFocus( FEditorViewportClient* InViewportClient, FViewport* Viewport )
{
	// Note that we don't reset FocusedViewportClient intentionally. EdModeInteractiveToolsContext
	// only ticks its objects once for the focused viewport to avoid multi-ticking, so if we cleared
	// it here, we'd stop ticking things in the level editor when clicking out of the viewport.
	// TODO: Conceptually, we should probably clear FocusedViewportClient here, but also have a
	// LastFocusedViewportClient property that we don't clear, to use in ticking.

	bool bHandled = InteractiveToolsContext->LostFocus(InViewportClient, Viewport);

	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient, Viewport](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->LostFocus(InViewportClient, Viewport);
			return true;
		});

	return bHandled;
}

/** Draws all active mode components */	
void FEditorModeTools::DrawActiveModes( const FSceneView* InView, FPrimitiveDrawInterface* PDI )
{
	ForEachEdMode<ILegacyEdModeDrawHelperInterface>([InView, PDI](ILegacyEdModeDrawHelperInterface* DrawHelper)
		{
			DrawHelper->Draw(InView, PDI);
			return true;
		});
}

/** Renders all active modes */
void FEditorModeTools::Render( const FSceneView* InView, FViewport* Viewport, FPrimitiveDrawInterface* PDI )
{
	InteractiveToolsContext->Render(InView, Viewport, PDI);

	ForEachEdMode<ILegacyEdModeWidgetInterface>([InView, Viewport, PDI](ILegacyEdModeWidgetInterface* Mode)
		{
			Mode->Render(InView, Viewport, PDI);
			return true;
		});
}

/** Draws the HUD for all active modes */
void FEditorModeTools::DrawHUD( FEditorViewportClient* InViewportClient,FViewport* Viewport, const FSceneView* View, FCanvas* Canvas )
{
	InteractiveToolsContext->DrawHUD(InViewportClient, Viewport, View, Canvas);

	DrawBrackets(InViewportClient, Viewport, View, Canvas);

	if (!(InViewportClient->EngineShowFlags.ModeWidgets))
	{
		return;
	}

	// Clear Hit proxies
	const bool bIsHitTesting = Canvas->IsHitTesting();
	if (!bIsHitTesting)
	{
		Canvas->SetHitProxy(nullptr);
	}

	ForEachEdMode<ILegacyEdModeWidgetInterface>([InViewportClient, Viewport, View, Canvas](ILegacyEdModeWidgetInterface* Mode)
		{
			Mode->DrawHUD(InViewportClient, Viewport, View, Canvas);
			return true;
		});

	// Draw vertices for selected BSP brushes and static meshes if the large vertices show flag is set.
	if (!InViewportClient->bDrawVertices)
	{
		return;
	}

	const bool bLargeVertices = View->Family->EngineShowFlags.LargeVertices;
	if (!bLargeVertices)
	{
		return;
	}

	// Temporaries.
	const bool bShowBrushes = View->Family->EngineShowFlags.Brushes;
	const bool bShowBSP = View->Family->EngineShowFlags.BSP;
	const bool bShowBuilderBrush = View->Family->EngineShowFlags.BuilderBrush != 0;

	UTexture2D* VertexTexture = GetVertexTexture();
	const float TextureSizeX = VertexTexture->GetSizeX() * (bLargeVertices ? 1.0f : 0.5f);
	const float TextureSizeY = VertexTexture->GetSizeY() * (bLargeVertices ? 1.0f : 0.5f);

	GetEditorSelectionSet()->ForEachSelectedObject<AStaticMeshActor>([View, Canvas, VertexTexture, TextureSizeX, TextureSizeY, bIsHitTesting](AStaticMeshActor* Actor)
		{
			TArray<FVector> Vertices;
			FCanvasItemTestbed::bTestState = !FCanvasItemTestbed::bTestState;

			// Static mesh vertices
			if (Actor->GetStaticMeshComponent() && Actor->GetStaticMeshComponent()->GetStaticMesh()
				&& Actor->GetStaticMeshComponent()->GetStaticMesh()->GetRenderData())
			{
				FTransform ActorToWorld = Actor->ActorToWorld();
				const FPositionVertexBuffer& VertexBuffer = Actor->GetStaticMeshComponent()->GetStaticMesh()->GetRenderData()->LODResources[0].VertexBuffers.PositionVertexBuffer;
				for (uint32 i = 0; i < VertexBuffer.GetNumVertices(); i++)
				{
					Vertices.AddUnique(ActorToWorld.TransformPosition((FVector)VertexBuffer.VertexPosition(i)));
				}

				const float InvDpiScale = 1.0f / Canvas->GetDPIScale();

				FCanvasTileItem TileItem(FVector2D(0.0f, 0.0f), FVector2D(0.0f, 0.0f), FLinearColor::White);
				TileItem.BlendMode = SE_BLEND_Translucent;
				for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
				{
					const FVector& Vertex = Vertices[VertexIndex];
					FVector2D PixelLocation;
					if (View->ScreenToPixel(View->WorldToScreen(Vertex), PixelLocation))
					{
						PixelLocation *= InvDpiScale;

						const bool bOutside =
							PixelLocation.X < 0.0f || PixelLocation.X > View->UnscaledViewRect.Width() * InvDpiScale ||
							PixelLocation.Y < 0.0f || PixelLocation.Y > View->UnscaledViewRect.Height() * InvDpiScale;
						if (!bOutside)
						{
							const double X = PixelLocation.X - (TextureSizeX / 2);
							const double Y = PixelLocation.Y - (TextureSizeY / 2);
							if (bIsHitTesting)
							{
								Canvas->SetHitProxy(new HStaticMeshVert(Actor, Vertex));
							}
							TileItem.Texture = VertexTexture->GetResource();

							TileItem.Size = FVector2D(TextureSizeX, TextureSizeY);
							Canvas->DrawItem(TileItem, FVector2D(X, Y));
							if (bIsHitTesting)
							{
								Canvas->SetHitProxy(nullptr);
							}
						}
					}
				}
			}

			return true;
		});
}

/** Calls PostUndo on all active modes */
void FEditorModeTools::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		ForEachEdMode([](UEdMode* Mode)
			{
				Mode->PostUndo();
				return true;
			});
	}
}
void FEditorModeTools::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

/** true if we should allow widget move */
bool FEditorModeTools::AllowWidgetMove() const
{
	bool bAllow = false;
	ForEachEdMode<ILegacyEdModeWidgetInterface>([&bAllow](ILegacyEdModeWidgetInterface* LegacyMode)
		{
			bAllow |= LegacyMode->AllowWidgetMove();
			return true;
		});

	return bAllow;
}

bool FEditorModeTools::DisallowMouseDeltaTracking() const
{
	bool bDisallow = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bDisallow](ILegacyEdModeViewportInterface* LegacyMode)
		{
			bDisallow |= LegacyMode->DisallowMouseDeltaTracking();
			return true;
		});

	return bDisallow;
}

bool FEditorModeTools::RequiresLegacyViewportInteractions() const
{
	bool bRequiresGizmos = false;
	ForEachEdMode([&bRequiresGizmos, this](UEdMode* EdMode)
		{
			if (ILegacyEdModeViewportInterface* Interface = Cast<ILegacyEdModeViewportInterface>(EdMode))
			{
				if (Interface->RequiresLegacyViewportInteractions())
				{
					if (bViewportITF)
					{
						UE_LOGF(LogEditorModes, Warning, "ITF Gizmos and Controls have been disabled. The \"%ls\" Editor Mode requires the legacy systems.", *EdMode->GetModeInfo().Name.ToString())
					}
					
					bRequiresGizmos = true;
				}
			}
			return true;
		});
	return bRequiresGizmos;	
}

bool FEditorModeTools::GetCursor(EMouseCursor::Type& OutCursor) const
{
	bool bHandled = false;

	if (CVarEnableITFCursorOverrideSupport.GetValueOnGameThread() > 0)
	{
		bHandled = InteractiveToolsContext->GetCursor(OutCursor);
	}

	if (!bHandled)
	{
		for (const UEdMode* Mode : ActiveScriptableModes)
		{
			bHandled |= Mode->GetCursor(OutCursor);
		}
	}
	return bHandled;
}

TSharedPtr<SWidget> FEditorModeTools::MapCursor() const
{
	if (CVarEnableITFCursorOverrideSupport.GetValueOnGameThread() > 0)
	{
		return InteractiveToolsContext->GetCursorWidget();
	}

	return nullptr;
}

bool FEditorModeTools::GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const
{
	return GetCursorVisibilityOverride(bWantsOverride, bHardwareCursorVisible, bSoftwareCursorVisible);
}

bool FEditorModeTools::GetCursorVisibilityOverride(bool& bOutWantsOverride, bool& bOutHardwareCursorVisible, bool& bOutSoftwareCursorVisible) const
{
	if (CVarEnableITFCursorOverrideSupport.GetValueOnGameThread() > 0)
	{
		if (InteractiveToolsContext->GetCursorVisibilityOverride(bOutHardwareCursorVisible, bOutSoftwareCursorVisible))
		{
			bOutWantsOverride = true;
			return true;
		}
	}

	bool bHandled = false;
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		bHandled |= Mode->GetOverrideCursorVisibility(bOutWantsOverride, bOutHardwareCursorVisible, bOutSoftwareCursorVisible);
	}
	return bHandled;
}

bool FEditorModeTools::PreConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient](ILegacyEdModeViewportInterface* Mode)
		{
			bHandled |= Mode->PreConvertMouseMovement(InViewportClient);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::PostConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	bool bHandled = false;
	ForEachEdMode<ILegacyEdModeViewportInterface>([&bHandled, InViewportClient](ILegacyEdModeViewportInterface* ViewportInterface)
		{
			bHandled |= ViewportInterface->PostConvertMouseMovement(InViewportClient);
			return true;
		});

	return bHandled;
}

bool FEditorModeTools::GetShowWidget() const
{
	if (!bShowWidget)
	{
		return false;
	}
	
	bool bDrawModeSupportsWidgetDrawing = false;
	// Check to see of any active modes support widget drawing
	ForEachEdMode<ILegacyEdModeWidgetInterface>([&bDrawModeSupportsWidgetDrawing](ILegacyEdModeWidgetInterface* LegacyMode)
		{
			bDrawModeSupportsWidgetDrawing |= LegacyMode->ShouldDrawWidget();
			return !bDrawModeSupportsWidgetDrawing;
		});
	return bDrawModeSupportsWidgetDrawing;
}

/**
 * Used to cycle widget modes
 */
void FEditorModeTools::CycleWidgetMode (void)
{
	//make sure we're not currently tracking mouse movement.  If we are, changing modes could cause a crash due to referencing an axis/plane that is incompatible with the widget
	for (FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (ViewportClient->IsTracking())
		{
			return;
		}
	}

	//only cycle when the mode is requesting the drawing of a widget
	if( GetShowWidget() )
	{
		const int32 CurrentWk = GetWidgetMode();
		int32 Wk = CurrentWk;
		do
		{
			Wk++;
			if ((Wk == UE::Widget::WM_TranslateRotateZ) && (!GetDefault<ULevelEditorViewportSettings>()->bAllowTranslateRotateZWidget))
			{
				Wk++;
			}
			// Roll back to the start if we go past UE::Widget::WM_Scale
			if( Wk >= UE::Widget::WM_Max)
			{
				Wk -= UE::Widget::WM_Max;
			}
		}
		while (!UsesTransformWidget((UE::Widget::EWidgetMode)Wk) && Wk != CurrentWk);
		SetWidgetMode( (UE::Widget::EWidgetMode)Wk );
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
	}
}

/**Save Widget Settings to Ini file*/
void FEditorModeTools::SaveWidgetSettings(void)
{
	GetMutableDefault<UEditorPerProjectUserSettings>()->SaveConfig();
}

/**Load Widget Settings from Ini file*/
void FEditorModeTools::LoadWidgetSettings(void)
{
}

/**
 * Returns a good location to draw the widget at.
 */

FVector FEditorModeTools::GetWidgetLocation() const
{
	for (int Index = ActiveScriptableModes.Num() - 1; Index >= 0; Index--)
	{
		if (ILegacyEdModeWidgetInterface* LegacyMode = Cast<ILegacyEdModeWidgetInterface>(ActiveScriptableModes[Index]))
		{
			if (LegacyMode->UsesTransformWidget())
			{
				return LegacyMode->GetWidgetLocation();
			}
		}
	}
	
	return FVector(ForceInitToZero);
}

/**
 * Changes the current widget mode.
 */

void FEditorModeTools::SetWidgetMode( UE::Widget::EWidgetMode InWidgetMode )
{
	WidgetMode = InWidgetMode;
}

/**
 * Allows you to temporarily override the widget mode.  Call this function again
 * with UE::Widget::WM_None to turn off the override.
 */

void FEditorModeTools::SetWidgetModeOverride( UE::Widget::EWidgetMode InWidgetMode )
{
	OverrideWidgetMode = InWidgetMode;
}

/**
 * Retrieves the current widget mode, taking overrides into account.
 */

UE::Widget::EWidgetMode FEditorModeTools::GetWidgetMode() const
{
	if( OverrideWidgetMode != UE::Widget::WM_None )
	{
		return OverrideWidgetMode;
	}

	return WidgetMode;
}


/**
* Set Scale On The Widget
*/

void FEditorModeTools::SetWidgetScale(float InScale)
{
	WidgetScale = InScale;
}

/**
* Get Scale On The Widget
*/

float FEditorModeTools::GetWidgetScale() const
{
	return WidgetScale;
}

void FEditorModeTools::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects(ActiveScriptableModes);
	Collector.AddReferencedObjects(PendingDeactivateModes);
	Collector.AddReferencedObjects(RecycledScriptableModes);
	Collector.AddReferencedObject(InteractiveToolsContext);
}

FEdMode* FEditorModeTools::GetActiveMode( FEditorModeID InID )
{
	if (UEdMode* Mode = GetActiveScriptableMode(InID))
	{
		return Mode->AsLegacyMode();
	}

	return nullptr;
}

const FEdMode* FEditorModeTools::GetActiveMode( FEditorModeID InID ) const
{
	if (UEdMode* Mode = GetActiveScriptableMode(InID))
	{
		return Mode->AsLegacyMode();
	}

	return nullptr;
}

const FModeTool* FEditorModeTools::GetActiveTool( FEditorModeID InID ) const
{
	ILegacyEdModeToolInterface* ActiveMode = Cast<ILegacyEdModeToolInterface>(GetActiveScriptableMode( InID ));
	const FModeTool* Tool = nullptr;
	if( ActiveMode )
	{
		Tool = ActiveMode->GetCurrentTool();
	}
	return Tool;
}

bool FEditorModeTools::IsModeActive( FEditorModeID InID ) const
{
	return (GetActiveScriptableMode(InID) != nullptr);
}

bool FEditorModeTools::IsDefaultModeActive() const
{
	bool bAllDefaultModesActive = true;
	for( const FEditorModeID& ModeID : DefaultModeIDs )
	{
		if( !IsModeActive( ModeID ) )
		{
			bAllDefaultModesActive = false;
			break;
		}
	}
	return bAllDefaultModesActive;
}

bool FEditorModeTools::CanCycleWidgetMode() const
{
	bool bCanCycleWidget = false;
	ForEachEdMode<ILegacyEdModeWidgetInterface>([&bCanCycleWidget](ILegacyEdModeWidgetInterface* LegacyMode)
		{
			bCanCycleWidget = LegacyMode->CanCycleWidgetMode();
			return !bCanCycleWidget;
		});
	return bCanCycleWidget;
}


bool FEditorModeTools::CanAutoSave() const
{
	return FEditorModeTools::TestAllModes([](UEdMode* Mode) { return Mode->CanAutoSave(); }, true);
}


bool FEditorModeTools::OnRequestClose()
{
	return FEditorModeTools::TestAllModes([](UEdMode* Mode) { return Mode->OnRequestClose(); }, true);
}

bool FEditorModeTools::IsOperationSupportedForCurrentAsset(EAssetOperation InOperation) const
{
	return FEditorModeTools::TestAllModes([InOperation](UEdMode* Mode) { return Mode->IsOperationSupportedForCurrentAsset(InOperation); }, true);
}

UModeManagerInteractiveToolsContext* FEditorModeTools::GetInteractiveToolsContext() const
{
	return InteractiveToolsContext;
}

bool FEditorModeTools::UsesViewportITF() const
{
	return bViewportITF && InteractiveToolsContext && InteractiveToolsContext->UsesViewportInteractions() && UEditorInteractiveGizmoManager::UsesNewTRSGizmos();
}

void FEditorModeTools::SetSupportsViewportITF(bool bIsSupported)
{
	bViewportITF = bIsSupported;
}

bool FEditorModeTools::SupportsViewportITF() const
{
	return bViewportITF;
}

IGizmoStateTarget* FEditorModeTools::GetGizmoStateTarget()
{
	if (!WeakGizmoStateTarget.IsValid())
	{
		WeakGizmoStateTarget = UEditorGizmoStateTarget::Construct(
			this,
			NSLOCTEXT("UTransformGizmo", "UTransformGizmoTransaction", "Transform"),
			InteractiveToolsContext->GizmoManager);
	}
	return WeakGizmoStateTarget.Get();
}

UEditorTransformGizmoContextObject* FEditorModeTools::GetGizmoContext() const
{
	if (!WeakGizmoContext.IsValid())
	{
		const UModeManagerInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext();
		const UContextObjectStore* ContextStore = ToolsContext ? ToolsContext->ToolManager->GetContextObjectStore() : nullptr;
		WeakGizmoContext = ContextStore ? ContextStore->FindContext<UEditorTransformGizmoContextObject>() : nullptr;
	}
	return WeakGizmoContext.Get();
}

const FRotationContext& FEditorModeTools::GetRotationContext() const
{
	static const FRotationContext DefaultContext;
	return WeakGizmoContext.IsValid() ? WeakGizmoContext->RotationContext : DefaultContext;
}

bool FEditorModeTools::BeginTransform(const FGizmoState& InState)
{
	bool bHandled = false;
	ForEachEdMode<IGizmoEdModeInterface>([&bHandled, &InState](IGizmoEdModeInterface* GizmoInterface)
	{
		bHandled |= GizmoInterface->BeginTransform(InState);
		return true;
	});

	// Give the focused VPC an opportunity to open the transform transacting if it has not been handled before 
	if (!bHandled && FocusedViewportClient)
	{
		bHandled = FocusedViewportClient->BeginTransform(InState);
	}

	bHasOngoingTransform = bHandled;
	
	return bHandled;
}

bool FEditorModeTools::EndTransform(const FGizmoState& InState) const
{
	bool bHandled = false;
	ForEachEdMode<IGizmoEdModeInterface>([&bHandled, &InState](IGizmoEdModeInterface* GizmoInterface)
	{
		bHandled |= GizmoInterface->EndTransform(InState);
		return true;
	});

	// Give the focused VPC an opportunity to close the transform transacting if it has not been handled before 
	if (!bHandled && FocusedViewportClient)
	{
		bHandled = FocusedViewportClient->EndTransform(InState);
	}

	// NOTE bHasOngoingTransform is not set to false here but in EndTracking as its needed there.
	// See header file more more explanations
	
	return bHandled;
}

bool FEditorModeTools::OnTransformCanceled(const FGizmoState& InState) const
{
	bool bHandled = false;
	ForEachEdMode<IGizmoEdModeInterface>([&bHandled, &InState](IGizmoEdModeInterface* GizmoInterface)
	{
		bHandled |= GizmoInterface->OnTransformCanceled(InState);
		return true;
	});

	// Give the focused VPC an opportunity to close the transform transacting if it has not been handled before 
	if (!bHandled && FocusedViewportClient)
	{
		bHandled = FocusedViewportClient->OnTransformCanceled(InState);
	}

	// NOTE bHasOngoingTransform is not set to false here but in EndTracking as its needed there.
	// See header file more more explanations
	
	return bHandled;
}

bool FEditorModeTools::HasOngoingTransform() const
{
	return bHasOngoingTransform;
}
