// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeRegistry.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "EdMode.h"
#include "EditorModes.h"

#include "IPlacementModeModule.h"
#include "LandscapeEditorModule.h"
#include "MeshPaintModule.h"
#include "ActorPickerMode.h"
#include "SceneDepthPickerMode.h"
#include "FoliageEditModule.h"
#include "VirtualTexturingEditorModule.h"
#include "Subsystems/AssetEditorSubsystem.h"

FEditorModeFactory::FEditorModeFactory(const FEditorModeInfo& InModeInfo)
	: ModeInfo(InModeInfo)
{
}

FEditorModeFactory::FEditorModeFactory(FEditorModeInfo&& InModeInfo)
	: ModeInfo(InModeInfo)
{
}

FEditorModeFactory::~FEditorModeFactory()
{
}

FEditorModeInfo FEditorModeFactory::GetModeInfo() const
{
	return ModeInfo;
}

TSharedRef<FEdMode> FEditorModeFactory::CreateMode() const
{
	return FactoryCallback.Execute();
}

void FEditorModeRegistry::Initialize()
{
	// Send notifications for any legacy modes that were registered before the asset subsystem started up
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	for (const FactoryMap::ElementType& ModeEntry : ModeFactories)
	{
		AssetEditorSubsystem->OnEditorModeRegistered().Broadcast(ModeEntry.Key);
	}

	// Add default editor modes
	FModuleManager::LoadModuleChecked<FActorPickerModeModule>(TEXT("ActorPickerMode"));
	FModuleManager::LoadModuleChecked<FSceneDepthPickerModeModule>(TEXT("SceneDepthPickerMode"));
	FModuleManager::LoadModuleChecked<ILandscapeEditorModule>(TEXT("LandscapeEditor"));
	FModuleManager::LoadModuleChecked<IFoliageEditModule>(TEXT("FoliageEdit"));
	FModuleManager::LoadModuleChecked<IVirtualTexturingEditorModule>(TEXT("VirtualTexturingEditor"));

	bInitialized = true;
}

void FEditorModeRegistry::Shutdown()
{
	bInitialized = false;

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	for (auto& ModeEntry : ModeFactories)
	{
		AssetEditorSubsystem->OnEditorModeUnregistered().Broadcast(ModeEntry.Key);
	}

	ModeFactories.Empty();
}

FEditorModeRegistry& FEditorModeRegistry::Get()
{
	static TSharedRef<FEditorModeRegistry> GModeRegistry = MakeShared<FEditorModeRegistry>();
	return GModeRegistry.Get();	
}

TArray<FEditorModeInfo> FEditorModeRegistry::GetSortedModeInfo() const
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetEditorModeInfoOrderedByPriority();
}

FEditorModeInfo FEditorModeRegistry::GetModeInfo(FEditorModeID ModeID) const
{
	FEditorModeInfo Result;
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorModeInfo(ModeID, Result);
	return Result;
}

TSharedPtr<FEdMode> FEditorModeRegistry::CreateMode(FEditorModeID ModeID, FEditorModeTools& Owner)
{
	const TSharedRef<IEditorModeFactory>* ModeFactory = ModeFactories.Find(ModeID);
	if (ModeFactory)
	{
		TSharedRef<FEdMode> Instance = (*ModeFactory)->CreateMode();

		// Assign the mode info from the factory before we initialize
		Instance->Info = (*ModeFactory)->GetModeInfo();
		Instance->Owner = &Owner;

		Instance->Initialize();

		return Instance;
	}

	return nullptr;
}

void FEditorModeRegistry::RegisterMode(FEditorModeID ModeID, TSharedRef<IEditorModeFactory> Factory)
{
	check(ModeID != FBuiltinEditorModes::EM_None);
	check(!ModeFactories.Contains(ModeID));

	ModeFactories.Add(ModeID, Factory);

	if (bInitialized)
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		AssetEditorSubsystem->OnEditorModeRegistered().Broadcast(ModeID);
		AssetEditorSubsystem->OnEditorModesChanged().Broadcast();
	}
}

void FEditorModeRegistry::UnregisterMode(FEditorModeID ModeID)
{
	// First off delete the factory
	if (ModeFactories.Remove(ModeID) > 0 && bInitialized)
	{
		if (GEditor)
		{
			if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				AssetEditorSubsystem->OnEditorModeUnregistered().Broadcast(ModeID);
				AssetEditorSubsystem->OnEditorModesChanged().Broadcast();
			}
		}
	}
}

FRegisteredModesChangedEvent& FEditorModeRegistry::OnRegisteredModesChanged()
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnEditorModesChanged();
}


FOnModeRegistered& FEditorModeRegistry::OnModeRegistered()
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnEditorModeRegistered();
}


FOnModeUnregistered& FEditorModeRegistry::OnModeUnregistered()
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnEditorModeUnregistered();
}
