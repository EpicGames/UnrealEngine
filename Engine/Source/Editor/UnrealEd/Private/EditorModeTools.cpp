// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorModeTools.h"


FModeTool::FModeTool():
	ID( MT_None ),
	bUseWidget( 1 )
{}

FModeTool::~FModeTool()
{
}

void FModeTool::DrawHUD(FEditorViewportClient* ViewportClient,FViewport* Viewport,const FSceneView* View,FCanvas* Canvas)
{
}

void FModeTool::Render(const FSceneView* View,FViewport* Viewport,FPrimitiveDrawInterface* PDI)
{
}


