// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorObject.cpp: Unreal Editor object manipulation code.
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyPortFlags.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Misc/AsciiSet.h"
#include "Misc/PackageName.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "Model.h"
#include "Engine/Brush.h"
#include "Editor/EditorEngine.h"
#include "Factories/ModelFactory.h"
#include "GameFramework/Volume.h"
#include "Editor.h"
#include "BSPOps.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "FoliageType.h"
#include "InstancedFoliageActor.h"
#include "InstancedFoliage.h"
#include "Components/BrushComponent.h"
#include "Algo/Transform.h"
#include "UObject/OverriddenPropertySet.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorObject, Log, All);

/*
Subobject Terms -
Much of the confusion in dealing with subobjects and instancing can be traced to the ambiguity of the words used to work with the various concepts.
A standardized method of referring to these terms is highly recommended - it makes the code much more consistent, and well thought-out variable names
make the concepts and especially the relationships between each of the concepts easier to grasp.  This will become even more apparent once archetypes
and prefabs are implemented.

Once we've decided on standard terms, we should try to use these words as the name for any variables which refer to the associated concept, in any
code that deals with that concept (where possible).

Here are some terms I came up with for starters.  If you're reading this, and you have a more appropriate name for one of these concepts, feel that any
of the descriptions or terms isn't clear enough, or know of a concept that isn't represented here, feel free to modify this comment and update
the appropriate code, if applicable.



Instance:
a UObject that has been instanced from a subobject template

Template (or template object):
the UObject associated with [or created by] an inline subobject definition; stored in the UClass's Defaults array (in the case of a .h subobject).  

TemplateName:
the name of the template object

TemplateClass:
the class of the Template object

TemplateOwner:
the UObject that contains the template object;  when dealing with templates created via inline subobject 
definitions, this corresponds to the class that contains the Begin Object block for the template

SubobjectRoot:
when dealing with nested subobjects, corresponds to the top-most Outer that is not a subobject or template (generally
the same as Outer)
*/

class FDefaultPropertiesContextSupplier : public FContextSupplier
{
public:
	/** the current line number */
	int32 CurrentLine;

	/** the package we're processing */
	FString PackageName;

	/** the class we're processing */
	FString ClassName;

	FString GetContext()
	{
		return FString::Printf
		(
			TEXT("%sDevelopment/Src/%s/Classes/%s.h(%i)"),
			*FPaths::RootDir(),
			*PackageName,
			*ClassName,
			CurrentLine
		);
	}

	FDefaultPropertiesContextSupplier() {}
	FDefaultPropertiesContextSupplier( const TCHAR* Package, const TCHAR* Class, int32 StartingLine )
	: CurrentLine(StartingLine), PackageName(Package), ClassName(Class)
	{
	}

};

static FDefaultPropertiesContextSupplier* ContextSupplier = NULL;


void UEditorEngine::RenameObject(UObject* Object,UObject* NewOuter,const TCHAR* NewName, ERenameFlags Flags)
{
	Object->Rename(NewName, NewOuter, Flags);
	Object->SetFlags(RF_Public | RF_Standalone);
	Object->MarkPackageDirty();
}


static void RemapProperty(FProperty* Property, int32 Index, const TMap<FSoftObjectPath, UObject*>& ObjectRemapper, uint8* DestData)
{
	if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		// If there's a concrete index, use that, otherwise iterate all array members (for the case that this property is inside a struct, or there is exactly one element)
		const int32 Num = (Index == INDEX_NONE) ? ObjectProperty->ArrayDim : 1;
		const int32 StartIndex = (Index == INDEX_NONE) ? 0 : Index;
		for (int32 Count = 0; Count < Num; Count++)
		{
			uint8* PropertyAddr = ObjectProperty->ContainerPtrToValuePtr<uint8>(DestData, StartIndex + Count);
			UObject* Object = ObjectProperty->GetObjectPropertyValue(PropertyAddr);
			if (Object)
			{
				UObject* const* RemappedObject = ObjectRemapper.Find(Object);
				if (RemappedObject && (*RemappedObject)->GetClass()->IsChildOf(ObjectProperty->PropertyClass))
				{
					ObjectProperty->SetObjectPropertyValue(PropertyAddr, *RemappedObject);
				}
			}
			else if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
			{
				const FSoftObjectPtr& SoftObjectPtr = SoftObjectProperty->GetPropertyValue(PropertyAddr);
				UObject* const* RemappedObject = ObjectRemapper.Find(SoftObjectPtr.ToSoftObjectPath());
				if (RemappedObject && (*RemappedObject)->GetClass()->IsChildOf(SoftObjectProperty->PropertyClass))
				{
					SoftObjectProperty->SetObjectPropertyValue(PropertyAddr, *RemappedObject);
				}
			}
		}
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(DestData));
		if (Index != INDEX_NONE)
		{
			RemapProperty(ArrayProperty->Inner, INDEX_NONE, ObjectRemapper, ArrayHelper.GetRawPtr(Index));
		}
		else
		{
			for (int32 ArrayIndex = 0; ArrayIndex < ArrayHelper.Num(); ArrayIndex++)
			{
				RemapProperty(ArrayProperty->Inner, INDEX_NONE, ObjectRemapper, ArrayHelper.GetRawPtr(ArrayIndex));
			}
		}
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (StructProperty->Struct == TBaseStructure<FSoftObjectPath>::Get())
		{
			// If there's a concrete index, use that, otherwise iterate all array members (for the case that this property is inside a struct, or there is exactly one element)
			const int32 Num = (Index == INDEX_NONE) ? StructProperty->ArrayDim : 1;
			const int32 StartIndex = (Index == INDEX_NONE) ? 0 : Index;
			for (int32 Count = 0; Count < Num; Count++)
			{
				FSoftObjectPath* SoftObjectPathPtr = StructProperty->ContainerPtrToValuePtr<FSoftObjectPath>(DestData, StartIndex + Count);
				if (UObject* const* RemappedObject = ObjectRemapper.Find(*SoftObjectPathPtr))
				{
					*SoftObjectPathPtr = *RemappedObject;
				}
			}
		}
		else if (Index != INDEX_NONE)
		{
			// If a concrete index was given, remap just that
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				RemapProperty(*It, INDEX_NONE, ObjectRemapper, StructProperty->ContainerPtrToValuePtr<uint8>(DestData, Index));
			}
		}
		else
		{
			// If no concrete index was given, either the ArrayDim is 1 (i.e. not a static array), or the struct is within
			// a deeper structure (an array or another struct) and we cannot know which element was changed, so iterate through all elements.
			for (int32 Count = 0; Count < StructProperty->ArrayDim; Count++)
			{
				for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
				{
					RemapProperty(*It, INDEX_NONE, ObjectRemapper, StructProperty->ContainerPtrToValuePtr<uint8>(DestData, Count));
				}
			}
		}
	}
}


//
//	ImportProperties
//

bool HandleLineNumber(const TCHAR* Str)
{
	int32 NewLineNumber;
	if (FParse::Value(Str, TEXT("linenumber="), NewLineNumber))
	{
		if (ContextSupplier != nullptr)
		{
			ContextSupplier->CurrentLine = NewLineNumber;
		}

		return true;
	}

	return false;
}

static bool IsEndOfProperties(const TCHAR* Str, int32 Depth)
{
	return GetEND(&Str, TEXT("Actor")) || GetEND(&Str, TEXT("DefaultProperties")) || GetEND(&Str, TEXT("structdefaultproperties")) || (GetEND(&Str, TEXT("Object")) && Depth);
}

/**
 * Parse and import text as property values for the object specified.  This function should never be called directly - use ImportObjectProperties instead.
 * 
 * @param	ObjectStruct				the struct for the data we're importing
 * @param	DestData					the location to import the property values to
 * @param	SourceText					pointer to a buffer containing the values that should be parsed and imported
 * @param	SubobjectRoot					when dealing with nested subobjects, corresponds to the top-most outer that
 *										is not a subobject/template
 * @param	SubobjectOuter				the outer to use for creating subobjects/components. NULL when importing structdefaultproperties
 * @param	Warn						output device to use for log messages
 * @param	Depth						current nesting level
 * @param	InstanceGraph				contains the mappings of instanced objects and components to their templates
 * @param	ObjectRemapper				a map of existing objects, typically actors, to new instances, used to replace internal references when a number of objects are copy+pasted
 *
 * @return	NULL if the default values couldn't be imported
 */
static const TCHAR* ImportProperties(
	uint8*						DestData,
	const TCHAR*				SourceText,
	UStruct*					ObjectStruct,
	UObject*					SubobjectRoot,
	UObject*					SubobjectOuter,
	FFeedbackContext*			Warn,
	int32						Depth,
	FObjectInstancingGraph&		InstanceGraph,
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper
	)
{
	check(ObjectStruct!=NULL);
	check(DestData!=NULL);

	if ( SourceText == NULL )
	{
		return NULL;
	}
	
	FOverriddenPropertySet* OverriddenProperties = nullptr;
	if (SubobjectOuter && FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization))
	{
		OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(SubobjectOuter);
	}
	FEnableOverridableSerializationScope Scope(OverriddenProperties != nullptr, OverriddenProperties);

	// Cannot create subobjects when importing struct defaults, or if SubobjectOuter (used as the Outer for any subobject declarations encountered) is NULL
	bool bSubObjectsAllowed = !ObjectStruct->IsA(UScriptStruct::StaticClass()) && SubobjectOuter != NULL;

	// true when DestData corresponds to a subobject in a class default object
	bool bSubObject = false;

	UClass* ComponentOwnerClass = NULL;

	if ( bSubObjectsAllowed )
	{
		bSubObject = SubobjectRoot != NULL && SubobjectRoot->HasAnyFlags(RF_ClassDefaultObject);
		if ( SubobjectRoot == NULL )
		{
			SubobjectRoot = SubobjectOuter;
		}

		ComponentOwnerClass = SubobjectOuter != NULL
			? SubobjectOuter->IsA(UClass::StaticClass())
				? CastChecked<UClass>(SubobjectOuter)
				: SubobjectOuter->GetClass()
			: NULL;
	}
	

	// The PortFlags to use for all ImportText calls
	uint32 PortFlags = PPF_Delimited | PPF_CheckReferences;
	if (GIsImportingT3D)
	{
		PortFlags |= PPF_AttemptNonQualifiedSearch;
	}

	FString StrLine;

	TArray<FDefinedProperty> DefinedProperties;

	// Local graphs to use for subobjects that require dedicated instancing graphs.
	TMap<UObject*, FObjectInstancingGraph> DedicatedGraphs;

	// ObjectRemapper to use if none is passed in. Use TOptional to avoid heap allocations and only pay for construction/destruction if Emplace is called.
	TOptional<TMap<FSoftObjectPath, UObject*>> ExportedPathToObjectCreated;

	// Parse all objects stored in the actor.
	// Build list of all text properties.
	bool ImportedBrush = 0;
	int32 LinesConsumed = 0;
	while (FParse::LineExtended(&SourceText, StrLine, LinesConsumed, FParse::ELineExtendedFlags::OldExactMode))
	{
		// remove extra whitespace and optional semicolon from the end of the line
		{
			int32 Length = StrLine.Len();
			while ( Length > 0 &&
					(StrLine[Length - 1] == TCHAR(';') || StrLine[Length - 1] == TCHAR(' ') || StrLine[Length - 1] == 9) )
			{
				Length--;
			}
			if (Length != StrLine.Len())
			{
				StrLine.LeftInline(Length, EAllowShrinking::No);
			}
		}

		if ( ContextSupplier != NULL )
		{
			ContextSupplier->CurrentLine += LinesConsumed;
		}
		if (StrLine.Len() == 0)
		{
			continue;
		}

		const TCHAR* Str = *StrLine;

		if (HandleLineNumber(Str))
		{
			continue;
		}
		else if( GetBEGIN(&Str,TEXT("Brush")) && ObjectStruct->IsChildOf(ABrush::StaticClass()) )
		{
			// If SubobjectOuter is NULL, we are importing defaults for a UScriptStruct's defaultproperties block
			if ( !bSubObjectsAllowed )
			{
				Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN BRUSH: Subobjects are not allowed in this context"));
				return NULL;
			}

			// Parse brush on this line.
			TCHAR BrushName[NAME_SIZE];
			if( FParse::Value( Str, TEXT("Name="), BrushName, NAME_SIZE ) )
			{
				// If an initialized brush with this name already exists in the level, rename the existing one.
				// It is deemed to be initialized if it has a non-zero poly count.
				// If it is uninitialized, the existing object will have been created by a forward reference in the import text,
				// and it will now be redefined.  This relies on the behavior that NewObject<> will return an existing pointer
				// if an object with the same name and outer is passed.
				UModel* ExistingBrush = FindObject<UModel>( SubobjectRoot, BrushName );
				if (ExistingBrush && ExistingBrush->Polys && ExistingBrush->Polys->Element.Num() > 0)
				{
					ExistingBrush->Rename();
				}

				// Create model.
				UModelFactory* ModelFactory = NewObject<UModelFactory>();
				ModelFactory->FactoryCreateText(UModel::StaticClass(), SubobjectRoot, FName(BrushName, FNAME_Add), RF_NoFlags, NULL, TEXT("t3d"), SourceText, SourceText+FCString::Strlen(SourceText), Warn);
				ImportedBrush = 1;
			}
		}
		else if (GetBEGIN(&Str, TEXT("Foliage")))
		{
			UFoliageType* SourceFoliageType;
			FName ComponentName;
			if (SubobjectRoot &&
				ParseObject<UFoliageType>(Str, TEXT("FoliageType="), SourceFoliageType, nullptr) &&
				FParse::Value(Str, TEXT("Component="), ComponentName) )
			{
				UPrimitiveComponent* ActorComponent = FindObjectFast<UPrimitiveComponent>(SubobjectRoot, ComponentName);

				if (ActorComponent && ActorComponent->GetComponentLevel())
				{
					UWorld* World = ActorComponent->GetWorld();
					TMap<AInstancedFoliageActor*, TArray<FFoliageInstance>> FoliageInstances;
					
					const TCHAR* StrPtr;
					FString TextLine;
					while (FParse::Line(&SourceText, TextLine))
					{
						StrPtr = *TextLine;
						if (GetEND(&StrPtr, TEXT("Foliage")))
						{
							break;
						}

						// Parse the instance properties
						FFoliageInstance Instance;
						FString Temp;
						if (FParse::Value(StrPtr, TEXT("Location="), Temp, false))
						{
							FVector Location;
							GetFVECTOR(*Temp, Location);
							Instance.Location = Location;
						}
						if (FParse::Value(StrPtr, TEXT("Rotation="), Temp, false))
						{
							GetFROTATOR(*Temp, Instance.Rotation, 1);
						}
						if (FParse::Value(StrPtr, TEXT("PreAlignRotation="), Temp, false))
						{
							GetFROTATOR(*Temp, Instance.PreAlignRotation, 1);
						}
						if (FParse::Value(StrPtr, TEXT("DrawScale3D="), Temp, false))
						{
							FVector DrawScale3D;
							GetFVECTOR(*Temp, DrawScale3D);
							Instance.DrawScale3D = (FVector3f)DrawScale3D;
						}
						FParse::Value(StrPtr, TEXT("Flags="), Instance.Flags);

						Instance.BaseComponent = ActorComponent;

						if (AInstancedFoliageActor* IFA = AInstancedFoliageActor::Get(World, true, ActorComponent->GetComponentLevel(), Instance.Location))
						{
							FoliageInstances.FindOrAdd(IFA).Add(MoveTemp(Instance));
						}
					}

					for (const auto& Pair : FoliageInstances)
					{
						AInstancedFoliageActor* IFA = Pair.Key;
						FFoliageInfo* MeshInfo = nullptr;
						UFoliageType* FoliageType = IFA->AddFoliageType(SourceFoliageType, &MeshInfo);
						TArray<const FFoliageInstance*> InstancePtrs;
						InstancePtrs.Reserve(Pair.Value.Num());
						Algo::Transform(Pair.Value, InstancePtrs, [](const FFoliageInstance& FoliageInstance) { return &FoliageInstance; });
						if (MeshInfo)
						{
							MeshInfo->AddInstances(FoliageType, InstancePtrs);
						}
					}
				}
			}
		}
		else if( GetBEGIN(&Str,TEXT("Object")))
		{
			// If SubobjectOuter is NULL, we are importing defaults for a UScriptStruct's defaultproperties block
			if ( !bSubObjectsAllowed )
			{
				Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: Subobjects are not allowed in this context"));
				return NULL;
			}

			// Parse subobject default properties.
			// Note: default properties subobjects have compiled class as their Outer (used for localization).
			UClass*	TemplateClass = NULL;
			bool bInvalidClass = false;
			ParseObject<UClass>(Str, TEXT("Class="), TemplateClass, nullptr, &bInvalidClass);
			
			if (bInvalidClass)
			{
				Warn->Logf(ELogVerbosity::Error,TEXT("BEGIN OBJECT: Invalid class specified: %s"), *StrLine);
				return NULL;
			}

			// parse the name of the template
			FName	TemplateName = NAME_None;
			FParse::Value(Str,TEXT("Name="),TemplateName);
			if(TemplateName == NAME_None)
			{
				Warn->Logf(ELogVerbosity::Error,TEXT("BEGIN OBJECT: Must specify valid name for subobject/component: %s"), *StrLine);
				return NULL;
			}

			FString ExportedObjectFullName;
			const bool bExportPath = FParse::Value(Str, TEXT("ExportPath="), ExportedObjectFullName);
			
			// points to the parent class's template subobject/component, if we are overriding a subobject/component declared in our parent class
			UObject* BaseTemplate = NULL;
			bool bRedefiningSubobject = false;
			if( TemplateClass )
			{
			}
			else
			{
				// next, verify that a template actually exists in the parent class
				UClass* ParentClass = ComponentOwnerClass->GetSuperClass();
				check(ParentClass);

				UObject* ParentCDO = ParentClass->GetDefaultObject();
				check(ParentCDO);

				BaseTemplate = StaticFindObjectFast(UObject::StaticClass(), SubobjectOuter, TemplateName);
				bRedefiningSubobject = (BaseTemplate != NULL);

				if (BaseTemplate == NULL)
				{
					BaseTemplate = StaticFindObjectFast(UObject::StaticClass(), ParentCDO, TemplateName);
				}
				
				if ( BaseTemplate == NULL )
				{
					// wasn't found
					Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: No base template named %s found in parent class %s: %s"), *TemplateName.ToString(), *ParentClass->GetName(), *StrLine);
					return NULL;
				}

				TemplateClass = BaseTemplate->GetClass();
			}
			
			const bool bOverrideT3DSupported = FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization);

			// Using override semantics means we do not import (shadowed) subobject references, as they are not explicitly overridden.
			// The legacy path will always use exported subobjects and replace references to default subobjects after moving them aside.
			const bool bUseOverrideSemantics = bOverrideT3DSupported && FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::SubObjectsShadowSerialization);

			// because the outer won't be a default object

			checkSlow(TemplateClass != NULL);
			if (bRedefiningSubobject)
			{
				// When using override semantics, during the initial definition phase (the 'else' case below), the exported
				// subobject may have been found to not be type-compatible with the default subobject that was instanced for
				// the outer object at construction time (e.g. due to schema changes after exporting). In that case, we will
				// have created a unique instance to serve as the destination for the redefinition here. This means that any
				// exported instance data will not be imported to the instanced default subobject, and its data will be lost.
				//
				// Note: Referencing properties are also NOT imported in this case unless explicitly overridden at export time,
				// so outside of that scenario, inner references (by design) are not remapped to the unique instance on import.
				//
				// @todo - IDO support will be needed to capture the unique instance and preserve its imported instance data.
				if (bUseOverrideSemantics
					&& FOverridableManager::Get().IsEnabled(BaseTemplate))
				{
					if (!ObjectRemapper)
					{
						// When using override T3D, always map exported subobject paths so they can be resolved at redefinition time
						ObjectRemapper = &ExportedPathToObjectCreated.Emplace();
					}

					FSoftObjectPath ExportedObjectPath(ExportedObjectFullName);
					if (!ensureMsgf(bExportPath, TEXT("Subobject redefinition is missing the object's export path while importing it.")))
					{
						// Fall back to the resolved subobject, however its path can differ from the exported path that it was mapped
						// to if e.g. there was a name collision within the outer scope, so it's not as reliable as the exported path.
						ExportedObjectPath = FSoftObjectPath(BaseTemplate);
					}

					UObject* const* RemappedObject = ObjectRemapper->Find(ExportedObjectPath);
					if (RemappedObject)
					{
						BaseTemplate = *RemappedObject;
					}
				}

				FObjectInstancingGraph* InstanceGraphToUse = DedicatedGraphs.Find(BaseTemplate);
				if (!InstanceGraphToUse)
				{
					if (FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(BaseTemplate))
					{
						InstanceGraphToUse = &DedicatedGraphs.Emplace(BaseTemplate, FObjectInstancingGraph(BaseTemplate));
					}
					else
					{
						InstanceGraphToUse = &InstanceGraph;
					}
				}

				// since we're redefining an object in the same text block, only need to import properties again
				SourceText = ImportObjectProperties( (uint8*)BaseTemplate, SourceText, TemplateClass, SubobjectRoot, BaseTemplate,
													Warn, Depth + 1, ContextSupplier ? ContextSupplier->CurrentLine : 0, InstanceGraphToUse, ObjectRemapper );

				// If the object import fails, early out and stop parsing.
				if (SourceText == nullptr)
				{
					Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: Could not import properties from sub-object %s."), *SubobjectRoot->GetName());
					return nullptr;
				}
			}
			else 
			{
				FName OverriddenOperationName = NAME_None;
				const bool bParsed = FParse::Value(Str,TEXT("OverriddenOperation="), OverriddenOperationName);
				TOptional<EOverriddenPropertyOperation> OverriddenOperation = bParsed && bOverrideT3DSupported ? GetOverriddenOperationFromName(OverriddenOperationName) : TOptional<EOverriddenPropertyOperation>();

				UObject* Archetype = NULL;
				UObject* ComponentTemplate = NULL;

				// Since we are changing the class we can't use the Archetype,
				// however that is fine since we will have been editing the CDO anyways
				if (!FBlueprintEditorUtils::IsAnonymousBlueprintClass(SubobjectOuter->GetClass()))
				{
					// if an archetype was specified in the Begin Object block, use that as the template for the ConstructObject call.
					FString ArchetypeName;
					if (FParse::Value(Str, TEXT("Archetype="), ArchetypeName))
					{
						// if given a name, break it up along the ' so separate the class from the name
						FString ObjectClass;
						FString ObjectPath;
						if ( FPackageName::ParseExportTextPath(ArchetypeName, &ObjectClass, &ObjectPath) )
						{
							UClass* ArchetypeClass = nullptr;

							// find the class
							if (FPackageName::IsValidObjectPath(ObjectClass))
							{
								ArchetypeClass = (UClass*)StaticFindObject(UClass::StaticClass(), nullptr, *ObjectClass);
							}
							else
							{
								// The text might be from before the full path of the class was exported (ensure if the name is ambiguous)
								ArchetypeClass = (UClass*)StaticFindFirstObject(UClass::StaticClass(), *ObjectClass, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous, ELogVerbosity::Warning);
							}

							if (ArchetypeClass)
							{
								ObjectPath = ObjectPath.TrimQuotes();
								// if we had the class, find the archetype
								if (!FPackageName::IsShortPackageName(ObjectPath))
								{
									Archetype = StaticFindObject(ArchetypeClass, nullptr, *ObjectPath);
								}
								else
								{
									Archetype = StaticFindFirstObject(ArchetypeClass, *ObjectPath, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
								}
							}
						}
					}
				}

				if (SubobjectOuter->HasAnyFlags(RF_ClassDefaultObject))
				{
					if (!Archetype) // if an archetype was specified explicitly, we will stick with that
					{
						Archetype = ComponentOwnerClass->GetDefaultSubobjectByName(TemplateName);
						if(Archetype)
						{
							if ( BaseTemplate == NULL )
							{
								// BaseTemplate should only be NULL if the Begin Object line specified a class
								Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: The component name %s is already used (if you want to override the component, don't specify a class): %s"), *TemplateName.ToString(), *StrLine);
								return NULL;
							}

							// the component currently in the component template map and the base template should be the same
							checkf(Archetype==BaseTemplate,TEXT("OverrideComponent: '%s'   BaseTemplate: '%s'"), *Archetype->GetFullName(), *BaseTemplate->GetFullName());
						}
					}
				}
				else // handle the non-template case (subobjects and non-template components)
				{
					ComponentTemplate = FindObject<UObject>(SubobjectOuter, *TemplateName.ToString());
					if (ComponentTemplate != NULL)
					{
						// if we're overriding a subobject declared in a parent class, we should already have an object with that name that
						// was instanced when ComponentOwnerClass's CDO was initialized; if so, it's archetype should be the BaseTemplate.  If it
						// isn't, then there are two unrelated subobject definitions using the same name.
						if ( ComponentTemplate->GetArchetype() != BaseTemplate )
						{
						}
						else if ( BaseTemplate == NULL )
						{
							// BaseTemplate should only be NULL if the Begin Object line specified a class
							Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: A subobject named %s is already declared in a parent class.  If you intended to override that subobject, don't specify a class in the derived subobject definition: %s"), *TemplateName.ToString(), *StrLine);
							return NULL;
						}
					}

				}

				// Propagate object flags to the sub object.
				EObjectFlags NewFlags = SubobjectOuter->GetMaskedFlags( RF_PropagateToSubObjects );

				if (!Archetype) // no override and we didn't find one from the class table, so go with the base
				{
					Archetype = BaseTemplate;
				}

				UObject* OldComponent = nullptr;
				if (ComponentTemplate)
				{
					bool bIsOkToReuse = ComponentTemplate->GetClass() == TemplateClass
						&& ComponentTemplate->GetOuter() == SubobjectOuter
						&& ComponentTemplate->GetFName() == TemplateName 
						&& (ComponentTemplate->GetArchetype() == Archetype || !Archetype);

					if (!bIsOkToReuse)
					{
						if (bUseOverrideSemantics
							&& FOverridableManager::Get().IsEnabled(ComponentTemplate)
							&& !ComponentTemplate->GetClass()->IsChildOf(TemplateClass))
						{
							// If we found an existing instanced subobject above that is not type-compatible,
							// create a new object to serve as the destination for importing instance data.
							// Otherwise, we allow instance data to be imported into the existing subobject.
							// 
							// This differs from the non-override path, which moves the instanced subobject
							// aside and constructs a new type-compatible subobject in its place (see below).
							// That path will remain in place for legacy support and backwards-compatibility.
							// 
							// When exporting override T3D, all reference property values will be "shadowed" on
							// import if they were not otherwise explicitly overridden at export time. Shadowed
							// reference values are skipped at import time, so it means we intentionally sidestep
							// the reference replacement pass below, and instead defer replacement to occur only
							// when we import object properties and find an explicitly-overridden reference value.
							// In that case the override will be resolved through the export path remapping table.
							// 
							// This effectively matches what happens on level load when a serialized export blocks
							// deferred subobject instancing due to e.g. a schema change that occurs while unloaded.
							// In that case archetype lookup will fail due to type incompatibility, and the exported
							// subobject is moved aside to allow a default instance of the updated type to be created.
							//
							// @todo - IDO support will be needed to capture the new instance that will result
							// in order to avoid data loss. If the new instance is not otherwise referenced by
							// an explicitly-overridden property at import time, then it will become unreachable.
							TemplateName = MakeUniqueObjectName(SubobjectOuter, TemplateClass, TemplateName);
						}
						else
						{
							UE_LOGF(LogEditorObject, Log, "Could not reuse component instance %ls, name clash?", *ComponentTemplate->GetFullName());
							ComponentTemplate->Rename(nullptr, nullptr, REN_DontCreateRedirectors); // just abandon the existing component, we are going to create
							OldComponent = ComponentTemplate;
						}

						// Forces a new instance to be created below
						ComponentTemplate = nullptr;
					}
				}

				FObjectInstancingGraph* InstanceGraphToUse = &InstanceGraph;
				if (!ComponentTemplate)
				{
					bool bUseDedicatedInstancingGraph = FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraphForNewObject(TemplateClass, NewFlags, Archetype, bOverrideT3DSupported && OverriddenOperation);
					
					FObjectInstancingGraph TempGraph;
					ComponentTemplate = NewObject<UObject>(
						SubobjectOuter,
						TemplateClass,
						TemplateName,
						NewFlags,
						Archetype,
						!!SubobjectOuter,
						bUseDedicatedInstancingGraph ? &TempGraph : &InstanceGraph
						);

					if (bUseDedicatedInstancingGraph)
					{
						if (!TempGraph.HasDestinationRoot())
						{
							TempGraph.SetDestinationRoot(ComponentTemplate);
						}

						InstanceGraphToUse = &DedicatedGraphs.Emplace(ComponentTemplate, MoveTemp(TempGraph));
					}
				}
				else
				{
					if (FObjectInstancingGraph* DedicatedGraph = DedicatedGraphs.Find(ComponentTemplate))
					{
						InstanceGraphToUse = DedicatedGraph;
					}
					else if (FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(ComponentTemplate, bOverrideT3DSupported && OverriddenOperation))
					{
						InstanceGraphToUse = &DedicatedGraphs.Emplace(ComponentTemplate, FObjectInstancingGraph(ComponentTemplate));
					}

					// We do not want to set RF_Transactional for construction script created components, so we have to monkey with things here
					if (NewFlags & RF_Transactional)
					{
						UActorComponent* Component = Cast<UActorComponent>(ComponentTemplate);
						if (Component && Component->IsCreatedByConstructionScript())
						{
							NewFlags &= ~RF_Transactional;
						}
					}
					
					// Ensure DefaultSubojbect flag persists through the clearing of flags
					if (ComponentTemplate->HasAllFlags(RF_DefaultSubObject))
					{
						NewFlags |= RF_DefaultSubObject;
					}

					// fix crash when comment below is true as it is now forbidden to clear RF_PendingKill using ClearFlags (see UObjectBaseUtility.h)
					ComponentTemplate->ClearGarbage();
					
					// Make sure desired flags are set - existing object could be pending kill
					ComponentTemplate->ClearGarbage();
					ComponentTemplate->ClearFlags(RF_AllFlags);
					ComponentTemplate->ClearInternalFlags(EInternalObjectFlags_AllFlags);
					ComponentTemplate->SetFlags(NewFlags);
				}

				// Make sure we have an up-to-date mapping in the object graph. This may not always be the case after defining an object.
				//
				// For example, if it was exported as an instance-added subobject with no non-default archetype, UE will bypass the subobject
				// instancing pass at construction time, and so the object graph will not include a mapping for it. This can cause the explicit
				// instancing pass that gets run after importing properties to assert, for example, if this object has its own default subobjects.
				//
				// Notes:
				//	- It is ok if the archetype is NULL. Including it avoids a redundant GetArchetype() call when it has already been set.
				//	- It is safe to assume that we're always inside a root object scope here, because we're importing a Begin Object block.
				InstanceGraphToUse->AddNewInstance(ComponentTemplate, Archetype);

				// replace all properties in this subobject outer' class that point to the original subobject with the new subobject
				TMap<UObject*, UObject*> ReplacementMap;
				if (Archetype)
				{
					checkSlow(ComponentTemplate->GetArchetype() == Archetype);
					ReplacementMap.Add(Archetype, ComponentTemplate);
				}
				if (OldComponent)
				{
					ReplacementMap.Add(OldComponent, ComponentTemplate);
				}
				FArchiveReplaceObjectRef<UObject> ReplaceAr(SubobjectOuter, ReplacementMap, EArchiveReplaceObjectFlags::IgnoreArchetypeRef);

				if (bExportPath && ObjectRemapper)
				{
					ObjectRemapper->Add(MoveTemp(ExportedObjectFullName), ComponentTemplate);
				}

				if (bOverrideT3DSupported)
				{
					if (OverriddenOperation.IsSet())
					{
						FOverridableManager::Get().RestoreOverrideOperation(ComponentTemplate, OverriddenOperation.GetValue(), /*bNeedsSubObjectTemplateInstantiation*/true);
					}
					else
					{
						FOverridableManager::Get().Disable(ComponentTemplate);
					}
				}

				// import the properties for the subobject
				SourceText = ImportObjectProperties(
					(uint8*)ComponentTemplate, 
					SourceText, 
					TemplateClass, 
					SubobjectRoot, 
					ComponentTemplate, 
					Warn, 
					Depth+1,
					ContextSupplier ? ContextSupplier->CurrentLine : 0,
					InstanceGraphToUse,
					ObjectRemapper
					);

				// If the object import fails, early out and stop parsing.
				if (SourceText == nullptr)
				{
					Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: Could not import properties from sub-object %s."), *SubobjectRoot->GetName());
					return nullptr;
				}
			}
		}
		else if( FParse::Command(&Str,TEXT("CustomProperties")))
		{
			check(SubobjectOuter);

			SubobjectOuter->ImportCustomProperties(Str, Warn);
		}
		else if (IsEndOfProperties(Str, Depth))
		{
			// End of properties.
			break;
		}
		else if( GetREMOVE(&Str,TEXT("Component")) )
		{
			checkf(false, TEXT("Remove component is illegal in pasted text"));
		}
		else
		{
			// Property.
			FProperty::ImportSingleProperty(Str, DestData, ObjectStruct, SubobjectOuter, PortFlags, Warn, DefinedProperties);
		}
	}

	if (ObjectRemapper)
	{
		for (const auto& DefinedProperty : DefinedProperties)
		{
			RemapProperty(DefinedProperty.Property, DefinedProperty.Index, *ObjectRemapper, DestData);
		}
	}

	// Prepare brush.
	if( ImportedBrush && ObjectStruct->IsChildOf<ABrush>() && !ObjectStruct->IsChildOf<AVolume>() )
	{
		check(GIsEditor);
		ABrush* Actor = (ABrush*)DestData;
		check(Actor->GetBrushComponent());
		if( Actor->GetBrushComponent()->Mobility == EComponentMobility::Static )
		{
			// Prepare static brush.
			Actor->SetNotForClientOrServer();
		}
		else
		{
			// Prepare moving brush.
			FBSPOps::csgPrepMovingBrush( Actor );
		}
	}

	return SourceText;
}



/**
 * Parse and import text as property values for the object specified.
 * 
 * @param	InParams	Parameters for object import; see declaration of FImportObjectParams.
 *
 * @return	NULL if the default values couldn't be imported
 */

const TCHAR* ImportObjectProperties( FImportObjectParams& InParams )
{
	FDefaultPropertiesContextSupplier Supplier;
	if ( InParams.LineNumber != INDEX_NONE )
	{
		if ( InParams.SubobjectRoot == NULL )
		{
			Supplier.PackageName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetOutermost()->GetName() : InParams.ObjectStruct->GetOutermost()->GetName();
			Supplier.ClassName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetName() : FName(NAME_None).ToString();
			Supplier.CurrentLine = InParams.LineNumber; 

			ContextSupplier = &Supplier; //-V506
		}
		else
		{
			if ( ContextSupplier != NULL )
			{
				ContextSupplier->CurrentLine = InParams.LineNumber;
			}
		}
		InParams.Warn->SetContext(ContextSupplier);
	}

	if ( InParams.bShouldCallEditChange && InParams.SubobjectOuter != NULL )
	{
		InParams.SubobjectOuter->PreEditChange(NULL);
	}

	FObjectInstancingGraph TempGraph;
	FObjectInstancingGraph& InstanceGraph = InParams.InInstanceGraph ? *InParams.InInstanceGraph : TempGraph;
	if ( InParams.SubobjectRoot && InParams.SubobjectRoot != UObject::StaticClass()->GetDefaultObject() )
	{
		InstanceGraph.SetDestinationRoot(InParams.SubobjectRoot);
	}

	// Use TOptional to avoid heap allocations and only pay for construction/destruction if Emplace is called.
	TOptional<TMap<FSoftObjectPath, UObject*>> ExportedPathToObjectCreated;
	if (!InParams.ObjectRemapper && FOverridableManager::Get().IsEnabled(InParams.SubobjectOuter))
	{
		// When using override T3D, always map exported subobject paths so they can be resolved at redefinition time
		InParams.ObjectRemapper = &ExportedPathToObjectCreated.Emplace();
	}

	// Parse the object properties.
	const TCHAR* NewSourceText =
		ImportProperties(
			InParams.DestData,
			InParams.SourceText,
			InParams.ObjectStruct,
			InParams.SubobjectRoot,
			InParams.SubobjectOuter,
			InParams.Warn,
			InParams.Depth,
			InstanceGraph,
			InParams.ObjectRemapper
			);

	if ( InParams.SubobjectOuter != NULL )
	{
		check(InParams.SubobjectRoot);

		// Update the object properties to point to the newly imported component objects.
		// Templates inside classes never need to have components instanced.
 		if ( !InParams.SubobjectRoot->HasAnyFlags(RF_ClassDefaultObject) )
		{
			UObject* SubobjectArchetype = InParams.SubobjectOuter->GetArchetype();
			InParams.ObjectStruct->InstanceSubobjectTemplates(InParams.DestData, SubobjectArchetype, SubobjectArchetype->GetClass(),
				InParams.SubobjectOuter, &InstanceGraph);
		}

		if ( InParams.bShouldCallEditChange )
		{
			// notify the object that it has just been imported
			InParams.SubobjectOuter->PostEditImport();

			// notify the object that it has been edited
			InParams.SubobjectOuter->PostEditChange();
		}
		InParams.SubobjectRoot->CheckDefaultSubobjects();
	}

	if ( InParams.LineNumber != INDEX_NONE )
	{
		if ( ContextSupplier == &Supplier )
		{
			ContextSupplier = NULL;
			InParams.Warn->SetContext(NULL);
		}
	}

	if (ExportedPathToObjectCreated.IsSet())
	{
		// Remove dangling reference to ExportedPathToObjectCreated on the stack
		InParams.ObjectRemapper = nullptr;
	}

	// Suppress clang analyzer false positive about InParams.ObjectRemapper retaining reference to ExportedPathToObjectCreated
	UE_CLANG_SUPPRESS
	return NewSourceText;
}

	
/**
 * Parse and import text as property values for the object specified.
 * 
 * @param	DestData			the location to import the property values to
 * @param	SourceText			pointer to a buffer containing the values that should be parsed and imported
 * @param	ObjectStruct		the struct for the data we're importing
 * @param	SubobjectRoot		the original object that ImportObjectProperties was called for.
 *								if SubobjectOuter is a subobject, corresponds to the first object in SubobjectOuter's Outer chain that is not a subobject itself.
 *								if SubobjectOuter is not a subobject, should normally be the same value as SubobjectOuter
 * @param	SubobjectOuter		the object corresponding to DestData; this is the object that will used as the outer when creating subobjects from definitions contained in SourceText
 * @param	Warn				ouptut device to use for log messages
 * @param	Depth				current nesting level
 * @param	LineNumber			used when importing defaults during script compilation for tracking which line we're currently for the purposes of printing compile errors
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates; used when recursively calling ImportObjectProperties; generally
 *								not necessary to specify a value when calling this function from other code
 *
 * @return	NULL if the default values couldn't be imported
 */

const TCHAR* ImportObjectProperties(
	uint8*				DestData,
	const TCHAR*		SourceText,
	UStruct*			ObjectStruct,
	UObject*			SubobjectRoot,
	UObject*			SubobjectOuter,
	FFeedbackContext*	Warn,
	int32					Depth,
	int32					LineNumber,
	FObjectInstancingGraph* InInstanceGraph,
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper
	)
{
	FImportObjectParams Params;
	{
		Params.DestData = DestData;
		Params.SourceText = SourceText;
		Params.ObjectStruct = ObjectStruct;
		Params.SubobjectRoot = SubobjectRoot;
		Params.SubobjectOuter = SubobjectOuter;
		Params.Warn = Warn;
		Params.Depth = Depth;
		Params.LineNumber = LineNumber;
		Params.InInstanceGraph = InInstanceGraph;
		Params.ObjectRemapper = ObjectRemapper;

		// This implementation always calls PreEditChange/PostEditChange
		Params.bShouldCallEditChange = true;
	}

	return ImportObjectProperties( Params );
}


namespace EditorUtilities
{
// Multi step object import

/**
 * Parse and import text as subobjects for the object specified but doesn't serialize the properties.
 * 
 * @param	ObjectStruct				The struct for the data we're importing
 * @param	SourceText					View on buffer containing the values that should be parsed and imported
 * @param	SubobjectRoot				When dealing with nested subobjects, corresponds to the top-most outer that
 *										is not a subobject/template
 * @param	SubobjectOuter				The outer to use for creating subobjects/components. NULL when importing structdefaultproperties
 * @param	Warn						Output device to use for log messages
 * @param	Depth						Current nesting level
 * @param	InstanceGraph				Contains the mappings of instanced objects and components to their templates
 * @param	ObjectRemapper				a map of existing Object to new instances, used to replace internal references when a number of actors are copy+pasted
 *
 * @return	NULL if the default values couldn't be imported
 */
static const TCHAR* ImportCreateSubObjectsStep(
	FStringView SourceText,
	UStruct* ObjectStruct,
	UObject* SubobjectRoot,
	UObject* SubobjectOuter,
	FFeedbackContext* Warn,
	int32 Depth,
	FObjectInstancingGraph& InstanceGraph,
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper
	)
{
	check(ObjectStruct!=nullptr);

	if (SourceText.IsEmpty())
	{
		return nullptr;
	}

	// Cannot create subobjects when importing struct defaults, or if SubobjectOuter (used as the Outer for any subobject declarations encountered) is NULL
	bool bSubObjectsAllowed = !ObjectStruct->IsA(UScriptStruct::StaticClass()) && SubobjectOuter != nullptr;

	// true when DestData corresponds to a subobject in a class default object
	bool bSubObject = false;

	UClass* ComponentOwnerClass = nullptr;

	if ( bSubObjectsAllowed )
	{
		bSubObject = SubobjectRoot != nullptr && SubobjectRoot->HasAnyFlags(RF_ClassDefaultObject);
		if ( SubobjectRoot == nullptr )
		{
			SubobjectRoot = SubobjectOuter;
		}

		ComponentOwnerClass = SubobjectOuter != nullptr
			? SubobjectOuter->IsA(UClass::StaticClass())
				? CastChecked<UClass>(SubobjectOuter)
				: SubobjectOuter->GetClass()
			: nullptr;
	}

	const TCHAR* CurrentSourceText = SourceText.GetData();
	const TCHAR* EndOfSourceText = SourceText.GetData() + SourceText.Len();

	FString StrLine;

	// Local graphs to use for subobjects that require dedicated instancing graphs.
	TMap<UObject*, FObjectInstancingGraph> DedicatedGraphs;

	// Parse all objects stored in the actor.
	// Build list of all text properties.
	bool ImportedBrush = 0;
	int32 LinesConsumed = 0;
	while (CurrentSourceText < EndOfSourceText)
	{
		if (!FParse::LineExtended(&CurrentSourceText, StrLine, LinesConsumed, FParse::ELineExtendedFlags::OldExactMode))
		{
			break;
		}

		// remove extra whitespace and optional semicolon from the end of the line
		{
			int32 Length = StrLine.Len();
			while ( Length > 0 &&
					(StrLine[Length - 1] == TCHAR(';') || StrLine[Length - 1] == TCHAR(' ') || StrLine[Length - 1] == TCHAR('\t')) )
			{
				Length--;
			}
			if (Length != StrLine.Len())
			{
				StrLine.LeftInline(Length, EAllowShrinking::No);
			}
		}

		if (ContextSupplier)
		{
			ContextSupplier->CurrentLine += LinesConsumed;
		}
		if (StrLine.Len() == 0)
		{
			continue;
		}

		const TCHAR* Str = *StrLine;

		if (HandleLineNumber(Str))
		{
			continue;
		}
		else if(GetBEGIN(&Str,TEXT("Brush")))
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN Brush: Brushes are not supported in the MultiStepsImport use ImportObjectProperties to import these"));
			return nullptr;
		}
		else if (GetBEGIN(&Str, TEXT("Foliage")))
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN Foliage: Foliage is not supported in the MultiStepsImport use ImportObjectProperties to import these"));
			return nullptr;
		}
		else if(GetBEGIN(&Str,TEXT("Object")))
		{
			// If SubobjectOuter is NULL, we are importing defaults for a UScriptStruct's defaultproperties block
			if ( !bSubObjectsAllowed )
			{
				Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: Subobjects are not allowed in this context"));
				return nullptr;
			}

			// Parse subobject default properties.
			// Note: default properties subobjects have compiled class as their Outer (used for localization).
			UClass*	TemplateClass = nullptr;
			bool bInvalidClass = false;
			ParseObject<UClass>(Str, TEXT("Class="), TemplateClass, nullptr, &bInvalidClass);
			
			if (bInvalidClass)
			{
				Warn->Logf(ELogVerbosity::Error,TEXT("BEGIN OBJECT: Invalid class specified: %s"), *StrLine);
				return nullptr;
			}

			// parse the name of the template
			FName	TemplateName = NAME_None;
			FParse::Value(Str,TEXT("Name="),TemplateName);
			if(TemplateName == NAME_None)
			{
				Warn->Logf(ELogVerbosity::Error,TEXT("BEGIN OBJECT: Must specify valid name for subobject/component: %s"), *StrLine);
				return nullptr;
			}

			FString ExportedObjectFullName;
			FParse::Value(Str, TEXT("ExportPath="), ExportedObjectFullName);

			FName OverriddenOperationName = NAME_None;
			const bool bParsed = FParse::Value(Str,TEXT("OverriddenOperation="), OverriddenOperationName);
			const bool bOverrideT3DSupported = FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization);
			TOptional<EOverriddenPropertyOperation> OverriddenOperation = bParsed && bOverrideT3DSupported ? GetOverriddenOperationFromName(OverriddenOperationName) : TOptional<EOverriddenPropertyOperation>();

			// points to the parent class's template subobject/component, if we are overriding a subobject/component declared in our parent class
			UObject* BaseTemplate = nullptr;
			bool bRedefiningSubobject = false;
			if(!TemplateClass)
			{
				// next, verify that a template actually exists in the parent class
				UClass* ParentClass = ComponentOwnerClass->GetSuperClass();
				check(ParentClass);

				UObject* ParentCDO = ParentClass->GetDefaultObject();
				check(ParentCDO);

				BaseTemplate = StaticFindObjectFast(UObject::StaticClass(), SubobjectOuter, TemplateName);
				bRedefiningSubobject = (BaseTemplate != nullptr);

				if (BaseTemplate == nullptr)
				{
					BaseTemplate = StaticFindObjectFast(UObject::StaticClass(), ParentCDO, TemplateName);
				}
				
				if ( BaseTemplate == nullptr )
				{
					// wasn't found
					Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: No base template named %s found in parent class %s: %s"), *TemplateName.ToString(), *ParentClass->GetName(), *StrLine);
					return nullptr;
				}

				TemplateClass = BaseTemplate->GetClass();
			}

			// because the outer won't be a default object

			checkSlow(TemplateClass != nullptr);
			if (bRedefiningSubobject)
			{
				FObjectInstancingGraph* InstanceGraphToUse = DedicatedGraphs.Find(BaseTemplate);
				if (!InstanceGraphToUse)
				{
					if (FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(BaseTemplate))
					{
						InstanceGraphToUse = &DedicatedGraphs.Emplace(BaseTemplate, FObjectInstancingGraph(BaseTemplate));
					}
					else
					{
						InstanceGraphToUse = &InstanceGraph;
					}
				}
				
				// Simply recreate any missing sub object
				FMultiStepsImportObjectParams Params;
				Params.SourceText = FStringView(CurrentSourceText, SourceText.Len() - (CurrentSourceText - SourceText.GetData()));
				Params.ObjectStruct = TemplateClass;
				Params.SubobjectRoot = SubobjectRoot;
				Params.SubobjectOuter = BaseTemplate;
				Params.Warn = Warn;
				Params.Depth = Depth + 1;
				Params.LineNumber = ContextSupplier ? ContextSupplier->CurrentLine : 0;
				Params.InInstanceGraph = InstanceGraphToUse;
				Params.ObjectRemapper = ObjectRemapper;

				CurrentSourceText = ImportCreateObjectsStep(Params);
				
				if (!ExportedObjectFullName.IsEmpty() && ObjectRemapper)
				{
					ObjectRemapper->Add(MoveTemp(ExportedObjectFullName), BaseTemplate);
				}
			}
			else 
			{
				UObject* Archetype = nullptr;
				UObject* ComponentTemplate = nullptr;

				// Since we are changing the class we can't use the Archetype,
				// however that is fine since we will have been editing the CDO anyways
				if (!FBlueprintEditorUtils::IsAnonymousBlueprintClass(SubobjectOuter->GetClass()))
				{
					// if an archetype was specified in the Begin Object block, use that as the template for the ConstructObject call.
					FString ArchetypeName;
					if (FParse::Value(Str, TEXT("Archetype="), ArchetypeName))
					{
						// if given a name, break it up along the ' so separate the class from the name
						FString ObjectClass;
						FString ArchetypePath;
						if ( FPackageName::ParseExportTextPath(ArchetypeName, &ObjectClass, &ArchetypePath) )
						{
							// find the class
							check(FPackageName::IsValidObjectPath(ObjectClass));
							UClass* ArchetypeClass = (UClass*)StaticFindObject(UClass::StaticClass(), nullptr, *ObjectClass);
							if (ArchetypeClass)
							{
								ArchetypePath = ArchetypePath.TrimQuotes();
								// if we had the class, find the archetype
								if (!FPackageName::IsShortPackageName(ArchetypePath))
								{
									Archetype = StaticFindObject(ArchetypeClass, nullptr, *ArchetypePath);
								}
								else
								{
									Archetype = StaticFindFirstObject(ArchetypeClass, *ArchetypePath, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
								}
							}
						}
					}
				}

				if (SubobjectOuter->HasAnyFlags(RF_ClassDefaultObject))
				{
					if (!Archetype) // if an archetype was specified explicitly, we will stick with that
					{
						Archetype = ComponentOwnerClass->GetDefaultSubobjectByName(TemplateName);
						if(Archetype)
						{
							if (!BaseTemplate)
							{
								// BaseTemplate should only be NULL if the Begin Object line specified a class
								Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: The component name %s is already used (if you want to override the component, don't specify a class): %s"), *TemplateName.ToString(), *StrLine);
								return nullptr;
							}

							// the component currently in the component template map and the base template should be the same
							checkf(Archetype==BaseTemplate,TEXT("OverrideComponent: '%s'   BaseTemplate: '%s'"), *Archetype->GetFullName(), *BaseTemplate->GetFullName());
						}
					}
				}
				else // handle the non-template case (subobjects and non-template components)
				{
					ComponentTemplate = FindObject<UObject>(SubobjectOuter, *TemplateName.ToString());
					if (ComponentTemplate != nullptr)
					{
						// if we're overriding a subobject declared in a parent class, we should already have an object with that name that
						// was instanced when ComponentOwnerClass's CDO was initialized; if so, it's archetype should be the BaseTemplate.  If it
						// isn't, then there are two unrelated subobject definitions using the same name.
						if (ComponentTemplate->GetArchetype() != BaseTemplate)
						{
						}
						else if (BaseTemplate == nullptr)
						{
							// BaseTemplate should only be NULL if the Begin Object line specified a class
							Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: A subobject named %s is already declared in a parent class.  If you intended to override that subobject, don't specify a class in the derived subobject definition: %s"), *TemplateName.ToString(), *StrLine);
							return nullptr;
						}
					}
				}

				// Propagate object flags to the sub object.
				EObjectFlags NewFlags = SubobjectOuter->GetMaskedFlags( RF_PropagateToSubObjects );

				if (!Archetype) // no override and we didn't find one from the class table, so go with the base
				{
					Archetype = BaseTemplate;
				}

				UObject* OldComponent = nullptr;
				if (ComponentTemplate)
				{
					bool bIsOkToReuse = ComponentTemplate->GetClass() == TemplateClass
						&& ComponentTemplate->GetOuter() == SubobjectOuter
						&& ComponentTemplate->GetFName() == TemplateName 
						&& (ComponentTemplate->GetArchetype() == Archetype || !Archetype);

					if (!bIsOkToReuse)
					{
						UE_LOGF(LogEditorObject, Log, "Could not reuse component instance %ls, name clash?", *ComponentTemplate->GetFullName());
						ComponentTemplate->Rename(nullptr, nullptr, REN_DontCreateRedirectors); // just abandon the existing component, we are going to create
						OldComponent = ComponentTemplate;
						ComponentTemplate = nullptr;
					}
				}
				
				FObjectInstancingGraph* InstanceGraphToUse = &InstanceGraph;
				if (!ComponentTemplate)
				{
					bool bUseDedicatedInstancingGraph = FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraphForNewObject(TemplateClass, NewFlags, Archetype, bOverrideT3DSupported && OverriddenOperation);
					
					FObjectInstancingGraph TempGraph;
					ComponentTemplate = NewObject<UObject>(
						SubobjectOuter,
						TemplateClass,
						TemplateName,
						NewFlags,
						Archetype,
						!!SubobjectOuter,
						bUseDedicatedInstancingGraph ? &TempGraph : &InstanceGraph
						);

					if (bUseDedicatedInstancingGraph)
					{
						if (!TempGraph.HasDestinationRoot())
						{
							TempGraph.SetDestinationRoot(ComponentTemplate);
						}

						InstanceGraphToUse = &DedicatedGraphs.Emplace(ComponentTemplate, MoveTemp(TempGraph));
					}

					if (!ExportedObjectFullName.IsEmpty() && ObjectRemapper)
					{
						ObjectRemapper->Add(MoveTemp(ExportedObjectFullName), ComponentTemplate);
					}
				}
				else
				{
					if (FObjectInstancingGraph* DedicatedGraph = DedicatedGraphs.Find(ComponentTemplate))
					{
						InstanceGraphToUse = DedicatedGraph;
					}
					else if (FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(ComponentTemplate, bOverrideT3DSupported && OverriddenOperation))
					{
						InstanceGraphToUse = &DedicatedGraphs.Emplace(ComponentTemplate, FObjectInstancingGraph(ComponentTemplate));
					}

					// We do not want to set RF_Transactional for construction script created components, so we have to monkey with things here
					if (NewFlags & RF_Transactional)
					{
						UActorComponent* Component = Cast<UActorComponent>(ComponentTemplate);
						if (Component && Component->IsCreatedByConstructionScript())
						{
							NewFlags &= ~RF_Transactional;
						}
					}
					
					// Ensure DefaultSubojbect flag persists through the clearing of flags
					if (ComponentTemplate->HasAllFlags(RF_DefaultSubObject))
					{
						NewFlags |= RF_DefaultSubObject;
					}
					
					// Make sure desired flags are set - existing object could be pending kill
					ComponentTemplate->ClearGarbage();
					ComponentTemplate->ClearFlags(RF_AllFlags);
					ComponentTemplate->ClearInternalFlags(EInternalObjectFlags_AllFlags);
					ComponentTemplate->SetFlags(NewFlags);
				}

				// Make sure we have an up-to-date mapping in the object graph. This may not always be the case after defining an object.
				//
				// For example, if it was exported as an instance-added subobject with no non-default archetype, UE will bypass the subobject
				// instancing pass at construction time, and so the object graph will not include a mapping for it. This can cause the explicit
				// instancing pass that gets run after importing properties to assert, for example, if this object has its own default subobjects.
				//
				// Notes:
				//	- It is ok if the archetype is NULL. Including it avoids a redundant GetArchetype() call when it has already been set.
				//	- It is safe to assume that we're always inside a root object scope here, because we're importing a Begin Object block.
				InstanceGraphToUse->AddNewInstance(ComponentTemplate, Archetype);

				// replace all properties in this subobject outer' class that point to the original subobject with the new subobject
				TMap<UObject*, UObject*> ReplacementMap;
				if (Archetype)
				{
					checkSlow(ComponentTemplate->GetArchetype() == Archetype);
					ReplacementMap.Add(Archetype, ComponentTemplate);
				}
				if (OldComponent)
				{
					ReplacementMap.Add(OldComponent, ComponentTemplate);
				}
				FArchiveReplaceObjectRef<UObject> ReplaceAr(SubobjectOuter, ReplacementMap, EArchiveReplaceObjectFlags::IgnoreArchetypeRef);

				// Simply recreate any missing sub object
				FMultiStepsImportObjectParams Params;
				Params.SourceText = FStringView(CurrentSourceText, SourceText.Len() - (CurrentSourceText - SourceText.GetData()));
				Params.ObjectStruct = TemplateClass;
				Params.SubobjectRoot = SubobjectRoot;
				Params.SubobjectOuter = ComponentTemplate;
				Params.Warn = Warn;
				Params.Depth = Depth + 1;
				Params.LineNumber = ContextSupplier ? ContextSupplier->CurrentLine : 0;
				Params.InInstanceGraph = InstanceGraphToUse;
				Params.ObjectRemapper = ObjectRemapper;

				if (!ExportedObjectFullName.IsEmpty() && ObjectRemapper)
				{
					ObjectRemapper->Add(MoveTemp(ExportedObjectFullName), ComponentTemplate);
				}

				if (bOverrideT3DSupported)
				{
					if (OverriddenOperation.IsSet())
					{
						FOverridableManager::Get().RestoreOverrideOperation(ComponentTemplate, OverriddenOperation.GetValue(), /*bNeedsSubObjectTemplateInstantiation*/true);
					}
					else
					{
						FOverridableManager::Get().Disable(ComponentTemplate);
					}
				}

				// Create the subobjects for the subobject
				CurrentSourceText = ImportCreateObjectsStep(Params);
			}
		}
		else if(IsEndOfProperties(Str, Depth))
		{
			// End of properties.
			break;
		}
	}

	return CurrentSourceText;
}

/**
 * Parse and import text as property values for the object specified.  This function should never be called directly - use ImportObjectProperties instead.
 * 
 * @param	ObjectStruct				The struct for the data we're importing
 * @param	DestData					The location to import the property values to
 * @param	SourceText					Text containing the values that should be parsed and imported
 * @param	SubobjectRoot				When dealing with nested subobjects, corresponds to the top-most outer that
 *										is not a subobject/template
 * @param	SubobjectOuter				The outer to use for creating subobjects/components. NULL when importing structdefaultproperties
 * @param	Warn						Output device to use for log messages
 * @param	Depth						Current nesting level
 * @param	InstanceGraph				Contains the mappings of instanced objects and components to their templates
 * @param	ObjectRemapper				A map of exported names to new instances, used to replace internal references when a number of object are copy+pasted
 * @param	PropertiesToSkip				A set containing the properties that shouldn't be imported
 * 
 * @return	NULL if the default values couldn't be imported
 */
static const TCHAR* ImportPropertiesStep(
	uint8* DestData,
	FStringView SourceText,
	UStruct* ObjectStruct,
	UObject* SubobjectRoot,
	UObject* SubobjectOuter,
	FFeedbackContext* Warn,
	int32 Depth,
	FObjectInstancingGraph& InstanceGraph,
	TMap<FSoftObjectPath, UObject*>* ObjectRemapper,
	TSet<FProperty*>* PropertiesToSkip
	)
{
	check(ObjectStruct!=nullptr);
	check(DestData!=nullptr);

	if (SourceText == nullptr)
	{
		return nullptr;
	}

	FOverriddenPropertySet* OverriddenProperties = nullptr;
	if (SubobjectOuter && FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization))
	{
		OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(SubobjectOuter);
	}
	FEnableOverridableSerializationScope Scope(OverriddenProperties != nullptr, OverriddenProperties);

	// Cannot create subobjects when importing struct defaults, or if SubobjectOuter (used as the Outer for any subobject declarations encountered) is NULL
	bool bSubObjectsAllowed = !ObjectStruct->IsA(UScriptStruct::StaticClass()) && SubobjectOuter != NULL;

	// true when DestData corresponds to a subobject in a class default object
	bool bSubObject = false;

	UClass* ComponentOwnerClass = nullptr;

	if ( bSubObjectsAllowed )
	{
		bSubObject = SubobjectRoot != nullptr && SubobjectRoot->HasAnyFlags(RF_ClassDefaultObject);
		if ( SubobjectRoot == nullptr )
		{
			SubobjectRoot = SubobjectOuter;
		}

		ComponentOwnerClass = SubobjectOuter != nullptr
			? SubobjectOuter->IsA(UClass::StaticClass())
				? CastChecked<UClass>(SubobjectOuter)
				: SubobjectOuter->GetClass()
			: nullptr;
	}

	// The PortFlags to use for all ImportText calls
	uint32 PortFlags = PPF_Delimited | PPF_CheckReferences;
	if (GIsImportingT3D)
	{
		PortFlags |= PPF_AttemptNonQualifiedSearch;
	}

	const TCHAR* CurrentSourceText = SourceText.GetData();
	const TCHAR* EndOfSourceText = SourceText.GetData() + SourceText.Len();

	FString StrLine;

	TArray<FDefinedProperty> DefinedProperties;

	// Local graphs to use for subobjects that require dedicated instancing graphs.
	TMap<UObject*, FObjectInstancingGraph> DedicatedGraphs;

	// Parse all objects stored in the actor.
	// Build list of all text properties.
	bool ImportedBrush = 0;
	int32 LinesConsumed = 0;
	while (CurrentSourceText < EndOfSourceText)
	{
		if (FParse::LineExtended(&CurrentSourceText, StrLine, LinesConsumed, FParse::ELineExtendedFlags::OldExactMode))
		{
			// remove extra whitespace and optional semicolon from the end of the line
			{
				int32 Length = StrLine.Len();
				while ( Length > 0 &&
						(StrLine[Length - 1] == TCHAR(';') || StrLine[Length - 1] == TCHAR(' ') || StrLine[Length - 1] == TCHAR('\t')) )
				{
					Length--;
				}
				if (Length != StrLine.Len())
				{
					StrLine.LeftInline(Length, EAllowShrinking::No);
				}
			}

			if ( ContextSupplier != nullptr )
			{
				ContextSupplier->CurrentLine += LinesConsumed;
			}
			if (StrLine.Len() == 0)
			{
				continue;
			}

			const TCHAR* Str = *StrLine;

			if (HandleLineNumber(Str))
			{
				continue;
			}
			else if( GetBEGIN(&Str,TEXT("Object")))
			{
				// If SubobjectOuter is NULL, we are importing defaults for a UScriptStruct's defaultproperties block
				if ( !bSubObjectsAllowed )
				{
					Warn->Logf(ELogVerbosity::Error, TEXT("BEGIN OBJECT: Subobjects are not allowed in this context"));
					return NULL;
				}

				// parse the name of the template
				FName	TemplateName = NAME_None;
				FParse::Value(Str,TEXT("Name="),TemplateName);
				if(TemplateName == NAME_None)
				{
					Warn->Logf(ELogVerbosity::Error,TEXT("BEGIN OBJECT: Must specify valid name for subobject/component: %s"), *StrLine);
					return nullptr;
				}


				if (UObject* BaseTemplate = StaticFindObjectFast(UObject::StaticClass(), SubobjectOuter, TemplateName))
				{
					FObjectInstancingGraph* InstanceGraphToUse = DedicatedGraphs.Find(BaseTemplate);
					if (!InstanceGraphToUse)
					{
						if (FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(BaseTemplate))
						{
							InstanceGraphToUse = &DedicatedGraphs.Emplace(BaseTemplate, FObjectInstancingGraph(BaseTemplate));
						}
						else
						{
							InstanceGraphToUse = &InstanceGraph;
						}
					}

					FMultiStepsImportObjectParams Params;
					Params.DestData = static_cast<uint8*>(static_cast<void*>(BaseTemplate));
					Params.SourceText = FStringView(CurrentSourceText, SourceText.Len() - (CurrentSourceText - SourceText.GetData()));
					Params.ObjectStruct = BaseTemplate->GetClass();
					Params.SubobjectRoot = SubobjectRoot;
					Params.SubobjectOuter = BaseTemplate;
					Params.Warn = Warn;
					Params.Depth = Depth + 1;
					Params.LineNumber = ContextSupplier ? ContextSupplier->CurrentLine : 0;
					Params.InInstanceGraph = InstanceGraphToUse;
					Params.ObjectRemapper = ObjectRemapper;
					Params.PropertiesToSkip = PropertiesToSkip;

					CurrentSourceText = ImportObjectsPropertiesStep(Params);
				}
			}
			else if( FParse::Command(&Str,TEXT("CustomProperties")))
			{
				check(SubobjectOuter);

				SubobjectOuter->ImportCustomProperties(Str, Warn);
			}
			else if (IsEndOfProperties(Str, Depth))
			{
				// End of properties.
				break;
			}
			else
			{
				// Property.
				const TCHAR* TextToImport = Str;
				
				FString RedirectedProperty;
				if (ObjectRemapper || PropertiesToSkip)
				{
					constexpr FAsciiSet Delimiters("=([.");

					FStringView FullPropertyText(StrLine);
					FullPropertyText = FullPropertyText.TrimStartAndEnd();

					// find first delimiter
					if (const TCHAR* EndOfPropertyName = FAsciiSet::FindFirstOrEnd(FullPropertyText.GetData(), Delimiters))
					{
						if (PropertiesToSkip)
						{
							FStringView PropertyNameView(FullPropertyText.GetData(), EndOfPropertyName - FullPropertyText.GetData());

							if (!PropertyNameView.IsEmpty())
							{
								PropertyNameView.TrimEndInline();

								const FName PropertyName(PropertyNameView);
								FProperty* Property = FindFProperty<FProperty>(ObjectStruct, PropertyName);

								if (Property == nullptr)
								{
									// Check for redirects
									FName NewPropertyName = FProperty::FindRedirectedPropertyName(ObjectStruct, PropertyName);

									if (NewPropertyName != NAME_None)
									{
										Property = FindFProperty<FProperty>(ObjectStruct, NewPropertyName);
									}

									if (!Property)
									{
										Property = ObjectStruct->CustomFindProperty(PropertyName);
									}
								}

								if (PropertiesToSkip->Contains(Property))
								{
									// Do not import this property
									continue;
								}
							}
						}

						if (ObjectRemapper)
						{ 
							// Get the value
							constexpr FAsciiSet Equal("=");
							const TCHAR* PtrToEqual = FAsciiSet::FindFirstOrEnd(EndOfPropertyName, Equal);

							if (PtrToEqual && PtrToEqual + 1 < FullPropertyText.GetData() + FullPropertyText.Len())
							{ 
								// Select the next character
								PtrToEqual++;
								FStringView PropertyValue(PtrToEqual, FullPropertyText.Len() - (PtrToEqual - FullPropertyText.GetData()));
								if (UObject* const* PointerToObject = ObjectRemapper->Find(FSoftObjectPath(PropertyValue)))
								{
									FString RedirectedFullName = FObjectPropertyBase::GetExportPath(*PointerToObject, nullptr, nullptr, PortFlags | PPF_Delimited);

									FStringView TextBeforeValue(FullPropertyText.GetData(), PtrToEqual - FullPropertyText.GetData());
									
									RedirectedProperty.Reserve(RedirectedFullName.Len() + TextBeforeValue.Len());
									RedirectedProperty.Append(TextBeforeValue);
									RedirectedProperty.Append(RedirectedFullName);
									TextToImport = *RedirectedProperty;
								}
							}
						}
					}
				}

				FProperty::ImportSingleProperty(TextToImport, DestData, ObjectStruct, SubobjectOuter, PortFlags, Warn, DefinedProperties);
			}
		}
	}

	return CurrentSourceText;
}

const TCHAR* ImportCreateObjectsStep(FMultiStepsImportObjectParams& InParams)
{
	FDefaultPropertiesContextSupplier Supplier;
	if (InParams.LineNumber != INDEX_NONE)
	{
		if (InParams.SubobjectRoot == nullptr)
		{
			Supplier.PackageName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetOutermost()->GetName() : InParams.ObjectStruct->GetOutermost()->GetName();
			Supplier.ClassName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetName() : FName(NAME_None).ToString();
			Supplier.CurrentLine = InParams.LineNumber;

			ContextSupplier = &Supplier; //-V506
		}
		else
		{
			if (ContextSupplier != nullptr)
			{
				ContextSupplier->CurrentLine = InParams.LineNumber;
			}
		}
		InParams.Warn->SetContext(ContextSupplier);
	}

	FObjectInstancingGraph TempGraph;
	FObjectInstancingGraph& InstanceGraph = InParams.InInstanceGraph ? *InParams.InInstanceGraph : TempGraph;

	if (InParams.SubobjectRoot && InParams.SubobjectRoot != UObject::StaticClass()->GetDefaultObject())
	{
		InstanceGraph.SetDestinationRoot(InParams.SubobjectRoot);
	}

	// Parse the object properties.
	const TCHAR* NewSourceText =
		ImportCreateSubObjectsStep(
			InParams.SourceText,
			InParams.ObjectStruct,
			InParams.SubobjectRoot,
			InParams.SubobjectOuter,
			InParams.Warn,
			InParams.Depth,
			InstanceGraph,
			InParams.ObjectRemapper
		);

	if (InParams.LineNumber != INDEX_NONE)
	{
		if (ContextSupplier == &Supplier)
		{
			ContextSupplier = nullptr;
			InParams.Warn->SetContext(nullptr);
		}
	}

	return NewSourceText;
}


const TCHAR* ImportObjectsPropertiesStep(FMultiStepsImportObjectParams& InParams)
{
	FDefaultPropertiesContextSupplier Supplier;
	if (InParams.LineNumber != INDEX_NONE)
	{
		if (InParams.SubobjectRoot == nullptr)
		{
			Supplier.PackageName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetOutermost()->GetName() : InParams.ObjectStruct->GetOutermost()->GetName();
			Supplier.ClassName = InParams.ObjectStruct->GetOwnerClass() ? InParams.ObjectStruct->GetOwnerClass()->GetName() : FName(NAME_None).ToString();
			Supplier.CurrentLine = InParams.LineNumber;

			ContextSupplier = &Supplier; //-V506
		}
		else
		{
			if (ContextSupplier != nullptr)
			{
				ContextSupplier->CurrentLine = InParams.LineNumber;
			}
		}
		InParams.Warn->SetContext(ContextSupplier);
	}

	if (InParams.bShouldCallEditChange && InParams.SubobjectOuter != nullptr)
	{
		InParams.SubobjectOuter->PreEditChange(nullptr);
	}

	FObjectInstancingGraph TempGraph;
	FObjectInstancingGraph& InstanceGraph = InParams.InInstanceGraph ? *InParams.InInstanceGraph : TempGraph;

	if (InParams.SubobjectRoot && InParams.SubobjectRoot != UObject::StaticClass()->GetDefaultObject())
	{
		InstanceGraph.SetDestinationRoot(InParams.SubobjectRoot);
	}

	// Parse the object properties.
	const TCHAR* NewSourceText =
		ImportPropertiesStep(
			InParams.DestData,
			InParams.SourceText,
			InParams.ObjectStruct,
			InParams.SubobjectRoot,
			InParams.SubobjectOuter,
			InParams.Warn,
			InParams.Depth,
			InstanceGraph,
			InParams.ObjectRemapper,
			InParams.PropertiesToSkip
		);

	if (InParams.SubobjectOuter != nullptr)
	{
		check(InParams.SubobjectRoot);

		// Update the object properties to point to the newly imported component objects.
		// Templates inside classes never need to have components instanced.
		if (!InParams.SubobjectRoot->HasAnyFlags(RF_ClassDefaultObject))
		{
			UObject* SubobjectArchetype = InParams.SubobjectOuter->GetArchetype();
			InParams.ObjectStruct->InstanceSubobjectTemplates(InParams.DestData, SubobjectArchetype, SubobjectArchetype->GetClass(),
				InParams.SubobjectOuter, &InstanceGraph);
		}

		if (InParams.bShouldCallEditChange)
		{
			// notify the object that it has just been imported
			InParams.SubobjectOuter->PostEditImport();

			// notify the object that it has been edited
			InParams.SubobjectOuter->PostEditChange();
		}
		InParams.SubobjectRoot->CheckDefaultSubobjects();
	}

	if (InParams.LineNumber != INDEX_NONE)
	{
		if (ContextSupplier == &Supplier)
		{
			ContextSupplier = nullptr;
			InParams.Warn->SetContext(nullptr);
		}
	}

	return NewSourceText;
}

}	// End namespace EditorUtilities
