// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/UnrealString.h"
#include "MaterialEditor/DEditorDoubleVectorParameterValue.h"
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorMaterialLayersParameterValue.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "MaterialEditor/DEditorRuntimeVirtualTextureParameterValue.h"
#include "MaterialEditor/DEditorSparseVolumeTextureParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorStaticComponentMaskParameterValue.h"
#include "MaterialEditor/DEditorStaticSwitchParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorTextureCollectionParameterValue.h"
#include "MaterialEditor/DEditorParameterCollectionParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "Materials/MaterialParameters.h"
#include "Math/Color.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

UDEditorParameterValue::UDEditorParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorFontParameterValue::UDEditorFontParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorMaterialLayersParameterValue::UDEditorMaterialLayersParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorScalarParameterValue::UDEditorScalarParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorStaticComponentMaskParameterValue::UDEditorStaticComponentMaskParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorStaticSwitchParameterValue::UDEditorStaticSwitchParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorTextureParameterValue::UDEditorTextureParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorTextureCollectionParameterValue::UDEditorTextureCollectionParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorParameterCollectionParameterValue::UDEditorParameterCollectionParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorVectorParameterValue::UDEditorVectorParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorDoubleVectorParameterValue::UDEditorDoubleVectorParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorRuntimeVirtualTextureParameterValue::UDEditorRuntimeVirtualTextureParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UDEditorSparseVolumeTextureParameterValue::UDEditorSparseVolumeTextureParameterValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

static UDEditorParameterValue* CreateParameter_Scalar(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorScalarParameterValue* Parameter = NewObject<UDEditorScalarParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::Scalar)
	{
		Parameter->ParameterValue = Meta.Value.AsScalar();
		Parameter->SliderMin = Meta.ScalarMin;
		Parameter->SliderMax = Meta.ScalarMax;
		Parameter->Enumeration = Meta.ScalarEnumeration;
		Parameter->EnumerationIndex = Meta.ScalarEnumerationIndex;
		Parameter->AtlasData.bIsUsedAsAtlasPosition = Meta.bUsedAsAtlasPosition;
		Parameter->AtlasData.Atlas = Meta.ScalarAtlas;
		Parameter->AtlasData.Curve = Meta.ScalarCurve;
		Parameter->bUseCustomPrimitiveData = Meta.PrimitiveDataIndex != INDEX_NONE;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_Vector(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorVectorParameterValue* Parameter = NewObject<UDEditorVectorParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::Vector)
	{
		Parameter->ParameterValue = Meta.Value.AsLinearColor();
		Parameter->bIsUsedAsChannelMask = Meta.bUsedAsChannelMask;
		Parameter->bUseCustomPrimitiveData = Meta.PrimitiveDataIndex != INDEX_NONE;
		Parameter->ChannelNames = Meta.ChannelNames;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_DoubleVector(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorDoubleVectorParameterValue* Parameter = NewObject<UDEditorDoubleVectorParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::DoubleVector)
	{
		Parameter->ParameterValue = Meta.Value.AsVector4d();
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_Texture(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorTextureParameterValue* Parameter = NewObject<UDEditorTextureParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::Texture)
	{
		Parameter->ParameterValue = Meta.Value.Texture;
		Parameter->ChannelNames = Meta.ChannelNames;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_TextureCollection(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorTextureCollectionParameterValue* Parameter = NewObject<UDEditorTextureCollectionParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::TextureCollection)
	{
		Parameter->ParameterValue = Meta.Value.TextureCollection;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_RuntimeVirtualTexture(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorRuntimeVirtualTextureParameterValue* Parameter = NewObject<UDEditorRuntimeVirtualTextureParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::RuntimeVirtualTexture)
	{
		Parameter->ParameterValue = Meta.Value.RuntimeVirtualTexture;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_SparseVolumeTexture(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorSparseVolumeTextureParameterValue* Parameter = NewObject<UDEditorSparseVolumeTextureParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::SparseVolumeTexture)
	{
		Parameter->ParameterValue = Meta.Value.SparseVolumeTexture;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_Font(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorFontParameterValue* Parameter = NewObject<UDEditorFontParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::Font)
	{
		Parameter->ParameterValue.FontValue = Meta.Value.Font.Value;
		Parameter->ParameterValue.FontPage = Meta.Value.Font.Page;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_StaticSwitch(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorStaticSwitchParameterValue* Parameter = NewObject<UDEditorStaticSwitchParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::StaticSwitch)
	{
		Parameter->ParameterValue = Meta.Value.AsStaticSwitch();
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_ParameterCollection(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorParameterCollectionParameterValue* Parameter = NewObject<UDEditorParameterCollectionParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::ParameterCollection)
	{
		Parameter->ParameterValue = Meta.Value.ParameterCollection;
	}
	return Parameter;
}

static UDEditorParameterValue* CreateParameter_StaticComponentMask(UObject* Owner, const FMaterialParameterMetadata& Meta)
{
	UDEditorStaticComponentMaskParameterValue* Parameter = NewObject<UDEditorStaticComponentMaskParameterValue>(Owner);
	if (Meta.Value.Type == EMaterialParameterType::StaticComponentMask)
	{
		Parameter->ParameterValue.R = Meta.Value.Bool[0];
		Parameter->ParameterValue.G = Meta.Value.Bool[1];
		Parameter->ParameterValue.B = Meta.Value.Bool[2];
		Parameter->ParameterValue.A = Meta.Value.Bool[3];
	}
	return Parameter;
}

UDEditorParameterValue* UDEditorParameterValue::Create(UObject* Owner,
	EMaterialParameterType Type,
	const FMaterialParameterInfo& ParameterInfo,
	const FMaterialParameterMetadata& Meta)
{
	UDEditorParameterValue* Parameter = nullptr;
	switch (Type)
	{
	case EMaterialParameterType::Scalar: Parameter = CreateParameter_Scalar(Owner, Meta); break;
	case EMaterialParameterType::Vector: Parameter = CreateParameter_Vector(Owner, Meta); break;
	case EMaterialParameterType::DoubleVector: Parameter = CreateParameter_DoubleVector(Owner, Meta); break;
	case EMaterialParameterType::Texture: Parameter = CreateParameter_Texture(Owner, Meta); break;
	case EMaterialParameterType::TextureCollection: Parameter = CreateParameter_TextureCollection(Owner, Meta); break;
	case EMaterialParameterType::RuntimeVirtualTexture: Parameter = CreateParameter_RuntimeVirtualTexture(Owner, Meta); break;
	case EMaterialParameterType::SparseVolumeTexture: Parameter = CreateParameter_SparseVolumeTexture(Owner, Meta); break;
	case EMaterialParameterType::Font: Parameter = CreateParameter_Font(Owner, Meta); break;
	case EMaterialParameterType::StaticSwitch: Parameter = CreateParameter_StaticSwitch(Owner, Meta); break;
	case EMaterialParameterType::ParameterCollection: Parameter = CreateParameter_ParameterCollection(Owner, Meta); break;
	case EMaterialParameterType::StaticComponentMask: Parameter = CreateParameter_StaticComponentMask(Owner, Meta); break;
	default: break;
	}

	check(Parameter);
	Parameter->ParameterInfo = ParameterInfo;
	Parameter->ExpressionId = Meta.ExpressionGuid;
	Parameter->Description = Meta.Description;
	Parameter->SortPriority = Meta.SortPriority;
	Parameter->bOverride = Meta.bOverride;
	return Parameter;
}
