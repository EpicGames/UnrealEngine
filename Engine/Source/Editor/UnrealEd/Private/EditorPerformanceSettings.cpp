// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/EditorPerformanceSettings.h"
#include "Settings/EditorProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorPerformanceSettings)

TAutoConsoleVariable<int32> CVarEditorViewportHighDPI(
	TEXT("r.Editor.Viewport.HighDPI"), 1,
	TEXT("Controls whether editor & PIE viewports can be displayed at high DPI."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarEditorViewportOverrideGameScreenPercentage(
	TEXT("r.Editor.Viewport.OverridePIEScreenPercentage"), 1,
	TEXT("Apply editor viewports' default screen percentage settings to game viewport clients in PIE."),
	ECVF_Default);

UEditorPerformanceSettings::UEditorPerformanceSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShowFrameRateAndMemory(false)
	, bThrottleCPUWhenNotForeground(true)
	, bAllowSlateThrottling(true)
	, bDisableRealtimeViewportsInRemoteSessions(true)
	, bMonitorEditorPerformance(false)
	, bEnableScalabilityWarningIndicator(true)
	, bDisplayHighDPIViewports(true)
	, bOverridePIEScreenPercentage(true)
	, RealtimeScreenPercentageMode(EEditorUserScreenPercentageModeOverride::ProjectDefault)
	, MobileScreenPercentageMode(EEditorUserScreenPercentageModeOverride::ProjectDefault)
	, VRScreenPercentageMode(EEditorUserScreenPercentageModeOverride::ProjectDefault)
	, PathTracerScreenPercentageMode(EEditorUserScreenPercentageModeOverride::ProjectDefault)
	, NonRealtimeScreenPercentageMode(EEditorUserScreenPercentageModeOverride::ProjectDefault)
	, bOverrideManualScreenPercentage(false)
	, ManualScreenPercentage(100.0f)
	, bOverrideMinViewportRenderingResolution(false)
	, MinViewportRenderingResolution(720)
	, bOverrideMaxViewportRenderingResolution(false)
	, MaxViewportRenderingResolution(2160)
{

}

void UEditorPerformanceSettings::PostInitProperties()
{
	Super::PostInitProperties();

	CVarEditorViewportHighDPI->Set(bDisplayHighDPIViewports != 0, ECVF_SetByProjectSetting);
	CVarEditorViewportOverrideGameScreenPercentage->Set(bOverridePIEScreenPercentage != 0, ECVF_SetByProjectSetting);

	if (FProperty* EnableVSyncProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bEnableVSync)))
	{
		ExportValuesToConsoleVariables(EnableVSyncProperty);
	}

	if (FProperty* AllowSlateThrottlingProperty = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bAllowSlateThrottling)))
	{
		ExportValuesToConsoleVariables(AllowSlateThrottlingProperty);
	}

	UEditorPerformanceProjectSettings::ExportResolutionValuesToConsoleVariables();
}

void UEditorPerformanceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.Property)
	{
		return;
	}
		
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bDisplayHighDPIViewports) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, RealtimeScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, MobileScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, VRScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, PathTracerScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, NonRealtimeScreenPercentageMode) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bOverrideManualScreenPercentage) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, ManualScreenPercentage) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bOverrideMinViewportRenderingResolution) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, MinViewportRenderingResolution) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bOverrideMaxViewportRenderingResolution) ||
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, MaxViewportRenderingResolution))
	{
		UEditorPerformanceProjectSettings::ExportResolutionValuesToConsoleVariables();
	}
	else
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
