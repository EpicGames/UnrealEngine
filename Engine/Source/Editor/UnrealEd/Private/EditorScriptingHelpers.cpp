// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorScriptingHelpers.h"
#include "Utils.h"
#include "Algo/Count.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Editor.h"

namespace EditorScriptingHelpersInternal
{
	FString RemoveFullName(const FString& AnyAssetPath, FString& OutFailureReason)
	{
		FString Result = AnyAssetPath.TrimStartAndEnd();
		SIZE_T NumberOfSpace = Algo::Count(AnyAssetPath, TEXT(' '));

		if (NumberOfSpace == 0)
		{
			return MoveTemp(Result);
		}
		else if (NumberOfSpace > 1)
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert path '%s' because there are too many spaces."), *AnyAssetPath);
			return FString();
		}
		else// if (NumberOfSpace == 1)
		{
			int32 FoundIndex = 0;
			AnyAssetPath.FindChar(TEXT(' '), FoundIndex);
			check(FoundIndex > INDEX_NONE && FoundIndex < AnyAssetPath.Len()); // because of TrimStartAndEnd

			// Confirm that it's a valid Class
			FString ClassName = AnyAssetPath.Left(FoundIndex);

			// Convert \ to /
			ClassName.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);

			// Test ClassName for invalid Char
			const int32 StrLen = FCString::Strlen(INVALID_OBJECTNAME_CHARACTERS);
			for (int32 Index = 0; Index < StrLen; ++Index)
			{
				int32 InvalidFoundIndex = 0;
				if (ClassName.FindChar(INVALID_OBJECTNAME_CHARACTERS[Index], InvalidFoundIndex))
				{
					OutFailureReason = FString::Printf(TEXT("Can't convert the path %s because it contains invalid characters (probably spaces)."), *AnyAssetPath);
					return FString();
				}
			}

			// Return the path without the Class name
			return AnyAssetPath.Mid(FoundIndex + 1);
		}
	}

	FString ConvertAnyPathToObjectPathInternal(const FString& AnyAssetPath, bool bIncludeSubObject, FString& OutFailureReason)
	{
		// Convert "AssetClass'/Game/Folder/MyAsset.MyAsset'" -> "/Game/Folder/MyAsset.MyAsset"
		FString TextPath = FPackageName::ExportTextPathToObjectPath(AnyAssetPath);

		// Convert "AssetClass /Game/Folder/MyAsset.MyAsset" -> "/Game/Folder/MyAsset.MyAsset"
		TextPath = EditorScriptingHelpersInternal::RemoveFullName(TextPath, OutFailureReason);
		if (TextPath.IsEmpty())
		{
			return FString();
		}

		// Convert \ to /
		TextPath.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
		FPaths::RemoveDuplicateSlashes(TextPath);

		// Extract the subobject path, if any
		FString SubObjectPath;
		if (int32 SubObjectDelimiterIdx = INDEX_NONE;
			TextPath.FindChar(SUBOBJECT_DELIMITER_CHAR, SubObjectDelimiterIdx))
		{
			SubObjectPath = TextPath.Mid(SubObjectDelimiterIdx + 1);
			TextPath.LeftInline(SubObjectDelimiterIdx);
		}

		// Extract and validate the object name
		FString ObjectName;
		if (int32 ObjectDelimiterIdx = INDEX_NONE;
			TextPath.FindChar(TEXT('.'), ObjectDelimiterIdx))
		{
			ObjectName = TextPath.Mid(ObjectDelimiterIdx + 1);
			TextPath.LeftInline(ObjectDelimiterIdx);
		}
		else
		{
			// Infer from the package name
			ObjectName = FPackageName::GetShortName(TextPath);
		}
		if (ObjectName.IsEmpty())
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it doesn't contain an asset name."), *AnyAssetPath);
			return FString();
		}
		if (!EditorScriptingHelpers::IsAValidPath(ObjectName, INVALID_OBJECTNAME_CHARACTERS, OutFailureReason))
		{
			return FString();
		}

		// TextPath should now be a valid package name, so verify that
		if (!EditorScriptingHelpers::IsAValidPath(TextPath, INVALID_LONGPACKAGE_CHARACTERS, OutFailureReason))
		{
			return FString();
		}

		// Reject disallowed roots
		if (FPackageName::IsScriptPackage(TextPath))
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it start with /Script/"), *AnyAssetPath);
			return FString();
		}
		if (FPackageName::IsMemoryPackage(TextPath))
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it start with /Memory/"), *AnyAssetPath);
			return FString();
		}

		// Confirm that the path starts with a valid root
		if (!FPackageName::IsValidPath(TextPath))
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it does not map to a root."), *AnyAssetPath);
			return FString();
		}

		// Rebuild the full object path
		TextPath += TEXT(".");
		TextPath += ObjectName;
		if (bIncludeSubObject && !SubObjectPath.IsEmpty())
		{
			TextPath += TEXT(":");
			TextPath += SubObjectPath;
		}

		return TextPath;
	}
}

bool EditorScriptingHelpers::CheckIfInEditorAndPIE()
{
	if (!IsInGameThread())
	{
		UE_LOGF(LogUtils, Error, "You are not on the main thread.");
		return false;
	}
	if (!GIsEditor)
	{
		UE_LOGF(LogUtils, Error, "You are not in the Editor.");
		return false;
	}
	if (GEditor->PlayWorld || GIsPlayInEditorWorld)
	{
		UE_LOGF(LogUtils, Error, "The Editor is currently in a play mode.");
		return false;
	}
	return true;
}

FString EditorScriptingHelpers::ConvertAnyPathToLongPackagePath(const FString& AnyPath, FString& OutFailureReason)
{
	// Convert "AssetClass'/Game/Folder/MyAsset.MyAsset'" -> "/Game/Folder/MyAsset.MyAsset"
	FString TextPath = FPackageName::ExportTextPathToObjectPath(AnyPath);

	// Convert "AssetClass /Game/Folder/MyAsset.MyAsset" -> "/Game/Folder/MyAsset.MyAsset"
	TextPath = EditorScriptingHelpersInternal::RemoveFullName(TextPath, OutFailureReason);
	if (TextPath.IsEmpty())
	{
		return FString();
	}

	// Convert \ to /
	TextPath.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	FPaths::RemoveDuplicateSlashes(TextPath);

	// Remove the object path, if any
	if (int32 ObjectDelimiterIdx = INDEX_NONE;
		TextPath.FindChar(TEXT('.'), ObjectDelimiterIdx))
	{
		TextPath.LeftInline(ObjectDelimiterIdx);
	}

	// TextPath should now be a valid package name, so verify that
	if (!EditorScriptingHelpers::IsAValidPath(TextPath, INVALID_LONGPACKAGE_CHARACTERS, OutFailureReason))
	{
		return FString();
	}

	// Reject disallowed roots
	if (FPackageName::IsScriptPackage(TextPath))
	{
		OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it start with /Script/"), *AnyPath);
		return FString();
	}
	if (FPackageName::IsMemoryPackage(TextPath))
	{
		OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it start with /Memory/"), *AnyPath);
		return FString();
	}

	// Confirm that the path starts with a valid root
	if (!FPackageName::IsValidPath(TextPath))
	{
		OutFailureReason = FString::Printf(TEXT("Can't convert the path '%s' because it does not map to a root."), *AnyPath);
		return FString();
	}

	return TextPath;
}

bool EditorScriptingHelpers::HasValidRoot(const FString& ObjectPath)
{
	FStringView PackageName = FPackageName::ObjectPathToPackageName(FStringView(ObjectPath));
	return FPackageName::IsValidPath(PackageName);
}

// Test for invalid characters
bool EditorScriptingHelpers::IsAValidPath(const FString& Path, const TCHAR* InvalidChar, FString& OutFailureReason)
{
	// Like !FName::IsValidGroupName(Path)), but with another list and no conversion to from FName
    // InvalidChar may be INVALID_OBJECTPATH_CHARACTERS or INVALID_LONGPACKAGE_CHARACTERS or ...
	const int32 StrLen = FCString::Strlen(InvalidChar);
	for (int32 Index = 0; Index < StrLen; ++Index)
	{
		int32 FoundIndex = 0;
		if (Path.FindChar(InvalidChar[Index], FoundIndex))
		{
			OutFailureReason = FString::Printf(TEXT("Can't convert the path %s because it contains invalid characters."), *Path);
			return false;
		}
	}

	if (Path.Len() > FPlatformMisc::GetMaxPathLength())
	{
		OutFailureReason = FString::Printf(TEXT("Can't convert the path because it is too long (%d characters). This may interfere with cooking for consoles. Unreal filenames should be no longer than %d characters. Full path value: %s"), Path.Len(), FPlatformMisc::GetMaxPathLength(), *Path);
		return false;
	}
	return true;
}

bool EditorScriptingHelpers::IsAValidPathForCreateNewAsset(const FString& ObjectPath, FString& OutFailureReason)
{
	const FString ObjectName = FPackageName::ObjectPathToPathWithinPackage(ObjectPath);

	// Make sure the name is not already a class or otherwise invalid for saving
	FText FailureReason;
	if (!FFileHelper::IsFilenameValidForSaving(ObjectName, FailureReason))
	{
		OutFailureReason = FailureReason.ToString();
		return false;
	}

	// Make sure the new name only contains valid characters
	if (!FName::IsValidXName(ObjectName, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &FailureReason))
	{
		OutFailureReason = FailureReason.ToString();
		return false;
	}

	// Make sure we are not creating an FName that is too large
	if (ObjectPath.Len() >= NAME_SIZE)
	{
		OutFailureReason = TEXT("This asset name is too long (") + FString::FromInt(ObjectPath.Len()) + TEXT(" characters), the maximum is ") + FString::FromInt(NAME_SIZE - 1) + TEXT(". Please choose a shorter name.");
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
	if (AssetData.IsValid())
	{
		OutFailureReason = TEXT("An asset already exists at this location.");
		return false;
	}

	return true;
}

FString EditorScriptingHelpers::ConvertAnyPathToObjectPath(const FString& AnyAssetPath, FString& OutFailureReason)
{
	return EditorScriptingHelpersInternal::ConvertAnyPathToObjectPathInternal(AnyAssetPath, false, OutFailureReason);
}

FString EditorScriptingHelpers::ConvertAnyPathToSubObjectPath(const FString& AnyAssetPath, FString& OutFailureReason)
{
	return EditorScriptingHelpersInternal::ConvertAnyPathToObjectPathInternal(AnyAssetPath, true, OutFailureReason);
}
