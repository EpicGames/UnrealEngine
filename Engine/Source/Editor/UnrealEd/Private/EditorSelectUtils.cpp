// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/GroupActor.h"
#include "Components/ChildActorComponent.h"
#include "Components/DecalComponent.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Engine/Selection.h"
#include "EdMode.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "ScopedTransaction.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "LevelEditorViewport.h"
#include "StatsViewerModule.h"
#include "SnappingUtils.h"
#include "Logging/MessageLog.h"
#include "ActorGroupingUtils.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Interfaces/TypedElementEditorVisualizerInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"

#if PLATFORM_MAC
#include "HAL/PlatformApplicationMisc.h"
#endif // PLATFORM_MAC

#define LOCTEXT_NAMESPACE "EditorSelectUtils"

DEFINE_LOG_CATEGORY_STATIC(LogEditorSelectUtils, Log, All);

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/


// Click flags.
enum EViewportClick
{
	CF_MOVE_ACTOR	= 1,	// Set if the actors have been moved since first click
	CF_MOVE_TEXTURE = 2,	// Set if textures have been adjusted since first click
	CF_MOVE_ALL     = (CF_MOVE_ACTOR | CF_MOVE_TEXTURE),
};

/*-----------------------------------------------------------------------------
   Change transacting.
-----------------------------------------------------------------------------*/


void UUnrealEdEngine::NoteActorMovement()
{
	if( !GUndo && !(GEditor->ClickFlags & CF_MOVE_ACTOR) )
	{
		GEditor->ClickFlags |= CF_MOVE_ACTOR;

		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ActorMovement", "Actor Movement") );
		GLevelEditorModeTools().Snapping=0;
		
		AActor* SelectedActor = NULL;
		{
			FSelectionIterator It(GetSelectedActorIterator());
			if (It)
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				SelectedActor = Actor;
			}
		}

		// Look for an actor that requires snapping.
		{
			FSelectionIterator It(GetSelectedActorIterator());
			if (It)
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				GLevelEditorModeTools().Snapping = 1;
			}
		}

		TSet<AGroupActor*> GroupActors;

		// Modify selected actors.
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			Actor->Modify();

			if (UActorGroupingUtils::IsGroupingActive())
			{
				// if this actor is in a group, add the GroupActor into a list to be modified shortly
				AGroupActor* ActorLockedRootGroup = AGroupActor::GetRootForActor(Actor, true);
				if (ActorLockedRootGroup != nullptr)
				{
					GroupActors.Add(ActorLockedRootGroup);
				}
			}
		}

		// Modify unique group actors
		for (auto* GroupActor : GroupActors)
		{
			GroupActor->Modify();
		}
	}
}

void UUnrealEdEngine::FinishAllSnaps()
{
	if(!IsRunningCommandlet())
	{
		if( ClickFlags & CF_MOVE_ACTOR )
		{
			ClickFlags &= ~CF_MOVE_ACTOR;

			for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				Actor->Modify();
				Actor->InvalidateLightingCache();
				Actor->PostEditMove( true );
			}
		}
	}
}


void UUnrealEdEngine::Cleanse( bool ClearSelection, bool Redraw, const FText& Reason, bool bResetTrans )
{
	if (GIsRunning)
	{
		FMessageLog("MapCheck").NewPage(LOCTEXT("MapCheck", "Map Check"));

		FMessageLog("LightingResults").NewPage(LOCTEXT("LightingBuildNewLogPage", "Lighting Build"));

		FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
		StatsViewerModule.Clear();
	}

	Super::Cleanse( ClearSelection, Redraw, Reason, bResetTrans );
}


FVector UUnrealEdEngine::GetPivotLocation()
{
	return GLevelEditorModeTools().PivotLocation;
}


void UUnrealEdEngine::SetPivot( FVector NewPivot, bool bSnapPivotToGrid, bool bIgnoreAxis, bool bAssignPivot/*=false*/ )
{
	if (!GCurrentLevelEditingViewportClient)
	{
		return;
	}

	FEditorModeTools& EditorModeTools = GLevelEditorModeTools();

	if( !bIgnoreAxis )
	{
		// Don't stomp on orthonormal axis.
		// TODO: this breaks if there is genuinely a need to set the pivot to a coordinate containing a zero component
 		if( NewPivot.X==0 ) NewPivot.X=EditorModeTools.PivotLocation.X;
 		if( NewPivot.Y==0 ) NewPivot.Y=EditorModeTools.PivotLocation.Y;
 		if( NewPivot.Z==0 ) NewPivot.Z=EditorModeTools.PivotLocation.Z;
	}

	// Set the pivot.
	EditorModeTools.SetPivotLocation(NewPivot, false);

	if( bSnapPivotToGrid )
	{
		FRotator DummyRotator(0,0,0);
		FSnappingUtils::SnapToBSPVertex( EditorModeTools.SnappedLocation, EditorModeTools.GridBase, DummyRotator );
		EditorModeTools.PivotLocation = EditorModeTools.SnappedLocation;
	}

	// Check all elements.
	int32 NumElements = 0;

	//default to using the x axis for the translate rotate widget
	EditorModeTools.TranslateRotateXAxisAngle = 0.0f;
	EditorModeTools.TranslateRotate2DAngle = 0.0f;
	
	FVector TranslateRotateWidgetWorldXAxis = FVector::ZeroVector;
	FVector Widget2DWorldXAxis = FVector::ZeroVector;

	// Pick a new common pivot, or not.
	TTypedElement<ITypedElementWorldInterface> SingleWorldElement;

	FTypedElementListConstRef ElementsToManipulate = GCurrentLevelEditingViewportClient->GetElementsToManipulate();
	ElementsToManipulate->ForEachElement<ITypedElementWorldInterface>([&NumElements, &TranslateRotateWidgetWorldXAxis, &Widget2DWorldXAxis, &SingleWorldElement](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
	{
		if (NumElements == 0)
		{
			FTransform ElementWorldTransform;
			InWorldElement.GetWorldTransform(ElementWorldTransform);

			TranslateRotateWidgetWorldXAxis = ElementWorldTransform.TransformVector(FVector(1.0f, 0.0f, 0.0f));
			//get the xy plane project of this vector
			TranslateRotateWidgetWorldXAxis.Z = 0.0f;
			if (!TranslateRotateWidgetWorldXAxis.Normalize())
			{
				TranslateRotateWidgetWorldXAxis = FVector(1.0f, 0.0f, 0.0f);
			}

			Widget2DWorldXAxis = ElementWorldTransform.TransformVector(FVector(1, 0, 0));
			Widget2DWorldXAxis.Y = 0;
			if (!Widget2DWorldXAxis.Normalize())
			{
				Widget2DWorldXAxis = FVector(1, 0, 0);
			}
		}

		SingleWorldElement = InWorldElement;
		++NumElements;
		return true;
	});
	
	if (bAssignPivot && UActorGroupingUtils::IsGroupingActive())
	{
		if (TTypedElement<ITypedElementObjectInterface> ObjectElement = ElementsToManipulate->GetElement<ITypedElementObjectInterface>(SingleWorldElement))
		{
			if (AActor* SingleActor = Cast<AActor>(ObjectElement.GetObject()))
			{
				// set group pivot for the root-most group
				if (AGroupActor* ActorGroupRoot = AGroupActor::GetRootForActor(SingleActor, true, true))
				{
					ActorGroupRoot->SetActorLocation(EditorModeTools.PivotLocation, false);
				}
			}
		}
	}

	//if there are multiple elements selected, just use the x-axis for the "translate/rotate" or 2D widgets
	if (NumElements == 1)
	{
		EditorModeTools.TranslateRotateXAxisAngle = static_cast<float>(TranslateRotateWidgetWorldXAxis.Rotation().Yaw);
		EditorModeTools.TranslateRotate2DAngle = static_cast<float>(FMath::RadiansToDegrees(FMath::Atan2(Widget2DWorldXAxis.Z, Widget2DWorldXAxis.X)));
	}

	// Update showing.
	EditorModeTools.PivotShown = NumElements > 0;
}


void UUnrealEdEngine::ResetPivot()
{
	GLevelEditorModeTools().PivotShown	= 0;
	GLevelEditorModeTools().Snapping		= 0;
	GLevelEditorModeTools().SnappedActor	= 0;
}

/*-----------------------------------------------------------------------------
	Selection.
-----------------------------------------------------------------------------*/

void UUnrealEdEngine::OnEditorElementSelectionPtrChanged(USelection* Selection, UTypedElementSelectionSet* OldSelectionSet, UTypedElementSelectionSet* NewSelectionSet)
{
	if (Selection == GetSelectedActors())
	{
		if (OldSelectionSet)
		{
			OldSelectionSet->OnChanged().RemoveAll(this);
		}

		if (NewSelectionSet)
		{
			NewSelectionSet->OnChanged().AddUObject(this, &UUnrealEdEngine::OnEditorElementSelectionChanged);
		}
	}
}


void UUnrealEdEngine::OnEditorElementSelectionChanged(const UTypedElementSelectionSet* SelectionSet)
{
	VisualizersForSelection.Empty();

	auto GetVisualizersForSelection = [this](AActor* Actor, const UActorComponent* SelectedComponent)
	{
		// Iterate over components of that actor (and recurse through child components)
		TInlineComponentArray<UActorComponent*> Components;

		constexpr bool bIncludeFromChildActors = true;
		Actor->GetComponents(Components, bIncludeFromChildActors);

		for (int32 CompIdx = 0; CompIdx < Components.Num(); CompIdx++)
		{
			TWeakObjectPtr<UActorComponent> Comp(Components[CompIdx]);
			if (Comp.IsValid() && Comp->IsRegistered())
			{
				// Try and find a visualizer
				TSharedPtr<FComponentVisualizer> Visualizer = FindComponentVisualizer(Comp->GetClass());
				if (Visualizer.IsValid())
				{
					FCachedComponentVisualizer CachedComponentVisualizer(Comp.Get(), Visualizer);
					FComponentVisualizerForSelection Temp{ CachedComponentVisualizer };

					FComponentVisualizerForSelection& ComponentVisualizerForSelection = VisualizersForSelection.Add_GetRef(MoveTemp(Temp));

					ComponentVisualizerForSelection.IsEnabledDelegate.Emplace(
						[bIsSelectedComponent = (Comp == SelectedComponent),
						 WeakViz = TWeakPtr<FComponentVisualizer>(Visualizer),
						 CompPath = CachedComponentVisualizer.ComponentPropertyPath]()
						{
							if (bIsSelectedComponent == true)
							{
								return true;
							}

							if (GetDefault<UEditorPerProjectUserSettings>()->bShowSelectionSubcomponents == false)
							{
								return false;
							}

							bool bShouldShowWithSelection = true;
							if (TSharedPtr<FComponentVisualizer> PinnedViz = WeakViz.Pin())
							{
								bShouldShowWithSelection = PinnedViz->ShouldShowForSelectedSubcomponents(CompPath.GetComponent());
							}
							return bShouldShowWithSelection;
						}
					);
				}
			}
		}
	};

	UTypedElementSelectionSet* LevelEditorSelectionSet = GetSelectedActors()->GetElementSelectionSet();
	TSet<AActor*> ActorsProcessed;
	LevelEditorSelectionSet->ForEachSelectedObject<UActorComponent>([&ActorsProcessed, &GetVisualizersForSelection](UActorComponent* InActorComponent)
		{
			if (AActor* Actor = InActorComponent->GetOwner())
			{
				if (!ActorsProcessed.Contains(Actor))
				{
					GetVisualizersForSelection(Actor, InActorComponent);
					ActorsProcessed.Emplace(Actor);
				}
			}
			return true;
		});

	if (ActorsProcessed.Num() == 0)
	{
		LevelEditorSelectionSet->ForEachSelectedObject<AActor>([&GetVisualizersForSelection](AActor* InActor)
			{
				GetVisualizersForSelection(InActor, InActor->GetRootComponent());
				return true;
			});
	}

	// Try getting visualizers to use from typed elements. This could be used as the sole path instead
	//  of doing the above if we choose to implement the visualization interface for actors and components. 
	//  For now, however, it is used for non-actors that implement their visualizations via bridge/hack actors.
	LevelEditorSelectionSet->ForEachSelectedElement<ITypedElementEditorVisualizerInterface>(
		[this, SelectionSet](const TTypedElement<ITypedElementEditorVisualizerInterface>& Element)
	{
		TArray<ITypedElementEditorVisualizerInterface::FComponentVisualizerForSelection> Visualizers;
		if (Element.GetComponentVisualizersToUse(SelectionSet, Visualizers))
		{
			for (ITypedElementEditorVisualizerInterface::FComponentVisualizerForSelection& Visualizer : Visualizers)
			{
				// We'll avoid adding visualizers that target the same component since we don't currently remove
				//  any conflicting ones from regular component/actor selection
				bool bComponentAlreadyTargeted = VisualizersForSelection.ContainsByPredicate([&Visualizer](const FComponentVisualizerForSelection& Existing)
					{ return Existing.ComponentVisualizer.ComponentPropertyPath == Visualizer.ComponentVisualizer.ComponentPropertyPath; });

				if (!bComponentAlreadyTargeted)
				{
					VisualizersForSelection.Add({ Visualizer.ComponentVisualizer, Visualizer.IsEnabledDelegate });
				}
			}//end for each returned visualizer
		}
		return true; // continue iterating
	});
	
	// Restore the active component visualizer, since an undo/redo may have changed the selection
	bool bDidSetVizThisTick = false;
	if (VisualizersForSelection.Num() > 0)
	{
		for (FComponentVisualizerForSelection& VisualizerForSelection : VisualizersForSelection)
		{
			if (!ComponentVisManager.IsActive() && VisualizerForSelection.ComponentVisualizer.Visualizer->GetEditedComponent() != nullptr)
			{
				ComponentVisManager.SetActiveComponentVis(GCurrentLevelEditingViewportClient, VisualizerForSelection.ComponentVisualizer.Visualizer);
				bDidSetVizThisTick = true;
				break;
			}

			if (ComponentVisManager.IsActive() && VisualizerForSelection.ComponentVisualizer.Visualizer->GetEditedComponent() == ComponentVisManager.GetActiveComponentVis()->GetEditedComponent())
			{
				bDidSetVizThisTick = true;
				break;
			}
		}
	}

	if (ComponentVisManager.IsActive() && (VisualizersForSelection.Num() == 0 || !bDidSetVizThisTick))
	{
		ComponentVisManager.ClearActiveComponentVis();
	}

#if PLATFORM_MAC
	FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
#endif

	NoteSelectionChange();
}


void UUnrealEdEngine::PostActorSelectionChanged()
{
	// Whenever selection changes, recompute whether the selection contains a locked actor
	bCheckForMovementLockedItems = true;

	// Whenever selection changes, recompute whether the selection contains a world info actor
	bCheckForWorldSettingsActors = true;
}


void UUnrealEdEngine::SetPivotMovedIndependently(bool bMovedIndependently)
{
	bPivotMovedIndependently = bMovedIndependently;
}


bool UUnrealEdEngine::IsPivotMovedIndependently() const
{
	return bPivotMovedIndependently;
}


void UUnrealEdEngine::UpdatePivotLocationForSelection( bool bOnChange )
{
	if (!GCurrentLevelEditingViewportClient)
	{
		return;
	}

	// Pick a new common pivot, or not.
	TTypedElement<ITypedElementWorldInterface> SingleWorldElement;
	
	FTypedElementListConstRef ElementsToManipulate = GCurrentLevelEditingViewportClient->GetElementsToManipulate();
	ElementsToManipulate->ForEachElement<ITypedElementWorldInterface>([&SingleWorldElement](const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
	{
#if DO_CHECK
		if (ULevel* OwnerLevel = InWorldElement.GetOwnerLevel())
		{
			const bool bIsTemplate = InWorldElement.IsTemplateElement();
			const bool bLevelLocked = FLevelUtils::IsLevelLocked(OwnerLevel);
			check(bIsTemplate || !bLevelLocked);
		}
#endif	// DO_CHECK

		SingleWorldElement = InWorldElement;
		return true;
	});
	
	if (SingleWorldElement)
	{
		UBrushEditingSubsystem* BrushSubsystem = GEditor->GetEditorSubsystem<UBrushEditingSubsystem>();
		const bool bGeometryMode = BrushSubsystem && BrushSubsystem->IsGeometryEditorModeActive();

		// For geometry mode use current pivot location as it's set to selected face, not actor
		// TODO: If geometry used elements for face selection, then this could work via the world interface and this special case could be removed
		if (!bGeometryMode || bOnChange)
		{
			FTransform ElementWorldTransform;
			SingleWorldElement.GetWorldTransform(ElementWorldTransform);

			FVector ElementPivotOffset = FVector::ZeroVector;
			SingleWorldElement.GetPivotOffset(ElementPivotOffset);

			// Set pivot point to the element location, accounting for any set pivot offset
			FVector PivotPoint = ElementWorldTransform.TransformPosition(ElementPivotOffset);

			// If grouping is active, see if this element is an actor that's part of a locked group and use that pivot instead
			if (UActorGroupingUtils::IsGroupingActive())
			{
				if (TTypedElement<ITypedElementObjectInterface> ObjectElement = ElementsToManipulate->GetElement<ITypedElementObjectInterface>(SingleWorldElement))
				{
					if (AActor* SingleActor = Cast<AActor>(ObjectElement.GetObject()))
					{
						if (AGroupActor* ActorGroupRoot = AGroupActor::GetRootForActor(SingleActor, true, true))
						{
							PivotPoint = ActorGroupRoot->GetActorLocation();
						}
					}
				}
			}

			SetPivot(PivotPoint, false, true);
		}
	}
	else
	{
		ResetPivot();
	}

	SetPivotMovedIndependently(false);
}

void UUnrealEdEngine::NoteSelectionChange(bool bNotify)
{
	// The selection changed, so make sure the pivot (widget) is located in the right place
	UpdatePivotLocationForSelection( true );

	const bool bComponentSelectionChanged = GetSelectedComponentCount() > 0;
	if (bNotify)
	{
		USelection* Selection = bComponentSelectionChanged ? GetSelectedComponents() : GetSelectedActors();
		Selection->NoteSelectionChanged();
	}
	
	if (!bComponentSelectionChanged)
	{
		PostActorSelectionChanged();
	}

	UpdateFloatingPropertyWindows(/*bForceRefresh*/false, !bComponentSelectionChanged);
	RedrawLevelEditingViewports();
}

void UUnrealEdEngine::SelectGroup(AGroupActor* InGroupActor, bool bForceSelection/*=false*/, bool bInSelected/*=true*/, bool bNotify/*=true*/)
{
	USelection* ActorSelection = GetSelectedActors();

	if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
	{
		FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange;
		if (!bNotify)
		{
			ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
		}

		const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
			.SetWarnIfLocked(true)
			.SetAllowGroups(false)
			.SetAllowLegacyNotifications(false);

		bool bSelectionChanged = false;

		// Select/deselect all actors within the group (if locked or forced)
		if (bForceSelection || InGroupActor->IsLocked())
		{
			TArray<AActor*> GroupActors;
			InGroupActor->GetGroupActors(GroupActors);

			TArray<FTypedElementHandle, TInlineAllocator<256>> GroupElements;
			GroupElements.Reserve(GroupActors.Num());
			for (AActor* Actor : GroupActors)
			{
				if (FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
				{
					GroupElements.Add(MoveTemp(ElementHandle));
				}
			}

			if (GroupElements.Num() > 0)
			{
				bSelectionChanged |= bInSelected
					? SelectionSet->SelectElements(GroupElements, SelectionOptions)
					: SelectionSet->DeselectElements(GroupElements, SelectionOptions);
			}
		}

		if (bSelectionChanged)
		{
			if (bNotify)
			{
				SelectionSet->NotifyPendingChanges();
			}
		}
	}
}

bool UUnrealEdEngine::CanSelectActor(AActor* Actor, bool bInSelected, bool bSelectEvenIfHidden, bool bWarnIfLevelLocked ) const
{
	if (!Actor)
	{
		return false;
	}

	USelection* ActorSelection = GetSelectedActors();

	if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
	{
		const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
			.SetAllowHidden(bSelectEvenIfHidden)
			.SetWarnIfLocked(bWarnIfLevelLocked);

		return bInSelected
			? SelectionSet->CanSelectElement(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor), SelectionOptions)
			: SelectionSet->CanDeselectElement(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor), SelectionOptions);
	}

	return false;
}

void UUnrealEdEngine::SelectActor(AActor* Actor, bool bInSelected, bool bNotify, bool bSelectEvenIfHidden, bool bForceRefresh)
{
	if (!Actor || Actor->GetRootSelectionParent() != nullptr)
	{
		return;
	}

	USelection* ActorSelection = GetSelectedActors();

	if (UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet())
	{
		FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange;
		if (!bNotify)
		{
			ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
		}

		const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
			.SetAllowHidden(bSelectEvenIfHidden)
			.SetWarnIfLocked(true)
			.SetAllowLegacyNotifications(false)
			.SetAllowSubRootSelection(false);

		const bool bSelectionChanged = bInSelected
			? SelectionSet->SelectElement(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor), SelectionOptions)
			: SelectionSet->DeselectElement(UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor), SelectionOptions);

		if (bSelectionChanged)
		{
			if (bNotify)
			{
				SelectionSet->NotifyPendingChanges();
			}
		}
		else if (bNotify || bForceRefresh)
		{
			// Reset the property windows, in case something has changed since previous selection
			UpdateFloatingPropertyWindows(bForceRefresh);
		}
	}
}

void UUnrealEdEngine::SelectComponent(UActorComponent* Component, bool bInSelected, bool bNotify, bool bSelectEvenIfHidden)
{
	if (!Component || (Component->GetOwner() && Component->GetOwner()->GetRootSelectionParent() != nullptr))
	{
		return;
	}

	USelection* ComponentSelection = GetSelectedComponents();

	if (UTypedElementSelectionSet* SelectionSet = ComponentSelection->GetElementSelectionSet())
	{
		FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange;
		if (!bNotify)
		{
			ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
		}

		const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
			.SetAllowHidden(bSelectEvenIfHidden)
			.SetWarnIfLocked(true)
			.SetAllowLegacyNotifications(false);

		const bool bSelectionChanged = bInSelected
			? SelectionSet->SelectElement(UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component), SelectionOptions)
			: SelectionSet->DeselectElement(UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component), SelectionOptions);

		if (bSelectionChanged)
		{
			if (bNotify)
			{
				SelectionSet->NotifyPendingChanges();
			}
		}
	}
}

bool UUnrealEdEngine::IsComponentSelected(const UPrimitiveComponent* PrimComponent)
{
	USelection* ComponentSelection = GetSelectedComponents();

	if (UTypedElementSelectionSet* SelectionSet = ComponentSelection->GetElementSelectionSet())
	{
		return SelectionSet->IsElementSelected(UEngineElementsLibrary::AcquireEditorComponentElementHandle(PrimComponent), FTypedElementIsSelectedOptions().SetAllowIndirect(true));
	}

	return false;
}

void UUnrealEdEngine::SelectBSPSurf(UModel* InModel, int32 iSurf, bool bSelected, bool bNoteSelectionChange)
{
	if( GEdSelectionLock )
	{
		return;
	}

	FBspSurf& Surf = InModel->Surfs[ iSurf ];
	InModel->ModifySurf( iSurf, false );

	if( bSelected )
	{
		Surf.PolyFlags |= PF_Selected;
	}
	else
	{
		Surf.PolyFlags &= ~PF_Selected;
	}

	if( bNoteSelectionChange )
	{
		NoteSelectionChange();
	}

	PostActorSelectionChanged();
}

/**
 * Deselects all BSP surfaces in the specified level.
 *
 * @param	Level		The level for which to deselect all surfaces.
 * @return				The number of surfaces that were deselected
 */
static uint32 DeselectAllSurfacesForLevel(ULevel* Level)
{
	uint32 NumSurfacesDeselected = 0;
	if ( Level )
	{
		UModel* Model = Level->Model;
		for( int32 SurfaceIndex = 0 ; SurfaceIndex < Model->Surfs.Num() ; ++SurfaceIndex )
		{
			FBspSurf& Surf = Model->Surfs[SurfaceIndex];
			if( Surf.PolyFlags & PF_Selected )
			{
				Model->ModifySurf( SurfaceIndex, false );
				Surf.PolyFlags &= ~PF_Selected;
				++NumSurfacesDeselected;
			}
		}
	}
	return NumSurfacesDeselected;
}

/**
 * Deselects all BSP surfaces in the specified world.
 *
 * @param	World		The world for which to deselect all surfaces.
 * @return				The number of surfaces that were deselected
 */
static uint32 DeselectAllSurfacesForWorld(UWorld* World)
{
	uint32 NumSurfacesDeselected = 0;
	if (World)
	{
		NumSurfacesDeselected += DeselectAllSurfacesForLevel(World->PersistentLevel);
		for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
		{
			if (StreamingLevel)
			{
				if (ULevel* Level = StreamingLevel->GetLoadedLevel())
				{
					NumSurfacesDeselected += DeselectAllSurfacesForLevel(Level);
				}
			}
		}
	}
	return NumSurfacesDeselected;
}

void UUnrealEdEngine::DeselectAllSurfaces()
{
	DeselectAllSurfacesForWorld(GWorld);
}

void UUnrealEdEngine::SelectNone(bool bNoteSelectionChange, bool bDeselectBSPSurfs, bool WarnAboutManyActors)
{
	if (GEdSelectionLock)
	{
		return;
	}

	bool bSelectionChanged = false;

	USelection* ActorSelection = GetSelectedActors();
	UTypedElementSelectionSet* SelectionSet = ActorSelection->GetElementSelectionSet();

	FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange;

	if (SelectionSet)
	{
		if (!bNoteSelectionChange)
		{ 
			ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
		}
		bSelectionChanged |= SelectionSet->ClearSelection(FTypedElementSelectionOptions().SetAllowLegacyNotifications(false));
	}

	if (bDeselectBSPSurfs)
	{
		bSelectionChanged |= DeselectAllSurfacesForWorld(GWorld) > 0;
	}

	if (bSelectionChanged)
	{
		PostActorSelectionChanged();

		if (SelectionSet)
		{
			if (bNoteSelectionChange)
			{
				SelectionSet->NotifyPendingChanges();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
