// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ObjectThumbnail.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectAnnotation.h"
#include "Serialization/ArchiveCountMem.h"
#include "Misc/PackageName.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/ReferenceChainSearch.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/Level.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Components/LightComponent.h"
#include "Model.h"
#include "Exporters/Exporter.h"
#include "Materials/Material.h"
#include "Editor/Transactor.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "Animation/AnimSequence.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/Factory.h"
#include "Factories/PolysFactory.h"
#include "Engine/Texture.h"
#include "Factories/WorldFactory.h"
#include "Editor/GroupActor.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Editor/PropertyEditorTestObject.h"
#include "Animation/SkeletalMeshActor.h"
#include "Editor/TransBuffer.h"
#include "Components/ShapeComponent.h"
#include "Particles/Emitter.h"
#include "Components/SkeletalMeshComponent.h"
#include "Particles/ParticleSystemComponent.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundWave.h"
#include "GameFramework/Volume.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Components/BillboardComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DrawFrustumComponent.h"
#include "Layers/Layer.h"
#include "Engine/Polys.h"
#include "Engine/Selection.h"
#include "UnrealEngine.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdMisc.h"
#include "Utils.h"
#include "FileHelpers.h"
#include "Dialogs/Dialogs.h"
#include "UnrealEdGlobals.h"
#include "EditorSupportDelegates.h"
#include "BusyCursor.h"
#include "AudioDevice.h"
#include "Engine/LevelStreaming.h"
#include "LevelUtils.h"
#include "LevelEditorViewport.h"
#include "Layers/LayersSubsystem.h"
#include "ScopedTransaction.h"
#include "SurfaceIterators.h"
#include "LightMap.h"
#include "Engine/BrushBuilder.h"
#include "BSPOps.h"
#include "EditorLevelUtils.h"
#include "Interfaces/IMainFrameModule.h"
#include "PackageTools.h"
#include "LevelEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "LandscapeProxy.h"
#include "Lightmass/PrecomputedVisibilityOverrideVolume.h"
#include "Animation/AnimSet.h"
#include "InstancedFoliageActor.h"
#include "IMovieSceneCapture.h"
#include "MovieSceneCaptureModule.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationMapCheckErrorHandler.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "PropertyEditorModule.h"
#include "IPropertyTable.h"
#include "IDetailsView.h"
#include "IStructureDetailsView.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SnappingUtils.h"

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"

#include "Editor/ActorPositioning.h"

#include "StatsViewerModule.h"
#include "ActorEditorUtils.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Misc/ScopedSlowTask.h"

#include "ComponentReregisterContext.h"
#include "Engine/DocumentationActor.h"
#include "ShaderCompiler.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "NavLinkRenderingComponent.h"
#include "Analytics/AnalyticsPrivacySettings.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "AnalyticsEventAttribute.h"
#include "ISlateReflectorModule.h"
#include "MaterialUtilities.h"
#include "ActorGroupingUtils.h"
#include "ILauncherPlatform.h"
#include "LauncherPlatformModule.h"
#include "HAL/PlatformApplicationMisc.h"
#include "AssetExportTask.h"
#include "EditorBuildUtils.h"
#include "Subsystems/BrushEditingSubsystem.h"
#include "EdMode.h"
#include "LevelEditorSubsystem.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Component/ComponentElementData.h"
#include "Elements/Framework/TypedElementCommonActions.h"

#include "Serialization/StructuredArchive.h"
#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/LoadTimeTrace.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorServer, Log, All);

/** Used for the "tagsounds" and "checksounds" commands only			*/
static FUObjectAnnotationSparseBool DebugSoundAnnotation;

/** Internal struct to hold undo/redo transaction object context */
struct FEditorTransactionDeltaContext
{
	FGuid	OuterOperationId;
	int32	OperationDepth = 0;
	TArray<TPair<UObject*, FTransactionObjectEvent>> TransactionObjects;

	void Reset()
	{
		OuterOperationId.Invalidate();
		TransactionObjects.Empty();
		OperationDepth = 0;
	}
};

namespace
{
	/**
	 * A stat group use to track memory usage.
	 */
	class FWaveCluster
	{
	public:
		FWaveCluster() {}
		FWaveCluster(const TCHAR* InName)
			:	Name( InName )
			,	Num( 0 )
			,	Size( 0 )
		{}

		FString Name;
		int32 Num;
		int32 Size;
	};

	TAutoConsoleVariable<bool> CVarCheckForWorldGCLeaksAreFatal(
		TEXT("Editor.CheckForWorldGCLeaksAreFatal"),
		true,
		TEXT("If true, CheckForWorldGCLeaks() will cause a Fatal error if leaks are found. If false, it is an Error."),
		ECVF_Default
	);
}

/**
* @param		bPreviewOnly		If true, don't actually clear material references.  Useful for e.g. map error checking.
* @param		bLogReferences		If true, write to the log any references that were cleared (brush name and material name).
* @return							The number of surfaces that need cleaning or that were cleaned
*/
static int32 CleanBSPMaterials(UWorld* InWorld, bool bPreviewOnly, bool bLogBrushes)
{
	// Clear the mark flag the polys of all non-volume, non-builder brushes.
	// Make a list of all brushes that were encountered.
	TArray<ABrush*> Brushes;
	for ( TActorIterator<ABrush> It(InWorld) ; It ; ++It )
	{
		ABrush* ItBrush = *It;
		if ( !ItBrush->IsVolumeBrush() && !FActorEditorUtils::IsABuilderBrush(ItBrush) && !ItBrush->IsBrushShape() )
		{
			if( ItBrush->Brush && ItBrush->Brush->Polys )
			{
				for ( int32 PolyIndex = 0 ; PolyIndex < ItBrush->Brush->Polys->Element.Num() ; ++PolyIndex )
				{
					ItBrush->Brush->Polys->Element[PolyIndex].PolyFlags &= ~PF_EdProcessed;
				}
				Brushes.Add( ItBrush );
			}
		}
	}

	// Iterate over all surfaces and mark the corresponding brush polys.
	for ( TSurfaceIterator<> It(InWorld) ; It ; ++It )
	{
		if ( It->Actor && It->iBrushPoly != INDEX_NONE )
		{
			It->Actor->Brush->Polys->Element[ It->iBrushPoly ].PolyFlags |= PF_EdProcessed;
		}
	}

	// Go back over all brushes and clear material references on all unmarked polys.
	int32 NumRefrencesCleared = 0;
	for ( int32 BrushIndex = 0 ; BrushIndex < Brushes.Num() ; ++BrushIndex )
	{
		ABrush* Actor = Brushes[BrushIndex];
		for ( int32 PolyIndex = 0 ; PolyIndex < Actor->Brush->Polys->Element.Num() ; ++PolyIndex )
		{
			// If the poly was marked . . .
			if ( (Actor->Brush->Polys->Element[PolyIndex].PolyFlags & PF_EdProcessed) != 0 )
			{
				// . . . simply clear the mark flag.
				Actor->Brush->Polys->Element[PolyIndex].PolyFlags &= ~PF_EdProcessed;
			}
			else
			{
				// This poly wasn't marked, so clear its material reference if one exists.
				auto& ReferencedMaterial = Actor->Brush->Polys->Element[PolyIndex].Material;
				if ( ReferencedMaterial && ReferencedMaterial != UMaterial::GetDefaultMaterial(MD_Surface) )
				{
					NumRefrencesCleared++;
					if ( bLogBrushes )
					{
						UE_LOGF(LogEditorServer, Log, "Cleared %ls:%ls", *Actor->GetPathName(), *ReferencedMaterial->GetPathName() );
					}
					if ( !bPreviewOnly )
					{
						ReferencedMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
					}
				}
			}
		}
	}

	return NumRefrencesCleared;
}


void UEditorEngine::RedrawAllViewports(bool bInvalidateHitProxies)
{
	for (FEditorViewportClient* ViewportClient : AllViewportClients)
	{
		if (ViewportClient)
		{
			constexpr bool bForceChildViewportRedraw = false;
			ViewportClient->Invalidate(bForceChildViewportRedraw, bInvalidateHitProxies);
		}
	}
}

bool UEditorEngine::SafeExec( UWorld* InWorld, const TCHAR* InStr, FOutputDevice& Ar )
{
	const TCHAR* Str = InStr;

	// Keep a pointer to the beginning of the string to use for message displaying purposes
	const TCHAR* const FullStr = InStr;

	if( FParse::Command(&Str,TEXT("MACRO")) || FParse::Command(&Str,TEXT("EXEC")) )//oldver (exec)
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if( FParse::Command( &Str, TEXT( "EXECFILE" ) ) )
	{
		// Executes a file that contains a list of commands
		FString FilenameString;
		if( FParse::Token( Str, FilenameString, false ) )
		{
			ExecFile( InWorld, *FilenameString, Ar );
		}

		return true;
	}
	else if( FParse::Command(&Str,TEXT("NEW")) )
	{
		// Generalized object importing.
		EObjectFlags Flags = RF_Public|RF_Standalone;
		if( FParse::Command(&Str,TEXT("STANDALONE")) )
		{
			Flags = RF_Public|RF_Standalone;
		}
		else if( FParse::Command(&Str,TEXT("PUBLIC")) )
		{
			Flags = RF_Public;
		}
		else if( FParse::Command(&Str,TEXT("PRIVATE")) )
		{
			Flags = RF_NoFlags;
		}

		const FString ClassName     = FParse::Token(Str,0);
		UClass* Class         = UClass::TryFindTypeSlow<UClass>(ClassName);
		if( !Class )
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized or missing factor class %s"), *ClassName ));
			return true;
		}

		FString  PackageName  = ParentContext ? ParentContext->GetName() : TEXT("");
		FString	 GroupName	  = TEXT("");
		FString  FileName     = TEXT("");
		FString  ObjectName   = TEXT("");
		UClass*  ContextClass = NULL;
		UObject* Context      = NULL;

		FParse::Value( Str, TEXT("Package="), PackageName );
		FParse::Value( Str, TEXT("Group="), GroupName );
		FParse::Value( Str, TEXT("File="), FileName );

		ParseObject<UClass>( Str, TEXT("ContextClass="), ContextClass, NULL );
		ParseObject( Str, TEXT("Context="), ContextClass, Context, NULL );

		if ( !FParse::Value( Str, TEXT("Name="), ObjectName ) && FileName != TEXT("") )
		{
			// Deduce object name from filename.
			ObjectName = FileName;
			for( ; ; )
			{
				int32 i=ObjectName.Find(TEXT("/"), ESearchCase::CaseSensitive);
				if( i==-1 )
				{
					i=ObjectName.Find(TEXT("\\"), ESearchCase::CaseSensitive);
				}
				if( i==-1 )
				{
					break;
				}
				ObjectName.MidInline( i+1, MAX_int32, EAllowShrinking::No);
			}
			if( ObjectName.Find(TEXT("."), ESearchCase::CaseSensitive)>=0 )
			{
				ObjectName.LeftInline( ObjectName.Find(TEXT("."), ESearchCase::CaseSensitive), EAllowShrinking::No);
			}
		}

		UFactory* Factory = NULL;
		if( Class->IsChildOf(UFactory::StaticClass()) )
		{
			Factory = NewObject<UFactory>(GetTransientPackage(), Class);
		}

		UObject* NewObject = NULL;
		bool bOperationCanceled = false;

		// Make sure the user isn't trying to create a class with a factory that doesn't
		// advertise its supported type.
		UClass* FactoryClass = Factory ? Factory->GetSupportedClass() : Class;
		if ( FactoryClass )
		{
			NewObject = UFactory::StaticImportObject
			(
				FactoryClass,
				CreatePackage(*(GroupName != TEXT("") ? (PackageName+TEXT(".")+GroupName) : PackageName)),
				*ObjectName,
				Flags,
				bOperationCanceled,
				*FileName,
				Context,
				Factory,
				Str,
				GWarn
			);
		}

		if( !NewObject && !bOperationCanceled )
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Failed factoring: %s"), InStr ));
		}

		return true;
	}
	else if( FParse::Command( &Str, TEXT("LOAD") ) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if( FParse::Command( &Str, TEXT("MESHMAP")) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if( FParse::Command(&Str,TEXT("ANIM")) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if( FParse::Command(&Str,TEXT("MESH")) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if( FParse::Command( &Str, TEXT("AUDIO")) )
	{
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"),FText::FromString(FullStr)) );
	}
	else if ( FParse::Command( &Str, TEXT("DumpThumbnailStats") ) )
	{
		bool bShowImageData = FParse::Command(&Str, TEXT("ShowImageData"));
		FArchiveCountMem UncompressedArc(NULL), CompressedArc(NULL);
		int32 TotalThumbnailCount=0, UncompressedThumbnailCount=0;
		int32 PackagesWithUncompressedThumbnails=0;
		SIZE_T SizeOfNames=0;

		SIZE_T TotalKB = 0;
		for ( TObjectIterator<UPackage> PackageIt; PackageIt; ++PackageIt )
		{

			UPackage* Pkg = *PackageIt;
			if ( Pkg->HasThumbnailMap() )
			{

				FThumbnailMap& Thumbs = Pkg->AccessThumbnailMap();
				FArchiveCountMem MemArc(NULL);
				MemArc << Thumbs;


				SIZE_T PkgThumbnailFootprint = MemArc.GetMax() / 1024;
				Ar.Logf(TEXT("Pkg %s has %i thumbnails (%i KB)"), *Pkg->GetName(), Thumbs.Num(), PkgThumbnailFootprint);

				TotalThumbnailCount += Thumbs.Num();
				TotalKB += PkgThumbnailFootprint;

				if ( bShowImageData )
				{
					bool bHasUncompressedImageData = false;
					for ( TMap<FName,FObjectThumbnail>::TIterator ThumbnailIt(Thumbs); ThumbnailIt; ++ThumbnailIt )
					{
						FName& ThumbName = ThumbnailIt.Key();

						FObjectThumbnail& ThumbData = ThumbnailIt.Value();
						ThumbData.CountImageBytes_Uncompressed(UncompressedArc);
						ThumbData.CountImageBytes_Compressed(CompressedArc);

						TArray<uint8>& UncompressedData = ThumbData.AccessImageData();
						if ( UncompressedData.Num() > 0 )
						{
							bHasUncompressedImageData = true;
							UncompressedThumbnailCount++;
						}
					}

					if ( bHasUncompressedImageData )
					{
						PackagesWithUncompressedThumbnails++;
					}
				}
			}
		}

		if ( bShowImageData )
		{
			SIZE_T UncompressedImageSize = UncompressedArc.GetMax() / 1024;
			SIZE_T CompressedImageSize = CompressedArc.GetMax() / 1024;

			Ar.Log(TEXT("Total size of image data:"));
			Ar.Logf(TEXT("%i total thumbnails (%i uncompressed) across %i packages"), TotalThumbnailCount, UncompressedThumbnailCount, PackagesWithUncompressedThumbnails);
			Ar.Logf(TEXT("Total size of compressed image data: %i KB"), CompressedImageSize);
			Ar.Logf(TEXT("Total size of UNcompressed image data: %i KB"), UncompressedImageSize);
		}
		Ar.Logf(TEXT("Total memory required for all package thumbnails: %i KB"), TotalKB);
		return true;
	}
	return false;
}

/*-----------------------------------------------------------------------------
	UnrealEd command line.
-----------------------------------------------------------------------------*/

//@hack: this needs to be cleaned up!
static const TCHAR* GStream = NULL;

bool UEditorEngine::Exec_StaticMesh( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	bool bResult = false;
#if !UE_BUILD_SHIPPING
	// Not supported on shipped builds because PC cooking strips raw mesh data.
	ABrush* WorldBrush = InWorld->GetDefaultBrush();
	if(FParse::Command(&Str,TEXT("TO")))
	{
		if(FParse::Command(&Str,TEXT("BRUSH")))
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "StaticMeshToBrush", "StaticMesh to Brush") );
			WorldBrush->Brush->Modify(false);

			// Find the first selected static mesh actor.
			AStaticMeshActor* SelectedActor = NULL;
			for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>( Actor );
				if( StaticMeshActor )
				{
					SelectedActor = StaticMeshActor;
					break;
				}
			}

			if(SelectedActor)
			{
				WorldBrush->SetActorLocation(SelectedActor->GetActorLocation(), false);
				SelectedActor->SetActorLocation(FVector::ZeroVector, false);

				CreateModelFromStaticMesh(WorldBrush->Brush,SelectedActor);

				SelectedActor->SetActorLocation(WorldBrush->GetActorLocation(), false);
			}
			else
			{
				Ar.Logf(TEXT("No suitable actors found."));
			}

			RedrawLevelEditingViewports();
			bResult = true;
		}
	}
	else if( FParse::Command(&Str,TEXT("DEFAULT")) )	// STATICMESH DEFAULT NAME=<name>
	{
		GetSelectedObjects()->DeselectAll( UStaticMesh::StaticClass() );
		UStaticMesh* StaticMesh = NULL;
		bResult = ParseObject<UStaticMesh>(Str,TEXT("NAME="), StaticMesh, nullptr);
		if( bResult && StaticMesh)
		{
			GetSelectedObjects()->Select( StaticMesh );
		}
	}
#endif // UE_BUILD_SHIPPING
	return bResult;
}

void UEditorEngine::LoadAndSelectAssets( TArray<FAssetData>& Assets, UClass* TypeOfAsset )
{
	USelection* EditorSelection = GetSelectedObjects();
	if ( EditorSelection != NULL )
	{
		EditorSelection->BeginBatchSelectOperation();
		for ( int32 CurrentAssetIndex=0; CurrentAssetIndex < Assets.Num(); CurrentAssetIndex++ )
		{
			FAssetData& SelectedAsset = Assets[CurrentAssetIndex];
			if ( TypeOfAsset == NULL || SelectedAsset.IsInstanceOf( TypeOfAsset ) )
			{
				// GetAsset() will load the asset if necessary
				UObject* LoadedAsset = SelectedAsset.GetAsset();

				EditorSelection->Select( LoadedAsset );
			}
		}
		EditorSelection->EndBatchSelectOperation();
	}
}

bool UEditorEngine::UsePercentageBasedScaling() const
{
	return GetDefault<ULevelEditorViewportSettings>()->UsePercentageBasedScaling();
}

bool UEditorEngine::Exec_Brush( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	// Keep a pointer to the beginning of the string to use for message displaying purposes
	const TCHAR* const FullStr = Str;
	ABrush* WorldBrush = InWorld->GetDefaultBrush();
	if( FParse::Command(&Str,TEXT("APPLYTRANSFORM")) )
	{
		CommandIsDeprecated( TEXT("APPLYTRANSFORM"), Ar );
		return false;
	}
	else if( FParse::Command(&Str,TEXT("SET")) )
	{
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushSet", "Brush Set") );
			FRotator Temp(0.0f, 0.0f, 0.0f);
			FVector SnapLocation(0.0f, 0.0f, 0.0f);
			FVector PrePivot(0.0f, 0.0f, 0.0f);
			ABrush* DefaultBrush = InWorld->GetDefaultBrush();
			if (DefaultBrush != NULL)
			{
				DefaultBrush->Brush->Modify(false);
				SnapLocation = DefaultBrush->GetActorLocation();
				PrePivot = DefaultBrush->GetPivotOffset();
			}

			FSnappingUtils::SnapToBSPVertex( SnapLocation, FVector::ZeroVector, Temp );

			WorldBrush->SetActorLocation(SnapLocation - PrePivot, false);
			WorldBrush->SetPivotOffset( FVector::ZeroVector );
			WorldBrush->Brush->Polys->Element.Empty();
			UPolysFactory* It = NewObject<UPolysFactory>();
			It->FactoryCreateText( UPolys::StaticClass(), WorldBrush->Brush->Polys->GetOuter(), *WorldBrush->Brush->Polys->GetName(), RF_NoFlags, WorldBrush->Brush->Polys, TEXT("t3d"), GStream, GStream+FCString::Strlen(GStream), GWarn );
			// Do NOT merge faces.
			FBSPOps::bspValidateBrush( WorldBrush->Brush, 0, 1 );
			WorldBrush->Brush->BuildBound();
		}
		NoteSelectionChange();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("RESET")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushReset", "Brush Reset") );
		WorldBrush->Modify(false);
		WorldBrush->InitPosRotScale();
		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SCALE")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushScale", "Brush Scale") );

		FVector Scale;
		GetFVECTOR( Str, Scale );
		if( !Scale.X ) Scale.X = 1.f;
		if( !Scale.Y ) Scale.Y = 1.f;
		if( !Scale.Z ) Scale.Z = 1.f;

		const FVector InvScale( 1.f / Scale.X, 1.f / Scale.Y, 1.f / Scale.Z );

		// Fire ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied		LevelDirtyCallback;

		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			ABrush* Brush = Cast< ABrush >( Actor );
			if( Brush )
			{
				if ( Brush->Brush )
				{
					Brush->Brush->Modify(false);
					for( int32 poly = 0 ; poly < Brush->Brush->Polys->Element.Num() ; poly++ )
					{
						FPoly* Poly = &(Brush->Brush->Polys->Element[poly]);

						Poly->TextureU *= (FVector3f)InvScale;
						Poly->TextureV *= (FVector3f)InvScale;
						Poly->Base = ((Poly->Base - FVector3f(Brush->GetPivotOffset() * Scale))) + (FVector3f)Brush->GetPivotOffset();

						for( int32 vtx = 0 ; vtx < Poly->Vertices.Num() ; vtx++ )
						{
							Poly->Vertices[vtx] = (Poly->Vertices[vtx] - FVector3f(Brush->GetPivotOffset() * Scale)) + (FVector3f)Brush->GetPivotOffset();
						}

						Poly->CalcNormal();
					}

					Brush->Brush->BuildBound();

					Brush->MarkPackageDirty();
					LevelDirtyCallback.Request();
				}
			}
		}

		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("MOVETO")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushMoveTo", "Brush MoveTo") );
		WorldBrush->Modify(false);
		FVector TempVector(0.f);
		GetFVECTOR( Str, TempVector );
		WorldBrush->SetActorLocation(TempVector, false);
		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("MOVEREL")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushMoveRel", "Brush MoveRel") );
		WorldBrush->Modify(false);
		FVector TempVector( 0, 0, 0 );
		GetFVECTOR( Str, TempVector );
		FVector NewLocation = WorldBrush->GetActorLocation();
		NewLocation.AddBounded( TempVector, HALF_WORLD_MAX1 );
		WorldBrush->SetActorLocation(NewLocation, false);
		RedrawLevelEditingViewports();
		return true;
	}
	else if (FParse::Command(&Str,TEXT("ADD")))
	{
		ABrush* NewBrush = NULL;
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushAdd", "Brush Add") );
			FinishAllSnaps();
			int32 DWord1=0;
			FParse::Value( Str, TEXT("FLAGS="), DWord1 );
			NewBrush = FBSPOps::csgAddOperation( WorldBrush, DWord1, Brush_Add );
			if( NewBrush )
			{
				if( NewBrush->GetBrushBuilder() )
				{
					FActorLabelUtilities::SetActorLabelUnique(NewBrush, FText::Format(NSLOCTEXT("UnrealEd", "BrushName", "{0} Brush"), FText::FromString(NewBrush->GetBrushBuilder()->GetClass()->GetDescription())).ToString());
				}

				// Materials selected in the Content Browser, but not actually loaded, will not be
				// in the global selection set, which is expected by bspBrushCSG when it comes to
				// applying the material to the surfaces. This goes through the set of objects selected
				// in the primary content browser and, if it is a material type, ensures it is loaded
				// and selected ready for use.
				{
					TArray<FAssetData> SelectedAssets;
					{
						FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
						ContentBrowserModule.Get().GetSelectedAssets(SelectedAssets);
					}
					LoadAndSelectAssets( SelectedAssets, UMaterial::StaticClass() );
				}

				InWorld->GetModel()->Modify(false);
				NewBrush->Modify(false);
				bspBrushCSG( NewBrush, InWorld->GetModel(), DWord1, Brush_Add, CSG_None, true, true, true );

				if (FParse::Command(&Str, TEXT("SELECTNEWBRUSH")))
				{
					SelectNone(false, true);
					SelectActor(NewBrush, true, true);
				}
			}
			InWorld->InvalidateModelGeometry( InWorld->GetCurrentLevel() );
		}

		InWorld->GetCurrentLevel()->UpdateModelComponents();
		RedrawLevelEditingViewports();
		if ( NewBrush )
		{
			ULevel::LevelDirtiedEvent.Broadcast();
			RebuildStaticNavigableGeometry(InWorld->GetCurrentLevel());
		}

		return true;
	}
	else if (FParse::Command(&Str,TEXT("ADDVOLUME"))) // BRUSH ADDVOLUME
	{
		AVolume* Actor = nullptr;
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushAddVolume", "Brush AddVolume") );
			FinishAllSnaps();

			FString VolumeClassName;
			FParse::Value(Str, TEXT("CLASS="), VolumeClassName);
			UClass* VolumeClass = UClass::TryFindTypeSlow<UClass>(VolumeClassName);
			if( !VolumeClass || !VolumeClass->IsChildOf(AVolume::StaticClass()) )
			{
				VolumeClass = AVolume::StaticClass();
			}

			FVector SpawnLoc = WorldBrush->GetActorLocation();
			Actor = InWorld->SpawnActor<AVolume>( VolumeClass, SpawnLoc, FRotator::ZeroRotator );
			if( Actor )
			{
				Actor->PreEditChange(nullptr);

				FBSPOps::csgCopyBrush
				(
					Actor,
					WorldBrush,
					0,
					RF_Transactional,
					1,
					true
				);

				// Set the texture on all polys to NULL.  This stops invisible texture
				// dependencies from being formed on volumes.
				if( Actor->Brush )
				{
					for( int32 poly = 0 ; poly < Actor->Brush->Polys->Element.Num() ; ++poly )
					{
						FPoly* Poly = &(Actor->Brush->Polys->Element[poly]);
						Poly->Material = nullptr;
					}
				}
				Actor->PostEditChange();
			}
		}

		RedrawLevelEditingViewports();
		if ( Actor )
		{
			ULevel::LevelDirtiedEvent.Broadcast();
			InWorld->BroadcastLevelsChanged();
		}
		return true;
	}
	else if (FParse::Command(&Str,TEXT("SUBTRACT"))) // BRUSH SUBTRACT
	{
		ABrush* NewBrush = NULL;
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushSubtract", "Brush Subtract") );
			FinishAllSnaps();
			NewBrush = FBSPOps::csgAddOperation(WorldBrush,0,Brush_Subtract); // Layer
			if( NewBrush )
			{
				if( NewBrush->GetBrushBuilder() )
				{
					FActorLabelUtilities::SetActorLabelUnique(NewBrush, FText::Format(NSLOCTEXT("UnrealEd", "BrushName", "{0} Brush"), FText::FromString(NewBrush->GetBrushBuilder()->GetClass()->GetDescription())).ToString());
				}

				NewBrush->Modify(false);
				InWorld->GetModel()->Modify(false);
				bspBrushCSG( NewBrush, InWorld->GetModel(), 0, Brush_Subtract, CSG_None, true, true, true );
			}
			InWorld->InvalidateModelGeometry( InWorld->GetCurrentLevel() );
		}

		InWorld->GetCurrentLevel()->UpdateModelComponents();
		RedrawLevelEditingViewports();
		if ( NewBrush )
		{
			ULevel::LevelDirtiedEvent.Broadcast();
			RebuildStaticNavigableGeometry(InWorld->GetCurrentLevel());
		}

		if(FParse::Command(&Str,TEXT("SELECTNEWBRUSH")))
		{
			SelectNone(false, true);
			SelectActor(NewBrush, true, true);
		}

		return true;
	}
	else if (FParse::Command(&Str,TEXT("FROM"))) // BRUSH FROM INTERSECTION/DEINTERSECTION
	{
		if( FParse::Command(&Str,TEXT("INTERSECTION")) )
		{
			Ar.Log( TEXT("Brush from intersection") );
			{
				if( FParse::Command(&Str,TEXT("NOTRANSACTION")) )
				{
					BSPIntersectionHelper(InWorld, CSG_Intersect);
				}
				else
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushFromIntersection", "Brush From Intersection") );
					BSPIntersectionHelper(InWorld, CSG_Intersect);
				}
			}
			WorldBrush->ReregisterAllComponents();

			GLevelEditorModeTools().MapChangeNotify();
			RedrawLevelEditingViewports();
			return true;
		}
		else if( FParse::Command(&Str,TEXT("DEINTERSECTION")) )
		{
			Ar.Log( TEXT("Brush from deintersection") );
			{
				if( FParse::Command(&Str,TEXT("NOTRANSACTION")) )
				{
					BSPIntersectionHelper(InWorld, CSG_Deintersect);
				}
				else
				{
					const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushFromDeintersection", "Brush From Deintersection") );
					BSPIntersectionHelper(InWorld, CSG_Deintersect);
				}
			}
			WorldBrush->ReregisterAllComponents();

			GLevelEditorModeTools().MapChangeNotify();
			RedrawLevelEditingViewports();
			return true;
		}
	}
	else if( FParse::Command (&Str,TEXT("NEW")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushNew", "Brush New") );
		WorldBrush->Brush->Modify(false);
		WorldBrush->Brush->Polys->Element.Empty();
		RedrawLevelEditingViewports();
		return true;
	}
	else if( FParse::Command (&Str,TEXT("LOAD")) ) // BRUSH LOAD
	{
		FString TempFname;
		if( FParse::Value( Str, TEXT("FILE="), TempFname ) )
		{
			const FScopedBusyCursor BusyCursor;

			ResetTransaction( NSLOCTEXT("UnrealEd", "LoadingBrush", "Loading Brush") );
			const FVector TempVector = WorldBrush->GetActorLocation();
			LoadPackage( InWorld->GetOutermost(), *TempFname, 0 );
			WorldBrush->SetActorLocation(TempVector, false);
			FBSPOps::bspValidateBrush( WorldBrush->Brush, 0, 1 );
			Cleanse( false, 1, NSLOCTEXT("UnrealEd", "LoadingBrush", "Loading Brush") );
			return true;
		}
	}
	else if( FParse::Command( &Str, TEXT("SAVE") ) )
	{
		FString TempFname;
		if( FParse::Value(Str,TEXT("FILE="),TempFname) )
		{
			Ar.Logf( TEXT("Saving %s"), *TempFname );
			check(InWorld);
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_NoFlags;
			SaveArgs.Error = GWarn;
			this->SavePackage(WorldBrush->Brush->GetOutermost(), WorldBrush->Brush, *TempFname, SaveArgs);
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log(*NSLOCTEXT("UnrealEd", "MissingFilename", "Missing filename").ToString() ));
		}
		return true;
	}
	else if( FParse::Command( &Str, TEXT("IMPORT")) )
	{
		FString TempFname;
		if( FParse::Value(Str,TEXT("FILE="),TempFname) )
		{
			const FScopedBusyCursor BusyCursor;
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushImport", "Brush Import") );

			GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "ImportingBrush", "Importing brush"), true );

			WorldBrush->Brush->Polys->Modify(false);
			WorldBrush->Brush->Polys->Element.Empty();
			uint32 Flags=0;
			bool Merge=0;
			FParse::Bool( Str, TEXT("MERGE="), Merge );
			FParse::Value( Str, TEXT("FLAGS="), Flags );
			WorldBrush->Brush->Linked = 0;
			ImportObject<UPolys>( WorldBrush->Brush->Polys->GetOuter(), *WorldBrush->Brush->Polys->GetName(), RF_NoFlags, *TempFname );
			if( Flags )
			{
				for( int16 Word2=0; Word2<TempModel->Polys->Element.Num(); Word2++ )
				{
					WorldBrush->Brush->Polys->Element[Word2].PolyFlags |= Flags;
				}
			}
			for( int32 i=0; i<WorldBrush->Brush->Polys->Element.Num(); i++ )
			{
				WorldBrush->Brush->Polys->Element[i].iLink = i;
			}
			if( Merge )
			{
				bspMergeCoplanars( WorldBrush->Brush, 0, 1 );
				FBSPOps::bspValidateBrush( WorldBrush->Brush, 0, 1 );
			}
			WorldBrush->ReregisterAllComponents();
			GWarn->EndSlowTask();
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log( TEXT("Missing filename") ));
		}
		return true;
	}
	else if (FParse::Command(&Str,TEXT("EXPORT")))
	{
		FString TempFname;
		if( FParse::Value(Str,TEXT("FILE="),TempFname) )
		{
			const FScopedBusyCursor BusyCursor;

			GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "ExportingBrush", "Exporting brush"), true );
			UExporter::ExportToFile( WorldBrush->Brush->Polys, NULL, *TempFname, 0 );
			GWarn->EndSlowTask();
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log( TEXT("Missing filename") ));
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("MERGEPOLYS")) ) // BRUSH MERGEPOLYS
	{
		const FScopedBusyCursor BusyCursor;

		// Merges the polys on all selected brushes
		GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "MergePolys", "Merge polys"), true );
		const int32 ProgressDenominator = InWorld->GetProgressDenominator();

		// Fire ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied		LevelDirtyCallback;

		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );
			ABrush* Brush = Cast< ABrush >( Actor );
			if ( Brush )
			{
				FBSPOps::bspValidateBrush( Brush->Brush, 1, 1 );
				Brush->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
		}
		RedrawLevelEditingViewports();
		GWarn->EndSlowTask();
	}
	else if( FParse::Command(&Str,TEXT("SEPARATEPOLYS")) ) // BRUSH SEPARATEPOLYS
	{
		const FScopedBusyCursor BusyCursor;

		GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "SeparatePolys", "Separate polys"),  true );
		const int32 ProgressDenominator = InWorld->GetProgressDenominator();

		// Fire ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied		LevelDirtyCallback;

		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );
			ABrush* Brush = Cast< ABrush >( Actor );
			if ( Brush )
			{
				FBSPOps::bspUnlinkPolys( Brush->Brush );
				Brush->MarkPackageDirty();
				LevelDirtyCallback.Request();
			}
		}
		RedrawLevelEditingViewports();
		GWarn->EndSlowTask();
	}

	return false;
}

int32 UEditorEngine::BeginTransaction(const TCHAR* TransactionContext, const FText& Description, UObject* PrimaryObject)
{
	int32 Index = INDEX_NONE;

	if (CanTransact())
	{
		// generate transaction context
		Index = Trans->Begin(TransactionContext, Description);
		Trans->SetPrimaryUndoObject(PrimaryObject);
	}
	return Index;
}

int32 UEditorEngine::BeginTransaction(const FText& Description)
{
	return BeginTransaction(nullptr, Description, nullptr);
}

bool UEditorEngine::CanTransact()
{
	// we can transact if we have a transaction buffer and aren't currently loading packages or  routing postload.
	// No transaction should be created during loading
	return Trans != nullptr &&
		!UE::GetIsEditorLoadingPackage() &&
		!FUObjectThreadContext::Get().IsRoutingPostLoad;
}

int32 UEditorEngine::EndTransaction()
{
	int32 Index = INDEX_NONE;

	if (Trans)
	{
		Index = Trans->End();
	}

	return Index;
}

void UEditorEngine::CancelTransaction(int32 Index)
{
	if (Trans)
	{
		Trans->Cancel( Index );
	}
}

void UEditorEngine::ResetTransaction(const FText& Reason)
{
	if (Trans)
	{
		Trans->Reset(Reason);
	}
}

void UEditorEngine::ShowUndoRedoNotification(const FText& NotificationText, bool bSuccess)
{
	// Add a new notification item only if the previous one has expired or is otherwise done fading out (CS_None). This way multiple undo/redo notifications do not pollute the notification window.
	if(!UndoRedoNotificationItem.IsValid() || UndoRedoNotificationItem->GetCompletionState() == SNotificationItem::CS_None)
	{
		FNotificationInfo Info( NotificationText );
		Info.bUseLargeFont = false;
		Info.bUseSuccessFailIcons = false;

		UndoRedoNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	}

	if ( UndoRedoNotificationItem.IsValid() )
	{
		// Update the text and completion state to reflect current info
		UndoRedoNotificationItem->SetText( NotificationText );
		UndoRedoNotificationItem->SetCompletionState( bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail );

		// Restart the fade animation for the current undo/redo notification
		UndoRedoNotificationItem->ExpireAndFadeout();
	}
}

void UEditorEngine::HandleTransactorBeforeRedoUndo(const FTransactionContext& TransactionContext)
{
	// Before an undo, store the current operation and hook on object transaction, if we do not have an outer operation already
	if (CurrentUndoRedoContext->OperationDepth++ == 0)
	{
		check(!CurrentUndoRedoContext->OuterOperationId.IsValid());
		CurrentUndoRedoContext->OuterOperationId = TransactionContext.OperationId;
		FCoreUObjectDelegates::OnObjectTransacted.AddUObject(this, &UEditorEngine::HandleObjectTransacted);
	}
}

void UEditorEngine::HandleTransactorRedoUndo(const FTransactionContext& TransactionContext, bool Succeeded, bool WasUndo)
{
	if (!bIgnoreSelectionChange)
	{
		NoteSelectionChange(bNotifyUndoRedoSelectionChange);
	}
	PostUndo(Succeeded);

	// Broadcast only if you have an actual transaction context
	if (Succeeded)
	{
		check(CurrentUndoRedoContext->OuterOperationId.IsValid() && CurrentUndoRedoContext->OperationDepth > 0);
		if (!bSuspendBroadcastPostUndoRedo)
		{
			BroadcastPostUndoRedo(TransactionContext, WasUndo);
		}

		if (--CurrentUndoRedoContext->OperationDepth == 0)
		{
			// Undo/Redo is done clear out operation
			check(CurrentUndoRedoContext->OuterOperationId == TransactionContext.OperationId);
			CurrentUndoRedoContext->Reset();
			FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
		}
	}

	if (!bSquelchTransactionNotification
		// We skip this notification if we failed and there's nothing valid to display. Currently this can occur because
		// UTransBuffer::Undo/Redo issues broadcasts even when there wasn't a transaction to Undo/Redo, but in these cases
		// Succeeded is set to false and the context is invalid.
		&& !(!Succeeded && TransactionContext.Title.IsEmpty() && !TransactionContext.IsValid()))
	{
		const FText UndoRedoMessage = WasUndo ? NSLOCTEXT("UnrealEd", "UndoMessageFormat", "Undo: {0}") : NSLOCTEXT("UnrealEd", "RedoMessageFormat", "Redo: {0}");
		ShowUndoRedoNotification(FText::Format(UndoRedoMessage, TransactionContext.Title), Succeeded);
	}
}

void UEditorEngine::HandleTransactorRedo(const FTransactionContext& TransactionContext, bool Succeeded)
{
	HandleTransactorRedoUndo(TransactionContext, Succeeded, /*WasUndo*/false);
}

void UEditorEngine::HandleTransactorUndo(const FTransactionContext& TransactionContext, bool Succeeded)
{
	HandleTransactorRedoUndo(TransactionContext, Succeeded, /*WasUndo*/true);
}

void UEditorEngine::HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionObjectEvent)
{
	check(CurrentUndoRedoContext->OuterOperationId.IsValid() && CurrentUndoRedoContext->OperationDepth > 0);
	if (InTransactionObjectEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		CurrentUndoRedoContext->TransactionObjects.Add(TPair<UObject*, FTransactionObjectEvent>{ InObject, InTransactionObjectEvent });
	}
}

bool UEditorEngine::AreEditorAnalyticsEnabled() const
{
	return GetDefault<UAnalyticsPrivacySettings>()->bSendUsageData;
}

void UEditorEngine::CreateStartupAnalyticsAttributes( TArray<FAnalyticsEventAttribute>& StartSessionAttributes ) const
{
	Super::CreateStartupAnalyticsAttributes( StartSessionAttributes );

	ILauncherPlatform* LauncherPlatform = FLauncherPlatformModule::Get();
	if(LauncherPlatform != nullptr)
	{
		// If this is false, CanOpenLauncher will only return true if the launcher is already installed on the users machine
		const bool bIncludeLauncherInstaller = false;

		bool bIsLauncherInstalled = LauncherPlatform->CanOpenLauncher(bIncludeLauncherInstaller);
		StartSessionAttributes.Add(FAnalyticsEventAttribute(TEXT("IsLauncherInstalled"), bIsLauncherInstalled));
	}
}

UTransactor* UEditorEngine::CreateTrans()
{
	CurrentUndoRedoContext = MakePimpl<FEditorTransactionDeltaContext>();

	int32 UndoBufferSize;

	if (!GConfig->GetInt(TEXT("Undo"), TEXT("UndoBufferSize"), UndoBufferSize, GEditorPerProjectIni))
	{
		UndoBufferSize = 256;
	}

	UE_LOGF(LogInit, Log, "Undo buffer set to %d MB", UndoBufferSize);

	UTransBuffer* TransBuffer = NewObject<UTransBuffer>();
	TransBuffer->Initialize((SIZE_T)UndoBufferSize * 1024 * 1024);
	TransBuffer->OnBeforeRedoUndo().AddUObject(this, &UEditorEngine::HandleTransactorBeforeRedoUndo);
	TransBuffer->OnRedo().AddUObject(this, &UEditorEngine::HandleTransactorRedo);
	TransBuffer->OnUndo().AddUObject(this, &UEditorEngine::HandleTransactorUndo);

	return TransBuffer;
}

void UEditorEngine::PostUndo(bool)
{
	UTypedElementSelectionSet* LevelEditorSelection = GetSelectedActors()->GetElementSelectionSet();

	// This is a safeguard in case the function call is triggered after shutdown has initiated
	if (LevelEditorSelection)
	{
		// Cache any Actor that needs to be re-instanced because it still points to a REINST_ class
		TMap< UClass*, UClass* > OldToNewClassMapToReinstance;
		LevelEditorSelection->ForEachSelectedObject<AActor>([&OldToNewClassMapToReinstance](AActor* InActor)
			{
				// If the Actor's Class is not the AuthoritativeClass, then it needs to be re-instanced
				UClass* OldClass = InActor->GetClass();
				if (OldClass->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					UClass* NewClass = OldClass->GetAuthoritativeClass();
					if (!ensure(NewClass != OldClass))
					{
						UE_LOGF(LogActor, Warning, "WARNING: %ls is out of date and is the same as its AuthoritativeClass during PostUndo!", *OldClass->GetName());
					};

					OldToNewClassMapToReinstance.Add(OldClass, NewClass);
				}
				return true;
			});

		// Re-instance any actors that need it
		FBlueprintCompileReinstancer::BatchReplaceInstancesOfClass(OldToNewClassMapToReinstance, FReplaceInstancesOfClassParameters());
	}

	RedrawLevelEditingViewports();
}

bool UEditorEngine::UndoTransaction(bool bCanRedo)
{
	// make sure we're in a valid state to perform this
	if (UE::IsSavingPackage() || IsGarbageCollecting())
	{
		return false;
	}

	return Trans && Trans->Undo(bCanRedo);
}

bool UEditorEngine::RedoTransaction()
{
	// make sure we're in a valid state to perform this
	if (UE::IsSavingPackage() || IsGarbageCollecting())
	{
		return false;
	}

	return Trans && Trans->Redo();
}

bool UEditorEngine::IsTransactionActive() const
{
	return Trans && Trans->IsActive();
}

FText UEditorEngine::GetTransactionName() const
{
	return Trans ? Trans->GetUndoContext(false).Title : FText();
}

bool UEditorEngine::IsObjectInTransactionBuffer( const UObject* Object ) const
{
	return Trans && Trans->IsObjectInTransactionBuffer(Object);
}

bool UEditorEngine::Map_Select( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar)
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectBrushes", "Select Brushes") );

	GetSelectedActors()->BeginBatchSelectOperation();
	GetSelectedActors()->Modify();

	SelectNone( false, true );

	if( FParse::Command(&Str,TEXT("ADDS")) )
	{
		MapSelectOperation( InWorld, Brush_Add );
	}
	else if( FParse::Command(&Str,TEXT("SUBTRACTS")) )
	{
		MapSelectOperation( InWorld, Brush_Subtract );
	}
	else if( FParse::Command(&Str,TEXT("SEMISOLIDS")) )
	{
		MapSelectFlags( InWorld, PF_Semisolid );
	}
	else if( FParse::Command(&Str,TEXT("NONSOLIDS")) )
	{
		MapSelectFlags( InWorld, PF_NotSolid );
	}

	GetSelectedActors()->EndBatchSelectOperation();
	NoteSelectionChange();

	RedrawLevelEditingViewports();

	return true;
}

bool UEditorEngine::Map_Brush(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar)
{
	bool bSuccess = false;

	if( FParse::Command (&Str,TEXT("GET")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushGet", "Brush Get") );
		GetSelectedActors()->Modify();
		MapBrushGet(InWorld);
		RedrawLevelEditingViewports();
		bSuccess = true;
	}
	else if( FParse::Command (&Str,TEXT("PUT")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "BrushPut", "Brush Put") );
		mapBrushPut();
		RedrawLevelEditingViewports();
		bSuccess = true;
	}

	return bSuccess;
}

bool UEditorEngine::Map_Sendto(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar)
{
	bool bSuccess = false;

	if( FParse::Command(&Str,TEXT("FIRST")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MapSendToFront", "Send To Front") );
		mapSendToFirst(InWorld);
		RedrawLevelEditingViewports();
		RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
		bSuccess = true;
	}
	else if( FParse::Command(&Str,TEXT("LAST")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MapSendToBack", "Send To Back") );
		mapSendToLast(InWorld);
		RedrawLevelEditingViewports();
		RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
		bSuccess = true;
	}
	else if( FParse::Command(&Str,TEXT("SWAP")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MapSwap", "Swap") );
		mapSendToSwap(InWorld);
		RedrawLevelEditingViewports();
		RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush
		bSuccess = true;
	}

	return bSuccess;
}

bool UEditorEngine::Map_Rebuild(UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar)
{
	if (ShouldAbortBecauseOfPIEWorld())
	{
		return false;
	}

	TMap<AActor*, TArray<int32>> VisibleBSPSurfaceMap;
	bool bAllVisible = false;

	if (GUnrealEd)
	{
		// Get the map of visible BSP surfaces.
		// bAllVisible will tell us if all the current geometry was visible. If any of the current geometry is hidden, we do not want any new geometry that is made during rebuild to be visible.
		// If this is true, all geometry automatically becomes visible due to reconstruction and will remain so, new geometry included.
		GUnrealEd->CreateBSPVisibilityMap(InWorld, VisibleBSPSurfaceMap, bAllVisible );
	}

	EMapRebuildType RebuildType = EMapRebuildType::MRT_Current;

	if( FParse::Command(&Str,TEXT("ALLVISIBLE")) )
	{
		RebuildType = EMapRebuildType::MRT_AllVisible;
	}
	else if( FParse::Command(&Str,TEXT("ALLDIRTYFORLIGHTING")) )
	{
		RebuildType = EMapRebuildType::MRT_AllDirtyForLighting;
	}

	RebuildMap(InWorld, RebuildType);

	//Clean BSP references afterward (artist request)
	const int32 NumReferences = CleanBSPMaterials(InWorld, false, false);
	if (NumReferences > 0)
	{
		UE_LOGF(LogEditorServer, Log, "Cleared %d NULL BSP materials after rebuild.", NumReferences);
	}

	// Not all of our geometry is visible, so we need to make any that were not before hidden. If the geometry is new, it will also be made hidden.
	if(GUnrealEd && !bAllVisible)
	{
		// Force visible any objects that were previously visible.
		GUnrealEd->MakeBSPMapVisible(VisibleBSPSurfaceMap, InWorld );
	}
	return true;
}


void UEditorEngine::RebuildMap(UWorld* InWorld, EMapRebuildType RebuildType)
{
	FlushRenderingCommands();

	ResetTransaction( NSLOCTEXT("UnrealEd", "RebuildingMap", "Rebuilding Map") );
	GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "RebuildingGeometry", "Rebuilding geometry"), false);

	if ( InWorld->IsNavigationRebuilt() )
	{
		UE_LOGF(LogEditorServer, Log, "Rebuildmap Clear paths rebuilt");
	}

	TSet<ULevel*> UpdatedLevels;

	switch (RebuildType)
	{
		case EMapRebuildType::MRT_AllVisible:
		{
			// Store old current level
			ULevel* OldCurrentLevel = InWorld->GetCurrentLevel();

			// Build CSG for the persistent level
			ULevel* Level = InWorld->PersistentLevel;
			InWorld->SetCurrentLevel( Level );
			if ( FLevelUtils::IsLevelVisible( Level ) )
			{
				csgRebuild( InWorld );
				InWorld->InvalidateModelGeometry( Level );
				Level->bGeometryDirtyForLighting = false;
				UpdatedLevels.Add( Level );
			}

			// Build CSG for all visible streaming levels
			for( int32 LevelIndex = 0; LevelIndex < InWorld->GetStreamingLevels().Num() && !GEngine->GetMapBuildCancelled(); ++LevelIndex )
			{
				ULevelStreaming* StreamingLevel = InWorld->GetStreamingLevels()[ LevelIndex ];
				if( StreamingLevel != NULL && FLevelUtils::IsStreamingLevelVisibleInEditor( StreamingLevel ) )
				{
					Level = StreamingLevel->GetLoadedLevel();
					if ( Level != NULL )
					{
						InWorld->SetCurrentLevel( Level );
						csgRebuild( InWorld );
						InWorld->InvalidateModelGeometry( Level );
						InWorld->GetCurrentLevel()->bGeometryDirtyForLighting = false;
						UpdatedLevels.Add( Level );
					}
				}
			}
			// Restore the current level
			InWorld->SetCurrentLevel( OldCurrentLevel );
		}
		break;

		case EMapRebuildType::MRT_AllDirtyForLighting:
		{
			// Store old current level
			ULevel* OldCurrent = InWorld->GetCurrentLevel();
			{
				// Build CSG for the persistent level if it's out of date
				if (InWorld->PersistentLevel->bGeometryDirtyForLighting)
				{
					ULevel* Level = InWorld->PersistentLevel;
					InWorld->SetCurrentLevel( Level );
					csgRebuild( InWorld );
					InWorld->InvalidateModelGeometry( Level );
					Level->bGeometryDirtyForLighting = false;
					UpdatedLevels.Add( Level );
				}

				// Build CSG for each streaming level that is out of date
				for( int32 LevelIndex = 0 ; LevelIndex < InWorld->GetStreamingLevels().Num() && !GEngine->GetMapBuildCancelled(); ++LevelIndex )
				{
					ULevelStreaming* StreamingLevel = InWorld->GetStreamingLevels()[ LevelIndex ];
					if( StreamingLevel != NULL )
					{
						ULevel* Level = StreamingLevel->GetLoadedLevel();
						if ( Level != NULL && Level->bGeometryDirtyForLighting )
						{
							InWorld->SetCurrentLevel( Level );
							csgRebuild( InWorld );
							InWorld->InvalidateModelGeometry( Level );
							Level->bGeometryDirtyForLighting = false;
							UpdatedLevels.Add( Level );
						}
					}
				}
			}
			// Restore the current level.
			InWorld->SetCurrentLevel( OldCurrent );
		}
		break;

		case EMapRebuildType::MRT_Current:
		{
			// Just build the current level
			csgRebuild( InWorld );
			InWorld->InvalidateModelGeometry( InWorld->GetCurrentLevel() );
			InWorld->GetCurrentLevel()->bGeometryDirtyForLighting = false;
			UpdatedLevels.Add( InWorld->GetCurrentLevel() );
		}
		break;
	}

	// See if there is any foliage that also needs to be updated
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		if (UpdatedLevels.Contains(IFA->GetLevel()))
		{
			IFA->MapRebuild();
		}
	}

	GWarn->StatusUpdate( -1, -1, NSLOCTEXT("UnrealEd", "CleaningUpE", "Cleaning up...") );

	RedrawLevelEditingViewports();

	// Building the map can cause actors be created, so trigger a notification for that
	FEditorDelegates::MapChange.Broadcast(MapChangeEventFlags::MapRebuild);
	GEngine->BroadcastLevelActorListChanged();

	GWarn->EndSlowTask();
}


void UEditorEngine::RebuildLevel(ULevel& Level)
{
	// Early out if BSP auto-updating is disabled
	if (!GetDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate)
	{
		return;
	}

	FScopedSlowTask SlowTask(2);
	SlowTask.MakeDialogDelayed(3.0f);

	SlowTask.EnterProgressFrame(1);

	// Note: most of the following code was taken from UEditorEngine::csgRebuild()
	FinishAllSnaps();
	FBSPOps::GFastRebuild = 1;

	UWorld* World = Level.OwningWorld;
	// Build CSG for the level
	World->InvalidateModelGeometry(&Level);
	FlushRenderingCommands();

	RebuildModelFromBrushes(Level.Model, false);

	Level.MarkPackageDirty();
	ULevel::LevelDirtiedEvent.Broadcast();

	// Actors in the level may have changed due to a rebuild
	GEngine->BroadcastLevelActorListChanged();

	FBSPOps::GFastRebuild = 1;

	SlowTask.EnterProgressFrame(1);
	Level.UpdateModelComponents();

	RebuildStaticNavigableGeometry(&Level);

	// See if there is any foliage that also needs to be updated
	for (TActorIterator<AInstancedFoliageActor> It(World); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		if (IFA->GetLevel() == &Level)
		{
			IFA->MapRebuild();
		}
	}

	if (!IsRunningCommandlet())
	{
		GLevelEditorModeTools().MapChangeNotify();
	}
}

void UEditorEngine::RebuildModelFromBrushes(UModel* Model, bool bSelectedBrushesOnly, bool bTreatMovableBrushesAsStatic)
{
	TUniquePtr<FBspPointsGrid> BspPoints = MakeUnique<FBspPointsGrid>(50.0f, THRESH_POINTS_ARE_SAME);
	TUniquePtr<FBspPointsGrid> BspVectors = MakeUnique<FBspPointsGrid>(1/16.0f, FMath::Max(THRESH_NORMALS_ARE_SAME, THRESH_VECTORS_ARE_NEAR));
	FBspPointsGrid::GBspPoints = BspPoints.Get();
	FBspPointsGrid::GBspVectors = BspVectors.Get();

	// Empty the model out.
	const int32 NumPoints = Model->Points.Num();
	const int32 NumNodes = Model->Nodes.Num();
	const int32 NumVerts = Model->Verts.Num();
	const int32 NumVectors = Model->Vectors.Num();
	const int32 NumSurfs = Model->Surfs.Num();

	Model->Modify(false);
	Model->EmptyModel(1, 1);

	// Reserve arrays an eighth bigger than the previous allocation
	Model->Points.Empty(NumPoints + NumPoints / 8);
	Model->Nodes.Empty(NumNodes + NumNodes / 8);
	Model->Verts.Empty(NumVerts + NumVerts / 8);
	Model->Vectors.Empty(NumVectors + NumVectors / 8);
	Model->Surfs.Empty(NumSurfs + NumSurfs / 8);

	// Limit the brushes used to the level the model is for
	ULevel* Level = Model->GetTypedOuter<ULevel>();
	if ( !Level )
	{
		// If the model doesn't have a level, use the world's current level instead.
		FWorldContext &Context = GetEditorWorldContext();
		check(Context.World() == GWorld);
		Level = Context.World()->GetCurrentLevel();
	}
	check( Level );

	// Build list of all static brushes, first structural brushes and portals
	TArray<ABrush*> StaticBrushes;
	for (auto It(Level->Actors.CreateConstIterator()); It; ++It)
	{
		ABrush* Brush = Cast<ABrush>(*It);
		if ((Brush && (Brush->IsStaticBrush() || bTreatMovableBrushesAsStatic) && !FActorEditorUtils::IsABuilderBrush(Brush)) &&
			(!bSelectedBrushesOnly || Brush->IsSelected()) &&
			(!(Brush->PolyFlags & PF_Semisolid) || (Brush->BrushType != Brush_Add) || (Brush->PolyFlags & PF_Portal)))
		{
			StaticBrushes.Add(Brush);

			// Treat portals as solids for cutting.
			if (Brush->PolyFlags & PF_Portal)
			{
				Brush->PolyFlags = (Brush->PolyFlags & ~PF_Semisolid) | PF_NotSolid;
			}
		}
	}

	// Next append all detail brushes
	for (auto It(Level->Actors.CreateConstIterator()); It; ++It)
	{
		ABrush* Brush = Cast<ABrush>(*It);
		if (Brush && Brush->IsStaticBrush() && !FActorEditorUtils::IsABuilderBrush(Brush) &&
			(!bSelectedBrushesOnly || Brush->IsSelected()) &&
			(Brush->PolyFlags & PF_Semisolid) && !(Brush->PolyFlags & PF_Portal) && (Brush->BrushType == Brush_Add))
		{
			StaticBrushes.Add(Brush);
		}
	}

	// Build list of dynamic brushes
	TArray<ABrush*> DynamicBrushes;
	if (!bTreatMovableBrushesAsStatic)
	{
		for( auto It(Level->Actors.CreateConstIterator()); It; ++It )
		{
			ABrush* DynamicBrush = Cast<ABrush>(*It);
			if (DynamicBrush && DynamicBrush->Brush && !DynamicBrush->IsStaticBrush() && DynamicBrush->IsSelected())
			{
				DynamicBrushes.Add(DynamicBrush);
			}
		}
	}

	FScopedSlowTask SlowTask(StaticBrushes.Num() + DynamicBrushes.Num());
	SlowTask.MakeDialogDelayed(3.0f);

	// Compose all static brushes
	for (ABrush* Brush : StaticBrushes)
	{
		SlowTask.EnterProgressFrame(1);
		Brush->Modify(false);
		bspBrushCSG(Brush, Model, Brush->PolyFlags, (EBrushType)Brush->BrushType, CSG_None, false, true, false, false);
	}

	// Rebuild dynamic brush BSP's (if they weren't handled earlier)
	for (ABrush* DynamicBrush : DynamicBrushes)
	{
		SlowTask.EnterProgressFrame(1);
		BspPoints = MakeUnique<FBspPointsGrid>(50.0f, THRESH_POINTS_ARE_SAME);
		BspVectors = MakeUnique<FBspPointsGrid>(1 / 16.0f, FMath::Max(THRESH_NORMALS_ARE_SAME, THRESH_VECTORS_ARE_NEAR));
		FBspPointsGrid::GBspPoints = BspPoints.Get();
		FBspPointsGrid::GBspVectors = BspVectors.Get();

		FBSPOps::csgPrepMovingBrush(DynamicBrush);
	}

	FBspPointsGrid::GBspPoints = nullptr;
	FBspPointsGrid::GBspVectors = nullptr;
}

void UEditorEngine::RebuildModelFromBrushes(TArray<ABrush*> &BrushesToBuild, UModel* Model)
{
	TUniquePtr<FBspPointsGrid> BspPoints = MakeUnique<FBspPointsGrid>(50.0f, THRESH_POINTS_ARE_SAME);
	TUniquePtr<FBspPointsGrid> BspVectors = MakeUnique<FBspPointsGrid>(1 / 16.0f, FMath::Max(THRESH_NORMALS_ARE_SAME, THRESH_VECTORS_ARE_NEAR));
	FBspPointsGrid::GBspPoints = BspPoints.Get();
	FBspPointsGrid::GBspVectors = BspVectors.Get();

	// Empty the model out.
	const int32 NumPoints = Model->Points.Num();
	const int32 NumNodes = Model->Nodes.Num();
	const int32 NumVerts = Model->Verts.Num();
	const int32 NumVectors = Model->Vectors.Num();
	const int32 NumSurfs = Model->Surfs.Num();

	Model->Modify(false);
	Model->EmptyModel(1, 1);

	// Reserve arrays an eighth bigger than the previous allocation
	Model->Points.Empty(NumPoints + NumPoints / 8);
	Model->Nodes.Empty(NumNodes + NumNodes / 8);
	Model->Verts.Empty(NumVerts + NumVerts / 8);
	Model->Vectors.Empty(NumVectors + NumVectors / 8);
	Model->Surfs.Empty(NumSurfs + NumSurfs / 8);

	FScopedSlowTask SlowTask(BrushesToBuild.Num());
	SlowTask.MakeDialogDelayed(3.0f);

	// Compose all brushes
	for (ABrush* Brush : BrushesToBuild)
	{
		SlowTask.EnterProgressFrame(1);
		Brush->Modify(false);
		bspBrushCSG(Brush, Model, Brush->PolyFlags, (EBrushType)Brush->BrushType, CSG_None, false, true, false, false);
	}

	FBspPointsGrid::GBspPoints = nullptr;
	FBspPointsGrid::GBspVectors = nullptr;
}


void UEditorEngine::RebuildAlteredBSP()
{
	if (bIsRebuildingAlteredBSP)
	{
		return;
	}

	TGuardValue<bool> GuardIsRebuildingAlteredBSP(bIsRebuildingAlteredBSP, true);

	if( !GIsTransacting )
	{
		// Early out if BSP auto-updating is disabled
		if (!GetDefault<ULevelEditorMiscSettings>()->bBSPAutoUpdate)
		{
			return;
		}

		FlushRenderingCommands();

		// A list of all the levels that need to be rebuilt
		TArray< TWeakObjectPtr< ULevel > > LevelsToRebuild;
		ABrush::NeedsRebuild(&LevelsToRebuild);

		// Determine which levels need to be rebuilt
		for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = static_cast<AActor*>(*It);
			checkSlow(Actor->IsA(AActor::StaticClass()));

			ABrush* SelectedBrush = Cast< ABrush >(Actor);
			if (SelectedBrush && !FActorEditorUtils::IsABuilderBrush(Actor))
			{
				ULevel* Level = SelectedBrush->GetLevel();
				if (Level)
				{
					LevelsToRebuild.AddUnique(Level);
				}
			}
			else
			{
				// In addition to any selected brushes, any brushes attached to a selected actor should be rebuilt
				TArray<AActor*> AttachedActors;
				Actor->GetAttachedActors(AttachedActors);

				TArray<AActor*> AttachedBrushes;
				// Get any brush actors attached to the selected actor
				if (ContainsObjectOfClass(AttachedActors, ABrush::StaticClass(), EFindObjectFlags::ExactClass, &AttachedBrushes))
				{
					for (int32 BrushIndex = 0; BrushIndex < AttachedBrushes.Num(); ++BrushIndex)
					{
						ULevel* Level = CastChecked<ABrush>(AttachedBrushes[BrushIndex])->GetLevel();
						if (Level)
						{
							LevelsToRebuild.AddUnique(Level);
						}
					}
				}

			}

		}

		// Rebuild the levels
		{
			FScopedSlowTask SlowTask(LevelsToRebuild.Num(), NSLOCTEXT("EditorServer", "RebuildingBSP", "Rebuilding BSP..."));
			SlowTask.MakeDialogDelayed(3.0f);

			for (int32 LevelIdx = 0; LevelIdx < LevelsToRebuild.Num(); ++LevelIdx)
			{
				SlowTask.EnterProgressFrame(1.0f);

				TWeakObjectPtr< ULevel > LevelToRebuild = LevelsToRebuild[LevelIdx];
				if (LevelToRebuild.IsValid())
				{
					RebuildLevel(*LevelToRebuild.Get());
				}
			}
		}

		RedrawLevelEditingViewports();

		ABrush::OnRebuildDone();
	}
	else
	{
 		ensureMsgf(0, TEXT("Rebuild BSP ignored during undo/redo") );
		ABrush::OnRebuildDone();
	}
}

void UEditorEngine::BSPIntersectionHelper(UWorld* InWorld, ECsgOper Operation)
{
	if (UBrushEditingSubsystem* BrushSubsystem = GetEditorSubsystem<UBrushEditingSubsystem>())
	{
		BrushSubsystem->DeselectAllEditingGeometry();
	}

	ABrush* DefaultBrush = InWorld->GetDefaultBrush();
	if (DefaultBrush != NULL)
	{
		DefaultBrush->Modify(false);
		InWorld->GetModel()->Modify(false);
		FinishAllSnaps();
		bspBrushCSG(DefaultBrush, InWorld->GetModel(), 0, Brush_MAX, Operation, false, true, true);
	}
}

void UEditorEngine::CheckForWorldGCLeaks( UWorld* NewWorld, UPackage* WorldPackage )
{
	int32 NumFailedToCleanup = 0;
	// Make sure the old world is completely gone, except if the new world was one of it's sublevels
	for(TObjectIterator<UWorld> It; It; ++It)
	{
		UWorld* RemainingWorld = *It;
		const bool bIsNewWorld = (NewWorld && RemainingWorld == NewWorld);
		const bool bIsPersistantWorldType = (
			RemainingWorld->WorldType == EWorldType::Inactive) ||
			(RemainingWorld->WorldType == EWorldType::EditorPreview) ||
			(RemainingWorld->WorldType == EWorldType::GamePreview);
		if(!bIsNewWorld && !bIsPersistantWorldType && !WorldHasValidContext(RemainingWorld))
		{
			FReferenceChainSearch::FindAndPrintStaleReferencesToObject(RemainingWorld, EPrintStaleReferencesOptions::Error);
			NumFailedToCleanup++;
		}
	}

	if(WorldPackage != nullptr)
	{
		UPackage* NewWorldPackage = NewWorld ? NewWorld->GetOutermost() : nullptr;
		for(TObjectIterator<UPackage> It; It; ++It)
		{
			UPackage* RemainingPackage = *It;
			const bool bIsNewWorldPackage = (NewWorldPackage && RemainingPackage == NewWorldPackage);
			if(!bIsNewWorldPackage && RemainingPackage == WorldPackage)
			{
				FReferenceChainSearch::FindAndPrintStaleReferencesToObject(RemainingPackage, EPrintStaleReferencesOptions::Error);
				NumFailedToCleanup++;
			}
		}
	}

	if (NumFailedToCleanup > 0)
	{
		if (CVarCheckForWorldGCLeaksAreFatal.GetValueOnGameThread())
		{
			UE_LOGF(LogEditorServer, Fatal, "World Memory Leaks: %d leaks objects and packages. See The output above.", NumFailedToCleanup);
		} 
		else
		{
			UE_LOGF(LogEditorServer, Error, "World Memory Leaks: %d leaks objects and packages. See The output above.", NumFailedToCleanup);
		}
	}
}

void UEditorEngine::EditorDestroyWorld( FWorldContext & Context, const FText& CleanseText, UWorld* NewWorld )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::EditorDestroyWorld);

	if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

		// Notify level editors of the map change
		LevelEditor.BroadcastMapChanged( Context.World(), EMapChangeType::TearDownWorld );
	}


	UWorld* ContextWorld = Context.World();

	if (ContextWorld == NULL )
	{
		return;		// We cannot destroy a world if the pointer is not valid
	}

	UPackage* WorldPackage = CastChecked<UPackage>(ContextWorld->GetOuter());
	if (WorldPackage == GetTransientPackage())
	{
		// Don't check if the package was properly cleaned up if we were created in the transient package
		WorldPackage = NULL;
	}

	if (ContextWorld->WorldType != EWorldType::EditorPreview && ContextWorld->WorldType != EWorldType::Inactive)
	{
		// Go away, come again never!
		ContextWorld->ClearFlags(RF_Standalone | RF_Transactional);
		ContextWorld->RemoveFromRoot();

		// If this was a memory-only world, we should inform the asset registry that this asset is going away forever.
		if (WorldPackage)
		{
			const FString PackageName = WorldPackage->GetName();
			const bool bIncludeReadOnlyRoots = false;
			if (FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots))
			{
				// Now check if the file exists on disk. If it does, it won't be "lost" when GC'd.
				if (!FPackageName::DoesPackageExist(PackageName))
				{
					// We are preparing the object for GC and there is no file on disk to reload it. Count this as a delete.
					FAssetRegistryModule::AssetDeleted(ContextWorld);
				}
			}
		}

		ContextWorld->SetFlags(RF_Transient);
	}

	if (GUnrealEd)
	{
		GUnrealEd->CurrentLODParentActor = NULL;
	}
	SelectNone( true, true );

	ContextWorld->ClearWorldComponents();
	ClearPreviewComponents();
	// Remove all active groups, they belong to a map being unloaded
	ContextWorld->ActiveGroupActors.Empty();

	// Make sure we don't have any apps open on for assets owned by the world we are closing
	CloseEditedWorldAssets(ContextWorld);

	// Stop all audio and remove references
	if (FAudioDevice* AudioDevice = ContextWorld->GetAudioDeviceRaw())
	{
		AudioDevice->Flush(ContextWorld);
	}

	// Reset the editor transform to avoid loading the new world with an offset if loading a sublevel
	if (NewWorld)
	{
		ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(NewWorld->PersistentLevel);
		if (LevelStreaming && NewWorld->PersistentLevel->bAlreadyMovedActors)
		{
			FLevelUtils::RemoveEditorTransform(LevelStreaming);
			NewWorld->PersistentLevel->bAlreadyMovedActors = false;
		}
	}

	FEditorSupportDelegates::PrepareToCleanseEditorObject.Broadcast(ContextWorld);
	for (ULevel* Level : ContextWorld->GetLevels())
	{
		if (ensureAlways(Level))
		{
			const bool bDontCreate = true;
			if (ULevelScriptBlueprint* LSBP = Level->GetLevelScriptBlueprint(bDontCreate))
			{
				// Signals that the associated LSBP is about to be unloaded.
				LSBP->ClearEditorReferences();
			}

			UWorld* LevelWorld = Level->GetTypedOuter<UWorld>();
			if (ensureAlways(LevelWorld) && LevelWorld != ContextWorld && LevelWorld != NewWorld)
			{
				FEditorSupportDelegates::PrepareToCleanseEditorObject.Broadcast(LevelWorld);
			}
		}
	}

	ContextWorld->DestroyWorld( true, NewWorld );
	Context.SetCurrentWorld(NULL);

	// Add the new world to root if it wasn't already and keep track of it so we can remove it from root later if appropriate
	bool bNewWorldAddedToRoot = false;
	if ( NewWorld )
	{
		if ( !NewWorld->IsRooted() )
		{
			NewWorld->AddToRoot();
			bNewWorldAddedToRoot = true;
		}

		// Reset the owning level to allow the old world to GC if it was a sublevel
		NewWorld->PersistentLevel->OwningWorld = NewWorld;
	}

	// Cleanse which should remove the old world which we are going to verify.
	Cleanse( true, 0, CleanseText );

	// If we added the world to the root set above, remove it now that the GC is complete.
	if ( bNewWorldAddedToRoot )
	{
		NewWorld->RemoveFromRoot();
	}

	// Verify that any pending typed elements have been cleanly destroyed from the GC above
	UTypedElementRegistry::GetInstance()->ProcessDeferredElementsToDestroy();

	CheckForWorldGCLeaks( NewWorld, WorldPackage );
}

bool UEditorEngine::ShouldAbortBecauseOfPIEWorld()
{
	// If a PIE world exists, warn the user that the PIE session will be terminated.
	if ( PlayWorld )
	{
		if( EAppReturnType::Yes == FMessageDialog::Open( EAppMsgType::YesNo, EAppReturnType::Yes, NSLOCTEXT("UnrealEd", "Prompt_ThisActionWillTerminatePIEContinue", "This action will terminate your Play In Editor session.  Continue?") ) )
		{
			// End the play world.
			EndPlayMap();
		}
		else
		{
			// User didn't want to end the PIE session -- abort the load.
			return true;
		}
	}
	return false;
}

bool UEditorEngine::ShouldAbortBecauseOfUnsavedWorld()
{
	// If an unsaved world exists that would be lost in a map transition, give the user the option to cancel a map load.

	// First check if we have a world and it is dirty
	UWorld* LevelEditorWorld = GetEditorWorldContext().World();
	if (LevelEditorWorld && LevelEditorWorld->GetOutermost()->IsDirty())
	{
		// Now check if the world is in a path that can be saved (otherwise it is in something like the transient package or temp)
		const FString PackageName = LevelEditorWorld->GetOutermost()->GetName();
		const bool bIncludeReadOnlyRoots = false;
		if ( FPackageName::IsValidLongPackageName(PackageName, bIncludeReadOnlyRoots) )
		{
			// Now check if the file exists on disk. If it does, it won't be "lost" when GC'd.
			if ( !FPackageName::DoesPackageExist(PackageName) )
			{
				// This world will be completely lost if a map transition happens. Warn the user that this is happening and ask them how to proceed.
				if (EAppReturnType::Yes != FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, FText::Format(NSLOCTEXT("UnrealEd", "Prompt_ThisActionWillDiscardWorldContinue", "The unsaved level {0} will be lost.  Continue?"), FText::FromString(LevelEditorWorld->GetName()))))
				{
					// User doesn't want to lose the world -- abort the load.
					return true;
				}
			}
		}
	}
	return false;
}

/**
 * Prompts the user to save the current map if necessary, then creates a new (blank) map.
 */
void UEditorEngine::CreateNewMapForEditing(bool bPromptUserToSave, bool bIsPartitionedWorld)
{
	// If a PIE world exists, warn the user that the PIE session will be terminated.
	// Abort if the user refuses to terminate the PIE session.
	if ( ShouldAbortBecauseOfPIEWorld() )
	{
		return;
	}

	// If there are any unsaved changes to the current level, see if the user wants to save those first.
	bool bSaveMapPackages = true;
	bool bSaveContentPackages = false;

	if( FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages) == false )
	{
		// something went wrong or the user pressed cancel.  Return to the editor so the user doesn't lose their changes
		return;
	}

	if ( ShouldAbortBecauseOfUnsavedWorld() )
	{
		return;
	}

	const FScopedBusyCursor BusyCursor;

	// Deactivate any editor modes when loading a new map
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		if (FEditorModeTools* ModeManager = LevelEditorSubsystem->GetLevelEditorModeManager())
		{
			ModeManager->DeactivateAllModes();
		}
	}

	NewMap(bIsPartitionedWorld);

	FEditorFileUtils::ResetLevelFilenames();
}

#define LOCTEXT_NAMESPACE "EditorEngine"

UWorld* UEditorEngine::NewMap(bool bIsPartitionedWorld)
{
	// If we have a PIE session kill it before creating a new map
	if (PlayWorld)
	{
		EndPlayMap();
	}

	const FScopedBusyCursor BusyCursor;

	FWorldContext &Context = GetEditorWorldContext();

	// Clear the lighting build results
	FMessageLog("LightingResults").NewPage(LOCTEXT("LightingBuildNewLogPage", "Lighting Build"));

	FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
	StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->Clear();

	// Destroy the old world if there is one
	const FText CleanseText = LOCTEXT("LoadingMap_Template", "New Map");
	EditorDestroyWorld( Context, CleanseText );

	// Create a new world
	UWorldFactory* Factory = NewObject<UWorldFactory>();
	Factory->WorldType = EWorldType::Editor;
	Factory->bCreateWorldPartition = bIsPartitionedWorld;
	Factory->bInformEngineOfWorld = true;
	Factory->FeatureLevel = DefaultWorldFeatureLevel;
	UPackage* Pkg = CreatePackage(nullptr);
	Pkg->SetPackageFlags(PKG_NewlyCreated);
	EObjectFlags Flags = RF_Public | RF_Standalone;
	UWorld* NewWorld = CastChecked<UWorld>(Factory->FactoryCreateNew(UWorld::StaticClass(), Pkg, TEXT("Untitled"), Flags, NULL, GWarn));
	Context.SetCurrentWorld(NewWorld);
	GWorld = NewWorld;
	NewWorld->AddToRoot();
	// Register components in the persistent level (current)
	NewWorld->UpdateWorldComponents(true, true);

	NoteSelectionChange();

	// Starting a new map will wipe existing actors and add some defaults actors to the scene, so we need
	// to notify other systems about this
	GEngine->BroadcastLevelActorListChanged();
	FEditorDelegates::MapChange.Broadcast(MapChangeEventFlags::NewMap);

	FMessageLog("LoadErrors").NewPage(LOCTEXT("NewMapLogPage", "New Map"));
	FEditorDelegates::DisplayLoadErrors.Broadcast();

	if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

		// Notify slate level editors of the map change
		LevelEditor.BroadcastMapChanged( NewWorld, EMapChangeType::NewMap );
	}

	// Move the brush to the origin.
	if (Context.World()->GetDefaultBrush() != NULL)
	{
		Context.World()->GetDefaultBrush()->SetActorLocation(FVector::ZeroVector, false);
	}

	// Let navigation system know we're done creating new world
	FNavigationSystem::AddNavigationSystemToWorld(*Context.World(), FNavigationSystemRunMode::EditorMode);

	// Deselect all
	SelectNone( false, true );

	// Clear the transaction buffer so the user can't remove the builder brush
	ResetTransaction( CleanseText );

	// Invalidate all the level viewport hit proxies
	RedrawLevelEditingViewports();

	return NewWorld;
}


bool UEditorEngine::PackageIsAMapFile( const TCHAR* PackageFilename, FText& OutNotMapReason )
{
	// make sure that the file is a map
	OutNotMapReason = FText::GetEmpty();
	FArchive* CheckMapPackageFile = IFileManager::Get().CreateFileReader( PackageFilename );
	if( CheckMapPackageFile )
	{
		FPackageFileSummary Summary;

#if WITH_TEXT_ARCHIVE_SUPPORT
		if (FPackageName::IsTextPackageExtension(*FPaths::GetExtension(PackageFilename)))
		{
			FJsonArchiveInputFormatter Formatter(*CheckMapPackageFile);
			FStructuredArchive Archive(Formatter);
			Archive.Open().EnterRecord() << SA_VALUE(TEXT("Summary"), Summary);
			Archive.Close();
		}
		else
#endif
		{
			(*CheckMapPackageFile) << Summary;
		}

		delete CheckMapPackageFile;

		// Check flag.
		if( ( Summary.GetPackageFlags() & PKG_ContainsMap ) == 0 )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("File"), FText::FromString( FString( PackageFilename ) ));
			OutNotMapReason = FText::Format( LOCTEXT( "FileIsAnAsset", "{File} appears to be an asset file." ),
				Arguments );
			return false;
		}

		// Validate the summary.
		if (Summary.IsFileVersionTooOld())
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("File"), FText::FromString( FString( PackageFilename ) ));
			Arguments.Add(TEXT("Version"), Summary.GetFileVersionUE().FileVersionUE4);
			Arguments.Add(TEXT("First"), VER_UE4_OLDEST_LOADABLE_PACKAGE);
			OutNotMapReason = FText::Format( LOCTEXT( "UEFileIsOlder", "{File} is an UE4 map [File:v{Version}], from an engine release no longer supported [Min:v{First}]." ),
				Arguments);
			return false;
		}

		const int32 UELicenseeVersion = Summary.GetFileVersionLicenseeUE();

		// Don't load packages that were saved with an engine version newer than the current one.
		if (Summary.IsFileVersionTooNew())
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("File"), FText::FromString( FString( PackageFilename ) ));
			Arguments.Add(TEXT("UE4Version"), Summary.GetFileVersionUE().FileVersionUE4);
			Arguments.Add(TEXT("UE5Version"), Summary.GetFileVersionUE().FileVersionUE5);
			Arguments.Add(TEXT("UE4Lastest"), GPackageFileUEVersion.FileVersionUE4);
			Arguments.Add(TEXT("UE5Lastest"), GPackageFileUEVersion.FileVersionUE5);
			OutNotMapReason = FText::Format( LOCTEXT( "UEFileIsNewerUE5", "{File} is a UE map [File:v{UE4Version}|{UE5Version}], from an engine release newer than this [Cur:v{UE4Lastest}|{UE5Lastest}]." ),
				Arguments);
			return false;
		}
		else if (UELicenseeVersion > GPackageFileLicenseeUEVersion)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("File"), FText::FromString( FString( PackageFilename ) ));
			Arguments.Add(TEXT("Version"), UELicenseeVersion);
			Arguments.Add(TEXT("Last"), GPackageFileLicenseeUEVersion);
			OutNotMapReason = FText::Format( LOCTEXT( "UEFileIsNewer", "{File} is a UE map [File:v{Version}], from an engine release newer than this [Cur:v{Last}]." ),
				Arguments);
			return false;
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE

bool UEditorEngine::Map_Load(const TCHAR* Str, FOutputDevice& Ar)
{
	// Temporary solution to avoid triggering the BPVM runaway detection when loading large maps with Verse objects.
	FBlueprintCoreDelegates::SetScriptMaximumLoopIterations(MAX_int32);
	ON_SCOPE_EXIT { FBlueprintCoreDelegates::SetScriptMaximumLoopIterations(GEngine->MaximumLoopIterationCount); };

	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorEngine::Map_Load);
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("LoadMap - %s"), Str);

	auto FindWorldInPackageOrFollowRedirector = [](UPackage*& InOutPackage)
	{
		UWorld* RetVal = nullptr;
		TArray<UObject*> PotentialWorlds;
		GetObjectsWithOuter(InOutPackage, PotentialWorlds, EGetObjectsFlags::None);
		for (auto ObjIt = PotentialWorlds.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			RetVal = Cast<UWorld>(*ObjIt);
			if (RetVal)
			{
				break;
			}
			else if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(*ObjIt))
			{
				RetVal = Cast<UWorld>(Redirector->DestinationObject);
				if (RetVal)
				{
					// Patch up the WorldType if found in the PreLoad map
					EWorldType::Type* PreLoadWorldType = UWorld::WorldTypePreLoadMap.Find(Redirector->GetOuter()->GetFName());
					if (PreLoadWorldType)
					{
						RetVal->WorldType = *PreLoadWorldType;
					}

					// If we followed a redirector also update the package pointer to the actual returned world package
					InOutPackage = RetVal->GetOutermost();
					break;
				}
			}
		}
		return RetVal;
	};

#define LOCTEXT_NAMESPACE "EditorEngine"
	// We are beginning a map load
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, true);

	FWorldContext &Context = GetEditorWorldContext();
	check(Context.World() == GWorld);

	FString TempFname;
	if( FParse::Value( Str, TEXT("FILE="), TempFname ) )
	{
		FString LongTempFname;
		if ( FPackageName::TryConvertFilenameToLongPackageName(TempFname, LongTempFname) )
		{
			// Is the new world already loaded?
			UPackage* ExistingPackage = FindPackage(nullptr, *LongTempFname);
			UWorld* ExistingWorld = nullptr;
			if (ExistingPackage)
			{
				ExistingWorld = FindWorldInPackageOrFollowRedirector(ExistingPackage);
			}

			FString UnusedAlteredPath;
			if ( ExistingWorld || FPackageName::DoesPackageExist(LongTempFname, &UnusedAlteredPath) )
			{
				FText NotMapReason;
				if( !ExistingWorld && !PackageIsAMapFile( *TempFname, NotMapReason ) )
				{
					// Map load failed
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("Reason"), NotMapReason);
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("MapLoadFailed", "Failed to load map!\n{Reason}"), Arguments));
					return false;
				}

				const FScopedBusyCursor BusyCursor;

				// Are we loading a template map that should be loaded into an untitled package?
				int32 bIsLoadingMapTemplate = 0;
				FParse::Value(Str, TEXT("TEMPLATE="), bIsLoadingMapTemplate);

				// Should we display progress while loading?
				int32 bShowProgress = 1;
				FParse::Value(Str, TEXT("SHOWPROGRESS="), bShowProgress);

				FString MapFileName = FPaths::GetCleanFilename(*TempFname);

				// Detect whether the map we are loading is a template map and alter the undo
				// readout accordingly.
				FText LocalizedLoadingMap;
				if (!bIsLoadingMapTemplate)
				{
					LocalizedLoadingMap = FText::Format( NSLOCTEXT("UnrealEd", "LoadingMap_F", "Loading map: {0}..."), FText::FromString( MapFileName ) );
				}
				else
				{
					LocalizedLoadingMap = NSLOCTEXT("UnrealEd", "LoadingMap_Template", "New Map");
				}

				// Don't show progress dialogs when loading one of our startup maps. They should load rather quickly.
				FScopedSlowTask SlowTask(100, LocalizedLoadingMap, bShowProgress != 0);
				SlowTask.MakeDialogDelayed(3.0f);

				SlowTask.EnterProgressFrame(10, FText::Format( NSLOCTEXT("UnrealEd", "LoadingMapStatus_CleaningUp", "{0} (Clearing existing world)"), LocalizedLoadingMap ));

				{
					// Clear the lighting build results
					FMessageLog("LightingResults").NewPage(LOCTEXT("LightingBuildNewLogPage", "Lighting Build"));

					FStatsViewerModule& StatsViewerModule = FModuleManager::Get().LoadModuleChecked<FStatsViewerModule>(TEXT("StatsViewer"));
					StatsViewerModule.GetPage(EStatsPage::LightingBuildInfo)->Clear();

					ResetTransaction( LocalizedLoadingMap );

					// Don't clear errors if we are loading a startup map so we can see all startup load errors
					if (!FEditorFileUtils::IsLoadingStartupMap())
					{
						FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("MapFileName"), FText::FromString( MapFileName ));
						FMessageLog("LoadErrors").NewPage( FText::Format( LOCTEXT("LoadMapLogPage", "Loading map: {MapFileName}"), Arguments ) );
					}

					// Only worlds that are uninitialized may be considered valid for re-use.
					// All other worlds need to be reloaded from disk, as a world is only initialized correctly as part of the level loading process.
					auto IsWorldValidForReuse = [](UWorld* WorldToConsider)
					{
						return !WorldToConsider->HasEverBeenInitialized();
					};

					// If we are loading the same world again (reloading) then we must not specify that we want to keep this world in memory.
					// Otherwise, try to keep the existing world in memory since there is not reason to reload it.
					UWorld* NewWorld = nullptr;
					if (ExistingWorld && IsWorldValidForReuse(ExistingWorld) && Context.World() != ExistingWorld && !bIsLoadingMapTemplate)
					{
						NewWorld = ExistingWorld;
					}
					EditorDestroyWorld( Context, LocalizedLoadingMap, NewWorld );

					// Unload all other map packages before opening a new map.
					{
						TArray<UPackage*> WorldPackages;
						for (TObjectIterator<UWorld> It; It; ++It)
						{
							// Don't unload transient or newly created worlds, nor the world we're attempting to keep-alive
							UPackage* Package = It->GetPackage();
							if (Package != GetTransientPackage() && !Package->HasAnyPackageFlags(PKG_NewlyCreated) && *It != NewWorld)
							{
								WorldPackages.AddUnique(Package);
							}
						}
						UPackageTools::UnloadPackages(WorldPackages);
					}

					// Refresh ExistingPackage and Existing World now that GC has occurred.
					ExistingPackage = FindPackage(nullptr, *LongTempFname);
					if (ExistingPackage)
					{
						ExistingWorld = FindWorldInPackageOrFollowRedirector(ExistingPackage);

						if (!ExistingWorld)
						{
							// If we have a world package without a world, try and unload the package so we can hopefully reload it again correctly from disk
							TArray<UPackage*> WorldPackages;
							WorldPackages.Add(ExistingPackage);
							UPackageTools::UnloadPackages(WorldPackages);

							ExistingPackage = FindPackage(nullptr, *LongTempFname);
						}
					}
					else
					{
						ExistingWorld = nullptr;
					}

					// If the existing world is not valid for re-use then the unload attempts above failed, and we need to
					// fatally error and dump any lingering references (like we would when unloading the main editor world).
					if ((ExistingWorld && !IsWorldValidForReuse(ExistingWorld)) || (ExistingPackage && !ExistingWorld))
					{
						int32 NumFailedToCleanup = 0;

						if (ExistingWorld)
						{
							// If this fires then something in the logic of deciding whether to keep this world
							// alive is out-of-sync with ensuring that only uninitialized worlds are kept alive
							check(ExistingWorld != NewWorld);

							FReferenceChainSearch RefChainSearch(ExistingWorld, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::PrintResults);
							UE_LOG(LogEditorServer, Error, TEXT("Old world %s not cleaned up by garbage collection while loading new map! Referenced by:") LINE_TERMINATOR TEXT("%s"), *ExistingWorld->GetPathName(), *RefChainSearch.GetRootPath());
							++NumFailedToCleanup;
						}

						if (ExistingPackage)
						{
							FReferenceChainSearch RefChainSearch(ExistingPackage, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::PrintResults);
							UE_LOG(LogEditorServer, Error, TEXT("Old level package %s not cleaned up by garbage collection while loading new map! Referenced by:") LINE_TERMINATOR TEXT("%s"), *ExistingPackage->GetPathName(), *RefChainSearch.GetRootPath());
							++NumFailedToCleanup;
						}

						if (NumFailedToCleanup > 0)
						{
							UE_LOGF(LogEditorServer, Fatal, "World Memory Leaks: %d leaks objects and packages. See The output above.", NumFailedToCleanup);
						}
					}

					SlowTask.EnterProgressFrame( 70, LocalizedLoadingMap );
				}

				// Record the name of this file to make sure we load objects in this package on top of in-memory objects in this package.
				UserOpenedFile				= TempFname;

				uint32 LoadFlags = LOAD_None;

				const int32 MAX_STREAMLVL_SIZE = 16384;  // max cmd line size (16kb)
				TCHAR StreamLvlBuf[MAX_STREAMLVL_SIZE]; //There can be a lot of streaming levels with very large path names

				if(FParse::Value(Str, TEXT("STREAMLVL="), StreamLvlBuf, UE_ARRAY_COUNT(StreamLvlBuf)))
				{
					TCHAR *ContextStr = NULL;

					TCHAR* CurStreamMap = FCString::Strtok(StreamLvlBuf, TEXT(";"), &ContextStr);

					while(CurStreamMap)
					{
						LoadPackage(NULL, CurStreamMap, LoadFlags);

						CurStreamMap = FCString::Strtok(NULL, TEXT(";"), &ContextStr);
					}
				}


				FLinkerInstancingContext WorldPackageInstancingContext;
				WorldPackageInstancingContext.AddTag(UWorld::KeepInitializedDuringLoadTag);

				UPackage* WorldPackage;
				// Load startup maps and templates into new outermost packages so that the Save function in the editor won't overwrite the original
				if (bIsLoadingMapTemplate)
				{
					FScopedSlowTask LoadScope(2);

					LoadScope.EnterProgressFrame();

					//create a package with the proper name
					WorldPackage = CreatePackage( *(MakeUniqueObjectName(NULL, UPackage::StaticClass()).ToString()));

					LoadScope.EnterProgressFrame();

					//now load the map into the package created above
					const FName WorldPackageFName = WorldPackage->GetFName();
					UWorld::WorldTypePreLoadMap.FindOrAdd(WorldPackageFName) = EWorldType::Editor;

					// SoftObjectPaths: Specific case for new maps (/Temp/Untitled) where we need to remap the AssetPath and not just the Package name because the World gets renamed (See UWorld::PostLoad)
					const FString ShortPackageName = FPackageName::GetShortName(LongTempFname);
					const FString ShortWorldPackageName = FPackageName::GetShortName(WorldPackage);
					WorldPackageInstancingContext.AddPathMapping(
						FSoftObjectPath(*WriteToString<256>(LongTempFname, TEXT("."), ShortPackageName)),
						FSoftObjectPath(*WriteToString<256>(WorldPackage->GetName(), TEXT("."), ShortWorldPackageName))
					);

					// Add remapping for the _BuiltData.uasset package file as well (assuming that the template has one)
					FName BuiltDataOriginal = FName(*WriteToString<256>(LongTempFname, TEXT("_BuiltData")));
					FName BuiltDataRemapped = FName(*WriteToString<256>(WorldPackageFName, TEXT("_BuiltData")));

					WorldPackageInstancingContext.AddPackageMapping(BuiltDataOriginal, BuiltDataRemapped);

					WorldPackage = LoadPackage( WorldPackage, *LongTempFname, LoadFlags, nullptr /* InReaderOverride */, &WorldPackageInstancingContext);
					WorldPackage->SetPackageFlags(PKG_NewlyCreated);
					UWorld::WorldTypePreLoadMap.Remove(WorldPackageFName);
				}
				else
				{
					if ( ExistingPackage )
					{
						WorldPackage = ExistingPackage;

						if (!ExistingPackage->IsFullyLoaded())
						{
							const FName WorldPackageFName = FName(*LongTempFname);
							UWorld::WorldTypePreLoadMap.FindOrAdd(WorldPackageFName) = EWorldType::Editor;
							ExistingPackage->FullyLoad();
							UWorld::WorldTypePreLoadMap.Remove(WorldPackageFName);
						}
					}
					else
					{
						//Load the map normally into a new package
						const FName WorldPackageFName = FName(*LongTempFname);
						UWorld::WorldTypePreLoadMap.FindOrAdd(WorldPackageFName) = EWorldType::Editor;
						WorldPackage = LoadPackage( NULL, *LongTempFname, LoadFlags, nullptr /* InReaderOverride */, &WorldPackageInstancingContext);
						UWorld::WorldTypePreLoadMap.Remove(WorldPackageFName);
					}
				}

				if (WorldPackage == nullptr)
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "MapPackageLoadFailed", "Failed to open map file. This is most likely because the map was saved with a newer version of the engine."));
					return false;
				}

				// Reset the opened package to nothing.
				UserOpenedFile				= FString();

				UWorld* World = FindWorldInPackageOrFollowRedirector(WorldPackage);

				if (World == nullptr)
				{
					FText Message = FText::Format(NSLOCTEXT("UnrealEd", "MapPackageFindWorldFailed", "Failed to find the world in already loaded world package {0}! See log for more details."), FText::FromString(WorldPackage->GetPathName()));
					FMessageDialog::Open(EAppMsgType::Ok, Message);

					FReferenceChainSearch RefChainSearch(WorldPackage, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::PrintResults);
					UE_LOG(LogEditorServer, Warning, TEXT("Failed to find the world in already loaded world package %s! Referenced by:") LINE_TERMINATOR TEXT("%s"), *WorldPackage->GetPathName(), *RefChainSearch.GetRootPath());
					return false;
				}

				if (Context.AudioDeviceID == INDEX_NONE && GEngine->GetAudioDeviceManager())
				{
					Context.AudioDeviceID = GEngine->GetAudioDeviceManager()->GetMainAudioDeviceID();
				}
				Context.SetCurrentWorld(World);
				GWorld = World;

				World->WorldType = EWorldType::Editor;

				Context.World()->PersistentLevel->HandleLegacyMapBuildData();

				// Parse requested feature level if supplied
				int32 FeatureLevelIndex = (int32)GMaxRHIFeatureLevel;
				FParse::Value(Str, TEXT("FEATURELEVEL="), FeatureLevelIndex);
				FeatureLevelIndex = FMath::Clamp(FeatureLevelIndex, 0, (int32)ERHIFeatureLevel::Num);
				ERHIFeatureLevel::Type FeatureLevel = (ERHIFeatureLevel::Type)FeatureLevelIndex;

				if (World->HasEverBeenInitialized())
				{
					// We do not handle reinitializing. If the World was initialized during Load, we require that it is still fully initialized
					// This should have been guaranteed by UWorld::KeepInitializedDuringLoadTag
					check(World->IsInitialized());
					World->ShaderPlatformChanged(GShaderPlatformForFeatureLevel[FeatureLevel]);
				}
				else
				{
					World->SetFeatureLevel(FeatureLevel);
					World->InitWorld(GetEditorWorldInitializationValues());
				}

				SlowTask.EnterProgressFrame(20, FText::Format( LOCTEXT( "LoadingMapStatus_Initializing", "Loading map: {0}... (Initializing world)" ), FText::FromString(MapFileName) ));
				{
					FBSPOps::bspValidateBrush(Context.World()->GetDefaultBrush()->Brush, 0, 1);

					// This is a relatively long process, so break it up a bit
					FScopedSlowTask InitializingFeedback(5);
					InitializingFeedback.EnterProgressFrame();

					Context.World()->AddToRoot();


					Context.World()->PersistentLevel->SetFlags( RF_Transactional );
					Context.World()->GetModel()->SetFlags( RF_Transactional );
					if( Context.World()->GetModel()->Polys )
					{
						Context.World()->GetModel()->Polys->SetFlags( RF_Transactional );
					}

					// Process any completed shader maps since we at a loading screen anyway
					// Do this before we register components, as USkinnedMeshComponents require the GPU skin cache global shaders when creating render state.
					if (GShaderCompilingManager)
					{
						// Process any asynchronous shader compile results that are ready, limit execution time
						GShaderCompilingManager->ProcessAsyncResults(false, true);
					}

					// Register components in the persistent level (current)
					Context.World()->UpdateWorldComponents(true, true);

					// Make sure secondary levels are loaded & visible.
					Context.World()->FlushLevelStreaming();

					// Update any actors that can be affected by CullDistanceVolumes
					Context.World()->UpdateCullDistanceVolumes();

					InitializingFeedback.EnterProgressFrame();

					// A new level was loaded into the editor, so we need to let other systems know about the new
					// actors in the scene
					FEditorDelegates::MapChange.Broadcast(MapChangeEventFlags::NewMap);
					GEngine->BroadcastLevelActorListChanged();

					NoteSelectionChange();

					InitializingFeedback.EnterProgressFrame();

					// Look for 'orphan' actors - that is, actors which are in the Package of the level we just loaded, but not in the Actors array.
					// If we find any, set IsPendingKill() to 'true', so that PendingKill will return 'true' for them. We can NOT use FActorIterator here
					// as it just traverses the Actors list.
					const double StartTime = FPlatformTime::Seconds();
					for( TObjectIterator<AActor> It; It; ++It )
					{
						AActor* Actor = *It;

						// If Actor is part of the world we are loading's package, but not in Actor list, clear it
						if( Actor->GetOutermost() == WorldPackage && !Context.World()->ContainsActor(Actor) && IsValidChecked(Actor)
							&& !Actor->HasAnyFlags(RF_ArchetypeObject) )
						{
							UE_LOGF(LogEditorServer, Log,  "Destroying orphan Actor: %ls", *Actor->GetName() );
							Actor->MarkAsGarbage();
							Actor->MarkComponentsAsGarbage();
						}
					}
					UE_LOGF(LogEditorServer, Log,  "Finished looking for orphan Actors (%3.3lf secs)", FPlatformTime::Seconds() - StartTime );

					// Set Transactional flag.
					for( FActorIterator It(Context.World()); It; ++It )
					{
						AActor* Actor = *It;

						// Child actors of non-transactional components should not be transactional
						UChildActorComponent* CAC = Actor->GetParentComponent();
						if (CAC == nullptr || CAC->HasAnyFlags(RF_Transactional))
						{
							Actor->SetFlags(RF_Transactional);
						}
					}

					InitializingFeedback.EnterProgressFrame();

					Context.World()->CreateAISystem();
					FNavigationSystem::AddNavigationSystemToWorld(*Context.World(), FNavigationSystemRunMode::EditorMode);

					// Assign stationary light channels for previewing
					ULightComponent::ReassignStationaryLightChannels(Context.World(), false, NULL);

					// Process Layers
					{
						ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();
						for( auto LayerIter = Context.World()->Layers.CreateIterator(); LayerIter; ++LayerIter )
						{
							// Clear away any previously cached actor stats
							(*LayerIter)->ClearActorStats();
						}

						for( FActorIterator It(Context.World()); It; ++It )
						{
							TWeakObjectPtr< AActor > Actor = *It;

							if( !LayersSubsystem->IsActorValidForLayer( Actor.Get() ) )
							{
								continue;
							}

							for( auto NameIt = Actor->Layers.CreateConstIterator(); NameIt; ++NameIt )
							{
								auto Name = *NameIt;
								if( !LayersSubsystem->IsLayer( Name ) )
								{
									LayersSubsystem->CreateLayer( Name );
								}

								Actor->Layers.AddUnique( Name );
							}

							LayersSubsystem->InitializeNewActorLayers( Actor.Get() );
						}
					}

					InitializingFeedback.EnterProgressFrame();

					FEditorDelegates::DisplayLoadErrors.Broadcast();

					if( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
					{
						FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

						// Notify level editors of the map change
						LevelEditor.BroadcastMapChanged( Context.World(), EMapChangeType::LoadMap );
					}

					// Tell the engine about this new world
					if( GEngine )
					{
						GEngine->WorldAdded( Context.World() );
					}

					// Invalidate all the level viewport hit proxies
					RedrawLevelEditingViewports();

					// Collect any stale components or other objects that are no longer required after loading the map
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
				}
			}
			else
			{
				UE_LOGF(LogEditorServer, Warning, "%ls", *FString::Printf( TEXT("Can't find file '%s'"), *TempFname) );
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log(*FText::Format(NSLOCTEXT("Editor", "MapLoad_BadFilenameFmt", "Map_Load failed. The filename '{0}' could not be converted to a long package name."), FText::FromString(TempFname)).ToString()));
		}
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Log(*NSLOCTEXT("UnrealEd", "MissingFilename", "Missing filename").ToString() ));
	}

	// Done loading a map
	return true;
#undef LOCTEXT_NAMESPACE
}

bool UEditorEngine::Map_Import( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	FString FileName;
	if( FParse::Value( Str, TEXT("FILE="), FileName ) )
	{
		const FScopedBusyCursor BusyCursor;

		FFormatNamedArguments Args;
		Args.Add( TEXT("MapFilename"), FText::FromString( FPaths::GetCleanFilename(*FileName) ) );
		const FText LocalizedImportingMap = FText::Format( NSLOCTEXT("UnrealEd", "ImportingMap_F", "Importing map: {MapFilename}..." ), Args );

		ResetTransaction( LocalizedImportingMap );
		FScopedSlowTask SlowTask(0, LocalizedImportingMap);

		UClass* WorldClass = UWorld::StaticClass();

		// Hotfix for 4.23 this should be refactored later
		TArray<UFactory*> Factories;
		{
			auto TransientPackage = GetTransientPackage();
			FString Extension = FPaths::GetExtension(Str);

			// try all automatic factories, sorted by priority
			for (TObjectIterator<UClass> It; It; ++It)
			{
				if (It->IsChildOf(UFactory::StaticClass()))
				{
					UFactory* Default = It->GetDefaultObject<UFactory>();

					if (WorldClass->IsChildOf(Default->SupportedClass) && Default->ImportPriority >= 0)
					{
						TArray<FString> FactoryExtension;
						Default->GetSupportedFileExtensions(FactoryExtension);

						if (Extension.IsEmpty() || (FactoryExtension.Contains(Extension) && Default->FactoryCanImport(FileName)))
						{
							Factories.Add(NewObject<UFactory>(TransientPackage, *It));
						}
					}
				}
			}
		}
		Factories.Sort(&UFactory::SortFactoriesByPriority);


		if ( Factories.Num() > 0 )
		{
			InWorld->ClearWorldComponents();
			InWorld->CleanupWorld();
			UWorld* NewWorld = nullptr;

			for ( UFactory* Factory : Factories )
			{
				bool bTemp;
				NewWorld = Cast<UWorld>( Factory->ImportObject(WorldClass, InWorld->GetOuter(), InWorld->GetFName(), RF_Transactional, FileName, nullptr, bTemp) );
				if ( NewWorld )
				{
					break;
				}
			}

			if ( NewWorld )
			{
				if ( !GWorld )
				{
					GWorld = NewWorld;
				}

				// Importing content into a map will likely cause the list of actors in the level to change,
				// so we'll trigger an event to notify other systems
				FEditorDelegates::MapChange.Broadcast( MapChangeEventFlags::NewMap );
				GEngine->BroadcastLevelActorListChanged();

				NoteSelectionChange();
				Cleanse( false, 1, NSLOCTEXT("UnrealEd", "ImportingActors", "Importing actors") );
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log( TEXT("Unsupported file format") ));
		}
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Log( TEXT("Missing filename") ));
	}

	return true;
}


void UEditorEngine::ExportMap(UWorld* InWorld, const TCHAR* InFilename, bool bExportSelectedActorsOnly)
{
	const FScopedBusyCursor BusyCursor;

	FString MapFileName = FPaths::GetCleanFilename(InFilename);
	const FText LocalizedExportingMap = FText::Format( NSLOCTEXT("UnrealEd", "ExportingMap_F", "Exporting map: {0}..." ), FText::FromString(MapFileName) );
	GWarn->BeginSlowTask( LocalizedExportingMap, true);

	UAssetExportTask* ExportTask = NewObject<UAssetExportTask>();
	FGCObjectScopeGuard ExportTaskGuard(ExportTask);
	ExportTask->Object = InWorld;
	ExportTask->Exporter = NULL;
	ExportTask->Filename = InFilename;
	ExportTask->bSelected = bExportSelectedActorsOnly;
	ExportTask->bReplaceIdentical = true;
	ExportTask->bPrompt = false;
	ExportTask->bUseFileArchive = false;
	ExportTask->bWriteEmptyFiles = false;

	UExporter::RunAssetExportTask(ExportTask);

	GWarn->EndSlowTask();
}

/**
 * Helper structure for finding meshes at the same point in space.
 */
struct FGridBounds
{
	/**
	 * Constructor, intializing grid bounds based on passed in center and extent.
	 *
	 * @param InCenter	Center location of bounds.
	 * @param InExtent	Extent of bounds.
	 */
	FGridBounds( const FVector& InCenter, const FVector& InExtent )
	{
		const int32 GRID_SIZE_XYZ = 16;
		CenterX = InCenter.X / GRID_SIZE_XYZ;
		CenterY = InCenter.Y / GRID_SIZE_XYZ;
		CenterZ = InCenter.Z / GRID_SIZE_XYZ;
		ExtentX = InExtent.X / GRID_SIZE_XYZ;
		ExtentY = InExtent.Y / GRID_SIZE_XYZ;
		ExtentZ = InExtent.Z / GRID_SIZE_XYZ;
	}

	/** Center integer coordinates */
	int32	CenterX, CenterY, CenterZ;

	/** Extent integer coordinates */
	int32	ExtentX, ExtentY, ExtentZ;

	/**
	 * Equals operator.
	 *
	 * @param Other	Other gridpoint to compare agains
	 * @return true if equal, false otherwise
	 */
	bool operator == ( const FGridBounds& Other ) const
	{
		return CenterX == Other.CenterX
			&& CenterY == Other.CenterY
			&& CenterZ == Other.CenterZ
			&& ExtentX == Other.ExtentX
			&& ExtentY == Other.ExtentY
			&& ExtentZ == Other.ExtentZ;
	}

	/**
	 * Helper function for TMap support, generating a hash value for the passed in
	 * grid bounds.
	 *
	 * @param GridBounds Bounds to calculated hash value for
	 * @return Hash value of passed in grid bounds.
	 */
	friend inline uint32 GetTypeHash( const FGridBounds& GridBounds )
	{
		return FCrc::MemCrc_DEPRECATED( &GridBounds,sizeof(FGridBounds) );
	}
};


namespace MoveSelectedActors {
/**
 * A collection of actors and prefabs to move that all belong to the same level.
 */
class FCopyJob
{
public:
	/** A list of actors to move. */
	TArray<AActor*>	Actors;

	/** The index of the selected surface to copy. */
	int32 SurfaceIndex;

	/** The source level that all actors in the Actors array and/or selected BSP surface come from. */
	ULevel*			SrcLevel;

	explicit FCopyJob( ULevel* SourceLevel )
		:	SurfaceIndex(INDEX_NONE)
		,	SrcLevel(SourceLevel)
	{
		check(SrcLevel);
	}

	/**
	* Moves the job's actors to the destination level.  The move happens via the
	* buffer level if one is specified; this is so that references are cleared
	* when the source actors refer to objects whose names also exist in the destination
	* level.  By serializing through a temporary level, the references are cleanly
	* severed.
	*
	* @param	OutNewActors			[out] Newly created actors are appended to this list.
	* @param	DestLevel				The level to duplicate the actors in this job to.
	*/
	void MoveActorsToLevel(TArray<AActor*>& OutNewActors, ULevel* DestLevel, ULevel* BufferLevel, bool bCopyOnly, bool bIsMove, FString* OutClipboardContents )
	{
		UWorld* World = SrcLevel->OwningWorld;
		ULevel* OldCurrentLevel = World->GetCurrentLevel();
		World->SetCurrentLevel( SrcLevel );

		UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet();
		check(SelectionSet);
		{
			FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();

			const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
				.SetAllowHidden(false)
				.SetWarnIfLocked(false)
				.SetAllowLegacyNotifications(false)
				.SetAllowSubRootSelection(true);

			// Set the selection set to be precisely the actors belonging to this job,
			// but make sure not to deselect selected BSP surfaces.
			GEditor->SelectNone(false, true);
			for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex)
			{
				AActor* Actor = Actors[ActorIndex];
				FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);

				SelectionSet->SelectElement(ElementHandle, SelectionOptions);

				// Groups cannot contain actors in different levels.  If the current actor is in a group but not being moved to the same level as the group
				// then remove the actor from the group
				AGroupActor* GroupActor = AGroupActor::GetParentForActor(Actor);
				if (GroupActor && GroupActor->GetLevel() != DestLevel)
				{
					GroupActor->Remove(*Actor);
				}
			}
		}

		FString ScratchData;

		// Cut actors from src level.
		GEditor->edactCopySelected( World, &ScratchData );

		if( !bCopyOnly )
		{
			const bool bSuccess = GEditor->edactDeleteSelected( World, false, true, !bIsMove);
			if ( !bSuccess )
			{
				// The deletion was aborted.
				World->SetCurrentLevel( OldCurrentLevel );
				GEditor->SelectNone( false, true );
				return;
			}
		}

		if ( BufferLevel )
		{
			// Paste to the buffer level.
			World->SetCurrentLevel( BufferLevel );
			GEditor->edactPasteSelected( World, true, false, false, &ScratchData );

			const bool bCopySurfaceToBuffer = (SurfaceIndex != INDEX_NONE);
			UModel* OldModel = BufferLevel->Model;

			if( bCopySurfaceToBuffer )
			{
				// When copying surfaces, we need to override the level's UModel to
				// point to the existing UModel containing the BSP surface. This is
				// because a buffer level is setup with an empty UModel.
				BufferLevel->Model = SrcLevel->Model;

				// Select the surface because we deselected everything earlier because
				// we wanted to deselect all but the first selected BSP surface.
				GEditor->SelectBSPSurf( BufferLevel->Model, SurfaceIndex, true, false );
			}

			// Cut Actors from the buffer level.
			World->SetCurrentLevel( BufferLevel );
			GEditor->edactCopySelected( World, &ScratchData );

			if( bCopySurfaceToBuffer )
			{
				// Deselect the surface.
				GEditor->SelectBSPSurf( BufferLevel->Model, SurfaceIndex, false, false );

				// Restore buffer level's original empty UModel
				BufferLevel->Model = OldModel;
			}

			if( OutClipboardContents != NULL )
			{
				*OutClipboardContents = *ScratchData;
			}

			GEditor->edactDeleteSelected( World, false, false, false );
		}

		if( DestLevel )
		{
			// Paste to the dest level.
			World->SetCurrentLevel( DestLevel );
			//A hidden level must be shown first, otherwise the paste will fail (it will not properly import the properties because that is based on selection)
			bool bReHideLevel = !FLevelUtils::IsLevelVisible(DestLevel);
			if (bReHideLevel)
			{
				const bool bShouldBeVisible = true;
				const bool bForceGroupsVisible = false;
				EditorLevelUtils::SetLevelVisibility(DestLevel, bShouldBeVisible, bForceGroupsVisible);
			}

			GEditor->edactPasteSelected( World, true, false, true, &ScratchData );

			//if the level was hidden, hide it again
			if (bReHideLevel)
			{
				//empty selection
				GEditor->SelectNone( false, true );

				const bool bShouldBeVisible = false;
				const bool bForceGroupsVisible = false;
				EditorLevelUtils::SetLevelVisibility(DestLevel, bShouldBeVisible, bForceGroupsVisible);
			}
		}

		// The current selection set is the actors that were moved during this job; copy them over to the output array.
		for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );
			OutNewActors.Add( Actor );
		}

		if( !bCopyOnly )
		{
			// Delete prefabs that were instanced into the new level.
			World->SetCurrentLevel( SrcLevel );
			GEditor->SelectNone( false, true );
		}

		// Restore the current level
		World->SetCurrentLevel( OldCurrentLevel  );
	}

};
} // namespace MoveSelectedActors

TArray<UFoliageType*> UEditorEngine::GetFoliageTypesInWorld(UWorld* InWorld)
{
	TSet<UFoliageType*> FoliageSet;

	// Iterate over all foliage actors in the world
	for (TActorIterator<AInstancedFoliageActor> It(InWorld); It; ++It)
	{
		for (const auto& Pair : It->GetFoliageInfos())
		{
			FoliageSet.Add(Pair.Key);
		}
	}

	return FoliageSet.Array();
}

ULevel*  UEditorEngine::CreateTransLevelMoveBuffer( UWorld* InWorld )
{
	ULevel* BufferLevel = NewObject<ULevel>(GetTransientPackage(), TEXT("TransLevelMoveBuffer"));
	BufferLevel->Initialize(FURL(nullptr));
	check( BufferLevel );
	BufferLevel->AddToRoot();
	BufferLevel->OwningWorld = InWorld;
	BufferLevel->Model = NewObject<UModel>(BufferLevel);
	BufferLevel->Model->Initialize(nullptr, true);
	BufferLevel->bIsVisible = true;

	BufferLevel->SetFlags( RF_Transactional );
	BufferLevel->Model->SetFlags( RF_Transactional );

	// Spawn worldsettings.
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.OverrideLevel = BufferLevel;
	AWorldSettings* WorldSettings = InWorld->SpawnActor<AWorldSettings>( GEngine->WorldSettingsClass, SpawnInfo );
	BufferLevel->SetWorldSettings(WorldSettings);

	// Spawn builder brush for the buffer level.
	ABrush* BufferDefaultBrush = InWorld->SpawnActor<ABrush>( SpawnInfo );

	check( BufferDefaultBrush->GetBrushComponent() );
	BufferDefaultBrush->Brush = CastChecked<UModel>(StaticFindObject(UModel::StaticClass(), BufferLevel->OwningWorld->GetOuter(), TEXT("Brush"), EFindObjectFlags::ExactClass), ECastCheckedType::NullAllowed);
	if (!BufferDefaultBrush->Brush)
	{
		BufferDefaultBrush->Brush = NewObject<UModel>(InWorld, TEXT("Brush"));
		BufferDefaultBrush->Brush->Initialize(BufferDefaultBrush, 1);
	}
	BufferDefaultBrush->GetBrushComponent()->Brush = BufferDefaultBrush->Brush;
	BufferDefaultBrush->SetNotForClientOrServer();
	BufferDefaultBrush->SetFlags( RF_Transactional );
	BufferDefaultBrush->Brush->SetFlags( RF_Transactional );

	// Find the index in the array the default brush has been spawned at. Not necessarily
	// the last index as the code might spawn the default physics volume afterwards.
	const int32 DefaultBrushActorIndex = BufferLevel->Actors.Find( BufferDefaultBrush );

	// The default brush needs to reside at index 1.
	Exchange(BufferLevel->Actors[1],BufferLevel->Actors[DefaultBrushActorIndex]);

	// Re-sort actor list as we just shuffled things around.
	BufferLevel->SortActorList();

	InWorld->AddLevel( BufferLevel );
	BufferLevel->UpdateLevelComponents(true);
	return BufferLevel;
}

bool UEditorEngine::CanCopySelectedActorsToClipboard( UWorld* InWorld, FCopySelectedInfo* OutCopySelected )
{
	FCopySelectedInfo CopySelected;

	// For faster performance, if all actors belong to the same level then we can just go ahead and copy normally
	{
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			CopySelected.bHasSelectedActors = true;
			if( CopySelected.LevelAllActorsAreIn == NULL )
			{
				CopySelected.LevelAllActorsAreIn = Actor->GetLevel();
			}

			IConsoleVariable* TemporaryActorCopyCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("TypedElements.EnableTemporaryActorCopy"));

			if (TemporaryActorCopyCvar && !TemporaryActorCopyCvar->GetBool())
			{
				if (Actor->GetLevel())
				{
					if (UWorld* ActorWorld = Actor->GetLevel()->GetWorld())
					{
						// If the actor is in a PIE world but doesn't have an editor counterpart it means it's a temporary
						// actor spawned to the world. These actors can cause issues when copied so have been disabled.
						if (ActorWorld->WorldType == EWorldType::PIE && !GEditor->ObjectsThatExistInEditorWorld.Get(Actor))
						{
							return false;
						}
					}
				}
			}

			if( Actor->GetLevel() != CopySelected.LevelAllActorsAreIn )
			{
				CopySelected.bAllActorsInSameLevel = false;
				CopySelected.LevelAllActorsAreIn = NULL;
				break;
			}
		}
	}

	// Next, check for selected BSP surfaces.
	{
		for( TSelectedSurfaceIterator<> SurfaceIter(InWorld); SurfaceIter; ++SurfaceIter )
		{
			ULevel* OwningLevel = SurfaceIter.GetLevel();

			if( CopySelected.LevelWithSelectedSurface == NULL )
			{
				CopySelected.LevelWithSelectedSurface = OwningLevel;
				CopySelected.bHasSelectedSurfaces = true;
			}

			if( OwningLevel != CopySelected.LevelWithSelectedSurface )
			{
				CopySelected.LevelWithSelectedSurface = NULL;
				break;
			}
		}
	}


	// Copy out to the user, if they require it
	if ( OutCopySelected )
	{
		*OutCopySelected = CopySelected;
	}


	// Return whether or not a copy can be performed
	if ( CopySelected.CanPerformQuickCopy() || CopySelected.bHasSelectedActors || CopySelected.bHasSelectedSurfaces )
	{
		return true;
	}
	return false;
}

void UEditorEngine::CopySelectedActorsToClipboard( UWorld* InWorld, bool bShouldCut, bool bIsMove, bool bWarnAboutReferences, FString* DestinationData )
{
	FCopySelectedInfo CopySelected;
	if ( !CanCopySelectedActorsToClipboard( InWorld, &CopySelected ) )
	{
		return;
	}

	using namespace MoveSelectedActors;

	// Perform a quick copy if all the conditions are right.
	if( CopySelected.CanPerformQuickCopy() )
	{
		UWorld* World = NULL;
		ULevel* OldCurrentLevel = NULL;

		if( CopySelected.LevelAllActorsAreIn != NULL )
		{
			World = CopySelected.LevelAllActorsAreIn->OwningWorld;
			OldCurrentLevel = InWorld->GetCurrentLevel();
			World->SetCurrentLevel( CopySelected.LevelAllActorsAreIn );
		}
		else if( CopySelected.LevelWithSelectedSurface != NULL )
		{
			World = CopySelected.LevelWithSelectedSurface->OwningWorld;
			OldCurrentLevel = World->GetCurrentLevel();
			World->SetCurrentLevel( CopySelected.LevelWithSelectedSurface );
		}

		// We should have a valid world by now.
		check(World);

		if( bShouldCut )
		{
			// Cut!
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "Cut", "Cut") );
			edactCopySelected( World, DestinationData );
			const bool bVerifyDeletionCanHappen = true;

			edactDeleteSelected( World, bVerifyDeletionCanHappen, bWarnAboutReferences, !bIsMove );
		}
		else
		{
			// Copy!
			edactCopySelected( World, DestinationData );
		}

		World->SetCurrentLevel( OldCurrentLevel );
	}
	else
	{
		// OK, we'll use a copy method that supports cleaning up references for actors in multiple levels
		if( bShouldCut )
		{
			// Provide the option to abort up-front.
			if ( ShouldAbortActorDeletion() )
			{
				return;
			}
		}

		// Take a note of the current selection, so it can be restored at the end of this process
		TArray<TWeakObjectPtr<AActor>> CurrentlySelectedActors;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			CurrentlySelectedActors.Add(Actor);
		}

		const FScopedBusyCursor BusyCursor;

		// If we have selected actors and/or selected BSP surfaces, we need to setup some copy jobs.
		if( CopySelected.bHasSelectedActors || CopySelected.bHasSelectedSurfaces )
		{
			// Create per-level job lists.
			typedef TMap<ULevel*, FCopyJob*>	CopyJobMap;
			CopyJobMap							CopyJobs;


			// First, create new copy jobs for BSP surfaces if we have selected surfaces.
			if( CopySelected.bHasSelectedSurfaces )
			{
				// Create copy job for the selected surfaces that need copying.
				for( TSelectedSurfaceIterator<> SurfaceIter(InWorld); SurfaceIter; ++SurfaceIter )
				{
					ULevel* LevelWithSelectedSurface = SurfaceIter.GetLevel();

					// Currently, we only support one selected surface per level. So, If the
					// level is already in the map, we don't need to copy this surface.
					if( !CopyJobs.Find(LevelWithSelectedSurface) )
					{
						FCopyJob* NewJob = new FCopyJob( LevelWithSelectedSurface );
						NewJob->SurfaceIndex = SurfaceIter.GetSurfaceIndex();

						check( NewJob->SurfaceIndex != INDEX_NONE );

						CopyJobs.Add( NewJob->SrcLevel, NewJob );
					}
				}
			}

			// Add selected actors to the per-level job lists.
			for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				ULevel* OldLevel = Actor->GetLevel();
				FCopyJob** Job = CopyJobs.Find( OldLevel );
				if ( Job )
				{
					(*Job)->Actors.Add( Actor );
				}
				else
				{
					// Allocate a new job for the level.
					FCopyJob* NewJob = new FCopyJob(OldLevel);
					NewJob->Actors.Add( Actor );
					CopyJobs.Add( OldLevel, NewJob );
				}
			}


			if ( CopyJobs.Num() > 0 )
			{
				// Create a buffer level that actors will be moved through to cleanly break references.
				// Create a new ULevel and UModel.
				ULevel* BufferLevel = CreateTransLevelMoveBuffer( InWorld );

				// We'll build up our final clipboard string with the result of each copy
				FString ClipboardString;

				if( bShouldCut )
				{
					if (Trans)
					{
						Trans->Begin( NULL, NSLOCTEXT("UnrealEd", "Cut", "Cut") );
					}
					GetSelectedActors()->Modify();
				}

				// For each level, select the actors in that level and copy-paste into the destination level.
				TArray<AActor*>	NewActors;
				for ( CopyJobMap::TIterator It( CopyJobs ) ; It ; ++It )
				{
					FCopyJob* Job = It.Value();
					check( Job );

					FString CopiedActorsString;
					const bool bCopyOnly = !bShouldCut;
					Job->MoveActorsToLevel( NewActors, NULL, BufferLevel, bCopyOnly, bIsMove, &CopiedActorsString );

					// Append our copied actors to our final clipboard string
					ClipboardString += CopiedActorsString;
				}

				if( bShouldCut )
				{
					if (Trans)
					{
						Trans->End();
					}
				}

				if (DestinationData)
				{
					// Fill DestinationData with the final string
					*DestinationData = ClipboardString;
				}
				else
				{
					// Update the clipboard with the final string
					FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
				}


				// Cleanup.
				for ( CopyJobMap::TIterator It( CopyJobs ) ; It ; ++It )
				{
					FCopyJob* Job = It.Value();
					delete Job;
				}

				BufferLevel->ClearLevelComponents();
				InWorld->RemoveLevel( BufferLevel );
				BufferLevel->OwningWorld = NULL;
				BufferLevel->RemoveFromRoot();
			}
		}

		// Restore old selection
		GEditor->SelectNone( false, true );

		UTypedElementSelectionSet* SelectionSet = GEditor->GetSelectedActors()->GetElementSelectionSet();
		check(SelectionSet);
		{
			FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
			const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
				.SetAllowHidden(false)
				.SetWarnIfLocked(false)
				.SetAllowLegacyNotifications(false)
				.SetAllowSubRootSelection(true);

			for (const TWeakObjectPtr<AActor>& Actor : CurrentlySelectedActors)
			{
				if (AActor* ActorPtr = Actor.Get())
				{
					FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(ActorPtr);

					SelectionSet->SelectElement(ElementHandle, SelectionOptions);
				}
			}
		}
	}
}


bool UEditorEngine::CanPasteSelectedActorsFromClipboard( UWorld* InWorld )
{
	// Intentionally not checking if the level is locked/hidden here, as it's better feedback for the user if they attempt to paste
	// and get the message explaining why it's failed, than just not having the option available to them.
	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	return PasteString.StartsWith( "BEGIN MAP" );
}


void UEditorEngine::PasteSelectedActorsFromClipboard( UWorld* InWorld, const FText& TransDescription, const EPasteTo PasteTo )
{
	if ( !CanPasteSelectedActorsFromClipboard( InWorld ) )
	{
		return;
	}

	const FSnappedPositioningData PositioningData = FSnappedPositioningData(GCurrentLevelEditingViewportClient, ClickLocation, ClickPlane)
		.AlignToSurfaceRotation(false);
	FVector SaveClickLocation = FActorPositioning::GetSnappedSurfaceAlignedTransform(PositioningData).GetLocation();

	ULevel* DesiredLevel = InWorld->GetCurrentLevel();

	// Don't allow pasting to levels that are locked
	if( !FLevelUtils::IsLevelLocked( DesiredLevel ) )
	{
		// Make sure the desired level is current
		ULevel* OldCurrentLevel = InWorld->GetCurrentLevel();
		InWorld->SetCurrentLevel( DesiredLevel );

		const FScopedTransaction Transaction( TransDescription );

		const bool bNoteSelectionChange = false;
		const bool bDeselectBSPSurfs = false;
		SelectNone( bNoteSelectionChange, bDeselectBSPSurfs );
		ABrush::SetSuppressBSPRegeneration(true);
		edactPasteSelected( InWorld, false, false, true );
		ABrush::SetSuppressBSPRegeneration(false);

		if( PasteTo != PT_OriginalLocation )
		{
			// Get a bounding box for all the selected actors locations.
			FBox bbox(ForceInit);
			int32 NumActorsToMove = 0;

			for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow( Actor->IsA(AActor::StaticClass()) );

				bbox += Actor->GetActorLocation();
				++NumActorsToMove;
			}

			if ( NumActorsToMove > 0 )
			{
				// Figure out which location to center the actors around.
				const FVector Origin( PasteTo == PT_Here ? SaveClickLocation : FVector::ZeroVector );

				// Compute how far the actors have to move.
				const FVector Location = bbox.GetCenter();
				const FVector Adjust = Origin - Location;

				// List of group actors in the selection
				TArray<AGroupActor*> GroupActors;

				struct FAttachData
				{
					FAttachData(AActor* InParentActor, FName InSocketName)
						: ParentActor(InParentActor)
						, SocketName(InSocketName)
					{}

					AActor* ParentActor;
					FName SocketName;
				};

				TArray<FAttachData, TInlineAllocator<8>> AttachData;
				AttachData.Reserve(NumActorsToMove);

				// Break any parent attachments
				AActor* SingleActor = NULL;
				for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
				{
					AActor* Actor = static_cast<AActor*>( *It );

					AActor* ParentActor = Actor->GetAttachParentActor();
					FName SocketName = Actor->GetAttachParentSocketName();
					Actor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
					AttachData.Emplace(ParentActor, SocketName);

					// If this actor is in a group, add it to the list
					if (UActorGroupingUtils::IsGroupingActive())
					{
						AGroupActor* ActorGroupRoot = AGroupActor::GetRootForActor(Actor, true, true);
						if (ActorGroupRoot)
						{
							GroupActors.AddUnique(ActorGroupRoot);
						}
					}

					SingleActor = Actor;
				}

				// Move the actors
				for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
				{
					AActor* Actor = static_cast<AActor*>(*It);
					Actor->SetActorLocation(Actor->GetActorLocation() + Adjust, false);
				}

				// Restore attachments
				int Index = 0;
				for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
				{
					AActor* Actor = static_cast<AActor*>(*It);
					Actor->AttachToActor(AttachData[Index].ParentActor, FAttachmentTransformRules::KeepWorldTransform, AttachData[Index].SocketName);
					Actor->PostEditMove(true);
					Index++;
				}

				// Update the pivot location.
				check(SingleActor);
				SetPivot( SingleActor->GetActorLocation(), false, true );

				// If grouping is active, go through the unique group actors and update the group actor location
				if (UActorGroupingUtils::IsGroupingActive())
				{
					for (AGroupActor* GroupActor : GroupActors)
					{
						GroupActor->CenterGroupLocation();
					}
				}
			}
		}

		InWorld->SetCurrentLevel( OldCurrentLevel );

		RedrawLevelEditingViewports();

		for (FSelectionIterator It(GetSelectedActorIterator()); It; ++It)
		{
			AActor* Actor = static_cast<AActor*>( *It );
			
			// If TypedElements copy paste for actors is enabled, the fixup happens in ULevelEditorCommonActions::PasteNormalizedElements
			if (!TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
			{
				UE::Editor::DataStorage::ActorFolders::FixupActorPostDuplicate(Actor);
			}
			
			if (ABrush* Brush = Cast<ABrush>(Actor))
			{
				if (Brush->IsStaticBrush())
				{
					// If required, update the Bsp of any levels that received a pasted brush actor
					RebuildAlteredBSP();
					break;
				}
			}
		}
	}
	else
	{
		FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelPasteActor", "PasteActor: The requested operation could not be completed because the level is locked.") );
	}
}

namespace
{
	/** Property value used for property-based coloration. */
	static FString				GPropertyColorationValue;

	/** Property used for property-based coloration. */
	static FProperty*			GPropertyColorationProperty = NULL;

	/** Class of object to which property-based coloration is applied. */
	static UClass*				GPropertyColorationClass = NULL;

	/** true if GPropertyColorationClass is an actor class. */
	static bool				GbColorationClassIsActor = false;

	/** true if GPropertyColorationProperty is an object property. */
	static bool				GbColorationPropertyIsObjectProperty = false;

	/** The chain of properties from member to lowest priority. */
	static TSharedRef<FEditPropertyChain>*	GPropertyColorationChain = nullptr;

	/** Used to collect references to actors that match the property coloration settings. */
	static TArray<AActor*>*		GPropertyColorationActorCollector = NULL;
}


void UEditorEngine::SetPropertyColorationTarget(UWorld* InWorld, const FString& PropertyValue, FProperty* Property, UClass* CommonBaseClass, TSharedRef<FEditPropertyChain>* PropertyChain, bool bSelectActors /*=false*/)
{

	if (bSelectActors ||
		GPropertyColorationProperty != Property ||
		GPropertyColorationClass != CommonBaseClass ||
		!!GPropertyColorationChain != !!PropertyChain ||
		GPropertyColorationValue != PropertyValue )
	{
		const FScopedBusyCursor BusyCursor;
		delete GPropertyColorationChain;
		GPropertyColorationChain = nullptr;

		GPropertyColorationValue = PropertyValue;
		GPropertyColorationProperty = Property;
		GPropertyColorationClass = CommonBaseClass;

		if (PropertyChain)
		{
			GPropertyColorationChain = new TSharedRef<FEditPropertyChain>(*PropertyChain);
			GbColorationClassIsActor = GPropertyColorationClass && GPropertyColorationClass->IsChildOf(AActor::StaticClass());
			GbColorationPropertyIsObjectProperty = CastField<FObjectPropertyBase>(GPropertyColorationProperty) != nullptr;


			TArray<AActor*> Actors;
			if (bSelectActors)
			{
				GPropertyColorationActorCollector = &Actors;
			}
			
			FActorPrimitiveColorHandler::Get().RefreshPrimitiveColorHandler(TEXT("PropertyColor"), InWorld);

			if (bSelectActors)
			{
				GPropertyColorationActorCollector = nullptr;

				const FScopedTransaction Transaction(NSLOCTEXT("PropertyColor", "SelectByPropertyColor", "Select by Property Color"));
				USelection* SelectedActors = GetSelectedActors();
				SelectedActors->BeginBatchSelectOperation();
				SelectedActors->Modify();
				SelectNone(false, true);
				for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex)
				{
					AActor* Actor = Actors[ActorIndex];
					SelectActor(Actor, true, false);
				}
				SelectedActors->EndBatchSelectOperation();
				NoteSelectionChange();
			}
		}

		RedrawLevelEditingViewports();
	}
}

bool UEditorEngine::GetPropertyColorationMatch(UObject* Object)
{
	bool bResult = false;
	if (GPropertyColorationChain)
	{
		UObject* MatchingBase = NULL;
		AActor* Actor = NULL;
		if ( Object->IsA(GPropertyColorationClass) )
		{
			// The querying object matches the coloration class.
			MatchingBase = Object;
		}
		else {
			// If the coloration class is an actor, check if the querying object is a component.
			// If so, compare the class of the component's owner against the coloration class.
			if ( GbColorationClassIsActor )
			{
				UActorComponent* ActorComponent = Cast<UActorComponent>( Object );
				if ( ActorComponent )
				{
					Actor = ActorComponent->GetOwner();
					if (Actor && Actor->IsA( GPropertyColorationClass ) )
					{
						MatchingBase = Actor;
					}
				}
			}
		}

		// Do we have a matching object?
		if ( MatchingBase )
		{
			bool bDontCompareProps = false;

			uint8* Base = (uint8*) MatchingBase;
			int32 TotalChainLength = (*GPropertyColorationChain)->Num();
			int32 ChainIndex = 0;
			for ( FEditPropertyChain::TIterator It((*GPropertyColorationChain)->GetHead()); It; ++It )
			{
				FProperty* Prop = *It;
				FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(Prop);
				if( CastField<FArrayProperty>(Prop) )
				{
					// @todo DB: property coloration -- add support for array properties.
					bDontCompareProps = true;
					break;
				}
				else if ( ObjectPropertyBase && (ChainIndex != TotalChainLength-1))
				{
					uint8* ObjAddr = Prop->ContainerPtrToValuePtr<uint8>(Base);
					UObject* ReferencedObject = ObjectPropertyBase->GetObjectPropertyValue(ObjAddr);
					Base = (uint8*) ReferencedObject;
				}
				else if (Base)
				{
					Base = Prop->ContainerPtrToValuePtr<uint8>(Base);
				}
				ChainIndex++;
			}

			// Export the property value.  We don't want to exactly compare component properties.
			if ( !bDontCompareProps && Base)
			{
				FString PropertyValue;
				GPropertyColorationProperty->ExportText_Direct(PropertyValue, Base, Base, NULL, 0 );
				if ( PropertyValue == GPropertyColorationValue )
				{
					bResult  = true;

					// Collect actor references.
					if (Actor == NULL)
					{
						Actor = Cast<AActor>(Object);
					}
					if (GPropertyColorationActorCollector && Actor)
					{
						GPropertyColorationActorCollector->AddUnique(Actor);
					}
				}
			}
		}
	}
	return bResult;
}

bool UEditorEngine::IsPropertyColorationModeActive()
{
	if (const FViewport* ActiveViewport = GetActiveViewport())
	{
		for (const TArray<FLevelEditorViewportClient*>& ViewportClients = GetLevelViewportClients(); 
			const FLevelEditorViewportClient* ViewportClient : ViewportClients)
		{
			if (ViewportClient == nullptr)
			{
				continue;
			}
			if (const FViewport* Viewport = ViewportClient->Viewport)
			{
				if (Viewport == ActiveViewport)
				{
					if (ViewportClient->GetViewMode() == VMI_VisualizeActorColoration)
					{
						return true;
					}
				}
			}
		}
	}
	
	return false;
}


bool UEditorEngine::Map_Check( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar, bool bCheckDeprecatedOnly, EMapCheckNotification::Type Notification/*= EMapCheckNotification::DisplayResults*/, bool bClearLog/*= true*/)
{
#define LOCTEXT_NAMESPACE "EditorEngine"
	const FText CheckMapLocText = NSLOCTEXT("UnrealEd", "CheckingMap", "Checking map");
	GWarn->BeginSlowTask( CheckMapLocText, false );
	const double StartTime = FPlatformTime::Seconds();

	FMessageLog MapCheckLog("MapCheck");

	if (bClearLog)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Name"), FText::FromString(FPackageName::GetShortName(InWorld->GetOutermost())));
		Arguments.Add(TEXT("TimeStamp"), FText::AsDateTime(FDateTime::UtcNow()));
		FText MapCheckPageName(FText::Format(LOCTEXT("MapCheckPageName", "{Name} - {TimeStamp}"), Arguments));
		MapCheckLog.NewPage(MapCheckPageName);
	}

	TMap<FGuid,AActor*>			LightGuidToActorMap;
	const int32 ProgressDenominator = InWorld->GetProgressDenominator();

	if ( !bCheckDeprecatedOnly )
	{
		// Report if any brush material references could be cleaned by running 'Clean BSP Materials'.
		const int32 NumRefrencesCleared = CleanBSPMaterials( InWorld, true, false );
		if ( NumRefrencesCleared > 0 )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("NumReferencesCleared"), NumRefrencesCleared);
			FMessageLog("MapCheck").Warning()
				->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_CleanBSPMaterials", "Run 'Clean BSP Materials' to clear {NumReferencesCleared} unnecessary materal references" ), Arguments ) ) )
				->AddToken(FMapErrorToken::Create(FMapErrors::CleanBSPMaterials));
		}
	}

	// Check to see if any of the streaming levels have streaming levels of their own
	// Grab the world info, and loop through the streaming levels
	for (ULevelStreaming* LevelStreaming: InWorld->GetStreamingLevels())
	{
		if (LevelStreaming)
		{
			if (const ULevel* Level = LevelStreaming->GetLoadedLevel())
			{
				// Grab the world info of the streaming level, and loop through it's streaming levels
				AWorldSettings* SubLevelWorldSettings = Level->GetWorldSettings();
				UWorld *SubLevelWorld = CastChecked<UWorld>(Level->GetOuter());
				if (SubLevelWorld && SubLevelWorldSettings)
				{
					for (ULevelStreaming* SubLevelStreaming : SubLevelWorld->GetStreamingLevels())
					{
						// If it has any and they aren't loaded flag a warning to the user
						if (SubLevelStreaming && SubLevelStreaming->GetLoadedLevel() == nullptr)
						{
							UE_LOGF(LogEditorServer, Warning, "%ls contains streaming level '%ls' which isn't loaded.", *SubLevelWorldSettings->GetName(), *SubLevelStreaming->GetWorldAssetPackageName());
						}
					}
				}
			}
		}
	}

	// Make sure all levels in the world have a filename length less than the max limit
	// Filenames over the max limit interfere with cooking for consoles.
	const int32 MaxFilenameLen = FPlatformMisc::GetMaxPathLength();
	for ( int32 LevelIndex = 0; LevelIndex < InWorld->GetNumLevels(); LevelIndex++ )
	{
		ULevel* Level = InWorld->GetLevel( LevelIndex );
		UPackage* LevelPackage = Level->GetOutermost();
		FString PackageFilename;
		if( FPackageName::DoesPackageExist( LevelPackage->GetName(), &PackageFilename ) &&
			FPaths::GetBaseFilename(PackageFilename).Len() > MaxFilenameLen )
		{
			const FString BaseFilenameOfPackageFilename = FPaths::GetBaseFilename(PackageFilename);
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Filename"), FText::FromString(BaseFilenameOfPackageFilename));
			Arguments.Add(TEXT("FilenameLength"), BaseFilenameOfPackageFilename.Len());
			Arguments.Add(TEXT("MaxFilenameLength"), MaxFilenameLen);
			FMessageLog("MapCheck").Warning()
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_FilenameIsTooLongForCooking", "Filename is too long ({FilenameLength} characters) - this may interfere with cooking for consoles. Unreal filenames should be no longer than {MaxFilenameLength} characters. Filename value: {Filename}" ), Arguments ) ))
				->AddToken(FMapErrorToken::Create(FMapErrors::FilenameIsTooLongForCooking));
		}
	}

	Game_Map_Check(InWorld, Str, Ar, bCheckDeprecatedOnly);

	CheckTextureStreamingBuildValidity(InWorld);
	if (InWorld->NumTextureStreamingUnbuiltComponents > 0 || InWorld->NumTextureStreamingDirtyResources > 0)
	{
		FMessageLog("MapCheck").Warning()->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_TextureStreamingNeedsRebuild", "Texture streaming needs to be rebuilt ({0} Components, {1} Resource Refs), run 'Build Texture Streaming'."), InWorld->NumTextureStreamingUnbuiltComponents, InWorld->NumTextureStreamingDirtyResources)));
	}

	if (const UWorldPartition* WorldPartition = InWorld->GetWorldPartition())
	{
		FStreamingGenerationMapCheckErrorHandler MapCheckErrorHandler;
		WorldPartition->CheckForErrors(&MapCheckErrorHandler);
	}

	if (InWorld->ContentBundleManager)
	{
		InWorld->ContentBundleManager->CheckForErrors();
	}

	GWarn->StatusUpdate( 0, ProgressDenominator, CheckMapLocText );

	int32 LastUpdateCount = 0;
	int32 UpdateGranularity = ProgressDenominator / 5;
	TMap<ULevel*, TMap<FGuid, AActor*>> ActorGuidMap;
	for( FActorIterator It(InWorld); It; ++It )
	{
		if(It.GetProgressNumerator() >= LastUpdateCount + UpdateGranularity)
		{
			GWarn->UpdateProgress( It.GetProgressNumerator(), ProgressDenominator );
			LastUpdateCount=It.GetProgressNumerator();
		}

		AActor* Actor = *It;
		if(bCheckDeprecatedOnly)
		{
			Actor->CheckForDeprecated();
		}
		else
		{
			Actor->CheckForErrors();

			// Determine actor location and bounds, falling back to actor location property and 0 extent
			FVector Center = Actor->GetActorLocation();
			FVector Extent = FVector::ZeroVector;
			AStaticMeshActor*	StaticMeshActor		= Cast<AStaticMeshActor>(Actor);
			ASkeletalMeshActor*	SkeletalMeshActor	= Cast<ASkeletalMeshActor>(Actor);
			ALight*				LightActor			= Cast<ALight>(Actor);
			UMeshComponent*		MeshComponent		= NULL;
			if( StaticMeshActor )
			{
				MeshComponent = StaticMeshActor->GetStaticMeshComponent();
			}
			else if( SkeletalMeshActor )
			{
				MeshComponent = SkeletalMeshActor->GetSkeletalMeshComponent();
			}

			// See whether there are lights that ended up with the same component. This was possible in earlier versions of the engine.
			if( LightActor )
			{
				ULightComponent* LightComponent = LightActor->GetLightComponent();
				// LightComponent component can be null, for example when creating a blueprint deriving from ALight.
				// Movable light components have a light guid of 0, so skip them as well
				if (LightComponent && LightComponent->HasStaticShadowing())
				{
					AActor* ExistingLightActor = LightGuidToActorMap.FindRef( LightComponent->LightGuid );
					if( ExistingLightActor )
					{
						{
							FFormatNamedArguments Arguments;
							Arguments.Add(TEXT("LightActor0"), FText::FromString(LightActor->GetName()));
							Arguments.Add(TEXT("LightActor1"), FText::FromString(ExistingLightActor->GetName()));
							FMessageLog("MapCheck").Warning()
								->AddToken(FUObjectToken::Create(LightActor))
								->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_MatchingLightGUID", "'{LightActor0}' has same light GUID as '{LightActor1}' (Duplicate and replace the orig with the new one)" ), Arguments ) ))
								->AddToken(FMapErrorToken::Create(FMapErrors::MatchingLightGUID));
						}

						{
							FFormatNamedArguments Arguments;
							Arguments.Add(TEXT("LightActor0"), FText::FromString(ExistingLightActor->GetName()));
							Arguments.Add(TEXT("LightActor1"), FText::FromString(LightActor->GetName()));
							FMessageLog("MapCheck").Warning()
								->AddToken(FUObjectToken::Create(ExistingLightActor))
								->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Message_MatchingLightGUID", "'{LightActor0}' has same light GUID as '{LightActor1}' (Duplicate and replace the orig with the new one)" ), Arguments ) ))
								->AddToken(FMapErrorToken::Create(FMapErrors::MatchingLightGUID));
						}
					}
					else
					{
						LightGuidToActorMap.Add( LightComponent->LightGuid, LightActor );
					}
				}
			}

			// Check for duplicated Guids
			TMap<FGuid, AActor*>& PerLevelActorGuidMap = ActorGuidMap.FindOrAdd(Actor->GetLevel());
			if (AActor** DuplicatedGuidActor = PerLevelActorGuidMap.Find(Actor->GetActorGuid()))
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Actor0"), FText::FromString(Actor->GetName()));
				Arguments.Add(TEXT("Actor1"), FText::FromString((*DuplicatedGuidActor)->GetName()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(Actor))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT( "MapCheck_Message_MatchingActorGUID", "'{Actor0}' has same GUID as '{Actor1}' (Duplicate and replace the orig with the new one)" ), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::MatchingActorGUID));
			}
			else
			{
				PerLevelActorGuidMap.Add(Actor->GetActorGuid(), Actor);
			}
		}

		Game_Map_Check_Actor(Str, Ar, bCheckDeprecatedOnly, Actor);
	}

	// Add a summary of the Map Check
	const int32 ErrorCount = MapCheckLog.NumMessages( EMessageSeverity::Error );
	const int32 WarningCount = MapCheckLog.NumMessages( EMessageSeverity::Warning );
	const double CurrentTime = (FPlatformTime::Seconds() - StartTime) * 1000.0f;
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Errors"), ErrorCount);
		Arguments.Add(TEXT("Warnings"), WarningCount - ErrorCount);
		Arguments.Add(TEXT("Time"), CurrentTime);
		FMessageLog("MapCheck").Info()
			->AddToken(FTextToken::Create(FText::Format( LOCTEXT( "MapCheck_Complete", "Map check complete: {Errors} Error(s), {Warnings} Warning(s), took {Time}ms to complete." ), Arguments ) ));
	}

	GWarn->EndSlowTask();

	if( Notification != EMapCheckNotification::DontDisplayResults )
	{
		if(bCheckDeprecatedOnly)
		{
			if(ErrorCount > 0)
			{
				MapCheckLog.Notify(LOCTEXT("MapCheckGenErrors", "Map check generated errors!"));
			}
			else if (WarningCount > 0)
			{
				MapCheckLog.Notify(LOCTEXT("MapCheckGenWarnings", "Map check generated warnings!"));
			}
		}
		else
		{
			if(Notification == EMapCheckNotification::DisplayResults)
			{
				MapCheckLog.Open(EMessageSeverity::Info, true);
			}
			else if(Notification == EMapCheckNotification::NotifyOfResults)
			{
				if (ErrorCount > 0)
				{
					MapCheckLog.Notify(LOCTEXT("MapCheckFoundErrors", "Map Check found some errors!"));
				}
				else if (WarningCount > 0)
				{
					MapCheckLog.Notify(LOCTEXT("MapCheckFoundWarnings", "Map Check found some issues!"));
				}
				else
				{
					// Nothing to notify about.  Everything went fine!
				}
			}
		}
	}
	return true;
#undef LOCTEXT_NAMESPACE
}

bool UEditorEngine::Map_Scale( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	float Factor = 1.f;
	if( FParse::Value(Str,TEXT("FACTOR="),Factor) )
	{
		bool bAdjustLights=0;
		FParse::Bool( Str, TEXT("ADJUSTLIGHTS="), bAdjustLights );
		bool bScaleSprites=0;
		FParse::Bool( Str, TEXT("SCALESPRITES="), bScaleSprites );
		bool bScaleLocations=0;
		FParse::Bool( Str, TEXT("SCALELOCATIONS="), bScaleLocations );
		bool bScaleCollision=0;
		FParse::Bool( Str, TEXT("SCALECOLLISION="), bScaleCollision );

		const FScopedBusyCursor BusyCursor;

		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "MapScaling", "Scale") );
		const FText LocalizeScaling = NSLOCTEXT("UnrealEd", "Scaling", "Scaling");
		GWarn->BeginSlowTask( LocalizeScaling, true );

		NoteActorMovement();
		const int32 ProgressDenominator = InWorld->GetProgressDenominator();

		// Fire ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied		LevelDirtyCallback;

		int32 Progress = 0;
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			GWarn->StatusUpdate( Progress++, GetSelectedActors()->Num(), LocalizeScaling );
			Actor->PreEditChange(NULL);
			Actor->Modify();

			LevelDirtyCallback.Request();

			ABrush* Brush = Cast< ABrush >( Actor );
			if( Brush )
			{
				for( int32 poly = 0 ; poly < Brush->Brush->Polys->Element.Num() ; poly++ )
				{
					FPoly* Poly = &(Brush->Brush->Polys->Element[poly]);

					Poly->TextureU /= Factor;
					Poly->TextureV /= Factor;
					Poly->Base = ((Poly->Base - (FVector3f)Brush->GetPivotOffset()) * Factor) + (FVector3f)Brush->GetPivotOffset();

					for( int32 vtx = 0 ; vtx < Poly->Vertices.Num() ; vtx++ )
					{
						Poly->Vertices[vtx] = ((Poly->Vertices[vtx] - (FVector3f)Brush->GetPivotOffset()) * Factor) + (FVector3f)Brush->GetPivotOffset();
					}

					Poly->CalcNormal();
				}

				Brush->Brush->BuildBound();
			}
			else if( Actor->GetRootComponent() != NULL )
			{
				Actor->GetRootComponent()->SetRelativeScale3D(Actor->GetRootComponent()->GetRelativeScale3D() * Factor);
			}

			if( bScaleLocations )
			{
				FVector ScaledLocation = Actor->GetActorLocation();
				ScaledLocation.X *= Factor;
				ScaledLocation.Y *= Factor;
				ScaledLocation.Z *= Factor;
				Actor->SetActorLocation(ScaledLocation, false);
			}

			Actor->PostEditChange();
		}
		GWarn->EndSlowTask();
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Log(*NSLOCTEXT("UnrealEd", "MissingScaleFactor", "Missing scale factor").ToString()));
	}

	return true;
}

bool UEditorEngine::Map_Setbrush( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SetBrushProperties", "Set Brush Properties") );

	uint16 PropertiesMask = 0;

	int32 BrushType = 0;
	if (FParse::Value(Str,TEXT("BRUSHTYPE="),BrushType))	PropertiesMask |= MSB_BrushType;

	uint16 BrushColor = 0;
	if (FParse::Value(Str,TEXT("COLOR="),BrushColor))		PropertiesMask |= MSB_BrushColor;

	FName GroupName = NAME_None;
	if (FParse::Value(Str,TEXT("GROUP="),GroupName))		PropertiesMask |= MSB_Group;

	int32 SetFlags = 0;
	if (FParse::Value(Str,TEXT("SETFLAGS="),SetFlags))		PropertiesMask |= MSB_PolyFlags;

	int32 ClearFlags = 0;
	if (FParse::Value(Str,TEXT("CLEARFLAGS="),ClearFlags))	PropertiesMask |= MSB_PolyFlags;

	MapSetBrush( InWorld,
				static_cast<EMapSetBrushFlags>( PropertiesMask ),
				 BrushColor,
				 GroupName,
				 SetFlags,
				 ClearFlags,
				 BrushType,
				 0 // Draw type
				 );

	RedrawLevelEditingViewports();
	RebuildAlteredBSP(); // Update the Bsp of any levels containing a modified brush

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace {
	/** Implements texmult and texpan*/
	static void ScaleTexCoords(UWorld* InWorld, const TCHAR* Str, int16 Word2)
	{
		// Ensure each polygon has unique texture vector indices.
		for ( TSelectedSurfaceIterator<> It(InWorld) ; It ; ++It )
		{
			FBspSurf* Surf = *It;
			UModel* Model = It.GetModel();
			Model->Modify(false);
			const FVector3f TextureU( Model->Vectors[Surf->vTextureU] );
			const FVector3f TextureV( Model->Vectors[Surf->vTextureV] );
			Surf->vTextureU = Model->Vectors.Add(TextureU);
			Surf->vTextureV = Model->Vectors.Add(TextureV);
		}

		float UU,UV,VU,VV;
		UU=1.0; FParse::Value(Str,TEXT("UU="),UU);
		UV=0.0; FParse::Value(Str,TEXT("UV="),UV);
		VU=0.0; FParse::Value(Str,TEXT("VU="),VU);
		VV=1.0; FParse::Value(Str,TEXT("VV="),VV);

		for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
		{
			UModel* Model = (*Iterator)->Model;
			Model->Modify(false);
			GEditor->polyTexScale( Model, UU, UV, VU, VV, !!Word2 );
		}
	}
} // namespace

void UEditorEngine::ExecuteCommandForAllLevelModels( UWorld* InWorld, FSelectCommand InSelectCommand, const FText& TransDesription )
{
	const FScopedTransaction Transaction( TransDesription );
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		InSelectCommand.ExecuteIfBound( Model );
	}
	USelection::NoteUnknownSelectionChanged();
}

void UEditorEngine::ExecuteCommandForAllLevelModels( UWorld* InWorld, FSelectInWorldCommand InSelectCommand, const FText& TransDesription )
{
	const FScopedTransaction Transaction( TransDesription );
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		InSelectCommand.ExecuteIfBound( InWorld, Model );
	}
	USelection::NoteUnknownSelectionChanged();
}

void UEditorEngine::FlagModifyAllSelectedSurfacesInLevels( UWorld* InWorld )
{
	for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
	{
		UModel* Model = (*Iterator)->Model;
		Model->ModifySelectedSurfs( true );
	}
}

bool UEditorEngine::Exec_Poly( UWorld* InWorld, const TCHAR* Str, FOutputDevice& Ar )
{
	if( FParse::Command(&Str,TEXT("SELECT")) ) // POLY SELECT [ALL/NONE/INVERSE] FROM [LEVEL/SOLID/GROUP/ITEM/ADJACENT/MATCHING]
	{
		if( FParse::Command(&Str,TEXT("NONE")) )
		{
			return Exec( InWorld, TEXT("SELECT NONE") );
		}
		else if( FParse::Command(&Str,TEXT("ALL")) )
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectAll", "Select All") );
			GetSelectedActors()->Modify();
			SelectNone( false, true );

			for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
			{
				UModel* Model = (*Iterator)->Model;
				polySelectAll( Model );
			}
			NoteSelectionChange();
			return true;
		}
		else if( FParse::Command(&Str,TEXT("REVERSE")) )
		{
			FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectReverse);
			ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "ReverseSelection", "Reverse Selection") );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MATCHING")) )
		{
			if (FParse::Command(&Str,TEXT("GROUPS")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectMatchingGroups);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectMatchingGroups", "Selet Matching Groups") );
			}
			else if (FParse::Command(&Str,TEXT("ITEMS")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectMatchingItems);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectMatchingItems", "Select Matching Items") );
			}
			else if (FParse::Command(&Str,TEXT("BRUSH")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectMatchingBrush);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectMatchingBrush", "Select Matching Brush") );
			}
			else if (FParse::Command(&Str,TEXT("TEXTURE")))
			{
				polySelectMatchingMaterial( InWorld, false );
				USelection::NoteUnknownSelectionChanged();
			}
			else if (FParse::Command(&Str,TEXT("RESOLUTION")))
			{
				if (FParse::Command(&Str,TEXT("CURRENT")))
				{
					polySelectMatchingResolution(InWorld, true);
				}
				else
				{
					polySelectMatchingResolution(InWorld, false);
				}
				USelection::NoteUnknownSelectionChanged();
			}

			return true;
		}
		else if( FParse::Command(&Str,TEXT("ADJACENT")) )
		{
			if (FParse::Command(&Str,TEXT("ALL")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectMatchingBrush);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAllAdjacent", "Select All Adjacent") );
			}
			if (FParse::Command(&Str,TEXT("COPLANARS")))
			{
				FSelectInWorldCommand SelectCommand = FSelectInWorldCommand::CreateUObject(this, &UEditorEngine::polySelectCoplanars);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAdjacentCoplanars", "Select Adjacent Coplanars") );
			}
			else if (FParse::Command(&Str,TEXT("WALLS")))
			{
				FSelectInWorldCommand SelectCommand = FSelectInWorldCommand::CreateUObject(this, &UEditorEngine::polySelectAdjacentWalls);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAdjacentWalls", "Select Adjacent Walls") );
			}
			else if (FParse::Command(&Str,TEXT("FLOORS")))
			{
				FSelectInWorldCommand SelectCommand = FSelectInWorldCommand::CreateUObject(this, &UEditorEngine::polySelectAdjacentFloors);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAdjacentFloors", "Select Adjacent Floors") );
			}
			else if (FParse::Command(&Str,TEXT("CEILINGS")))
			{
				FSelectInWorldCommand SelectCommand = FSelectInWorldCommand::CreateUObject(this, &UEditorEngine::polySelectAdjacentFloors);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAdjacentCeilings", "Select Adjacent Ceilings") );
			}
			else if (FParse::Command(&Str,TEXT("SLANTS")))
			{
				FSelectInWorldCommand SelectCommand = FSelectInWorldCommand::CreateUObject(this, &UEditorEngine::polySelectAdjacentSlants);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectAdjacentSlants", "Select Adjacent Slants") );
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MEMORY")) )
		{

			if (FParse::Command(&Str,TEXT("SET")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polyMemorizeSet);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "MemorizeSelectionSet", "Memorize Selection Set") );
			}
			else if (FParse::Command(&Str,TEXT("RECALL")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polyRememberSet);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "RememberSelectionSet", "Recall Selection Set") );
			}
			else if (FParse::Command(&Str,TEXT("UNION")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polyUnionSet);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "UnionSelectionSet", "Union Selection Set") );
			}
			else if (FParse::Command(&Str,TEXT("INTERSECT")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polyIntersectSet);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "IntersectSelectionSet", "Intersect Selection Set") );
			}
			else if (FParse::Command(&Str,TEXT("XOR")))
			{
				FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polyXorSet);
				ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "XorSelectionSet", "XOR Selection Set") );
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("ZONE")) )
		{
			FSelectCommand SelectCommand = FSelectCommand::CreateUObject(this, &UEditorEngine::polySelectZone);
			ExecuteCommandForAllLevelModels( InWorld, SelectCommand, NSLOCTEXT("UnrealEd", "SelectZone", "Select Zone") );
			return true;
		}
		RedrawLevelEditingViewports();
	}
	else if( FParse::Command(&Str,TEXT("DEFAULT")) ) // POLY DEFAULT <variable>=<value>...
	{
		//CurrentMaterial=NULL;
		//ParseObject<UMaterial>(Str,TEXT("TEXTURE="),CurrentMaterial,nullptr);
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SETMATERIAL")) )
	{
		bool bModelDirtied = false;
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetMaterial", "Set Material") );
			FlagModifyAllSelectedSurfacesInLevels( InWorld );

			UMaterialInterface* SelectedMaterialInstance = GetSelectedObjects()->GetTop<UMaterialInterface>();

			for ( TSelectedSurfaceIterator<> It(InWorld) ; It ; ++It )
			{
				UModel* Model = It.GetModel();
				const int32 SurfaceIndex = It.GetSurfaceIndex();

				Model->Surfs[SurfaceIndex].Material = SelectedMaterialInstance;
				const bool bUpdateTexCoords = false;
				const bool bOnlyRefreshSurfaceMaterials = true;
				polyUpdateBrush(Model, SurfaceIndex, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
				Model->MarkPackageDirty();

				bModelDirtied = true;
			}
		}
		RedrawLevelEditingViewports();
		if ( bModelDirtied )
		{
			ULevel::LevelDirtiedEvent.Broadcast();
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SET")) ) // POLY SET <variable>=<value>...
	{
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetTexture", "Set Texture") );
			FlagModifyAllSelectedSurfacesInLevels( InWorld );
			uint64 Ptr;
			if( !FParse::Value(Str,TEXT("TEXTURE="),Ptr) )
			{
				Ptr = 0;
			}

			UMaterialInterface*	Material = (UMaterialInterface*)Ptr;
			if( Material )
			{
				for ( TSelectedSurfaceIterator<> It(InWorld) ; It ; ++It )
				{
					const int32 SurfaceIndex = It.GetSurfaceIndex();
					It.GetModel()->Surfs[SurfaceIndex].Material = Material;
					const bool bUpdateTexCoords = false;
					const bool bOnlyRefreshSurfaceMaterials = true;
					polyUpdateBrush(It.GetModel(), SurfaceIndex, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
				}
			}

			int32 SetBits = 0;
			int32 ClearBits = 0;

			FParse::Value(Str,TEXT("SETFLAGS="),SetBits);
			FParse::Value(Str,TEXT("CLEARFLAGS="),ClearBits);

			// Update selected polys' flags.
			if ( SetBits != 0 || ClearBits != 0 )
			{
				for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
				{
					UModel* Model = (*Iterator)->Model;
					polySetAndClearPolyFlags( Model,SetBits,ClearBits,1,1 );
				}
			}
		}
		RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("TEXSCALE")) ) // POLY TEXSCALE [U=..] [V=..] [UV=..] [VU=..]
	{
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetTexscale", "Set Texscale") );

			FlagModifyAllSelectedSurfacesInLevels( InWorld );

			int16 Word2 = 1; // Scale absolute
			if( FParse::Command(&Str,TEXT("RELATIVE")) )
			{
				Word2=0;
			}
			ScaleTexCoords( InWorld, Str, Word2 );
		}
		RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("TEXMULT")) ) // POLY TEXMULT [U=..] [V=..]
	{
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetTexmult", "Set Texmult") );
			FlagModifyAllSelectedSurfacesInLevels( InWorld );
			int16 Word2 = 0; // Scale relative;
			ScaleTexCoords( InWorld, Str, Word2 );
		}
		RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("TEXPAN")) ) // POLY TEXPAN [RESET] [U=..] [V=..]
	{
		{
			const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PolySetTexpan", "Set Texpan") );
			FlagModifyAllSelectedSurfacesInLevels( InWorld );

			// Ensure each polygon has a unique base point index.
			for ( TSelectedSurfaceIterator<> It(InWorld) ; It ; ++It )
			{
				FBspSurf* Surf = *It;
				UModel* Model = It.GetModel();
				Model->Modify(false);
				const FVector3f Base( Model->Points[Surf->pBase] );
				Surf->pBase = Model->Points.Add(Base);
			}

			if( FParse::Command (&Str,TEXT("RESET")) )
			{
				for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
				{
					UModel* Model = (*Iterator)->Model;
					Model->Modify(false);
					polyTexPan( Model, 0, 0, 1 );
				}
			}

			int32 PanU = 0; FParse::Value(Str,TEXT("U="),PanU);
			int32 PanV = 0; FParse::Value(Str,TEXT("V="),PanV);
			for( FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator )
			{
				UModel* Model = (*Iterator)->Model;
				Model->Modify(false);
				polyTexPan( Model, PanU, PanV, 0 );
			}
		}

		RedrawLevelEditingViewports();
		ULevel::LevelDirtiedEvent.Broadcast();
		return true;
	}

	return false;
}

bool UEditorEngine::Exec_Obj( const TCHAR* Str, FOutputDevice& Ar )
{
	if( FParse::Command(&Str,TEXT("EXPORT")) )//oldver
	{
		FName Package=NAME_None;

		UObject* Res;
		FParse::Value( Str, TEXT("PACKAGE="), Package );
		FString TypeName;
		FParse::Value(Str, TEXT("TYPE="), TypeName);
		UClass* Type = UClass::TryFindTypeSlow<UClass>(TypeName);
		FString TempFname;

		if
		(	Type
		&&	FParse::Value( Str, TEXT("FILE="), TempFname )
		&&	ParseObject( Str, TEXT("NAME="), Type, Res, nullptr ) )
		{
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				It->UnMark(EObjectMark(OBJECTMARK_TagImp | OBJECTMARK_TagExp));
			}
			UExporter* Exporter = UExporter::FindExporter( Res, *FPaths::GetExtension(TempFname) );
			if( Exporter )
			{
				Exporter->ParseParms( Str );
				UExporter::ExportToFile( Res, Exporter, *TempFname, 0 );
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log(TEXT("Missing file, name, or type") ));
		}
		return true;
	}
	else if( FParse::Command( &Str, TEXT( "SavePackage" ) ) )
	{
		UPackage* Pkg;
		bool bWasSuccessful = true;

		FString TempFname;
		if( FParse::Value( Str, TEXT( "FILE=" ), TempFname ) && ParseObject<UPackage>( Str, TEXT( "Package=" ), Pkg, NULL ) )
		{
			// Allow commandlets proceed without testing if we need to check out on assumption that they know what they are doing.
			if ( Pkg == nullptr || ( !IsRunningCommandlet() && ( GUnrealEd == nullptr || !GUnrealEd->CanSavePackage(Pkg ) ) ) )
			{
				return false;
			}

			const FScopedBusyCursor BusyCursor;

			bool bSilent = false;
			bool bAutosaving = false;
			bool bKeepDirty = false;
			FParse::Bool( Str, TEXT( "SILENT=" ), bSilent );
			FParse::Bool( Str, TEXT( "AUTOSAVING=" ), bAutosaving );
			FParse::Bool( Str, TEXT( "KEEPDIRTY=" ), bKeepDirty );

			// Save the package.
			const bool bIsMapPackage = UWorld::FindWorldInPackage(Pkg) != nullptr;
			const FText SavingPackageText = (bIsMapPackage)
				? FText::Format(NSLOCTEXT("UnrealEd", "SavingMapf", "Saving map {0}"), FText::FromString(Pkg->GetName()))
				: FText::Format(NSLOCTEXT("UnrealEd", "SavingAssetf", "Saving asset {0}"), FText::FromString(Pkg->GetName()));

			FScopedSlowTask SlowTask(100, SavingPackageText, !bSilent);

			uint32 SaveFlags = bAutosaving ? SAVE_FromAutosave : SAVE_None;
			if ( bKeepDirty )
			{
				SaveFlags |= SAVE_KeepDirty;
			}

			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.Error = &Ar;
			SaveArgs.bWarnOfLongFilename = !bAutosaving;
			SaveArgs.SaveFlags = SaveFlags;
			bWasSuccessful = this->SavePackage(Pkg, nullptr, *TempFname, SaveArgs);
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Log( TEXT("Missing filename") ) );
		}

		return bWasSuccessful;
	}
	else if( FParse::Command(&Str,TEXT("Rename")) )
	{
		UObject* Object=NULL;
		UObject* OldPackage=NULL, *OldGroup=NULL;
		FString NewName, NewGroup, NewPackage;
		ParseObject<UObject>( Str, TEXT("OLDPACKAGE="), OldPackage, NULL );
		ParseObject<UObject>( Str, TEXT("OLDGROUP="), OldGroup, OldPackage );
		Cast<UPackage>(OldPackage)->SetDirtyFlag(true);
		if( OldGroup )
		{
			OldPackage = OldGroup;
		}
		ParseObject<UObject>( Str, TEXT("OLDNAME="), Object, OldPackage );
		FParse::Value( Str, TEXT("NEWPACKAGE="), NewPackage );
		UPackage* Pkg = CreatePackage(*NewPackage);
		Pkg->SetDirtyFlag(true);
		FParse::Value( Str, TEXT("NEWNAME="), NewName );
		if( Object )
		{
			Object->Rename( *NewName, Pkg );
			Object->SetFlags(RF_Public|RF_Standalone);
		}

		return true;
	}

	return false;

}


AActor* UEditorEngine::SelectNamedActor(const TCHAR* TargetActorName)
{
	AActor* Actor = FindFirstObject<AActor>( TargetActorName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("SelectNamedActor") );
	if( Actor && !Actor->IsA(AWorldSettings::StaticClass()) )
	{
		SelectActor( Actor, true, true );
		return Actor;
	}
	return NULL;
}


/**
 * Handy util to tell us if Obj is 'within' a ULevel.
 *
 * @return Returns whether or not an object is 'within' a ULevel.
 */
static bool IsInALevel(UObject* Obj)
{
	return Obj->GetTypedOuter<ULevel>() != nullptr;
}


void UEditorEngine::MoveViewportCamerasToActor(AActor& Actor,  bool bActiveViewportOnly)
{
	// Pack the provided actor into a array and call the more robust version of this function.
	TArray<AActor*> Actors;

	Actors.Add( &Actor );

	MoveViewportCamerasToActor( Actors, TArray<UPrimitiveComponent*>(), bActiveViewportOnly );
}

void UEditorEngine::MoveViewportCamerasToActor(const TArray<AActor*> &Actors, bool bActiveViewportOnly)
{
	MoveViewportCamerasToActor( Actors, TArray<UPrimitiveComponent*>(), bActiveViewportOnly );
}

#if WITH_EDITORONLY_DATA

namespace UE::EditorEngine::Private
{
	// For components with an active deformer graph the true world-space vertex positions live in a GPU buffer
	// and are not yet reflected in PrimitiveComponent::Bounds. Request an async readback for each and defer the camera movement until all results
	// have been computed. InInitialBox already contains the non-deformer bounds.
	
	void DeferDeformedMeshesFocus(
		const TSet<USkinnedMeshComponent*>& InDeformedMeshes,
		const FBox& InInitialBox,
		TFunction<void(const FBox&)>&& InFocusFunction)
	{
		if (InDeformedMeshes.IsEmpty())
		{
			return;
		}
		
		struct FPendingFocusState
		{
			FPendingFocusState(const int32 NumDeformedMeshes, const FBox& InBox)
				: NumPendingMeshes(NumDeformedMeshes)
				, Box(InBox)
			{}
			
			int32 NumPendingMeshes = 0;			
			FBox Box{ ForceInit };
			FCriticalSection Mutex;
		};
		TSharedRef<FPendingFocusState> State = MakeShared<FPendingFocusState>(InDeformedMeshes.Num(), InInitialBox);

		for (USkinnedMeshComponent* DeformedMeshComponent : InDeformedMeshes)
		{
			TUniquePtr<FMeshDeformerGeometryReadbackRequest> Request = MakeUnique<FMeshDeformerGeometryReadbackRequest>();
			
			
			Request->VertexDataArraysCallback_AnyThread =
				[WeakComp = TWeakObjectPtr<USkinnedMeshComponent>(DeformedMeshComponent), State, InFocusFunction]
				(const FMeshDeformerGeometryReadbackVertexDataArrays& Data)
					{
						FBox ComponentBox(ForceInit);
						if (USkinnedMeshComponent* DeformedMeshComponent = WeakComp.Get())
						{
							if (!Data.Positions.IsEmpty())
							{
								const FTransform& Transform = DeformedMeshComponent->GetComponentTransform();
								for (const FVector3f& Pos : Data.Positions)
								{
									ComponentBox += Transform.TransformPosition(FVector(Pos));
								}
							}
							else
							{
								ComponentBox = DeformedMeshComponent->Bounds.GetBox();
							}
						}

						bool bLast = false;
						FBox FinalBox(ForceInit);
						{
							FScopeLock Lock(&State->Mutex);
							State->Box += ComponentBox;
							bLast = (--State->NumPendingMeshes == 0);
							FinalBox = State->Box;
						}

						if (bLast)
						{
							AsyncTask(ENamedThreads::GameThread, [InFocusFunction, FinalBox]()
							{
								InFocusFunction(FinalBox);
							});
						}
					};

				DeformedMeshComponent->RequestReadbackRenderGeometry(MoveTemp(Request));
			}
		}
	}

#endif

void UEditorEngine::MoveViewportCamerasToActor(const TArray<AActor*>& Actors, const TArray<UPrimitiveComponent*>& Components, bool bActiveViewportOnly)
{
	if( Actors.Num() == 0 && Components.Num() == 0 )
	{
		return;
	}

	// If the first actor is a documentation actor open its document link
	if (Actors.Num() == 1)
	{
		ADocumentationActor* DocActor = Cast<ADocumentationActor>(Actors[0]);
		if (DocActor != nullptr)
		{
			DocActor->OpenDocumentLink();
		}
	}

	TArray<AActor*> InvisLevelActors;

	// Create a bounding volume of all of the selected actors.
	FBox BoundingBox(ForceInit);

	// Skeletal mesh components deformed via a deformer graph report bounds derived from bone transforms only, the GPU-displaced vertex positions
	// are not reflected in Bounds yet. Collect them separately so we can issue an async GPU readback after the loop.
#if WITH_EDITORONLY_DATA
	TSet<USkinnedMeshComponent*> DeformedComponents;
	auto ShouldDeferFocus = [&DeformedComponents](UPrimitiveComponent* PrimitiveComponent)
	{
		USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(PrimitiveComponent);
		if ( SkinnedMeshComponent && SkinnedMeshComponent->HasMeshDeformer())
		{
			DeformedComponents.Add(SkinnedMeshComponent);
			return true;
		}
		return false;
	};
#else
	auto ShouldDeferFocus = [](UPrimitiveComponent* PrimitiveComponent) { return false; }
#endif

	if( Components.Num() > 0 )
	{
		// First look at components
		for(UPrimitiveComponent* PrimitiveComponent : Components)
		{
			if(PrimitiveComponent)
			{
				if(!FLevelUtils::IsLevelVisible(PrimitiveComponent->GetComponentLevel()))
				{
					continue;
				}

				// Some components can have huge bounds but are not visible.  Ignore these components unless it is the only component on the actor
				const bool bIgnore = Components.Num() > 1 && PrimitiveComponent->GetIgnoreBoundsForEditorFocus();

				if(!bIgnore && PrimitiveComponent->IsRegistered())
				{
					if (!ShouldDeferFocus(PrimitiveComponent))
					{
						BoundingBox += PrimitiveComponent->Bounds.GetBox();
					}
				}
			}
		}
	}
	else
	{
		TSet<AActor*> AlignActors;
		for (AActor* RootActor : Actors)
		{
			if (RootActor)
			{
				// Don't allow moving the viewport cameras to actors in invisible levels
				if (!FLevelUtils::IsLevelVisible(RootActor->GetLevel()))
				{
					InvisLevelActors.Add(RootActor);
					continue;
				}

				AlignActors.Empty(AlignActors.Num());
				AlignActors.Add(RootActor);
				RootActor->EditorGetUnderlyingActors(AlignActors);
				for (AActor* AlignActor : AlignActors)
				{
					const bool bActorIsEmitter = (Cast<AEmitter>(AlignActor) != NULL);

					if (bActorIsEmitter && bCustomCameraAlignEmitter)
					{
						const FVector DefaultExtent(CustomCameraAlignEmitterDistance, CustomCameraAlignEmitterDistance, CustomCameraAlignEmitterDistance);
						const FBox DefaultSizeBox(AlignActor->GetActorLocation() - DefaultExtent, AlignActor->GetActorLocation() + DefaultExtent);
						BoundingBox += DefaultSizeBox;
					}
					else if (USceneComponent* RootComponent = AlignActor->GetRootComponent())
					{
						TArray<USceneComponent*> SceneComponents;
						RootComponent->GetChildrenComponents(true, SceneComponents);
						SceneComponents.Add(RootComponent);

						bool bHasAtLeastOnePrimitiveComponent = false;
						for (USceneComponent* SceneComponent : SceneComponents)
						{
							UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent);

							if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
							{
								// Some components can have huge bounds but are not visible.  Ignore these components unless it is the only component on the actor
								const bool bIgnore = SceneComponents.Num() > 1 && PrimitiveComponent->GetIgnoreBoundsForEditorFocus();

								if (!bIgnore)
								{
									if (!ShouldDeferFocus(PrimitiveComponent))
									{
										FBox LocalBox(ForceInit);
										if (GLevelEditorModeTools().ComputeBoundingBoxForViewportFocus(AlignActor, PrimitiveComponent, LocalBox))
										{
											BoundingBox += LocalBox;
										}
										else
										{
											BoundingBox += PrimitiveComponent->Bounds.GetBox();
										}
									}

									bHasAtLeastOnePrimitiveComponent = true;
								}
							}
						}

						if (!bHasAtLeastOnePrimitiveComponent)
						{
							BoundingBox += RootComponent->GetComponentLocation();
						}

					}
				}
			}
		}
	}

	// Warn the user with a suppressable dialog if they attempted to zoom to actors that are in an invisible level
	if ( InvisLevelActors.Num() > 0 )
	{
		FString InvisLevelActorString;
		for ( TArray<AActor*>::TConstIterator InvisLevelActorIter( InvisLevelActors ); InvisLevelActorIter; ++InvisLevelActorIter )
		{
			const AActor* CurActor = *InvisLevelActorIter;
			InvisLevelActorString += FString::Printf( TEXT("%s\n"), *CurActor->GetName() );
		}
		const FText WarningMessage = FText::Format( NSLOCTEXT("UnrealEd", "MoveCameraToInvisLevelActor_Message", "Attempted to move camera to actors whose levels are currently not visible:\n{0}"), FText::FromString(InvisLevelActorString) );

		FSuppressableWarningDialog::FSetupInfo Info( WarningMessage, NSLOCTEXT("UnrealEd", "MoveCameraToInvisLevelActor_Title", "Hidden Actors"), TEXT("MoveViewportCamerasToActorsInInvisLevel") );
		Info.ConfirmText = NSLOCTEXT("UnrealEd", "InvalidMoveCommand", "Close");

		FSuppressableWarningDialog InvisLevelActorWarning( Info );
		InvisLevelActorWarning.ShowModal();
	}

#if WITH_EDITORONLY_DATA
	if (!DeformedComponents.IsEmpty())
	{
		UE::EditorEngine::Private::DeferDeformedMeshesFocus(DeformedComponents, BoundingBox, [this, bActiveViewportOnly, Actors](const FBox& FinalBox)
		{
			MoveViewportCamerasToBox(FinalBox, bActiveViewportOnly);
			FEditorDelegates::OnFocusViewportOnActors.Broadcast(Actors);
		});
		return;
	}
#endif // WITH_EDITORONLY_DATA

	MoveViewportCamerasToBox(BoundingBox, bActiveViewportOnly);
	// Notify 'focus on actors' delegate
	FEditorDelegates::OnFocusViewportOnActors.Broadcast(Actors);
}

void UEditorEngine::MoveViewportCamerasToComponent(const USceneComponent* Component, bool bActiveViewportOnly)
{
	/** Limit the minimum size of the bounding box when centering cameras on individual components to avoid extreme zooming */
	constexpr float MinComponentBoundsForZoom = 50.0f;

	if (Component != nullptr)
	{
		if (FLevelUtils::IsLevelVisible(Component->GetComponentLevel()) && Component->IsRegistered())
		{
			FBox Box = Component->Bounds.GetBox();
			FVector Center;
			FVector Extents;
			Box.GetCenterAndExtents(Center, Extents);

			// Apply a minimum size to the extents of the component's box to avoid the camera's zooming too close to small or zero-sized components
			if (Extents.SizeSquared() < MinComponentBoundsForZoom * MinComponentBoundsForZoom)
			{
				FVector NewExtents(MinComponentBoundsForZoom, SMALL_NUMBER, SMALL_NUMBER);
				Box = FBox(Center - NewExtents, Center + NewExtents);
			}

			MoveViewportCamerasToBox(Box, bActiveViewportOnly);
		}
	}
}

void UEditorEngine::MoveViewportCamerasToElement(const UTypedElementSelectionSet* SelectionSet, bool bActiveViewportOnly) const
{
	if (!SelectionSet || !SelectionSet->HasSelectedElements(ITypedElementWorldInterface::UClassType::StaticClass()))
	{
		return;
	}

	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	if (!Registry)
	{
		return;
	}

	// GetFocusBounds now returns false for any USkinnedMeshComponent that has an active deformer graph. BoundingBox therefore contains only the
	// correctly-bounded non-deformer primitives.
	FBox BoundingBox(ForceInit);

	FTypedElementListConstRef ElementsToView = SelectionSet->GetNormalizedElementList(SelectionSet->GetElementList(), FTypedElementSelectionNormalizationOptions());

	if (bActiveViewportOnly)
	{
		FBoxSphereBounds Bounds(ForceInit);
		if (GCurrentLevelEditingViewportClient->GetFocusBounds(ElementsToView, Bounds))
		{
			BoundingBox += Bounds.GetBox();
		}
	}
	else
	{
		// Use all viewports.
		for (FLevelEditorViewportClient* LinkedViewportClient : GetLevelViewportClients())
		{
			FBoxSphereBounds Bounds(ForceInit);
			if (LinkedViewportClient->GetFocusBounds(ElementsToView, Bounds))
			{
				BoundingBox += Bounds.GetBox();
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// Skeletal mesh components deformed via a deformer graph report bounds derived from bone transforms only, the GPU-displaced vertex positions
	// are not reflected in Bounds yet. Collect them separately so we can issue an async GPU readback after the loop.
	TSet<USkinnedMeshComponent*> DeformedComponents;

	ElementsToView->ForEachElementHandle( [&DeformedComponents](const FTypedElementHandle& InHandle)
    {
		constexpr bool bSilent = true;
	    if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InHandle, bSilent))
	    {
		    TInlineComponentArray<USkinnedMeshComponent*> SkinnedMeshComponents(Actor);
		    for (USkinnedMeshComponent* SkinnedMeshComponent : SkinnedMeshComponents)
		    {
			    if (SkinnedMeshComponent && SkinnedMeshComponent->HasMeshDeformer())
			    {
				    DeformedComponents.Add(SkinnedMeshComponent);
			    }
		    }
	    }
	    else if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InHandle, bSilent))
	    {
		    if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(Component))
		    {
			    if (SkinnedMeshComponent->HasMeshDeformer())
			    {
				    DeformedComponents.Add(SkinnedMeshComponent);
			    }
		    }
	    }
	    return true;
    });

	if (!DeformedComponents.IsEmpty())
	{
		UE::EditorEngine::Private::DeferDeformedMeshesFocus(DeformedComponents, BoundingBox, [this, bActiveViewportOnly](const FBox& FinalBox)
		{
			MoveViewportCamerasToBox(FinalBox, bActiveViewportOnly);
		});
		return;
	}
#endif // WITH_EDITORONLY_DATA
	
	MoveViewportCamerasToBox(BoundingBox, bActiveViewportOnly);
}

void UEditorEngine::MoveViewportCamerasToBox(const FBox& BoundingBox, bool bActiveViewportOnly, float DrawDebugBoxTimeInSeconds) const
{
	// Make sure we had at least one non-null actor in the array passed in.
	if (BoundingBox.GetSize() != FVector::ZeroVector || BoundingBox.GetCenter() != FVector::ZeroVector)
	{
		if (bActiveViewportOnly)
		{
			if (GCurrentLevelEditingViewportClient)
			{
				GCurrentLevelEditingViewportClient->FocusViewportOnBox(BoundingBox);

				// Update Linked Orthographic viewports.
				if (GCurrentLevelEditingViewportClient->IsOrtho() && GetDefault<ULevelEditorViewportSettings>()->bUseLinkedOrthographicViewports)
				{
					// Search through all viewports
					for (FLevelEditorViewportClient* LinkedViewportClient : GetLevelViewportClients())
					{
						// Only update other orthographic viewports
						if (LinkedViewportClient && LinkedViewportClient != GCurrentLevelEditingViewportClient && LinkedViewportClient->IsOrtho())
						{
							LinkedViewportClient->FocusViewportOnBox(BoundingBox);
						}
					}
				}
			}

		}
		else
		{
			// Update all viewports.
			for (FLevelEditorViewportClient* LinkedViewportClient : GetLevelViewportClients())
			{
				// Skip viewports that are locked on an actor, like actor previews, except the current viewport as we want to allow focusing while piloting
				if (!LinkedViewportClient->IsAnyActorLocked() || LinkedViewportClient == GCurrentLevelEditingViewportClient)
				{
					LinkedViewportClient->FocusViewportOnBox(BoundingBox);
				}
			}
		}

		if (DrawDebugBoxTimeInSeconds > 0.f)
		{
			const FWorldContext& Context = const_cast<UEditorEngine*>(this)->GetEditorWorldContext();
			if (const UWorld* World = Context.World())
			{
				const FColor DarkGray(40, 40, 40);
				DrawDebugBox(World, BoundingBox.GetCenter(), BoundingBox.GetExtent(), DarkGray, false, DrawDebugBoxTimeInSeconds);
			}
		}
	}
}

bool UEditorEngine::SnapElementTo(const FTypedElementHandle& InElementHandle, const bool InAlign, const bool InUseLineTrace, const bool InUseBounds, const bool InUsePivot, const FTypedElementHandle& InDestination, TArrayView<const FTypedElementHandle> InElementsToIgnore)
{
	if (!InElementHandle || InElementHandle == InDestination)
	{
		return false;
	}

	const UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	TTypedElement<ITypedElementWorldInterface> ElementWorldHandle = Registry->GetElement<ITypedElementWorldInterface>(InElementHandle);
	if (!ElementWorldHandle)
	{
		return false;
	}

	FTransform ElementTransform;
	if (!ElementWorldHandle.GetWorldTransform(ElementTransform))
	{
		return false;
	}

	FBoxSphereBounds ElementBounds;
	if (!ElementWorldHandle.GetBounds(ElementBounds))
	{
		return false;
	}

	ABrush* Brush = nullptr;
	if (TTypedElement<ITypedElementObjectInterface> ElementObjectHandle = Registry->GetElement<ITypedElementObjectInterface>(InElementHandle))
	{
		Brush = Cast<ABrush>(ElementObjectHandle.GetObject());
	}

	FVector	StartLocation = ElementTransform.GetLocation();
	FVector	LocationOffset = FVector::ZeroVector;
	FVector	Extent = FVector::ZeroVector;
	bool UseLineTrace = Brush ? true : InUseLineTrace;
	bool UseBounds = Brush ? true : InUseBounds;

	if (UseLineTrace && UseBounds)
	{
		if (InUsePivot)
		{
			// Will do a line trace from the pivot location.
			StartLocation = GetPivotLocation();
		}
		else
		{
			// Will do a line trace from the center bottom of the bounds through the world. Will begin at the bottom center of the component's bounds.
			StartLocation = ElementBounds.Origin;
			StartLocation.Z -= ElementBounds.BoxExtent.Z;
		}

		// Forces a line trace.
		Extent = FVector::ZeroVector;
		LocationOffset = StartLocation - ElementTransform.GetLocation();
	}
	else if (UseLineTrace)
	{
		// This will be false if multiple objects are selected. In that case the actor's position should be used so all the objects do not go to the same point.
		if (InUsePivot && !InDestination)	// @todo: If the destination actor is part of the selection tho, we can't use the pivot! (remove check if not)
		{
			StartLocation = GetPivotLocation();
		}
		else
		{
			StartLocation = ElementTransform.GetLocation();
		}

		// Forces a line trace.
		Extent = FVector::ZeroVector;
		LocationOffset = StartLocation - ElementTransform.GetLocation();
	}
	else
	{
		StartLocation = ElementBounds.Origin;

		Extent = ElementBounds.BoxExtent;
		LocationOffset = StartLocation - ElementTransform.GetLocation();
	}

	FVector Direction = FVector(0.f, 0.f, -1.f);
	if (TTypedElement<ITypedElementWorldInterface> DestinationWorldHandle = Registry->GetElement<ITypedElementWorldInterface>(InDestination))	// If a destination actor was specified, work out the direction
	{
		FTransform DestinationTransform;
		if (!DestinationWorldHandle.GetWorldTransform(DestinationTransform))
		{
			return false;
		}

		FBoxSphereBounds DestinationBounds;
		if (!DestinationWorldHandle.GetBounds(DestinationBounds))
		{
			return false;
		}

		FVector	EndLocation = DestinationTransform.GetLocation();

		// Code here assumes you want to same type of end point as the start point used, comment out to just use the destination actors origin!
		if (UseLineTrace && UseBounds)
		{
			EndLocation = DestinationBounds.Origin;
			EndLocation.Z -= DestinationBounds.BoxExtent.Z;
		}
		else if (UseLineTrace)
		{
			EndLocation = DestinationTransform.GetLocation();
		}
		else
		{
			EndLocation = DestinationBounds.Origin;
		}

		if (EndLocation.Equals(StartLocation))
		{
			return false;
		}
		Direction = (EndLocation - StartLocation);
		Direction.Normalize();
	}

	// In the case that we're about to do a line trace from a brush, move the start position so it's guaranteed to be very slightly
	// outside of the brush bounds. The BSP geometry is double-sided which will give rise to an unwanted hit.
	if (Brush)
	{
		const float fTinyOffset = 0.01f;
		StartLocation.Z = Brush->GetRootComponent()->Bounds.Origin.Z - Brush->GetRootComponent()->Bounds.BoxExtent.Z - fTinyOffset;
	}

	// Do the actual sweep test. We try to collide against the world, straight down from our current position.
	// If we hit anything, we will move the actor to a position that lets it rest on the floor.
	FTransform NewTransform;
	if (ElementWorldHandle.FindSuitableTransformAlongPath(StartLocation, StartLocation + Direction * WORLD_MAX, FCollisionShape::MakeBox(Extent), InElementsToIgnore, NewTransform))
	{
		NewTransform.SetTranslation(NewTransform.GetTranslation() - LocationOffset);
		if (!InAlign)
		{
			NewTransform.SetRotation(ElementTransform.GetRotation());
		}
		NewTransform.SetScale3D(ElementTransform.GetScale3D());

		// Switch to the pie world if we have one
		FScopedConditionalWorldSwitcher WorldSwitcher(GCurrentLevelEditingViewportClient);

		ElementWorldHandle.NotifyMovementStarted();
		ElementWorldHandle.SetWorldTransform(NewTransform);
		ElementWorldHandle.NotifyMovementEnded();

		if (Brush)
		{
			RebuildAlteredBSP();
		}

		GCurrentLevelEditingViewportClient->GetModeTools()->ActorMoveNotify();

		return true;
	}

	return false;
}

void UEditorEngine::MoveActorInFrontOfCamera( AActor& InActor, const FVector& InCameraOrigin, const FVector& InCameraDirection )
{
	const FVector NewLocation = FActorPositioning::GetActorPositionInFrontOfCamera(InActor, InCameraOrigin, InCameraDirection);

	// Move the actor to its new location.  Not checking for collisions
	InActor.TeleportTo( NewLocation, InActor.GetActorRotation(), false, true );

	if( InActor.IsSelected() )
	{
		// If the actor was selected, reselect it so the widget is set in the correct location
		SelectNone( false, true );
		SelectActor( &InActor, true, true );
	}

	// Switch to the pie world if we have one
	FScopedConditionalWorldSwitcher WorldSwitcher( GCurrentLevelEditingViewportClient );

	InActor.InvalidateLightingCache();
	InActor.PostEditMove( true );
}

void UEditorEngine::SnapViewTo(const FTypedElementHandle& InElementHandle)
{
	if (TTypedElement<ITypedElementWorldInterface> ElementWorldHandle = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementWorldInterface>(InElementHandle))
	{
		FTransform ElementTransform;
		if (ElementWorldHandle.GetWorldTransform(ElementTransform))
		{
			for (FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
			{
				if (ViewportClient->IsPerspective())
				{
					ViewportClient->SetViewLocation(ElementTransform.GetLocation());
					ViewportClient->SetViewRotation(ElementTransform.Rotator());
					ViewportClient->Invalidate();
				}
			}
		}
	}
}

void UEditorEngine::RemovePerspectiveViewRotation(bool Roll, bool Pitch, bool Yaw)
{
	for(FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
	{
		if (ViewportClient->IsPerspective() && !ViewportClient->GetActiveActorLock().IsValid())
		{
			FVector RotEuler = ViewportClient->GetViewRotation().Euler();

			if (Roll)
			{
				RotEuler.X = 0.0f;
			}
			if (Pitch)
			{
				RotEuler.Y = 0.0f;
			}
			if (Yaw)
			{
				RotEuler.Z = 0.0f;
			}

			ViewportClient->SetViewRotation( FRotator::MakeFromEuler(RotEuler) );
			ViewportClient->Invalidate();
		}
	}
}

bool UEditorEngine::Exec_Camera( const TCHAR* Str, FOutputDevice& Ar )
{
	const bool bAlign = FParse::Command( &Str,TEXT("ALIGN") );
	const bool bSnap = !bAlign && FParse::Command( &Str, TEXT("SNAP") );

	if ( !bAlign && !bSnap )
	{
		return false;
	}

	AActor* TargetSelectedActor = NULL;

	if( bAlign )
	{
		// Try to select the named actor if specified.
		FString TempStr;
		if( FParse::Value( Str, TEXT("NAME="), TempStr ) )
		{
			TargetSelectedActor = SelectNamedActor( *TempStr );
			if ( TargetSelectedActor )
			{
				NoteSelectionChange();
			}
		}

		// Position/orient viewports to look at the selected actor.
		const bool bActiveViewportOnly = FParse::Command( &Str,TEXT("ACTIVEVIEWPORTONLY") );

		// If they specifed a specific Actor to align to, then align to that actor only.
		// Otherwise, build a list of all selected actors and fit the camera to them.
		// If there are no actors selected, give an error message and return false.
		if ( TargetSelectedActor )
		{
			MoveViewportCamerasToActor( *TargetSelectedActor, bActiveViewportOnly );
			Ar.Log( TEXT("Aligned camera to the specified actor.") );
		}
		else
		{
			FBox ComponentVisBoundingBox;
			bool bComponentVisHasFocusOnSelectionBoundingBox = GUnrealEd && GUnrealEd->ComponentVisManager.HasFocusOnSelectionBoundingBox(ComponentVisBoundingBox);

			if (bComponentVisHasFocusOnSelectionBoundingBox)
			{
				MoveViewportCamerasToBox(ComponentVisBoundingBox, bActiveViewportOnly);
			}
			else
			{
				const UTypedElementSelectionSet* SelectionSet = GetSelectedActors()->GetElementSelectionSet();
				if (SelectionSet && SelectionSet->HasSelectedElements())
				{
					MoveViewportCamerasToElement(SelectionSet, bActiveViewportOnly);
				}
				/*
				if (Actors.Num() || SelectedComponents.Num())
				{
					MoveViewportCamerasToActor(Actors, SelectedComponents, bActiveViewportOnly);
					return true;
				}
				*/
				else
				{
					Ar.Log(TEXT("Can't find target element."));
					return false;
				}
			}
		}
	}
	else if ( bSnap )
	{
		const UTypedElementSelectionSet* SelectionSet = GetSelectedActors()->GetElementSelectionSet();
		if (SelectionSet && SelectionSet->HasSelectedElements())
		{
			if (FTypedElementHandle SelectedElement = SelectionSet->GetElementList()->GetElementHandleAt(0))
			{
				// Set perspective viewport camera parameters to that of the selected camera.
				SnapViewTo(SelectedElement);
				Ar.Log(TEXT("Snapped camera to the first selected element."));
			}
		}
	}

	return true;
}

bool UEditorEngine::Exec_Transaction(const TCHAR* Str, FOutputDevice& Ar)
{
	if (FParse::Command(&Str,TEXT("REDO")))
	{
		RedoTransaction();
	}
	else if (FParse::Command(&Str,TEXT("UNDO")))
	{
		UndoTransaction();
	}

	return true;
}

void UEditorEngine::BroadcastPostUndoRedo(const FTransactionContext& UndoContext, bool bWasUndo)
{
	// This sanitization code can be removed once blueprint ::Conform(ImplementedEvents/ImplementedInterfaces)
	// functions have been fixed. For the time being it improves editor stability, though:
	UEdGraphPin::SanitizePinsPostUndoRedo();

	check(InflightUndoClients.IsEmpty());

	// Note that we use a copy here as clients can register/unregister with the undo system while in PostUndo()/PostRedo()
	// which modifies UndoClients during the loop. This can cause an infinite loop where the iterator never finishes.
	InflightUndoClients = UndoClients.Array();
	for (FEditorUndoClient* Client : InflightUndoClients)
	{
		if (Client && Client->MatchesContext(UndoContext, CurrentUndoRedoContext->TransactionObjects))
		{
			if (bWasUndo)
			{
				Client->PostUndo( true );
			}
			else
			{
				Client->PostRedo( true );
			}
		}
	}

	// Invalidate all viewports
	InvalidateAllViewportsAndHitProxies();

	InflightUndoClients.Empty();
	FEditorDelegates::PostUndoRedo.Broadcast();
}

bool UEditorEngine::Exec_Particle(const TCHAR* Str, FOutputDevice& Ar)
{
	bool bHandled = false;
	UE_LOGF(LogEditorServer, Log, "Exec Particle!");

	// Store off the input string here, as it is adjusted by subsequent parsing commands...
	const TCHAR* InputStr = Str;
	if (FParse::Command(&Str,TEXT("RESET")))
	{
		TArray<AEmitter*> EmittersToReset;
		if (FParse::Command(&Str,TEXT("SELECTED")))
		{
			// Reset any selected emitters in the level
			for (FSelectionIterator It(GetSelectedActorIterator()) ; It ; ++It)
			{
				AActor* Actor = static_cast<AActor*>( *It );
				checkSlow(Actor->IsA(AActor::StaticClass()));

				AEmitter* Emitter = Cast<AEmitter>(Actor);
				if (Emitter)
				{
					Emitter->ResetInLevel();
				}
			}
		}
		else if (FParse::Command(&Str,TEXT("ALL")))
		{
			// Reset ALL emitters in the level
			for (TObjectIterator<AEmitter> It;It;++It)
			{
				AEmitter* Emitter = *It;
				Emitter->ResetInLevel();
			}
		}
	}

	// Invoke any downstream handlers (like the Niagara editor plugin)
	if (ExecParticleInvokedEvent.IsBound())
	{
		ExecParticleInvokedEvent.Broadcast(InputStr);
	}
	return bHandled;
}


void UEditorEngine::ExecFile( UWorld* InWorld, const TCHAR* InFilename, FOutputDevice& Ar )
{
	FString FileTextContents;
	if( FFileHelper::LoadFileToString( FileTextContents, InFilename ) )
	{
		UE_LOGF(LogEditorServer, Log,  "Execing file: %ls...", InFilename );

		const TCHAR* FileString = *FileTextContents;
		FString LineString;
		while( FParse::Line( &FileString, LineString ) )
		{
			Exec( InWorld, *LineString, Ar );
		}
	}
	else
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Can't find file '%s'"), InFilename));
	}
}

/**
 *	Fix up bad animnotifiers that has wrong outers
 *	It uses all loaded animsets
 */
bool FixUpBadAnimNotifiers()
{
	// Iterate over all interp groups in the current level and remove the unreferenced anim sets
	for (TObjectIterator<UAnimSet> It; It; ++It)
	{
		UAnimSet* AnimSet = *It;

		for (int32 J=0; J<AnimSet->Sequences.Num(); ++J)
		{
			UAnimSequence * AnimSeq = AnimSet->Sequences[J];
			// iterate over all animnotifiers
			// if any animnotifier outer != current animsequence
			// then add to map
			for (int32 I=0; I<AnimSeq->Notifies.Num(); ++I)
			{
				if (AnimSeq->Notifies[I].Notify && AnimSeq->Notifies[I].Notify->GetOuter()!=AnimSeq)
				{
					// fix animnotifiers
					UE_LOGF(LogEditorServer, Log, "Animation[%ls] Notifier[%ls:%d] is being fixed (Current Outer:%ls)", *AnimSeq->GetName(), *AnimSeq->Notifies[I].Notify->GetName(), I, *AnimSeq->Notifies[I].Notify->GetOuter()->GetName());
					AnimSeq->Notifies[I].Notify = NewObject<UAnimNotify>(AnimSeq, AnimSeq->Notifies[I].Notify->GetClass(), NAME_None, RF_NoFlags, AnimSeq->Notifies[I].Notify);
					UE_LOGF(LogEditorServer, Log, "After fixed (Current Outer:%ls)", *AnimSeq->Notifies[I].Notify->GetOuter()->GetName());
					AnimSeq->MarkPackageDirty();
				}
			}
		}
	}

	return true;
}

/**
 *	Helper function for listing package dependencies
 *
 *	@param	InStr					The EXEC command string
 */
void ListMapPackageDependencies(const TCHAR* InStr)
{
	TArray<UPackage*> PackagesToProcess;
	TMap<FString,bool> ReferencedPackages;
	TMap<FString,bool> ReferencedPackagesWithTextures;
	bool bTexturesOnly = false;
	bool bResave = false;

	// Check the 'command line'
	if (FParse::Command(&InStr,TEXT("TEXTURES"))) // LISTMAPPKGDEPENDENCIES TEXTURE
	{
		bTexturesOnly = true;
		//@todo. Implement resave option!
		if (FParse::Command(&InStr,TEXT("RESAVE"))) // LISTMAPPKGDEPENDENCIES TEXTURE RESAVE
		{
			bResave = true;
		}
	}
	UE_LOGF(LogEditorServer, Warning, "Listing MAP package dependencies%ls%ls",
		bTexturesOnly ? TEXT(" with TEXTURES") : TEXT(""),
		bResave ? TEXT(" RESAVE") : TEXT(""));

	// For each loaded level, list out it's dependency map
	for (TObjectIterator<ULevel> LevelIt; LevelIt; ++LevelIt)
	{
		ULevel* Level = *LevelIt;
		UPackage* LevelPackage = Level->GetOutermost();
		FString LevelPackageName = LevelPackage->GetName();
		UE_LOGF(LogEditorServer, Warning, "\tFound level %ls - %ls", *Level->GetPathName(), *LevelPackageName);

		if (LevelPackageName.StartsWith(TEXT("/Temp/Untitled")) == false)
		{
			PackagesToProcess.AddUnique(LevelPackage);
		}
	}

	// For each package in the list, generate the appropriate package dependency list
	for (int32 PkgIdx = 0; PkgIdx < PackagesToProcess.Num(); PkgIdx++)
	{
		UPackage* ProcessingPackage = PackagesToProcess[PkgIdx];
		FString ProcessingPackageName = ProcessingPackage->GetName();
		UE_LOGF(LogEditorServer, Warning, "Processing package %ls...", *ProcessingPackageName);
		if (ProcessingPackage->IsDirty() == true)
		{
			UE_LOGF(LogEditorServer, Warning, "\tPackage is dirty so results may not contain all references!");
			UE_LOGF(LogEditorServer, Warning, "\tResave packages and run again to ensure accurate results.");
		}

		auto Linker = ProcessingPackage->GetLinker();
		if (!Linker)
		{
			// Create a new linker object which goes off and tries load the file.
			Linker = GetPackageLinker(nullptr, FPackagePath::FromPackageNameChecked(ProcessingPackage->GetName()), LOAD_None, nullptr, nullptr);
		}
		if (Linker)
		{
			for (int32 ImportIdx = 0; ImportIdx < Linker->ImportMap.Num(); ImportIdx++)
			{
				// don't bother outputting package references, just the objects
				if (Linker->ImportMap[ImportIdx].ClassName != NAME_Package)
				{
					// get package name of the import
					FString ImportPackage = FPackageName::FilenameToLongPackageName(Linker->GetImportPathName(ImportIdx));
					int32 PeriodIdx = ImportPackage.Find(TEXT("."), ESearchCase::CaseSensitive);
					if (PeriodIdx != INDEX_NONE)
					{
						ImportPackage.LeftInline(PeriodIdx, EAllowShrinking::No);
					}
					ReferencedPackages.Add(ImportPackage, true);
				}
			}
		}
		else
		{
			UE_LOGF(LogEditorServer, Warning, "\t\tCouldn't get package linker. Skipping...");
		}
	}

	if (bTexturesOnly == true)
	{
		FName CheckTexture2DName(TEXT("Texture2D"));
		FName CheckCubeTextureName(TEXT("TextureCube"));
		FName CheckTexture2DArrayName(TEXT("Texture2DArray"));
		FName CheckLightmap2DName(TEXT("Lightmap2D"));
		FName CheckShadowmap2DName(TEXT("Shadowmap2D"));

		for (TMap<FString,bool>::TIterator PkgIt(ReferencedPackages); PkgIt; ++PkgIt)
		{
			FString RefdPkgName = PkgIt.Key();
			UPackage* RefdPackage = LoadPackage(NULL, *RefdPkgName, LOAD_None);
			if (RefdPackage != NULL)
			{
				auto Linker = RefdPackage->GetLinker();
				if (!Linker)
				{
					// Create a new linker object which goes off and tries load the file.
					FPackagePath PackagePath;
					if (!FPackagePath::TryFromPackageName(RefdPkgName, PackagePath))
					{
						UE_LOGF(LogEditorServer, Warning, "Invalid PackageName \"%ls\"", *RefdPkgName);
					}
					else
					{
						Linker = GetPackageLinker(nullptr, PackagePath, LOAD_None, nullptr, nullptr);
					}
				}
				if (Linker)
				{
					for (int32 ExportIdx = 0; ExportIdx < Linker->ExportMap.Num(); ExportIdx++)
					{
						FTopLevelAssetPath CheckClassPathName(Linker->GetExportClassPackage(ExportIdx), Linker->GetExportClassName(ExportIdx));
						UClass* CheckClass = FindObject<UClass>(CheckClassPathName, EFindObjectFlags::ExactClass);
						if (
							(CheckClass != NULL) &&
							(CheckClass->IsChildOf(UTexture::StaticClass()) == true)
							)
						{
							ReferencedPackagesWithTextures.Add(RefdPkgName, true);
							break;
						}
					}
				}
			}
		}
		ReferencedPackages.Empty();
		ReferencedPackages = ReferencedPackagesWithTextures;
	}

	UE_LOGF(LogEditorServer, Warning, "--------------------------------------------------------------------------------");
	UE_LOGF(LogEditorServer, Warning, "Referenced packages%ls...",
		bTexturesOnly ? TEXT(" (containing Textures)") : TEXT(""));
	for (TMap<FString,bool>::TIterator PkgIt(ReferencedPackages); PkgIt; ++PkgIt)
	{
		UE_LOGF(LogEditorServer, Warning, "\t%ls", *(PkgIt.Key()));
	}
}

COREUOBJECT_API void DumpClassSchemas(const TCHAR* Str, FOutputDevice& Ar);

bool UEditorEngine::Exec_Editor( UWorld* InWorld, const TCHAR* Stream, FOutputDevice& Ar )
{
	TCHAR ErrorTemp[256]=TEXT("Setup: ");
	bool bProcessed=false;

	// Echo the command to the log window
	if( FCString::Strlen(Stream)<200 )
	{
		FCString::Strcat( ErrorTemp, Stream );
		DEFINE_LOG_CATEGORY_STATIC(Cmd, All, All);
		UE_LOGF(Cmd, Log, "%ls", Stream );
	}

	GStream = Stream;

	FString CommandTemp = Stream;
	const TCHAR* Str = *CommandTemp;

	FCString::Strncpy( ErrorTemp, Str, 79 );
	ErrorTemp[79]=0;

	if( SafeExec( InWorld, Stream, Ar ) )
	{
		return true;
	}

	//------------------------------------------------------------------------------------
	// MISC
	//
	else if( FParse::Command(&Str,TEXT("EDCALLBACK")) )
	{
		HandleCallbackCommand( InWorld, Str, Ar );
	}
	else if(FParse::Command(&Str,TEXT("STATICMESH")))
	{
		if( Exec_StaticMesh( InWorld, Str, Ar ) )
		{
			return true;
		}
	}
	else if( FParse::Command(&Str,TEXT("TESTPROPS")))
	{
		return HandleTestPropsCommand( Str, Ar );
	}
	//------------------------------------------------------------------------------------
	// BRUSH
	//
	else if( FParse::Command(&Str,TEXT("BRUSH")) )
	{
		if( Exec_Brush( InWorld, Str, Ar ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// BSP
	//
	else if( FParse::Command( &Str, TEXT("BSP") ) )
	{
		return CommandIsDeprecated( *CommandTemp, Ar );
	}
	//------------------------------------------------------------------------------------
	// LIGHT
	//
	else if( FParse::Command( &Str, TEXT("LIGHT") ) )
	{
		return CommandIsDeprecated( *CommandTemp, Ar );
	}
	//------------------------------------------------------------------------------------
	// MAP
	//
	else if (FParse::Command(&Str,TEXT("MAP")))
	{
		if( HandleMapCommand( Str, Ar, InWorld ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// SELECT: Rerouted to mode-specific command
	//
	else if( FParse::Command(&Str,TEXT("SELECT")) )
	{
		HandleSelectCommand( Str, Ar, InWorld );
	}
	//------------------------------------------------------------------------------------
	// DELETE: Rerouted to mode-specific command
	//
	else if (FParse::Command(&Str,TEXT("DELETE")))
	{
		return HandleDeleteCommand( Str, Ar, InWorld );
	}
	//------------------------------------------------------------------------------------
	// DUPLICATE: Rerouted to mode-specific command
	//
	else if (FParse::Command(&Str,TEXT("DUPLICATE")))
	{
		return Exec( InWorld, TEXT("ACTOR DUPLICATE") );
	}
	//------------------------------------------------------------------------------------
	// POLY: Polygon adjustment and mapping
	//
	else if( FParse::Command(&Str,TEXT("POLY")) )
	{
		if( Exec_Poly( InWorld, Str, Ar ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// ANIM: All mesh/animation management.
	//
	else if( FParse::Command(&Str,TEXT("NEWANIM")) )
	{
		return CommandIsDeprecated( *CommandTemp, Ar );
	}
	//------------------------------------------------------------------------------------
	// Transaction tracking and control
	//
	else if( FParse::Command(&Str,TEXT("TRANSACTION")) )
	{
		if( Exec_Transaction( Str, Ar ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// General objects
	//
	else if( FParse::Command(&Str,TEXT("OBJ")) )
	{
		if( Exec_Obj( Str, Ar ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// CAMERA: cameras
	//
	else if( FParse::Command(&Str,TEXT("CAMERA")) )
	{
		if( Exec_Camera( Str, Ar ) )
		{
			return true;
		}
	}
	//------------------------------------------------------------------------------------
	// PARTICLE: Particle system-related commands
	//
	if (FParse::Command(&Str,TEXT("PARTICLE")))
	{
		if( Exec_Particle(Str, Ar) )
		{
			return true;
		}
	}
	//----------------------------------------------------------------------------------
	// QUIT_EDITOR - Closes the wx main editor frame.  We need to do this in slate but it is routed differently.
	// Don't call quit_editor directly with slate
	//
	else if( FParse::Command(&Str,TEXT("QUIT_EDITOR")) )
	{
		CloseEditor();
		return true;
	}
	else if( FParse::Command(&Str,TEXT("CLOSE_SLATE_MAINFRAME")) )
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.RequestCloseEditor();
		return true;
	}
	//----------------------------------------------------------------------------------
	// LIGHTMASSDEBUG - Toggles whether UnrealLightmass.exe is launched automatically (default),
	// or must be launched manually (e.g. through a debugger) with the -debug command line parameter.
	//
	else if( FParse::Command(&Str,TEXT("LIGHTMASSDEBUG")) )
	{
		return HandleLightmassDebugCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LIGHTMASSSTATS - Toggles whether all participating Lightmass agents will report
	// back detailed stats to the log.
	//
	else if( FParse::Command(&Str,TEXT("LIGHTMASSSTATS")) )
	{
		return HandleLightmassStatsCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// SWARMDISTRIBUTION - Toggles whether to enable Swarm distribution for Jobs.
	// Default is off (local builds only).
	//
	else if( FParse::Command(&Str,TEXT("SWARMDISTRIBUTION")) )
	{
		return HandleSwarmDistributionCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMIMM - Toggles Lightmass ImmediateImport mode.
	//	If true, Lightmass will import mappings immediately as they complete.
	//	It will not process them, however.
	//	Default value is false
	//
	else if (
		( FParse::Command(&Str,TEXT("LMIMMEDIATE")) ) ||
		( FParse::Command(&Str,TEXT("LMIMM")) ))
	{
		return HandleLightmassImmediateImportCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMIMP - Toggles Lightmass ImmediateProcess mode.
	//	If true, Lightmass will process appropriate mappings as they are imported.
	//	NOTE: Requires ImmediateMode be enabled to actually work.
	//	Default value is false
	//
	else if ( FParse::Command(&Str,TEXT("LMIMP")) )
	{
		return HandleLightmassImmediateProcessCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMSORT - Toggles Lightmass sorting mode.
	//	If true, Lightmass will sort mappings by texel cost.
	//
	else if ( FParse::Command(&Str,TEXT("LMSORT")) )
	{
		return HandleLightmassSortCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMDEBUGMAT - Toggles Lightmass dumping of exported material samples.
	//	If true, Lightmass will write out BMPs for each generated material property
	//	sample to <GAME>\ScreenShots\Materials.
	//
	else if ( FParse::Command(&Str,TEXT("LMDEBUGMAT")) )
	{
		return HandleLightmassDebugMaterialCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMPADDING - Toggles Lightmass padding of mappings.
	//
	else if ( FParse::Command(&Str,TEXT("LMPADDING")) )
	{
		return HandleLightmassPaddingCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMDEBUGPAD - Toggles Lightmass debug padding of mappings.
	// Means nothing if LightmassPadMappings is not enabled...
	//
	else if ( FParse::Command(&Str,TEXT("LMDEBUGPAD")) )
	{
		return HandleLightmassDebugPaddingCommand( Str, Ar );
	}
	//----------------------------------------------------------------------------------
	// LMPROFILE - Switched settings for Lightmass to a mode suited for profiling.
	// Specifically, it disabled ImmediateImport and ImmediateProcess of completed mappings.
	//
	else if( FParse::Command(&Str,TEXT("LMPROFILE")) )
	{
		return HandleLightmassProfileCommand( Str, Ar );
	}
	//------------------------------------------------------------------------------------
	// Other handlers.
	//
	else if( InWorld && InWorld->Exec( InWorld, Stream, Ar) )
	{
		// The level handled it.
		bProcessed = true;
	}
	else if( UEngine::Exec_Editor( InWorld, Stream, Ar ) )
	{
		// The engine handled it.
		bProcessed = true;
	}
	else if( FParse::Command(&Str,TEXT("SELECTNAME")) )
	{
		bProcessed = HandleSelectNameCommand( Str, Ar, InWorld );
	}
	// Dump a list of all public UObjects in the level
	else if( FParse::Command(&Str,TEXT("DUMPPUBLIC")) )
	{
		HandleDumpPublicCommand( Str, Ar );
	}
	else if (FParse::Command(&Str, TEXT("DumpClassSchemas")))
	{
		DumpClassSchemas(Str, Ar);
	}
	else if( FParse::Command(&Str,TEXT("JUMPTO")) )
	{
		return HandleJumpToCommand( Str, Ar );
	}
	else if (FParse::Command(&Str, TEXT("BugIt")))
	{
		return HandleBugItCommand(Str, Ar);
	}
	else if( FParse::Command(&Str,TEXT("BugItGo")) )
	{
		return HandleBugItGoCommand( Str, Ar );
	}
	else if ( FParse::Command(&Str,TEXT("TAGSOUNDS")) )
	{
		return HandleTagSoundsCommand( Str, Ar );
	}
	else if ( FParse::Command(&Str,TEXT("CHECKSOUNDS")) )
	{
		return HandlecheckSoundsCommand( Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("FIXUPBADANIMNOTIFIERS")) )
	{
		return HandleFixupBadAnimNotifiersCommand( Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("SETDETAILMODE")) )
	{
		bProcessed = HandleSetDetailModeCommand( Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("SETDETAILMODEVIEW")) )
	{
		bProcessed = HandleSetDetailModeViewCommand( Str, Ar, InWorld );
	}
	else if( FParse::Command(&Str,TEXT("CLEANBSPMATERIALS")) )
	{
		bProcessed = HandleCleanBSPMaterialCommand( Str, Ar, InWorld );
	}
	else if( FParse::Command(&Str,TEXT("AUTOMERGESM")) )
	{
		bProcessed = HandleAutoMergeStaticMeshCommand( Str, Ar );
	}
	else if (FParse::Command(&Str, TEXT("ADDSELECTED")))
	{
		HandleAddSelectedCommand( Str, Ar );
	}
	else if (FParse::Command(&Str, TEXT("TOGGLESOCKETGMODE")))
	{
		HandleToggleSocketGModeCommand( Str, Ar );
	}
	else if (FParse::Command(&Str, TEXT("LISTMAPPKGDEPENDENCIES")))
	{
		ListMapPackageDependencies(Str);
	}
	else if( FParse::Command(&Str,TEXT("REBUILDVOLUMES")) )
	{
		HandleRebuildVolumesCommand( Str, Ar, InWorld );
	}
	else if ( FParse::Command(&Str,TEXT("REMOVEARCHETYPEFLAG")) )
	{
		HandleRemoveArchtypeFlagCommand( Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("STARTMOVIECAPTURE")) )
	{
		bProcessed = HandleStartMovieCaptureCommand( Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("BUILDMATERIALTEXTURESTREAMINGDATA")) )
	{
		bProcessed = HandleBuildMaterialTextureStreamingData( Str, Ar );
	}
	else
	{
		bProcessed = FBlueprintEditorUtils::KismetDiagnosticExec(Stream, Ar);
	}

	return bProcessed;
}

bool UEditorEngine::HandleCallbackCommand( UWorld* InWorld, const TCHAR* Str , FOutputDevice& Ar )
{
	bool bResult = true;
	if ( FParse::Command(&Str,TEXT("SELECTEDPROPS")) )
	{
		FEditorDelegates::SelectedProps.Broadcast();
	}
	else if( FParse::Command( &Str, TEXT( "FITTEXTURETOSURFACE" ) ) )
	{
		FEditorDelegates::FitTextureToSurface.Broadcast(InWorld);
	}
	else
	{
		bResult = false;
	}
	return bResult;
}

bool UEditorEngine::HandleTestPropsCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title( NSLOCTEXT("UnrealEd", "PropertyEditorTestWindowTitle", "Property Editor Test") )
		.ClientSize(FVector2D(500,1000));

	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );

	if( FParse::Command(&Str,TEXT("TREE")) )
	{
		UObject* Object;
		UClass* Class = NULL;
		if( ParseObject<UClass>( Str, TEXT("CLASS="), Class, nullptr ) != false )
		{
			Object = NewObject<UObject>(GetTransientPackage(), Class);
		}
		else
		{
			Object = NewObject<UPropertyEditorTestObject>();
		}

		FDetailsViewArgs Args;
		Args.bHideSelectionTip = true;

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TSharedPtr<IDetailsView> DetailsView = PropertyModule.CreateDetailView(Args);
		DetailsView->SetObject(Object);

		// TreeView
		Window->SetContent
			(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				DetailsView.ToSharedRef()
			]
		);
	}
	else if ( FParse::Command(&Str,TEXT("TABLE")) )
	{
		// TableView
		const TSharedRef< IPropertyTable > Table = Module.CreatePropertyTable();

		TArray< UObject* > Objects;

		for (int Count = 0; Count < 50; Count++)
		{
			Objects.Add(NewObject<UPropertyEditorTestObject>());
		}

		Table->SetObjects( Objects );

		for (TFieldIterator<FProperty> PropertyIter( UPropertyEditorTestObject::StaticClass(), EFieldIteratorFlags::IncludeSuper); PropertyIter; ++PropertyIter)
		{
			const TWeakFieldPtr< FProperty >& Property = *PropertyIter;
			Table->AddColumn( Property );
		}

		Window->SetContent
			(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				Module.CreatePropertyTableWidget( Table )
			]
		);
	}
	else if ( FParse::Command(&Str, TEXT("STRUCT")) )
	{
		FDetailsViewArgs DetailsArgs;

		FStructureDetailsViewArgs StructArgs;
		StructArgs.bShowObjects = true;
		StructArgs.bShowInterfaces = true;

		UStruct* Struct = nullptr;
		if (!ParseObject<UStruct>(Str, TEXT("STRUCT="), Struct, nullptr))
		{
			Struct = FPropertyEditorTestBasicStruct::StaticStruct();
		}

		TSharedRef<IStructureDetailsView> StructDetails = Module.CreateStructureDetailView(DetailsArgs, StructArgs, MakeShared<FStructOnScope>(Struct));

		Window->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				StructDetails->GetWidget().ToSharedRef()
			]
		);
	}
	else if ( FParse::Command(&Str, TEXT("GENERATOR")) )
	{
		UPropertyEditorRowGeneratorTest* TestGenerator = NewObject<UPropertyEditorRowGeneratorTest>();

		Window->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				TestGenerator->GenerateWidget()
			]
		);
	}
	else
	{
		UObject* Object;
		UClass* Class = nullptr;
		if( ParseObject<UClass>( Str, TEXT("CLASS="), Class, nullptr ) != false )
		{
			Object = NewObject<UObject>(GetTransientPackage(), Class);
		}
		else
		{
			Object = NewObject<UPropertyEditorTestObject>();
		}

		//Details
		TArray<UObject*> Objects;
		Objects.Add( Object );

		FDetailsViewArgs Args;
		TSharedRef<IDetailsView> DetailsView = Module.CreateDetailView( Args );

		Window->SetContent
			(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				DetailsView
			]
		);


		DetailsView->SetObjects( Objects );
	}

	// Parent to the main frame window
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		TSharedPtr<SWindow> ParentWindow = MainFrame.GetParentWindow();

		FSlateApplication::Get().AddWindowAsNativeChild(Window, ParentWindow.ToSharedRef());
	}


	return true;
}

bool  UEditorEngine::CommandIsDeprecated( const TCHAR* Str, FOutputDevice& Ar )
{
	FMessageDialog::Open( EAppMsgType::Ok, FText::Format( NSLOCTEXT("UnrealEd", "Error_TriedToExecDeprecatedCmd", "Tried to execute deprecated command: {0}"), FText::FromString(Str) ) );
	return false;
}

bool UEditorEngine::HandleMapCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	if (FParse::Command(&Str,TEXT("SELECT")))
	{
		return Map_Select( InWorld, Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("BRUSH")) )
	{
		return Map_Brush( InWorld, Str, Ar );
	}
	else if (FParse::Command(&Str,TEXT("SENDTO")))
	{
		return Map_Sendto( InWorld, Str, Ar );
	}
	else if( FParse::Command(&Str,TEXT("REBUILD")) )
	{
		return Map_Rebuild( InWorld, Str, Ar );
	}
	else if( FParse::Command (&Str,TEXT("NEW")) )
	{
		return CommandIsDeprecated( TEXT("NEW"), Ar );
	}
	else if( FParse::Command( &Str, TEXT("LOAD") ) )
	{
		return Map_Load( Str, Ar );
	}
	else if( FParse::Command( &Str, TEXT("IMPORTADD") ) )
	{
		SelectNone( false, true );
		return Map_Import( InWorld, Str, Ar );
	}
	else if (FParse::Command (&Str,TEXT("EXPORT")))
	{
		return CommandIsDeprecated( TEXT("EXPORT"), Ar );
	}
	else if (FParse::Command (&Str,TEXT("SETBRUSH"))) // MAP SETBRUSH (set properties of all selected brushes)
	{
		return Map_Setbrush( InWorld, Str, Ar );
	}
	else if (FParse::Command (&Str,TEXT("CHECK")))
	{
		EMapCheckNotification::Type Notification = EMapCheckNotification::DisplayResults;
		bool bClearLog = true;
		if(FParse::Command(&Str,TEXT("DONTDISPLAYDIALOG")))
		{
			Notification = EMapCheckNotification::DontDisplayResults;
		}
		else if(FParse::Command(&Str,TEXT("NOTIFYRESULTS")))
		{
			Notification = EMapCheckNotification::NotifyOfResults;
		}
		if (FParse::Command(&Str, TEXT("NOCLEARLOG")))
		{
			bClearLog = false;
		}
		return Map_Check(InWorld, Str, Ar, false, Notification, bClearLog);
	}
	else if (FParse::Command (&Str,TEXT("CHECKDEP")))
	{
		EMapCheckNotification::Type Notification = EMapCheckNotification::DisplayResults;
		bool bClearLog = true;
		if(FParse::Command(&Str,TEXT("DONTDISPLAYDIALOG")))
		{
			Notification = EMapCheckNotification::DontDisplayResults;
		}
		else if(FParse::Command(&Str,TEXT("NOTIFYRESULTS")))
		{
			Notification = EMapCheckNotification::NotifyOfResults;
		}
		if (FParse::Command(&Str, TEXT("NOCLEARLOG")))
		{
			bClearLog = false;
		}
		return Map_Check(InWorld, Str, Ar, true, Notification, bClearLog);
	}
	else if (FParse::Command (&Str,TEXT("SCALE")))
	{
		return Map_Scale( InWorld, Str, Ar );
	}
	return false;
}

bool UEditorEngine::HandleSelectCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	if( FParse::Command(&Str,TEXT("NONE")) )
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SelectNone", "Select None") );
		SelectNone( true, true );
		RedrawLevelEditingViewports();
		return true;
	}

	return false;
}

bool UEditorEngine::HandleDeleteCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	bool bHandled = false;
	UBrushEditingSubsystem* BrushSubsystem = GetEditorSubsystem<UBrushEditingSubsystem>();
	if (BrushSubsystem)
	{
		bHandled = BrushSubsystem->HandleActorDelete();
	}

	if(!bHandled)
	{
		return Exec( InWorld, TEXT("ACTOR DELETE") );
	}
	return true;
}

bool UEditorEngine::HandleLightmassDebugCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	extern UNREALED_API bool GLightmassDebugMode;
	GLightmassDebugMode = !GLightmassDebugMode;
	Ar.Logf( TEXT("Lightmass Debug Mode: %s"), GLightmassDebugMode ? TEXT("true (launch UnrealLightmass.exe manually)") : TEXT("false") );
	return true;
}

bool UEditorEngine::HandleLightmassStatsCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	extern UNREALED_API bool GLightmassStatsMode;
	GLightmassStatsMode = !GLightmassStatsMode;
	Ar.Logf( TEXT("Show detailed Lightmass statistics: %s"), GLightmassStatsMode ? TEXT("ENABLED") : TEXT("DISABLED") );
	return true;
}

bool UEditorEngine::HandleSwarmDistributionCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	extern FSwarmDebugOptions GSwarmDebugOptions;
	GSwarmDebugOptions.bDistributionEnabled = !GSwarmDebugOptions.bDistributionEnabled;
	UE_LOGF(LogEditorServer, Log, "Swarm Distribution Mode: %ls", GSwarmDebugOptions.bDistributionEnabled ? TEXT("true (Jobs will be distributed)") : TEXT("false (Jobs will be local only)"));
	return true;
}

bool UEditorEngine::HandleLightmassImmediateImportCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bUseImmediateImport = !GLightmassDebugOptions.bUseImmediateImport;
	UE_LOGF(LogEditorServer, Log, "Lightmass Immediate Import will be %ls", GLightmassDebugOptions.bUseImmediateImport ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEditorEngine::HandleLightmassImmediateProcessCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bImmediateProcessMappings = !GLightmassDebugOptions.bImmediateProcessMappings;
	UE_LOGF(LogEditorServer, Log, "Lightmass Immediate Process will be %ls", GLightmassDebugOptions.bImmediateProcessMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
	if ((GLightmassDebugOptions.bImmediateProcessMappings == true) && (GLightmassDebugOptions.bUseImmediateImport == false))
	{
		UE_LOGF(LogEditorServer, Log, "\tLightmass Immediate Import needs to be enabled for this to matter...");
	}
	return true;
}

bool UEditorEngine::HandleLightmassSortCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bSortMappings = !GLightmassDebugOptions.bSortMappings;
	UE_LOGF(LogEditorServer, Log, "Lightmass Sorting is now %ls", GLightmassDebugOptions.bSortMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}
bool UEditorEngine::HandleLightmassDebugMaterialCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bDebugMaterials = !GLightmassDebugOptions.bDebugMaterials;
	UE_LOGF(LogEditorServer, Log, "Lightmass Dump Materials is now %ls", GLightmassDebugOptions.bDebugMaterials ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEditorEngine::HandleLightmassPaddingCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bPadMappings = !GLightmassDebugOptions.bPadMappings;
	UE_LOGF(LogEditorServer, Log, "Lightmass Mapping Padding is now %ls", GLightmassDebugOptions.bPadMappings ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEditorEngine::HandleLightmassDebugPaddingCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bDebugPaddings = !GLightmassDebugOptions.bDebugPaddings;
	UE_LOGF(LogEditorServer, Log, "Lightmass Mapping Debug Padding is now %ls", GLightmassDebugOptions.bDebugPaddings ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEditorEngine::HandleLightmassProfileCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	GLightmassDebugOptions.bUseImmediateImport = false;
	GLightmassDebugOptions.bImmediateProcessMappings = false;
	UE_LOGF(LogEditorServer, Log, "Lightmass Profiling mode is ENABLED");
	UE_LOGF(LogEditorServer, Log, "\tLightmass ImmediateImport mode is DISABLED");
	UE_LOGF(LogEditorServer, Log, "\tLightmass ImmediateProcess mode is DISABLED");
	return true;
}

bool UEditorEngine::HandleSelectNameCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld  )
{
	FName FindName=NAME_None;
	FParse::Value( Str, TEXT("NAME="), FindName );

	USelection* Selection = GetSelectedActors();
	Selection->BeginBatchSelectOperation();
	for( FActorIterator It(InWorld); It; ++It )
	{
		AActor* Actor = *It;
		SelectActor( Actor, Actor->GetFName()==FindName, 0 );
	}

	Selection->EndBatchSelectOperation();
	return true;
}

bool UEditorEngine::HandleDumpPublicCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	for( FThreadSafeObjectIterator It; It; ++It )
	{
		UObject* Obj = *It;
		if(Obj && IsInALevel(Obj) && Obj->HasAnyFlags(RF_Public))
		{
			UE_LOGF(LogEditorServer, Log,  "--%ls", *(Obj->GetFullName()) );
		}
	}
	return true;
}

bool UEditorEngine::HandleJumpToCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	FVector Loc;
	if( GetFVECTOR( Str, Loc ) )
	{
		for(FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
		{
			ViewportClient->SetViewLocation( Loc );
		}
	}
	return true;
}


bool UEditorEngine::HandleBugItCommand(const TCHAR* Str, FOutputDevice& Ar)
{
	if(GCurrentLevelEditingViewportClient && !PlayWorld)
	{
		FVector ViewLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
		FRotator ViewRotation = GCurrentLevelEditingViewportClient->GetViewRotation();

		FString GoString = FString::Printf(TEXT("BugItGo %f %f %f %f %f %f"), ViewLocation.X, ViewLocation.Y, ViewLocation.Z, ViewRotation.Pitch, ViewRotation.Yaw, ViewRotation.Roll);
		UE_LOGF(LogEditorServer, Log, "%ls", *GoString);

		FPlatformApplicationMisc::ClipboardCopy(*GoString);

		return true;
	}

	return false;
}

bool UEditorEngine::HandleBugItGoCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	if (PlayWorld)
	{
		// in PIE, let the in-game codepath handle it
		return false;
	}

	const TCHAR* Stream = Str;
	FVector Loc;
	Stream = GetFVECTORSpaceDelimited( Stream, Loc );
	if( Stream != NULL )
	{
		for(FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
		{
			ViewportClient->SetViewLocation( Loc );
		}
	}

	// so here we need to do move the string forward by a ' ' to get to the Rotator data
	if( Stream != NULL )
	{
		Stream = FCString::Strchr(Stream,' ');
		if( Stream != NULL )
		{
			++Stream;
		}
	}


	FRotator Rot;
	Stream = GetFROTATORSpaceDelimited( Stream, Rot, 1.0f );
	if( Stream != NULL )
	{
		// Zero out roll as the editor camera should not roll
		Rot.Roll = 0;
		for(FLevelEditorViewportClient* ViewportClient : GetLevelViewportClients())
		{
			ViewportClient->SetViewRotation( Rot );
		}
	}

	if (GetLevelViewportClients().Num())
	{
		// Now that all viewports are at the same position, broadcast the change
		Loc = GetLevelViewportClients()[0]->GetViewLocation();
		Rot = GetLevelViewportClients()[0]->GetViewRotation();
		PostBugItGoCalled.Broadcast(Loc, Rot);
	}

	RedrawLevelEditingViewports();

	return true;
}

bool UEditorEngine::HandleTagSoundsCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	int32 NumObjects = 0;
	int32 TotalSize = 0;
	for( FThreadSafeObjectIterator It(USoundWave::StaticClass()); It; ++It )
	{
		++NumObjects;
		DebugSoundAnnotation.Set(*It);

		USoundWave* Wave = static_cast<USoundWave*>(*It);
		const SIZE_T Size = Wave->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
		TotalSize += Size;
	}
	UE_LOGF(LogEditorServer, Log,  "Marked %i sounds %10.2fMB", NumObjects, ((float)TotalSize) /(1024.f*1024.f) );
	return true;
}

bool UEditorEngine::HandlecheckSoundsCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	TArray<USoundWave*> WaveList;
		for( FThreadSafeObjectIterator It(USoundWave::StaticClass()); It; ++It )
		{
			USoundWave* Wave = static_cast<USoundWave*>(*It);
			if ( !DebugSoundAnnotation.Get(Wave))
			{
				WaveList.Add( Wave );
			}
		}
		DebugSoundAnnotation.ClearAll();

		struct FCompareUSoundWaveByPathName
		{
			FORCEINLINE bool operator()(const USoundWave& A, const USoundWave& B) const
			{
				return A.GetPathName() < B.GetPathName();
			}
		};
		// Sort based on full path name.
		WaveList.Sort( FCompareUSoundWaveByPathName() );

		TArray<FWaveCluster> Clusters;
		Clusters.Add( FWaveCluster(TEXT("Total")) );
		Clusters.Add( FWaveCluster(TEXT("Ambient")) );
		Clusters.Add( FWaveCluster(TEXT("Foley")) );
		Clusters.Add( FWaveCluster(TEXT("Chatter")) );
		Clusters.Add( FWaveCluster(TEXT("Dialog")) );
		Clusters.Add( FWaveCluster(TEXT("Efforts")) );
		const int32 NumCoreClusters = Clusters.Num();

		// Output information.
		int32 TotalSize = 0;
		UE_LOGF(LogEditorServer, Log,  "=================================================================================" );
		UE_LOGF(LogEditorServer, Log,  "%60ls %10ls", TEXT("Wave Name"), TEXT("Size") );
		for ( int32 WaveIndex = 0 ; WaveIndex < WaveList.Num() ; ++WaveIndex )
		{
			USoundWave* Wave = WaveList[WaveIndex];
			const SIZE_T WaveSize = Wave->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
			UPackage* WavePackage = Wave->GetOutermost();
			const FString PackageName( WavePackage->GetName() );

			// Totals.
			Clusters[0].Num++;
			Clusters[0].Size += WaveSize;

			// Core clusters
			for ( int32 ClusterIndex = 1 ; ClusterIndex < NumCoreClusters ; ++ClusterIndex )
			{
				FWaveCluster& Cluster = Clusters[ClusterIndex];
				if ( PackageName.Find( Cluster.Name ) != -1 )
				{
					Cluster.Num++;
					Cluster.Size += WaveSize;
				}
			}

			// Package
			bool bFoundMatch = false;
			for ( int32 ClusterIndex = NumCoreClusters ; ClusterIndex < Clusters.Num() ; ++ClusterIndex )
			{
				FWaveCluster& Cluster = Clusters[ClusterIndex];
				if ( PackageName == Cluster.Name )
				{
					// Found a cluster with this package name.
					Cluster.Num++;
					Cluster.Size += WaveSize;
					bFoundMatch = true;
					break;
				}
			}
			if ( !bFoundMatch )
			{
				// Create a new cluster with the package name.
				FWaveCluster NewCluster( *PackageName );
				NewCluster.Num = 1;
				NewCluster.Size = WaveSize;
				Clusters.Add( NewCluster );
			}

			// Dump bulk sound list.
			UE_LOGF(LogEditorServer, Log,  "%70ls %10.2fk", *Wave->GetPathName(), ((float)WaveSize)/1024.f );
		}
		UE_LOGF(LogEditorServer, Log,  "=================================================================================" );
		UE_LOGF(LogEditorServer, Log,  "%60ls %10ls %10ls", TEXT("Cluster Name"), TEXT("Num"), TEXT("Size") );
		UE_LOGF(LogEditorServer, Log,  "=================================================================================" );
		int32 TotalClusteredSize = 0;
		for ( int32 ClusterIndex = 0 ; ClusterIndex < Clusters.Num() ; ++ClusterIndex )
		{
			const FWaveCluster& Cluster = Clusters[ClusterIndex];
			if ( ClusterIndex == NumCoreClusters )
			{
				UE_LOGF(LogEditorServer, Log,  "---------------------------------------------------------------------------------" );
				TotalClusteredSize += Cluster.Size;
			}
			UE_LOGF(LogEditorServer, Log,  "%60ls %10i %10.2fMB", *Cluster.Name, Cluster.Num, ((float)Cluster.Size)/(1024.f*1024.f) );
		}
		UE_LOGF(LogEditorServer, Log,  "=================================================================================" );
		UE_LOGF(LogEditorServer, Log,  "Total Clusterd: %10.2fMB", ((float)TotalClusteredSize)/(1024.f*1024.f) );
		return true;
}

bool UEditorEngine::HandleFixupBadAnimNotifiersCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	// Clear out unreferenced animsets from groups...
	FixUpBadAnimNotifiers();
	return true;
}

bool UEditorEngine::HandleSetDetailModeCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	TArray<AActor*> ActorsToDeselect;

	uint8 ParsedDetailMode = DM_Epic;
	if ( FParse::Value( Str, TEXT("MODE="), ParsedDetailMode ) )
	{
		for ( FSelectionIterator It( GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			checkSlow( Actor->IsA(AActor::StaticClass()) );

			for (UActorComponent* Component : Actor->GetComponents())
			{
				UPrimitiveComponent* primComp = Cast<UPrimitiveComponent>(Component);

				if (primComp && primComp->DetailMode != ParsedDetailMode )
				{
					primComp->Modify();
					primComp->DetailMode = EDetailMode(ParsedDetailMode);
					primComp->MarkRenderStateDirty();

					// If the actor will not be visible after changing the detail mode, deselect it
					if( primComp->DetailMode > GetCachedScalabilityCVars().DetailMode )
					{
						ActorsToDeselect.AddUnique( Actor );
					}
				}
			}
		}

		for( int32 x = 0 ; x < ActorsToDeselect.Num() ; ++x )
		{
			SelectActor( ActorsToDeselect[x], false, false );
		}
	}

	ULevel::LevelDirtiedEvent.Broadcast();
	FEditorSupportDelegates::RefreshPropertyWindows.Broadcast();
	FEditorDelegates::RefreshEditor.Broadcast();

	RedrawLevelEditingViewports( true );

	return true;
}

bool UEditorEngine::HandleSetDetailModeViewCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	uint8 DM = DM_Epic;
	if ( FParse::Value( Str, TEXT("MODE="), DM ) )
	{
		DetailMode = (EDetailMode)DM;

		// Detail mode was modified, so store in the CVar
		static IConsoleVariable* DetailModeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DetailMode"));
		check (DetailMode);
		DetailModeCVar->Set(DetailMode);
	}

	RedrawLevelEditingViewports( true );
	return true;
}

bool UEditorEngine::HandleCleanBSPMaterialCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	const FScopedBusyCursor BusyCursor;
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "CleanBSPMaterials", "Clean BSP Materials") );
	const int32 NumRefrencesCleared = CleanBSPMaterials( InWorld, false, true );
	// Prompt the user that the operation is complete.
	FMessageDialog::Open( EAppMsgType::Ok, FText::Format(NSLOCTEXT("UnrealEd", "CleanBSPMaterialsReportF", "Cleared {0} BSP material references.  Check log window for further details."), FText::AsNumber(NumRefrencesCleared)) );
	return true;
}

bool UEditorEngine::HandleAutoMergeStaticMeshCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	AutoMergeStaticMeshes();

	return true;
}

bool UEditorEngine::HandleAddSelectedCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	bool bVisible = true;
	FString OverrideGroup;
	FString VolumeName;
	if (FParse::Value(Str, TEXT("GROUP="), OverrideGroup))
	{
		if (OverrideGroup.ToUpper() == TEXT("INVISIBLE"))
		{
			bVisible = false;
		}
	}

	if (FParse::Value(Str, TEXT("VOLUME="), VolumeName))
	{
		UE_LOGF(LogEditorServer, Log, "Adding selected actors to %ls group of PrecomputedVisibiltyOverrideVolume %ls",
			bVisible ? TEXT(" VISIBLE ") : TEXT("INVISIBLE"), *VolumeName);

		APrecomputedVisibilityOverrideVolume* PrecompOverride = NULL;
		// Find the selected volume
		for (TObjectIterator<APrecomputedVisibilityOverrideVolume> VolumeIt; VolumeIt; ++VolumeIt)
		{
			APrecomputedVisibilityOverrideVolume* CheckPrecompOverride = *VolumeIt;
			if (CheckPrecompOverride->GetName() == VolumeName)
			{
				// Found the volume
				PrecompOverride = CheckPrecompOverride;
				break;
			}
		}

		if (PrecompOverride != NULL)
		{
			auto* OverrideActorList =
				bVisible ? &(PrecompOverride->OverrideVisibleActors) : &(PrecompOverride->OverrideInvisibleActors);
			// Grab a list of selected actors...
			for (FSelectionIterator ActorIt(GetSelectedActorIterator()) ; ActorIt; ++ActorIt)
			{
				AActor* Actor = static_cast<AActor*>(*ActorIt);
				checkSlow(Actor->IsA(AActor::StaticClass()));
				OverrideActorList->AddUnique(Actor);
			}
		}
		else
		{
			UE_LOGF(LogEditorServer, Warning, "Unable to find PrecomputedVisibilityOverrideVolume %ls", *VolumeName);
		}
	}
	else
	{
		UE_LOGF(LogEditorServer, Warning, "Usage: ADDSELECTED GROUP=<VISIBLE/INVISIBLE> VOLUME=<Name of volume actor>");
	}
	return true;
}

bool UEditorEngine::HandleToggleSocketGModeCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	bDrawSocketsInGMode = !bDrawSocketsInGMode;
	UE_LOGF(LogEditorServer, Warning, "Draw sockets in 'G' mode is now %ls", bDrawSocketsInGMode ? TEXT("ENABLED") : TEXT("DISABLED"));
	return true;
}

bool UEditorEngine::HandleListMapPackageDependenciesCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	ListMapPackageDependencies(Str);
	return true;
}

bool UEditorEngine::HandleRebuildVolumesCommand( const TCHAR* Str, FOutputDevice& Ar, UWorld* InWorld )
{
	for( TActorIterator<AVolume> It(InWorld); It; ++It )
	{
		AVolume* Volume = *It;

		if(!Volume->IsTemplate() && Volume->GetBrushComponent())
		{
			UE_LOGF(LogEditorServer, Log, "BSBC: %ls", *Volume->GetPathName() );
			Volume->GetBrushComponent()->BuildSimpleBrushCollision();
		}
	}
	return true;
}

bool UEditorEngine::HandleRemoveArchtypeFlagCommand( const TCHAR* Str, FOutputDevice& Ar )
{
	USelection* SelectedAssets = GetSelectedObjects();
	for (FSelectionIterator Iter(*SelectedAssets); Iter; ++Iter)
	{
		UObject* Asset = *Iter;
		if (Asset->HasAnyFlags(RF_ArchetypeObject))
		{
			// Strip archetype flag, resave
			Asset->ClearFlags(RF_ArchetypeObject);
			Asset->Modify();
		}
	}
	return true;
}

bool UEditorEngine::HandleStartMovieCaptureCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	IMovieSceneCaptureInterface* CaptureInterface = IMovieSceneCaptureModule::Get().GetFirstActiveMovieSceneCapture();
	if (CaptureInterface)
	{
		CaptureInterface->StartCapturing();
		return true;
	}

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			FSlatePlayInEditorInfo* SlatePlayInEditorSession = SlatePlayInEditorMap.Find(Context.ContextHandle);
			if (SlatePlayInEditorSession && SlatePlayInEditorSession->SlatePlayInEditorWindowViewport.IsValid())
			{
				IMovieSceneCaptureModule::Get().CreateMovieSceneCapture(SlatePlayInEditorSession->SlatePlayInEditorWindowViewport.ToSharedRef());
				return true;
			}
		}
	}

	return false;
}

bool UEditorEngine::HandleBuildMaterialTextureStreamingData( const TCHAR* Cmd, FOutputDevice& Ar )
{
	FEditorBuildUtils::EditorBuildMaterialTextureStreamingData(nullptr);
	return true;
}



/**
 * @return true if the given component's StaticMesh can be merged with other StaticMeshes
 */
bool IsComponentMergable(UStaticMeshComponent* Component)
{
	// we need a component to work
	if (Component == NULL)
	{
		return false;
	}

	// we need a static mesh to work
	if (Component->GetStaticMesh() == NULL || Component->GetStaticMesh()->GetRenderData() == NULL)
	{
		return false;
	}

	// only components with a single LOD can be merged
	if (Component->GetStaticMesh()->GetNumLODs() != 1)
	{
		return false;
	}

	// only components with a single material can be merged
	int32 NumSetElements = 0;
	for (int32 ElementIndex = 0; ElementIndex < Component->GetNumMaterials(); ElementIndex++)
	{
		if (Component->GetMaterial(ElementIndex) != NULL)
		{
			NumSetElements++;
		}
	}

	if (NumSetElements > 1)
	{
		return false;
	}

	return true;
}

void UEditorEngine::RegisterForUndo( FEditorUndoClient* Client)
{
	if (Client)
	{
		UndoClients.Add(Client);
	}
}

void UEditorEngine::UnregisterForUndo( FEditorUndoClient* Client)
{
	if (Client)
	{
		UndoClients.Remove(Client);
		int32 Index = INDEX_NONE;
		if (InflightUndoClients.Find(Client, Index))
		{
			// Can't remove from InflightUndoClients during iteration in UEditorEngine::BroadcastPostUndoRedo.
			// Just null out the element.
			InflightUndoClients[Index] = nullptr;
		}
	}
}


void UEditorEngine::AutoMergeStaticMeshes()
{
#ifdef TODO_STATICMESH
	TArray<AStaticMeshActor*> SMAs;
	for (FActorIterator It; It; ++It)
	{
		if (It->GetClass() == AStaticMeshActor::StaticClass())
		{
			SMAs.Add((AStaticMeshActor*)*It);
		}
	}

	// keep a mapping of actors and the other components that will be merged in to them
	TMap<AStaticMeshActor*, TArray<UStaticMeshComponent*> > ActorsToComponentForMergingMap;

	for (int32 SMAIndex = 0; SMAIndex < SMAs.Num(); SMAIndex++)
	{
		AStaticMeshActor* SMA = SMAs[SMAIndex];
		UStaticMeshComponent* Component = SMAs[SMAIndex]->StaticMeshComponent;

		// can this component merge with others?
		bool bCanBeMerged = IsComponentMergable(Component);

		// look for an already collected component to merge in to if I can be merged
		if (bCanBeMerged)
		{
			UMaterialInterface* Material = Component->GetMaterial(0);
			UObject* Outermost = SMA->GetOutermost();

			for (int32 OtherSMAIndex = 0; OtherSMAIndex < SMAIndex; OtherSMAIndex++)
			{
				AStaticMeshActor* OtherSMA = SMAs[OtherSMAIndex];
				UStaticMeshComponent* OtherComponent = OtherSMA->StaticMeshComponent;

				// is this other mesh mergable?
				bool bCanOtherBeMerged = IsComponentMergable(OtherComponent);

				// has this other mesh already been merged into another one? (after merging, DestroyActor
				// is called on it, setting IsPendingKillPending())
				bool bHasAlreadyBeenMerged = OtherSMA->IsPendingKillPending() == true;

				// only look at this mesh if it can be merged and the actor hasn't already been merged
				if (bCanOtherBeMerged && !bHasAlreadyBeenMerged)
				{
					// do materials match?
					bool bHasMatchingMaterials = Material == OtherComponent->GetMaterial(0);

					// we shouldn't go over 65535 verts so the index buffer can use 16 bit indices
					bool bWouldResultingMeshBeSmallEnough =
						(Component->StaticMesh->RenderData->LODResources[0].VertexBuffer.GetNumVertices() +
						 OtherComponent->StaticMesh->RenderData->LODResources[0].VertexBuffer.GetNumVertices()) < 65535;

					// make sure they are in the same level
					bool bHasMatchingOutermost = Outermost == OtherSMA->GetOutermost();

					// now, determine compatibility between components/meshes
					if (bHasMatchingMaterials && bHasMatchingOutermost && bWouldResultingMeshBeSmallEnough)
					{
						// if these two can go together, collect the information for later merging
						TArray<UStaticMeshComponent*>* ComponentsForMerging = ActorsToComponentForMergingMap.Find(OtherSMA);
						if (ComponentsForMerging == NULL)
						{
							ComponentsForMerging = &ActorsToComponentForMergingMap.Add(OtherSMA, TArray<UStaticMeshComponent*>());
						}

						// @todo: Remove this limitation, and improve the lightmap UV packing below
						if (ComponentsForMerging->Num() == 16)
						{
							continue;
						}

						// add my component as a component to merge in to the other actor
						ComponentsForMerging->Add(Component);

						// and remove this actor from the world, it is no longer needed (it won't be deleted
						// until after this function returns, so it's safe to use it's components below)
						World->DestroyActor(SMA);

						break;
					}
				}
			}
		}
	}

	// now that everything has been gathered, we can build some meshes!
	for (TMap<AStaticMeshActor*, TArray<UStaticMeshComponent*> >::TIterator It(ActorsToComponentForMergingMap); It; ++It)
	{
		AStaticMeshActor* OwnerActor = It.Key();
		TArray<UStaticMeshComponent*>& MergeComponents = It.Value();

		// get the component for the owner actor (its component is not in the TArray)
		UStaticMeshComponent* OwnerComponent = OwnerActor->StaticMeshComponent;

		// all lightmap UVs will go in to channel 1
		// @todo: This needs to look at the material and look for the smallest UV not used by the material
		int32 LightmapUVChannel = 1;

		// first, create an empty mesh
		TArray<FStaticMeshTriangle> EmptyTris;
		UStaticMesh* NewStaticMesh = CreateStaticMesh(EmptyTris, OwnerComponent->StaticMesh->LODModels[0].Elements, OwnerActor->GetOutermost(), NAME_None);

		// set where the lightmap UVs come from
		NewStaticMesh->LightMapCoordinateIndex = LightmapUVChannel;

		// figure out how much to grow the lightmap resolution by, since it needs to be square, start by sqrt'ing the number
		int32 LightmapMultiplier = FMath::TruncToInt(FMath::Sqrt(MergeComponents.Num()));

		// increase the sqrt by 1 unless it was a perfect square
		if (LightmapMultiplier * LightmapMultiplier != MergeComponents.Num())
		{
			LightmapMultiplier++;
		}

		// cache the 1 over
		float InvLightmapMultiplier = 1.0f / (float)LightmapMultiplier;

		// look for the largest lightmap resolution
		int32 MaxLightMapResolution = OwnerComponent->bOverrideLightMapRes ? OwnerComponent->OverriddenLightMapRes : OwnerComponent->StaticMesh->LightMapResolution;
		for (int32 ComponentIndex = 0; ComponentIndex < MergeComponents.Num(); ComponentIndex++)
		{
			UStaticMeshComponent* Component = MergeComponents[ComponentIndex];
			MaxLightMapResolution = FMath::Max(MaxLightMapResolution,
				Component->bOverrideLightMapRes ? Component->OverriddenLightMapRes : Component->StaticMesh->LightMapResolution);
		}

		// clamp the multiplied res to 1024
		// @todo: maybe 2048?
		int32 LightmapRes = FMath::Min(1024, MaxLightMapResolution * LightmapMultiplier);

		// now, use the max resolution in the new mesh
		if (OwnerComponent->bOverrideLightMapRes)
		{
			OwnerComponent->OverriddenLightMapRes = LightmapRes;
		}
		else
		{
			NewStaticMesh->LightMapResolution = LightmapRes;
		}

		// set up the merge parameters
		FMergeStaticMeshParams Params;
		Params.bDeferBuild = true;
		Params.OverrideElement = 0;
		Params.bUseUVChannelRemapping = true;
		Params.UVChannelRemap[LightmapUVChannel] = OwnerComponent->StaticMesh->LightMapCoordinateIndex;
		Params.bUseUVScaleBias = true;
		Params.UVScaleBias[LightmapUVChannel] = FVector4(InvLightmapMultiplier, InvLightmapMultiplier, 0.0f, 0.0f);

		// merge in to the empty mesh
		MergeStaticMesh(NewStaticMesh, OwnerComponent->StaticMesh, Params);

		// the component now uses this mesh
		// @todo: Is this needed? I think the Merge handles this
		{
			FComponentReregisterContext ReregisterContext(OwnerComponent);
			OwnerComponent->StaticMesh = NewStaticMesh;
		}

		// now merge all of the other component's meshes in to me
		for (int32 ComponentIndex = 0; ComponentIndex < MergeComponents.Num(); ComponentIndex++)
		{
			UStaticMeshComponent* Component = MergeComponents[ComponentIndex];

			// calculate a matrix to go from my component space to the owner's component's space
			FMatrix TransformToOwnerSpace = Component->GetComponentTransform().ToMatrixWithScale() * OwnerComponent->GetComponentTransform().ToMatrixWithScale().Inverse();

			// if we have negative scale, we need to munge the matrix and scaling
			if (TransformToOwnerSpace.Determinant() < 0.0f)
			{
				// get and remove the scale vector from the matrix
				Params.ScaleFactor3D = TransformToOwnerSpace.ExtractScaling();

				// negate X scale and top row of the matrix (will result in same transform, but then
				// MergeStaticMesh will fix the poly winding)
				Params.ScaleFactor3D.X = -Params.ScaleFactor3D.X;
				TransformToOwnerSpace.SetAxis(0, -TransformToOwnerSpace.GetScaledAxis( EAxis::X ));
			}
			else
			{
				Params.ScaleFactor3D = TransformToOwnerSpace.GetScaleVector();
			}

			// now get the offset and rotation from the transform
			Params.Offset = TransformToOwnerSpace.GetOrigin();
			Params.Rotation = TransformToOwnerSpace.Rotator();

			// set the UV offset
			int32 XSlot = (ComponentIndex + 1) % LightmapMultiplier;
			int32 YSlot = (ComponentIndex + 1) / LightmapMultiplier;
			Params.UVScaleBias[LightmapUVChannel].Z = (float)XSlot * InvLightmapMultiplier;
			Params.UVScaleBias[LightmapUVChannel].W = (float)YSlot * InvLightmapMultiplier;

			// route our lightmap UVs to the final lightmap channel
			Params.UVChannelRemap[LightmapUVChannel] = Component->StaticMesh->LightMapCoordinateIndex;

			// if compatible, merge them
			MergeStaticMesh(OwnerComponent->StaticMesh, Component->StaticMesh, Params);
		}

		// now that everything has been merged in, perform the slow build operation
		OwnerComponent->StaticMesh->Build();
	}
#endif // #if TODO_STATICMESH
}
