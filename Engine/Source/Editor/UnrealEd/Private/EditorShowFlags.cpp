// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorShowFlags.h"
#include "LevelEditorViewport.h"

bool ShouldFlagsInGroupKeepDeclarationOrder(EShowFlagGroup InGroup)
{
	switch (InGroup)
	{
	case SFG_Lumen:
	case SFG_MegaLights:
		return true;
	default:
		return false;
	}
}

FShowFlagData::FShowFlagData(const FString& InName, const FText& InDisplayName, const uint32 InEngineShowFlagIndex, EShowFlagGroup InGroup )
	:	DisplayName( InDisplayName )
	,	EngineShowFlagIndex( InEngineShowFlagIndex )
	,	Group( InGroup )
{
	ShowFlagName = *InName;
}

FShowFlagData::FShowFlagData(const FString& InName, const FText& InDisplayName, const uint32 InEngineShowFlagIndex, EShowFlagGroup InGroup, const FInputChord& InInputChord)
	:	DisplayName( InDisplayName )
	,	EngineShowFlagIndex( InEngineShowFlagIndex )
	,	Group( InGroup )
	,	InputChord( InInputChord )
{
	ShowFlagName = *InName;
}

bool FShowFlagData::IsEnabled(const FLevelEditorViewportClient* ViewportClient) const
{
	return ViewportClient->EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
}

void FShowFlagData::ToggleState(FLevelEditorViewportClient* ViewportClient) const
{
	bool bOldState = ViewportClient->EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
	ViewportClient->EngineShowFlags.SetSingleFlag(EngineShowFlagIndex, !bOldState);
}

TArray<FShowFlagData>& GetShowFlagMenuItems()
{
	static TArray<FShowFlagData> OutShowFlags;

	static bool bInvalidated = false;
	static bool bFirst = true; 
	if(bFirst)
	{
		// do this only once
		bFirst = false;
		check(IsInGameThread());
		FEngineShowFlags::OnCustomShowFlagRegistered.AddLambda([&]() {
			check(IsInGameThread());
			bInvalidated = true;
		});

		// FEngineShowFlags 
		{
			TMap<FString, FInputChord> EngineFlagsChords;

			EngineFlagsChords.Add("Navigation", FInputChord(EKeys::P) );
			EngineFlagsChords.Add("BSP", FInputChord() );
			EngineFlagsChords.Add("Collision", FInputChord(EKeys::C, EModifierKey::Alt) );
			EngineFlagsChords.Add("Fog", FInputChord(EKeys::F, EModifierKey::Alt ) );
			EngineFlagsChords.Add("LightRadius", FInputChord(EKeys::R, EModifierKey::Alt) );
			EngineFlagsChords.Add("StaticMeshes", FInputChord() );
			EngineFlagsChords.Add("Landscape", FInputChord(EKeys::L, EModifierKey::Alt) );
			EngineFlagsChords.Add("Volumes", FInputChord(EKeys::O, EModifierKey::Alt) );
			EngineFlagsChords.Add("Cameras", FInputChord() );
			EngineFlagsChords.Add("Grid", FInputChord(EKeys::G, EModifierKey::Alt | EModifierKey::Shift) );
			EngineFlagsChords.Add("MeshEdges", FInputChord(EKeys::Nine, EModifierKey::Alt) );

			struct FIterSink
			{
				FIterSink(TArray<FShowFlagData>& InShowFlagData, const TMap<FString, FInputChord>& InChordsMap) 
					: ShowFlagData(InShowFlagData), ChordsMap(InChordsMap)
				{
				}

				bool HandleShowFlag(uint32 InIndex, const FString& InName)
				{
					EShowFlagGroup Group = FEngineShowFlags::FindShowFlagGroup(*InName);
					if( Group != SFG_Hidden && Group != SFG_Transient )
					{
						FText FlagDisplayName;
						FEngineShowFlags::FindShowFlagDisplayName(InName, FlagDisplayName);

						const FInputChord* Chord = ChordsMap.Find(InName);
						if (Chord != NULL)
						{
							ShowFlagData.Add( FShowFlagData( InName, FlagDisplayName, InIndex, Group, *Chord ) );
						}
						else
						{
							ShowFlagData.Add( FShowFlagData( InName, FlagDisplayName, InIndex, Group ) );
						}
					}
					return true;
				}

				bool OnEngineShowFlag(uint32 InIndex, const FString& InName)
				{
					return HandleShowFlag(InIndex, InName);
				}

				bool OnCustomShowFlag(uint32 InIndex, const FString& InName)
				{
					return HandleShowFlag(InIndex, InName);
				}

				TArray<FShowFlagData>& ShowFlagData;
				const TMap<FString, FInputChord>& ChordsMap;
			};

			FIterSink Sink(OutShowFlags, EngineFlagsChords);

			FEngineShowFlags::IterateAllFlags(Sink);
		}	

		// Sort the show flags
		struct FShowFlagSorter
		{
			FORCEINLINE bool operator()( const FShowFlagData& A, const FShowFlagData& B ) const
			{
				if (A.Group != B.Group)
				{
					// first sort by group
					return A.Group < B.Group;
				}
				else if (ShouldFlagsInGroupKeepDeclarationOrder(A.Group))
				{
					// if group wants to keep declaration order do that
					return A.EngineShowFlagIndex < B.EngineShowFlagIndex;
				}
				else
				{
					// otherwise sort alphabetically by display name
					return A.DisplayName.ToString() < B.DisplayName.ToString();
				}
			}
		};
		OutShowFlags.Sort(FShowFlagSorter());
	}

	if (bInvalidated)
	{
		bInvalidated = false;
		TSet<uint32> SeenIndices;
		for (const FShowFlagData& Data : OutShowFlags)
		{
			SeenIndices.Add(Data.EngineShowFlagIndex);
		}

		FEngineShowFlags::IterateCustomFlags([&](uint32 Index, const FString& Name) {
			if (SeenIndices.Contains(Index)) 
			{
				return true;
			}

			EShowFlagGroup Group = FEngineShowFlags::FindShowFlagGroup(*Name);
			FText FlagDisplayName;
			FEngineShowFlags::FindShowFlagDisplayName(Name, FlagDisplayName);
			OutShowFlags.Add(FShowFlagData(Name, FlagDisplayName, Index, Group));
			return true;
		});
	}

	return OutShowFlags;
}





