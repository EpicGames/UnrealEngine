// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorState/ActorEditorContextEditorState.h"

#include "ActorEditorContextState.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "IActorEditorContextClient.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorEditorContextEditorState)

#define LOCTEXT_NAMESPACE "ActorContextEditorState"

UActorEditorContextEditorState::UActorEditorContextEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bApplyActorEditorContextOnLoad(true)
{
}

FText UActorEditorContextEditorState::GetCategoryText() const
{
	return FText(LOCTEXT("ActorEditorContextEditorStateCategoryText", "Actor Editor Context"));
}

UEditorState::FOperationResult UActorEditorContextEditorState::CaptureState()
{
	if (ActorEditorContextStateCollection == nullptr)
	{
		ActorEditorContextStateCollection = NewObject<UActorEditorContextStateCollection>(this);
	}

	UActorEditorContextSubsystem::Get()->CaptureContext(ActorEditorContextStateCollection);

	// Context is empty, no need to keep object around
	if (ActorEditorContextStateCollection->IsEmpty())
	{
		ActorEditorContextStateCollection = nullptr;
	}

	return FOperationResult(FOperationResult::Success);
}

UEditorState::FOperationResult UActorEditorContextEditorState::RestoreState() const
{
	UWorld* CurrentWorld = GetStateWorld();

	if (!bApplyActorEditorContextOnLoad)
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_ApplyActorEditorContextOnLoad", "User manually disabled application of the actor editor context"));
	}

	if (ActorEditorContextStateCollection != nullptr)
	{
		UActorEditorContextSubsystem::Get()->RestoreContext(ActorEditorContextStateCollection);
	}
	else
	{
		UActorEditorContextSubsystem::Get()->ResetContext();
	}

	return FOperationResult(FOperationResult::Success);
}

#undef LOCTEXT_NAMESPACE
