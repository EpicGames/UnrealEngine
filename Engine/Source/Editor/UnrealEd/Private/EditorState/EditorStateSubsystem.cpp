// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorState/EditorStateSubsystem.h"
#include "EditorState/WorldEditorState.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorStateSubsystem)

DEFINE_LOG_CATEGORY_STATIC(LogEditorState, All, All);

#define LOCTEXT_NAMESPACE "EditorStateSubsystem"

UEditorStateSubsystem* UEditorStateSubsystem::Get()
{
	check(GEditor);
	return GEditor->GetEditorSubsystem<UEditorStateSubsystem>();
}

void UEditorStateSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	UEditorStateSubsystem::RegisterEditorStateType<UWorldEditorState>();
}

void UEditorStateSubsystem::Deinitialize()
{
	UEditorStateSubsystem::UnregisterEditorStateType<UWorldEditorState>();
	Super::Deinitialize();
}

void UEditorStateSubsystem::RegisterEditorStateType(TSubclassOf<UEditorState> InEditorStateType)
{
	check(!RegisteredEditorStateTypes.Contains(InEditorStateType));
	RegisteredEditorStateTypes.Add(InEditorStateType);
}

void UEditorStateSubsystem::UnregisterEditorStateType(TSubclassOf<UEditorState> InEditorStateType)
{
	check(RegisteredEditorStateTypes.Contains(InEditorStateType));
	RegisteredEditorStateTypes.RemoveSwap(InEditorStateType);
}

void UEditorStateSubsystem::OutputOperationResult(const UEditorState* InState, bool bRestore, const UEditorState::FOperationResult& InOperationResult) const
{
	const FString ColonSeparator = InOperationResult.GetResultText().IsEmpty() ? TEXT("") : TEXT(": ");

	switch (InOperationResult.GetResult())
	{
	case UEditorState::FOperationResult::Success:
		UE_LOGF(LogEditorState, Log, "[%ls] Success %ls%ls", *InState->GetCategoryText().ToString(), *ColonSeparator, *InOperationResult.GetResultText().ToString());
		break;
	case UEditorState::FOperationResult::Skipped:
		UE_LOGF(LogEditorState, Log, "[%ls] Skipping %ls%ls", *InState->GetCategoryText().ToString(), *ColonSeparator, *InOperationResult.GetResultText().ToString());
		break;
	case UEditorState::FOperationResult::Warning:
		UE_LOGF(LogEditorState, Warning, "[%ls] Warning %ls%ls", *InState->GetCategoryText().ToString(), *ColonSeparator, *InOperationResult.GetResultText().ToString());
		break;
	case UEditorState::FOperationResult::Failure:
		{
			UE_LOGF(LogEditorState, Error, "[%ls] Failure %ls%ls", *InState->GetCategoryText().ToString(), *ColonSeparator, *InOperationResult.GetResultText().ToString());

			FText OperationFailureText = bRestore ? LOCTEXT("RestoreFailed", "Failed to restore bookmark state!") : LOCTEXT("CaptureFailed", "Failed to capture bookmark state!");
			FNotificationInfo Info(OperationFailureText);
			Info.SubText = InOperationResult.GetResultText();
			Info.bUseSuccessFailIcons = true;
			Info.ExpireDuration = 5.0f;
			Info.bFireAndForget = true;
			Info.bUseLargeFont = false;
			Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
			TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
			NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
		}
		break;
	}
}

void UEditorStateSubsystem::CaptureEditorState(FEditorStateCollection& OutState, UObject* InStateOuter)
{
	CaptureEditorState(OutState, {}, InStateOuter);
}

void UEditorStateSubsystem::CaptureEditorState(FEditorStateCollection& OutState, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter, UObject* InStateOuter)
{
	UE_LOGF(LogEditorState, Log, "Capturing editor state...");

	if (InEditorStateTypeFilter.IsEmpty())
	{
		OutState.States.Reset();

		for (TSubclassOf<UEditorState> EditorStateType : RegisteredEditorStateTypes)
		{
			OutState.States.Emplace(NewObject<UEditorState>(InStateOuter, EditorStateType));
		}
	}
	else
	{
		for (TSubclassOf<UEditorState> EditorStateType : InEditorStateTypeFilter)
		{
			OutState.States.Emplace(NewObject<UEditorState>(InStateOuter, EditorStateType));
		}
	}

	OutState.ForEachState([&OutState, this](UEditorState* StateToCapture, bool bCapturedDependencies)
	{
		bool bCaptureSuccess = bCapturedDependencies;
		if (bCaptureSuccess)
		{
			UEditorState::FOperationResult Result = StateToCapture->CaptureState();
			OutputOperationResult(StateToCapture, false, Result);

			bCaptureSuccess = Result == UEditorState::FOperationResult::Success ||
							  Result == UEditorState::FOperationResult::Warning;
		}
		else
		{
			OutputOperationResult(StateToCapture, false, UEditorState::FOperationResult(UEditorState::FOperationResult::Skipped, LOCTEXT("CaptureStateSkipped_MissingDependencies", "Missing dependant states, ignoring")));
		}

		// Couldn't capture state, invalidate it
		if (!bCaptureSuccess)
		{
			StateToCapture->Rename(nullptr, GetTransientPackage());
			OutState.GetStateChecked(StateToCapture->GetClass()) = nullptr;
		}

		return bCaptureSuccess;
	}, InEditorStateTypeFilter);

	// Remove invalidated states
	for (TArray<TObjectPtr<UEditorState>>::TIterator It(OutState.States); It; ++It)
	{
		if (It->Get() == nullptr)
		{
			It.RemoveCurrent();
		}
	}	

	UE_LOGF(LogEditorState, Log, "Captured editor state... DONE");
}

void UEditorStateSubsystem::RestoreEditorState(const FEditorStateCollection& InState)
{
	RestoreEditorState(InState, {});
}

void UEditorStateSubsystem::RestoreEditorState(const FEditorStateCollection& InState, const TArray<TSubclassOf<UEditorState>>& InEditorStateTypeFilter)
{
	check(!bIsRestoringEditorState);

	TGuardValue<bool> RestoringEditorState(bIsRestoringEditorState, true);

	UE_LOGF(LogEditorState, Log, "Restoring editor state...");

	InState.ForEachState([this](const UEditorState* StateToRestore, bool bRestoredDependencies)
	{
		if (!bRestoredDependencies)
		{
			OutputOperationResult(StateToRestore, true, UEditorState::FOperationResult(UEditorState::FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_MissingDependencies", "Missing dependant states, ignoring")));
			return false;
		}
		
		UEditorState::FOperationResult Result = StateToRestore->RestoreState();
		OutputOperationResult(StateToRestore, true, Result);

		return Result == UEditorState::FOperationResult::Success ||
			   Result == UEditorState::FOperationResult::Warning;
	}, InEditorStateTypeFilter);

	UE_LOGF(LogEditorState, Log, "Restored editor state... DONE");
}

bool UEditorStateSubsystem::IsRestoringEditorState() const
{
	return bIsRestoringEditorState;
}

#undef LOCTEXT_NAMESPACE