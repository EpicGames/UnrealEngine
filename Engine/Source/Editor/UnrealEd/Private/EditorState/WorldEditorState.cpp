// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorState/WorldEditorState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/World.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "Misc/MessageDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldEditorState)

#define LOCTEXT_NAMESPACE "WorldEditorState"

UWorldEditorState::UWorldEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UWorldEditorState::GetCategoryText() const
{
	return FText(LOCTEXT("WorldEditorStateCategoryText", "World"));
}

TSoftObjectPtr<UWorld> UWorldEditorState::GetStateWorld() const
{
	return World;
}

UEditorState::FOperationResult UWorldEditorState::CaptureState()
{
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	if (!CurrentWorld || FPackageName::IsTempPackage(CurrentWorld->GetPackage()->GetName()))
	{
		return FOperationResult(FOperationResult::Failure, LOCTEXT("CaptureStateFailure_UnsavedWorld", "Current world must be saved"));
	}

	// Set the current world.
	World = CurrentWorld;
	return FOperationResult(FOperationResult::Success);
}

UEditorState::FOperationResult UWorldEditorState::RestoreState() const
{
	if (World.IsNull())
	{
		return FOperationResult(FOperationResult::Failure, LOCTEXT("RestoreStateFailure_UnsavedWorld", "World is invalid"));
	}

	FString WorldPackageString = World.GetLongPackageName();
	FText WorldPackageText = FText::FromString(WorldPackageString);
	FAssetData WorldAsset;

	if (IAssetRegistry::GetChecked().TryGetAssetByObjectPath(World.ToSoftObjectPath(), WorldAsset) != UE::AssetRegistry::EExists::Exists)
	{
		return FOperationResult(FOperationResult::Failure, FText::Format(LOCTEXT("RestoreStateFailure_TargetWorldNotFound", "World {0} couldn't be resolved"), WorldPackageText));
	}

	if (WorldAsset.AssetClassPath != UWorld::StaticClass()->GetClassPathName())
	{
		return FOperationResult(FOperationResult::Failure, FText::Format(LOCTEXT("RestoreStateFailure_TargetNotAWorld", "{0} is not a world."), WorldPackageText));
	}

	// Already in the right world.
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	if (CurrentWorld == World)
	{
		return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("RestoreStateSuccess_WorldAlreadyLoaded", "World {0} was already active"), WorldPackageText));
	}

	// If there are any unsaved changes to the current level, see if the user wants to save those first
	// If they do not wish to save, then we will bail out of opening this asset.
	constexpr bool bPromptUserToSave = true;
	constexpr bool bSaveMapPackages = true;
	constexpr bool bSaveContentPackages = true;
	if (FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
	{
		UWorld* TargetWorld = World.Get();
		if (TargetWorld != nullptr)
		{
			// Validate that Asset was saved or isn't loaded meaning it can be loaded
			if (TargetWorld->GetPackage() == nullptr)
			{
				return FOperationResult(FOperationResult::Failure, FText::Format(LOCTEXT("RestoreStateFailure_CannotOpenNotInPackage", "The world you are trying to open ({0}) needs to be saved first."), WorldPackageText));
			}

			if (TargetWorld->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				return FOperationResult(FOperationResult::Failure, FText::Format(LOCTEXT("RestoreStateFailure_CannotOpenNewlyCreatedMapWithoutSaving", "The world you are trying to open ({0}) needs to be saved first."), WorldPackageText));
			}
		}
	}
	else
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_UserDeclinedToSave", "Declined to save dirty packages"));
	}

	const bool bLoadAsTemplate = false;
	const bool bShowProgress = true;
	const bool bMapLoaded = FEditorFileUtils::LoadMap(FPackageName::LongPackageNameToFilename(WorldPackageString, FPackageName::GetMapPackageExtension()), bLoadAsTemplate, bShowProgress);

	if (!bMapLoaded)
	{
		return FOperationResult(FOperationResult::Failure, FText::Format(LOCTEXT("RestoreStateFailure_LoadWorld", "Failed to load world {0}"), WorldPackageText));
	}
	
	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("RestoreStateSuccess_WorldLoaded", "Loaded world {0}"), WorldPackageText));
}
	
UWorldDependantEditorState::UWorldDependantEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UWorld* UWorldDependantEditorState::GetStateWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}

TArray<TSubclassOf<UEditorState>> UWorldDependantEditorState::GetDependencies() const
{
	return { UWorldEditorState::StaticClass() };
}

#undef LOCTEXT_NAMESPACE
