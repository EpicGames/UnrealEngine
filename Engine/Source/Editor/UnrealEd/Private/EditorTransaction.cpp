// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "UObject/Object.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "Algo/Find.h"
#include "Algo/Reverse.h"
#include "Engine/Level.h"
#include "Components/ActorComponent.h"
#include "Model.h"
#include "Misc/ITransactionObjectAnnotation.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Editor/TransBuffer.h"
#include "Components/ModelComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "BSPOps.h"
#include "Engine/DataTable.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorTransaction, Log, All);

struct FTransactionPackageDirtyFenceCounter
{
public:
	static FTransactionPackageDirtyFenceCounter Get()
	{
		static FTransactionPackageDirtyFenceCounter Instance;
		return Instance;
	}

	int32 GetFenceCount(const UPackage* Pkg) const
	{
		return PackageFenceCounts.FindRef(Pkg);
	}

	~FTransactionPackageDirtyFenceCounter()
	{
		UPackage::PackageSavedWithContextEvent.RemoveAll(this);
		FEditorDelegates::OnPackageDeleted.RemoveAll(this);
		FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);
	}

private:
	FTransactionPackageDirtyFenceCounter()
	{
		UPackage::PackageSavedWithContextEvent.AddRaw(this, &FTransactionPackageDirtyFenceCounter::OnPackageSaved);
		FEditorDelegates::OnPackageDeleted.AddRaw(this, &FTransactionPackageDirtyFenceCounter::OnPackageDeleted);
		FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FTransactionPackageDirtyFenceCounter::OnPostGarbageCollect);
	}

	void IncrementCount(UPackage* Pkg)
	{
		int32& PackageSaveCount = PackageFenceCounts.FindOrAdd(Pkg);
		++PackageSaveCount;
	}

	void OnPackageSaved(const FString& Filename, UPackage* Pkg, FObjectPostSaveContext ObjectSaveContext)
	{
		if (!ObjectSaveContext.IsFromAutoSave() && !ObjectSaveContext.IsProceduralSave())
		{
			IncrementCount(Pkg);
		}
	}

	void OnPackageDeleted(UPackage* Pkg)
	{
		IncrementCount(Pkg);
	}

	void OnPostGarbageCollect()
	{
		for (auto It = PackageFenceCounts.CreateIterator(); It; ++It)
		{
			if (!It->Key.IsValid())
			{
				It.RemoveCurrent();
			}
		}
	}

	TMap<TWeakObjectPtr<const UPackage>, int32, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<const UPackage>, int32>> PackageFenceCounts;
};

/*-----------------------------------------------------------------------------
	A single transaction.
-----------------------------------------------------------------------------*/

FTransaction::FObjectRecord::FObjectRecord(FTransaction* Owner, UObject* InObject, TUniquePtr<FChange> InCustomChange, FScriptArray* InArray, int32 InIndex, int32 InCount, int32 InOper, int32 InElementSize, uint32 InElementAlignment, STRUCT_DC InDefaultConstructor, STRUCT_AR InSerializer, STRUCT_DTOR InDestructor)
	:	Object				( InObject )
	,	CustomChange		( MoveTemp( InCustomChange ) )
	,	Array				( InArray )
	,	Index				( InIndex )
	,	Count				( InCount )
	,	Oper				( InOper )
	,	ElementSize			( InElementSize )
	,	ElementAlignment	( InElementAlignment )
	,	DefaultConstructor	( InDefaultConstructor )
	,	Serializer			( InSerializer )
	,	Destructor			( InDestructor )
{
#if WITH_RELOAD
	// With Hot Reload or Live Coding enabled, the layout of structures and classes can change.  We can't use binary serialization when this might happen.
	bWantsBinarySerialization = false;
#else
	// Blueprint compile-in-place can alter class layout so use tagged serialization for objects relying on a UBlueprint's Class
	if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(InObject->GetClass()))
	{
		bWantsBinarySerialization = false;
	}
	// Data tables can contain user structs, so it's unsafe to use binary
	if (UDataTable* DataTable = Cast<UDataTable>(InObject))
	{
		bWantsBinarySerialization = false;
	}
#endif

	// Update any sub-object caches for use by ARO (to keep sub-objects alive during GC)
	UObject* CurrentObject = Object.Get();
	checkSlow(CurrentObject == InObject);

	// Don't bother saving the object state if we have a custom change which can perform the undo operation
	if( CustomChange.IsValid() )
	{
		// @todo mesheditor debug
		//GWarn->Logf( TEXT( "------------ Saved Undo Change ------------" ) );
		//CustomChange->PrintToLog( *GWarn );
		//GWarn->Logf( TEXT( "-------------------------------------------" ) );
	}
	else
	{
		{
			SerializedObject.SetObject(CurrentObject);
			FWriter Writer(SerializedObject, bWantsBinarySerialization);
			SerializeContents(Writer, Oper);
		}

		if (!Array)
		{
			DiffableObject = MakeUnique<UE::Transaction::FDiffableObject>(GetDiffableObject());
		}
	}
}

UE::Transaction::FDiffableObject FTransaction::FObjectRecord::GetDiffableObject(TArrayView<const FProperty*> PropertiesToSerialize, UE::Transaction::DiffUtil::EGetDiffableObjectMode ObjectSerializationMode) const
{
	check(!Array);
	check(ObjectSerializationMode != UE::Transaction::DiffUtil::EGetDiffableObjectMode::Custom);

	if (UObject* CurrentObject = Object.Get())
	{
		UE::Transaction::DiffUtil::FGetDiffableObjectOptions ObjectOptions;
		ObjectOptions.PropertiesToSerialize = PropertiesToSerialize;
		ObjectOptions.ObjectSerializationMode = UE::Transaction::DiffUtil::EGetDiffableObjectMode::Custom;
		ObjectOptions.CustomSerializer = [this, ObjectSerializationMode](UE::Transaction::FDiffableObjectDataWriter& DiffWriter)
		{
			if (ObjectSerializationMode == UE::Transaction::DiffUtil::EGetDiffableObjectMode::SerializeProperties)
			{
				if (UObject* ObjectToSerialize = Object.Get())
				{
					ObjectToSerialize->SerializeScriptProperties(DiffWriter);
				}
			}
			else
			{
				SerializeObject(DiffWriter);
			}
		};

		return UE::Transaction::DiffUtil::GetDiffableObject(CurrentObject, ObjectOptions);
	}

	return UE::Transaction::FDiffableObject();
}

void FTransaction::FObjectRecord::SerializeContents( FArchive& Ar, int32 InOper ) const
{
	if( Array )
	{
		const bool bWasArIgnoreOuterRef = Ar.ArIgnoreOuterRef;
		if (Object.IsSubObjectReference())
		{
			Ar.ArIgnoreOuterRef = true;
		}

		UObject* CurrentObject = Object.Get();

		//UE_LOGF( LogEditorTransaction, Log, "Array %ls %i*%i: %i", CurrentObject ? *CurrentObject->GetFullName() : TEXT("Invalid Object"), Index, ElementSize, InOper);

		check((SIZE_T)Array >= (SIZE_T)CurrentObject + sizeof(UObject));
		check((SIZE_T)Array + sizeof(FScriptArray) <= (SIZE_T)CurrentObject + CurrentObject->GetClass()->GetPropertiesSize());
		check(ElementSize!=0);
		check(DefaultConstructor!=NULL);
		check(Serializer!=NULL);
		check(Index>=0);
		check(Count>=0);
		if( InOper==1 )
		{
			// "Saving add order" or "Undoing add order" or "Redoing remove order".
			if( Ar.IsLoading() )
			{
				checkSlow(Index+Count<=Array->Num());
				for( int32 i=Index; i<Index+Count; i++ )
				{
					Destructor( (uint8*)Array->GetData() + i*ElementSize );
				}
				Array->Remove( Index, Count, ElementSize, ElementAlignment );
			}
		}
		else
		{
			// "Undo/Redo Modify" or "Saving remove order" or "Undoing remove order" or "Redoing add order".
			if( InOper==-1 && Ar.IsLoading() )
			{
				Array->InsertZeroed( Index, Count, ElementSize, ElementAlignment );
				for( int32 i=Index; i<Index+Count; i++ )
				{
					DefaultConstructor( (uint8*)Array->GetData() + i*ElementSize );
				}
			}

			// Serialize changed items.
			check(Index+Count<=Array->Num());
			for( int32 i=Index; i<Index+Count; i++ )
			{
				Serializer( Ar, (uint8*)Array->GetData() + i*ElementSize );
			}
		}

		Ar.ArIgnoreOuterRef = bWasArIgnoreOuterRef;
	}
	else
	{
		//UE_LOGF(LogEditorTransaction, Log,  "Object %ls", *Object.Get()->GetFullName());
		check(Index==0);
		check(ElementSize==0);
		check(DefaultConstructor==NULL);
		check(Serializer==NULL);
		SerializeObject( Ar );
	}
}

void FTransaction::FObjectRecord::SerializeObject( FArchive& Ar ) const
{
	check(!Array);

	UObject* CurrentObject = Object.Get();
	if (CurrentObject)
	{
		const bool bWasArIgnoreOuterRef = Ar.ArIgnoreOuterRef;
		if (Object.IsSubObjectReference())
		{
			Ar.ArIgnoreOuterRef = true;
		}
		CurrentObject->Serialize(Ar);
		Ar.ArIgnoreOuterRef = bWasArIgnoreOuterRef;
	}
}

void FTransaction::FObjectRecord::Restore( FTransaction* Owner )
{
	// @TODO now that Matinee is gone, it's worth investigating removing this
	if( !bRestored )
	{
		bRestored = true;
		check(!Owner->bFlip);
		check(!CustomChange.IsValid());

		FReader Reader( Owner, SerializedObject, bWantsBinarySerialization );
	
		SerializeContents( Reader, Oper );
	}
}

void FTransaction::FObjectRecord::Save(FTransaction* Owner)
{
	// if record has a custom change, no need to do anything here
	if( CustomChange.IsValid() )
	{
		return;
	}

	// common undo/redo path, before applying undo/redo buffer we save current state:
	check(Owner->bFlip);
	if (!bRestored)
	{
		// Since the transaction will be reset, we want to preserve if this transaction affected the garbage flag at all
		// (since the only thing that should change is the updated state of the object)
		FSerializedObject::EPendingKillChange PendingKillChange = SerializedObjectFlip.PendingKillChange;

		SerializedObjectFlip.Reset();

		UObject* CurrentObject = Object.Get();
		if (CurrentObject)
		{
			SerializedObjectFlip.SetObject(CurrentObject);
		}

		FWriter Writer(SerializedObjectFlip, bWantsBinarySerialization);
		SerializeContents(Writer, -Oper);

		SerializedObjectFlip.PendingKillChange = PendingKillChange;
	}
}

void FTransaction::FObjectRecord::Load(FTransaction* Owner)
{
	// common undo/redo path, we apply the saved state and then swap it for the state we cached in ::Save above
	check(Owner->bFlip);
	if (!bRestored)
	{
		bRestored = true;

		if (CustomChange.IsValid())
		{
			if (CustomChange->HasExpired(Object.Get()) == false)		// skip expired changes
			{
				if (CustomChange->GetChangeType() == FChange::EChangeStyle::InPlaceSwap)
				{
					TUniquePtr<FChange> InvertedChange = CustomChange->Execute(Object.Get());
					ensure(InvertedChange->GetChangeType() == FChange::EChangeStyle::InPlaceSwap);
					CustomChange = MoveTemp(InvertedChange);
				}
				else
				{
					bool bIsRedo = (Owner->Inc == 1);
					if (bIsRedo)
					{
						CustomChange->Apply(Object.Get());
					}
					else
					{
						CustomChange->Revert(Object.Get());
					}
				}
			}
		}
		else
		{
			// When objects are created outside the transaction system we can end up
			// finding them but not having any data for them, so don't serialize 
			// when that happens:
			if (SerializedObject.SerializedData.Num() > 0)
			{
				FReader Reader(Owner, SerializedObject, bWantsBinarySerialization);
				SerializeContents(Reader, Oper);
			}
			SerializedObject.Swap(SerializedObjectFlip);
		}
		Oper *= -1;
	}
}

void FTransaction::FObjectRecord::Finalize( FTransaction* Owner, UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache& ArchetypeCache, TSharedPtr<ITransactionObjectAnnotation>& OutFinalizedObjectAnnotation )
{
	OutFinalizedObjectAnnotation.Reset();

	if (Array)
	{
		// Can only diff objects
		return;
	}

	if (!bFinalized)
	{
		bFinalized = true;

		if (CustomChange)
		{
			// Cannot diff custom changes
			return;
		}

		UObject* CurrentObject = Object.Get();
		if (CurrentObject)
		{
			// Serialize the finalized object for the redo operation
			SerializedObjectFlip.Reset();
			{
				SerializedObjectFlip.SetObject(CurrentObject);
				OutFinalizedObjectAnnotation = SerializedObjectFlip.ObjectAnnotation;
				FWriter Writer(SerializedObjectFlip, bWantsBinarySerialization);
				SerializeObject(Writer);
			}

			// Determine if the actual transacted object has changes, as we use this to detect if the transaction was a no-op and can be discarded
			// Note: We don't use the DeltaChange for this as Ar.IsTransacting() may have added extra data in the transacted object that is important for undo/redo, but not for 
			//       delta change detection. This can result in an empty delta for a change that still needs to be kept for undo/redo (eg, changes to transient object data).
			bHasSerializedObjectChanges 
				 = SerializedObject.SerializedData != SerializedObjectFlip.SerializedData
				|| SerializedObject.ReferencedNames != SerializedObjectFlip.ReferencedNames
				|| SerializedObject.ReferencedObjects != SerializedObjectFlip.ReferencedObjects;

			// Serialize the object so we can diff it
			const UE::Transaction::FDiffableObject CurrentDiffableObject = GetDiffableObject();
			
			// Diff against the object state when the transaction started
			UE::Transaction::DiffUtil::GenerateObjectDiff(*DiffableObject, CurrentDiffableObject, DeltaChange, UE::Transaction::DiffUtil::FGenerateObjectDiffOptions(), &ArchetypeCache);

			if (DeltaChange.bHasPendingKillChange)
			{
				SerializedObject.PendingKillChange = IsValid(CurrentObject) ? FSerializedObject::EPendingKillChange::DeadToAlive: FSerializedObject::EPendingKillChange::AliveToDead;
				SerializedObjectFlip.PendingKillChange = IsValid(CurrentObject) ? FSerializedObject::EPendingKillChange::AliveToDead : FSerializedObject::EPendingKillChange::DeadToAlive;
			}

			// If we have a previous snapshot then we need to consider that part of the diff for the finalized object, as systems may 
			// have been tracking delta-changes between snapshots and this finalization will need to account for those changes too
			if (DiffableObjectSnapshot)
			{
				UE::Transaction::DiffUtil::FGenerateObjectDiffOptions DiffOptions;
				DiffOptions.bFullDiff = false;

				UE::Transaction::DiffUtil::GenerateObjectDiff(*DiffableObjectSnapshot, CurrentDiffableObject, DeltaChange, DiffOptions, &ArchetypeCache);
			}
		}

		// Clear out any diff data now as we won't be getting any more diff requests once finalized
		DiffableObject.Reset();
		DiffableObjectSnapshot.Reset();
		ObjectAnnotationSnapshot.Reset();
		AllPropertiesSnapshot.Reset();
	}
}

void FTransaction::FObjectRecord::Snapshot( FTransaction* Owner, UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache& ArchetypeCache, TArrayView<const FProperty*> Properties )
{
	if (Array)
	{
		// Can only diff objects
		return;
	}

	if (bFinalized)
	{
		// Cannot snapshot once finalized
		return;
	}

	if (CustomChange)
	{
		// Cannot snapshot custom changes
		return;
	}

	UObject* CurrentObject = Object.Get();
	if (CurrentObject)
	{
		// Merge these properties into the combined list of properties to snapshot
		// This means that one snapshot call for A followed by B will actually make a diff for both A and B
		{
			// If AllPropertiesSnapshot is empty after the first snapshot, then it means that a previous snapshot was unfiltered and we should respect that
			const bool bCanMergeProperties = !DiffableObjectSnapshot || AllPropertiesSnapshot.Num() > 0;
			if (bCanMergeProperties && Properties.Num() > 0)
			{
				for (const FProperty* Property : Properties)
				{
					AllPropertiesSnapshot.AddUnique(Property);
				}
			}
			else
			{
				// No properties means a request to snapshot everything
				AllPropertiesSnapshot.Reset();
			}
		}

		// Serialize the object so we can diff it
		UE::Transaction::FDiffableObject CurrentDiffableObject = GetDiffableObject(AllPropertiesSnapshot, UE::Transaction::DiffUtil::EGetDiffableObjectMode::SerializeProperties);

		// Diff against the correct serialized data depending on whether we already had a snapshot
		const UE::Transaction::FDiffableObject& InitialDiffableObject = DiffableObjectSnapshot ? *DiffableObjectSnapshot : *DiffableObject;
		FTransactionObjectDeltaChange SnapshotDeltaChange;
		{
			UE::Transaction::DiffUtil::FGenerateObjectDiffOptions DiffOptions;
			DiffOptions.bFullDiff = false;

			UE::Transaction::DiffUtil::GenerateObjectDiff(InitialDiffableObject, CurrentDiffableObject, SnapshotDeltaChange, DiffOptions, &ArchetypeCache);
		}

		// Update the snapshot data for next time
		if (!DiffableObjectSnapshot)
		{
			DiffableObjectSnapshot = MakeUnique<UE::Transaction::FDiffableObject>();
		}
		DiffableObjectSnapshot->Swap(CurrentDiffableObject);

		TSharedPtr<ITransactionObjectAnnotation> InitialObjectTransactionAnnotation = ObjectAnnotationSnapshot ? ObjectAnnotationSnapshot : SerializedObject.ObjectAnnotation;
		ObjectAnnotationSnapshot = CurrentObject->FindOrCreateTransactionAnnotation();

		// Notify any listeners of this change
		if (SnapshotDeltaChange.HasChanged() || ObjectAnnotationSnapshot.IsValid())
		{
			TMap<UObject*, FTransactionObjectChange> AdditionalObjectChanges;
			if (ObjectAnnotationSnapshot)
			{
				ObjectAnnotationSnapshot->ComputeAdditionalObjectChanges(InitialObjectTransactionAnnotation.Get(), AdditionalObjectChanges);
			}

			CurrentObject->PostTransacted(FTransactionObjectEvent(Owner->GetId(), Owner->GetOperationId(), ETransactionObjectEventType::Snapshot, ETransactionObjectChangeCreatedBy::TransactionRecord, FTransactionObjectChange{ InitialDiffableObject.ObjectInfo, InitialDiffableObject.ObjectFlags, SnapshotDeltaChange }, ObjectAnnotationSnapshot));
			for (const TTuple<UObject*, FTransactionObjectChange>& AdditionalObjectChangePair : AdditionalObjectChanges)
			{
				AdditionalObjectChangePair.Key->PostTransacted(FTransactionObjectEvent(Owner->GetId(), Owner->GetOperationId(), ETransactionObjectEventType::Snapshot, ETransactionObjectChangeCreatedBy::TransactionAnnotation, AdditionalObjectChangePair.Value, nullptr));
			}
		}
	}
}

int32 FTransaction::GetRecordCount() const
{
	return Records.Num();
}

bool FTransaction::IsTransient() const
{
	bool bHasChanges = false;
	for (const FObjectRecord& Record : Records)
	{
		if (Record.ContainsPieObject())
		{
			return true;
		}
		bHasChanges |= Record.HasChanges();
	}
	return !bHasChanges;
}

bool FTransaction::ContainsPieObjects() const
{
	for( const FObjectRecord& Record : Records )
	{
		if( Record.ContainsPieObject() )
		{
			return true;
		}
	}
	return false;
}

bool FTransaction::HasExpired() const
{
	if (Records.Num() == 0)		// only return true if we definitely have expired changes
	{
		return false;
	}
	for (const FObjectRecord& Record : Records)
	{
		if (Record.HasExpired() == false)
		{
			return false;
		}
	}
	return true;
}


bool FTransaction::IsObjectTransacting(const UObject* Object) const
{
	// This function is meaningless when called outside of a transaction context. Without this
	// ensure clients will commonly introduced bugs by having some logic that runs during
	// the transacting and some logic that does not, yielding asymmetrical results.
	ensure(GIsTransacting);
	ensure(ChangedObjects.Num() != 0);
	return ChangedObjects.Contains(Object);
}

/**
 * Outputs the contents of the ObjectMap to the specified output device.
 */
void FTransaction::DumpObjectMap(FOutputDevice& Ar) const
{
	Ar.Logf( TEXT("===== DumpObjectMap %s ==== "), *Title.ToString() );
	for ( auto It = ObjectRecordsMap.CreateConstIterator(); It; ++It )
	{
		const UObject* CurrentObject	= It.Key().Get();
		const int32 SaveCount			= It.Value().SaveCount;
		Ar.Logf( TEXT("%i\t: %s"), SaveCount, *CurrentObject->GetPathName() );
	}
	Ar.Logf( TEXT("=== EndDumpObjectMap %s === "), *Title.ToString() );
}

FArchive& operator<<( FArchive& Ar, FTransaction::FObjectRecord& R )
{
	FMemMark Mark(FMemStack::Get());
	Ar << R.Object;
	Ar << R.SerializedObject.SerializedData;
	Ar << R.SerializedObject.ReferencedObjects;
	Ar << R.SerializedObject.ReferencedNames;
	Mark.Pop();
	return Ar;
}

void FTransaction::FObjectRecord::AddReferencedObjects( FReferenceCollector& Collector )
{
	Object.AddReferencedObjects(Collector);

	auto AddSerializedObjectReferences = [&Collector](FSerializedObject& InSerializedObject)
	{
		for (UE::Transaction::FPersistentObjectRef& ReferencedObject : InSerializedObject.ReferencedObjects)
		{
			ReferencedObject.AddReferencedObjects(Collector);
		}

		if (InSerializedObject.ObjectAnnotation.IsValid())
		{
			InSerializedObject.ObjectAnnotation->AddReferencedObjects(Collector);
		}
	};
	AddSerializedObjectReferences(SerializedObject);
	AddSerializedObjectReferences(SerializedObjectFlip);

	if (CustomChange.IsValid())
	{
		CustomChange->AddReferencedObjects(Collector);
	}
}

bool FTransaction::FObjectRecord::ContainsPieObject() const
{
	{
		const UObject* Obj = Object.Get();
		if(Obj && Obj->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			return true;
		}
	}

	auto SerializedObjectContainPieObjects = [](const FSerializedObject& InSerializedObject) -> bool
	{
		for (const UE::Transaction::FPersistentObjectRef& ReferencedObject : InSerializedObject.ReferencedObjects)
		{
			const UObject* Obj = ReferencedObject.Get();
			if (Obj && Obj->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
			{
				return true;
			}
		}
		return false;
	};

	if (SerializedObjectContainPieObjects(SerializedObject))
	{
		return true;
	}
	if (SerializedObjectContainPieObjects(SerializedObjectFlip))
	{
		return true;
	}
	return false;
}

bool FTransaction::FObjectRecord::HasChanges() const
{
	// A record contains change if it has a detected delta or a custom change or an object annotation
	return bHasSerializedObjectChanges || CustomChange || SerializedObject.ObjectAnnotation || SerializedObjectFlip.ObjectAnnotation;
}

bool FTransaction::FObjectRecord::HasExpired() const
{
	if (CustomChange && CustomChange->HasExpired(Object.Get()) == true)
	{
		return true;
	}
	return false;
}

void FTransaction::AddReferencedObjects( FReferenceCollector& Collector )
{
	for( FObjectRecord& ObjectRecord : Records )
	{
		ObjectRecord.AddReferencedObjects( Collector );
	}

	for (TTuple<UE::Transaction::FPersistentObjectRef, FObjectRecords>& ObjectRecordsPair : ObjectRecordsMap)
	{
		ObjectRecordsPair.Key.AddReferencedObjects(Collector);
	}
}

void FTransaction::SavePackage(UPackage* Package)
{
	check(Package);

	const bool bIsTransactional = Package->HasAnyFlags(RF_Transactional);
	const bool bIsTransient = Package->HasAnyFlags(RF_Transient);
	const bool bIsScriptPackage = Package->HasAnyPackageFlags(PKG_ContainsScript);

	if (bIsTransactional && !bIsTransient && !bIsScriptPackage)
	{
		UE::Transaction::FPersistentObjectRef PackageRef(Package);
		if (!PackageRecordMap.Contains(PackageRef))
		{
			FPackageRecord& PackageRecord = PackageRecordMap.Add(PackageRef);
			PackageRecord.DirtyFenceCount = FTransactionPackageDirtyFenceCounter::Get().GetFenceCount(Package);
			PackageRecord.bWasDirty = Package->IsDirty();
		}
	}
}

void FTransaction::SaveObject( UObject* Object )
{
	check(Object);
	Object->CheckDefaultSubobjects();

	SavePackage(Object->GetPackage());

	FObjectRecords* ObjectRecords = &ObjectRecordsMap.FindOrAdd(UE::Transaction::FPersistentObjectRef(Object));
	if (ObjectRecords->Records.Num() == 0)
	{
		// Save the object.
		FObjectRecord* Record = new FObjectRecord(this, Object, nullptr, nullptr, 0, 0, 0, 0, 0, nullptr, nullptr, nullptr);

		// Side effects of FObjectRecord() may have grown ObjectRecordsMap
		ObjectRecords = &ObjectRecordsMap.FindChecked(UE::Transaction::FPersistentObjectRef(Object));
		ObjectRecords->Records.Add(Record);
		Records.Add(Record);
	}
	++ObjectRecords->SaveCount;
}

void FTransaction::SaveArray( UObject* Object, FScriptArray* Array, int32 Index, int32 Count, int32 Oper, int32 ElementSize, uint32 ElementAlignment, STRUCT_DC DefaultConstructor, STRUCT_AR Serializer, STRUCT_DTOR Destructor )
{
	check(Object);
	check(Array);
	check(ElementSize);
	check(ElementAlignment);
	check(DefaultConstructor);
	check(Serializer);
	check(Object->IsValidLowLevel());
	check((SIZE_T)Array>=(SIZE_T)Object);
	check((SIZE_T)Array+sizeof(FScriptArray)<=(SIZE_T)Object+Object->GetClass()->PropertiesSize);
	check(Index>=0);
	check(Count>=0);
	check(Index+Count<=Array->Num());

	// don't serialize the array if the object is contained within a PIE package
	if( Object->HasAnyFlags(RF_Transactional) && !Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		// Save the array.
		Records.Add(new FObjectRecord( this, Object, nullptr, Array, Index, Count, Oper, ElementSize, ElementAlignment, DefaultConstructor, Serializer, Destructor ));
	}
}

void FTransaction::StoreUndo(UObject* Object, TUniquePtr<FChange> UndoChange)
{
	check(Object);
	Object->CheckDefaultSubobjects();

	SavePackage(Object->GetPackage());

	// Save the undo record
	FObjectRecords& ObjectRecords = ObjectRecordsMap.FindOrAdd(UE::Transaction::FPersistentObjectRef(Object));
	FObjectRecord* UndoRecord = ObjectRecords.Records.Add_GetRef(new FObjectRecord(this, Object, MoveTemp(UndoChange), nullptr, 0, 0, 0, 0, 0, nullptr, nullptr, nullptr));
	Records.Add(UndoRecord);
}

void FTransaction::SetPrimaryObject(UObject* InObject)
{
	if (PrimaryObject == NULL)
	{
		PrimaryObject = InObject;
	}
}

void FTransaction::SnapshotObject( UObject* InObject, TArrayView<const FProperty*> Properties )
{
	check(InObject);

	if (const FObjectRecords* ObjectRecords = ObjectRecordsMap.Find(UE::Transaction::FPersistentObjectRef(InObject)))
	{
		UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache ArchetypeCache;

		for (FObjectRecord* Record : ObjectRecords->Records)
		{
			checkSlow(Record->Object.Get() == InObject);
			if (!Record->CustomChange)
			{
				Record->Snapshot(this, ArchetypeCache, Properties);
			}
		}
	}
}

bool FTransaction::ContainsObject(const UObject* Object) const
{
	UE::Transaction::FPersistentObjectRef PersistentObjectRef(const_cast<UObject*>(Object));
	if (const FObjectRecords* ObjectRecords = ObjectRecordsMap.Find(PersistentObjectRef))
	{
		for (FObjectRecord* Record : ObjectRecords->Records)
		{
			if (Record->Object == PersistentObjectRef)
			{
				return true;
			}
		}
	}

	return false;
}

void FTransaction::BeginOperation()
{
	check(!OperationId.IsValid());
	OperationId = FGuid::NewGuid();
}

void FTransaction::EndOperation()
{
	check(OperationId.IsValid());
	OperationId.Invalidate();
}

void FTransaction::Apply()
{
	checkSlow(Inc==1||Inc==-1);

	// Figure out direction.
	const int32 Start = Inc==1 ? 0             : Records.Num()-1;
	const int32 End   = Inc==1 ? Records.Num() :              -1;

	UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache ArchetypeCache;

	// Update the package dirty states
	// We do this prior to applying any object updates since a package cannot be re-dirtied during an undo/redo operation
	if (bFlip)
	{
		for (TTuple<UE::Transaction::FPersistentObjectRef, FPackageRecord>& PackageRecordPair : PackageRecordMap)
		{
			if (UPackage* Package = Cast<UPackage>(PackageRecordPair.Key.Get()))
			{
				const int32 CurrentDirtyFenceCount = FTransactionPackageDirtyFenceCounter::Get().GetFenceCount(Package);
				const bool bCurrentDirtyFlag = Package->IsDirty();

				// When restoring an undo, any package that has been "fenced" since this transaction was made needs to be considered dirty
				// since the undo may restore it to a state that no longer matches the file on disk
				const bool bNewDirtyFlag = PackageRecordPair.Value.bWasDirty || PackageRecordPair.Value.DirtyFenceCount != CurrentDirtyFenceCount;
				Package->SetDirtyFlag(bNewDirtyFlag);
				
				// Store the current state for any inverse undo/redo operation
				PackageRecordPair.Value.DirtyFenceCount = CurrentDirtyFenceCount;
				PackageRecordPair.Value.bWasDirty = bCurrentDirtyFlag;
			}
		}
	}

	// Init objects.
	for( int32 i=Start; i!=End; i+=Inc )
	{
		FObjectRecord& Record = Records[i];
		Record.bRestored = false;

		// Apply may be called before Finalize in order to revert an object back to its prior state in the case that a transaction is canceled
		// In this case we still need to generate a diff for the transaction so that we notify correctly
		if (!Record.bFinalized)
		{
			TSharedPtr<ITransactionObjectAnnotation> FinalizedObjectAnnotation;
			Record.Finalize(this, ArchetypeCache, FinalizedObjectAnnotation);
		}

		UObject* Object = Record.Object.Get();
		if (Object)
		{
			if (!ChangedObjects.Contains(Object))
			{
				Object->CheckDefaultSubobjects();
				Object->PreEditUndo();
			}
			
			// If we get here, we are undoing - in that case, we need to reverse whatever was in the transaction.
			if (Record.SerializedObject.PendingKillChange == FObjectRecord::FSerializedObject::EPendingKillChange::AliveToDead)
			{
				Object->ClearGarbage();
			}
			else if (Record.SerializedObject.PendingKillChange == FObjectRecord::FSerializedObject::EPendingKillChange::DeadToAlive)
			{
				Object->MarkAsGarbage();
			}

			ChangedObjects.Add(Object, FChangedObjectValue(i, Record.SerializedObject.ObjectAnnotation));
		}
	}

	if (bFlip)
	{
		for (int32 i = Start; i != End; i += Inc)
		{
			Records[i].Save(this);
		}
		for (int32 i = Start; i != End; i += Inc)
		{
			Records[i].Load(this);
		}
	}
	else
	{
		for (int32 i = Start; i != End; i += Inc)
		{
			Records[i].Restore(this);
		}
	}

	// An Actor's components must always get its PostEditUndo before the owning Actor
	// so do a quick sort on Outer depth, component will deeper than their owner
	ChangedObjects.KeyStableSort([](UObject& A, UObject& B)
	{
		return Cast<UActorComponent>(&A) != nullptr;
	});

	auto ObjectWasInvalidatedDuringTransaction = [](const UObject* InObject) -> bool
	{
		// skip records which point to a SKEL_ or REINST_ class
		// (any class that we're keeping around only to GC old instances of the class).
		// a previous record may have caused a blueprint compilation
		// which may cause the consecutive changed object to turn invalid.
		if(InObject != nullptr && !IsValid(InObject) &&
			InObject->GetOutermost() == GetTransientPackage())
		{
			if(InObject->GetClass() && InObject->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
			{
				return true;
			}
		}
		return false;
	};

	TArray<ULevel*> LevelsToCommitModelSurface;
	HistoricDiffMap.Reset();
	for (const TTuple<UObject*, FChangedObjectValue>& ChangedObjectIt : ChangedObjects)
	{
		UObject* ChangedObject = ChangedObjectIt.Key;
		if(ObjectWasInvalidatedDuringTransaction(ChangedObject))
		{
			continue;
		}
		
		UModel* Model = Cast<UModel>(ChangedObject);
		if (Model && Model->Nodes.Num())
		{
			FBSPOps::bspBuildBounds(Model);
		}
		
		if (UModelComponent* ModelComponent = Cast<UModelComponent>(ChangedObject))
		{
			ULevel* Level = ModelComponent->GetTypedOuter<ULevel>();
			check(Level);
			LevelsToCommitModelSurface.AddUnique(Level);
		}

		TSharedPtr<ITransactionObjectAnnotation> ChangedObjectTransactionAnnotation = ChangedObjectIt.Value.Annotation;
		if (ChangedObjectTransactionAnnotation.IsValid())
		{
			ChangedObject->PostEditUndo(ChangedObjectTransactionAnnotation);
		}
		else
		{
			ChangedObject->PostEditUndo();
		}

		const FObjectRecord& ChangedObjectRecord = Records[ChangedObjectIt.Value.RecordIndex];
		const FTransactionObjectDeltaChange& DeltaChange = ChangedObjectRecord.DeltaChange;
		if (DeltaChange.HasChanged() || ChangedObjectTransactionAnnotation.IsValid() || ChangedObjectRecord.bHasSerializedObjectChanges)
		{
			const FObjectRecord::FSerializedObject& InitialSerializedObject = ChangedObjectRecord.SerializedObject;

			// If this object changed from being dead to alive, then we need to compute a new delta change against its archetype
			// This is so that anything listening for changes will get a full update for the object, as if it had been newly constructed with its current state
			FTransactionObjectDeltaChange PrimaryDeltaChange;
			if (DeltaChange.bHasPendingKillChange && IsValid(ChangedObject))
			{
				UE::Transaction::FDiffableObject FakeDiffableComponent;
				FakeDiffableComponent.SetObject(ChangedObject);
				static_cast<FTransactionObjectId&>(FakeDiffableComponent.ObjectInfo) = InitialSerializedObject.ObjectId;
				FakeDiffableComponent.ObjectInfo.bIsPendingKill = true;

				const UE::Transaction::FDiffableObject CurrentDiffableObject = ChangedObjectRecord.GetDiffableObject();

				PrimaryDeltaChange = UE::Transaction::DiffUtil::GenerateObjectDiff(FakeDiffableComponent, CurrentDiffableObject, UE::Transaction::DiffUtil::FGenerateObjectDiffOptions(), &ArchetypeCache);
			}
			else
			{
				PrimaryDeltaChange = DeltaChange;
			}

			TMap<UObject*, FTransactionObjectChange> AdditionalObjectChanges;
			if (ChangedObjectTransactionAnnotation)
			{
				ChangedObjectTransactionAnnotation->ComputeAdditionalObjectChanges(InitialSerializedObject.ObjectAnnotation.Get(), AdditionalObjectChanges);
			}

			PostTransactAndSaveHistoricDiff(ChangedObject, FTransactionObjectEvent(Id, OperationId, ETransactionObjectEventType::UndoRedo, ETransactionObjectChangeCreatedBy::TransactionRecord, FTransactionObjectChange{ InitialSerializedObject.ObjectId, InitialSerializedObject.ObjectFlags, MoveTemp(PrimaryDeltaChange) }, ChangedObjectTransactionAnnotation));
			for (const TTuple<UObject*, FTransactionObjectChange>& AdditionalObjectChangePair : AdditionalObjectChanges)
			{
				PostTransactAndSaveHistoricDiff(AdditionalObjectChangePair.Key, FTransactionObjectEvent(Id, OperationId, ETransactionObjectEventType::UndoRedo, ETransactionObjectChangeCreatedBy::TransactionAnnotation, AdditionalObjectChangePair.Value, nullptr));
			}
		}
	}

	// Commit model surfaces for unique levels within the transaction
	for (ULevel* Level : LevelsToCommitModelSurface)
	{
		Level->CommitModelSurfaces();
	}

	// Flip it.
	if (bFlip)
	{
		Inc *= -1;
	}
	for (auto ChangedObjectIt : ChangedObjects)
	{
		UObject* ChangedObject = ChangedObjectIt.Key;
		if(ObjectWasInvalidatedDuringTransaction(ChangedObject))
        {
        	continue;
        }
		(void)ChangedObject->CheckDefaultSubobjects();
	}

	ChangedObjects.Reset();
}

void FTransaction::Finalize()
{
	UE::Transaction::DiffUtil::FDiffableObjectArchetypeCache ArchetypeCache;

	for (int32 i = 0; i < Records.Num(); ++i)
	{
		FObjectRecord& ObjectRecord = Records[i];

		TSharedPtr<ITransactionObjectAnnotation> FinalizedObjectAnnotation;
		TSharedPtr<ITransactionObjectAnnotation> SnapshotObjectAnnotation = ObjectRecord.ObjectAnnotationSnapshot;
		ObjectRecord.Finalize(this, ArchetypeCache, FinalizedObjectAnnotation);

		UObject* Object = ObjectRecord.Object.Get();
		if (Object)
		{
			if (!ChangedObjects.Contains(Object))
			{
				ChangedObjects.Add(Object, FChangedObjectValue(i, FinalizedObjectAnnotation, SnapshotObjectAnnotation));
			}
		}
	}

	// An Actor's components must always be notified before the owning Actor
	// so do a quick sort on Outer depth, component will deeper than their owner
	ChangedObjects.KeyStableSort([](UObject& A, UObject& B)
	{
		return Cast<UActorComponent>(&A) != nullptr;
	});

	HistoricDiffMap.Reset();
	for (const TTuple<UObject*, FChangedObjectValue>& ChangedObjectIt : ChangedObjects)
	{
		const FObjectRecord& ChangedObjectRecord = Records[ChangedObjectIt.Value.RecordIndex];
		TSharedPtr<ITransactionObjectAnnotation> ChangedObjectTransactionAnnotation = ChangedObjectIt.Value.Annotation;		
		const FTransactionObjectDeltaChange& DeltaChange = ChangedObjectRecord.DeltaChange;
		if (DeltaChange.HasChanged() || ChangedObjectTransactionAnnotation.IsValid() || ChangedObjectRecord.bHasSerializedObjectChanges)
		{
			UObject* ChangedObject = ChangedObjectIt.Key;
			
			const FObjectRecord::FSerializedObject& InitialSerializedObject = ChangedObjectRecord.SerializedObject;

			TMap<UObject*, FTransactionObjectChange> AdditionalObjectChanges;
			if (ChangedObjectTransactionAnnotation)
			{
				ChangedObjectTransactionAnnotation->ComputeAdditionalObjectChanges(InitialSerializedObject.ObjectAnnotation.Get(), AdditionalObjectChanges);
			}
			if (ChangedObjectTransactionAnnotation && ChangedObjectIt.Value.AnnotationSnapshot)
			{
				ChangedObjectTransactionAnnotation->ComputeAdditionalObjectChanges(ChangedObjectIt.Value.AnnotationSnapshot.Get(), AdditionalObjectChanges);
			}

			PostTransactAndSaveHistoricDiff(ChangedObject, FTransactionObjectEvent(Id, OperationId, ETransactionObjectEventType::Finalized, ETransactionObjectChangeCreatedBy::TransactionRecord, FTransactionObjectChange{ InitialSerializedObject.ObjectId, InitialSerializedObject.ObjectFlags, DeltaChange }, ChangedObjectTransactionAnnotation));
			for (const TTuple<UObject*, FTransactionObjectChange>& AdditionalObjectChangePair : AdditionalObjectChanges)
			{
				PostTransactAndSaveHistoricDiff(AdditionalObjectChangePair.Key, FTransactionObjectEvent(Id, OperationId, ETransactionObjectEventType::Finalized, ETransactionObjectChangeCreatedBy::TransactionAnnotation, AdditionalObjectChangePair.Value, nullptr));
			}
		}
	}
	ChangedObjects.Reset();
}

SIZE_T FTransaction::DataSize() const
{
	SIZE_T Result=0;
	for( int32 i=0; i<Records.Num(); i++ )
	{
		const FObjectRecord& Record = Records[i];
		const SIZE_T CustomChangeSize = Record.CustomChange ? Record.CustomChange->GetSize() : 0;
		Result += Record.SerializedObject.SerializedData.Num() + CustomChangeSize;
	}
	return Result;
}

/**
 * Get all the objects that are part of this transaction.
 * @param	Objects		[out] Receives the object list.  Previous contents are cleared.
 */
void FTransaction::GetTransactionObjects(TArray<UObject*>& Objects) const
{
	Objects.Empty(); // Just in case.

	for(int32 i=0; i<Records.Num(); i++)
	{
		UObject* Obj = Records[i].Object.Get();
		if (Obj)
		{
			Objects.AddUnique(Obj);
		}
	}
}

FTransactionDiff FTransaction::GenerateDiff() const
{
	FTransactionDiff TransactionDiff{Id, Title.ToString()};
	
	// Only generate diff if the transaction is finalized.
	if (ChangedObjects.Num() == 0)
	{
		// For each record, create a diff 
		for (int32 i = 0; i < Records.Num(); ++i)
		{
			const FObjectRecord& ObjectRecord = Records[i];
			if (UObject* TransactedObject = ObjectRecord.Object.Get())
			{
				if (ObjectRecord.DeltaChange.HasChanged() || ObjectRecord.SerializedObject.ObjectAnnotation)
				{
					// Since this transaction is not currently in an undo operation, generate a valid Guid.
					FGuid OperationGuid = FGuid::NewGuid();

					TransactionDiff.DiffMap.Emplace(
						FName(*TransactedObject->GetPathName()), 
						MakeShared<FTransactionObjectEvent>(this->GetId(), OperationGuid, ETransactionObjectEventType::Finalized, ETransactionObjectChangeCreatedBy::TransactionRecord, FTransactionObjectChange{ ObjectRecord.SerializedObject.ObjectId, ObjectRecord.SerializedObject.ObjectFlags, ObjectRecord.DeltaChange }, ObjectRecord.SerializedObject.ObjectAnnotation)
						);

					if (ObjectRecord.SerializedObjectFlip.ObjectAnnotation && ObjectRecord.SerializedObject.ObjectAnnotation)
					{
						TMap<UObject*, FTransactionObjectChange> AdditionalObjectChanges;
						ObjectRecord.SerializedObjectFlip.ObjectAnnotation->ComputeAdditionalObjectChanges(ObjectRecord.SerializedObject.ObjectAnnotation.Get(), AdditionalObjectChanges);

						for (const TTuple<UObject*, FTransactionObjectChange>& AdditionalObjectChangePair : AdditionalObjectChanges)
						{
							TransactionDiff.DiffMap.Emplace(
								FName(*AdditionalObjectChangePair.Key->GetPathName()), 
								MakeShared<FTransactionObjectEvent>(this->GetId(), OperationGuid, ETransactionObjectEventType::Finalized, ETransactionObjectChangeCreatedBy::TransactionAnnotation, AdditionalObjectChangePair.Value, nullptr)
								);
						}
					}
				}
			}
		}
	}

	return TransactionDiff;
}

FTransactionDiff FTransaction::GetHistoricDiff() const
{
	FTransactionDiff TransactionDiff{Id, Title.ToString()};
	TransactionDiff.DiffMap = HistoricDiffMap;
	return TransactionDiff;
}

void FTransaction::PostTransactAndSaveHistoricDiff(UObject* Obj, FTransactionObjectEvent&& TransactionEvent)
{
	Obj->PostTransacted(TransactionEvent);
	HistoricDiffMap.Add(*Obj->GetPathName(), MakeShared<FTransactionObjectEvent>(MoveTemp(TransactionEvent)));
}


/*-----------------------------------------------------------------------------
	Transaction tracking system.
-----------------------------------------------------------------------------*/
UTransactor::UTransactor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTransBuffer::Initialize(SIZE_T InMaxMemory)
{
	MaxMemory = InMaxMemory;
	// Reset.
	Reset( NSLOCTEXT("UnrealEd", "Startup", "Startup") );
	CheckState();

	FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &UTransBuffer::OnObjectsReinstanced);

	UE_LOGF(LogInit, Log, "Transaction tracking system initialized" );
}

// UObject interface.
void UTransBuffer::Serialize( FArchive& Ar )
{
	check( !Ar.IsPersistent() || this->HasAnyFlags(RF_ClassDefaultObject) );

	CheckState();

	Super::Serialize( Ar );

	if ( IsObjectSerializationEnabled() || !Ar.IsObjectReferenceCollector() )
	{
		Ar << UndoBuffer;
	}
	Ar << ResetReason << UndoCount << ActiveCount << ActiveRecordCounts;

	CheckState();
}

void UTransBuffer::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		CheckState();
		UE_LOGF(LogExit, Log, "Transaction tracking system shut down" );
	}
	Super::FinishDestroy();
}

void UTransBuffer::OnObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstances)
{
	if (OldToNewInstances.Num() == 0)
	{
		return;
	}

	// Reinstantiation coming from the loading thread only concerns objects still being loaded and
	// shouldn't be referenced in any way by an existing transaction.
	if (IsInActualLoadingThread())
	{
		return;
	}

	class FReinstancingReferenceCollector : public FReferenceCollector
	{
	public:
		virtual bool IsIgnoringArchetypeRef() const override
		{
			return false;
		}

		virtual bool IsIgnoringTransient() const override
		{
			return false;
		}

		virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
		{
			if (UObject* NewObject = OldToNewInstancesPtr->FindRef(Object))
			{
				Object = NewObject;
			}
		}

		const TMap<UObject*, UObject*>* OldToNewInstancesPtr = nullptr;
	};

	FReinstancingReferenceCollector Collector;
	Collector.OldToNewInstancesPtr = &OldToNewInstances;
	CallAddReferencedObjects(Collector);
}

void UTransBuffer::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{	
	UTransBuffer* This = CastChecked<UTransBuffer>(InThis);
	This->CheckState();

	if ( This->IsObjectSerializationEnabled() )
	{
		// We cannot support undoing across GC if we allow it to eliminate references so we need
		// to suppress it.
		Collector.AllowEliminatingReferences(false);
		for (const TSharedRef<FTransaction>& SharedTrans : This->UndoBuffer)
		{
			SharedTrans->AddReferencedObjects( Collector );
		}
		for (const TSharedRef<FTransaction>& SharedTrans : This->RemovedTransactions)
		{
			SharedTrans->AddReferencedObjects(Collector);
		}
		Collector.AllowEliminatingReferences(true);
	}

	This->CheckState();

	Super::AddReferencedObjects( This, Collector );
}

int32 UTransBuffer::Begin( const TCHAR* SessionContext, const FText& Description )
{
	return BeginInternal<FTransaction>(SessionContext, Description);
}

namespace TransBuffer
{
	static FAutoConsoleVariable DumpTransBufferObjectMap(TEXT("TransBuffer.DumpObjectMap"), false, TEXT("Whether to dump the object map each time a transaction is written for debugging purposes."));
}

int32 UTransBuffer::End()
{
	CheckState();
	const int32 Result = ActiveCount;
	FGuid TransactionId = FGuid();
	bool bTransactionFinalized = false;

	// Don't assert as we now purge the buffer when resetting.
	// So, the active count could be 0, but the code path may still call end.
	if (ActiveCount >= 1)
	{
		if (ActiveCount == 1 && GUndo != nullptr)
		{
			TransactionStateChangedDelegate.Broadcast(GUndo->GetContext(), ETransactionStateEventType::PreTransactionFinalized);
		}

		if( --ActiveCount==0 )
		{
			if (GUndo)
			{
				if (GLog && TransBuffer::DumpTransBufferObjectMap->GetBool())
				{
					// @todo DB: Fix this potentially unsafe downcast.
					static_cast<FTransaction*>(GUndo)->DumpObjectMap( *GLog );
				}

				// End the current transaction.
				GUndo->Finalize();
				TransactionStateChangedDelegate.Broadcast(GUndo->GetContext(), ETransactionStateEventType::TransactionFinalized);
				bTransactionFinalized = true;
				TransactionId = GUndo->GetContext().TransactionId;
				GUndo->EndOperation();

				// Once the transaction is finalized, remove it from the undo buffer if it's flagged as transient. (i.e contains PIE objects is no-op)
				if (GUndo->IsTransient())
				{
					check(UndoCount == 0);
					UndoBuffer.Pop(EAllowShrinking::No);
					UndoBuffer.Reserve(UndoBuffer.Num() + RemovedTransactions.Num());

					// Restore the transactions state to what it was before that transient (i.e. ineffective) transaction (like in the Cancel case : allows to not lose the Redo stack in case 
					//  we had inserted a transient transaction in the middle) : 
					if (PreviousUndoCount > 0)
					{
						UndoBuffer.Append(RemovedTransactions);
					}
					else
					{
						UndoBuffer.Insert(RemovedTransactions, 0);
					}

					RemovedTransactions.Reset();
					UndoCount = PreviousUndoCount;

					UndoBufferChangedDelegate.Broadcast();
				}
			}
			GUndo = nullptr;
			PreviousUndoCount = INDEX_NONE;
			RemovedTransactions.Reset();
		}
		ActiveRecordCounts.Pop();
		if (bTransactionFinalized)
		{
			FTransactionContext Context;
			Context.TransactionId = TransactionId;
			TransactionStateChangedDelegate.Broadcast(Context, ETransactionStateEventType::PostTransactionFinalized);
		}
		CheckState();
	}
	return Result;
}


void UTransBuffer::Reset( const FText& Reason )
{
	if (ensure(!GIsTransacting))
	{
		CheckState();

		if (ActiveCount != 0)
		{
			FString ErrorMessage = TEXT("");
			ErrorMessage += FString::Printf(TEXT("Non zero active count in UTransBuffer::Reset") LINE_TERMINATOR);
			ErrorMessage += FString::Printf(TEXT("ActiveCount : %d") LINE_TERMINATOR, ActiveCount);
			ErrorMessage += FString::Printf(TEXT("SessionName : %s") LINE_TERMINATOR, *GetUndoContext(false).Context);
			ErrorMessage += FString::Printf(TEXT("Reason      : %s") LINE_TERMINATOR, *Reason.ToString());

			ErrorMessage += FString::Printf(LINE_TERMINATOR);
			ErrorMessage += FString::Printf(TEXT("Purging the undo buffer...") LINE_TERMINATOR);

			UE_LOGF(LogEditorTransaction, Log, "%ls", *ErrorMessage);


			// Clear out the transaction buffer...
			Cancel(0);
		}

		// Reset all transactions.
		UndoBuffer.Empty();
		UndoCount = 0;
		ResetReason = Reason;
		ActiveCount = 0;
		ActiveRecordCounts.Empty();
		ClearUndoBarriers();
		UndoBufferChangedDelegate.Broadcast();

		CheckState();
	}
}


void UTransBuffer::Cancel( int32 StartIndex /*=0*/ )
{
	CheckState();

	// if we don't have any active actions, we shouldn't have an active transaction at all
	if ( ActiveCount > 0 )
	{
		// Canceling partial transaction isn't supported properly at this time, just cancel the transaction entirely
		if (StartIndex != 0)
		{
			FString TransactionTitle = GUndo ? GUndo->GetContext().Title.ToString() : FString(TEXT("Unknown"));
			UE_LOGF(LogEditorTransaction, Warning, "Canceling transaction partially is unsupported. Canceling %ls entirely.", *TransactionTitle);
			StartIndex = 0;
		}

		// StartIndex needs to be 0 when cancelling
		{
			if (GUndo)
			{
				TransactionStateChangedDelegate.Broadcast(GUndo->GetContext(), ETransactionStateEventType::TransactionCanceled);
				GUndo->EndOperation();
			}

			// clear the global pointer to the soon-to-be-deleted transaction
			GUndo = nullptr;
			
			UndoBuffer.Pop(EAllowShrinking::No);
			UndoBuffer.Reserve(UndoBuffer.Num() + RemovedTransactions.Num());

			if (PreviousUndoCount > 0)
			{
				UndoBuffer.Append(RemovedTransactions);
			}
			else
			{
				UndoBuffer.Insert(RemovedTransactions, 0);
			}

			RemovedTransactions.Reset();

			UndoCount = PreviousUndoCount;
			PreviousUndoCount = INDEX_NONE;
			UndoBufferChangedDelegate.Broadcast();
		}

		// reset the active count
		ActiveCount = StartIndex;
		ActiveRecordCounts.SetNum(StartIndex);
	}

	CheckState();
}


bool UTransBuffer::CanUndo( FText* Text )
{
	CheckState();
	if (ActiveCount || CurrentTransaction != nullptr )
	{
		if (Text)
		{
			*Text = GUndo ? FText::Format(NSLOCTEXT("TransactionSystem", "CantUndoDuringTransactionX", "(Can't undo while '{0}' is in progress)"), GUndo->GetContext().Title) : NSLOCTEXT("TransactionSystem", "CantUndoDuringTransaction", "(Can't undo while action is in progress)");
		}
		return false;
	}
	
	if (UndoBarrierStack.Num())
	{
		const int32 UndoBarrier = UndoBarrierStack.Last();
		if (UndoBuffer.Num() - UndoCount <= UndoBarrier)
		{
			if (Text)
			{
				*Text = NSLOCTEXT("TransactionSystem", "HitUndoBarrier", "(Hit Undo barrier; can't undo any further)");
			}
			return false;
		}
	}

	if (UndoBuffer.Num() == UndoCount)
	{
		if( Text )
		{
			*Text = FText::Format( NSLOCTEXT("TransactionSystem", "CantUndoAfter", "(Can't undo after: {0})"), ResetReason );
		}
		return false;
	}
	return true;
}


bool UTransBuffer::CanRedo( FText* Text )
{
	CheckState();
	if( ActiveCount || CurrentTransaction != nullptr )
	{
		if( Text )
		{
			*Text = GUndo ? FText::Format(NSLOCTEXT("TransactionSystem", "CantRedoDuringTransactionX", "(Can't redo while '{0}' is in progress)"), GUndo->GetContext().Title) : NSLOCTEXT("TransactionSystem", "CantRedoDuringTransaction", "(Can't redo while action is in progress)");
		}
		return 0;
	}
	if( UndoCount==0 )
	{
		if( Text )
		{
			*Text = NSLOCTEXT("TransactionSystem", "NothingToRedo", "(Nothing to redo)");
		}
		return 0;
	}
	return 1;
}


int32 UTransBuffer::FindTransactionIndex(const FGuid & TransactionId) const
{
	for (int32 Index = 0; Index < UndoBuffer.Num(); ++Index)
	{
		if (UndoBuffer[Index]->GetId() == TransactionId)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}


const FTransaction* UTransBuffer::GetTransaction( int32 QueueIndex ) const
{
	if (UndoBuffer.Num() > QueueIndex && QueueIndex != INDEX_NONE)
	{
		return &UndoBuffer[QueueIndex].Get();
	}

	return NULL;
}


FTransactionContext UTransBuffer::GetUndoContext( bool bCheckWhetherUndoPossible )
{
	FTransactionContext Context;
	FText Title;
	if( bCheckWhetherUndoPossible && !CanUndo( &Title ) )
	{
		Context.Title = Title;
		return Context;
	}

	if (UndoBuffer.Num() > UndoCount)
	{
		TSharedRef<FTransaction>& Transaction = UndoBuffer[UndoBuffer.Num() - (UndoCount + 1)];
		return Transaction->GetContext();
	}

	return Context;
}


FTransactionContext UTransBuffer::GetRedoContext()
{
	FTransactionContext Context;
	FText Title;
	if( !CanRedo( &Title ) )
	{
		Context.Title = Title;
		return Context;
	}

	TSharedRef<FTransaction>& Transaction = UndoBuffer[ UndoBuffer.Num() - UndoCount ];
	return Transaction->GetContext();
}


void UTransBuffer::SetUndoBarrier()
{
	UndoBarrierStack.Push(UndoBuffer.Num() - UndoCount);
}


void UTransBuffer::RemoveUndoBarrier()
{
	if (UndoBarrierStack.Num() > 0)
	{
		UndoBarrierStack.Pop();
	}
}


void UTransBuffer::ClearUndoBarriers()
{
	UndoBarrierStack.Empty();
}

int32 UTransBuffer::GetCurrentUndoBarrier() const
{
	if (UndoBarrierStack.Num() > 0)
	{
		return UndoBarrierStack.Last();
	}

	return INDEX_NONE;
}


bool UTransBuffer::Undo(bool bCanRedo)
{
	CheckState();

	if (!CanUndo())
	{
		UndoDelegate.Broadcast(FTransactionContext(), false);

		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UTransBuffer::Undo);

	// Apply the undo changes.
	GIsTransacting = true;

	// custom changes (FChange) can be applied to temporary objects that require undo/redo for some time,
	// but we want to skip over these changes later (eg in the context of a Tool that is used for a while and
	// then closed). In this case the Transaction is "expired" and we continue to Undo until we find a 
	// non-Expired Transaction. 
	bool bDoneTransacting = false;
	do
	{
		FTransaction& Transaction = UndoBuffer[ UndoBuffer.Num() - ++UndoCount ].Get();
		if (Transaction.HasExpired() == false)
		{
			UE_LOGF(LogEditorTransaction, Log, "Undo %ls", *Transaction.GetTitle().ToString());
			CurrentTransaction = &Transaction;
			CurrentTransaction->BeginOperation();

			const FTransactionContext TransactionContext = CurrentTransaction->GetContext();
			TransactionStateChangedDelegate.Broadcast(TransactionContext, ETransactionStateEventType::UndoRedoStarted);
			BeforeRedoUndoDelegate.Broadcast(TransactionContext);
			Transaction.Apply();
			UndoDelegate.Broadcast(TransactionContext, true);
			TransactionStateChangedDelegate.Broadcast(TransactionContext, ETransactionStateEventType::UndoRedoFinalized);

			CurrentTransaction->EndOperation();
			CurrentTransaction = nullptr;

			bDoneTransacting = true;
		}

		if (!bCanRedo)
		{
			UndoBuffer.RemoveAt(UndoBuffer.Num() - UndoCount, UndoCount);
			UndoCount = 0;

			UndoBufferChangedDelegate.Broadcast();
		}
	} 
	while (bDoneTransacting == false && CanUndo());

	GIsTransacting = false;

	// if all transactions were expired, reproduce the !CanUndo() branch at the top of the function
	if (bDoneTransacting == false)
	{
		UndoDelegate.Broadcast(FTransactionContext(), false);
		return false;
	}

	CheckState();

	return true;
}

bool UTransBuffer::Redo()
{
	CheckState();

	if (!CanRedo())
	{
		RedoDelegate.Broadcast(FTransactionContext(), false);

		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UTransBuffer::Redo);

	// Apply the redo changes.
	GIsTransacting = true;

	// Skip over Expired transactions (see comments in ::Undo)
	bool bDoneTransacting = false;
	do
	{
		FTransaction& Transaction = UndoBuffer[ UndoBuffer.Num() - UndoCount-- ].Get();
		if (Transaction.HasExpired() == false)
		{
			UE_LOGF(LogEditorTransaction, Log, "Redo %ls", *Transaction.GetTitle().ToString());
			CurrentTransaction = &Transaction;
			CurrentTransaction->BeginOperation();

			const FTransactionContext TransactionContext = CurrentTransaction->GetContext();
			TransactionStateChangedDelegate.Broadcast(TransactionContext, ETransactionStateEventType::UndoRedoStarted);
			BeforeRedoUndoDelegate.Broadcast(TransactionContext);
			Transaction.Apply();
			RedoDelegate.Broadcast(TransactionContext, true);
			TransactionStateChangedDelegate.Broadcast(TransactionContext, ETransactionStateEventType::UndoRedoFinalized);

			CurrentTransaction->EndOperation();
			CurrentTransaction = nullptr;

			bDoneTransacting = true;
		}
	} 
	while (bDoneTransacting == false && CanRedo());

	GIsTransacting = false;

	// if all transactions were expired, reproduce the !CanRedo() branch at the top of the function
	if (bDoneTransacting == false)
	{
		RedoDelegate.Broadcast(FTransactionContext(), false);
		return false;
	}

	CheckState();

	return true;
}

bool UTransBuffer::EnableObjectSerialization()
{
	return --DisallowObjectSerialization == 0;
}

bool UTransBuffer::DisableObjectSerialization()
{
	return ++DisallowObjectSerialization == 0;
}


SIZE_T UTransBuffer::GetUndoSize() const
{
	SIZE_T Result=0;
	for( int32 i=0; i<UndoBuffer.Num(); i++ )
	{
		Result += UndoBuffer[i]->DataSize();
	}
	return Result;
}


void UTransBuffer::CheckState() const
{
	// Validate the internal state.
	check(UndoBuffer.Num()>=UndoCount);
	check(ActiveCount>=0);
	check(ActiveRecordCounts.Num() == ActiveCount);
}


void UTransBuffer::SetPrimaryUndoObject(UObject* PrimaryObject)
{
	// Only record the primary object if its transactional, not in any of the temporary packages and theres an active transaction
	if ( PrimaryObject && PrimaryObject->HasAnyFlags( RF_Transactional ) &&
		(PrimaryObject->GetOutermost()->HasAnyPackageFlags( PKG_PlayInEditor|PKG_ContainsScript|PKG_CompiledIn ) == false) )
	{
		const int32 NumTransactions = UndoBuffer.Num();
		const int32 CurrentTransactionIdx = NumTransactions - (UndoCount + 1);

		if ( CurrentTransactionIdx >= 0 )
		{
			TSharedRef<FTransaction>& Transaction = UndoBuffer[ CurrentTransactionIdx ];
			Transaction->SetPrimaryObject(PrimaryObject);
		}
	}
}

bool UTransBuffer::IsObjectInTransactionBuffer( const UObject* Object ) const
{
	TArray<UObject*> TransactionObjects;
	for( const TSharedRef<FTransaction>& Transaction : UndoBuffer )
	{
		Transaction->GetTransactionObjects(TransactionObjects);

		if( TransactionObjects.Contains(Object) )
		{
			return true;
		}
		
		TransactionObjects.Reset();
	}

	return false;
}

bool UTransBuffer::IsObjectTransacting(const UObject* Object) const
{
	// We can't provide a truly meaningful answer to this question when not transacting:
	if (ensure(CurrentTransaction))
	{
		return CurrentTransaction->IsObjectTransacting(Object);
	}

	return false;
}

bool UTransBuffer::ContainsPieObjects() const
{
	for( const TSharedRef<FTransaction>& Transaction : UndoBuffer )
	{
		if( Transaction->ContainsPieObjects() )
		{
			return true;
		}
	}

	return false;
}
