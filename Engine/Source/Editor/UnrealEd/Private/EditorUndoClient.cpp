// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUndoClient.h"
#include "Editor.h"

FEditorUndoClient::~FEditorUndoClient()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

FSelfRegisteringEditorUndoClient::FSelfRegisteringEditorUndoClient()
{
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}