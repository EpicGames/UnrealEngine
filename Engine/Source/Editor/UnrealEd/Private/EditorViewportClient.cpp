// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportClient.h"

#include "ActorFactories/ActorFactory.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "PreviewScene.h"
#include "HAL/FileManager.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/DebugDisplayProperty.h"
#include "Engine/RendererSettings.h"
#include "MeshEdges.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/BillboardComponent.h"
#include "Audio/AudioDebug.h"
#include "Debug/DebugDrawService.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "EditorModes.h"
#include "MouseDeltaTracker.h"
#include "CameraController.h"
#include "HighResScreenshot.h"
#include "EditorDragTools.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "EngineModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SEditorViewport.h"
#include "AssetEditorModeManager.h"
#include "PixelInspectorModule.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "IXRCamera.h"
#include "SceneViewExtension.h"
#include "LegacyScreenPercentageDriver.h"
#include "ComponentRecreateRenderStateContext.h"
#include "EditorBuildUtils.h"
#include "AudioDevice.h"
#include "Elements/Framework/TypedElementViewportInteraction.h"
#include "ImageWriteQueue.h"
#include "Misc/ScopedSlowTask.h"
#include "UnrealEngine.h"
#include "BufferVisualizationData.h"
#include "EditorInteractiveGizmoManager.h"
#include "NaniteVisualizationData.h"
#include "LumenVisualizationData.h"
#include "MegaLightsVisualizationData.h"
#include "RayTracingVisualizationData.h"
#include "SubstrateVisualizationData.h"
#include "GroomVisualizationData.h"
#include "VirtualShadowMapVisualizationData.h"
#include "VT/VirtualTextureVisualizationData.h"
#include "GPUSkinCacheVisualizationData.h"
#include "UnrealWidget.h"
#include "Engine/World.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "ProfilingDebugging/TraceScreenshot.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "IImageWrapperModule.h"
#include "HDRHelper.h"
#include "GlobalRenderResources.h"
#include "Settings/EditorStyleSettings.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "LandscapeRender.h"
#include "LandscapeSettings.h"
#include "TransformGizmoEditorSettings.h"
#include "Misc/AxisDisplayInfo.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportClientNavigationHelper.h"
#include "Engine/LevelBounds.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportInteraction.h"
#include "WorldPartition/WorldPartition.h"
#include "RenderViewportFeedback.h"
#include "Slate/SceneViewport.h"

#define LOCTEXT_NAMESPACE "EditorViewportClient"

const EViewModeIndex FEditorViewportClient::DefaultPerspectiveViewMode = VMI_Lit;
const EViewModeIndex FEditorViewportClient::DefaultOrthoViewMode = VMI_Lit;

static TAutoConsoleVariable<int32> CVarAlignedOrthoZoom(
	TEXT("r.Editor.AlignedOrthoZoom"),
	1,
	TEXT("Only affects the editor ortho viewports.\n")
	TEXT(" 0: Each ortho viewport zoom in defined by the viewport width\n")
	TEXT(" 1: All ortho viewport zoom are locked to each other to allow axis lines to be aligned with each other."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarEditorDisableFogInOrthoDebugViews(
	TEXT("r.Editor.DisableFogInOrthoDebugViews"),
	true,
	TEXT("Fog can obstruct the view when displaying it in ortho debug views without the correct settings adjusted, so this override disables by default."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarEditorViewportTest(
	TEXT("r.Test.EditorConstrainedView"),
	0,
	TEXT("Allows to test different viewport rectangle configuations (in game only) as they can happen when using cinematics/Editor.\n")
	TEXT("0: off(default)\n")
	TEXT("1..7: Various Configuations"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarOrthoEditorDebugClipPlaneScale(
	TEXT("r.Ortho.EditorDebugClipPlaneScale"),
	1.0f,
	TEXT("Only affects the editor ortho viewports in Lit modes.\n")
	TEXT("Set the scale to proportionally alter the near plane based on current Ortho width that is set.\n")
	TEXT("This changes when geometry clips in the scene as the Orthozoom is changed. Helpful for varying mesh sizes.\n")
	TEXT("Other light artefacts may appear when this value changes, this is unavoidable for now.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarEditorViewGPUMirrorTest(
	TEXT("r.Test.EditorViewGPUMirror"),
	0,
	TEXT("Tests FSceneViewStateInterface system memory mirror functionality in the editor viewport for debugging.\n")
	TEXT("Mirroring is used for Movie Render Queue high resolution tiled rendering, but debugging it there can be\n")
	TEXT("difficult, due to lack of interactivity, and multiple scene renders happening per frame.\n"),
	ECVF_RenderThreadSafe);

static bool GetDefaultLowDPIPreviewValue()
{
	static auto CVarEditorViewportHighDPIPtr = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Editor.Viewport.HighDPI"));
	return CVarEditorViewportHighDPIPtr->GetInt() == 0;
}

/**
 * Returns true if a viewport interaction with the specified name is enabled and able to act 
 */
bool IsViewportInteractionEnabled(const FEditorModeTools* InModeTools, FName InViewportInteraction)
{
	if (InModeTools)
	{
		if (UViewportInteractionsBehaviorSource* InputToolsBehaviorSource = InModeTools->GetInteractiveToolsContext()->GetViewportInteractionsBehaviorSource())
		{
			if (UViewportInteraction* Interaction = InputToolsBehaviorSource->FindInteraction(InViewportInteraction))
			{
				return Interaction->IsEnabled();
			}
		}
	}

	return false;
}

/**
 * Returns true if a viewport interaction with the specified name is active and currently acting 
 */
bool IsViewportInteractionActive(const FEditorModeTools* InModeTools, FName InViewportInteraction)
{
	if (InModeTools)
	{
		if (UViewportInteractionsBehaviorSource* InputToolsBehaviorSource = InModeTools->GetInteractiveToolsContext()->GetViewportInteractionsBehaviorSource())
		{
			if (UViewportInteraction* Interaction = InputToolsBehaviorSource->FindInteraction(InViewportInteraction))
			{
				return Interaction->IsActive();
			}
		}
	}

	return false;
}

/**
 * Returns true if any viewport interaction in the specified group is enabled and able to act. 
 */
bool IsViewportInteractionGroupEnabled(const FEditorModeTools* InModeTools, FName InViewportInteractionGroup)
{
	if (InModeTools)
	{
		if (UViewportInteractionsBehaviorSource* InputToolsBehaviorSource = InModeTools->GetInteractiveToolsContext()->GetViewportInteractionsBehaviorSource())
		{
			return InputToolsBehaviorSource->IsAnyInteractionInGroupEnabled(InViewportInteractionGroup);\
		}
	}

	return false;
}

float ComputeOrthoZoomFactor(const float ViewportWidth)
{
	float Ret = 1.0f;

	if(CVarAlignedOrthoZoom.GetValueOnGameThread())
	{
		// We want to have all ortho view ports scale the same way to have the axis aligned with each other.
		// So we take out the usual scaling of a view based on it's width.
		// That means when a view port is resized in x or y it shows more content, not the same content larger (for x) or has no effect (for y).
		// 500 is to get good results with existing view port settings.
		Ret = ViewportWidth / 500.0f;
	}

	return Ret;
}

void PixelInspectorRealtimeManagement(FEditorViewportClient *CurrentViewport, bool bMouseEnter)
{
	FPixelInspectorModule& PixelInspectorModule = FModuleManager::LoadModuleChecked<FPixelInspectorModule>(TEXT("PixelInspectorModule"));
	bool bViewportIsRealtime = CurrentViewport->IsRealtime();
	bool bViewportShouldBeRealtime = PixelInspectorModule.GetViewportRealtime(CurrentViewport->ViewIndex, bViewportIsRealtime, bMouseEnter);
	if (bViewportIsRealtime != bViewportShouldBeRealtime)
	{
		CurrentViewport->SetRealtime(bViewportShouldBeRealtime);
	}
}

namespace EditorViewportClient
{
	static const float GridSize = 2048.0f;
	static const int8 CellSize = 16;
	static const float LightRotSpeed = 0.22f;
}

namespace OrbitConstants
{
	const float OrbitPanSpeed = 1.0f;
	const float IntialLookAtDistance = 1024.f;
}

namespace FocusConstants
{
	const float TransitionTime = 0.25f;
}

namespace PreviewLightConstants
{
	const float MovingPreviewLightTimerDuration = 1.0f;

	const float MinMouseRadius = 100.0f;
	const float MinArrowLength = 10.0f;
	const float ArrowLengthToSizeRatio = 0.1f;
	const float MouseLengthToArrowLengthRatio = 0.2f;

	const float ArrowLengthToThicknessRatio = 0.05f;
	const float MinArrowThickness = 2.0f;

	// Note: MinMouseRadius must be greater than MinArrowLength
}

namespace GeometryInspectionZebraMaterialParameterNames
{
	const FName Orientation = TEXT("XYZ");
	const FName StripeCount = TEXT("Stripe Counts");
}

/**
 * Cached off joystick input state
 */
class FCachedJoystickState
{
public:
	uint32 JoystickType;
	TMap <FKey, float> AxisDeltaValues;
	TMap <FKey, EInputEvent> KeyEventValues;
};


///////////////////////////////////////////////////////////////////////////////////////////////////
//
//	FViewportCursorLocation
//	Contains information about a mouse cursor position within a viewport, transformed into the correct
//	coordinate system for the viewport.
//
///////////////////////////////////////////////////////////////////////////////////////////////////
FViewportCursorLocation::FViewportCursorLocation(const FSceneView* View, FEditorViewportClient* InViewportClient, int32 X, int32 Y, const bool bInClampToBounds)
	: Origin(ForceInit), Direction(ForceInit), CursorPos(X, Y)
{
	const FVector4 ScreenPos = View->CursorToScreen(static_cast<float>(X), static_cast<float>(Y), 0.0f, bInClampToBounds);

	const FMatrix InvViewMatrix = View->ViewMatrices.GetViewToWorld();
	const FMatrix InvProjMatrix = View->ViewMatrices.GetClipToView();

	const double ScreenX = ScreenPos.X;
	const double ScreenY = ScreenPos.Y;

	ViewportClient = InViewportClient;

	if (ViewportClient->IsPerspective())
	{
		Origin = View->ViewMatrices.GetViewOrigin();
		Direction = InvViewMatrix.TransformVector(FVector(InvProjMatrix.TransformFVector4(FVector4(ScreenX * GNearClippingPlane, ScreenY * GNearClippingPlane, 0.0f, GNearClippingPlane)))).GetSafeNormal();
	}
	else
	{
		Origin = InvViewMatrix.TransformFVector4(InvProjMatrix.TransformFVector4(FVector4(ScreenX, ScreenY, 0.5f, 1.0f)));
		Direction = InvViewMatrix.TransformVector(FVector(0, 0, 1)).GetSafeNormal();
	}
}

FViewportCursorLocation::~FViewportCursorLocation()
{
}

ELevelViewportType FViewportCursorLocation::GetViewportType() const
{
	return ViewportClient->GetViewportType();
}

///////////////////////////////////////////////////////////////////////////////////////////////////
//
//	FViewportClick::FViewportClick - Calculates useful information about a click for the below ClickXXX functions to use.
//
///////////////////////////////////////////////////////////////////////////////////////////////////
FViewportClick::FViewportClick(const FSceneView* View, FEditorViewportClient* ViewportClient, FKey InKey, EInputEvent InEvent, int32 X, int32 Y)
	: FViewportCursorLocation(View, ViewportClient, X, Y)
	, Key(InKey), Event(InEvent)
{
	ControlDown = ViewportClient->IsCtrlPressed();
	ShiftDown = ViewportClient->IsShiftPressed();
	AltDown = ViewportClient->IsAltPressed();
}

static const FName InputChordName_CameraLockedToWidget = FName("InputChordName_CameraLockedToWidget");

FViewportClick::~FViewportClick()
{
}

FViewportCameraTransform::FViewportCameraTransform()
	: TransitionCurve( new FCurveSequence( 0.0f, FocusConstants::TransitionTime, ECurveEaseFunction::CubicOut ) )
	, ViewLocation( FVector::ZeroVector )
	, ViewRotation( FRotator::ZeroRotator )
	, DesiredLocation( FVector::ZeroVector )
	, LookAt( FVector::ZeroVector )
	, StartLocation( FVector::ZeroVector )
	, OrthoZoom( DEFAULT_ORTHOZOOM )
	, MaxLocation( UE_DOUBLE_HUGE_DISTANCE )
{}

void FViewportCameraTransform::SetLocation( const FVector& Position )
{
	FVector ClampedPosition = Position.BoundToCube(MaxLocation);
	ViewLocation = ClampedPosition;
	DesiredLocation = ViewLocation;
}

void FViewportCameraTransform::TransitionToLocation(const FVector& InDesiredLocation, TWeakPtr<SWidget> EditorViewportWidget, bool bInstant)
{
	if( bInstant || !EditorViewportWidget.IsValid() )
	{
		SetLocation( InDesiredLocation );
		TransitionCurve->JumpToEnd();
	}
	else
	{
		DesiredLocation = InDesiredLocation;
		StartLocation = ViewLocation;

		TransitionCurve->Play(EditorViewportWidget.Pin().ToSharedRef());
	}
}


bool FViewportCameraTransform::UpdateTransition()
{
	bool bIsAnimating = false;
	if (TransitionCurve->IsPlaying() || ViewLocation != DesiredLocation)
	{
		float LerpWeight = TransitionCurve->GetLerp();

		if( LerpWeight == 1.0f )
		{
			// Failsafe for the value not being exact on lerps
			ViewLocation = DesiredLocation;
		}
		else
		{
			ViewLocation = FMath::Lerp( StartLocation, DesiredLocation, LerpWeight );
		}


		bIsAnimating = true;
	}

	return bIsAnimating;
}

FMatrix FViewportCameraTransform::ComputeOrbitMatrix() const
{
	FTransform Transform =
	FTransform( -LookAt ) *
	FTransform( FRotator(0,ViewRotation.Yaw,0) ) *
	FTransform( FRotator(0, 0, ViewRotation.Pitch) ) *
	FTransform( FVector(0,(ViewLocation - LookAt).Size(), 0) );

	return Transform.ToMatrixNoScale() * FInverseRotationMatrix( FRotator(0,90.f,0) );
}

bool FViewportCameraTransform::IsPlaying()
{
	return TransitionCurve->IsPlaying();
}

/**The Maximum Mouse/Camera Speeds Setting supported */
const uint32 FEditorViewportClient::MaxCameraSpeeds = 8;

float FEditorViewportClient::GetCameraSpeed() const
{
	return GetCameraSpeedSettings().GetCurrentSpeed();
}

const FEditorViewportCameraSpeedSettings& FEditorViewportClient::GetCameraSpeedSettings() const
{
	return CameraSpeedSettings;
}

void FEditorViewportClient::SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings)
{
	CameraSpeedSettings = InCameraSpeedSettings;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float FEditorViewportClient::GetCameraSpeed(int32 SpeedSetting) const
{
	//previous mouse speed values were as follows...
	//(note: these were previously all divided by 4 when used be the viewport)
	//#define MOVEMENTSPEED_SLOW			4	~ 1
	//#define MOVEMENTSPEED_NORMAL			12	~ 3
	//#define MOVEMENTSPEED_FAST			32	~ 8
	//#define MOVEMENTSPEED_VERYFAST		64	~ 16

	const int32 SpeedToUse = FMath::Clamp<int32>(SpeedSetting, 1, MaxCameraSpeeds);
	const float Speed[] = { 0.033f, 0.1f, 0.33f, 1.f, 3.f, 8.f, 16.f, 32.f };

	return Speed[SpeedToUse - 1];
}

void FEditorViewportClient::SetCameraSpeedSetting(int32 SpeedSetting)
{
	CameraSpeedSetting = SpeedSetting;
}

int32 FEditorViewportClient::GetCameraSpeedSetting() const
{
	return CameraSpeedSetting;
}

float FEditorViewportClient::GetCameraSpeedScalar() const
{
	return CameraSpeedScalar;
}

void FEditorViewportClient::SetCameraSpeedScalar(float SpeedScalar)
{
	CameraSpeedScalar = FMath::Clamp<float>(SpeedScalar, 1.0f, TNumericLimits <float>::Max());
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FEditorViewportClient::TakeOwnershipOfModeManager(TSharedPtr<FEditorModeTools>& ModeManagerPtr)
{
	ModeManagerPtr = ModeTools;
}

float const FEditorViewportClient::SafePadding = 0.075f;

static int32 ViewOptionIndex = 0;
static TArray<ELevelViewportType> ViewOptions;

void InitViewOptionsArray()
{
	ViewOptions.Empty();

	ELevelViewportType Left = ELevelViewportType::LVT_OrthoLeft;
	ELevelViewportType Right = ELevelViewportType::LVT_OrthoRight;
	ELevelViewportType Top = ELevelViewportType::LVT_OrthoTop;
	ELevelViewportType Bottom = ELevelViewportType::LVT_OrthoBottom;
	ELevelViewportType Back = ELevelViewportType::LVT_OrthoBack;
	ELevelViewportType Front = ELevelViewportType::LVT_OrthoFront;

	ViewOptions.Add(Front);
	ViewOptions.Add(Back);
	ViewOptions.Add(Top);
	ViewOptions.Add(Bottom);
	ViewOptions.Add(Left);
	ViewOptions.Add(Right);
}

FEditorViewportClient::FEditorViewportClient(FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: bAllowCinematicControl(false)
	, CameraSpeedSetting(4)
	, CameraSpeedScalar(1.0f)
	, ImmersiveDelegate()
	, VisibilityDelegate()
	, Viewport(nullptr)
	, ViewportType(LVT_Perspective)
	, ViewState()
	, StereoViewStates()
	, EngineShowFlags(ESFIM_Editor)
	, LastEngineShowFlags(ESFIM_Game)
	, ExposureSettings()
	, CurrentBufferVisualizationMode(NAME_None)
	, CurrentNaniteVisualizationMode(NAME_None)
	, CurrentLumenVisualizationMode(NAME_None)
	, CurrentMegaLightsVisualizationMode(NAME_None)
	, CurrentSubstrateVisualizationMode(NAME_None)
	, CurrentGroomVisualizationMode(NAME_None)
	, CurrentVirtualShadowMapVisualizationMode(NAME_None)
	, CurrentVirtualTextureVisualizationMode(NAME_None)
	, CurrentRayTracingDebugVisualizationMode(NAME_None)
	, CurrentGPUSkinCacheVisualizationMode(NAME_None)
	, FramesSinceLastDraw(0)
	, ViewIndex(INDEX_NONE)
	, ViewFOV(EditorViewportDefs::DefaultPerspectiveFOVAngle)
	, FOVAngle(EditorViewportDefs::DefaultPerspectiveFOVAngle)
	, AspectRatio(1.777777f)
	, bForcingUnlitForNewMap(false)
	, bWidgetAxisControlledByDrag(false)
	, bNeedsRedraw(true)
	, bNeedsLinkedRedraw(false)
	, bNeedsInvalidateHitProxy(false)
	, bUsingOrbitCamera(false)
	, bUseNumpadCameraControl(true)
	, bDisableInput(false)
	, bDrawAxes(true)
	, bDrawAxesGame(false)
	, bSetListenerPosition(false)
	, LandscapeLODOverride(-1)
	, bDrawVertices(false)
	, bShouldApplyViewModifiers(true)
	, ModeTools(InModeTools ? InModeTools->AsShared() : TSharedPtr<FEditorModeTools>())
	, Widget(new FWidget)
	, bShowWidget(true)
	, MouseDeltaTracker(new FMouseDeltaTracker)
	, bHasMouseMovedSinceClick(false)
	, CameraController(new FEditorCameraController())
	, CameraUserImpulseData(new FCameraControllerUserImpulseData())
	, ViewportNavigationHelper(new FViewportClientNavigationHelper())
	, TimeForForceRedraw(0.0)
	, FlightCameraSpeedScale(1.0f)
	, bUseControllingActorViewInfo(false)
	, LastMouseX(0)
	, LastMouseY(0)
	, CachedMouseX(0)
	, CachedMouseY(0)
	, CurrentMousePos(-1, -1)
	, bIsTracking(false)
	, bDraggingByHandle(false)
	, CurrentGestureDragDelta(FVector::ZeroVector)
	, CurrentGestureRotDelta(FRotator::ZeroRotator)
	, GestureMoveForwardBackwardImpulse(0.0f)
	, bForceAudioRealtime(false)
	, RealTimeUntilFrameNumber(0)
	, bIsRealtime(false)
	, bShowStats(false)
	, bHasAudioFocus(false)
	, bShouldCheckHitProxy(false)
	, bUsesDrawHelper(true)
	, bIsSimulateInEditorViewport(false)
	, bCameraLock(false)
	, bIsCameraMoving(false)
	, bIsCameraMovingOnTick(false)
	, EditorViewportWidget(InEditorViewportWidget)
	, ViewportInteraction(NewObject<UTypedElementViewportInteraction>())
	, PreviewScene(InPreviewScene)
	, MovingPreviewLightSavedScreenPos(ForceInitToZero)
	, MovingPreviewLightTimer(0.0f)
	, bLockFlightCamera(false)
	, SceneDPIMode(ESceneDPIMode::EditorDefault)
	, PerspViewModeIndex(DefaultPerspectiveViewMode)
	, OrthoViewModeIndex(DefaultOrthoViewMode)
	, ViewModeParam(-1)
	, NearPlane(-1.0f)
	, FarPlane(0.0f)
	, CachedWorldBounds(ForceInit)
	, bInGameViewMode(false)
	, bInVREditViewMode(false)
	, bShouldInvalidateViewportWidget(false)
	, DragStartView(nullptr)
	, DragStartViewFamily(nullptr)
	, bIsTrackingBeingStopped(false)
{
	InitViewOptionsArray();
	if (!ModeTools)
	{
		ModeTools = MakeShared<FAssetEditorModeManager>();
	}

	FSceneInterface* Scene = GetScene();
	ViewState.Allocate(Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel);

	// NOTE: StereoViewState will be allocated on demand, for viewports than end up drawing in stereo

	// add this client to list of views, and remember the index
	ViewIndex = GEditor->AddViewportClients(this);

	// Initialize the Cursor visibility struct
	RequiredCursorVisibiltyAndAppearance.bSoftwareCursorVisible = false;
	RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible = true;
	RequiredCursorVisibiltyAndAppearance.bDontResetCursor = false;
	RequiredCursorVisibiltyAndAppearance.bOverrideAppearance = false;
	RequiredCursorVisibiltyAndAppearance.RequiredCursor = EMouseCursor::Default;

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = false; // disable this since we rely on the show flags
	DrawHelper.GridColorAxis = FColor(160, 160, 160);
	DrawHelper.GridColorMajor = FColor(144, 144, 144);
	DrawHelper.GridColorMinor = FColor(128, 128, 128);
	DrawHelper.PerspectiveGridSize = EditorViewportClient::GridSize;
	DrawHelper.NumCells = DrawHelper.PerspectiveGridSize / ( EditorViewportClient::CellSize * 2 );

	// Most editor viewports do not want motion blur.
	EngineShowFlags.MotionBlur = 0;

	EngineShowFlags.SetSnap(1);

	SetViewMode(IsPerspective() ? PerspViewModeIndex : OrthoViewModeIndex);

	ModeTools->OnEditorModeIDChanged().AddRaw(this, &FEditorViewportClient::OnEditorModeIDChanged);

	FCoreDelegates::StatCheckEnabled.AddRaw(this, &FEditorViewportClient::HandleViewportStatCheckEnabled);
	FCoreDelegates::StatEnabled.AddRaw(this, &FEditorViewportClient::HandleViewportStatEnabled);
	FCoreDelegates::StatDisabled.AddRaw(this, &FEditorViewportClient::HandleViewportStatDisabled);
	FCoreDelegates::StatDisableAll.AddRaw(this, &FEditorViewportClient::HandleViewportStatDisableAll);

	RegisterPrioritizedInputChord(FPrioritizedInputChord(10, InputChordName_CameraLockedToWidget, EModifierKey::Shift));

	RequestUpdateDPIScale();

	FSlateApplication::Get().OnWindowDPIScaleChanged().AddRaw(this, &FEditorViewportClient::HandleWindowDPIScaleChanged);

	double MaxViewLocation = HALF_WORLD_MAX;
	if(GetScene() != nullptr && GetScene()->GetWorld() != nullptr)
	{
		AWorldSettings* WorldSettings = GetScene()->GetWorld()->GetWorldSettings(false);
		bool bEnableWorldBoundsChecks = WorldSettings ? WorldSettings->bEnableWorldBoundsChecks : true;
		if (!bEnableWorldBoundsChecks)
		{
			MaxViewLocation = UE_DOUBLE_HUGE_DISTANCE;
		}
	}
	ViewTransformPerspective.SetMaxLocation(MaxViewLocation);
	ViewTransformOrthographic.SetMaxLocation(MaxViewLocation);
	
	FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FEditorViewportClient::OnObjectTransacted);
	FEditorDelegates::MapChange.AddRaw(this, &FEditorViewportClient::OnPostLoadMap);
}

FEditorViewportClient::~FEditorViewportClient()
{
	if (ModeTools)
	{
		ModeTools->OnEditorModeIDChanged().RemoveAll(this);
	}

	delete Widget;
	delete MouseDeltaTracker;

	delete CameraController;
	CameraController = NULL;

	delete CameraUserImpulseData;
	CameraUserImpulseData = NULL;

	if(Viewport)
	{
		UE_LOGF(LogEditorViewport, Fatal, "Viewport != nullptr in FEditorViewportClient destructor.");
	}

	if(GEditor)
	{
		GEditor->RemoveViewportClients(this);
	}

	FCoreDelegates::StatCheckEnabled.RemoveAll(this);
	FCoreDelegates::StatEnabled.RemoveAll(this);
	FCoreDelegates::StatDisabled.RemoveAll(this);
	FCoreDelegates::StatDisableAll.RemoveAll(this);

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnWindowDPIScaleChanged().RemoveAll(this);
	}

	ModeTools.Reset();
	
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
}

FEditorViewportClient::FColorChannelMaskParams::FColorChannelMaskParams()
	: ColorChannelMask(EColorChannelMask::All)
	, bInvertAlphaChannelMask(false)
	, bDrawAlphaBlendedCheckerboard(false)
{
}

FLevelEditorViewportClient::FRealtimeOverride::FRealtimeOverride(bool bInIsRealtime, FText InSystemDisplayName)
	: SystemDisplayName(InSystemDisplayName)
	, bIsRealtime(bInIsRealtime)
{
}

void FEditorViewportClient::AddRealtimeOverride(bool bShouldBeRealtime, FText SystemDisplayName)
{
	RealtimeOverrides.Add(FRealtimeOverride(bShouldBeRealtime, SystemDisplayName));

	bShouldInvalidateViewportWidget = true;
}

bool FEditorViewportClient::HasRealtimeOverride(FText SystemDisplayName) const
{
	const FString SystemDisplayString = SystemDisplayName.BuildSourceString();
	for (int32 Index = 0; Index < RealtimeOverrides.Num(); ++Index)
	{
		const FString RealtimeOverrideSystemDisplayString = RealtimeOverrides[Index].SystemDisplayName.BuildSourceString();
		if (RealtimeOverrideSystemDisplayString == SystemDisplayString)
		{
			return true;
		}
	}
	return false;
}

bool FEditorViewportClient::RemoveRealtimeOverride(FText SystemDisplayName, bool bCheckMissingOverride)
{
	bool bRemoved = false;
	const FString SystemDisplayString = SystemDisplayName.BuildSourceString();
	for (int32 Index = RealtimeOverrides.Num() - 1; Index >= 0; --Index)
	{
		const FString RealtimeOverrideSystemDisplayString = RealtimeOverrides[Index].SystemDisplayName.BuildSourceString();
		if (RealtimeOverrideSystemDisplayString == SystemDisplayString)
		{
			RealtimeOverrides.RemoveAt(Index);
			bRemoved = true;
			break;
		}
	}
	ensureMsgf(!bCheckMissingOverride || bRemoved, TEXT("No realtime override was found with %s"), *SystemDisplayString);
	
	if (bRemoved)
	{
		bShouldInvalidateViewportWidget = true;
	}

	return bRemoved;
}

bool FEditorViewportClient::PopRealtimeOverride()
{
	if (RealtimeOverrides.Num() > 0)
	{
		RealtimeOverrides.Pop();

		bShouldInvalidateViewportWidget = true;

		return true;
	}

	return false;
}

bool FEditorViewportClient::ToggleRealtime()
{
	SetRealtime(!bIsRealtime);
	return bIsRealtime;
}

void FEditorViewportClient::SetRealtime(bool bInRealtime)
{
	bIsRealtime = bInRealtime;
	bShouldInvalidateViewportWidget = true;
}

bool FEditorViewportClient::DoRealtimeAndOverridesMatch(bool bInValue) const
{
	return bIsRealtime == bInValue && (RealtimeOverrides.Num() == 0 || RealtimeOverrides.Last().bIsRealtime == bInValue);
}

FText FEditorViewportClient::GetRealtimeOverrideMessage() const
{
	return RealtimeOverrides.Num() > 0 ? RealtimeOverrides.Last().SystemDisplayName : FText::GetEmpty();
}

void FEditorViewportClient::SetRealtime(bool bInRealtime, bool bStoreCurrentValue)
{
	SetRealtime(bInRealtime);
}

void FEditorViewportClient::RestoreRealtime(const bool bAllowDisable)
{
	PopRealtimeOverride();
}

void FEditorViewportClient::SaveRealtimeStateToConfig(bool& ConfigVar) const
{
	// Note, this should not ever look at anything but the true realtime state.  No overrides etc.
	ConfigVar = bIsRealtime;
}

void FEditorViewportClient::SetShowStats(bool bWantStats)
{
	bShowStats = bWantStats;
}

void FEditorViewportClient::InvalidateViewportWidget()
{
	if (EditorViewportWidget.IsValid())
	{
		// Invalidate the viewport widget to register its active timer
		EditorViewportWidget.Pin()->Invalidate();
	}
	bShouldInvalidateViewportWidget = false;
}

void FEditorViewportClient::RedrawRequested(FViewport* InViewport)
{
	bNeedsRedraw = true;
}

void FEditorViewportClient::RequestInvalidateHitProxy(FViewport* InViewport)
{
	bNeedsInvalidateHitProxy = true;
}

void FEditorViewportClient::OnEditorModeIDChanged(const FEditorModeID& EditorModeID, bool bIsEntering)
{
	if (Viewport)
	{
		RequestInvalidateHitProxy(Viewport);
	}
}

float FEditorViewportClient::GetOrthoUnitsPerPixel(const FViewport* InViewport) const
{
	const float SizeX = static_cast<float>(InViewport->GetSizeXY().X);

	// 15.0f was coming from the CAMERA_ZOOM_DIV marco, seems it was chosen arbitrarily
	return (GetOrthoZoom() / (SizeX * 15.f)) * ComputeOrthoZoomFactor(SizeX);
}

FVector FEditorViewportClient::GetForwardVector() const
{
	switch (GetViewportType())
	{
	case LVT_Perspective:
		return ViewTransformPerspective.GetRotation().Vector();
	case LVT_OrthoFreelook:
		return ViewTransformOrthographic.GetRotation().Vector();
	case LVT_OrthoTop:
		return FVector::DownVector;
	case LVT_OrthoBottom:
		return FVector::UpVector;
	case LVT_OrthoLeft:
		return FVector::LeftVector;
	case LVT_OrthoRight:
		return FVector::RightVector;
	case LVT_OrthoFront:
		return FVector::BackwardVector;
	case LVT_OrthoBack:
		return FVector::ForwardVector;
	default:
		return FVector::Zero();
	}
}

void FEditorViewportClient::SetViewLocationForOrbiting(const FVector& LookAtPoint, float DistanceToCamera )
{
	FMatrix Matrix = FTranslationMatrix(-GetViewLocation());
	Matrix = Matrix * FInverseRotationMatrix(GetViewRotation());
	FMatrix CamRotMat = Matrix.InverseFast();
	FVector CamDir = FVector(CamRotMat.M[0][0],CamRotMat.M[0][1],CamRotMat.M[0][2]);
	SetViewLocation( LookAtPoint - DistanceToCamera * CamDir );
	SetLookAtLocation( LookAtPoint );
}

void FEditorViewportClient::SetInitialViewTransform(ELevelViewportType InViewportType, const FVector& ViewLocation, const FRotator& ViewRotation, float InOrthoZoom )
{
	check(InViewportType < LVT_MAX);

	const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

	FViewportCameraTransform& ViewTransform = (InViewportType == LVT_Perspective) ? ViewTransformPerspective : ViewTransformOrthographic;
	ViewTransform.SetLocation(ViewLocation);
	ViewTransform.SetRotation(ViewRotation);

	// Make a look at location in front of the camera
	const FQuat CameraOrientation = FQuat::MakeFromEuler(ViewRotation.Euler());
	FVector Direction = CameraOrientation.RotateVector( FVector(1,0,0) );

	ViewTransform.SetLookAt(ViewLocation + Direction * OrbitConstants::IntialLookAtDistance);
	ViewTransform.SetOrthoZoom(InOrthoZoom);
}

void FEditorViewportClient::ToggleOrbitCamera( bool bEnableOrbitCamera )
{
	if( bUsingOrbitCamera != bEnableOrbitCamera )
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();

		bUsingOrbitCamera = bEnableOrbitCamera;

		// Convert orbit view to regular view
		FMatrix OrbitMatrix = ViewTransform.ComputeOrbitMatrix();
		OrbitMatrix = OrbitMatrix.InverseFast();

		if( !bUsingOrbitCamera )
		{
			// Ensure that the view location and rotation is up to date to ensure smooth transition in and out of orbit mode
			ViewTransform.SetRotation(OrbitMatrix.Rotator());
		}
		else
		{
			FRotator ViewRotation = ViewTransform.GetRotation();

			bool bUpsideDown = (ViewRotation.Pitch < -90.0f || ViewRotation.Pitch > 90.0f ||
				FMath::IsNearlyZero(FMath::Abs(ViewRotation.Roll) - 180.f, KINDA_SMALL_NUMBER));

			// if the camera is upside down compute the rotation differently to preserve pitch
			// otherwise the view will pop to right side up when transferring to orbit controls
			if( bUpsideDown )
			{
				FMatrix OrbitViewMatrix = FTranslationMatrix(-ViewTransform.GetLocation());
				OrbitViewMatrix *= FInverseRotationMatrix(ViewRotation);
				OrbitViewMatrix *= FRotationMatrix( FRotator(0,90.f,0) );

				FMatrix RotMat = FTranslationMatrix(-ViewTransform.GetLookAt()) * OrbitViewMatrix;
				FMatrix RotMatInv = RotMat.InverseFast();
				FRotator RollVec = RotMatInv.Rotator();
				FMatrix YawMat = RotMatInv * FInverseRotationMatrix( FRotator(0, 0, -RollVec.Roll));
				FMatrix YawMatInv = YawMat.InverseFast();
				FRotator YawVec = YawMat.Rotator();
				FRotator rotYawInv = YawMatInv.Rotator();
				ViewTransform.SetRotation(FRotator(-RollVec.Roll, YawVec.Yaw, 0));
			}
			else
			{
				ViewTransform.SetRotation(OrbitMatrix.Rotator());
			}
		}

		ViewTransform.SetLocation(OrbitMatrix.GetOrigin());
	}
}

void FEditorViewportClient::FocusViewportOnBox( const FBox& BoundingBox, bool bInstant /* = false */ )
{
	if (!Viewport)
	{
		return;
	}

	const FVector Position = BoundingBox.GetCenter();
	float Radius = FMath::Max<FVector::FReal>(BoundingBox.GetExtent().Size(), MinimumFocusRadius);

	float AspectToUse = AspectRatio;
	const FIntPoint ViewportSize = Viewport->GetSizeXY();
	if (!bUseControllingActorViewInfo && ViewportSize.X > 0 && ViewportSize.Y > 0)
	{
		AspectToUse = Viewport->GetDesiredAspectRatio();
	}

	CameraController->ResetVelocity();

	const bool bEnable = false;
	ToggleOrbitCamera(bEnable);

	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();

		if (!IsOrtho())
		{
			/**
				* We need to make sure we are fitting the sphere into the viewport completely, so if the height of the viewport is less
				* than the width of the viewport, we scale the radius by the aspect ratio in order to compensate for the fact that we have
				* less visible vertically than horizontally.
				*/
			if (AspectToUse > 1.0f)
			{
				Radius *= AspectToUse;
			}

			/**
				* Now that we have a adjusted radius, we are taking half of the viewport's FOV,
				* converting it to radians, and then figuring out the camera's distance from the center
				* of the bounding sphere using some simple trig.  Once we have the distance, we back up
				* along the camera's forward vector from the center of the sphere, and set our new view location.
				*/

			const float HalfFOVRadians = FMath::DegreesToRadians(ViewFOV / 2.0f);
			const float DistanceFromSphere = Radius / FMath::Tan(HalfFOVRadians);
			FVector CameraOffsetVector = ViewTransform.GetRotation().Vector() * -DistanceFromSphere;

			ViewTransform.SetLookAt(Position);
			ViewTransform.TransitionToLocation(Position + CameraOffsetVector, EditorViewportWidget, bInstant);

		}
		else
		{
			const FVector StartingLocation = ViewTransform.GetLocation();
			const FVector Forward = GetForwardVector();
				
			// Adjust the camera by the current forward instantly
			const FVector LocationDelta = Position - StartingLocation;
			const FVector ForwardLocationDelta = LocationDelta.ProjectOnToNormal(Forward);
			ViewTransform.SetLocation(StartingLocation + ForwardLocationDelta);
				
			// Adjust the near & far planes so that they remain fixed in world space
			const double ForwardDelta = ForwardLocationDelta.Dot(Forward);
				
			if (OrthographicNearPlane.IsSet())
			{
				OrthographicNearPlane = OrthographicNearPlane.GetValue() - ForwardDelta;
			}
				
			if (OrthographicFarPlane.IsSet())
			{
				OrthographicFarPlane = OrthographicFarPlane.GetValue() - ForwardDelta;
			}
				
			if (GetClipToSelectionOnFocus())
			{
				// Shift the orthographic near view plane back so that it is just before the object
				OrthographicNearPlane = -Radius;
					
				// Ensure the orthographic far view plane can display the object
				if (OrthographicFarPlane.IsSet())
				{
					OrthographicFarPlane = FMath::Max(OrthographicFarPlane.GetValue(), Radius);
				}
			}
		
			// For ortho viewports just set the camera position to the center of the bounding volume.
			//SetViewLocation( Position );
			ViewTransform.TransitionToLocation(Position, EditorViewportWidget, bInstant);
			
			if (!(Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl)))
			{
				/**
				* We also need to zoom out till the entire volume is in view.  The following block of code first finds the minimum dimension
				* size of the viewport.  It then calculates backwards from what the view size should be (The radius of the bounding volume),
				* to find the new OrthoZoom value for the viewport. The 15.0f is a fudge factor.
				*/
				float NewOrthoZoom = DEFAULT_ORTHOZOOM;
				
				if (ViewportSize.X > 0 && ViewportSize.Y > 0)
				{
					uint32 MinAxisSize = (AspectToUse > 1.0f) ? ViewportSize.Y : ViewportSize.X;
					float Zoom = Radius / (MinAxisSize / 2.0f);
					NewOrthoZoom = Zoom * (MinAxisSize * 15.0f);
				}

				NewOrthoZoom = FMath::Clamp<float>(NewOrthoZoom, GetMinimumOrthoZoom(), MAX_ORTHOZOOM);
				ViewTransform.SetOrthoZoom(NewOrthoZoom);
			}
		}
	}

	// Tell the viewport to redraw itself.
	Invalidate();
}

void FEditorViewportClient::SetClipToSelectionOnFocus(bool bInClipToSelection)
{
	bClipToSelectionOnFocus = bInClipToSelection;
}

void FEditorViewportClient::CenterViewportAtPoint(const FVector& NewLookAt, bool bInstant /* = false */)
{
	const bool bEnable = false;
	ToggleOrbitCamera(bEnable);

	FViewportCameraTransform& ViewTransform = GetViewTransform();
	FQuat Rotation(ViewTransform.GetRotation());
	FVector LookatVec = ViewTransform.GetLookAt() - ViewTransform.GetLocation();
	// project current lookat vector onto forward vector to get lookat distance, new position is that far along forward vector
	double LookatDist = FVector::DotProduct(Rotation.GetForwardVector(), LookatVec);
	FVector NewLocation = NewLookAt - LookatDist * Rotation.GetForwardVector();

	// ortho and perspective are treated the same here
	ViewTransform.SetLookAt(NewLookAt);
	ViewTransform.TransitionToLocation(NewLocation, EditorViewportWidget, bInstant);

	// Tell the viewport to redraw itself.
	Invalidate();
}


//////////////////////////////////////////////////////////////////////////
//
// Configures the specified FSceneView object with the view and projection matrices for this viewport.
FSceneView* FEditorViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
    const bool bStereoRendering = StereoViewIndex != INDEX_NONE;

	FSceneViewInitOptions ViewInitOptions;

	FViewportCameraTransform& ViewTransform = GetViewTransform();
	const ELevelViewportType EffectiveViewportType = GetViewportType();

	// Apply view modifiers.
	FEditorViewportViewModifierParams ViewModifierParams;
	{
		ViewModifierParams.ViewportClient = this;
		ViewModifierParams.ViewInfo.Location = ViewTransform.GetLocation();
		ViewModifierParams.ViewInfo.Rotation = ViewTransform.GetRotation();

		if (bUseControllingActorViewInfo)
		{
			ViewModifierParams.ViewInfo.FOV = ControllingActorViewInfo.FOV;
		}
		else
		{
			ViewModifierParams.ViewInfo.FOV = ViewFOV;
		}

		if (bShouldApplyViewModifiers)
		{
			ViewModifiers.Broadcast(ViewModifierParams);
		}
	}
	const FVector ModifiedViewLocation = ViewModifierParams.ViewInfo.Location;
	FRotator ModifiedViewRotation = ViewModifierParams.ViewInfo.Rotation;
	const float ModifiedViewFOV = ViewModifierParams.ViewInfo.FOV;
	if (bUseControllingActorViewInfo)
	{
		ControllingActorViewInfo.Location = ViewModifierParams.ViewInfo.Location;
		ControllingActorViewInfo.Rotation = ViewModifierParams.ViewInfo.Rotation;
		ControllingActorViewInfo.FOV = ViewModifierParams.ViewInfo.FOV;
	}

	ViewInitOptions.ViewOrigin = ModifiedViewLocation;

	// Apply head tracking!  Note that this won't affect what the editor *thinks* the view location and rotation is, it will
	// only affect the rendering of the scene.
	if( bStereoRendering && GEngine->XRSystem.IsValid() && GEngine->XRSystem->IsHeadTrackingAllowed() )
	{
		FQuat CurrentHmdOrientation;
		FVector CurrentHmdPosition;
		GEngine->XRSystem->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, CurrentHmdOrientation, CurrentHmdPosition );

		const FQuat VisualRotation = ModifiedViewRotation.Quaternion() * CurrentHmdOrientation;
		ModifiedViewRotation = VisualRotation.Rotator();
		ModifiedViewRotation.Normalize();
	}

	FIntPoint ViewportSize = Viewport->GetSizeXY();
	ViewportSize.X = FMath::Max(ViewportSize.X, 1);
	ViewportSize.Y = FMath::Max(ViewportSize.Y, 1);
	FIntPoint ViewportOffset(0, 0);

	// We expect some size to avoid problems with the view rect manipulation
	if (ViewportSize.X > 50 && ViewportSize.Y > 50)
	{
		int32 Value = CVarEditorViewportTest.GetValueOnGameThread();

		if (Value)
		{
			int InsetX = ViewportSize.X / 4;
			int InsetY = ViewportSize.Y / 4;

			// this allows to test various typical view port situations
			switch (Value)
			{
			case 1: ViewportOffset.X += InsetX; ViewportOffset.Y += InsetY; ViewportSize.X -= InsetX * 2; ViewportSize.Y -= InsetY * 2; break;
			case 2: ViewportOffset.Y += InsetY; ViewportSize.Y -= InsetY * 2; break;
			case 3: ViewportOffset.X += InsetX; ViewportSize.X -= InsetX * 2; break;
			case 4: ViewportSize.X /= 2; ViewportSize.Y /= 2; break;
			case 5: ViewportSize.X /= 2; ViewportSize.Y /= 2; ViewportOffset.X += ViewportSize.X;	break;
			case 6: ViewportSize.X /= 2; ViewportSize.Y /= 2; ViewportOffset.Y += ViewportSize.Y; break;
			case 7: ViewportSize.X /= 2; ViewportSize.Y /= 2; ViewportOffset.X += ViewportSize.X; ViewportOffset.Y += ViewportSize.Y; break;
			}
		}
	}

	ViewInitOptions.SetViewRectangle(FIntRect(ViewportOffset, ViewportOffset + ViewportSize));

	// no matter how we are drawn (forced or otherwise), reset our time here
	TimeForForceRedraw = 0.0;

	bool bConstrainAspectRatio = bUseControllingActorViewInfo && ControllingActorViewInfo.bConstrainAspectRatio;
	if (GEditor != nullptr && GEditor->IsPreviewPlatformActive() && GEditor->GetAllowConstrainedAspectRatioInPreview())
	{
		float PlatformAspectRatio = GEditor->PreviewPlatform.GetConstrainedAspectRatio();
		if (GEditor->GetFlipConstrainedAspectRatioInPreview() && PlatformAspectRatio != 0.f)
		{
			PlatformAspectRatio = 1 / PlatformAspectRatio;
		}
		FSlateApplication::Get().OnConstrainedAspectRatioChanged.Broadcast(PlatformAspectRatio);
	}
	else
	{
		FSlateApplication::Get().OnConstrainedAspectRatioChanged.Broadcast(0.f);
	}

	EAspectRatioAxisConstraint AspectRatioAxisConstraint = GetDefault<ULevelEditorViewportSettings>()->AspectRatioAxisConstraint;
	if (bUseControllingActorViewInfo && ControllingActorAspectRatioAxisConstraint.IsSet())
	{
		AspectRatioAxisConstraint = ControllingActorAspectRatioAxisConstraint.GetValue();
	}

	AWorldSettings* WorldSettings = nullptr;
	if( GetScene() != nullptr && GetScene()->GetWorld() != nullptr )
	{
		WorldSettings = GetScene()->GetWorld()->GetWorldSettings();
	}
	if( WorldSettings != nullptr )
	{
		ViewInitOptions.WorldToMetersScale = WorldSettings->WorldToMeters;
	}

	if (bUseControllingActorViewInfo)
	{
		// @todo vreditor: Not stereo friendly yet
		ViewInitOptions.ViewRotationMatrix = CalcViewRotationMatrixForControllingActorView(ModifiedViewRotation) * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));

		FMinimalViewInfo::CalculateProjectionMatrixGivenView(ControllingActorViewInfo, AspectRatioAxisConstraint, Viewport, /*inout*/ ViewInitOptions);
	}
	else
	{
		//
		if (EffectiveViewportType == LVT_Perspective)
		{
		    // If stereo rendering is enabled, update the size and offset appropriately for this pass
		    // @todo vreditor: Also need to update certain other use cases of ViewFOV like culling, streaming, etc.  (needs accessor)
		    if( bStereoRendering )
		    {
		        int32 X = 0;
		        int32 Y = 0;
		        uint32 SizeX = ViewportSize.X;
		        uint32 SizeY = ViewportSize.Y;
			    GEngine->StereoRenderingDevice->AdjustViewRect( StereoViewIndex, X, Y, SizeX, SizeY );
		        const FIntRect StereoViewRect = FIntRect( X, Y, X + SizeX, Y + SizeY );
		        ViewInitOptions.SetViewRectangle( StereoViewRect );

				GEngine->StereoRenderingDevice->CalculateStereoViewOffset( StereoViewIndex, ModifiedViewRotation, ViewInitOptions.WorldToMetersScale, ViewInitOptions.ViewOrigin );
			}

			// Calc view rotation matrix
			ViewInitOptions.ViewRotationMatrix = CalcViewRotationMatrix(ModifiedViewRotation);

		    // Rotate view 90 degrees
			ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
				FPlane(0, 0, 1, 0),
				FPlane(1, 0, 0, 0),
				FPlane(0, 1, 0, 0),
				FPlane(0, 0, 0, 1));

		    if( bStereoRendering )
		    {
			    // @todo vreditor: bConstrainAspectRatio is ignored in this path, as it is in the game client as well currently
			    // Let the stereoscopic rendering device handle creating its own projection matrix, as needed
			    ViewInitOptions.ProjectionMatrix = GEngine->StereoRenderingDevice->GetStereoProjectionMatrix(StereoViewIndex);
				ViewInitOptions.StereoPass = GEngine->StereoRenderingDevice->GetViewPassForIndex(bStereoRendering, StereoViewIndex);
		    }
		    else
		    {
			    const float MinZ = GetNearClipPlane();
			    const float MaxZ = MinZ;

			    if (bConstrainAspectRatio)
			    {
					// Avoid zero ViewFOV's which cause divide by zero's in projection matrix
					const float MatrixFOV = FMath::Max(0.001f, ModifiedViewFOV) * (float)PI / 360.0f;

					ViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
						MatrixFOV,
						MatrixFOV,
						1.0f,
						AspectRatio,
						MinZ,
						MaxZ
						);
			    }
			    else
			    {
				    float XAxisMultiplier;
				    float YAxisMultiplier;

					const bool bMaintainXFOV = (((ViewportSize.X > ViewportSize.Y) && (AspectRatioAxisConstraint == AspectRatio_MajorAxisFOV)) || (AspectRatioAxisConstraint == AspectRatio_MaintainXFOV));

				    if (bMaintainXFOV)
				    {
					    //if the viewport is wider than it is tall
					    XAxisMultiplier = 1.0f;
					    YAxisMultiplier = ViewportSize.X / (float)ViewportSize.Y;
				    }
				    else
				    {
					    //if the viewport is taller than it is wide
					    XAxisMultiplier = ViewportSize.Y / (float)ViewportSize.X;
					    YAxisMultiplier = 1.0f;
				    }

					// Here we do something similar to FMinimalViewInfo::CalculateProjectionMatrixGivenViewRectangle
					// TODO: unify both codebases
					float MatrixFOV;
					if (!bMaintainXFOV && AspectRatio != 0.f) // TODO: read CVarUseLegacyMaintainYFOV
					{
						const float HalfXFOV = FMath::DegreesToRadians(FMath::Max(0.001f, ModifiedViewFOV) / 2.f);
						const float HalfYFOV = FMath::Atan(FMath::Tan(HalfXFOV) / AspectRatio);
						MatrixFOV = HalfYFOV;
					}
					else
					{
						MatrixFOV = FMath::Max(0.001f, ModifiedViewFOV) * (float)UE_PI / 360.0f;
					}

					ViewInitOptions.ProjectionMatrix = FReversedZPerspectiveMatrix(
						MatrixFOV,
						MatrixFOV,
						XAxisMultiplier,
						YAxisMultiplier,
						MinZ,
						MaxZ
						);
			    }
			}
		}
		else
		{
			//The divisor for the matrix needs to match the translation code.
			const float Zoom = GetOrthoUnitsPerPixel(Viewport);
			float OrthoWidth = FMath::Clamp(Zoom * ViewportSize.X/2.0f, 0.0f, UE_LARGE_HALF_WORLD_MAX);
			float OrthoHeight = FMath::Clamp(Zoom * ViewportSize.Y/2.0f, 0.0f, UE_LARGE_HALF_WORLD_MAX);


			if (EffectiveViewportType == LVT_OrthoXY)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(0, -1, 0, 0),
					FPlane(-1, 0, 0, 0),
					FPlane(0, 0, -1, 0),
					FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoXZ)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(1, 0, 0, 0),
					FPlane(0, 0, -1, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoYZ)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(0, 0, 1, 0),
					FPlane(1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoNegativeXY)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
						FPlane(0, -1, 0, 0),
						FPlane(1, 0, 0, 0),
						FPlane(0, 0, 1, 0),
						FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoNegativeXZ)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(-1, 0, 0, 0),
					FPlane(0, 0, 1, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoNegativeYZ)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(0, 0, -1, 0),
					FPlane(-1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
			}
			else if (EffectiveViewportType == LVT_OrthoFreelook)
			{
				ViewInitOptions.ViewRotationMatrix = FMatrix(
					FPlane(0, 0, 1, 0),
					FPlane(1, 0, 0, 0),
					FPlane(0, 1, 0, 0),
					FPlane(0, 0, 0, 1));
			}
			else
			{
				// Unknown viewport type
				check(false);
			}

			FMatrix::FReal ZScale = 0.5f / UE_OLD_WORLD_MAX;
			FMatrix::FReal ZOffset = UE_OLD_WORLD_MAX;
			
			//Only lit mode render passes require the restricted depth, so wireframe and unlit can just use the standard path
			bool bShouldCalculateDepthRange = ViewFamily->ViewMode > VMI_Unlit;

			FMinimalViewInfo CalculatePlanesViewInfo;
			CalculatePlanesViewInfo.Rotation = ViewInitOptions.ViewRotationMatrix.Rotator();
			CalculatePlanesViewInfo.AspectRatio = AspectRatio;
			CalculatePlanesViewInfo.bConstrainAspectRatio = false;
			CalculatePlanesViewInfo.ProjectionMode = ECameraProjectionMode::Orthographic;
			CalculatePlanesViewInfo.OrthoWidth = OrthoWidth;
			CalculatePlanesViewInfo.bAutoCalculateOrthoPlanes = bShouldCalculateDepthRange;
			CalculatePlanesViewInfo.AutoPlaneShift = 0.0f;
			CalculatePlanesViewInfo.bUpdateOrthoPlanes = bShouldCalculateDepthRange;

			CalculatePlanesViewInfo.bUseCameraHeightAsViewTarget = false;
			CalculatePlanesViewInfo.OrthoNearClipPlane = -UE_OLD_WORLD_MAX;
			CalculatePlanesViewInfo.OrthoFarClipPlane = UE_OLD_WORLD_MAX;

			SDepthBar::FDepthSpace DepthSpace(GetViewLocation(), GetForwardVector(), GetWorldBounds());

			if (OrthographicNearPlane.IsSet())
			{
				CalculatePlanesViewInfo.OrthoNearClipPlane = OrthographicNearPlane.GetValue();
			}
			else if (!bShouldCalculateDepthRange)
			{
				CalculatePlanesViewInfo.OrthoNearClipPlane = DepthSpace.GetAlignedMin();
			}

			if (OrthographicFarPlane.IsSet())
			{
				CalculatePlanesViewInfo.OrthoFarClipPlane = OrthographicFarPlane.GetValue();
			}
			else if (!bShouldCalculateDepthRange)
			{
				CalculatePlanesViewInfo.OrthoFarClipPlane = DepthSpace.GetAlignedMax();
			}

			ViewInitOptions.UpdateOrthoPlanes(CalculatePlanesViewInfo);
			const float Denominator = CalculatePlanesViewInfo.OrthoFarClipPlane - CalculatePlanesViewInfo.OrthoNearClipPlane;
			if (Denominator > 0.0f)
			{
				ZScale = 1.0f / Denominator;
				ZOffset = -CalculatePlanesViewInfo.OrthoNearClipPlane;
			}
			
			ViewInitOptions.ProjectionMatrix = FReversedZOrthoMatrix(
				OrthoWidth,
				OrthoHeight,
				ZScale,
				ZOffset
			);

			/**
			* Unmanaged fog obscures the Ortho lit view modes so we disable it for these views unless the user overrides the setting.
			*/
			if (CVarEditorDisableFogInOrthoDebugViews->GetBool())
			{
				ViewFamily->EngineShowFlags.SetFog(false);
			}
		};

		if (bConstrainAspectRatio)
		{
			ViewInitOptions.SetConstrainedViewRectangle(Viewport->CalculateViewExtents(AspectRatio, ViewInitOptions.GetViewRect()));
		}
	}

	if (!ViewInitOptions.IsValidViewRectangle())
	{
		// Zero sized rects are invalid, so fake to 1x1 to avoid asserts later on
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, 1, 1));
	}

	// Allocate our stereo view state on demand, so that only viewports that actually use stereo features have one
	const int32 ViewStateIndex = (StereoViewIndex != INDEX_NONE) ? StereoViewIndex : 0;
	if (bStereoRendering)
	{
		if (StereoViewStates.Num() <= ViewStateIndex)
		{
			StereoViewStates.SetNum(ViewStateIndex + 1);
		}

		if (StereoViewStates[ViewStateIndex].GetReference() == nullptr)
		{
			FSceneInterface* Scene = GetScene();
			StereoViewStates[ViewStateIndex].Allocate(Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel);
		}
	}

	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SceneViewStateInterface = ( (ViewStateIndex == 0) ? ViewState.GetReference() : StereoViewStates[ViewStateIndex].GetReference() );
	ViewInitOptions.StereoViewIndex = StereoViewIndex;

	if (CVarEditorViewGPUMirrorTest.GetValueOnGameThread())
	{
		static TPimplPtr<FSceneViewStateSystemMemoryMirror> Mirror;
		if (!Mirror)
		{
			Mirror = FSceneViewStateInterface::SystemMemoryMirrorAllocate();
		}
		ViewInitOptions.SceneViewStateInterface->SystemMemoryMirrorBackup(Mirror.Get());
		ViewInitOptions.SceneViewStateInterface->SystemMemoryMirrorRestore(Mirror.Get());
	}

	ViewInitOptions.ViewElementDrawer = this;

	ViewInitOptions.BackgroundColor = GetBackgroundColor();

	ViewInitOptions.EditorViewBitflag = (uint64)1 << ViewIndex, // send the bit for this view - each actor will check it's visibility bits against this

	ViewInitOptions.FOV = ModifiedViewFOV;
	if (bUseControllingActorViewInfo)
	{
		ViewInitOptions.bUseFieldOfViewForLOD = ControllingActorViewInfo.bUseFieldOfViewForLOD;
		ViewInitOptions.FOV = ControllingActorViewInfo.FOV;
		ViewInitOptions.FirstPersonParams = FFirstPersonParameters(ControllingActorViewInfo.CalculateFirstPersonFOVCorrectionFactor(), ControllingActorViewInfo.FirstPersonScale, ControllingActorViewInfo.bUseFirstPersonParameters);
		ViewInitOptions.OverscanResolutionFraction = ControllingActorViewInfo.OverscanResolutionFraction;
		ViewInitOptions.CropFraction = ControllingActorViewInfo.CropFraction;
		ViewInitOptions.AsymmetricCropFraction = ControllingActorViewInfo.AsymmetricCropFraction;
	}

	ViewInitOptions.OverrideFarClippingPlaneDistance = FarPlane;
	ViewInitOptions.CursorPos = CurrentMousePos;

#if !UE_BUILD_SHIPPING
	{
		static const auto CVarVSync = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Test.ConstrainedView"));
		int32 Value = CVarVSync->GetValueOnGameThread();

		if (Value)
		{
			const FIntRect& ViewRect = ViewInitOptions.GetViewRect();
			FIntRect ConstrainedViewRect = ViewInitOptions.GetConstrainedViewRect();

			int InsetX = ConstrainedViewRect.Width() / 4;
			int InsetY = ConstrainedViewRect.Height() / 4;

			// this allows to test various typical view port situations
			switch (Value)
			{
			case 1:
				ConstrainedViewRect.Min.X += InsetX;
				ConstrainedViewRect.Min.Y += InsetY;
				ConstrainedViewRect.Max.X -= InsetX;
				ConstrainedViewRect.Max.Y -= InsetY;
				break;

			case 2:
				ConstrainedViewRect.Min.Y += InsetY;
				ConstrainedViewRect.Max.Y -= InsetY;
				break;

			case 3:
				ConstrainedViewRect.Min.X += InsetX;
				ConstrainedViewRect.Max.X -= InsetX;
				break;

			case 4:
				ConstrainedViewRect.Max.X -= 2 * InsetX;
				ConstrainedViewRect.Max.Y -= 2 * InsetY;
				break;

			case 5:
				ConstrainedViewRect.Min.X += 2 * InsetX;
				ConstrainedViewRect.Max.Y -= 2 * InsetY;
				break;

			case 6:
				ConstrainedViewRect.Max.X -= 2 * InsetX;
				ConstrainedViewRect.Min.Y += 2 * InsetY;
				break;

			case 7:
				ConstrainedViewRect.Min.X += 2 * InsetX;
				ConstrainedViewRect.Min.Y += 2 * InsetY;
				break;
			}

			ViewInitOptions.SetConstrainedViewRectangle(ConstrainedViewRect);
		}
	}
#endif

	FSceneView* View = new FSceneView(ViewInitOptions);

	View->ViewLocation = ModifiedViewLocation;
	View->ViewRotation = ModifiedViewRotation;

	View->SubduedSelectionOutlineColor = GEngine->GetSubduedSelectionOutlineColor();
	const UEditorStyleSettings* EditorStyle = GetDefault<UEditorStyleSettings>();
	check(View->AdditionalSelectionOutlineColors.Num() <= UE_ARRAY_COUNT(EditorStyle->AdditionalSelectionColors))
	for (int OutlineColorIndex = 0; OutlineColorIndex < View->AdditionalSelectionOutlineColors.Num(); ++OutlineColorIndex)
	{
		View->AdditionalSelectionOutlineColors[OutlineColorIndex] = EditorStyle->AdditionalSelectionColors[OutlineColorIndex];
	}
	
	View->CompositePrimitiveParams.OccludedDitheringPatternSize = EditorStyle->OccludedDitheringPatternSize;
	View->CompositePrimitiveParams.OccludedDithering = EditorStyle->OccludedDithering;
	View->CompositePrimitiveParams.OccludedBrightness = EditorStyle->OccludedBrightness;

	int32 FamilyIndex = ViewFamily->Views.Add(View);
	check(FamilyIndex == View->StereoViewIndex || View->StereoViewIndex == INDEX_NONE);

	View->StartFinalPostprocessSettings( View->ViewLocation );

	if (bUseControllingActorViewInfo)
	{
		// Pass on the previous view transform of the controlling actor to the view
		View->PreviousViewTransform = ControllingActorViewInfo.PreviousViewTransform;
		View->OverridePostProcessSettings(ControllingActorViewInfo.PostProcessSettings, ControllingActorViewInfo.PostProcessBlendWeight);

		for (int32 ExtraPPBlendIdx = 0; ExtraPPBlendIdx < ControllingActorExtraPostProcessBlends.Num(); ++ExtraPPBlendIdx)
		{
			FPostProcessSettings const& PPSettings = ControllingActorExtraPostProcessBlends[ExtraPPBlendIdx];
			float const Weight = ControllingActorExtraPostProcessBlendWeights[ExtraPPBlendIdx];
			View->OverridePostProcessSettings(PPSettings, Weight);
		}
	}
	else
	{
		OverridePostProcessSettings(*View);
	}

	if (ViewModifierParams.ViewInfo.PostProcessBlendWeight > 0.f)
	{
		View->OverridePostProcessSettings(ViewModifierParams.ViewInfo.PostProcessSettings, ViewModifierParams.ViewInfo.PostProcessBlendWeight);
	}
	const int32 PPNum = FMath::Min(ViewModifierParams.PostProcessSettings.Num(), ViewModifierParams.PostProcessBlendWeights.Num());
	for (int32 PPIndex = 0; PPIndex < PPNum; ++PPIndex)
	{
		const FPostProcessSettings& PPSettings = ViewModifierParams.PostProcessSettings[PPIndex];
		const float PPWeight = ViewModifierParams.PostProcessBlendWeights[PPIndex];
		View->OverridePostProcessSettings(PPSettings, PPWeight);
	}

	View->EndFinalPostprocessSettings(ViewInitOptions);

	if (RenderViewportFeedback && (StereoViewIndex == INDEX_NONE || StereoViewIndex == 0))
	{
		View->ViewportFeedback = RenderViewportFeedback->MakeCollector();
	}

	for (int ViewExt = 0; ViewExt < ViewFamily->ViewExtensions.Num(); ViewExt++)
	{
		ViewFamily->ViewExtensions[ViewExt]->SetupView(*ViewFamily, *View);
	}

	return View;
}

void FEditorViewportClient::ReceivedFocus(FViewport* InViewport)
{
	// Viewport has changed got to reset the cursor as it could of been left in any state
	UpdateRequiredCursorVisibility();
	ApplyRequiredCursorVisibility( true );

	// Force a cursor update to make sure its returned to default as it could of been left in any state and wont update itself till an action is taken
	SetRequiredCursorOverride(false, EMouseCursor::Default);
	FSlateApplication::Get().QueryCursor();

	ModeTools->ReceivedFocus(this, Viewport);
}

void FEditorViewportClient::LostFocus(FViewport* InViewport)
{
	StopTracking();
	ModeTools->LostFocus(this, Viewport);
}

void FEditorViewportClient::Tick(float DeltaTime)
{
	SCOPED_NAMED_EVENT(FEditorViewportClient_Tick, FColor::Red);
	
	if (!CachedWorldBounds.IsValid)
	{
		if (UWorld* World = GetWorld(); World && !World->IsGameWorld())
		{
			if (UWorldPartition* WorldPartition = World->GetWorldPartition())
			{
				CachedWorldBounds = WorldPartition->GetEditorWorldBounds();
			}
			else
			{
				CachedWorldBounds = FBox(ForceInitToZero);
				
				for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); ++LevelIndex)
				{
					if (ULevel* Level = World->GetLevel(LevelIndex))
					{
						if (ALevelBounds* LevelBounds = Level->LevelBoundsActor.Get())
						{
							CachedWorldBounds += LevelBounds->GetComponentsBoundingBox();
						}
						else
						{
							CachedWorldBounds += ALevelBounds::CalculateLevelBounds(Level);
						}
					}
				}
			}
		}
	}
	
	ConditionalCheckHoveredHitProxy();

	FViewportCameraTransform& ViewTransform = GetViewTransform();
	const bool bIsAnimating = ViewTransform.UpdateTransition();
	if (bIsAnimating && GetViewportType() == LVT_Perspective)
	{
		PerspectiveCameraMoved();
	}

	if ( bIsTracking )
	{
		FEditorViewportStats::BeginFrame();
	}

	if( !bIsAnimating )
	{
		bIsCameraMovingOnTick = bIsCameraMoving;

		// Update any real-time camera movement
		UpdateCameraMovement( DeltaTime );

		if (ModeTools && ModeTools->UsesViewportITF())
		{
			ConsumeExternalCameraMovement();
		}
		else
		{
			UpdateMouseDelta();
		}
		
		UpdateGestureDelta();

		EndCameraMovement();
	}

	const bool bStereoRendering = GEngine->XRSystem.IsValid() && GEngine->IsStereoscopic3D( Viewport );
	if( bStereoRendering )
	{
		// Every frame, we'll push our camera position to the HMD device, so that it can properly compute a head-relative offset for each eye
		if( GEngine->XRSystem->IsHeadTrackingAllowed() )
		{
			auto XRCamera = GEngine->XRSystem->GetXRCamera();
			if (XRCamera.IsValid())
		{
			FQuat PlayerOrientation = GetViewRotation().Quaternion();
			FVector PlayerLocation = GetViewLocation();
				XRCamera->UseImplicitHMDPosition(false);
				XRCamera->UpdatePlayerCamera(PlayerOrientation, PlayerLocation, DeltaTime);
			}
		}
	}

	if ( bIsTracking )
	{
		// If a mouse button or modifier is pressed we want to assume the user is still in a mode
		// they haven't left to perform a non-action in the frame to keep the last used operation
		// from being reset.
		const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) ? true : false;
		const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton) ? true : false;
		const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton) ? true : false;
		const bool bMouseButtonDown = ( LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown );

		const bool AltDown = IsAltPressed();
		const bool ShiftDown = IsShiftPressed();
		const bool ControlDown = IsCtrlPressed();
		const bool bModifierDown = AltDown || ShiftDown || ControlDown;
		if ( bMouseButtonDown || bModifierDown )
		{
			FEditorViewportStats::NoOpUsing();
		}

		FEditorViewportStats::EndFrame();
	}

	// refresh ourselves if animating or told to from another view
	if ( bIsAnimating || ( TimeForForceRedraw != 0.0 && FPlatformTime::Seconds() > TimeForForceRedraw ) )
	{
		Invalidate();
	}

	// Update the fade out animation
	if (MovingPreviewLightTimer > 0.0f)
	{
		MovingPreviewLightTimer = FMath::Max(MovingPreviewLightTimer - DeltaTime, 0.0f);

		if (MovingPreviewLightTimer == 0.0f)
		{
			Invalidate();
		}
	}

	// Invalidate the viewport widget if pending
	if (bShouldInvalidateViewportWidget)
	{
		InvalidateViewportWidget();
	}

	// Tick the editor modes
	ModeTools->Tick(this, DeltaTime);
}

namespace ViewportDeadZoneConstants
{
	enum
	{
		NO_DEAD_ZONE,
		STANDARD_DEAD_ZONE
	};
};

float GetFilteredDelta(const float DefaultDelta, const uint32 DeadZoneType, const float StandardDeadZoneSize)
{
	if (DeadZoneType == ViewportDeadZoneConstants::NO_DEAD_ZONE)
	{
		return DefaultDelta;
	}
	else
	{
		//can't be one or normalizing won't work
		check(FMath::IsWithin<float>(StandardDeadZoneSize, 0.0f, 1.0f));
		//standard dead zone
		float ClampedAbsValue = FMath::Clamp(FMath::Abs(DefaultDelta), StandardDeadZoneSize, 1.0f);
		float NormalizedClampedAbsValue = (ClampedAbsValue - StandardDeadZoneSize)/(1.0f-StandardDeadZoneSize);
		float ClampedSignedValue = (DefaultDelta >= 0.0f) ? NormalizedClampedAbsValue : -NormalizedClampedAbsValue;
		return ClampedSignedValue;
	}
}


/**Applies Joystick axis control to camera movement*/
void FEditorViewportClient::UpdateCameraMovementFromJoystick(const bool bRelativeMovement, FCameraControllerConfig& InConfig)
{
	for(TMap<int32,FCachedJoystickState*>::TConstIterator JoystickIt(JoystickStateMap);JoystickIt;++JoystickIt)
	{
		FCachedJoystickState* JoystickState = JoystickIt.Value();
		check(JoystickState);
		for(TMap<FKey,float>::TConstIterator AxisIt(JoystickState->AxisDeltaValues);AxisIt;++AxisIt)
		{
			FKey Key = AxisIt.Key();
			float UnfilteredDelta = AxisIt.Value();
			const float StandardDeadZone = CameraController->GetConfig().ImpulseDeadZoneAmount;

			if (bRelativeMovement)
			{
				//XBOX Controller
				if (Key == EKeys::Gamepad_LeftX)
				{
					CameraUserImpulseData->MoveRightLeftImpulse += GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.TranslationMultiplier;
				}
				else if (Key == EKeys::Gamepad_LeftY)
				{
					CameraUserImpulseData->MoveForwardBackwardImpulse += GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.TranslationMultiplier;
				}
				else if (Key == EKeys::Gamepad_RightX)
				{
					float DeltaYawImpulse = GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.RotationMultiplier * (InConfig.bInvertX ? -1.0f : 1.0f);
					CameraUserImpulseData->RotateYawImpulse += DeltaYawImpulse;
					InConfig.bForceRotationalPhysics |= (DeltaYawImpulse != 0.0f);
				}
				else if (Key == EKeys::Gamepad_RightY)
				{
					float DeltaPitchImpulse = GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.RotationMultiplier * (InConfig.bInvertY ? -1.0f : 1.0f);
					CameraUserImpulseData->RotatePitchImpulse -= DeltaPitchImpulse;
					InConfig.bForceRotationalPhysics |= (DeltaPitchImpulse != 0.0f);
				}
				else if (Key == EKeys::Gamepad_LeftTriggerAxis)
				{
					CameraUserImpulseData->MoveWorldUpDownImpulse -= GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.TranslationMultiplier;
				}
				else if (Key == EKeys::Gamepad_RightTriggerAxis)
				{
					CameraUserImpulseData->MoveWorldUpDownImpulse += GetFilteredDelta(UnfilteredDelta, ViewportDeadZoneConstants::STANDARD_DEAD_ZONE, StandardDeadZone) * InConfig.TranslationMultiplier;
				}
			}
		}

		if (bRelativeMovement)
		{
			for(TMap<FKey,EInputEvent>::TConstIterator KeyIt(JoystickState->KeyEventValues);KeyIt;++KeyIt)
			{
				FKey Key = KeyIt.Key();
				EInputEvent KeyState = KeyIt.Value();

				const bool bPressed = (KeyState==IE_Pressed);
				const bool bRepeat = (KeyState == IE_Repeat);

				static const float MultiplierIncrement = 0.25f;
				static const float MaxTranslationMultiplier = 5.0f;
				static const float MaxRotationMultiplier = 3.0f;

				if ((Key == EKeys::Gamepad_LeftShoulder) && (bPressed || bRepeat))
				{
					CameraUserImpulseData->ZoomOutInImpulse +=  InConfig.ZoomMultiplier;
				}
				else if ((Key == EKeys::Gamepad_RightShoulder) && (bPressed || bRepeat))
				{
					CameraUserImpulseData->ZoomOutInImpulse -= InConfig.ZoomMultiplier;
				}
				else if ((Key == EKeys::Gamepad_DPad_Up) && (bPressed && !bRepeat))
				{
					InConfig.TranslationMultiplier = FMath::Clamp(InConfig.TranslationMultiplier + MultiplierIncrement, MultiplierIncrement, MaxTranslationMultiplier);
				}
				else if ((Key == EKeys::Gamepad_DPad_Down) && (bPressed && !bRepeat))
				{
					InConfig.TranslationMultiplier = FMath::Clamp(InConfig.TranslationMultiplier - MultiplierIncrement, MultiplierIncrement, MaxTranslationMultiplier);
				}
				else if ((Key == EKeys::Gamepad_DPad_Right) && (bPressed && !bRepeat))
				{
					InConfig.RotationMultiplier = FMath::Clamp(InConfig.RotationMultiplier + MultiplierIncrement, MultiplierIncrement, MaxRotationMultiplier);
				}
				else if ((Key == EKeys::Gamepad_DPad_Left) && (bPressed && !bRepeat))
				{
					InConfig.RotationMultiplier = FMath::Clamp(InConfig.RotationMultiplier - MultiplierIncrement, MultiplierIncrement, MaxRotationMultiplier);
				}
				
				if (bPressed)
				{
					//instantly set to repeat to stock rapid flickering until the time out
					JoystickState->KeyEventValues.Add(Key, IE_Repeat);
				}
			}
		}
	}
}

EMouseCursor::Type FEditorViewportClient::GetCursor(FViewport* InViewport,int32 X,int32 Y)
{
	EMouseCursor::Type MouseCursor = EMouseCursor::Default;

	// StaticFindObject is used lower down in this code, and that's not allowed while saving packages.
	if ( UE::IsSavingPackage() )
	{
		return MouseCursor;
	}

	bool bMoveCanvasMovement = !ModeTools->UsesViewportITF() && ShouldUseMoveCanvasMovement();

	if (RequiredCursorVisibiltyAndAppearance.bOverrideAppearance &&
		RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible)
	{
		MouseCursor = RequiredCursorVisibiltyAndAppearance.RequiredCursor;
	}
	else if( MouseDeltaTracker->UsingDragTool() )
	{
		MouseCursor = EMouseCursor::Default;
	}
	else if (!RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible)
	{
		MouseCursor = EMouseCursor::None;
	}
	//only camera movement gets the hand icon
	else if (bMoveCanvasMovement && (Widget->GetCurrentAxis() == EAxisList::None) && bHasMouseMovedSinceClick)
	{
		//We're grabbing the canvas so the icon should look "grippy"
		MouseCursor = EMouseCursor::GrabHandClosed;
	}
	else if (bMoveCanvasMovement &&
		bHasMouseMovedSinceClick &&
		(GetWidgetMode() == UE::Widget::WM_Translate || GetWidgetMode() == UE::Widget::WM_TranslateRotateZ || GetWidgetMode() == UE::Widget::WM_2D))
	{
		MouseCursor = EMouseCursor::CardinalCross;
	}
	//wyisyg mode
	else if (IsUsingAbsoluteTranslation(true) && bHasMouseMovedSinceClick)
	{
		MouseCursor = EMouseCursor::CardinalCross;
	}
	// Don't select widget axes by mouse over while they're being controlled by a mouse drag.
	else if (InViewport->IsCursorVisible() && !bWidgetAxisControlledByDrag)
	{
		// allow editor modes to override cursor
		EMouseCursor::Type EditorModeCursor = EMouseCursor::Default;
		if (ModeTools->GetCursor(EditorModeCursor))
		{
			MouseCursor = EditorModeCursor;
		}
		else if (!ModeTools->HasOngoingTransform())
		{
			HHitProxy* HitProxy = InViewport->GetHitProxy(X,Y);

			// Change the mouse cursor if the user is hovering over something they can interact with.
			if( HitProxy && !IsTracking() )
			{
				MouseCursor = HitProxy->GetMouseCursor();
				bShouldCheckHitProxy = true;
			}
			else
			{
				// Turn off widget highlight if there currently is one
				if( Widget->GetCurrentAxis() != EAxisList::None )
				{
					SetCurrentWidgetAxis( EAxisList::None );
					Invalidate( false, false );
				}
			}
		}
	}

	CachedMouseX = X;
	CachedMouseY = Y;

	return MouseCursor;
}

TOptional<TSharedRef<SWidget>> FEditorViewportClient::MapCursor(FViewport* InViewport, const FCursorReply& CursorReply)
{
	// allow editor modes to override cursor
	if (const TSharedPtr<SWidget> MappedCursor = ModeTools->MapCursor();
		MappedCursor.IsValid())
	{
		return MappedCursor.ToSharedRef();
	}
	
	return FCommonViewportClient::MapCursor(InViewport, CursorReply);
}

bool FEditorViewportClient::IsOrtho() const
{
	return !IsPerspective();
}

bool FEditorViewportClient::IsPerspective() const
{
	return (GetViewportType() == LVT_Perspective);
}

bool FEditorViewportClient::IsAspectRatioConstrained() const
{
	return bUseControllingActorViewInfo && ControllingActorViewInfo.bConstrainAspectRatio;
}

ELevelViewportType FEditorViewportClient::GetViewportType() const
{
	ELevelViewportType EffectiveViewportType = ViewportType;
	if (EffectiveViewportType == LVT_None)
	{
		EffectiveViewportType = LVT_Perspective;
	}
	if (bUseControllingActorViewInfo)
	{
		EffectiveViewportType = (ControllingActorViewInfo.ProjectionMode == ECameraProjectionMode::Perspective) ? LVT_Perspective : LVT_OrthoFreelook;
	}
	return EffectiveViewportType;
}

void FEditorViewportClient::SetViewportType( ELevelViewportType InViewportType )
{
	ViewportType = InViewportType;

	// Changing the type may also change the active view mode; re-apply that now
	ApplyViewMode(GetViewMode(), IsPerspective(), EngineShowFlags);

	// We might have changed to an orthographic viewport; if so, update any viewport links
	UpdateLinkedOrthoViewports(true);

	Invalidate();
}

void FEditorViewportClient::RotateViewportType()
{
	ViewportType = ViewOptions[ViewOptionIndex];

	// Changing the type may also change the active view mode; re-apply that now
	ApplyViewMode(GetViewMode(), IsPerspective(), EngineShowFlags);

	// We might have changed to an orthographic viewport; if so, update any viewport links
	UpdateLinkedOrthoViewports(true);

	Invalidate();

	if (ViewOptionIndex == 5)
	{
		ViewOptionIndex = 0;
	}
	else
	{
		ViewOptionIndex++;
	}
}

bool FEditorViewportClient::IsActiveViewportTypeInRotation() const
{
	return GetViewportType() == ViewOptions[ViewOptionIndex];
}

bool FEditorViewportClient::IsActiveViewportType(ELevelViewportType InViewportType) const
{
	return GetViewportType() == InViewportType;
}

// Updates real-time camera movement.  Should be called every viewport tick!
void FEditorViewportClient::UpdateCameraMovement( float DeltaTime )
{
	const bool bIsITFInteractions = ModeTools->UsesViewportITF();
	
	// We only want to move perspective cameras around like this
	if( Viewport && IsPerspective() && (bIsITFInteractions ? !IsViewportInteractionActive(ModeTools.Get(), UE::Editor::ViewportInteractions::Orbit) : !ShouldOrbitCamera()))
	{
		const bool bEnable = false;
		ToggleOrbitCamera(bEnable);

		const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

		// Certain keys are only available while the flight camera input mode is active
		const bool bUsingFlightInput = (bIsITFInteractions ? ViewportNavigationHelper->bIsMouseLooking : IsFlightCameraInputModeActive()) || bIsUsingTrackpad;

		// Apply impulse from magnify gesture and reset impulses if we're using WASD keys
		CameraUserImpulseData->MoveForwardBackwardImpulse = GestureMoveForwardBackwardImpulse;
		CameraUserImpulseData->MoveRightLeftImpulse = 0.0f;
		CameraUserImpulseData->MoveWorldUpDownImpulse = 0.0f;
		CameraUserImpulseData->MoveLocalUpDownImpulse = 0.0f;
		CameraUserImpulseData->ZoomOutInImpulse = 0.0f;
		CameraUserImpulseData->RotateYawImpulse = 0.0f;
		CameraUserImpulseData->RotatePitchImpulse = 0.0f;
		CameraUserImpulseData->RotateRollImpulse = 0.0f;

		GestureMoveForwardBackwardImpulse = 0.0f;
		
		if (bIsITFInteractions)
		{
			ViewportNavigationHelper->ConsumeImpulseData(*CameraUserImpulseData);
		}
		else
		{
			// Is the current press unmodified?
			const bool bUnmodifiedPress = !IsAltPressed() && !IsShiftPressed() && !IsCtrlPressed() && !IsCmdPressed();

			// Do we want to use the regular arrow keys for flight input?
			// Because the arrow keys are used for things like nudging actors, we'll only do this while the press is unmodified
			const bool bRemapArrowKeys = bUnmodifiedPress;

			// Do we want to remap the various WASD keys for flight input?
			const bool bRemapWASDKeys =
				(bUnmodifiedPress || (GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlExperimentalNavigation && IsShiftPressed())) &&
				(GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlType == WASD_Always ||
				( bUsingFlightInput &&
				( GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlType == WASD_RMBOnly && (Viewport->KeyState(EKeys::RightMouseButton ) ||Viewport->KeyState(EKeys::MiddleMouseButton) || Viewport->KeyState(EKeys::LeftMouseButton) || bIsUsingTrackpad ) ) ) ) &&
				!MouseDeltaTracker->UsingDragTool();
				
			bool bForwardKeyState = false;
			bool bBackwardKeyState = false;
			bool bRightKeyState = false;
			bool bLeftKeyState = false;

			bool bWorldUpKeyState = false;
			bool bWorldDownKeyState = false;
			bool bLocalUpKeyState = false;
			bool bLocalDownKeyState = false;
			bool bZoomOutKeyState = false;
			bool bZoomInKeyState = false;

			bool bRotateUpKeyState = false;
			bool bRotateDownKeyState = false;
			bool bRotateLeftKeyState = false;
			bool bRotateRightKeyState = false;
			// Iterate through all key mappings to generate key state flags
			for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
			{
				EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex> (i);
				bForwardKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Forward->GetActiveChord(ChordIndex)->Key);
				bBackwardKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Backward->GetActiveChord(ChordIndex)->Key);
				bRightKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Right->GetActiveChord(ChordIndex)->Key);
				bLeftKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().Left->GetActiveChord(ChordIndex)->Key);

				bWorldUpKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().WorldUp->GetActiveChord(ChordIndex)->Key);
				bWorldDownKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().WorldDown->GetActiveChord(ChordIndex)->Key);
				bLocalUpKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().LocalUp->GetActiveChord(ChordIndex)->Key);
				bLocalDownKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().LocalDown->GetActiveChord(ChordIndex)->Key);
				bZoomOutKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomOut->GetActiveChord(ChordIndex)->Key);
				bZoomInKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomIn->GetActiveChord(ChordIndex)->Key);

				bRotateUpKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().RotateUp->GetActiveChord(ChordIndex)->Key);
				bRotateDownKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().RotateDown->GetActiveChord(ChordIndex)->Key);
				bRotateLeftKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().RotateLeft->GetActiveChord(ChordIndex)->Key);
				bRotateRightKeyState |= Viewport->KeyState(FViewportNavigationCommands::Get().RotateRight->GetActiveChord(ChordIndex)->Key);
			}

			// Forward/back
			if( ( bRemapWASDKeys && bForwardKeyState ) ||
				( bRemapArrowKeys && Viewport->KeyState( EKeys::Up ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState(EKeys::NumPadEight) ) )
			{
				CameraUserImpulseData->MoveForwardBackwardImpulse += 1.0f;
			}
			if( (bRemapWASDKeys && bBackwardKeyState) ||
				( bRemapArrowKeys && Viewport->KeyState( EKeys::Down ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState( EKeys::NumPadTwo ) ) )
			{
				CameraUserImpulseData->MoveForwardBackwardImpulse -= 1.0f;
			}

			// Right/left
			if (( bRemapWASDKeys && bRightKeyState) ||
				( bRemapArrowKeys && Viewport->KeyState( EKeys::Right ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState( EKeys::NumPadSix ) ) )
			{
				CameraUserImpulseData->MoveRightLeftImpulse += 1.0f;
			}
			if( ( bRemapWASDKeys && bLeftKeyState) ||
				( bRemapArrowKeys && Viewport->KeyState( EKeys::Left ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState( EKeys::NumPadFour ) ) )
			{
				CameraUserImpulseData->MoveRightLeftImpulse -= 1.0f;
			}

			// World up/down
			if( ( bRemapWASDKeys && bWorldUpKeyState) ||
				( bUnmodifiedPress && Viewport->KeyState( EKeys::PageUp ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && ( Viewport->KeyState( EKeys::NumPadNine ) || Viewport->KeyState( EKeys::Add ) ) ) )
			{
				CameraUserImpulseData->MoveWorldUpDownImpulse += 1.0f;
			}
			if( ( bRemapWASDKeys && bWorldDownKeyState) ||
				( bUnmodifiedPress && Viewport->KeyState( EKeys::PageDown ) ) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && ( Viewport->KeyState( EKeys::NumPadSeven ) || Viewport->KeyState( EKeys::Subtract ) ) ) )
			{
				CameraUserImpulseData->MoveWorldUpDownImpulse -= 1.0f;
			}

			// Local up/down
			if( ( bRemapWASDKeys && bLocalUpKeyState) )
			{
				CameraUserImpulseData->MoveLocalUpDownImpulse += 1.0f;
			}
			if( ( bRemapWASDKeys && bLocalDownKeyState) )
			{
				CameraUserImpulseData->MoveLocalUpDownImpulse -= 1.0f;
			}

			// Zoom FOV out/in
			if( ( bRemapWASDKeys && bZoomOutKeyState) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState( EKeys::NumPadOne ) ) )
			{
				CameraUserImpulseData->ZoomOutInImpulse += 1.0f;
			}
			if( ( bRemapWASDKeys && bZoomInKeyState) ||
				( bUnmodifiedPress && bUseNumpadCameraControl && Viewport->KeyState( EKeys::NumPadThree ) ) )
			{
				CameraUserImpulseData->ZoomOutInImpulse -= 1.0f;
			}

			// Rotate up/down
			if (bRemapWASDKeys && bRotateUpKeyState)
			{
				CameraUserImpulseData->RotatePitchImpulse += 1.0f * CameraController->GetConfig().RotationMultiplier;
				CameraController->GetConfig().bForceRotationalPhysics = true;
			}
			if (bRemapWASDKeys && bRotateDownKeyState)
			{
				CameraUserImpulseData->RotatePitchImpulse -= 1.0f * CameraController->GetConfig().RotationMultiplier;
				CameraController->GetConfig().bForceRotationalPhysics = true;
			}

			// Rotate left/right
			if (bRemapWASDKeys && bRotateLeftKeyState)
			{
				CameraUserImpulseData->RotateYawImpulse -= 1.0f * CameraController->GetConfig().RotationMultiplier;
				CameraController->GetConfig().bForceRotationalPhysics = true;
			}
			if (bRemapWASDKeys && bRotateRightKeyState)
			{
				CameraUserImpulseData->RotateYawImpulse += 1.0f * CameraController->GetConfig().RotationMultiplier;
				CameraController->GetConfig().bForceRotationalPhysics = true;
			}
		}

		if (!CameraController->IsRotating())
		{
			CameraController->GetConfig().bForceRotationalPhysics = false;
		}
		
		if (CameraUserImpulseData->HasRotationalImpulse())
		{
			CameraController->GetConfig().bForceRotationalPhysics = true;
		}

		// Record Stats
		if ( CameraUserImpulseData->MoveForwardBackwardImpulse != 0 || CameraUserImpulseData->MoveRightLeftImpulse != 0 )
		{
			FEditorViewportStats::Using(FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_WASD);
		}
		else if ( CameraUserImpulseData->MoveWorldUpDownImpulse != 0 || CameraUserImpulseData->MoveLocalUpDownImpulse != 0 )
		{
			FEditorViewportStats::Using(FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_UP_DOWN);
		}
		else if ( CameraUserImpulseData->ZoomOutInImpulse != 0 )
		{
			FEditorViewportStats::Using(FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_FOV_ZOOM);
		}

		if( GetDefault<ULevelEditorViewportSettings>()->bLevelEditorJoystickControls )
		{
			//Now update for cached joystick info (relative movement first)
			UpdateCameraMovementFromJoystick(true, CameraController->GetConfig());

			//Now update for cached joystick info (absolute movement second)
			UpdateCameraMovementFromJoystick(false, CameraController->GetConfig());
		}

		FVector NewViewLocation = GetViewLocation();
		FRotator NewViewRotation = GetViewRotation();
		FVector NewViewEuler = GetViewRotation().Euler();
		float NewViewFOV = ViewFOV;

		// Only allow FOV recoil if flight camera mode is currently inactive.
		const bool bAllowRecoilIfNoImpulse = !bUsingFlightInput;

		// Update the camera's position, rotation and FOV
		float EditorMovementDeltaUpperBound = 1.0f;	// Never "teleport" the camera further than a reasonable amount after a large quantum

#if UE_BUILD_DEBUG
		// Editor movement is very difficult in debug without this, due to hitching
		// It is better to freeze movement during a hitch than to fly off past where you wanted to go
		// (considering there will be further hitching trying to get back to where you were)
		EditorMovementDeltaUpperBound = .15f;
#endif
		// Check whether the camera is being moved by the mouse or keyboard
		bool bHasMovement = GetDefault<ULevelEditorViewportSettings>()->bUseLegacyCameraMovementNotifications;

		if (CameraUserImpulseData->RotateYawVelocityModifier != 0.0f ||
			CameraUserImpulseData->RotatePitchVelocityModifier != 0.0f ||
			CameraUserImpulseData->RotateRollVelocityModifier != 0.0f ||
			CameraUserImpulseData->MoveForwardBackwardImpulse != 0.0f ||
			CameraUserImpulseData->MoveRightLeftImpulse != 0.0f ||
			CameraUserImpulseData->MoveWorldUpDownImpulse != 0.0f ||
			CameraUserImpulseData->MoveLocalUpDownImpulse != 0.0f ||
			CameraUserImpulseData->ZoomOutInImpulse != 0.0f ||
			CameraUserImpulseData->RotateYawImpulse != 0.0f ||
			CameraUserImpulseData->RotatePitchImpulse != 0.0f ||
			CameraUserImpulseData->RotateRollImpulse != 0.0f
			)
		{
			bHasMovement = true;
		}
		bHasMovement = bHasMovement && (bIsITFInteractions || bIsTracking);

		BeginCameraMovement(bHasMovement);

		CameraController->UpdateSimulation(
			*CameraUserImpulseData,
			FMath::Min(DeltaTime, EditorMovementDeltaUpperBound),
			bAllowRecoilIfNoImpulse,
			GetCameraSpeed(),
			NewViewLocation,
			NewViewEuler,
			NewViewFOV );

		// We'll zero out rotation velocity modifier after updating the simulation since these actions
		// are always momentary -- that is, when the user mouse looks some number of pixels,
		// we increment the impulse value right there
		{
			CameraUserImpulseData->RotateYawVelocityModifier = 0.0f;
			CameraUserImpulseData->RotatePitchVelocityModifier = 0.0f;
			CameraUserImpulseData->RotateRollVelocityModifier = 0.0f;
		}

		// Check for rotation difference within a small tolerance, ignoring winding
		if( !GetViewRotation().GetDenormalized().Equals( FRotator::MakeFromEuler( NewViewEuler ).GetDenormalized(), SMALL_NUMBER ) )
		{
			NewViewRotation = FRotator::MakeFromEuler( NewViewEuler );
		}

		// See if translation/rotation have changed
		const bool bTransformDifferent = !NewViewLocation.Equals(GetViewLocation(), SMALL_NUMBER) || NewViewRotation != GetViewRotation();
		// See if FOV has changed
		const bool bFOVDifferent = !FMath::IsNearlyEqual( NewViewFOV, ViewFOV, float(SMALL_NUMBER) );

		// If something has changed, tell the actor
		// However, don't move the actor if input is disabled: the only reason for transform or FOV changes to be reproduced on the actor
		// is if the user is piloting the actor directly or indirectly, which requires input.
		if ((bTransformDifferent || bFOVDifferent) && !bDisableInput)
		{
			// Something has changed!
			constexpr bool bInvalidateChildViews=true;

			// When flying the camera around the hit proxies don't need to be invalidated since we are flying around and not clicking on anything
			const bool bInvalidateHitProxies = bIsITFInteractions
            	? !ViewportNavigationHelper->bIsMouseLooking
            	: !IsFlightCameraActive();
			Invalidate(bInvalidateChildViews,bInvalidateHitProxies);

			// Update the FOV
			ViewFOV = NewViewFOV;

			// Actually move/rotate the camera
			if(bTransformDifferent)
			{
				MoveViewportPerspectiveCamera(
					NewViewLocation - GetViewLocation(),
					NewViewRotation - GetViewRotation() );
			}

			// Invalidate the viewport widget
			if (EditorViewportWidget.IsValid())
			{
				EditorViewportWidget.Pin()->Invalidate();
			}
		}
	}
}

bool FEditorViewportClient::ConsumeExternalCameraMovement()
{
	if (!Viewport || !ViewportNavigationHelper || !ViewportNavigationHelper->HasTransformDelta())
	{
		return false;
	}
	
	FVector Translation = FVector::ZeroVector;
	FRotator Rotation = FRotator::ZeroRotator;
	FVector Orbit = FVector::ZeroVector;
	
	ViewportNavigationHelper->ConsumeTransformDelta(Translation, Rotation, Orbit);
	
	BeginCameraMovement(!Translation.IsNearlyZero() || !Rotation.IsNearlyZero() || !Orbit.IsNearlyZero());
	
	if (IsPerspective())
	{
		bool bCameraHasMoved = false;
		if (!FMath::IsNearlyZero(Orbit.X) || !FMath::IsNearlyZero(Orbit.Y))
		{
			FRotator NewRotation = FRotator::ZeroRotator;
            NewRotation.Yaw = Orbit.X;
            NewRotation.Pitch = Orbit.Y;

            OrbitPerspectiveCamera(NewRotation);
			bCameraHasMoved = true;
		}
		
		if (!FMath::IsNearlyZero(Orbit.Z))
		{
			DollyPerspectiveCamera(Orbit.Z);
			bCameraHasMoved = true;
		}
		
		if (!Translation.IsNearlyZero() || !Rotation.IsNearlyZero())
		{
			ToggleOrbitCamera(false);
			MoveViewportPerspectiveCamera(Translation * GetCameraSpeed(), Rotation);
			bCameraHasMoved = true;
		}

		if (bCameraHasMoved)
		{
			PerspectiveCameraMoved();
		}
	}
	else
	{
		if (!FMath::IsNearlyZero(Orbit.Z))
		{
			float NewOrthoRatio = 1 + (FMath::Abs(Orbit.Z) / CAMERA_ZOOM_DAMPEN);
			if ( Orbit.Z < 0 )
			{
				NewOrthoRatio = 1 / NewOrthoRatio;
			}
		
			const float Zoom = GetOrthoZoom() * NewOrthoRatio;
			SetOrthoZoom( FMath::Clamp<float>( Zoom, GetMinimumOrthoZoom(), MAX_ORTHOZOOM ) );
		}
		
		if (!Translation.IsNearlyZero())
		{
			SetViewLocation(GetViewLocation() + Translation);
		}
	}
	
	return true;
}

void FEditorViewportClient::OrbitPerspectiveCamera(const FRotator& InOrbit)
{
	// This is pulled from InputAxisForOrbit(), with the input checking code removed
	if (GetDefault<ULevelEditorViewportSettings>()->bOrbitCameraAroundSelection)
	{
		FRotator NewRotation = InOrbit;
		
		FVector OrbitPoint;
		if (GetPivotForOrbit(OrbitPoint))
		{
			DefaultOrbitLocation = OrbitPoint;
		}

		// Override the default orbit behavior to allow orbiting around a given pivot
		// This uses different computations from the default orbit behavior so it must not ToggleOrbitCamera
		const bool bEnable = false;
		ToggleOrbitCamera(bEnable);
		
		const FRotator ViewRotation = GetViewRotation();
		
		// Compute the look-at and view location centered on the orbit point
		const FVector LookAtOffset = GetLookAtLocation() - DefaultOrbitLocation;
		const FVector ViewLocationOffset = GetViewLocation() - DefaultOrbitLocation;
		
		// When the roll is at 180 degrees, it means the view is upside down, so invert the yaw rotation
		if (ViewRotation.Roll == 180.f)
		{
			NewRotation.Yaw = -NewRotation.Yaw;
		}
		
		// Compute the delta rotation to apply as a transform
		FRotator DeltaRotation(NewRotation);
		FRotator ViewRotationNoPitch = ViewRotation;
		ViewRotationNoPitch.Pitch = 0;

		FTransform ViewRotationTransform = FTransform(ViewRotationNoPitch);
		FTransform DeltaRotationTransform = ViewRotationTransform.Inverse() * FTransform(DeltaRotation) * ViewRotationTransform;

		// Apply the delta rotation to the view rotation
		FRotator RotatedView = (FTransform(ViewRotation) * DeltaRotationTransform).Rotator();
	
		// Correct the rotation to remove drift in the roll
		if (FMath::IsNearlyEqual(FMath::Abs(RotatedView.Roll), 180.f, 1.f))
		{
			RotatedView.Roll = 180.f;
		}
		else if (FMath::IsNearlyZero(RotatedView.Roll, 1.f))
		{
			RotatedView.Roll = 0.f;
		}
		else
		{
			// FTransform::Rotator() returns an invalid RotatedView with roll in it when the initial pitch is at +/-90 degrees due to a singularity
			// FVector::ToOrientationRotator uses an alternate computation that doesn't suffer from the singularity. However, the roll it returns
			// is always 0 so it's not possible to tell if it's upside down and its yaw is flipped 180 degrees
			FVector ViewVector = ViewRotation.Vector();
			FVector RotatedViewVector = DeltaRotationTransform.TransformVector(ViewVector);
			FRotator RotatedViewRotator = RotatedViewVector.ToOrientationRotator();

			RotatedView = RotatedViewRotator + FRotator(0.f, -180.f, ViewRotation.Roll);
		}

		SetViewRotation(RotatedView);

		// Set the new rotated look-at
		FVector RotatedLookAtOffset = DeltaRotationTransform.TransformVector(LookAtOffset);
		FVector RotatedLookAt = DefaultOrbitLocation + RotatedLookAtOffset;
		SetLookAtLocation(RotatedLookAt);

		// Set the new rotated view location
		FVector RotatedViewOffset = DeltaRotationTransform.TransformVector(ViewLocationOffset);
		FVector RotatedViewLocation = DefaultOrbitLocation + RotatedViewOffset;
		SetViewLocation(RotatedViewLocation);
	}
	else
	{
		ToggleOrbitCamera(true);

		FViewportCameraTransform& ViewTransform = GetViewTransform();
		
		SetViewRotation( GetViewRotation() + FRotator( InOrbit.Pitch, -InOrbit.Yaw, InOrbit.Roll ) );

		/*
		 * Recalculates the view location according to the new SetViewRotation() did earlier.
		 */
		SetViewLocation(ViewTransform.ComputeOrbitMatrix().Inverse().GetOrigin());
	}
	
	PerspectiveCameraMoved();
}

void FEditorViewportClient::DollyPerspectiveCamera(double InDelta)
{
	ToggleOrbitCamera(true);
	
	FViewportCameraTransform& ViewTransform = GetViewTransform();

	const FVector ViewLocation = GetViewLocation();
	const FVector LookAtLocation = GetLookAtLocation();
	const FVector DeltaToTarget = LookAtLocation - ViewLocation;

	const float DistanceToTarget = DeltaToTarget.Length();
	const float CameraSpeed = GetCameraSpeed();
	const float CameraSpeedDistanceScale = ShouldScaleCameraSpeedByDistance() ? FMath::Min(DistanceToTarget / 1000.f, 1000.f) : 1.0f;
	const float DeltaLength = InDelta * CameraSpeed * CameraSpeedDistanceScale;

	// Make sure we don't move past the LookAt
	FVector DeltaLocation = FVector::UnitY() * FMath::Min(DeltaLength, DistanceToTarget);

	// Orient the delta down the view direction towards the look at
	FMatrix RotMat =
		FTranslationMatrix( -LookAtLocation ) *
		FRotationMatrix( FRotator(0,GetViewRotation().Yaw,0) ) *
		FRotationMatrix( FRotator(0, 0, GetViewRotation().Pitch));

	FVector TransformedDelta = RotMat.InverseFast().TransformVector(DeltaLocation);

	FMatrix OrbitMatrix = ViewTransform.ComputeOrbitMatrix().InverseFast();
	SetViewLocation(OrbitMatrix.GetOrigin() + TransformedDelta);
}

/**
 * Forcibly disables lighting show flags if there are no lights in the scene, or restores lighting show
 * flags if lights are added to the scene.
 */
void FEditorViewportClient::UpdateLightingShowFlags( FEngineShowFlags& InOutShowFlags )
{
	bool bViewportNeedsRefresh = false;

	if( bForcingUnlitForNewMap && !bInGameViewMode && IsPerspective() )
	{
		// We'll only use default lighting for viewports that are viewing the main world
		if (GWorld != NULL && GetScene() != NULL && GetScene()->GetWorld() != NULL && GetScene()->GetWorld() == GWorld )
		{
			// Check to see if there are any lights in the scene
			bool bAnyLights = GetScene()->HasAnyLights();
			if (bAnyLights)
			{
				// Is unlit mode currently enabled?  We'll make sure that all of the regular unlit view
				// mode show flags are set (not just EngineShowFlags.Lighting), so we don't disrupt other view modes
				if (!InOutShowFlags.Lighting)
				{
					// We have lights in the scene now so go ahead and turn lighting back on
					// designer can see what they're interacting with!
					InOutShowFlags.SetLighting(true);
				}

				// No longer forcing lighting to be off
				bForcingUnlitForNewMap = false;
			}
			else
			{
				// Is lighting currently enabled?
				if (InOutShowFlags.Lighting)
				{
					// No lights in the scene, so make sure that lighting is turned off so the level
					// designer can see what they're interacting with!
					InOutShowFlags.SetLighting(false);
				}
			}
		}
	}
}

bool FEditorViewportClient::CalculateEditorConstrainedViewRect(FSlateRect& OutSafeFrameRect, FViewport* InViewport, float DPIScale)
{
	const float SizeX = InViewport->GetSizeXY().X / DPIScale;
	const float SizeY = InViewport->GetSizeXY().Y / DPIScale;

	OutSafeFrameRect = FSlateRect(0, 0, SizeX, SizeY);
	float FixedAspectRatio;
	bool bSafeFrameActive = GetActiveSafeFrame(FixedAspectRatio);

	if (bSafeFrameActive)
	{
		// Get the size of the viewport
		float ActualAspectRatio = (float)SizeX / (float)SizeY;

		float SafeWidth = SizeX;
		float SafeHeight = SizeY;

		if (FixedAspectRatio < ActualAspectRatio)
		{
			// vertical bars required on left and right
			SafeWidth = FixedAspectRatio * SizeY;
			float CorrectedHalfWidth = SafeWidth * 0.5f;
			float CentreX = SizeX * 0.5f;
			float X1 = CentreX - CorrectedHalfWidth;
			float X2 = CentreX + CorrectedHalfWidth;
			OutSafeFrameRect = FSlateRect(X1, 0, X2, SizeY);
		}
		else
		{
			// horizontal bars required on top and bottom
			SafeHeight = SizeX / FixedAspectRatio;
			float CorrectedHalfHeight = SafeHeight * 0.5f;
			float CentreY = SizeY * 0.5f;
			float Y1 = CentreY - CorrectedHalfHeight;
			float Y2 = CentreY + CorrectedHalfHeight;
			OutSafeFrameRect = FSlateRect(0, Y1, SizeX, Y2);
		}
	}

	return bSafeFrameActive;
}

void FEditorViewportClient::DrawSafeFrames(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	if (EngineShowFlags.CameraAspectRatioBars || EngineShowFlags.CameraSafeFrames)
	{
		FSlateRect SafeRect;
		if (CalculateEditorConstrainedViewRect(SafeRect, &InViewport, Canvas.GetDPIScale()))
		{
			if (EngineShowFlags.CameraSafeFrames)
			{
				FSlateRect InnerRect = SafeRect.InsetBy(FMargin(0.5f * SafePadding * SafeRect.GetSize().Size()));
				FCanvasBoxItem BoxItem(FVector2D(InnerRect.Left, InnerRect.Top), InnerRect.GetSize());
				BoxItem.SetColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f));
				Canvas.DrawItem(BoxItem);
			}

			if (EngineShowFlags.CameraAspectRatioBars)
			{
				const int32 SizeX = InViewport.GetSizeXY().X;
				const int32 SizeY = InViewport.GetSizeXY().Y;
				FCanvasLineItem LineItem;
				LineItem.SetColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.75f));

				if (SafeRect.GetSize().X < SizeX)
				{
					DrawSafeFrameQuad(Canvas, FVector2D(0, SafeRect.Top), FVector2D(SafeRect.Left, SafeRect.Bottom));
					DrawSafeFrameQuad(Canvas, FVector2D(SafeRect.Right, SafeRect.Top), FVector2D(SizeX, SafeRect.Bottom));
					LineItem.Draw(&Canvas, FVector2D(SafeRect.Left, 0), FVector2D(SafeRect.Left, SizeY));
					LineItem.Draw(&Canvas, FVector2D(SafeRect.Right, 0), FVector2D(SafeRect.Right, SizeY));
				}

				if (SafeRect.GetSize().Y < SizeY)
				{
					DrawSafeFrameQuad(Canvas, FVector2D(SafeRect.Left, 0), FVector2D(SafeRect.Right, SafeRect.Top));
					DrawSafeFrameQuad(Canvas, FVector2D(SafeRect.Left, SafeRect.Bottom), FVector2D(SafeRect.Right, SizeY));
					LineItem.Draw(&Canvas, FVector2D(0, SafeRect.Top), FVector2D(SizeX, SafeRect.Top));
					LineItem.Draw(&Canvas, FVector2D(0, SafeRect.Bottom), FVector2D(SizeX, SafeRect.Bottom));
				}
			}
		}
	}
}

void FEditorViewportClient::DrawSafeFrameQuad( FCanvas &Canvas, FVector2D V1, FVector2D V2 )
{
	static const FLinearColor SafeFrameColor(0.0f, 0.0f, 0.0f, 1.0f);
	FCanvasUVTri UVTriItem;
	UVTriItem.V0_Pos = FVector2D(V1.X, V1.Y);
	UVTriItem.V1_Pos = FVector2D(V2.X, V1.Y);
	UVTriItem.V2_Pos = FVector2D(V1.X, V2.Y);
	FCanvasTriangleItem TriItem( UVTriItem, GWhiteTexture );
	UVTriItem.V0_Pos = FVector2D(V2.X, V1.Y);
	UVTriItem.V1_Pos = FVector2D(V2.X, V2.Y);
	UVTriItem.V2_Pos = FVector2D(V1.X, V2.Y);
	TriItem.TriangleList.Add( UVTriItem );
	TriItem.SetColor( SafeFrameColor );
	TriItem.Draw( &Canvas );
}

int32 FEditorViewportClient::SetStatEnabled(const TCHAR* InName, const bool bEnable, const bool bAll)
{
	if (bEnable)
	{
		check(!bAll);	// Not possible to enable all
		EnabledStats.AddUnique(InName);
	}
	else
	{
		if (bAll)
		{
			EnabledStats.Empty();
		}
		else
		{
			EnabledStats.Remove(InName);
		}
	}
	return EnabledStats.Num();
}


void FEditorViewportClient::HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled)
{
	// Check to see which viewports have this enabled (current, non-current)
	const bool bEnabled = IsStatEnabled(InName);
	if (GStatProcessingViewportClient == this)
	{
		// Only if realtime and stats are also enabled should we show the stat as visible
		bOutCurrentEnabled = IsRealtime() && ShouldShowStats() && bEnabled;
	}
	else
	{
		bOutOthersEnabled |= bEnabled;
	}
}

void FEditorViewportClient::HandleViewportStatEnabled(const TCHAR* InName)
{
	// Just enable this on the active viewport
	if (GStatProcessingViewportClient == this)
	{
		SetShowStats(true);
		AddRealtimeOverride(true, LOCTEXT("RealtimeOverrideMessage_Stats", "Stats Display"));
		SetStatEnabled(InName, true);
	}
}

void FEditorViewportClient::HandleViewportStatDisabled(const TCHAR* InName)
{
	// Just disable this on the active viewport
	if (GStatProcessingViewportClient == this)
	{
		if (SetStatEnabled(InName, false) == 0)
		{
			SetShowStats(false);
			RemoveRealtimeOverride(LOCTEXT("RealtimeOverrideMessage_Stats", "Stats Display"), /*bCheckMissingOverride*/false);
		}
	}
}

void FEditorViewportClient::HandleViewportStatDisableAll(const bool bInAnyViewport)
{
	// Disable all on either all or the current viewport (depending on the flag)
	if (bInAnyViewport || GStatProcessingViewportClient == this)
	{
		SetShowStats(false);
		SetStatEnabled(NULL, false, true);
		RemoveRealtimeOverride(LOCTEXT("RealtimeOverrideMessage_Stats", "Stats Display"), /*bCheckMissingOverride*/false);
	}
}

void FEditorViewportClient::HandleWindowDPIScaleChanged(TSharedRef<SWindow> InWindow)
{
	RequestUpdateDPIScale();
	Invalidate();
}

void FEditorViewportClient::UpdateMouseDelta()
{
	// Do nothing if a drag tool is being used.
	if (MouseDeltaTracker->UsingDragTool() || ModeTools->DisallowMouseDeltaTracking())
	{
		return;
	}

	// Stop tracking and do nothing else if we're tracking and the widget mode has changed mid-track.
	// It can confuse the widget code that handles the mouse movements.
	if (bIsTracking && MouseDeltaTracker->GetTrackingWidgetMode() != GetWidgetMode())
	{
		StopTracking();
		return;
	}

	FVector DragDelta = MouseDeltaTracker->GetDelta();

	GEditor->MouseMovement += DragDelta.GetAbs();

	if( Viewport )
	{
		// If mouse delta tracker has delta OR there is transform delta coming from ITF Viewport Interaction, handle it
		if (!DragDelta.IsNearlyZero())
		{
			const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
			const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
			const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
			const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();
			const bool bIsNonOrbitMiddleMouse = MiddleMouseButtonDown && !IsAltPressed();

			// If a tool is overriding current widget mode behavior, it may need to
			// temporarily set a different widget mode while converting mouse movement.
			ModeTools->PreConvertMouseMovement(this);

			// Convert the movement delta into drag/rotation deltas
			FVector Drag = FVector::ZeroVector;
			FRotator Rot = FRotator::ZeroRotator;
			FVector Scale = FVector::ZeroVector;
			EAxisList::Type CurrentAxis = Widget->GetCurrentAxis();

			if ( IsOrtho() && ( LeftMouseButtonDown || bIsUsingTrackpad ) && RightMouseButtonDown )
			{
				bWidgetAxisControlledByDrag = false;
				Widget->SetCurrentAxis( EAxisList::None );
				MouseDeltaTracker->ConvertMovementDeltaToDragRot(DragStartView, this, DragDelta, Drag, Rot, Scale);
				Widget->SetCurrentAxis( CurrentAxis );
				CurrentAxis = EAxisList::None;
			}
			else
			{
				if (DragStartView == nullptr)
				{
					// Compute a view.
					DragStartViewFamily = new FSceneViewFamilyContext(FSceneViewFamily::ConstructionValues(
						Viewport,
						GetScene(),
						EngineShowFlags)
						.SetRealtimeUpdate(IsRealtime()));
					DragStartView = CalcSceneView(DragStartViewFamily);
				}
				//if Absolute Translation, and not just moving the camera around
				if (IsUsingAbsoluteTranslation(false))
				{
					// TODO: yet to be checked out for ITF Parity
					MouseDeltaTracker->AbsoluteTranslationConvertMouseToDragRot(DragStartView, this, Drag, Rot, Scale);
				}
				else
				{
					MouseDeltaTracker->ConvertMovementDeltaToDragRot(DragStartView, this, DragDelta, Drag, Rot, Scale);
				}
			}

			ModeTools->PostConvertMouseMovement(this);

			const bool bInputHandledByGizmos = InputWidgetDelta( Viewport, CurrentAxis, Drag, Rot, Scale );

			if( !Rot.IsZero() )
			{
				Widget->UpdateDeltaRotation();
			}

			if( !bInputHandledByGizmos )
			{
				PeformDefaultCameraMovement(Drag, Rot, Scale);
			}

			// Clean up
			MouseDeltaTracker->ReduceBy( DragDelta );

			Invalidate( false, false );
		}
	}
}

void FEditorViewportClient::PeformDefaultCameraMovement(FVector& Drag, FRotator& Rot, FVector& Scale)
{
	// Check if there is an active ITF Orbit interaction, and if there's an orbit delta it might have generated 
	const bool bUseITFOrbitDelta = IsViewportInteractionEnabled(ModeTools.Get(), UE::Editor::ViewportInteractions::Orbit)
					  && !ViewportNavigationHelper->OrbitDelta.IsNearlyZero();

	FVector DragDelta;

	if (bUseITFOrbitDelta)
	{
		ViewportNavigationHelper->ConsumeOrbitDelta(DragDelta);
	}
	else // no ITF delta, try to use the one from mouse delta tracker
	{
		DragDelta = MouseDeltaTracker->GetDelta();
	}

	if (ShouldOrbitCamera())
	{
		bool bHasMovement = !DragDelta.IsNearlyZero();

		BeginCameraMovement(bHasMovement);

		FVector TempDrag;
		FRotator TempRot;
		InputAxisForOrbit(Viewport, DragDelta, TempDrag, TempRot);
	}
	else
	{
		// Disable orbit camera
		const bool bEnable = false;
		ToggleOrbitCamera(bEnable);

		if (ShouldPanOrDollyCamera())
		{
			bool bHasMovement = !Drag.IsNearlyZero() || !Rot.IsNearlyZero();

			BeginCameraMovement(bHasMovement);

			if (!IsOrtho())
			{
				const float CameraSpeed = GetCameraSpeed();
				Drag *= CameraSpeed;
			}
			MoveViewportCamera(Drag, Rot);

			const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
			const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
			const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);

			if (IsPerspective() && LeftMouseButtonDown && !MiddleMouseButtonDown && !RightMouseButtonDown)
			{
				FEditorViewportStats::Using(FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_DOLLY);
			}
			else
			{
				if (!Drag.IsZero())
				{
					FEditorViewportStats::Using(IsPerspective() ? FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_PAN : FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_PAN);
				}
			}
		}
	}
}

static bool IsOrbitRotationMode( FViewport* Viewport )
{
	bool	LeftMouseButton = Viewport->KeyState(EKeys::LeftMouseButton),
		MiddleMouseButton = Viewport->KeyState(EKeys::MiddleMouseButton),
		RightMouseButton = Viewport->KeyState(EKeys::RightMouseButton);
	return LeftMouseButton && !MiddleMouseButton && !RightMouseButton ;
}

static bool IsOrbitPanMode( FViewport* Viewport )
{
	bool	LeftMouseButton = Viewport->KeyState(EKeys::LeftMouseButton),
		MiddleMouseButton = Viewport->KeyState(EKeys::MiddleMouseButton),
		RightMouseButton = Viewport->KeyState(EKeys::RightMouseButton);

	bool bAltPressed = Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);

	return  (MiddleMouseButton && !LeftMouseButton && !RightMouseButton) || (!bAltPressed && MiddleMouseButton );
}

static bool IsOrbitZoomMode( FViewport* Viewport )
{
	bool	LeftMouseButton = Viewport->KeyState(EKeys::LeftMouseButton),
		MiddleMouseButton = Viewport->KeyState(EKeys::MiddleMouseButton),
		RightMouseButton = Viewport->KeyState(EKeys::RightMouseButton);

	 return RightMouseButton || (LeftMouseButton && MiddleMouseButton);
}

bool FEditorViewportClient::GetPivotForOrbit(FVector& Pivot) const
{
	return ModeTools->GetPivotForOrbit(Pivot);
}

void FEditorViewportClient::InputAxisForOrbit(FViewport* InViewport, const FVector& DragDelta, FVector& Drag, FRotator& Rot)
{
	FVector OrbitPoint;
	bool bHasCustomOrbitPivot = GetPivotForOrbit(OrbitPoint);

	if ( GetDefault<ULevelEditorViewportSettings>()->bOrbitCameraAroundSelection && IsOrbitRotationMode( InViewport ))
	{
		if (bHasCustomOrbitPivot)
		{
			DefaultOrbitLocation = OrbitPoint;
		}
		
		// Override the default orbit behavior to allow orbiting around a given pivot
		// This uses different computations from the default orbit behavior so it must not ToggleOrbitCamera
		const bool bEnable = false;
		ToggleOrbitCamera(bEnable);

		ConvertMovementToOrbitDragRot(DragDelta, Drag, Rot);

		const FRotator ViewRotation = GetViewRotation();

		// Compute the look-at and view location centered on the orbit point
		const FVector LookAtOffset = GetLookAtLocation() - DefaultOrbitLocation;
		const FVector ViewLocationOffset = GetViewLocation() - DefaultOrbitLocation;

		// When the roll is at 180 degrees, it means the view is upside down, so invert the yaw rotation
		if (ViewRotation.Roll == 180.f)
		{
			Rot.Yaw = -Rot.Yaw;
		}

		// Compute the delta rotation to apply as a transform
		FRotator DeltaRotation(Rot.Pitch, Rot.Yaw, Rot.Roll);
		FRotator ViewRotationNoPitch = ViewRotation;
		ViewRotationNoPitch.Pitch = 0;

		FTransform ViewRotationTransform = FTransform(ViewRotationNoPitch);
		FTransform DeltaRotationTransform = ViewRotationTransform.Inverse() * FTransform(DeltaRotation) * ViewRotationTransform;

		// Apply the delta rotation to the view rotation
		FRotator RotatedView = (FTransform(ViewRotation) * DeltaRotationTransform).Rotator();

		// Correct the rotation to remove drift in the roll
		if (FMath::IsNearlyEqual(FMath::Abs(RotatedView.Roll), 180.f, 1.f))
		{
			RotatedView.Roll = 180.f;
		}
		else if (FMath::IsNearlyZero(RotatedView.Roll, 1.f))
		{
			RotatedView.Roll = 0.f;
		}
		else
		{
			// FTransform::Rotator() returns an invalid RotatedView with roll in it when the initial pitch is at +/-90 degrees due to a singularity
			// FVector::ToOrientatioRotator uses an alternate computation that doesn't suffer from the singularity. However, the roll it returns
			// is always 0 so it's not possible to tell if it's upside down and its yaw is flipped 180 degrees
			FVector ViewVector = ViewRotation.Vector();
			FVector RotatedViewVector = DeltaRotationTransform.TransformVector(ViewVector);
			FRotator RotatedViewRotator = RotatedViewVector.ToOrientationRotator();

			RotatedView = RotatedViewRotator + FRotator(0.f, -180.f, ViewRotation.Roll);
		}

		SetViewRotation(RotatedView);

		// Set the new rotated look-at
		FVector RotatedLookAtOffset = DeltaRotationTransform.TransformVector(LookAtOffset);
		FVector RotatedLookAt = DefaultOrbitLocation + RotatedLookAtOffset;
		SetLookAtLocation(RotatedLookAt);

		// Set the new rotated view location
		FVector RotatedViewOffset = DeltaRotationTransform.TransformVector(ViewLocationOffset);
		FVector RotatedViewLocation = DefaultOrbitLocation + RotatedViewOffset;
		SetViewLocation(RotatedViewLocation);

		FEditorViewportStats::Using(IsPerspective() ? FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_ROTATION : FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ROTATION);

		if (IsPerspective())
		{
			PerspectiveCameraMoved();
		}
	}
	else
	{
		// Ensure orbit is enabled
		const bool bEnable=true;
		ToggleOrbitCamera(bEnable);

		FRotator TempRot = GetViewRotation();

		SetViewRotation( FRotator(0,90,0) );
		ConvertMovementToOrbitDragRot(DragDelta, Drag, Rot);
		SetViewRotation( TempRot );

		Drag.X = DragDelta.X;

		FViewportCameraTransform& ViewTransform = GetViewTransform();
		const float CameraSpeedDistanceScale = ShouldScaleCameraSpeedByDistance() ? FMath::Min(FVector::Dist( GetViewLocation(), GetLookAtLocation() ) / 1000.f, 1000.f) : 1.0f;

		if ( IsOrbitRotationMode( InViewport ) )
		{
			SetViewRotation( GetViewRotation() + FRotator( Rot.Pitch, -Rot.Yaw, Rot.Roll ) );
			FEditorViewportStats::Using(IsPerspective() ? FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_ROTATION : FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ROTATION);

			/*
			 * Recalculates the view location according to the new SetViewRotation() did earlier.
			 */
			SetViewLocation(ViewTransform.ComputeOrbitMatrix().Inverse().GetOrigin());
		}
		else if ( IsOrbitPanMode( InViewport ) )
		{
			const bool bInvert = GetDefault<ULevelEditorViewportSettings>()->bInvertMiddleMousePan;

			const float CameraSpeed = GetCameraSpeed();
			Drag *= CameraSpeed * CameraSpeedDistanceScale;

			FVector DeltaLocation = bInvert ? FVector(Drag.X, 0, -Drag.Z ) : FVector(-Drag.X, 0, Drag.Z);

			FVector LookAt = ViewTransform.GetLookAt();

			FMatrix RotMat =
				FTranslationMatrix( -LookAt ) *
				FRotationMatrix( FRotator(0,GetViewRotation().Yaw,0) ) *
				FRotationMatrix( FRotator(0, 0, GetViewRotation().Pitch));

			FVector TransformedDelta = RotMat.InverseFast().TransformVector(DeltaLocation);

			SetLookAtLocation( GetLookAtLocation() + TransformedDelta );
			SetViewLocation(ViewTransform.ComputeOrbitMatrix().Inverse().GetOrigin());

			FEditorViewportStats::Using(IsPerspective() ? FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_PAN : FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_PAN);
		}
		else if ( IsOrbitZoomMode( InViewport ) )
		{
			const bool bInvertY = GetDefault<ULevelEditorViewportSettings>()->bInvertRightMouseDollyYAxis;

			const float CameraSpeed = GetCameraSpeed();
			Drag *= CameraSpeed * CameraSpeedDistanceScale;

			const FVector ViewLocation = GetViewLocation();
			const FVector LookAtLocation = GetLookAtLocation();
			const FVector DeltaToTarget = LookAtLocation - ViewLocation;

			const float DistanceToTarget = DeltaToTarget.Length();
			const float DeltaLength = bInvertY ? Drag.X + Drag.Y : Drag.X+ -Drag.Y;

			// Make sure we don't move past the LookAt
			FVector DeltaLocation = FVector::UnitY() * FMath::Min(DeltaLength, DistanceToTarget);

			// Orient the delta down the view direction towards the look at
			FMatrix RotMat =
				FTranslationMatrix( -LookAtLocation ) *
				FRotationMatrix( FRotator(0,GetViewRotation().Yaw,0) ) *
				FRotationMatrix( FRotator(0, 0, GetViewRotation().Pitch));

			FVector TransformedDelta = RotMat.InverseFast().TransformVector(DeltaLocation);

			FMatrix OrbitMatrix = ViewTransform.ComputeOrbitMatrix().InverseFast();
			SetViewLocation( OrbitMatrix.GetOrigin() + TransformedDelta );

			FEditorViewportStats::Using(IsPerspective() ? FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_ZOOM : FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ZOOM);
		}

		if ( IsPerspective() )
		{
			PerspectiveCameraMoved();
		}
	}
}

/**
 * forces a cursor update and marks the window as a move has occurred
 */
void FEditorViewportClient::MarkMouseMovedSinceClick()
{
	if (!bHasMouseMovedSinceClick )
	{
		bHasMouseMovedSinceClick = true;
		//if we care about the cursor
		if (Viewport->IsCursorVisible() && Viewport->HasMouseCapture())
		{
			//force a refresh
			Viewport->UpdateMouseCursor(true);
		}
	}
}

/** Determines whether this viewport is currently allowed to use Absolute Movement */
bool FEditorViewportClient::IsUsingAbsoluteTranslation(bool bAlsoCheckAbsoluteRotation) const
{
	bool bIsHotKeyAxisLocked = Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
	bool bCameraLockedToWidget = !(Widget && Widget->GetCurrentAxis() & EAxisList::Screen) && IsPrioritizedInputChordPressed(InputChordName_CameraLockedToWidget);
	// Screen-space movement must always use absolute translation
	bool bScreenSpaceTransformation = Widget && (Widget->GetCurrentAxis() == EAxisList::Screen) && GetWidgetMode() != UE::Widget::WM_Rotate;
	bool bAbsoluteMovementEnabled = GetDefault<ULevelEditorViewportSettings>()->bUseAbsoluteTranslation || bScreenSpaceTransformation;
	bool bCurrentWidgetSupportsAbsoluteMovement = FWidget::AllowsAbsoluteTranslationMovement( GetWidgetMode()) || bScreenSpaceTransformation;
	EAxisList::Type AxisType = Widget ? Widget->GetCurrentAxis() : EAxisList::None;

	bool bCurrentWidgetSupportsAbsoluteRotation = bAlsoCheckAbsoluteRotation ? FWidget::AllowsAbsoluteRotationMovement(GetWidgetMode(), AxisType) : false;
	bool bWidgetActivelyTrackingAbsoluteMovement = Widget && (Widget->GetCurrentAxis() != EAxisList::None);

	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);

	const bool bAnyMouseButtonsDown = (LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown);

	return (!bCameraLockedToWidget && !bIsHotKeyAxisLocked && bAbsoluteMovementEnabled && (bCurrentWidgetSupportsAbsoluteMovement || bCurrentWidgetSupportsAbsoluteRotation) && bWidgetActivelyTrackingAbsoluteMovement && !IsOrtho() && bAnyMouseButtonsDown);
}

bool FEditorViewportClient::IsFlightCameraActive() const
{
	bool bIsFlightMovementKey = false;
	for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		auto ChordIndex = static_cast<EMultipleKeyBindingIndex>(i);
		bIsFlightMovementKey |= (Viewport->KeyState(FViewportNavigationCommands::Get().Forward->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().Backward->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().Left->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().Right->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().WorldUp->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().WorldDown->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().LocalUp->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().LocalDown->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomIn->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().FovZoomOut->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().RotateUp->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().RotateDown->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().RotateLeft->GetActiveChord(ChordIndex)->Key)
			|| Viewport->KeyState(FViewportNavigationCommands::Get().RotateRight->GetActiveChord(ChordIndex)->Key));
	}
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

	// Movement key pressed and automatic movement enabled
	bIsFlightMovementKey &= (GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlType == WASD_Always) | bIsUsingTrackpad;

	// Not using automatic movement but the flight camera is active
	bIsFlightMovementKey |= IsFlightCameraInputModeActive() && (GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlType == WASD_RMBOnly );

	return
		!(Viewport->KeyState( EKeys::LeftControl ) || Viewport->KeyState( EKeys::RightControl ) ) &&
		!(Viewport->KeyState( EKeys::LeftShift ) || Viewport->KeyState( EKeys::RightShift ) ) &&
		!(Viewport->KeyState( EKeys::LeftAlt ) || Viewport->KeyState( EKeys::RightAlt ) ) &&
		bIsFlightMovementKey;
}

void FEditorViewportClient::HandleToggleShowFlag(FEngineShowFlags::EShowFlag EngineShowFlagIndex)
{
	const bool bOldState = EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
	EngineShowFlags.SetSingleFlag(EngineShowFlagIndex, !bOldState);

	// If changing collision flag, need to do special handling for hidden objects.
	if (EngineShowFlagIndex == FEngineShowFlags::EShowFlag::SF_Collision)
	{
		UpdateHiddenCollisionDrawing();
	}

	// Invalidate clients which aren't real-time so we see the changes.
	Invalidate();
}

bool FEditorViewportClient::HandleIsShowFlagEnabled(FEngineShowFlags::EShowFlag EngineShowFlagIndex) const
{
	return EngineShowFlags.GetSingleFlag(EngineShowFlagIndex);
}

void FEditorViewportClient::ChangeBufferVisualizationMode( FName InName )
{
	SetViewMode(VMI_VisualizeBuffer);
	CurrentBufferVisualizationMode = InName;
}

bool FEditorViewportClient::IsBufferVisualizationModeSelected( FName InName ) const
{
	return IsViewModeEnabled( VMI_VisualizeBuffer ) && CurrentBufferVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentBufferVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeBuffer), TEXT("In order to call GetCurrentBufferVisualizationMode(), first you must set ViewMode to VMI_VisualizeBuffer."));
	return (CurrentBufferVisualizationMode.IsNone()
		? FBufferVisualizationData::GetMaterialDefaultDisplayName() : GetBufferVisualizationData().GetMaterialDisplayName(CurrentBufferVisualizationMode));
}

void FEditorViewportClient::ChangeNaniteVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeNanite);
	CurrentNaniteVisualizationMode = InName;
}

bool FEditorViewportClient::IsNaniteVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeNanite) && CurrentNaniteVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentNaniteVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeNanite), TEXT("In order to call GetCurrentNaniteVisualizationMode(), first you must set ViewMode to VMI_VisualizeNanite."));
	return GetNaniteVisualizationData().GetModeDisplayName(CurrentNaniteVisualizationMode);
}

void FEditorViewportClient::ChangeLumenVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeLumen);
	CurrentLumenVisualizationMode = InName;
}

bool FEditorViewportClient::IsLumenVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeLumen) && CurrentLumenVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentLumenVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeLumen), TEXT("In order to call GetCurrentLumenVisualizationMode(), first you must set ViewMode to VMI_VisualizeLumen."));
	return GetLumenVisualizationData().GetModeDisplayName(CurrentLumenVisualizationMode);
}

void FEditorViewportClient::ChangeMegaLightsVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeMegaLights);
	CurrentMegaLightsVisualizationMode = InName;
}

bool FEditorViewportClient::IsMegaLightsVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeMegaLights) && CurrentMegaLightsVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentMegaLightsVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeMegaLights), TEXT("In order to call GetCurrentMegaLightsVisualizationMode(), first you must set ViewMode to VMI_VisualizeMegaLights."));
	return GetMegaLightsVisualizationData().GetModeDisplayName(CurrentMegaLightsVisualizationMode);
}

void FEditorViewportClient::ChangeVirtualShadowMapVisualizationMode(FName InName)
{
	EViewModeIndex ViewModeIndex = GetVirtualShadowMapVisualizationData().GetViewModeIndexForVisualization(InName);
	SetViewMode(ViewModeIndex);
	CurrentVirtualShadowMapVisualizationMode = InName;
}

bool FEditorViewportClient::IsVirtualShadowMapVisualizationModeSelected(FName InName) const
{
	return GetVirtualShadowMapVisualizationData().IsVSMViewMode(GetViewMode()) && CurrentVirtualShadowMapVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentVirtualShadowMapVisualizationModeDisplayName() const
{
	checkf(GetVirtualShadowMapVisualizationData().IsVSMViewMode(GetViewMode()), TEXT("In order to call GetCurrentVirtualShadowMapVisualizationMode(), the ViewMode must first be set to VMI_VisualizeVirtualShadowMap or another compatible one."));
	return GetVirtualShadowMapVisualizationData().GetModeDisplayName(CurrentVirtualShadowMapVisualizationMode);
}

void FEditorViewportClient::ChangeVirtualTextureVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeVirtualTexture);
	CurrentVirtualTextureVisualizationMode = InName;
}

bool FEditorViewportClient::IsVirtualTextureVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeVirtualTexture) && CurrentVirtualTextureVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentVirtualTextureVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeVirtualTexture), TEXT("In order to call GetCurrentVirtualTextureVisualizationMode(), first you must set ViewMode to VMI_VisualizeVirtualTexture."));
	return GetVirtualTextureVisualizationData().GetModeDisplayName(CurrentVirtualTextureVisualizationMode);
}

void FEditorViewportClient::ChangeActorColorationVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeActorColoration);
	FActorPrimitiveColorHandler::Get().SetActivePrimitiveColorHandler(InName, GWorld);
}

bool FEditorViewportClient::IsActorColorationVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeActorColoration) && FActorPrimitiveColorHandler::Get().GetActivePrimitiveColorHandler() == InName;
}

FText FEditorViewportClient::GetCurrentActorColorationVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeActorColoration), TEXT("In order to call GetCurrentActorColorationVisualizationModeDisplayName(), first you must set ViewMode to VMI_VisualizeActorColoration."));
	return FActorPrimitiveColorHandler::Get().GetActivePrimitiveColorHandlerDisplayName();
}

void FEditorViewportClient::ChangeSubstrateVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeSubstrate);
	CurrentSubstrateVisualizationMode = InName;
}

bool FEditorViewportClient::IsSubstrateVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeSubstrate) && CurrentSubstrateVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentSubstrateVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeSubstrate), TEXT("In order to call GetCurrentSubstrateVisualizationMode(), first you must set ViewMode to VMI_VisualizeSubstrate."));
	return GetSubstrateVisualizationData().GetModeDisplayName(CurrentSubstrateVisualizationMode);
}

void FEditorViewportClient::ChangeGroomVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeGroom);
	CurrentGroomVisualizationMode = InName;
	EngineShowFlags.SetShaderPrint(true);
}

bool FEditorViewportClient::IsGroomVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeGroom) && CurrentGroomVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentGroomVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeGroom), TEXT("In order to call GetCurrentGroomVisualizationMode(), first you must set ViewMode to VMI_VisualizeGroom."));
	return GetGroomVisualizationData().GetModeDisplayName(CurrentGroomVisualizationMode);
}
bool FEditorViewportClient::IsVisualizeCalibrationMaterialEnabled() const
{
	// Get the list of requested buffers from the console
	const URendererSettings* Settings = GetDefault<URendererSettings>();
	check(Settings);

	return ((EngineShowFlags.VisualizeCalibrationCustom && Settings->VisualizeCalibrationCustomMaterialPath.IsValid()) ||
		(EngineShowFlags.VisualizeCalibrationColor  && Settings->VisualizeCalibrationColorMaterialPath.IsValid()) ||
		(EngineShowFlags.VisualizeCalibrationGrayscale && Settings->VisualizeCalibrationGrayscaleMaterialPath.IsValid()));
}

void FEditorViewportClient::ChangeRayTracingDebugVisualizationMode(FName InName)
{
	SetViewMode(VMI_RayTracingDebug);
	CurrentRayTracingDebugVisualizationMode = InName;
}

bool FEditorViewportClient::IsRayTracingDebugVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_RayTracingDebug) && CurrentRayTracingDebugVisualizationMode == InName;
}

void FEditorViewportClient::ChangeGPUSkinCacheVisualizationMode(FName InName)
{
	SetViewMode(VMI_VisualizeGPUSkinCache);
	CurrentGPUSkinCacheVisualizationMode = InName;
}

bool FEditorViewportClient::IsGPUSkinCacheVisualizationModeSelected(FName InName) const
{
	return IsViewModeEnabled(VMI_VisualizeGPUSkinCache) && CurrentGPUSkinCacheVisualizationMode == InName;
}

FText FEditorViewportClient::GetCurrentGPUSkinCacheVisualizationModeDisplayName() const
{
	checkf(IsViewModeEnabled(VMI_VisualizeGPUSkinCache), TEXT("In order to call GetCurrentGPUSkinCacheVisualizationMode(), first you must set ViewMode to VMI_VisualizeGPUSkinCache."));
	return GetGPUSkinCacheVisualizationData().GetModeDisplayName(CurrentGPUSkinCacheVisualizationMode);
}

bool FEditorViewportClient::SupportsPreviewResolutionFraction() const
{
	// Don't do preview screen percentage for some view mode.
	switch (GetViewMode())
	{
	case VMI_BrushWireframe:
	case VMI_Wireframe:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case VMI_Lit_Wireframe:
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	case VMI_LightComplexity:
	case VMI_LightmapDensity:
	case VMI_LitLightmapDensity:
	case VMI_ReflectionOverride:
	case VMI_StationaryLightOverlap:
	case VMI_CollisionPawn:
	case VMI_CollisionVisibility:
	case VMI_LODColoration:
	case VMI_PrimitiveDistanceAccuracy:
	case VMI_MeshUVDensityAccuracy:
	case VMI_HLODColoration:
	case VMI_GroupLODColoration:
	case VMI_VisualizeGPUSkinCache:
	case VMI_ShadowCasters:
		return false;
	}

	// Don't do preview screen percentage in certain cases.
	if (EngineShowFlags.VisualizeBuffer 
		|| EngineShowFlags.MeshEdges
		|| EngineShowFlags.VisualizeNanite
		|| IsVisualizeCalibrationMaterialEnabled())
	{
		return false;
	}

	return true;
}

EViewStatusForScreenPercentage FEditorViewportClient::GetViewStatusForScreenPercentage() const
{
	if (EngineShowFlags.PathTracing)
	{
		return EViewStatusForScreenPercentage::PathTracer;
	}
	else if (EngineShowFlags.StereoRendering || EngineShowFlags.VREditing)
	{
		return EViewStatusForScreenPercentage::VR;
	}
	else if (!bIsRealtime)
	{
		return EViewStatusForScreenPercentage::NonRealtime;
	}
	else if (GetWorld() && GetWorld()->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
	{
		return EViewStatusForScreenPercentage::Mobile;
	}
	else
	{
		return EViewStatusForScreenPercentage::Desktop;
	}
}

float FEditorViewportClient::GetDefaultPrimaryResolutionFractionTarget() const
{
	FStaticResolutionFractionHeuristic StaticHeuristic;
	StaticHeuristic.Settings.PullEditorRenderingSettings(GetViewStatusForScreenPercentage());

	if (SupportsLowDPIPreview() && IsLowDPIPreview()) // TODO: && ViewFamily.SupportsScreenPercentage())
	{
		StaticHeuristic.SecondaryViewFraction = GetDPIDerivedResolutionFraction();
	}

	StaticHeuristic.TotalDisplayedPixelCount = FMath::Max(Viewport->GetSizeXY().X * Viewport->GetSizeXY().Y, 1);
	StaticHeuristic.DPIScale = GetDPIScale();
	return StaticHeuristic.ResolveResolutionFraction();
}

bool FEditorViewportClient::IsPreviewingScreenPercentage() const
{
	return bIsPreviewingResolutionFraction;
}

void FEditorViewportClient::SetPreviewingScreenPercentage(bool bIsPreviewing)
{
	bIsPreviewingResolutionFraction = bIsPreviewing;
}

int32 FEditorViewportClient::GetPreviewScreenPercentage() const
{
	float ResolutionFraction = 1.0f;
	if (PreviewResolutionFraction.IsSet())
	{
		ResolutionFraction = PreviewResolutionFraction.GetValue();
	}
	else
	{
		ResolutionFraction = GetDefaultPrimaryResolutionFractionTarget();
	}

	// We expose the resolution fraction derived from DPI, to not lie to the artist when screen percentage = 100%.
	return FMath::RoundToInt(FMath::Clamp(
		ResolutionFraction,
		ISceneViewFamilyScreenPercentage::kMinTSRResolutionFraction,
		ISceneViewFamilyScreenPercentage::kMaxTSRResolutionFraction) * 100.0f);
}

void FEditorViewportClient::SetPreviewScreenPercentage(int32 PreviewScreenPercentage)
{
	PreviewResolutionFraction = FMath::Clamp(
		PreviewScreenPercentage / 100.0f,
		ISceneViewFamilyScreenPercentage::kMinTSRResolutionFraction,
		ISceneViewFamilyScreenPercentage::kMaxTSRResolutionFraction);
}

bool FEditorViewportClient::SupportsLowDPIPreview() const
{
	return GetDPIDerivedResolutionFraction() < 1.0f;
}

bool FEditorViewportClient::IsLowDPIPreview() const
{
	if (SceneDPIMode == ESceneDPIMode::EditorDefault)
	{
		return GetDefaultLowDPIPreviewValue();
	}
	return SceneDPIMode == ESceneDPIMode::EmulateLowDPI;
}

void FEditorViewportClient::SetLowDPIPreview(bool LowDPIPreview)
{
	if (LowDPIPreview == GetDefaultLowDPIPreviewValue())
	{
		SceneDPIMode = ESceneDPIMode::EditorDefault;
	}
	else
	{
		SceneDPIMode = LowDPIPreview ? ESceneDPIMode::EmulateLowDPI : ESceneDPIMode::HighDPI;
	}
}

bool FEditorViewportClient::ShouldScaleCameraSpeedByDistance() const
{
	return GetDefault<ULevelEditorViewportSettings>()->bUseDistanceScaledCameraSpeed;
}

bool FEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	return Internal_InputKey(EventArgs);
}

bool FEditorViewportClient::Internal_InputKey(const FInputKeyEventArgs& EventArgs)
{
	if (bDisableInput)
	{
		return true;
	}
	
	const FKey& Key = EventArgs.Key;
	const EInputEvent& Event = EventArgs.Event;
	const FViewport* InViewport = EventArgs.Viewport;

	bool bRouteToToolsContext = !bIsTracking || ModeTools->UsesViewportITF();
	// Let the current mode have a look at the input before reacting to it. 
	// Note that bIsTracking tells us whether the viewport client is capturing mouse behavior to fly around,
	// move objects, etc. In this case we don't want to pass the input the input router, because it is already
	// captured. One may reasonably say that we don't want to pass the input to the modes in this case either, 
	// but unfortunately the legacy behavior is that modes do get this input behavior, which we have to keep
	// for now. In the long term, only old FEdModes will get this input, while the new UEdModes won't, but
	// this is not yet split apart. Also in the long term, the viewport behaviors will be inside the
	// input router and we won't need this bIsTracking flag to begin with.
	if (ModeTools->InputKey(this, Viewport, Key, Event, bRouteToToolsContext))
	{
		RequiredCursorVisibiltyAndAppearance.bDontResetCursor = true;
		UpdateRequiredCursorVisibility();
		ApplyRequiredCursorVisibility(true);
	
		return true;
	}

	if (GetViewportNavigationHelper()->bIsMouseLooking)
	{
		return true;
	}

	const FInputEventState InputState(EventArgs.Viewport, Key, Event);
	
	bool bHandled = false;

	const bool bEnableGizmos = UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(ModeTools.Get());

	if ((IsOrtho() || InputState.IsAltButtonPressed()) && (Key == EKeys::Left || Key == EKeys::Right || Key == EKeys::Up || Key == EKeys::Down) && !bEnableGizmos)
	{
		NudgeSelectedObjects(InputState);

		bHandled = true;
	}
	else if (Key == EKeys::Escape && Event == IE_Pressed && bIsTracking)
	{
		// Pressing Escape cancels the current operation
		AbortTracking();
		bHandled = true;
	}

	// If in ortho and right mouse button and ctrl is pressed
	if (!ModeTools->UsesViewportITF())
	{
		if (!InputState.IsAltButtonPressed()
			&& InputState.IsCtrlButtonPressed()
			&& !InputState.IsButtonPressed(EKeys::LeftMouseButton)
			&& !InputState.IsButtonPressed(EKeys::MiddleMouseButton)
			&& InputState.IsButtonPressed(EKeys::RightMouseButton)
			&& IsOrtho())
		{
			ModeTools->SetWidgetModeOverride(UE::Widget::WM_Rotate);
		}
		else
		{
			ModeTools->SetWidgetModeOverride(UE::Widget::WM_None);
		}
	}

	if (FCachedJoystickState* JoystickState = GetJoystickState(EventArgs.InputDevice.GetId()))
	{
		JoystickState->KeyEventValues.Add(Key, Event);
	}

	const bool bWasCursorVisible = InViewport->IsCursorVisible();
	const bool bWasSoftwareCursorVisible = InViewport->IsSoftwareCursorVisible();

	RequiredCursorVisibiltyAndAppearance.bDontResetCursor = false;
	UpdateRequiredCursorVisibility();

	if( bWasCursorVisible != RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible || bWasSoftwareCursorVisible != RequiredCursorVisibiltyAndAppearance.bSoftwareCursorVisible )
	{
		bHandled = true;
	}


	// Compute a view.
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		InViewport,
		GetScene(),
		EngineShowFlags )
		.SetRealtimeUpdate( IsRealtime() ) );
	FSceneView* View = CalcSceneView( &ViewFamily );


	if (!InputState.IsAnyMouseButtonDown())
	{
		bHasMouseMovedSinceClick = false;
	}

	const bool bIsMultiMouseButtonInteractionEnabled = IsViewportInteractionEnabled(ModeTools.Get(), UE::Editor::ViewportInteractions::CameraDrag);

	// Make sure ITF viewport interactions dealing with multiple mouse buttons pressed at the same time DO NOT trigger StartTrackingDueToInput when a button is released
	if (!bIsMultiMouseButtonInteractionEnabled)
	{
		// Start tracking if any mouse button is down and it was a tracking event (MouseButton/Ctrl/Shift/Alt):
		if ( InputState.IsAnyMouseButtonDown()
			&& (Event == IE_Pressed || Event == IE_Released)
			&& (InputState.IsMouseButtonEvent() || InputState.IsCtrlButtonEvent() || InputState.IsAltButtonEvent() || InputState.IsShiftButtonEvent() ) )
		{
			StartTrackingDueToInput( InputState, *View );
			return true;
		}
	}

	const bool bIsITFSingleClickEnabled = IsViewportInteractionGroupEnabled(ModeTools.Get(), UE::Editor::ViewportInteractions::ViewportClick);

	// If we are tracking and no mouse button is down and this input event released the mouse button stop tracking and process any clicks if necessary
	if ( bIsTracking && !InputState.IsAnyMouseButtonDown() && InputState.IsMouseButtonEvent() && !bIsITFSingleClickEnabled)
	{
		// Handle possible mouse click viewport
		ProcessClickInViewport( InputState, *View );

		// Stop tracking if no mouse button is down
		StopTracking();

		bHandled |= true;
	}


	if ( Event == IE_DoubleClick )
	{
		ProcessDoubleClickInViewport( InputState, *View );
		return true;
	}

	if( ( Key == EKeys::MouseScrollUp || Key == EKeys::MouseScrollDown || Key == EKeys::Add || Key == EKeys::Subtract ) && (Event == IE_Pressed || Event == IE_Repeat ) && IsOrtho() )
	{
		OnOrthoZoom( InputState );
		bHandled |= true;

		if ( Key == EKeys::MouseScrollUp || Key == EKeys::MouseScrollDown )
		{
			FEditorViewportStats::Using(FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_SCROLL);
		}
	}
	else if( ( Key == EKeys::MouseScrollUp || Key == EKeys::MouseScrollDown ) && Event == IE_Pressed && IsPerspective() )
	{
		// If flight camera input is active, then the mouse wheel will control the speed of camera
		// movement
		if( IsFlightCameraInputModeActive() )
		{
			OnChangeCameraSpeed( InputState );
		}
		else
		{
			OnDollyPerspectiveCamera( InputState );

			FEditorViewportStats::Using(FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_SCROLL);
		}

		bHandled |= true;
	}
	else if( IsFlightCameraActive() && Event != IE_Repeat )
	{
		// Flight camera control is active, so simply absorb the key.  The camera will update based
		// on currently pressed keys (Viewport->KeyState) in the Tick function.

		//mark "externally moved" so context menu doesn't come up
		MouseDeltaTracker->SetExternalMovement();
		bHandled |= true;
	}

	//apply the visibility and set the cursor positions
	ApplyRequiredCursorVisibility( true );
	return bHandled;
}


void FEditorViewportClient::StopTracking()
{
	if( bIsTracking && !bIsTrackingBeingStopped )
	{
		bIsTrackingBeingStopped = true;

		DragStartView = nullptr;
		if (DragStartViewFamily != nullptr)
		{
			delete DragStartViewFamily;
			DragStartViewFamily = nullptr;
		}
		MouseDeltaTracker->EndTracking( this );

		Widget->SetCurrentAxis( EAxisList::None );

		// Force an immediate redraw of the viewport and hit proxy.
		// The results are required straight away, so it is not sufficient to defer the redraw until the next tick.
		constexpr bool bForceChildViewportRedraw = true;
		constexpr bool bInvalidateHitProxies = true;
		Invalidate(bForceChildViewportRedraw, bInvalidateHitProxies);

		SetRequiredCursorOverride( false );

		bWidgetAxisControlledByDrag = false;

 		// Update the hovered hit proxy here.  If the user didnt move the mouse
 		// they still need to be able to pick up the gizmo without moving the mouse again
 		HHitProxy* HitProxy = Viewport->GetHitProxy(CachedMouseX,CachedMouseY);
  		CheckHoveredHitProxy(HitProxy);

		bIsTracking = false;
		bIsTrackingBeingStopped = false;
	}

	bHasMouseMovedSinceClick = false;
	bDraggingByHandle = false;
}

void FEditorViewportClient::AbortTracking()
{
	StopTracking();
}

bool FEditorViewportClient::IsInImmersiveViewport() const
{
	return ImmersiveDelegate.IsBound() ? ImmersiveDelegate.Execute() : false;
}

void FEditorViewportClient::StartTrackingDueToInput( const struct FInputEventState& InputState, FSceneView& View )
{
	// Check to see if the current event is a modifier key and that key was already in the
	// same state.
	EInputEvent Event = InputState.GetInputEvent();
	FViewport* InputStateViewport = InputState.GetViewport();
	FKey Key = InputState.GetKey();

	bool bIsRedundantModifierEvent =
		( InputState.IsAltButtonEvent() && ( ( Event != IE_Released ) == IsAltPressed() ) ) ||
		( InputState.IsCtrlButtonEvent() && ( ( Event != IE_Released ) == IsCtrlPressed() ) ) ||
		( InputState.IsShiftButtonEvent() && ( ( Event != IE_Released ) == IsShiftPressed() ) );

	if( MouseDeltaTracker->UsingDragTool() && InputState.IsLeftMouseButtonPressed() && Event != IE_Released )
	{
		bIsRedundantModifierEvent = true;
	}

	const int32	HitX = InputStateViewport->GetMouseX();
	const int32	HitY = InputStateViewport->GetMouseY();


	//First mouse down, note where they clicked
	LastMouseX = HitX;
	LastMouseY = HitY;

	// Only start (or restart) tracking mode if the current event wasn't a modifier key that
	// was already pressed or released.
	if( !bIsRedundantModifierEvent )
	{
		const bool bWasTracking = bIsTracking;

		// Stop current tracking
		if ( bIsTracking )
		{
			MouseDeltaTracker->EndTracking( this );
			bIsTracking = false;
			CheckHoveredHitProxy(Viewport->GetHitProxy(CachedMouseX, CachedMouseY));
		}

		bDraggingByHandle = (Widget && Widget->GetCurrentAxis() != EAxisList::None);

		if( Event == IE_Pressed )
		{
			// Tracking initialization:
			GEditor->MouseMovement = FVector::ZeroVector;
		}

		// Start new tracking. Potentially reset the widget so that StartTracking can pick a new axis.
		if ( Widget && ( !bDraggingByHandle && InputState.IsCtrlButtonPressed() ) )
		{
			bWidgetAxisControlledByDrag = false;
			Widget->SetCurrentAxis( EAxisList::None );
		}
		const bool bNudge = false;
		MouseDeltaTracker->StartTracking( this, HitX, HitY, InputState, bNudge, !bWasTracking );
		bIsTracking = true;

		//if we are using a widget to drag by axis ensure the cursor is correct
		if( bDraggingByHandle == true )
		{
			//reset the flag to say we used a drag modifier	if we are using the widget handle
			if( bWidgetAxisControlledByDrag == false )
			{
				MouseDeltaTracker->ResetUsedDragModifier();
			}

			SetRequiredCursorOverride( true , EMouseCursor::CardinalCross );
		}

		//only reset the initial point when the mouse is actually clicked
		if (InputState.IsAnyMouseButtonDown() && Widget)
		{
			Widget->ResetInitialTranslationOffset();
		}

		//Don't update the cursor visibility if we don't have focus or mouse capture
		if( InputStateViewport->HasFocus() ||  InputStateViewport->HasMouseCapture())
		{
			//Need to call this one more time as the axis variable for the widget has just been updated
			UpdateRequiredCursorVisibility();
		}
	}
	ApplyRequiredCursorVisibility( true );
}



void FEditorViewportClient::ProcessClickInViewport( const FInputEventState& InputState, FSceneView& View )
{
	// Ignore actor manipulation if we're using a tool
	if ( !MouseDeltaTracker->UsingDragTool() )
	{
		EInputEvent Event = InputState.GetInputEvent();
		FViewport* InputStateViewport = InputState.GetViewport();
		FKey Key = InputState.GetKey();

		const int32	HitX = InputStateViewport->GetMouseX();
		const int32	HitY = InputStateViewport->GetMouseY();

		// Calc the raw delta from the mouse to detect if there was any movement
		FVector RawMouseDelta = MouseDeltaTracker->GetRawDelta();

		// Note: We are using raw mouse movement to double check distance moved in low performance situations.  In low performance situations its possible
		// that we would get a mouse down and a mouse up before the next tick where GEditor->MouseMovment has not been updated.
		// In that situation, legitimate drags are incorrectly considered clicks
		bool bNoMouseMovment = RawMouseDelta.SizeSquared() < MOUSE_CLICK_DRAG_DELTA && GEditor->MouseMovement.SizeSquared() < MOUSE_CLICK_DRAG_DELTA;

		// If the mouse haven't moved too far, treat the button release as a click.
		if( bNoMouseMovment && !MouseDeltaTracker->WasExternalMovement() )
		{
			TRefCountPtr<HHitProxy> HitProxy = InputStateViewport->GetHitProxy(HitX,HitY);

			// When clicking, the cursor should always appear at the location of the click and not move out from undere the user
			InputStateViewport->SetPreCaptureMousePosFromSlateCursor();
			ProcessClick(View,HitProxy,Key,Event,HitX,HitY);
		}
	}

}

bool FEditorViewportClient::IsAltPressed() const
{
	return Viewport->KeyState(EKeys::LeftAlt) || Viewport->KeyState(EKeys::RightAlt);
}

bool FEditorViewportClient::IsCtrlPressed() const
{
	return Viewport->KeyState(EKeys::LeftControl) || Viewport->KeyState(EKeys::RightControl);
}

bool FEditorViewportClient::IsShiftPressed() const
{
	return Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);
}

bool FEditorViewportClient::IsCmdPressed() const
{
	return Viewport->KeyState(EKeys::LeftCommand) || Viewport->KeyState(EKeys::RightCommand);
}

bool FEditorViewportClient::IsCommandChordPressed(const TSharedPtr<FUICommandInfo> InCommand, FKey InOptionalKey) const
{
	bool bIsChordPressed = false;
	// Check each bound chord
	for (uint32 i = 0; i < static_cast<uint32>(EMultipleKeyBindingIndex::NumChords); ++i)
	{
		EMultipleKeyBindingIndex ChordIndex = static_cast<EMultipleKeyBindingIndex> (i);
		const FInputChord& Chord = *InCommand->GetActiveChord(ChordIndex);

		bIsChordPressed |= Chord.IsValidChord()
			&& (Chord.NeedsControl() == IsCtrlPressed())
			&& (Chord.NeedsAlt() == IsAltPressed())
			&& (Chord.NeedsShift() == IsShiftPressed())
			&& (Chord.NeedsCommand() == IsCmdPressed())
			&& (InOptionalKey.IsValid() ? (Chord.Key == InOptionalKey) : Viewport->KeyState(Chord.Key));
	}
	return bIsChordPressed;
}

void FEditorViewportClient::RegisterPrioritizedInputChord(const FPrioritizedInputChord& InInputChord)
{
	const int32 PreceedingElementIndex = PrioritizedInputChords.FindLastByPredicate([InInputChord](const FPrioritizedInputChord& Element) { return Element.Priority < InInputChord.Priority; });
	const int32 TargetElementIndex = (PreceedingElementIndex != INDEX_NONE) ? PreceedingElementIndex + 1 : 0;
	PrioritizedInputChords.Insert(InInputChord, TargetElementIndex);
}

void FEditorViewportClient::UnregisterPrioritizedInputChord(const FName InInputChordName)
{
	PrioritizedInputChords.RemoveAll([InInputChordName](const FPrioritizedInputChord& Element) { return Element.Name == InInputChordName; });
}

bool FEditorViewportClient::IsPrioritizedInputChordPressed(const FName InInputChordName) const
{	
	EModifierKey::Type ConsumedModifiers = EModifierKey::None;
	TArray<FKey> ConsumedKeys;

	// Iterate over all chords preceding the argument to determine if any key presses should be consumed before the target chord is evaluated.
	for (const FPrioritizedInputChord& PrioritizedInputChord : PrioritizedInputChords)
	{
		const FInputChord& Chord = PrioritizedInputChord.InputChord;

		bool IsPressed = true;
		IsPressed &= !Chord.Key.IsValid()	|| (!ConsumedKeys.Contains(Chord.Key) && Viewport->KeyState(Chord.Key));
		IsPressed &= !Chord.NeedsControl()	|| (!(ConsumedModifiers & EModifierKey::Control) && IsCtrlPressed());
		IsPressed &= !Chord.NeedsAlt()		|| (!(ConsumedModifiers & EModifierKey::Alt) && IsAltPressed());
		IsPressed &= !Chord.NeedsShift()	|| (!(ConsumedModifiers & EModifierKey::Shift) && IsShiftPressed());
		IsPressed &= !Chord.NeedsCommand()	|| (!(ConsumedModifiers & EModifierKey::Command) && IsCmdPressed());

		if (IsPressed)
		{
			// Mark all of the keys required by this chord as used so that they cannot be considered by any other, lower priority chords.
			if (Chord.Key.IsValid()) { ConsumedKeys.Add(Chord.Key); }
			ConsumedModifiers |= EModifierKey::FromBools(Chord.NeedsControl(), Chord.NeedsAlt(), Chord.NeedsShift(), Chord.NeedsCommand());
		}

		if (PrioritizedInputChord.Name == InInputChordName)
		{
			return IsPressed;
		}
	}	

	return false;
}

void FEditorViewportClient::ProcessDoubleClickInViewport( const struct FInputEventState& InputState, FSceneView& View )
{
	// Stop current tracking
	if ( bIsTracking )
	{
		MouseDeltaTracker->EndTracking( this );
		bIsTracking = false;
	}

	FViewport* InputStateViewport = InputState.GetViewport();
	EInputEvent Event = InputState.GetInputEvent();
	FKey Key = InputState.GetKey();

	const int32	HitX = InputStateViewport->GetMouseX();
	const int32	HitY = InputStateViewport->GetMouseY();

	MouseDeltaTracker->StartTracking( this, HitX, HitY, InputState );
	bIsTracking = true;
	GEditor->MouseMovement = FVector::ZeroVector;
	TRefCountPtr<HHitProxy> HitProxy = InputStateViewport->GetHitProxy(HitX,HitY);
	ProcessClick(View,HitProxy,Key,Event,HitX,HitY);
	MouseDeltaTracker->EndTracking( this );
	bIsTracking = false;

	// This needs to be set to false to allow the axes to update
	bWidgetAxisControlledByDrag = false;
	MouseDeltaTracker->ResetUsedDragModifier();
	SetRequiredCursor(true, false);
	ApplyRequiredCursorVisibility();
}


/** Determines if the new MoveCanvas movement should be used
 * @return - true if we should use the new drag canvas movement.  Returns false for combined object-camera movement and marquee selection
 */
bool FEditorViewportClient::ShouldUseMoveCanvasMovement() const
{
	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) ? true : false;
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton) ? true : false;
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton) ? true : false;
	const bool bMouseButtonDown = (LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown );

	const bool AltDown = IsAltPressed();
	const bool ShiftDown = IsShiftPressed();
	const bool ControlDown = IsCtrlPressed();

	//if we're using the new move canvas mode, we're in an ortho viewport, and the mouse is down
	if (GetDefault<ULevelEditorViewportSettings>()->bPanMovesCanvas && IsOrtho() && bMouseButtonDown)
	{
		//MOVING CAMERA
		if ( !MouseDeltaTracker->UsingDragTool() && AltDown == false && ShiftDown == false && ControlDown == false && (Widget->GetCurrentAxis() == EAxisList::None) && (LeftMouseButtonDown ^ RightMouseButtonDown))
		{
			return true;
		}

		//OBJECT MOVEMENT CODE
		if ( ( AltDown == false && ShiftDown == false && ( LeftMouseButtonDown ^ RightMouseButtonDown ) ) &&
			( ( GetWidgetMode() == UE::Widget::WM_Translate && Widget->GetCurrentAxis() != EAxisList::None ) ||
			( GetWidgetMode() == UE::Widget::WM_TranslateRotateZ && Widget->GetCurrentAxis() != EAxisList::ZRotation &&  Widget->GetCurrentAxis() != EAxisList::None ) ||
			( GetWidgetMode() == UE::Widget::WM_2D && Widget->GetCurrentAxis() != EAxisList::Rotate2D &&  Widget->GetCurrentAxis() != EAxisList::None ) ) )
		{
			return true;
		}


		//ALL other cases hide the mouse
		return false;
	}
	else
	{
		//current system - do not show cursor when mouse is down
		return false;
	}
}

void FEditorViewportClient::DrawAxes(FViewport* InViewport, FCanvas* Canvas, const FRotator* InRotation, EAxisList::Type InAxis)
{
	FMatrix ViewTM = FMatrix::Identity;
	if ( bUsingOrbitCamera)
	{
		FViewportCameraTransform& ViewTransform = GetViewTransform();
		ViewTM = FRotationMatrix(ViewTransform.ComputeOrbitMatrix().InverseFast().Rotator());
	}
	else
	{
		ViewTM = FRotationMatrix( GetViewRotation() );
	}


	if( InRotation )
	{
		ViewTM = FRotationMatrix( *InRotation );
	}

	const int32 SizeY = InViewport->GetSizeXY().Y / Canvas->GetDPIScale();

	const FIntPoint AxisOrigin(45.0f, SizeY - 45.0f);
	const float AxisSize = 25.0f;

	UFont* Font = GEngine->GetSmallFont();
	int32 XL, YL;
	StringSize(Font, XL, YL, TEXT("Z"));

	FVector AxisVec;
	FVector TextVec;
	FIntPoint AxisEnd;
	FIntPoint TextCenter;
	const float TextDistance = 12.0f;
	FCanvasLineItem LineItem;
	LineItem.LineThickness = 1.0f;
	FCanvasTextItem TextItem( FVector2D::ZeroVector, FText::GetEmpty(), Font, FLinearColor::White );
	const FVector2D InsideCenterSize = FVector2D(2, 2);
	FCanvasTileItem InsideTileItem(FVector2D::ZeroVector, InsideCenterSize, FLinearColor(0.2f, 0.2f, 0.2f));
	const FVector2D OutsideCenterSize = FVector2D(4, 4);
	FCanvasTileItem OutsideTileItem(FVector2D::ZeroVector, OutsideCenterSize, FLinearColor(0.05f, 0.05f, 0.05f));
	
	auto GetShadowColor = [](const FLinearColor& Color) -> FLinearColor
		{
			FLinearColor ColorHSV = Color.LinearRGBToHSV();
			ColorHSV.B *= .2f;
			return ColorHSV.HSVToLinearRGB();
		};

	// If LUF isn't being used, it will automatically translate to its correlating XYZ value
	const FLinearColor XBaseColor = AxisDisplayInfo::GetAxisColor(EAxisList::Forward);
	const FLinearColor XShadowColor = GetShadowColor(XBaseColor);
	const FLinearColor YBaseColor = AxisDisplayInfo::GetAxisColor(EAxisList::Left);
	const FLinearColor YShadowColor = GetShadowColor(YBaseColor);
	const FLinearColor ZBaseColor = AxisDisplayInfo::GetAxisColor(EAxisList::Up);
	const FLinearColor ZShadowColor = GetShadowColor(ZBaseColor);

	auto GetShadowOffset = [this](FIntPoint EndPoint, FIntPoint OriginPoint) -> FIntPoint
		{
			int32 X = (FMath::Abs(EndPoint.X - OriginPoint.X) > FMath::Abs(EndPoint.Y - OriginPoint.Y)) ? 0 : 1;
			int32 Y = (FMath::Abs(EndPoint.X - OriginPoint.X) > FMath::Abs(EndPoint.Y - OriginPoint.Y)) ? 1 : 0;
			return FIntPoint(X, Y);
		};
		
	if ((InAxis & EAxisList::X) == EAxisList::X || (InAxis & EAxisList::Forward) == EAxisList::Forward)
	{
		AxisVec = AxisSize * ViewTM.InverseTransformVector( FVector(1,0,0) );
		AxisEnd = AxisOrigin + FIntPoint( AxisVec.Y, -AxisVec.Z );
		TextVec = (AxisSize + TextDistance) * ViewTM.InverseTransformVector(FVector(1, 0, 0));
		TextCenter = AxisOrigin + FIntPoint(TextVec.Y, -TextVec.Z);
		LineItem.SetColor(XShadowColor);
		LineItem.Draw(Canvas, AxisOrigin + GetShadowOffset(AxisEnd, AxisOrigin), AxisEnd + GetShadowOffset(AxisEnd, AxisOrigin));
		LineItem.SetColor(XBaseColor);
		LineItem.Draw( Canvas, AxisOrigin, AxisEnd );
		TextItem.Text = AxisDisplayInfo::GetAxisDisplayNameShort(EAxisList::Forward);
		TextItem.SetColor(XBaseColor);
		TextItem.EnableShadow(XShadowColor);
		TextItem.Draw( Canvas, FVector2D(TextCenter.X - 0.5*XL, TextCenter.Y - 0.5*YL) );
	}

	if ((InAxis & EAxisList::Y) == EAxisList::Y || (InAxis & EAxisList::Left) == EAxisList::Left)
	{
		// If we are using the LUF coordinate system, we need to reflect the Y axis
		if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
		{
			AxisVec = AxisSize * ViewTM.InverseTransformVector(FVector(0, -1, 0));
			TextVec = (AxisSize + TextDistance) * ViewTM.InverseTransformVector(FVector(0, -1, 0));
		}
		else
	{
		AxisVec = AxisSize * ViewTM.InverseTransformVector( FVector(0,1,0) );
			TextVec = (AxisSize + TextDistance) * ViewTM.InverseTransformVector(FVector(0, 1, 0));
		}

		AxisEnd = AxisOrigin + FIntPoint( AxisVec.Y, -AxisVec.Z );
		TextCenter = AxisOrigin + FIntPoint(TextVec.Y, -TextVec.Z);
		LineItem.SetColor(YShadowColor);
		LineItem.Draw(Canvas, AxisOrigin + GetShadowOffset(AxisEnd, AxisOrigin), AxisEnd + GetShadowOffset(AxisEnd, AxisOrigin));
		LineItem.SetColor(YBaseColor);
		LineItem.Draw( Canvas, AxisOrigin, AxisEnd );
		TextItem.Text = AxisDisplayInfo::GetAxisDisplayNameShort(EAxisList::Left);
		TextItem.SetColor(YBaseColor);
		TextItem.EnableShadow(YShadowColor);
		TextItem.Draw( Canvas, FVector2D(TextCenter.X - 0.5*XL, TextCenter.Y - 0.5*YL) );
	}

	if ((InAxis & EAxisList::Z) == EAxisList::Z || (InAxis & EAxisList::Up) == EAxisList::Up)
	{
		AxisVec = AxisSize * ViewTM.InverseTransformVector( FVector(0,0,1) );
		AxisEnd = AxisOrigin + FIntPoint( AxisVec.Y, -AxisVec.Z );
		TextVec = (AxisSize + TextDistance) * ViewTM.InverseTransformVector(FVector(0, 0, 1));
		TextCenter = AxisOrigin + FIntPoint(TextVec.Y, -TextVec.Z);
		LineItem.SetColor(ZShadowColor);
		LineItem.Draw(Canvas, AxisOrigin + GetShadowOffset(AxisEnd, AxisOrigin), AxisEnd + GetShadowOffset(AxisEnd, AxisOrigin));
		LineItem.SetColor(ZBaseColor);
		LineItem.Draw( Canvas, AxisOrigin, AxisEnd );
		TextItem.Text = AxisDisplayInfo::GetAxisDisplayNameShort(EAxisList::Up);
		TextItem.SetColor(ZBaseColor);
		TextItem.EnableShadow(ZShadowColor);
		TextItem.Draw( Canvas, FVector2D(TextCenter.X - 0.5*XL, TextCenter.Y - 0.5*YL) );
	}

	OutsideTileItem.Draw(Canvas, FIntPoint(AxisOrigin.X - FMath::Floor(OutsideCenterSize.X * 0.5), AxisOrigin.Y - FMath::Floor(OutsideCenterSize.Y * 0.5)));
	InsideTileItem.Draw(Canvas, FIntPoint(AxisOrigin.X - FMath::Floor(InsideCenterSize.X * 0.5), AxisOrigin.Y - FMath::Floor(InsideCenterSize.Y * 0.5) ));
}

/** Convert the specified number (in cm or unreal units) into a readable string with relevant si units */
FString FEditorViewportClient::UnrealUnitsToSiUnits(float UnrealUnits)
{
	// Put it in mm to start off with
	UnrealUnits *= 10.f;

	const int32 OrderOfMagnitude = UnrealUnits > 0 ? FMath::TruncToInt(FMath::LogX(10.0f, UnrealUnits)) : 0;

	// Get an exponent applied to anything >= 1,000,000,000mm (1000km)
	const int32 Exponent = (OrderOfMagnitude - 6)  / 3;
	const FString ExponentString = Exponent > 0 ? FString::Printf(TEXT("e+%d"), Exponent*3) : TEXT("");

	float ScaledNumber = UnrealUnits;

	// Factor the order of magnitude into thousands and clamp it to km
	const int32 OrderOfThousands = OrderOfMagnitude / 3;
	if (OrderOfThousands != 0)
	{
		// Scale units to m or km (with the order of magnitude in 1000s)
		ScaledNumber /= FMath::Pow(1000.f, OrderOfThousands);
	}

	// Round to 2 S.F.
	const TCHAR* Approximation = TEXT("");
	{
		const int32 ScaledOrder = OrderOfMagnitude % (FMath::Max(OrderOfThousands, 1) * 3);
		const float RoundingDivisor = FMath::Pow(10.f, ScaledOrder) / 10.f;
		const int32 Rounded = FMath::TruncToInt(ScaledNumber / RoundingDivisor) * RoundingDivisor;
		if (ScaledNumber - Rounded > KINDA_SMALL_NUMBER)
		{
			ScaledNumber = Rounded;
			Approximation = TEXT("~");
		}
	}

	if (OrderOfMagnitude <= 2)
	{
		// Always show cm not mm
		ScaledNumber /= 10;
	}

	static const TCHAR* UnitText[] = { TEXT("cm"), TEXT("m"), TEXT("km") };
	if (FMath::Fmod(ScaledNumber, 1.f) > KINDA_SMALL_NUMBER)
	{
		return FString::Printf(TEXT("%s%.1f%s%s"), Approximation, ScaledNumber, *ExponentString, UnitText[FMath::Min(OrderOfThousands, 2)]);
	}
	else
	{
		return FString::Printf(TEXT("%s%d%s%s"), Approximation, FMath::TruncToInt(ScaledNumber), *ExponentString, UnitText[FMath::Min(OrderOfThousands, 2)]);
	}
}

void FEditorViewportClient::DrawScaleUnits(FViewport* InViewport, FCanvas* Canvas, const FSceneView& InView)
{
	const float UnitsPerPixel = GetOrthoUnitsPerPixel(InViewport) * Canvas->GetDPIScale();

	// Find the closest power of ten to our target width
	static const int32 ApproxTargetMarkerWidthPx = 100;
	const float SegmentWidthUnits = UnitsPerPixel > 0 ? FMath::Pow(10.f, FMath::RoundToFloat(FMath::LogX(10.f, UnitsPerPixel * ApproxTargetMarkerWidthPx))) : 0.f;

	const FString DisplayText = UnrealUnitsToSiUnits(SegmentWidthUnits);

	UFont* Font = GEngine->GetTinyFont();
	int32 TextWidth, TextHeight;
	StringSize(Font, TextWidth, TextHeight, *DisplayText);

	// Origin is the bottom left of the scale
	const FIntPoint StartPoint(130, InViewport->GetSizeXY().Y/Canvas->GetDPIScale() - 45);
	const FIntPoint EndPoint = StartPoint + (UnitsPerPixel != 0 ? FIntPoint(SegmentWidthUnits / UnitsPerPixel, 0) : FIntPoint(0,0));

	// Sort out the color for the text and widget
	FLinearColor HSVBackground = InView.BackgroundColor.LinearRGBToHSV().CopyWithNewOpacity(1.f);
	const int32 Sign = (0.5f - HSVBackground.B) / FMath::Abs(HSVBackground.B - 0.5f);
	HSVBackground.B = HSVBackground.B + Sign*0.4f;
	const FLinearColor SegmentColor = HSVBackground.HSVToLinearRGB();

	const FIntPoint VerticalTickOffset(0, -3);

	// Draw the scale
	FCanvasLineItem LineItem;
	LineItem.SetColor(SegmentColor);
	LineItem.Draw(Canvas, StartPoint, StartPoint + VerticalTickOffset);
	LineItem.Draw(Canvas, StartPoint, EndPoint);
	LineItem.Draw(Canvas, EndPoint, EndPoint + VerticalTickOffset);

	// Draw the text
	FCanvasTextItem TextItem(EndPoint + FIntPoint(-(TextWidth + 3), -TextHeight), FText::FromString(DisplayText), Font, SegmentColor);
	TextItem.Draw(Canvas);
}

void FEditorViewportClient::OnOrthoZoom( const struct FInputEventState& InputState, float Scale )
{
	FViewport* InputStateViewport = InputState.GetViewport();
	FKey Key = InputState.GetKey();

	// Scrolling the mousewheel up/down zooms the orthogonal viewport in/out.
	float NewOrthoRatio = 1 + ((25 * Scale) / CAMERA_ZOOM_DAMPEN);
	if( Key == EKeys::MouseScrollUp || Key == EKeys::Add )
	{
		NewOrthoRatio = 1 / NewOrthoRatio;
	}

	//Extract current state
	int32 ViewportWidth = InputStateViewport->GetSizeXY().X;
	int32 ViewportHeight = InputStateViewport->GetSizeXY().Y;

	FVector OldOffsetFromCenter;

	const bool bCenterZoomAroundCursor = GetDefault<ULevelEditorViewportSettings>()->bCenterZoomAroundCursor && (Key == EKeys::MouseScrollDown || Key == EKeys::MouseScrollUp );

	if (bCenterZoomAroundCursor)
	{
		//Y is actually backwards, but since we're move the camera opposite the cursor to center, we negate both
		//therefore the x is negated
		//X Is backwards, negate it
		//default to viewport mouse position
		int32 CenterX = InputStateViewport->GetMouseX();
		int32 CenterY = InputStateViewport->GetMouseY();
		if (ShouldUseMoveCanvasMovement())
		{
			//use virtual mouse while dragging (normal mouse is clamped when invisible)
			CenterX = LastMouseX;
			CenterY = LastMouseY;
		}
		int32 DeltaFromCenterX = -(CenterX - (ViewportWidth>>1));
		int32 DeltaFromCenterY =  (CenterY - (ViewportHeight>>1));

		switch( GetViewportType() )
		{
		case LVT_OrthoTop:
			OldOffsetFromCenter.Set(-DeltaFromCenterY, -DeltaFromCenterX , 0.0f);
			break;
		case LVT_OrthoLeft:
			OldOffsetFromCenter.Set(DeltaFromCenterX, 0.0f, DeltaFromCenterY);
			break;
		case LVT_OrthoBack:
			OldOffsetFromCenter.Set(0.0f, DeltaFromCenterX, DeltaFromCenterY);
			break;
		case LVT_OrthoBottom:
			OldOffsetFromCenter.Set(-DeltaFromCenterY, DeltaFromCenterX, 0.0f);
			break;
		case LVT_OrthoRight:
			OldOffsetFromCenter.Set(-DeltaFromCenterX, 0.0f, DeltaFromCenterY);
			break;
		case LVT_OrthoFront:
			OldOffsetFromCenter.Set(0.0f, -DeltaFromCenterX, DeltaFromCenterY);
			break;
		case LVT_OrthoFreelook:
			//@TODO: CAMERA: How to handle this
			break;
		case LVT_Perspective:
			break;
		}
	}

	//save off old zoom
	const float OldUnitsPerPixel = GetOrthoUnitsPerPixel(Viewport);

	//update zoom based on input
	const float Zoom = GetOrthoZoom() * NewOrthoRatio;
	SetOrthoZoom( FMath::Clamp<float>( Zoom, GetMinimumOrthoZoom(), MAX_ORTHOZOOM ) );

	if (bCenterZoomAroundCursor)
	{
		//This is the equivalent to moving the viewport to center about the cursor, zooming, and moving it back a proportional amount towards the cursor
		FVector FinalDelta = (GetOrthoUnitsPerPixel(Viewport) - OldUnitsPerPixel)*OldOffsetFromCenter;

		//now move the view location proportionally
		SetViewLocation( GetViewLocation() + FinalDelta );
	}

	const bool bInvalidateViews = true;

	// Update linked ortho viewport movement based on updated zoom and view location,
	UpdateLinkedOrthoViewports( bInvalidateViews );

	const bool bInvalidateHitProxies = true;

	Invalidate( bInvalidateViews, bInvalidateHitProxies );

	//mark "externally moved" so context menu doesn't come up
	MouseDeltaTracker->SetExternalMovement();
}

void FEditorViewportClient::OnDollyPerspectiveCamera( const FInputEventState& InputState )
{
	FKey Key = InputState.GetKey();

	// Scrolling the mousewheel up/down moves the perspective viewport forwards/backwards.
	FVector Drag(0,0,0);

	const FRotator& ViewRotation = GetViewRotation();
	Drag.X = FMath::Cos( ViewRotation.Yaw * PI / 180.f ) * FMath::Cos( ViewRotation.Pitch * PI / 180.f );
	Drag.Y = FMath::Sin( ViewRotation.Yaw * PI / 180.f ) * FMath::Cos( ViewRotation.Pitch * PI / 180.f );
	Drag.Z = FMath::Sin( ViewRotation.Pitch * PI / 180.f );

	if( Key == EKeys::MouseScrollDown )
	{
		Drag = -Drag;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const float CameraSpeed = GetCameraSpeed(GetDefault<ULevelEditorViewportSettings>()->MouseScrollCameraSpeed);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Drag *= CameraSpeed * 32.f;

	const bool bDollyCamera = true;
	MoveViewportCamera( Drag, FRotator::ZeroRotator, bDollyCamera );
	Invalidate( true, true );

	FEditorDelegates::OnDollyPerspectiveCamera.Broadcast(Drag, ViewIndex);
}

void FEditorViewportClient::OnChangeCameraSpeed( const struct FInputEventState& InputState )
{
	FKey Key = InputState.GetKey();

	FEditorViewportCameraSpeedSettings SpeedSettings = GetCameraSpeedSettings();
	
	const float Speed = SpeedSettings.GetCurrentSpeed();
		
	float Delta = Speed * 0.1f;
	if (Key == EKeys::MouseScrollDown)
	{
		Delta *= -1.0f;
	}
	
	SpeedSettings.SetCurrentSpeed(Speed + Delta);
	
	SetCameraSpeedSettings(SpeedSettings);
}

void FEditorViewportClient::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject(ViewportInteraction);

	if( PreviewScene )
	{
		PreviewScene->AddReferencedObjects( Collector );
	}

	if (ViewState.GetReference())
	{
		ViewState.GetReference()->AddReferencedObjects(Collector);
	}

	for (FSceneViewStateReference &StereoViewState : StereoViewStates)
	{
		if (StereoViewState.GetReference())
		{
			StereoViewState.GetReference()->AddReferencedObjects(Collector);
		}
	}
}

FString FEditorViewportClient::GetReferencerName() const
{
	return TEXT("FEditorViewportClient");
}

void FEditorViewportClient::ProcessClick(class FSceneView& View, class HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	const FViewportClick Click(&View, this, Key, Event, HitX, HitY);
	ModeTools->HandleClick(this, HitProxy, Click);
}

bool FEditorViewportClient::InputWidgetDelta(FViewport* InViewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale)
{
	if (ModeTools->InputDelta(this, Viewport, Drag, Rot, Scale))
	{
		if (ModeTools->AllowWidgetMove())
		{
			ModeTools->PivotLocation += Drag;
			ModeTools->SnappedLocation += Drag;
		}

		// Update visuals of the rotate widget
		ApplyDeltaToRotateWidget(Rot);
		return true;
	}
	else
	{
		return false;
	}
}

void FEditorViewportClient::SetWidgetMode(UE::Widget::EWidgetMode NewMode)
{
	// Don't set hit proxies or redraw collapsed viewport widgets
	if (TSharedPtr<SEditorViewport> EditorViewportWidgetPinned = EditorViewportWidget.Pin())
	{
		if (!EditorViewportWidgetPinned->IsVisible() || EditorViewportWidgetPinned->GetVisibility() != EVisibility::Visible)
		{
			return;
		}
	}
	else
	{
		return;
	}

	if (!ModeTools->IsTracking() && !IsFlightCameraActive())
	{
		ModeTools->SetWidgetMode(NewMode);

		// force an invalidation (non-deferred) of the hit proxy here, otherwise we will
		// end up checking against an incorrect hit proxy if the cursor is not moved
		Viewport->InvalidateHitProxy();
		bShouldCheckHitProxy = true;

		// Fire event delegate
		ModeTools->BroadcastWidgetModeChanged(NewMode);
	}

	RedrawAllViewportsIntoThisScene();
}

bool FEditorViewportClient::CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const
{
	return ModeTools->UsesTransformWidget(NewMode) == true;
}

UE::Widget::EWidgetMode FEditorViewportClient::GetWidgetMode() const
{
	return ModeTools->GetWidgetMode();
}

FVector FEditorViewportClient::GetWidgetLocation() const
{
	return ModeTools->GetWidgetLocation();
}

FMatrix FEditorViewportClient::GetWidgetCoordSystem() const
{
	return ModeTools->GetCustomInputCoordinateSystem();
}

FMatrix FEditorViewportClient::GetLocalCoordinateSystem() const
{
	return ModeTools->GetLocalCoordinateSystem();
}

void FEditorViewportClient::SetWidgetCoordSystemSpace(ECoordSystem NewCoordSystem)
{
	ModeTools->SetCoordSystem(NewCoordSystem);
	RedrawAllViewportsIntoThisScene();
}

ECoordSystem FEditorViewportClient::GetWidgetCoordSystemSpace() const
{
	return ModeTools->GetCoordSystem();
}

void FEditorViewportClient::ApplyDeltaToRotateWidget(const FRotator& InRot)
{
	//apply rotation to translate rotate widget
	if (!InRot.IsZero())
	{
		FRotator TranslateRotateWidgetRotation(0, ModeTools->TranslateRotateXAxisAngle, 0);
		TranslateRotateWidgetRotation += InRot;
		ModeTools->TranslateRotateXAxisAngle = TranslateRotateWidgetRotation.Yaw;

		FRotator Widget2DRotation(ModeTools->TranslateRotate2DAngle, 0, 0);
		Widget2DRotation += InRot;
		ModeTools->TranslateRotate2DAngle = Widget2DRotation.Pitch;
	}
}

void FEditorViewportClient::RedrawAllViewportsIntoThisScene()
{
	Invalidate();
}

FSceneInterface* FEditorViewportClient::GetScene() const
{
	UWorld* World = GetWorld();
	if( World )
	{
		return World->Scene;
	}

	return NULL;
}

UWorld* FEditorViewportClient::GetWorld() const
{
	UWorld* OutWorldPtr = NULL;
	// If we have a valid scene get its world
	if( PreviewScene )
	{
		OutWorldPtr = PreviewScene->GetWorld();
	}
	if ( OutWorldPtr == NULL )
	{
		OutWorldPtr = GWorld;
	}
	return OutWorldPtr;
}

void FEditorViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	// Information string
	Canvas.DrawShadowedString(4, 4, *ModeTools->InfoString, GEngine->GetSmallFont(), FColor::White);

	// Render the marquee drag tool
	RenderDragTool(&View, &Canvas);
}

void FEditorViewportClient::SetupViewForRendering(FSceneViewFamily& ViewFamily, FSceneView& View)
{
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		// Wireframe color is emissive-only, and mesh-modifying materials do not use material substitution, hence...
		View.DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
		View.SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
	}
	else if (ViewFamily.EngineShowFlags.OverrideDiffuseAndSpecular)
	{
		View.DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
		View.SpecularOverrideParameter = FVector4f(.1f, .1f, .1f, 0.0f);
	}
	else if (ViewFamily.EngineShowFlags.LightingOnlyOverride)
	{
		View.DiffuseOverrideParameter = FVector4f(GEngine->LightingOnlyBrightness.R, GEngine->LightingOnlyBrightness.G, GEngine->LightingOnlyBrightness.B, 0.0f);
		View.SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
	}
	else if (ViewFamily.EngineShowFlags.ReflectionOverride)
	{
		View.DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
		View.SpecularOverrideParameter = FVector4f(1, 1, 1, 0.0f);
		View.NormalOverrideParameter = FVector4f(0, 0, 1, 0.0f);
		View.RoughnessOverrideParameter = FVector2f(0.0f, 0.0f);
	}

	if (!ViewFamily.EngineShowFlags.Diffuse)
	{
		View.DiffuseOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
	}

	if (!ViewFamily.EngineShowFlags.Specular)
	{
		View.SpecularOverrideParameter = FVector4f(0.f, 0.f, 0.f, 0.f);
	}

	if (!ViewFamily.EngineShowFlags.MaterialNormal)
	{
		View.NormalOverrideParameter = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
	}

	if (!ViewFamily.EngineShowFlags.MaterialAmbientOcclusion)
	{
		View.AmbientOcclusionOverrideParameter = FVector2f(1.0f, 0.0f);
	}

	View.CurrentBufferVisualizationMode = CurrentBufferVisualizationMode;
	View.CurrentNaniteVisualizationMode = CurrentNaniteVisualizationMode;
	View.CurrentLumenVisualizationMode = CurrentLumenVisualizationMode;
	View.CurrentMegaLightsVisualizationMode = CurrentMegaLightsVisualizationMode;
	View.CurrentSubstrateVisualizationMode = CurrentSubstrateVisualizationMode;
	View.CurrentGroomVisualizationMode = CurrentGroomVisualizationMode;
	View.CurrentVirtualShadowMapVisualizationMode = CurrentVirtualShadowMapVisualizationMode;
	View.CurrentVirtualTextureVisualizationMode = CurrentVirtualTextureVisualizationMode;
	View.CurrentGPUSkinCacheVisualizationMode = CurrentGPUSkinCacheVisualizationMode;
	View.CurrentRayTracingDebugVisualizationMode = CurrentRayTracingDebugVisualizationMode;

	// assign wireframe opacity to the view
	GetMeshEdgesViewSettings(View).Opacity = WireframeOpacity;
	
	//Look if the pixel inspector tool is on
	FPixelInspectorModule& PixelInspectorModule = FModuleManager::LoadModuleChecked<FPixelInspectorModule>(TEXT("PixelInspectorModule"));
	bool IsInspectorActive = PixelInspectorModule.IsPixelInspectorEnable();
	FIntPoint InspectViewportPos = FIntPoint(-1, -1);
	if (IsInspectorActive)
	{
		if (CurrentMousePos == FIntPoint(-1, -1))
		{
			uint32 CoordinateViewportId = 0;
			PixelInspectorModule.GetCoordinatePosition(InspectViewportPos, CoordinateViewportId);

			bool IsCoordinateInViewport = InspectViewportPos.X <= Viewport->GetSizeXY().X && InspectViewportPos.Y <= Viewport->GetSizeXY().Y;
			IsInspectorActive = IsCoordinateInViewport && (CoordinateViewportId == View.State->GetViewKey());
			if (IsInspectorActive)
			{
				PixelInspectorModule.SetViewportInformation(View.State->GetViewKey(), Viewport->GetSizeXY());
			}
		}
		else
		{
			InspectViewportPos = CurrentMousePos;
			PixelInspectorModule.SetViewportInformation(View.State->GetViewKey(), Viewport->GetSizeXY());
			PixelInspectorModule.SetCoordinatePosition(InspectViewportPos, false);
		}
	}
	View.bUsePixelInspector = IsInspectorActive;

	if (IsInspectorActive)
	{
		// Ready to send a request
		FSceneInterface *SceneInterface = GetScene();

		FVector2f InspectViewportUV = FVector2f(InspectViewportPos.X + 0.5f, InspectViewportPos.Y + 0.5f);
		InspectViewportUV -= FVector2f(View.UnscaledViewRect.Min); // Offset for aspect ratio constrained viewports
		InspectViewportUV /= FVector2f(View.UnscaledViewRect.Size());

		PixelInspectorModule.CreatePixelInspectorRequest(FDeprecateSlateVector2D(InspectViewportUV), View.State->GetViewKey(), SceneInterface, bInGameViewMode, View.State->GetPreExposure());
	}
	else if (!View.bUsePixelInspector && CurrentMousePos != FIntPoint(-1, -1))
	{
		//Track in case the user hit esc key to stop inspecting pixel
		PixelInspectorRealtimeManagement(this, true);
	}
}

void FEditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FViewport* ViewportBackup = Viewport;
	Viewport = InViewport ? InViewport : Viewport;

	UWorld* World = GetWorld();
	FGameTime Time;
	if (!World || (GetScene() != World->Scene) || UseAppTime()) 
	{
		Time = FGameTime::GetTimeSinceAppStart();
	}
	else
	{
		Time = World->GetTime();
	}

	// Early out if we are changing maps in editor as there is no reason to render the scene and it may not even be valid (For unsaved maps)
	if (World && World->IsPreparingMapChange())
	{
		return;
	}

	// Allow HMD to modify the view later, just before rendering
	const bool bStereoRendering = GEngine->IsStereoscopic3D(InViewport);
	if (FCanvas* DebugCanvas = Viewport->GetDebugCanvas())
	{
		DebugCanvas->SetScaledToRenderTarget(bStereoRendering);
		DebugCanvas->SetStereoRendering(bStereoRendering);
	}
	Canvas->SetScaledToRenderTarget(bStereoRendering);
	Canvas->SetStereoRendering(bStereoRendering);

	FEngineShowFlags UseEngineShowFlags = EngineShowFlags;
	if (OverrideShowFlagsFunc)
	{
		OverrideShowFlagsFunc(UseEngineShowFlags);
	}

	// Setup a FSceneViewFamily/FSceneView for the viewport.
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Canvas->GetRenderTarget(),
		GetScene(),
		UseEngineShowFlags)
		.SetTime(Time)
		.SetRealtimeUpdate( IsRealtime() && FSlateThrottleManager::Get().IsAllowingExpensiveTasks() )
		.SetViewModeParam( ViewModeParam, ViewModeParamName ) );

	ViewFamily.DebugDPIScale = GetDPIScale();

	ViewFamily.EngineShowFlags = UseEngineShowFlags;
	ViewFamily.ChannelMaskParams.ColorChannelMask = ChannelMaskParams.ColorChannelMask;
	ViewFamily.ChannelMaskParams.bInvertAlphaChannelMask = ChannelMaskParams.bInvertAlphaChannelMask;
	ViewFamily.ChannelMaskParams.bDrawAlphaBlendedCheckerboard = ChannelMaskParams.bDrawAlphaBlendedCheckerboard;

	ViewFamily.bIsHDR = Viewport->IsHDRViewport();
	
	// The view is in focus if it is currently in editing
	ViewFamily.SetIsInFocus(GetIsCurrentLevelEditingFocus());

	if( !AllowsCinematicControl() )
	{
		if( !UseEngineShowFlags.Game )
		{
			// in the editor, disable camera motion blur and other rendering features that rely on the former frame
			// unless the view port is cinematic controlled
			ViewFamily.EngineShowFlags.CameraInterpolation = 0;
		}

		if (!bStereoRendering)
		{
			// stereo is enabled, as many HMDs require this for proper visuals
			ViewFamily.EngineShowFlags.SetScreenPercentage(false);
		}
	}

	FSceneViewExtensionContext ViewExtensionContext(InViewport);
	ViewExtensionContext.bStereoEnabled = true;
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);

	for (auto ViewExt : ViewFamily.ViewExtensions)
	{
		ViewExt->SetupViewFamily(ViewFamily);
	}

	EViewModeIndex CurrentViewMode = GetViewMode();
	ViewFamily.ViewMode = CurrentViewMode;

	const bool bVisualizeBufferEnabled = CurrentViewMode == VMI_VisualizeBuffer && CurrentBufferVisualizationMode != NAME_None;
	const bool bRayTracingDebugEnabled = CurrentViewMode == VMI_RayTracingDebug && CurrentRayTracingDebugVisualizationMode != NAME_None;
	const bool bVisualizeGPUSkinCache = CurrentViewMode == VMI_VisualizeGPUSkinCache && CurrentGPUSkinCacheVisualizationMode != NAME_None;
	const bool bCanDisableTonemapper = bVisualizeBufferEnabled || bVisualizeGPUSkinCache || (bRayTracingDebugEnabled && !GetRayTracingVisualizationData().GetModeTonemapped(CurrentRayTracingDebugVisualizationMode));
	const bool bIsGeometryInspectionMode = CurrentViewMode == VMI_Clay || CurrentViewMode == VMI_Zebra || CurrentViewMode == VMI_FrontBackFace || CurrentViewMode == VMI_RandomColor;

	EngineShowFlagOverride(ESFIM_Editor, ViewFamily.ViewMode, ViewFamily.EngineShowFlags, bCanDisableTonemapper);
	EngineShowFlagOrthographicOverride(IsPerspective(), ViewFamily.EngineShowFlags);

	if (bIsGeometryInspectionMode && bIsGeometryInspectionWireframeOn)
	{
		ViewFamily.EngineShowFlags.SetMeshEdges(true);
	}

	UpdateLightingShowFlags( ViewFamily.EngineShowFlags );

	ViewFamily.ExposureSettings = ExposureSettings;

	ViewFamily.LandscapeLODOverride = LandscapeLODOverride;
	// Landscape brushes don't play well with TAA : 
	if (GLandscapeEditModeActive && GetDefault<ULandscapeSettings>()->ShouldDisableTemporalAntiAliasingInLandscapeMode())
	{
		ViewFamily.EngineShowFlags.SetTemporalAA(false);
	}

	// Set the view hit proxies flag as early as possible (but after EngineShowFlagOverride) so that view extensions can check whether they render in the hit proxy pass in SetupView
	if (Canvas->IsHitTesting())
	{
		ViewFamily.EngineShowFlags.SetHitProxies(1);
	}

	// Setup the screen percentage and upscaling method for the view family.
	{
		checkf(ViewFamily.GetScreenPercentageInterface() == nullptr,
			TEXT("Some code has tried to set up an alien screen percentage driver, that could be wrong if not supported very well by the RHI."));

		// If not doing VR rendering, apply DPI derived resolution fraction even if show flag is disabled
		if (!bStereoRendering && SupportsLowDPIPreview() && IsLowDPIPreview() && ViewFamily.SupportsScreenPercentage())
		{
			ViewFamily.SecondaryViewFraction = GetDPIDerivedResolutionFraction();
		}
	}

	// Setup final resolution multiplier used by system relying on the final pixel size. Only used by the high resolution screenshot
	if (GIsHighResScreenshot)
	{
		ViewFamily.FinalResolutionScale = GetHighResScreenshotConfig().ResolutionMultiplier;
	}

	FSceneView* View = nullptr;

	// Stereo rendering
	const bool bStereoDeviceActive = bStereoRendering && GEngine->StereoRenderingDevice.IsValid();
	int32 NumViews = bStereoRendering ? GEngine->StereoRenderingDevice->GetDesiredNumberOfViews(bStereoRendering) : 1;
	for( int StereoViewIndex = 0; StereoViewIndex < NumViews; ++StereoViewIndex )
	{
		View = CalcSceneView( &ViewFamily, bStereoRendering ? StereoViewIndex : INDEX_NONE);

		SetupViewForRendering(ViewFamily,*View);

	    FSlateRect SafeFrame;
	    View->CameraConstrainedViewRect = View->UnscaledViewRect;
	    if (CalculateEditorConstrainedViewRect(SafeFrame, Viewport, Canvas->GetDPIScale()))
	    {
		    View->CameraConstrainedViewRect = FIntRect(SafeFrame.Left, SafeFrame.Top, SafeFrame.Right, SafeFrame.Bottom);
	    }

		if (World)
		{		
			FWorldCachedViewInfo& WorldViewInfo = World->CachedViewInfoRenderedLastFrame.AddDefaulted_GetRef();
			WorldViewInfo.ViewMatrix = View->ViewMatrices.GetWorldToView();
			WorldViewInfo.ProjectionMatrix = View->ViewMatrices.GetViewToClip();
			WorldViewInfo.ViewProjectionMatrix = View->ViewMatrices.GetWorldToClip();
			WorldViewInfo.ViewToWorld = View->ViewMatrices.GetViewToWorld();
			World->LastRenderTime = World->GetTimeSeconds();
		}
 	}

	{
		// If a screen percentage interface was not set by one of the view extension, then set the legacy one.
		if (ViewFamily.GetScreenPercentageInterface() == nullptr)
		{
			float GlobalResolutionFraction = 1.0f;

			// Apply preview resolution fraction. Supported in stereo for VR Editor Mode only
			if ((!bStereoRendering || bInVREditViewMode) && 
				SupportsPreviewResolutionFraction() && ViewFamily.SupportsScreenPercentage())
			{
				if (PreviewResolutionFraction.IsSet() && bIsPreviewingResolutionFraction)
				{
					GlobalResolutionFraction = PreviewResolutionFraction.GetValue();
				}
				else
				{
					GlobalResolutionFraction = GetDefaultPrimaryResolutionFractionTarget();
				}

				// Force screen percentage's engine show flag to be turned on for preview screen percentage.
				ViewFamily.EngineShowFlags.ScreenPercentage = (GlobalResolutionFraction != 1.0);
			}

			// In editor viewport, we ignore r.ScreenPercentage and FPostProcessSettings::ScreenPercentage by design.
			ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
				ViewFamily, GlobalResolutionFraction));
		}

		check(ViewFamily.GetScreenPercentageInterface() != nullptr);
	}

	if (IsAspectRatioConstrained())
	{
		// Clear the background to black if the aspect ratio is constrained, as the scene view won't write to all pixels.
		Canvas->Clear(FLinearColor::Black);
	}

	ViewFamily.bSplitScreenDebugAllowed = true;
	ViewFamily.bIsMainViewFamily = !ViewFamily.EngineShowFlags.Wireframe;			// Prefer non-wireframe views as "main" view family

	// Draw the 3D scene
	GetRendererModule().BeginRenderingViewFamily(Canvas,&ViewFamily);

	if (View)
	{
		DrawCanvas( *Viewport, *View, *Canvas );

		// Draw any HUD from modes
		ModeTools->DrawHUD(this, Viewport, View, Canvas);

		DrawSafeFrames(*Viewport, *View, *Canvas);
	}

	// Remove temporary debug lines.
	// Possibly a hack. Lines may get added without the scene being rendered etc.
	if (World)
	{
		constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
		World->FlushLineBatchers(LineBatchersToFlush);
	}

	// Draw the widget.
	if (Widget && bShowWidget)
	{
		Widget->DrawHUD( Canvas );
	}

	// Axes indicators
	const bool bShouldDrawAxes = (bDrawAxes && !ViewFamily.EngineShowFlags.Game) || (bDrawAxesGame && ViewFamily.EngineShowFlags.Game);
	const bool bIsViewportUIHidden = ModeTools && ModeTools->IsViewportUIHidden();
	const EAxisList::Type AxisDisplayCoordinateSystem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem();

	auto DrawAxesInOrthoViewport = [this, AxisDisplayCoordinateSystem, Canvas, View](FRotator Rotator, EAxisList::Type AxisType)
		{
			DrawAxes(Viewport, Canvas, &Rotator, AxisType);

			if (View)
			{
				DrawScaleUnits(Viewport, Canvas, *View);
			}
		};

	if (bShouldDrawAxes && !bIsViewportUIHidden && !IsVisualizeCalibrationMaterialEnabled())
	{
		switch (GetViewportType())
		{
		case LVT_OrthoXY:
			{
				DrawAxesInOrthoViewport(FRotator(-90.0f, -90.0f, -90.0f), EAxisList::LF);
				break;
			}
		case LVT_OrthoXZ:
			{
				DrawAxesInOrthoViewport(FRotator(0.0f, -90.0f, 0.0f), EAxisList::UF);
				break;
			}
		case LVT_OrthoYZ:
			{
				DrawAxesInOrthoViewport(FRotator(0.0f, 0.0f, 0.0f), EAxisList::LU);
				break;
			}
		case LVT_OrthoNegativeXY:
			{
				DrawAxesInOrthoViewport(FRotator(90.0f, 90.0f, 90.0f), EAxisList::LF);
				break;
			}
		case LVT_OrthoNegativeXZ:
			{
				DrawAxesInOrthoViewport(FRotator(0.0f, 90.0f, 0.0f), EAxisList::UF);
				break;
			}
		case LVT_OrthoNegativeYZ:
			{
				DrawAxesInOrthoViewport(FRotator(0.0f, 180.0f, 0.0f), EAxisList::LU);
				break;
			}
		default:
			{
				DrawAxes(Viewport, Canvas, nullptr, AxisDisplayCoordinateSystem);
				break;
			}
		}
	}

	if (FCanvas* DebugCanvas = Viewport->GetDebugCanvas())
	{
		// NOTE: DebugCanvasObject will be created by UDebugDrawService::Draw() if it doesn't already exist.
		UDebugDrawService::Draw(ViewFamily.EngineShowFlags, Viewport, View, DebugCanvas);
		UCanvas* DebugCanvasObject = FindObjectChecked<UCanvas>(GetTransientPackage(), TEXT("DebugCanvasObject"));
		DebugCanvasObject->Canvas = DebugCanvas;
		DebugCanvasObject->Init(Viewport->GetSizeXY().X, Viewport->GetSizeXY().Y, View, DebugCanvas);

		// Stats display
		if( IsRealtime() && ShouldShowStats() && DebugCanvas)
		{
			TArray< FDebugDisplayProperty > EmptyPropertyArray;
			DrawStatsHUD( World, Viewport, DebugCanvas, NULL, EmptyPropertyArray, GetViewLocation(), GetViewRotation() );
		}
	}

	Viewport = ViewportBackup;
}

void FEditorViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	// Draw the drag tool.
	MouseDeltaTracker->Render3DDragTool( View, PDI );

	// Draw the widget.
	if (Widget && bShowWidget)
	{
		Widget->Render( View, PDI, this );
	}

	if( bUsesDrawHelper )
	{
		DrawHelper.Draw( View, PDI );
	}

	ModeTools->DrawActiveModes(View, PDI);

	// Draw the current editor mode.
	ModeTools->Render(View, Viewport, PDI);

	// Draw the preview scene light visualization
	DrawPreviewLightVisualization(View, PDI);

	// This viewport was just rendered, reset this value.
	FramesSinceLastDraw = 0;
}

void FEditorViewportClient::DrawPreviewLightVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	// Draw the indicator of the current light direction if it was recently moved
	if ((PreviewScene != nullptr) && (PreviewScene->DirectionalLight != nullptr) && (MovingPreviewLightTimer > 0.0f))
	{
		const float A = MovingPreviewLightTimer / PreviewLightConstants::MovingPreviewLightTimerDuration;

		ULightComponent* Light = PreviewScene->DirectionalLight;

		const FLinearColor ArrowColor = Light->LightColor;

		// Figure out where the light is (ignoring position for directional lights)
		const FTransform LightLocalToWorldRaw = Light->GetComponentToWorld();
		FTransform LightLocalToWorld = LightLocalToWorldRaw;
		if (Light->IsA(UDirectionalLightComponent::StaticClass()))
		{
			LightLocalToWorld.SetTranslation(FVector::ZeroVector);
		}
		LightLocalToWorld.SetScale3D(FVector(1.0f));

		// Project the last mouse position during the click into world space
		FVector LastMouseWorldPos;
		FVector LastMouseWorldDir;
		View->DeprojectFVector2D(MovingPreviewLightSavedScreenPos, /*out*/ LastMouseWorldPos, /*out*/ LastMouseWorldDir);

		// The world pos may be nuts due to a super distant near plane for orthographic cameras, so find the closest
		// point to the origin along the ray
		LastMouseWorldPos = FMath::ClosestPointOnLine(LastMouseWorldPos, LastMouseWorldPos + LastMouseWorldDir * WORLD_MAX, FVector::ZeroVector);

		// Figure out the radius to draw the light preview ray at
		const FVector LightToMousePos = LastMouseWorldPos - LightLocalToWorld.GetTranslation();
		const float LightToMouseRadius = FMath::Max<FVector::FReal>(LightToMousePos.Size(), PreviewLightConstants::MinMouseRadius);

		const float ArrowLength = FMath::Max(PreviewLightConstants::MinArrowLength, LightToMouseRadius * PreviewLightConstants::MouseLengthToArrowLengthRatio);
		const float ArrowSize = PreviewLightConstants::ArrowLengthToSizeRatio * ArrowLength;
		const float ArrowThickness = FMath::Max(PreviewLightConstants::ArrowLengthToThicknessRatio * ArrowLength, PreviewLightConstants::MinArrowThickness);

		const FVector ArrowOrigin = LightLocalToWorld.TransformPosition(FVector(-LightToMouseRadius - 0.5f * ArrowLength, 0.0f, 0.0f));
		const FVector ArrowDirection = LightLocalToWorld.TransformVector(FVector(-1.0f, 0.0f, 0.0f));

		const FQuatRotationTranslationMatrix ArrowToWorld(LightLocalToWorld.GetRotation(), ArrowOrigin);

		DrawDirectionalArrow(PDI, ArrowToWorld, ArrowColor, ArrowLength, ArrowSize, SDPG_World, ArrowThickness);
	}
}

void FEditorViewportClient::RenderDragTool(const FSceneView* View, FCanvas* Canvas)
{
	MouseDeltaTracker->RenderDragTool(View, Canvas);
}

FLinearColor FEditorViewportClient::GetBackgroundColor() const
{
	return PreviewScene ? PreviewScene->GetBackgroundColor() : FColor(55, 55, 55);
}

void FEditorViewportClient::SetCameraSetup(
	const FVector& LocationForOrbiting,
	const FRotator& InOrbitRotation,
	const FVector& InOrbitZoom,
	const FVector& InOrbitLookAt,
	const FVector& InViewLocation,
	const FRotator &InViewRotation )
{
	if( bUsingOrbitCamera )
	{
		SetViewRotation( InOrbitRotation );
		SetViewLocation( InViewLocation + InOrbitZoom );
		SetLookAtLocation( InOrbitLookAt );
	}
	else
	{
		SetViewLocation( InViewLocation );
		SetViewRotation( InViewRotation );
	}


	// Save settings for toggling between orbit and unlocked camera
	DefaultOrbitLocation = InViewLocation;
	DefaultOrbitRotation = InOrbitRotation;
	DefaultOrbitZoom = InOrbitZoom;
	DefaultOrbitLookAt = InOrbitLookAt;
}

// Determines which axis InKey and InDelta most refer to and returns
// a corresponding FVector.  This vector represents the mouse movement
// translated into the viewports/widgets axis space.
//
// @param InNudge		If 1, this delta is coming from a keyboard nudge and not the mouse

FVector FEditorViewportClient::TranslateDelta( FKey InKey, float InDelta, bool InNudge )
{
	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

	FVector vec(0.0f, 0.0f, 0.0f);

	float X = InKey == EKeys::MouseX ? InDelta : 0.f;
	float Y = InKey == EKeys::MouseY ? InDelta : 0.f;

	switch( GetViewportType() )
	{
	case LVT_OrthoXY:
	case LVT_OrthoXZ:
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeXY:
	case LVT_OrthoNegativeXZ:
	case LVT_OrthoNegativeYZ:
		{
			LastMouseX += X;
			LastMouseY -= Y;

			if ((X != 0.0f) || (Y!=0.0f))
			{
				MarkMouseMovedSinceClick();
			}

			//only invert x,y if we're moving the camera
			if( ShouldUseMoveCanvasMovement() )
			{
				if(Widget->GetCurrentAxis() == EAxisList::None)
				{
					X = -X;
					Y = -Y;
				}
			}

			//update the position
			Viewport->SetSoftwareCursorPosition( FVector2D( LastMouseX, LastMouseY ) );
			//UE_LOG(LogEditorViewport, Log, *FString::Printf( TEXT("can:%d %d") , LastMouseX , LastMouseY ));
			//change to grab hand
			SetRequiredCursorOverride( true , EMouseCursor::CardinalCross );
			//update and apply cursor visibility
			UpdateAndApplyCursorVisibility();

			UE::Widget::EWidgetMode WidgetMode = GetWidgetMode();
			bool bIgnoreOrthoScaling = (WidgetMode == UE::Widget::WM_Scale) && (Widget->GetCurrentAxis() != EAxisList::None);

			if( InNudge || bIgnoreOrthoScaling )
			{
				vec = FVector( X, Y, 0.f );
			}
			else
			{
				const float UnitsPerPixel = GetOrthoUnitsPerPixel(Viewport);
				vec = FVector( X * UnitsPerPixel, Y * UnitsPerPixel, 0.f );

				if( Widget->GetCurrentAxis() == EAxisList::None )
				{
					switch( GetViewportType() )
					{
					case LVT_OrthoTop:
						vec = FVector (-Y * UnitsPerPixel, -X * UnitsPerPixel, 0.0f);
						break;
					case LVT_OrthoLeft:
						vec = FVector(X * UnitsPerPixel, 0.f, Y * UnitsPerPixel);
						break;
					case LVT_OrthoBack:
						vec = FVector(0.f, X * UnitsPerPixel, Y * UnitsPerPixel);
						break;
 					case LVT_OrthoBottom:
 						vec = FVector(-Y * UnitsPerPixel, X * UnitsPerPixel, 0.0f);
 						break;
					case LVT_OrthoRight:
						vec = FVector(-X * UnitsPerPixel, 0.f, Y * UnitsPerPixel);
						break;
					case LVT_OrthoFront:
						vec = FVector(0.f, -X * UnitsPerPixel, Y * UnitsPerPixel);
						break;
					case LVT_OrthoFreelook:
					case LVT_Perspective:
						break;
					}
				}
			}
		}
		break;

	case LVT_OrthoFreelook://@TODO: CAMERA: Not sure what to do here
	case LVT_Perspective:
		// Update the software cursor position
		Viewport->SetSoftwareCursorPosition( FVector2D(Viewport->GetMouseX() , Viewport->GetMouseY() ) );
		vec = FVector( X, Y, 0.f );
		break;

	default:
		check(0);		// Unknown viewport type
		break;
	}

	if( IsOrtho() && ((LeftMouseButtonDown || bIsUsingTrackpad) && RightMouseButtonDown) && Y != 0.f )
	{
		vec = FVector(0,0,Y);
	}

	return vec;
}

bool FEditorViewportClient::InputAxis(const FInputKeyEventArgs& Args)
{
	return Internal_InputAxis(
		Args.Viewport,
		Args.InputDevice,
		Args.Key,
		Args.AmountDepressed,
		Args.DeltaTime,
		Args.NumSamples,
		Args.IsGamepad());
}

bool FEditorViewportClient::Internal_InputAxis(FViewport* InViewport, FInputDeviceId DeviceID, FKey Key, float Delta, float DeltaTime, int32 NumSamples, bool bGamepad)
{
	if (bDisableInput)
	{
		return true;
	}

	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(DeviceID);
	
	// Let the current mode have a look at the input before reacting to it.
	if (ModeTools->InputAxis(this, Viewport, FGenericPlatformMisc::GetUserIndexForPlatformUser(UserId), Key, Delta, DeltaTime))
	{
		return true;
	}

	// No need to process anything else here, since some ITF Viewport interaction is handling the camera
	if (GetViewportNavigationHelper()->bIsMouseLooking)
	{
		return true;
	}

	const bool bMouseButtonDown = InViewport->KeyState( EKeys::LeftMouseButton ) || InViewport->KeyState( EKeys::MiddleMouseButton ) || InViewport->KeyState( EKeys::RightMouseButton );
	const bool bLightMoveDown = InViewport->KeyState(EKeys::L);

	// Look at which axis is being dragged and by how much
	const float DragX = (Key == EKeys::MouseX) ? Delta : 0.f;
	const float DragY = (Key == EKeys::MouseY) ? Delta : 0.f;

	static FName LightRotationViewportInteractionName("LightRotation");
	if( bLightMoveDown && bMouseButtonDown && PreviewScene && !IsViewportInteractionActive(ModeTools.Get(), LightRotationViewportInteractionName) )
	{
		// Adjust the preview light direction
		FRotator LightDir = PreviewScene->GetLightDirection();

		LightDir.Yaw += -DragX * EditorViewportClient::LightRotSpeed;
		LightDir.Pitch += -DragY * EditorViewportClient::LightRotSpeed;

		PreviewScene->SetLightDirection( LightDir );

		// Remember that we adjusted it for the visualization
		MovingPreviewLightTimer = PreviewLightConstants::MovingPreviewLightTimerDuration;
		MovingPreviewLightSavedScreenPos = FVector2D(LastMouseX, LastMouseY);

		Invalidate();
	}
	else
	{
		/**Save off axis commands for future camera work*/
		FCachedJoystickState* JoystickState = GetJoystickState(DeviceID.GetId());
		if (JoystickState)
		{
			JoystickState->AxisDeltaValues.Add(Key, Delta);
		}

		if( bIsTracking	)
		{
			// Accumulate and snap the mouse movement since the last mouse button click.
			MouseDeltaTracker->AddDelta( this, Key, Delta, 0 );
		}
	}

	// If we are using a drag tool, paint the viewport so we can see it update.
	if( MouseDeltaTracker->UsingDragTool() )
	{
		Invalidate( false, false );
	}

	return true;
}

void FEditorViewportClient::AddAssociation(FViewport& InViewport)
{
	if (Viewport == nullptr)
	{
		Viewport = &InViewport;
	}
}

void FEditorViewportClient::RemoveAssociation(FViewport& InViewport)
{
	if (Viewport == &InViewport)
	{
		Viewport = nullptr;
	}
}

static float AdjustGestureCameraRotation(float Delta, float AdjustLimit, float DeltaCutoff)
{
	const float AbsDelta = FMath::Abs(Delta);
	const float Scale = AbsDelta * (1.0f / AdjustLimit);
	if (AbsDelta > 0.0f && AbsDelta <= AdjustLimit)
	{
		return Delta * Scale;
	}
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();
	return bIsUsingTrackpad ? Delta : FMath::Clamp(Delta, -DeltaCutoff, DeltaCutoff);
}

bool FEditorViewportClient::InputGesture(FViewport* InViewport, const FInputDeviceId DeviceId, EGestureEvent GestureType, EGesturePhase GesturePhase, const FVector2D& GestureDelta, bool bIsDirectionInvertedFromDevice, const uint64 Timestamp)
{
	if (bDisableInput)
	{
		return true;
	}

	const FRotator& ViewRotation = GetViewRotation();

	const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = InViewport->KeyState(EKeys::RightMouseButton);

	const ELevelViewportType LevelViewportType = GetViewportType();

	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

	switch (LevelViewportType)
	{
	case LVT_OrthoXY:
	case LVT_OrthoXZ:
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeXY:
	case LVT_OrthoNegativeXZ:
	case LVT_OrthoNegativeYZ:
		{
			if (GestureType == EGestureEvent::Scroll && !LeftMouseButtonDown && !RightMouseButtonDown)
			{
				const float UnitsPerPixel = GetOrthoUnitsPerPixel(InViewport);

				const EScrollGestureDirection DirectionSetting = GetDefault<ULevelEditorViewportSettings>()->ScrollGestureDirectionForOrthoViewports;
				const bool bUseDirectionInvertedFromDevice = DirectionSetting == EScrollGestureDirection::Natural || (DirectionSetting == EScrollGestureDirection::UseSystemSetting && bIsDirectionInvertedFromDevice);

				// GestureDelta is in window pixel coords.  Adjust for ortho units.
				const FVector2D AdjustedGestureDelta = (bUseDirectionInvertedFromDevice == bIsDirectionInvertedFromDevice ? GestureDelta : -GestureDelta) * UnitsPerPixel;

				switch (LevelViewportType)
				{
					case LVT_OrthoXY:
						CurrentGestureDragDelta += FVector(-AdjustedGestureDelta.X, -AdjustedGestureDelta.Y, 0);
						break;
					case LVT_OrthoXZ:
						CurrentGestureDragDelta += FVector(-AdjustedGestureDelta.X, 0, AdjustedGestureDelta.Y);
						break;
					case LVT_OrthoYZ:
						CurrentGestureDragDelta += FVector(0, -AdjustedGestureDelta.X, AdjustedGestureDelta.Y);
						break;
					case LVT_OrthoNegativeXY:
						CurrentGestureDragDelta += FVector(AdjustedGestureDelta.X, -AdjustedGestureDelta.Y, 0);
						break;
					case LVT_OrthoNegativeXZ:
						CurrentGestureDragDelta += FVector(AdjustedGestureDelta.X, 0, AdjustedGestureDelta.Y);
						break;
					case LVT_OrthoNegativeYZ:
						CurrentGestureDragDelta += FVector(0, AdjustedGestureDelta.X, AdjustedGestureDelta.Y);
						break;
					case LVT_OrthoFreelook:
					case LVT_Perspective:
						break;
				}

				FEditorViewportStats::Used(FEditorViewportStats::CAT_ORTHOGRAPHIC_GESTURE_SCROLL);
			}
			else if (GestureType == EGestureEvent::Magnify)
			{
				OnOrthoZoom(FInputEventState(InViewport, EKeys::MouseScrollDown, IE_Released), -10.0f * GestureDelta.X);
				FEditorViewportStats::Used(FEditorViewportStats::CAT_ORTHOGRAPHIC_GESTURE_MAGNIFY);
			}
		}
		break;

	case LVT_Perspective:
	case LVT_OrthoFreelook:
		{
			if (GestureType == EGestureEvent::Scroll)
			{
				const EScrollGestureDirection DirectionSetting = GetDefault<ULevelEditorViewportSettings>()->ScrollGestureDirectionFor3DViewports;
				const bool bUseDirectionInvertedFromDevice = DirectionSetting == EScrollGestureDirection::Natural || (DirectionSetting == EScrollGestureDirection::UseSystemSetting && bIsDirectionInvertedFromDevice);
				const FVector2D AdjustedGestureDelta = bUseDirectionInvertedFromDevice == bIsDirectionInvertedFromDevice ? GestureDelta : -GestureDelta;

				if( LeftMouseButtonDown )
				{
					// Pan left/right/up/down

					CurrentGestureDragDelta.X += AdjustedGestureDelta.X * -FMath::Sin( ViewRotation.Yaw * PI / 180.f );
					CurrentGestureDragDelta.Y += AdjustedGestureDelta.X *  FMath::Cos( ViewRotation.Yaw * PI / 180.f );
					CurrentGestureDragDelta.Z += -AdjustedGestureDelta.Y;
				}
				else
				{
					// Change viewing angle

					CurrentGestureRotDelta.Yaw += AdjustGestureCameraRotation( AdjustedGestureDelta.X, 20.0f, 35.0f ) * -0.35f;
					CurrentGestureRotDelta.Pitch += AdjustGestureCameraRotation( AdjustedGestureDelta.Y, 20.0f, 35.0f ) * 0.35f;
				}

				FEditorViewportStats::Used(FEditorViewportStats::CAT_ORTHOGRAPHIC_GESTURE_SCROLL);
			}
			else if (GestureType == EGestureEvent::Magnify)
			{
				GestureMoveForwardBackwardImpulse = GestureDelta.X * 4.0f;
			}
		}
		break;

	default:
		// Not a 3D viewport receiving this gesture.  Could be a canvas window.  Bail out.
		return false;
	}

	//mark "externally moved" so context menu doesn't come up
	MouseDeltaTracker->SetExternalMovement();

	return true;
}

void FEditorViewportClient::UpdateGestureDelta()
{
	if( CurrentGestureDragDelta != FVector::ZeroVector || CurrentGestureRotDelta != FRotator::ZeroRotator )
	{
		MoveViewportCamera( CurrentGestureDragDelta, CurrentGestureRotDelta, false );

		Invalidate( true, true );

		CurrentGestureDragDelta = FVector::ZeroVector;
		CurrentGestureRotDelta = FRotator::ZeroRotator;
	}
}

// Converts a generic movement delta into drag/rotation deltas based on the viewport and keys held down
void FEditorViewportClient::ConvertMovementToDragRot(const FVector& InDelta,
													 FVector& InDragDelta,
													 FRotator& InRotDelta) const
{
	const FRotator& ViewRotation = GetViewRotation();

	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

	InDragDelta = FVector::ZeroVector;
	InRotDelta = FRotator::ZeroRotator;

	switch( GetViewportType() )
	{
	case LVT_OrthoXY:
	case LVT_OrthoXZ:
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeXY:
	case LVT_OrthoNegativeXZ:
	case LVT_OrthoNegativeYZ:
		{
			if( ( LeftMouseButtonDown || bIsUsingTrackpad ) && RightMouseButtonDown )
			{
				// Both mouse buttons change the ortho viewport zoom.
				InDragDelta = FVector(0,0,InDelta.Z);
			}
			else if( RightMouseButtonDown )
			{
				// @todo: set RMB to move opposite to the direction of drag, in other words "grab and pull".
				InDragDelta = InDelta;
			}
			else if( LeftMouseButtonDown )
			{
				// LMB moves in the direction of the drag.
				InDragDelta = InDelta;
			}
		}
		break;

	case LVT_Perspective:
	case LVT_OrthoFreelook:
		{
			const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

			if( LeftMouseButtonDown && !RightMouseButtonDown )
			{
				// Move forward and yaw

				InDragDelta.X = InDelta.Y * FMath::Cos( ViewRotation.Yaw * PI / 180.f );
				InDragDelta.Y = InDelta.Y * FMath::Sin( ViewRotation.Yaw * PI / 180.f );

				InRotDelta.Yaw = InDelta.X * ViewportSettings->MouseSensitivty;
			}
			else if( MiddleMouseButtonDown || bIsUsingTrackpad || ( ( LeftMouseButtonDown || bIsUsingTrackpad ) && RightMouseButtonDown ) )
			{
				// Pan left/right/up/down
				const bool bInvert = !bIsUsingTrackpad && MiddleMouseButtonDown && GetDefault<ULevelEditorViewportSettings>()->bInvertMiddleMousePan;

				// Treat the middle mouse pan as a distinct gesture from the left+right mouse button 
				if (MiddleMouseButtonDown && !ViewportSettings->bWorldSpaceVerticalPan)
				{
					// Pan relative to the camera
					InDragDelta = FQuat(ViewRotation) * FVector( 0.0f, InDelta.X, InDelta.Y) * (bInvert ? -1.0f : 1.0f);
				}
				else
				{
					float Direction = bInvert ? 1 : -1;
					InDragDelta.X = InDelta.X * Direction * FMath::Sin( ViewRotation.Yaw * PI / 180.f );
					InDragDelta.Y = InDelta.X * -Direction * FMath::Cos( ViewRotation.Yaw * PI / 180.f );
					InDragDelta.Z = -Direction * InDelta.Y;
				}
			}
			else if( RightMouseButtonDown && !LeftMouseButtonDown )
			{
				// Change viewing angle

				// inverting orbit axis is handled elsewhere
				const bool bInvertY = !ShouldOrbitCamera() && GetDefault<ULevelEditorViewportSettings>()->bInvertMouseLookYAxis;
				float Direction = bInvertY ? -1 : 1;

				InRotDelta.Yaw = InDelta.X * ViewportSettings->MouseSensitivty;
				InRotDelta.Pitch = InDelta.Y * ViewportSettings->MouseSensitivty * Direction;
			}
		}
		break;

	default:
		check(0);	// unknown viewport type
		break;
	}
}

void FEditorViewportClient::ConvertMovementToOrbitDragRot(const FVector& InDelta,
																 FVector& InDragDelta,
																 FRotator& InRotDelta) const
{
	const FRotator& ViewRotation = GetViewRotation();

	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton);
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

	InDragDelta = FVector::ZeroVector;
	InRotDelta = FRotator::ZeroRotator;

	const float YawRadians = FMath::DegreesToRadians( ViewRotation.Yaw );

	switch( GetViewportType() )
	{
	case LVT_OrthoXY:
	case LVT_OrthoXZ:
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeXY:
	case LVT_OrthoNegativeXZ:
	case LVT_OrthoNegativeYZ:
		{
			if( ( LeftMouseButtonDown || bIsUsingTrackpad ) && RightMouseButtonDown )
			{
				// Change ortho zoom.
				InDragDelta = FVector(0,0,InDelta.Z);
			}
			else if( RightMouseButtonDown )
			{
				// Move camera.
				InDragDelta = InDelta;
			}
			else if( LeftMouseButtonDown )
			{
				// Move actors.
				InDragDelta = InDelta;
			}
		}
		break;

	case LVT_Perspective:
		{
			const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

			if( IsOrbitRotationMode( Viewport ) )
			{
				const bool bInvertY = GetDefault<ULevelEditorViewportSettings>()->bInvertOrbitYAxis;
				float Direction = bInvertY ? -1 : 1;

				// Change the viewing angle
				InRotDelta.Yaw = InDelta.X * ViewportSettings->MouseSensitivty;
				InRotDelta.Pitch = InDelta.Y * ViewportSettings->MouseSensitivty * Direction;
			}
			else if( IsOrbitPanMode( Viewport ) )
			{
				// Pan left/right/up/down
				InDragDelta.X = InDelta.X * -FMath::Sin( YawRadians );
				InDragDelta.Y = InDelta.X *  FMath::Cos( YawRadians );
				InDragDelta.Z = InDelta.Y;
			}
			else if( IsOrbitZoomMode( Viewport ) )
			{
				// Zoom in and out.
				InDragDelta.X = InDelta.Y * FMath::Cos( YawRadians );
				InDragDelta.Y = InDelta.Y* FMath::Sin( YawRadians );
			}
		}
		break;

	default:
		check(0);	// unknown viewport type
		break;
	}
}

bool FEditorViewportClient::ShouldPanOrDollyCamera() const
{
	const bool bIsCtrlDown = IsCtrlPressed();

	const bool bLeftMouseButtonDown = Viewport->KeyState( EKeys::LeftMouseButton ) && !bLockFlightCamera;
	const bool bRightMouseButtonDown = Viewport->KeyState( EKeys::RightMouseButton );
	const bool bIsMarqueeSelect = IsOrtho() && bLeftMouseButtonDown;

	const bool bOrthoRotateObjectMode = IsOrtho() && IsCtrlPressed() && bRightMouseButtonDown && !bLeftMouseButtonDown;
	// Pan the camera if not marquee selecting or the left and right mouse buttons are down
	return !bOrthoRotateObjectMode && !bIsCtrlDown && (!bIsMarqueeSelect || (bLeftMouseButtonDown && bRightMouseButtonDown) );
}

TSharedPtr<FDragTool> FEditorViewportClient::MakeDragTool(EDragTool::Type)
{
	return MakeShareable( new FDragTool(GetModeTools()) );
}

bool FEditorViewportClient::CanUseDragTool() const
{
	return !ShouldOrbitCamera() && (GetCurrentWidgetAxis() == EAxisList::None) && ((ModeTools == nullptr) || ModeTools->AllowsViewportDragTool());
}

bool FEditorViewportClient::ShouldOrbitCamera() const
{
	if( bCameraLock )
	{
		return true;
	}
	else
	{
		bool bDesireOrbit = false;

		if (!GetDefault<ULevelEditorViewportSettings>()->bUseUE3OrbitControls)
		{
			bDesireOrbit = (IsAltPressed() && !ModeTools->HasOngoingTransform()) && !IsCtrlPressed() && !IsShiftPressed();
		}
		else
		{
			bDesireOrbit = Viewport->KeyState(EKeys::U) || Viewport->KeyState(EKeys::L);
		}

		return bDesireOrbit && !IsFlightCameraInputModeActive() && !IsOrtho();
	}
}

/** Returns true if perspective flight camera input mode is currently active in this viewport */
bool FEditorViewportClient::IsFlightCameraInputModeActive() const
{
	if( (Viewport != nullptr ) && IsPerspective() )
	{
		if( CameraController != NULL )
		{
			if (ModeTools->UsesViewportITF())
			{
				return ViewportNavigationHelper->bIsMouseLooking;
			}

			const bool bLeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) && !bLockFlightCamera;
			const bool bMiddleMouseButtonDown = Viewport->KeyState( EKeys::MiddleMouseButton );
			const bool bRightMouseButtonDown = Viewport->KeyState( EKeys::RightMouseButton );
			const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

			const bool bIsNonOrbitMiddleMouse = bMiddleMouseButtonDown && !IsAltPressed();

			const bool bIsMouseLooking =
				bIsTracking &&
				Widget->GetCurrentAxis() == EAxisList::None &&
				( bLeftMouseButtonDown || bMiddleMouseButtonDown || bRightMouseButtonDown || bIsUsingTrackpad ) &&
				!IsCtrlPressed() && (GetDefault<ULevelEditorViewportSettings>()->FlightCameraControlExperimentalNavigation || !IsShiftPressed()) && !IsAltPressed();

			return bIsMouseLooking;
		}
	}

	return false;
}

bool FEditorViewportClient::IsMovingCamera() const
{
	return bUsingOrbitCamera || IsFlightCameraActive();
}

bool FEditorViewportClient::DropObjectsAtCoordinates(int32 MouseX, int32 MouseY, const TArray<UObject*>& DroppedObjects, 
	TArray<FTypedElementHandle>& OutNewObjects, const FDropObjectOptions& Options)
{
	// Forward things to the deprecated overload while it still exists, so that we don't break any
	// existing derivations of FEditorViewportClient. Once removed, this function will just return false.
	TArray<AActor*> OutputActors;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bSuccess = DropObjectsAtCoordinates(MouseX, MouseY, DroppedObjects, OutputActors,
		Options.bOnlyDropOnTarget, Options.bCreateDropPreview, Options.bSelectOutput,
		Cast<UActorFactory>(Options.FactoryToUse.GetObject()));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TArray<FTypedElementHandle> OutputElements;

	for (const AActor* Actor : OutputActors)
	{
		FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
		OutputElements.Add(Handle);
	}

	return bSuccess;
}

/** True if the window is maximized or floating */
bool FEditorViewportClient::IsVisible() const
{
	bool bIsVisible = false;

	if( VisibilityDelegate.IsBound() )
	{
		// Call the visibility delegate to see if our parent viewport and layout configuration says we arevisible
		bIsVisible = VisibilityDelegate.Execute();
	}

	return bIsVisible;
}

void FEditorViewportClient::GetViewportDimensions( FIntPoint& OutOrigin, FIntPoint& Outize )
{
	OutOrigin = FIntPoint(0,0);
	if ( Viewport != nullptr )
	{
		Outize.X = Viewport->GetSizeXY().X;
		Outize.Y = Viewport->GetSizeXY().Y;
	}
	else
	{
		Outize = FIntPoint(0,0);
	}
}

void FEditorViewportClient::UpdateAndApplyCursorVisibility()
{
	UpdateRequiredCursorVisibility();
	ApplyRequiredCursorVisibility();
}

void FEditorViewportClient::UpdateRequiredCursorVisibility()
{
	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) ? true : false;
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton) ? true : false;
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton) ? true : false;
	const bool bMouseButtonDown = (LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown );
	const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

	bool AltDown = IsAltPressed();
	bool ShiftDown = IsShiftPressed();
	bool ControlDown = IsCtrlPressed();

	bool bOverrideCursorVisibility = false;
	bool bHardwareCursorVisible = false;
	bool bSoftwareCursorVisible = false;
	if (ModeTools->GetCursorVisibilityOverride(bOverrideCursorVisibility, bHardwareCursorVisible, bSoftwareCursorVisible))
	{
		if (bOverrideCursorVisibility)
		{
			SetRequiredCursor(bHardwareCursorVisible, bSoftwareCursorVisible);
			return;
		}
	}

	if (ViewportType == LVT_None)
	{
		SetRequiredCursor(true, false);
		return;
	}

	//if we're using the new move canvas mode, we're in an ortho viewport, and the mouse is down
	if (IsOrtho() && bMouseButtonDown && !MouseDeltaTracker->UsingDragTool() && !ModeTools->UsesViewportITF())
	{
		//Translating an object, but NOT moving the camera AND the object (shift)
		if ( ( AltDown == false && ShiftDown == false && ( LeftMouseButtonDown ^ RightMouseButtonDown ) ) &&
			( ( GetWidgetMode() == UE::Widget::WM_Translate && Widget->GetCurrentAxis() != EAxisList::None ) ||
			(  GetWidgetMode() == UE::Widget::WM_TranslateRotateZ && Widget->GetCurrentAxis() != EAxisList::ZRotation &&  Widget->GetCurrentAxis() != EAxisList::None ) ||
			( GetWidgetMode() == UE::Widget::WM_2D && Widget->GetCurrentAxis() != EAxisList::Rotate2D &&  Widget->GetCurrentAxis() != EAxisList::None ) ) )
		{
			SetRequiredCursor(false, true);
			SetRequiredCursorOverride( true , EMouseCursor::CardinalCross );
			return;
		}

		if (GetDefault<ULevelEditorViewportSettings>()->bPanMovesCanvas && RightMouseButtonDown)
		{
			bool bMovingCamera = GetCurrentWidgetAxis() == EAxisList::None;
			bool bIsZoomingCamera = bMovingCamera && ( LeftMouseButtonDown || bIsUsingTrackpad );
			//moving camera without  zooming
			if ( bMovingCamera && !bIsZoomingCamera )
			{
				// Always turn the hardware cursor on before turning the software cursor off
				// so the hardware cursor will be be set where the software cursor was
				SetRequiredCursor(!bHasMouseMovedSinceClick, bHasMouseMovedSinceClick);
				SetRequiredCursorOverride( true , EMouseCursor::GrabHand );
				return;
			}
			SetRequiredCursor(false, false);
			return;
		}
	}

	//if Absolute Translation or arc rotate and not just moving the camera around
	if (IsUsingAbsoluteTranslation(true) && !MouseDeltaTracker->UsingDragTool())
	{
		//If we are dragging something we should hide the hardware cursor and show the s/w one
		SetRequiredCursor(false, true);
		SetRequiredCursorOverride( true , EMouseCursor::CardinalCross );
	}
	else
	{
		// Calc the raw delta from the mouse since we started dragging to detect if there was any movement
		FVector RawMouseDelta = MouseDeltaTracker->GetRawDelta();

		if (bMouseButtonDown && (RawMouseDelta.SizeSquared() >= MOUSE_CLICK_DRAG_DELTA || IsFlightCameraActive() || ShouldOrbitCamera()) && !MouseDeltaTracker->UsingDragTool())
		{
			//current system - do not show cursor when mouse is down
			SetRequiredCursor(false, false);
			return;
		}

		if( MouseDeltaTracker->UsingDragTool() )
		{
			RequiredCursorVisibiltyAndAppearance.bOverrideAppearance = false;
		}

		SetRequiredCursor(true, false);
	}
}

void FEditorViewportClient::SetRequiredCursor(const bool bHardwareCursorVisible, const bool bSoftwareCursorVisible)
{
	RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible = bHardwareCursorVisible;
	RequiredCursorVisibiltyAndAppearance.bSoftwareCursorVisible = bSoftwareCursorVisible;
}

void FEditorViewportClient::ApplyRequiredCursorVisibility( bool bUpdateSoftwareCursorPostion )
{
	if( RequiredCursorVisibiltyAndAppearance.bDontResetCursor == true )
	{
		Viewport->SetPreCaptureMousePosFromSlateCursor();
	}
	bool bOldCursorVisibility = Viewport->IsCursorVisible();
	bool bOldSoftwareCursorVisibility = Viewport->IsSoftwareCursorVisible();

	Viewport->ShowCursor( RequiredCursorVisibiltyAndAppearance.bHardwareCursorVisible );
	Viewport->ShowSoftwareCursor( RequiredCursorVisibiltyAndAppearance.bSoftwareCursorVisible );
	if( bUpdateSoftwareCursorPostion == true )
	{
		//if we made the software cursor visible set its position
		if( bOldSoftwareCursorVisibility != Viewport->IsSoftwareCursorVisible() )
		{
			Viewport->SetSoftwareCursorPosition( FVector2D(Viewport->GetMouseX() , Viewport->GetMouseY() ) );
		}
	}
}


void FEditorViewportClient::SetRequiredCursorOverride( bool WantOverride, EMouseCursor::Type RequiredCursor )
{
	RequiredCursorVisibiltyAndAppearance.bOverrideAppearance = WantOverride;
	RequiredCursorVisibiltyAndAppearance.RequiredCursor = RequiredCursor;
}

void FEditorViewportClient::SetWidgetModeOverride(UE::Widget::EWidgetMode InWidgetMode)
{
	ModeTools->SetWidgetModeOverride(InWidgetMode);
}

EAxisList::Type FEditorViewportClient::GetCurrentWidgetAxis() const
{
	return Widget->GetCurrentAxis();
}

void FEditorViewportClient::SetCurrentWidgetAxis(EAxisList::Type InAxis)
{
	Widget->SetCurrentAxis(InAxis);
	ModeTools->SetCurrentWidgetAxis(InAxis);
}

void FEditorViewportClient::AdjustTransformWidgetSize(const int32 SizeDelta)
{
	 if (UTransformGizmoEditorSettings* TransformEditorSettings = GetMutableDefault<UTransformGizmoEditorSettings>())
	 {
		 TransformEditorSettings->SetTransformGizmoSize(FMath::Clamp(TransformEditorSettings->TransformGizmoSize + SizeDelta, -10.0f, 150.0f));
	 }
}

float FEditorViewportClient::GetNearClipPlane() const
{
	return (NearPlane < 0.0f) ? GNearClippingPlane : NearPlane;
}

void FEditorViewportClient::OverrideNearClipPlane(float InNearPlane)
{
	NearPlane = InNearPlane;
}

float FEditorViewportClient::GetFarClipPlaneOverride() const
{
	return FarPlane;
}

void FEditorViewportClient::OverrideFarClipPlane(const float InFarPlane)
{
	FarPlane = InFarPlane;
}

TOptional<double> FEditorViewportClient::GetOrthographicNearPlaneOverride() const
{
	return OrthographicNearPlane;
}

void FEditorViewportClient::SetOrthographicNearPlaneOverride(const TOptional<double>& InValue)
{
	OrthographicNearPlane = InValue;
}

TOptional<double> FEditorViewportClient::GetOrthographicFarPlaneOverride() const
{
	return OrthographicFarPlane;
}

void FEditorViewportClient::SetOrthographicFarPlaneOverride(const TOptional<double>& InValue)
{
	OrthographicFarPlane = InValue;
}

FBox FEditorViewportClient::GetWorldBounds() const
{
	return CachedWorldBounds;
}

float FEditorViewportClient::GetSceneDepthAtLocation(int32 X, int32 Y)
{
	// #todo: in the future we will just sample the depth buffer
	return 0.f;
}

FVector FEditorViewportClient::GetHitProxyObjectLocation(int32 X, int32 Y)
{
	// #todo: for now we are just getting the actor and using its location for
	// depth. in the future we will just sample the depth buffer
	HHitProxy* const HitProxy = Viewport->GetHitProxy(X, Y);
	if (HitProxy && HitProxy->IsA(HActor::StaticGetType()))
	{
		HActor* const ActorHit = static_cast<HActor*>(HitProxy);

		// dist to component will be more reliable than dist to actor
		if (ActorHit->PrimComponent != nullptr)
		{
			return ActorHit->PrimComponent->GetComponentLocation();
		}

		if (ActorHit->Actor != nullptr)
		{
			return ActorHit->Actor->GetActorLocation();
		}
	}

	return FVector::ZeroVector;
}


void FEditorViewportClient::ShowWidget(const bool bShow)
{
	bShowWidget = bShow;
}

void FEditorViewportClient::MoveViewportCamera(const FVector& InDrag, const FRotator& InRot, bool bDollyCamera)
{
	const ELevelViewportType EffectiveViewportType = GetViewportType();
	switch( EffectiveViewportType )
	{
	case LVT_OrthoXY:
	case LVT_OrthoXZ:
	case LVT_OrthoYZ:
	case LVT_OrthoNegativeXY:
	case LVT_OrthoNegativeXZ:
	case LVT_OrthoNegativeYZ:
		{
			const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton);
			const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton);
			const bool bIsUsingTrackpad = FSlateApplication::Get().IsUsingTrackpad();

			if( ( LeftMouseButtonDown || bIsUsingTrackpad ) && RightMouseButtonDown )
			{
				float NewOrthoRatio = 1 + (FMath::Abs(InDrag.Z) / CAMERA_ZOOM_DAMPEN);
				if ( InDrag.Z < 0 )
				{
					NewOrthoRatio = 1 / NewOrthoRatio;
				}
				const float Zoom = GetOrthoZoom() * NewOrthoRatio;
				SetOrthoZoom( FMath::Clamp<float>( Zoom, GetMinimumOrthoZoom(), MAX_ORTHOZOOM ) );
			}
			else
			{
				const FVector ViewLocationAdjustment = InDrag;
				SetViewLocation( GetViewLocation() + ViewLocationAdjustment );
			}

			// Update any linked orthographic viewports.
			UpdateLinkedOrthoViewports();
		}
		break;

	case LVT_OrthoFreelook:
		//@TODO: CAMERA: Not sure how to handle this
		break;

	case LVT_Perspective:
		{
			// If the flight camera is active, we'll update the rotation impulse data for that instead
			// of rotating the camera ourselves here
			if( IsFlightCameraInputModeActive() && CameraController->GetConfig().bUsePhysicsBasedRotation )
			{
				const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();

				// NOTE: We damp the rotation for impulse input since the camera controller will
				//	apply its own rotation speed
				const float VelModRotSpeed = 900.0f;
				const FVector RotEuler = InRot.Euler();

				CameraUserImpulseData->RotateRollVelocityModifier += VelModRotSpeed * RotEuler.X / ViewportSettings->MouseSensitivty;
				CameraUserImpulseData->RotatePitchVelocityModifier += VelModRotSpeed * RotEuler.Y / ViewportSettings->MouseSensitivty;
				CameraUserImpulseData->RotateYawVelocityModifier += VelModRotSpeed * RotEuler.Z / ViewportSettings->MouseSensitivty;
			}
			else if (!bLockFlightCamera)
			{
				MoveViewportPerspectiveCamera( InDrag, InRot, bDollyCamera );
			}
		}
		break;
	}
}

bool FEditorViewportClient::ShouldLockPitch() const
{
	return CameraController->GetConfig().bLockedPitch;
}


void FEditorViewportClient::CheckHoveredHitProxy( HHitProxy* HoveredHitProxy )
{
	const EAxisList::Type SaveAxis = Widget->GetCurrentAxis();
	EAxisList::Type NewAxis = EAxisList::None;

	const bool LeftMouseButtonDown = Viewport->KeyState(EKeys::LeftMouseButton) ? true : false;
	const bool MiddleMouseButtonDown = Viewport->KeyState(EKeys::MiddleMouseButton) ? true : false;
	const bool RightMouseButtonDown = Viewport->KeyState(EKeys::RightMouseButton) ? true : false;
	const bool bMouseButtonDown = (LeftMouseButtonDown || MiddleMouseButtonDown || RightMouseButtonDown );

	// Change the mouse cursor if the user is hovering over something they can interact with.
	if( HoveredHitProxy )
	{
		if( HoveredHitProxy->IsA(HWidgetAxis::StaticGetType() ) && !bUsingOrbitCamera && !bMouseButtonDown )
		{
			// In the case of the widget mode being overridden we can have a hit proxy
			// from the previous mode with an inappropriate axis for rotation.
			EAxisList::Type ProxyAxis = ((HWidgetAxis*)HoveredHitProxy)->Axis;
			if ( !IsOrtho() || GetWidgetMode() != UE::Widget::WM_Rotate
				|| ProxyAxis == EAxisList::X || ProxyAxis == EAxisList::Y || ProxyAxis == EAxisList::Z )
			{
				NewAxis = ProxyAxis;
			}
			else
			{
				switch( GetViewportType() )
				{
				case LVT_OrthoXY:
				case LVT_OrthoNegativeXY:
					NewAxis = EAxisList::Z;
					break;
				case LVT_OrthoXZ:
				case LVT_OrthoNegativeXZ:
					NewAxis = EAxisList::Y;
					break;
				case LVT_OrthoYZ:
				case LVT_OrthoNegativeYZ:
					NewAxis = EAxisList::X;
					break;
				default:
					break;
				}
			}
		}


		// If the current axis on the widget changed, repaint the viewport.
		if( NewAxis != SaveAxis )
		{
			SetCurrentWidgetAxis( NewAxis );

			Invalidate( false, false );
		}
	}

}

void FEditorViewportClient::ConditionalCheckHoveredHitProxy()
{
	// If it has been decided that there is more important things to do than check hit proxies, or our mouse location is not valid, then don't check them.
	if( !bShouldCheckHitProxy || bWidgetAxisControlledByDrag == true || CurrentMousePos == FIntPoint(-1, -1) )
	{
		return;
	}

	HHitProxy* HitProxy = Viewport->GetHitProxy(CachedMouseX,CachedMouseY);

	CheckHoveredHitProxy( HitProxy );

	// We need to set this to false here as if mouse is moved off viewport fast, it will keep doing CheckHoveredOverHitProxy for this viewport when it should not.
	bShouldCheckHitProxy = false;
}

void FEditorViewportClient::TranslateViewportCamera(const FVector& InTranslation)
{
	SetViewLocation(GetViewLocation() + InTranslation);
	
	if (IsPerspective())
	{
		SetLookAtLocation(GetLookAtLocation() + InTranslation);	
		PerspectiveCameraMoved();
	}
	else if (IsOrtho())
	{
		// Update any linked orthographic viewports.
		UpdateLinkedOrthoViewports();
	}
}

/** Moves a perspective camera */
void FEditorViewportClient::MoveViewportPerspectiveCamera( const FVector& InDrag, const FRotator& InRot, bool bDollyCamera )
{
	check( IsPerspective() );

	FVector ViewLocation = GetViewLocation();
	FRotator ViewRotation = GetViewRotation();

	if ( ShouldLockPitch() )
	{
		// Update camera Rotation
		ViewRotation += FRotator( InRot.Pitch, InRot.Yaw, InRot.Roll );

		// normalize to -180 to 180
		ViewRotation.Pitch = FRotator::NormalizeAxis(ViewRotation.Pitch);
		// Make sure its withing  +/- 90 degrees (minus a small tolerance to avoid numerical issues w/ camera orientation conversions later on).
		ViewRotation.Pitch = FMath::Clamp( ViewRotation.Pitch, -90.f+KINDA_SMALL_NUMBER, 90.f-KINDA_SMALL_NUMBER );
	}
	else
	{
		//when not constraining the pitch we need to rotate differently to avoid a gimbal lock
		const FRotator PitchRot(InRot.Pitch, 0, 0);
		const FRotator LateralRot(0, InRot.Yaw, InRot.Roll);

		//update lateral rotation
		ViewRotation += LateralRot;

		//update pitch separately using quaternions
		const FQuat ViewQuat = ViewRotation.Quaternion();
		const FQuat PitchQuat = PitchRot.Quaternion();
		const FQuat ResultQuat = ViewQuat * PitchQuat;

		//get our correctly rotated ViewRotation
		ViewRotation = ResultQuat.Rotator();
	}

	const float DistanceToCurrentLookAt = FVector::Dist( GetViewLocation(), GetLookAtLocation() );
	const float CameraSpeedDistanceScale = ShouldScaleCameraSpeedByDistance() ? FMath::Min(DistanceToCurrentLookAt / 1000.f, 1000.f) : 1.f;

	// Update camera Location
	ViewLocation += InDrag * CameraSpeedDistanceScale;

	if( !bDollyCamera )
	{
		const FQuat CameraOrientation = FQuat::MakeFromEuler( ViewRotation.Euler() );
		FVector Direction = CameraOrientation.RotateVector( FVector(1,0,0) );

		SetLookAtLocation( ViewLocation + Direction * DistanceToCurrentLookAt );
	}

	SetViewLocation(ViewLocation);
	SetViewRotation(ViewRotation);

	if (bUsingOrbitCamera)
	{
		FVector LookAtPoint = GetLookAtLocation();
		const float DistanceToLookAt = FVector::Dist( ViewLocation, LookAtPoint );

		SetViewLocationForOrbiting( LookAtPoint, DistanceToLookAt );
	}

	PerspectiveCameraMoved();

}

void FEditorViewportClient::EnableCameraLock(bool bEnable)
{
	bCameraLock = bEnable;

	if(bCameraLock)
	{
		SetViewLocation( DefaultOrbitLocation + DefaultOrbitZoom );
		SetViewRotation( DefaultOrbitRotation );
		SetLookAtLocation( DefaultOrbitLookAt );
	}
	else
	{
		ToggleOrbitCamera( false );
	}

	bUsingOrbitCamera = bCameraLock;
}

FCachedJoystickState* FEditorViewportClient::GetJoystickState(const uint32 InControllerID)
{
	FCachedJoystickState* CurrentState = JoystickStateMap.FindRef(InControllerID);
	if (CurrentState == NULL)
	{
		/** Create new joystick state for cached input*/
		CurrentState = new FCachedJoystickState();
		CurrentState->JoystickType = 0;
		JoystickStateMap.Add(InControllerID, CurrentState);
	}

	return CurrentState;
}

void FEditorViewportClient::SetCameraLock()
{
	EnableCameraLock(!bCameraLock);
	Invalidate();
}

bool FEditorViewportClient::IsCameraLocked() const
{
	return bCameraLock;
}

void FEditorViewportClient::SetShowGrid()
{
	EngineShowFlags.SetGrid(!EngineShowFlags.Grid);
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("EngineShowFlags.Grid"), EngineShowFlags.Grid ? TEXT("True") : TEXT("False"));
	}
	Invalidate();
}

bool FEditorViewportClient::IsSetShowGridChecked() const
{
	return EngineShowFlags.Grid;
}

void FEditorViewportClient::SetShowBounds(bool bShow)
{
	EngineShowFlags.SetBounds(bShow);
}

void FEditorViewportClient::ToggleShowBounds()
{
	EngineShowFlags.SetBounds(!EngineShowFlags.Bounds);
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.Toolbar"), TEXT("Bounds"), FString::Printf(TEXT("%d"), EngineShowFlags.Bounds));
	}
	Invalidate();
}

bool FEditorViewportClient::IsSetShowBoundsChecked() const
{
	return EngineShowFlags.Bounds;
}

void FEditorViewportClient::UpdateHiddenCollisionDrawing()
{
	FSceneInterface* SceneInterface = GetScene();
	if (SceneInterface != nullptr)
	{
		UWorld* World = SceneInterface->GetWorld();
		if (World != nullptr)
		{
			// See if this is a collision view mode
			bool bCollisionMode = EngineShowFlags.Collision || EngineShowFlags.CollisionVisibility || EngineShowFlags.CollisionPawn;

			// Tell engine to create proxies for hidden components, so we can still draw collision
			if (World->bCreateRenderStateForHiddenComponentsWithCollsion != bCollisionMode)
			{
				World->bCreateRenderStateForHiddenComponentsWithCollsion = bCollisionMode;

				// Need to recreate scene proxies when this flag changes.
				FGlobalComponentRecreateRenderStateContext Recreate;
			}
		}
	}
}


void FEditorViewportClient::SetShowCollision()
{
	EngineShowFlags.SetCollision(!EngineShowFlags.Collision);
	UpdateHiddenCollisionDrawing();
	Invalidate();
}

bool FEditorViewportClient::IsSetShowCollisionChecked() const
{
	return EngineShowFlags.Collision;
}

void FEditorViewportClient::ToggleGeometryInspectionShowWireframe()
{
	bIsGeometryInspectionWireframeOn = !bIsGeometryInspectionWireframeOn;
	Invalidate();
}

bool FEditorViewportClient::IsSetGeometryInspectionShowWireframe() const
{
	return bIsGeometryInspectionWireframeOn;
}

void FEditorViewportClient::SetChannelMask(EColorChannelMask InColorChannelMask)
{
	// If the color channel mask is already set to the specified mask, toggle it off by setting the mask to All
	if (ChannelMaskParams.ColorChannelMask == InColorChannelMask)
	{
		ChannelMaskParams.ColorChannelMask = EColorChannelMask::All;
	}
	else
	{
		ChannelMaskParams.ColorChannelMask = InColorChannelMask;
	}
	
	Invalidate();
}

bool FEditorViewportClient::IsChannelBeingMasked(EColorChannelMask InColorChannelMask) const
{
	return ChannelMaskParams.ColorChannelMask == InColorChannelMask;
}

void FEditorViewportClient::SetViewMode(EViewModeIndex InViewModeIndex)
{
	ViewModeParam = -1; // Reset value when the viewmode changes
	ViewModeParamName = NAME_None;
	ViewModeParamNameMap.Empty();

	if (IsPerspective())
	{
		if (InViewModeIndex == VMI_MaterialTextureScaleAccuracy)
		{
			FEditorBuildUtils::UpdateTextureStreamingMaterialBindings(GetWorld());
		}

		PerspViewModeIndex = InViewModeIndex;
		ApplyViewMode(PerspViewModeIndex, true, EngineShowFlags);
		bForcingUnlitForNewMap = false;
	}
	else
	{
		OrthoViewModeIndex = InViewModeIndex;
		ApplyViewMode(OrthoViewModeIndex, false, EngineShowFlags);
	}

	UpdateHiddenCollisionDrawing();
	Invalidate();
}

void FEditorViewportClient::SetGeometryInspectionViewMode(EViewModeIndex ViewModeIndex)
{
	static const IConsoleVariable* ICNaniteVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite"));
	
	const bool bNanitesEnabled = (ICNaniteVar == nullptr || ICNaniteVar->GetBool());

	if (bNanitesEnabled)
	{
		const FText WarningText(LOCTEXT("GeometryInspectionViewModesDontWorkWithNanite", "Geometry Inspection view modes only display non-Nanite geometry."));
		UE_LOGF(LogEditorModes, Warning, "%ls", *WarningText.ToString());

		FNotificationInfo Info(WarningText);
		Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
		Info.ExpireDuration = 5.0f;
		Info.SubText = LOCTEXT("GeometryInspectionViewModesDontWorkWithNaniteSubText", "View fallback meshes by disabling Nanite in the Details panel (Rendering -> Disallow Nanite).");

		FSlateNotificationManager::Get().AddNotification(Info);
	}

	SetViewMode(ViewModeIndex);
}

void FEditorViewportClient::SetZebraViewMode(bool bHorizontalZebraMode)
{
	if (UMaterialInstanceDynamic* MaterialInstance = Cast<UMaterialInstanceDynamic>(GEngine->ZebraMaterialInstance))
	{
		FLinearColor Color  = bHorizontalZebraMode ? FLinearColor(0, 0, 1) : FLinearColor(1, 0, 0);

		MaterialInstance->SetVectorParameterValue(GeometryInspectionZebraMaterialParameterNames::Orientation, Color);
	}

	SetGeometryInspectionViewMode(VMI_Zebra);
}

bool FEditorViewportClient::IsZebraViewModeEnabled(bool bHorizontal) const
{
	if (IsViewModeEnabled(VMI_Zebra))
	{
		if (UMaterialInstanceDynamic* MaterialInstance = Cast<UMaterialInstanceDynamic>(GEngine->ZebraMaterialInstance))
		{
			FLinearColor Color;
			if (MaterialInstance->GetVectorParameterValue(GeometryInspectionZebraMaterialParameterNames::Orientation, Color))
			{
				if (Color.B == 1.0f && bHorizontal)
				{
					return true;
				}

				if (Color.R == 1.0f && !bHorizontal)
				{
					return true;
				}

				return false;
			}
		}
	}

	return false;
}

void FEditorViewportClient::SetZebraViewModeStripeCount(uint8 InStripeCount)
{
	if (InStripeCount>0 && InStripeCount<=64)
	{
		if (UMaterialInstanceDynamic* MaterialInstance = Cast<UMaterialInstanceDynamic>(GEngine->ZebraMaterialInstance))
		{
			MaterialInstance->SetScalarParameterValue(GeometryInspectionZebraMaterialParameterNames::StripeCount, InStripeCount);

			Invalidate();
		}
	}
}

uint8 FEditorViewportClient::GetZebraViewModeStripeCount()
{
	if (UMaterialInstanceDynamic* MaterialInstance = Cast<UMaterialInstanceDynamic>(GEngine->ZebraMaterialInstance))
	{
		FHashedMaterialParameterInfo ParameterInfo(GeometryInspectionZebraMaterialParameterNames::StripeCount);
		float Value;
		if (MaterialInstance->GetScalarParameterValue(ParameterInfo, Value))
		{
			return Value;
		}
	}

	return 1;
}

void FEditorViewportClient::SetViewModes(const EViewModeIndex InPerspViewModeIndex, const EViewModeIndex InOrthoViewModeIndex)
{
	PerspViewModeIndex = InPerspViewModeIndex;
	OrthoViewModeIndex = InOrthoViewModeIndex;

	if (IsPerspective())
	{
		ApplyViewMode(PerspViewModeIndex, true, EngineShowFlags);
	}
	else
	{
		ApplyViewMode(OrthoViewModeIndex, false, EngineShowFlags);
	}

	UpdateHiddenCollisionDrawing();
	Invalidate();
}

void FEditorViewportClient::SetViewModeParam(int32 InViewModeParam)
{
	ViewModeParam = InViewModeParam;
	FName* BoundName = ViewModeParamNameMap.Find(ViewModeParam);
	ViewModeParamName = BoundName ? *BoundName : FName();

	Invalidate();
}

bool FEditorViewportClient::IsViewModeParam(int32 InViewModeParam) const
{
	const FName* MappedName = ViewModeParamNameMap.Find(ViewModeParam);
	// Check if the param and names match. The param name only gets updated on click, while the map is built at menu creation.
	if (MappedName)
	{
		return ViewModeParam == InViewModeParam && ViewModeParamName == *MappedName;
	}
	else
	{
		return ViewModeParam == InViewModeParam && ViewModeParamName == NAME_None;
	}
}

EViewModeIndex FEditorViewportClient::GetViewMode() const
{
	return (IsPerspective()) ? PerspViewModeIndex : OrthoViewModeIndex;
}

void FEditorViewportClient::Invalidate(bool bInvalidateChildViews, bool bInvalidateHitProxies)
{
	if ( Viewport )
	{
		if ( bInvalidateHitProxies )
		{
			// Invalidate hit proxies and display pixels.
			Viewport->Invalidate();
		}
		else
		{
			// Invalidate only display pixels.
			Viewport->InvalidateDisplay();
		}
	}
}

void FEditorViewportClient::MouseEnter(FViewport* InViewport,int32 x, int32 y)
{
	ModeTools->MouseEnter(this, Viewport, x, y);

	MouseMove(InViewport, x, y);

	PixelInspectorRealtimeManagement(this, true);
}

void FEditorViewportClient::MouseMove(FViewport* InViewport,int32 x, int32 y)
{
	check(IsInGameThread());

	CurrentMousePos = FIntPoint(x, y);

	// Let the current editor mode know about the mouse movement.
	ModeTools->MouseMove(this, Viewport, x, y);

	CachedLastMouseX = x;
	CachedLastMouseY = y;
}

void FEditorViewportClient::MouseLeave(FViewport* InViewport)
{
	check(IsInGameThread());

	ModeTools->MouseLeave(this, Viewport);

	CurrentMousePos = FIntPoint(-1, -1);

	FCommonViewportClient::MouseLeave(InViewport);

	PixelInspectorRealtimeManagement(this, false);
}

FViewportCursorLocation FEditorViewportClient::GetCursorWorldLocationFromMousePos()
{
	// Create the scene view context
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Viewport,
		GetScene(),
		EngineShowFlags)
		.SetRealtimeUpdate(IsRealtime()));

	// Calculate the scene view
	FSceneView* View = CalcSceneView(&ViewFamily);

	// Construct an FViewportCursorLocation which calculates world space postion from the scene view and mouse pos.
	return FViewportCursorLocation(View,
		this,
		Viewport->GetMouseX(),
		Viewport->GetMouseY()
	);
}

void FEditorViewportClient::CapturedMouseMove( FViewport* InViewport, int32 InMouseX, int32 InMouseY )
{
	// Let the current editor mode know about the mouse movement.
	if (ModeTools->CapturedMouseMove(this, InViewport, InMouseX, InMouseY))
	{
		RequiredCursorVisibiltyAndAppearance.bDontResetCursor = true;
		UpdateRequiredCursorVisibility();
		ApplyRequiredCursorVisibility();
		
		if (Viewport->IsSoftwareCursorVisible())
		{
			Viewport->SetSoftwareCursorPosition(FVector2D(InMouseX, InMouseY));
		}
		return;
	}

	UpdateRequiredCursorVisibility();
	ApplyRequiredCursorVisibility();

	CapturedMouseMoves.Add(FIntPoint(InMouseX, InMouseY));
}

void FEditorViewportClient::ProcessAccumulatedPointerInput(FViewport* InViewport)
{
	ModeTools->ProcessCapturedMouseMoves(this, InViewport, CapturedMouseMoves);
	CapturedMouseMoves.Reset();
}

void FEditorViewportClient::OpenScreenshot( FString SourceFilePath )
{
	FPlatformProcess::ExploreFolder( *( FPaths::GetPath( SourceFilePath ) ) );
}

void FEditorViewportClient::TakeScreenshot(FViewport* InViewport, bool bInValidatViewport)
{
	FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();

	if (!ensure(HighResScreenshotConfig.ImageWriteQueue))
	{
		return;
	}

	// The old method for taking screenshots does this for us on mousedown, so we do not have
	//	to do this for all situations.
	if( bInValidatViewport )
	{
		// We need to invalidate the viewport in order to generate the correct pixel buffer for picking.
		Invalidate( false, true );
	}

	// Redraw the viewport so we don't end up with clobbered data from other viewports using the same frame buffer.
	InViewport->Draw();

	// Inform the user of the result of the operation
	FNotificationInfo Info(FText::GetEmpty());
	Info.ExpireDuration = 5.0f;
	Info.bUseSuccessFailIcons = false;
	Info.bUseLargeFont = false;

	TSharedPtr<SNotificationItem> SaveMessagePtr = FSlateNotificationManager::Get().AddNotification(Info);
	SaveMessagePtr->SetCompletionState(SNotificationItem::CS_Fail);

	TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();

	// Read the contents of the viewport into an array.
	bool bHdrEnabled = InViewport->GetSceneHDREnabled();
	const FIntRect CaptureRect = FIntRect(0, 0, InViewport->GetRenderTargetTextureSizeXY().X, InViewport->GetRenderTargetTextureSizeXY().Y);
	
	if (!bHdrEnabled)
	{
		TArray<FColor> RawPixels;
		RawPixels.SetNum(CaptureRect.Area());
		if(!InViewport->ReadPixels(RawPixels, FReadSurfaceDataFlags(RCM_UNorm, CubeFace_MAX), CaptureRect))
		{
			// Failed to read the image from the viewport
			SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotFailedViewport", "Screenshot failed, unable to read image from viewport" ));
			return;
		}

		TUniquePtr<TImagePixelData<FColor>> PixelData = MakeUnique<TImagePixelData<FColor>>(InViewport->GetRenderTargetTextureSizeXY(), TArray64<FColor>(MoveTemp(RawPixels)));

		check(PixelData->IsDataWellFormed());
		ImageTask->PixelData = MoveTemp(PixelData);

		// Ensure the alpha channel is full alpha (this happens on the background thread)
		ImageTask->AddPreProcessorToSetAlphaOpaque();
	}
	else
	{
		TArray<FLinearColor> RawPixels;
		RawPixels.SetNum(CaptureRect.Area());
		if (!InViewport->ReadLinearColorPixels(RawPixels, FReadSurfaceDataFlags(RCM_MinMax, CubeFace_MAX), CaptureRect))
		{
			// Failed to read the image from the viewport
			SaveMessagePtr->SetText(NSLOCTEXT("UnrealEd", "ScreenshotFailedViewport", "Screenshot failed, unable to read image from viewport"));
			return;
		}

		ConvertPixelDataToSCRGB(RawPixels, InViewport->GetDisplayOutputFormat());

		TUniquePtr<TImagePixelData<FLinearColor>> PixelData = MakeUnique<TImagePixelData<FLinearColor>>(InViewport->GetRenderTargetTextureSizeXY(), TArray64<FLinearColor>(MoveTemp(RawPixels)));

		check(PixelData->IsDataWellFormed());
		ImageTask->PixelData = MoveTemp(PixelData);

		// Ensure the alpha channel is full alpha (this happens on the background thread)
		ImageTask->AddPreProcessorToSetAlphaOpaque();
	}

	// Create screenshot folder if not already present.
	const FString& Directory = GetDefault<ULevelEditorMiscSettings>()->EditorScreenshotSaveDirectory.Path;
	if ( !IFileManager::Get().MakeDirectory(*Directory, true ) )
	{
		// Failed to make save directory
		UE_LOGF(LogEditorViewport, Warning, "Failed to create directory %ls", *FPaths::ConvertRelativePathToFull(Directory));
		SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotFailedFolder", "Screenshot capture failed, unable to create save directory (see log)" ));
		return;
	}

	// Save the contents of the array to a bitmap file.
	HighResScreenshotConfig.SetHDRCapture(bHdrEnabled);

	// Set the image task parameters
	ImageTask->Format = bHdrEnabled ? EImageFormat::EXR : EImageFormat::PNG;
	ImageTask->CompressionQuality = (int32)EImageCompressionQuality::Default;

	const TCHAR* FileExtension = bHdrEnabled ? TEXT("exr") : TEXT("png");

	bool bGeneratedFilename = FFileHelper::GenerateNextBitmapFilename(Directory / TEXT("ScreenShot"), FileExtension, ImageTask->Filename);
	if (!bGeneratedFilename)
	{
		SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotFailed_TooManyScreenshots", "Screenshot failed, too many screenshots in output directory" ));
		return;
	}

	FString HyperLinkString = FPaths::ConvertRelativePathToFull(ImageTask->Filename);

	// Define the callback to be called on the main thread when the image has completed
	// This will be called regardless of whether the write succeeded or not, and will update
	// the text and completion state of the notification.
	ImageTask->OnCompleted = [HyperLinkString, SaveMessagePtr](bool bCompletedSuccessfully)
	{
		if (bCompletedSuccessfully)
		{
			auto OpenScreenshotFolder = [HyperLinkString]
			{
				FPlatformProcess::ExploreFolder( *FPaths::GetPath(HyperLinkString) );
			};

			SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotSavedAs", "Screenshot capture saved as" ));

			SaveMessagePtr->SetHyperlink(FSimpleDelegate::CreateLambda(OpenScreenshotFolder), FText::FromString(HyperLinkString));
			SaveMessagePtr->SetCompletionState(SNotificationItem::CS_Success);
		}
		else
		{
			SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotFailed_CouldNotSave", "Screenshot failed, unable to save" ));
			SaveMessagePtr->SetCompletionState(SNotificationItem::CS_Fail);
		}
	};

	TFuture<bool> CompletionFuture = HighResScreenshotConfig.ImageWriteQueue->Enqueue(MoveTemp(ImageTask));
	if (CompletionFuture.IsValid())
	{
		SaveMessagePtr->SetCompletionState(SNotificationItem::CS_Pending);
	}
	else
	{
		// Unable to write the data for an unknown reason
		SaveMessagePtr->SetText(NSLOCTEXT( "UnrealEd", "ScreenshotFailedWrite", "Screenshot failed, corrupt data captured from the viewport." ));
	}
}

/**
 * Implements screenshot capture for editor viewports.
 */
bool FEditorViewportClient::InputTakeScreenshot(FViewport* InViewport, FKey Key, EInputEvent Event)
{
	const bool F9Down = InViewport->KeyState(EKeys::F9);

	// Whether or not we accept the key press
	bool bHandled = false;

	if ( F9Down )
	{
		if ( Key == EKeys::LeftMouseButton )
		{
			if( Event == IE_Pressed )
			{
				// We need to invalidate the viewport in order to generate the correct pixel buffer for picking.
				Invalidate( false, true );
			}
			else if( Event == IE_Released )
			{
				TakeScreenshot(InViewport,false);
			}
			bHandled = true;
		}
	}

	return bHandled;
}

void FEditorViewportClient::TakeHighResScreenShot()
{
	if(Viewport)
	{
		Viewport->TakeHighResScreenShot();
	}
}

template<class FColorType>
void ClipBitmapDataScreenshotDataEditor(bool& bWriteAlpha, TArray<FColorType>& Bitmap, FIntPoint& BitmapSize, const FIntRect& CaptureRect, bool bCaptureAreaValid)
{
	FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();

	// Determine which region of the captured data we want to save out. If the highres screenshot capture region
	// is not valid, we want to save out everything in the viewrect that we just grabbed.
	FIntRect SourceRect = FIntRect(0, 0, 0, 0);
	if (GIsHighResScreenshot && bCaptureAreaValid)
	{
		// Highres screenshot capture region is valid, so use that
		SourceRect = HighResScreenshotConfig.CaptureRegion;
	}

	bWriteAlpha = false;

	// If this is a high resolution screenshot and we are using the masking feature,
	// Get the results of the mask rendering pass and insert into the alpha channel of the screenshot.
	if (GIsHighResScreenshot && HighResScreenshotConfig.bMaskEnabled)
	{
		bWriteAlpha = HighResScreenshotConfig.MergeMaskIntoAlpha(Bitmap, CaptureRect);
	}

	// Clip the bitmap to just the capture region if valid
	if (!SourceRect.IsEmpty())
	{
		const int32 OldWidth = BitmapSize.X;
		const int32 OldHeight = BitmapSize.Y;

		//clamp in bounds:
		int CaptureMinX = FMath::Clamp(SourceRect.Min.X, 0, OldWidth);
		int CaptureMinY = FMath::Clamp(SourceRect.Min.Y, 0, OldHeight);

		int CaptureMaxX = FMath::Clamp(SourceRect.Max.X, 0, OldWidth);
		int CaptureMaxY = FMath::Clamp(SourceRect.Max.Y, 0, OldHeight);

		int32 NewWidth = CaptureMaxX - CaptureMinX;
		int32 NewHeight = CaptureMaxY - CaptureMinY;

		if (NewWidth > 0 && NewHeight > 0 && NewWidth != OldWidth && NewHeight != OldHeight)
		{
			FColorType* const Data = Bitmap.GetData();

			for (int32 Row = 0; Row < NewHeight; Row++)
			{
				FMemory::Memmove(Data + Row * NewWidth, Data + (Row + CaptureMinY) * OldWidth + CaptureMinX, NewWidth * sizeof(*Data));
			}

			Bitmap.RemoveAt(NewWidth * NewHeight, OldWidth * OldHeight - NewWidth * NewHeight, EAllowShrinking::No);
			BitmapSize = FIntPoint(NewWidth, NewHeight);
		}
	}
}

template<class FColorType, typename TChannelType>
bool RequestSaveScreenshot(bool bWriteAlpha, TArray<FColorType>& Bitmap, FIntPoint& BitmapSize, TChannelType OpaqueAlphaValue)
{
	FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();
	bool bIsScreenshotSaved = false;
	bool bSuppressWritingToFile = false;
#if UE_SCREENSHOT_TRACE_ENABLED
	if (SHOULD_TRACE_SCREENSHOT())
	{
		bSuppressWritingToFile = FTraceScreenshot::ShouldSuppressWritingToFile();
		FTraceScreenshot::TraceScreenshot(BitmapSize.X, BitmapSize.Y, Bitmap, FScreenshotRequest::GetFilename());
	}
#endif

	if (!bSuppressWritingToFile)
	{
		TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();
		ImageTask->PixelData = MakeUnique<TImagePixelData<FColorType>>(BitmapSize, TArray64<FColorType>(MoveTemp(Bitmap)));

		// Set full alpha on the bitmap
		if (!bWriteAlpha)
		{
			ImageTask->AddPreProcessorToSetAlphaOpaque();
		}

		HighResScreenshotConfig.PopulateImageTaskParams(*ImageTask);
		ImageTask->Filename = FScreenshotRequest::GetFilename();

		{
			// if not high dynamic range, get format from filename :
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
			EImageFormat ImageFormat = ImageWrapperModule.GetImageFormatFromExtension(*ImageTask->Filename);
			if (ImageFormat != EImageFormat::Invalid)
			{
				ImageTask->Format = ImageFormat;
			}
		}

		// Save the bitmap to disk
		TFuture<bool> CompletionFuture = HighResScreenshotConfig.ImageWriteQueue->Enqueue(MoveTemp(ImageTask));
		if (CompletionFuture.IsValid())
		{
			// this queues it then immediately waits? what's the point of ImageWriteQueue then?
			// just use FImageUtils::Save
			bIsScreenshotSaved = CompletionFuture.Get();
		}
	}

	return bIsScreenshotSaved;
}

bool FEditorViewportClient::ProcessScreenShots(FViewport* InViewport)
{
	bool bIsScreenshotSaved = false;

	if (GIsDumpingMovie || FScreenshotRequest::IsScreenshotRequested() || GIsHighResScreenshot)
	{
		// Default capture region is the entire viewport
		FIntRect CaptureRect(0, 0, 0, 0);

		FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();
		bool bCaptureAreaValid = HighResScreenshotConfig.CaptureRegion.Area() > 0;

		if (!ensure(HighResScreenshotConfig.ImageWriteQueue))
		{
			return false;
		}

		// If capture region isn't valid, we need to determine which rectangle to capture from.
		// We need to calculate a proper view rectangle so that we can take into account camera
		// properties, such as it being aspect ratio constrained
		if (GIsHighResScreenshot && !bCaptureAreaValid)
		{
			// Screen Percentage is an optimization and should not affect the editor by default, unless we're rendering in stereo
			bool bUseScreenPercentage = GEngine && GEngine->IsStereoscopic3D(InViewport);

			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				InViewport,
				GetScene(),
				EngineShowFlags)
				.SetRealtimeUpdate(IsRealtime())
				.SetViewModeParam(ViewModeParam, ViewModeParamName));

			ViewFamily.EngineShowFlags.SetScreenPercentage(bUseScreenPercentage);

			auto* ViewportBak = Viewport;
			Viewport = InViewport;
			FSceneView* View = CalcSceneView(&ViewFamily);
			Viewport = ViewportBak;
			CaptureRect = View->UnscaledViewRect;
		}

		bool bHdrEnabled = InViewport->GetSceneHDREnabled();

		// Determine the size of the captured viewport data.
		FIntPoint BitmapSize = CaptureRect.Area() > 0 ? CaptureRect.Size() : InViewport->GetSizeXY();
		if (!bHdrEnabled)
		{
			TArray<FColor> Bitmap;
			if (GetViewportScreenShot(InViewport, Bitmap, CaptureRect))
			{
				bool bWriteAlpha = false;
				ClipBitmapDataScreenshotDataEditor(bWriteAlpha, Bitmap, BitmapSize, CaptureRect, bCaptureAreaValid);

				if (FScreenshotRequest::OnScreenshotCaptured().IsBound())
				{
					TArray<FColor> BitmapForBroadcast(Bitmap);

					if (!bWriteAlpha)
					{
						// Set full alpha on the bitmap
						for (FColor& Pixel : BitmapForBroadcast) { Pixel.A = 255; }
					}

					FScreenshotRequest::OnScreenshotCaptured().Broadcast(BitmapSize.X, BitmapSize.Y, MoveTemp(BitmapForBroadcast));
				}

				bIsScreenshotSaved = RequestSaveScreenshot(bWriteAlpha, Bitmap, BitmapSize, 255);
			}
		}
		else
		{
			TArray<FLinearColor> BitmapHDR;
			if (GetViewportScreenShotHDR(InViewport, BitmapHDR, CaptureRect))
			{
				bool bWriteAlpha = false;
				ClipBitmapDataScreenshotDataEditor(bWriteAlpha, BitmapHDR, BitmapSize, CaptureRect, bCaptureAreaValid);
				bIsScreenshotSaved = RequestSaveScreenshot(bWriteAlpha, BitmapHDR, BitmapSize, 1.0f);
			}
		}

		// Done with the request
		FScreenshotRequest::Reset();
#if UE_SCREENSHOT_TRACE_ENABLED
		FTraceScreenshot::Reset();
#endif
		FScreenshotRequest::OnScreenshotRequestProcessed().Broadcast();

		// Re-enable screen messages - if we are NOT capturing a movie
		GAreScreenMessagesEnabled = GScreenMessagesRestoreState;

		InViewport->InvalidateHitProxy();
	}

	return bIsScreenshotSaved;
}

void FEditorViewportClient::DrawBoundingBox(FBox &Box, FCanvas* InCanvas, const FSceneView* InView, const FViewport* InViewport, const FLinearColor& InColor, const bool bInDrawBracket, const FString &InLabelText)
{
	FVector BoxCenter, BoxExtents;
	Box.GetCenterAndExtents( BoxCenter, BoxExtents );

	// Project center of bounding box onto screen.
	const FVector4 ProjBoxCenter = InView->WorldToScreen(BoxCenter);

	// Do nothing if behind camera
	if( ProjBoxCenter.W > 0.f )
	{
		// Project verts of world-space bounding box onto screen and take their bounding box
		const FVector Verts[8] = {	FVector( 1, 1, 1),
			FVector( 1, 1,-1),
			FVector( 1,-1, 1),
			FVector( 1,-1,-1),
			FVector(-1, 1, 1),
			FVector(-1, 1,-1),
			FVector(-1,-1, 1),
			FVector(-1,-1,-1) };

		const int32 HalfX = 0.5f * InViewport->GetSizeXY().X;
		const int32 HalfY = 0.5f * InViewport->GetSizeXY().Y;

		FVector2D ScreenBoxMin(1000000000, 1000000000);
		FVector2D ScreenBoxMax(-1000000000, -1000000000);

		for(int32 j=0; j<8; j++)
		{
			// Project vert into screen space.
			const FVector WorldVert = BoxCenter + (Verts[j]*BoxExtents);
			FVector2D PixelVert;
			if(InView->ScreenToPixel(InView->WorldToScreen(WorldVert),PixelVert))
			{
				// Update screen-space bounding box with with transformed vert.
				ScreenBoxMin.X = FMath::Min<int32>(ScreenBoxMin.X, PixelVert.X);
				ScreenBoxMin.Y = FMath::Min<int32>(ScreenBoxMin.Y, PixelVert.Y);

				ScreenBoxMax.X = FMath::Max<int32>(ScreenBoxMax.X, PixelVert.X);
				ScreenBoxMax.Y = FMath::Max<int32>(ScreenBoxMax.Y, PixelVert.Y);
			}
		}

		ScreenBoxMax/=InCanvas->GetDPIScale();
		ScreenBoxMin/=InCanvas->GetDPIScale();

		FCanvasLineItem LineItem( FVector2D( 0.0f, 0.0f ), FVector2D( 0.0f, 0.0f ) );
		LineItem.SetColor( InColor );
		if( bInDrawBracket )
		{
			// Draw a bracket when considering the non-current level.
			const float DeltaX = ScreenBoxMax.X - ScreenBoxMin.X;
			const float DeltaY = ScreenBoxMax.X - ScreenBoxMin.X;
			const FIntPoint Offset( DeltaX * 0.2f, DeltaY * 0.2f );

			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMin.X + Offset.X, ScreenBoxMin.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMin.X + Offset.X, ScreenBoxMax.Y) );

			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMax.X - Offset.X, ScreenBoxMin.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMax.X - Offset.X, ScreenBoxMax.Y) );

			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMin.X, ScreenBoxMin.Y + Offset.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMax.X, ScreenBoxMin.Y + Offset.Y) );

			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMin.X, ScreenBoxMax.Y - Offset.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMax.X, ScreenBoxMax.Y - Offset.Y) );
		}
		else
		{
			// Draw a box when considering the current level.
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMin.X, ScreenBoxMax.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMin.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMax.X, ScreenBoxMax.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMax.Y), FVector2D(ScreenBoxMax.X, ScreenBoxMin.Y) );
			LineItem.Draw( InCanvas, FVector2D(ScreenBoxMax.X, ScreenBoxMin.Y), FVector2D(ScreenBoxMin.X, ScreenBoxMin.Y) );
		}


		if (InLabelText.Len() > 0)
		{
			FCanvasTextItem TextItem( FVector2D( ScreenBoxMin.X + ((ScreenBoxMax.X - ScreenBoxMin.X) * 0.5f),ScreenBoxMin.Y), FText::FromString( InLabelText ), GEngine->GetMediumFont(), InColor );
			TextItem.bCentreX = true;
			InCanvas->DrawItem( TextItem );
		}
	}
}

void FEditorViewportClient::DrawActorScreenSpaceBoundingBox( FCanvas* InCanvas, const FSceneView* InView, FViewport* InViewport, AActor* InActor, const FLinearColor& InColor, const bool bInDrawBracket, const FString& InLabelText )
{
	check( InActor != NULL );


	// First check to see if we're dealing with a sprite, otherwise just use the normal bounding box
	UBillboardComponent* Sprite = InActor->FindComponentByClass<UBillboardComponent>();

	FBox ActorBox;
	if( Sprite != NULL )
	{
		ActorBox = Sprite->Bounds.GetBox();
	}
	else
	{
		const bool bNonColliding = true;
		ActorBox = InActor->GetComponentsBoundingBox( bNonColliding );
	}


	// If we didn't get a valid bounding box, just make a little one around the actor location
	if( !ActorBox.IsValid || ActorBox.GetExtent().GetMin() < KINDA_SMALL_NUMBER )
	{
		ActorBox = FBox( InActor->GetActorLocation() - FVector( -20 ), InActor->GetActorLocation() + FVector( 20 ) );
	}

	DrawBoundingBox(ActorBox, InCanvas, InView, InViewport, InColor, bInDrawBracket, InLabelText);
}

void FEditorViewportClient::SetGameView(bool bGameViewEnable)
{
	// backup this state as we want to preserve it
	bool bCompositeEditorPrimitives = EngineShowFlags.CompositeEditorPrimitives;

	// defaults
	FEngineShowFlags GameFlags(ESFIM_Game);
	FEngineShowFlags EditorFlags(ESFIM_Editor);
	{
		// likely we can take the existing state
		if(EngineShowFlags.Game)
		{
			GameFlags = EngineShowFlags;
			EditorFlags = LastEngineShowFlags;
		}
		else if(LastEngineShowFlags.Game)
		{
			GameFlags = LastEngineShowFlags;
			EditorFlags = EngineShowFlags;
		}
	}

	// toggle between the game and engine flags
	if(bGameViewEnable)
	{
		EngineShowFlags = GameFlags;
		LastEngineShowFlags = EditorFlags;
	}
	else
	{
		EngineShowFlags = EditorFlags;
		LastEngineShowFlags = GameFlags;
	}

	// maintain this state
	EngineShowFlags.SetCompositeEditorPrimitives(bCompositeEditorPrimitives);
	LastEngineShowFlags.SetCompositeEditorPrimitives(bCompositeEditorPrimitives);

	//reset game engine show flags that may have been turned on by making a selection in game view
	if(bGameViewEnable)
	{
		EngineShowFlags.SetModeWidgets(true); // Enable "Mode Widgets" by default when entering game mode
		EngineShowFlags.SetSelection(false);
		ShowWidget(false); // Hide the widget
	}

	EngineShowFlags.SetSelectionOutline(bGameViewEnable ? false : GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);

	ApplyViewMode(GetViewMode(), IsPerspective(), EngineShowFlags);

	bInGameViewMode = bGameViewEnable;

	Invalidate();
}


void FEditorViewportClient::SetVREditView(bool bVREditViewEnable)
{
	// backup this state as we want to preserve it
	bool bCompositeEditorPrimitives = EngineShowFlags.CompositeEditorPrimitives;

	// defaults
	FEngineShowFlags VREditFlags(ESFIM_VREditing);
	FEngineShowFlags EditorFlags(ESFIM_Editor);
	{
		// likely we can take the existing state
		if (EngineShowFlags.VREditing)
		{
			VREditFlags = EngineShowFlags;
			EditorFlags = LastEngineShowFlags;
		}
		else if (LastEngineShowFlags.VREditing)
		{
			VREditFlags = LastEngineShowFlags;
			EditorFlags = EngineShowFlags;
		}
	}

	// toggle between the game and engine flags
	if (bVREditViewEnable)
	{
		EngineShowFlags = VREditFlags;
		LastEngineShowFlags = EditorFlags;
	}
	else
	{
		EngineShowFlags = EditorFlags;
		LastEngineShowFlags = VREditFlags;
	}
	// maintain this state
	EngineShowFlags.SetCompositeEditorPrimitives(bCompositeEditorPrimitives);
	LastEngineShowFlags.SetCompositeEditorPrimitives(bCompositeEditorPrimitives);

	//reset game engine show flags that may have been turned on by making a selection in game view
	if (bVREditViewEnable)
	{
		EngineShowFlags.SetModeWidgets(false);
		EngineShowFlags.SetBillboardSprites(false);
	}

	EngineShowFlags.SetSelectionOutline(bVREditViewEnable ? true : GetDefault<ULevelEditorViewportSettings>()->bUseSelectionOutline);

	ApplyViewMode(GetViewMode(), IsPerspective(), EngineShowFlags);

	bInVREditViewMode = bVREditViewEnable;

	Invalidate();
}

FStatUnitData* FEditorViewportClient::GetStatUnitData() const
{
	return &StatUnitData;
}

FStatHitchesData* FEditorViewportClient::GetStatHitchesData() const
{
	return &StatHitchesData;
}

const TArray<FString>* FEditorViewportClient::GetEnabledStats() const
{
	return &EnabledStats;
}

void FEditorViewportClient::SetEnabledStats(const TArray<FString>& InEnabledStats)
{
	HandleViewportStatDisableAll(true);

	EnabledStats = InEnabledStats;
	if (EnabledStats.Num())
	{
		SetShowStats(true);
		AddRealtimeOverride(true, LOCTEXT("RealtimeOverrideMessage_Stats", "Stats Display"));
	}

#if ENABLE_AUDIO_DEBUG
	if (GEngine)
	{
		if (FAudioDeviceManager* DeviceManager = GEngine->GetAudioDeviceManager())
		{
			Audio::FAudioDebugger::ResolveDesiredStats(this);
		}
	}
#endif // ENABLE_AUDIO_DEBUG
}

bool FEditorViewportClient::IsStatEnabled(const FString& InName) const
{
	return EnabledStats.Contains(InName);
}

float FEditorViewportClient::UpdateViewportClientWindowDPIScale() const
{
	float DPIScale = 1.f;
	if(EditorViewportWidget.IsValid())
	{
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(EditorViewportWidget.Pin().ToSharedRef());
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}
	}

	return DPIScale;
}

FMatrix FEditorViewportClient::CalcViewRotationMatrixForControllingActorView(const FRotator& InViewRotation) const
{
	return FInverseRotationMatrix(InViewRotation);
}

FMatrix FEditorViewportClient::CalcViewRotationMatrix(const FRotator& InViewRotation) const
{
	const FViewportCameraTransform& ViewTransform = GetViewTransform();

	if (bUsingOrbitCamera)
	{
		// @todo vreditor: Not stereo friendly yet
		return FTranslationMatrix(ViewTransform.GetLocation()) * ViewTransform.ComputeOrbitMatrix();
	}
	else
	{
		// Create the view matrix
		return FInverseRotationMatrix(InViewRotation);
	}
}

void FEditorViewportClient::EnableOverrideEngineShowFlags(TUniqueFunction<void(FEngineShowFlags&)> OverrideFunc)
{
	OverrideShowFlagsFunc = MoveTemp(OverrideFunc);
}

void FEditorViewportClient::DisableOverrideEngineShowFlags()
{
	OverrideShowFlagsFunc = nullptr;
}

float FEditorViewportClient::GetMinimumOrthoZoom() const
{
	return FMath::Max(GetDefault<ULevelEditorViewportSettings>()->MinimumOrthographicZoom, MIN_ORTHOZOOM);
}

void FEditorViewportClient::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& Event)
{
	if (Event.GetEventType() == ETransactionObjectEventType::Snapshot)
	{
		return;
	}

	// Skip CDOs - calling GetWorld() on a CDO with a Within specifier triggers an ensure.
	if (Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (World && (Object->GetWorld() == World) && !World->IsGameWorld())
	{
		CachedWorldBounds.Init();
	}
}

void FEditorViewportClient::OnPostLoadMap(uint32 MapChangeFlags)
{
	if (GWorld == GetWorld() && !GWorld->IsGameWorld())
	{
		CachedWorldBounds.Init();
		OrthographicFarPlane.Reset();
		OrthographicNearPlane.Reset();
	}
}

////////////////

bool FEditorViewportStats::bInitialized(false);
bool FEditorViewportStats::bUsingCalledThisFrame(false);
FEditorViewportStats::Category FEditorViewportStats::LastUsing(FEditorViewportStats::CAT_MAX);
int32 FEditorViewportStats::DataPoints[FEditorViewportStats::CAT_MAX];

void FEditorViewportStats::Initialize()
{
	if ( !bInitialized )
	{
		bInitialized = true;
		FMemory::Memzero(DataPoints);
	}
}

void FEditorViewportStats::Used(FEditorViewportStats::Category InCategory)
{
	Initialize();
	DataPoints[InCategory] += 1;
}

void FEditorViewportStats::BeginFrame()
{
	Initialize();
	bUsingCalledThisFrame = false;
}

void FEditorViewportStats::Using(Category InCategory)
{
	Initialize();

	bUsingCalledThisFrame = true;

	if ( LastUsing != InCategory )
	{
		LastUsing = InCategory;
		DataPoints[InCategory] += 1;
	}
}

void FEditorViewportStats::NoOpUsing()
{
	Initialize();

	bUsingCalledThisFrame = true;
}

void FEditorViewportStats::EndFrame()
{
	Initialize();

	if ( !bUsingCalledThisFrame )
	{
		LastUsing = FEditorViewportStats::CAT_MAX;
	}
}

void FEditorViewportStats::SendUsageData()
{
	Initialize();

	static_assert(FEditorViewportStats::CAT_MAX == 22, "If the number of categories change you need to add more entries below!");

	TArray<FAnalyticsEventAttribute> PerspectiveUsage;
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.WASD"),			DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_WASD]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.UpDown"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_UP_DOWN]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.FovZoom"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_KEYBOARD_FOV_ZOOM]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Dolly"),			DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_DOLLY]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Pan"),				DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_PAN]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Scroll"),			DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_SCROLL]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Rotation"),	DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_ROTATION]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Pan"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_PAN]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Zoom"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_MOUSE_ORBIT_ZOOM]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Gesture.Scroll"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_GESTURE_SCROLL]));
	PerspectiveUsage.Add(FAnalyticsEventAttribute(FString("Gesture.Magnify"),		DataPoints[FEditorViewportStats::CAT_PERSPECTIVE_GESTURE_MAGNIFY]));

	TArray<FAnalyticsEventAttribute> OrthographicUsage;
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.WASD"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_KEYBOARD_WASD]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.UpDown"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_KEYBOARD_UP_DOWN]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Keyboard.FovZoom"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_KEYBOARD_FOV_ZOOM]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Zoom"),			DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ZOOM]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Pan"),			DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_PAN]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Scroll"),			DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_SCROLL]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Rotation"), DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ROTATION]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Pan"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_PAN]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Mouse.Orbit.Zoom"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_MOUSE_ORBIT_ZOOM]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Gesture.Scroll"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_GESTURE_SCROLL]));
	OrthographicUsage.Add(FAnalyticsEventAttribute(FString("Gesture.Magnify"),		DataPoints[FEditorViewportStats::CAT_ORTHOGRAPHIC_GESTURE_MAGNIFY]));

	FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.Viewport.Perspective"), PerspectiveUsage);
	FEngineAnalytics::GetProvider().RecordEvent(FString("Editor.Usage.Viewport.Orthographic"), OrthographicUsage);

	// Clear all the usage data in case we do it twice.
	FMemory::Memzero(DataPoints);
}

UE_DEFINE_TCOMMANDS(FViewportNavigationCommands)

FViewportNavigationCommands::FViewportNavigationCommands()
	: TCommands<FViewportNavigationCommands>(
		"EditorViewportClient", // Context name for fast lookup
		NSLOCTEXT("Contexts", "ViewportNavigation", "Viewport Navigation"), // Localized context name for displaying
		FName(),
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FViewportNavigationCommands::RegisterCommands()
{
	UI_COMMAND(Forward, "Forward", "Moves the camera Forward", EUserInterfaceActionType::Button, FInputChord(EKeys::W));
	UI_COMMAND(Backward, "Backward", "Moves the camera Backward", EUserInterfaceActionType::Button, FInputChord(EKeys::S));
	UI_COMMAND(Left, "Left", "Moves the camera Left", EUserInterfaceActionType::Button, FInputChord(EKeys::A));
	UI_COMMAND(Right, "Right", "Moves the camera Right", EUserInterfaceActionType::Button, FInputChord(EKeys::D));

	UI_COMMAND(WorldUp, "World Up", "Moves the camera Up in world space", EUserInterfaceActionType::Button, FInputChord(EKeys::E));
	UI_COMMAND(WorldDown, "World Down", "Moves the camera Down in world space", EUserInterfaceActionType::Button, FInputChord(EKeys::Q));
	UI_COMMAND(LocalUp, "Local Up", "Moves the camera Up in local space", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
	UI_COMMAND(LocalDown, "Local Down", "Moves the camera Down in local space", EUserInterfaceActionType::Button, FInputChord(EKeys::F));

	UI_COMMAND(FovZoomIn, "FOV Zoom In", "Narrows the camers FOV", EUserInterfaceActionType::Button, FInputChord(EKeys::C));
	UI_COMMAND(FovZoomOut, "FOV Zoom Out", "Widens the camera FOV", EUserInterfaceActionType::Button, FInputChord(EKeys::Z));

	UI_COMMAND(RotateUp, "Rotate Up", "Rotates the camera Up", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RotateDown, "Rotate Down", "Rotates the camera Down", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RotateLeft, "Rotate Left", "Rotates the camera Left", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RotateRight, "Rotate Right", "Rotates the camera Right", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
