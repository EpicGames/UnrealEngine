// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditorViewportCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Selection.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "ContentBrowserModule.h"
#include "EditorViewportClient.h"
#include "IContentBrowserSingleton.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "Engine/Texture2D.h"
#include "GPUSkinCacheVisualizationMenuCommands.h"

#define LOCTEXT_NAMESPACE "EditorViewportCommands"

namespace
{
	const FName ShowTextureScaleBundle = "ShowTextureScale";
	const FName ShowTextureResolutionBundle = "ShowTextureResolution";
}

UE_DEFINE_TCOMMANDS(FEditorViewportCommands)

FEditorViewportCommands::FEditorViewportCommands()
	: TCommands<FEditorViewportCommands>
	(
		TEXT("EditorViewport"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "EditorViewportCommands", "Common Viewport Commands"), // Localized context name for displaying
		TEXT("MainFrame"),
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
	AddBundle(ShowTextureScaleBundle, LOCTEXT("ShowTextureCommands_TextureScale_Bundle", "Show Texture Scale"));
	AddBundle(ShowTextureResolutionBundle, LOCTEXT("ShowTextureCommands_TextureResolution_Bundle", "Show Texture Resolution"));
}

void FEditorViewportCommands::RegisterCommands()
{
	UI_COMMAND( Perspective, "Perspective", "Switches the viewport to perspective view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::G ) );
	UI_COMMAND( Front, "Front", "Switches the viewport to front view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::H ) );
	UI_COMMAND( Back, "Back", "Switches the viewport to back view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt | EModifierKey::Shift, EKeys::H ) );
	UI_COMMAND( Top, "Top", "Switches the viewport to top view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::J ) );
	UI_COMMAND( Bottom, "Bottom", "Switches the viewport to bottom view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt | EModifierKey::Shift, EKeys::J ) );
	UI_COMMAND( Left, "Left", "Switches the viewport to left view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::K ) );
	UI_COMMAND( Right, "Right", "Switches the viewport to right view", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt | EModifierKey::Shift, EKeys::K ) );
	UI_COMMAND( Next, "Next", "Rotate through each view options", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Control | EModifierKey::Shift, EKeys::SpaceBar ) );

	UI_COMMAND(ViewportConfig_OnePane, "Layout One Pane", "Changes the viewport arrangement to one pane", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_TwoPanesH, "Layout Two Panes (horizontal)", "Changes the viewport arrangement to two panes, side-by-side", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_TwoPanesV, "Layout Two Panes (vertical)", "Changes the viewport arrangement to two panes, one above the other", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_ThreePanesLeft, "Layout Three Panes (one left, two right)", "Changes the viewport arrangement to three panes, one on the left, two on the right", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_ThreePanesRight, "Layout Three Panes (one right, two left)", "Changes the viewport arrangement to three panes, one on the right, two on the left", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_ThreePanesTop, "Layout Three Panes (one top, two bottom)", "Changes the viewport arrangement to three panes, one on the top, two on the bottom", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_ThreePanesBottom, "Layout Three Panes (one bottom, two top)", "Changes the viewport arrangement to three panes, one on the bottom, two on the top", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_FourPanesLeft, "Layout Four Panes (one left, three right)", "Changes the viewport arrangement to four panes, one on the left, three on the right", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_FourPanesRight, "Layout Four Panes (one right, three left)", "Changes the viewport arrangement to four panes, one on the right, three on the left", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_FourPanesTop, "Layout Four Panes (one top, three bottom)", "Changes the viewport arrangement to four panes, one on the top, three on the bottom", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_FourPanesBottom, "Layout Four Panes (one bottom, three top)", "Changes the viewport arrangement to four panes, one on the bottom, three on the top", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ViewportConfig_FourPanes2x2, "Layout Four Panes (2x2)", "Changes the viewport arrangement to four panes, in a 2x2 grid", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( WireframeMode, "Brush Wireframe View Mode", "Renders the scene in brush wireframe", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Two ) );
	UI_COMMAND( UnlitMode, "Unlit View Mode", "Renders the scene with no lights", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Three ) );
	UI_COMMAND( LitMode, "Lit View Mode", "Renders the scene with normal lighting", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Four ) );
	UI_COMMAND( LitWireframeMode, "Lit Wireframe View Mode", "Renders the scene with normal lighting and a wireframe overlay", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( PathTracingMode, "PathTracing View Mode", "Renders the scene using a path tracer", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( RayTracingDebugMode, "Ray tracing Debug View Mode", "Renders the scene in ray tracing debug mode", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( DetailLightingMode, "Detail Lighting View Mode", "Renders the scene with detailed lighting only", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Five ) );
	UI_COMMAND( LightingOnlyMode, "Lighting Only View Mode", "Renders the scene with lights only, no textures", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Six ) );
	UI_COMMAND( LightComplexityMode, "Light Complexity View Mode", "Renders the scene with light complexity visualization", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Seven ) );
	UI_COMMAND( ShaderComplexityMode, "Shader Complexity View Mode", "Renders the scene with shader complexity visualization", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Eight ) );
	UI_COMMAND( QuadOverdrawMode, "Quad Complexity View Mode", "Renders the scene with quad complexity visualization", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShaderComplexityWithQuadOverdrawMode, "Shader Complexity & Quads visualization", "Renders the scene with shader complexity and quad overdraw visualization", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( TexStreamAccPrimitiveDistanceMode, "Primitive Distance Accuracy View Mode", "Visualize the accuracy of the primitive distance computed for texture streaming", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( TexStreamAccMeshUVDensityMode, "Mesh UV Densities Accuracy View Mode", "Visualize the accuracy of the mesh UV densities computed for texture streaming", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( TexStreamAccMeshUVDensityAll, "All UV Channels", "Visualize the densities accuracy of all UV channels", EUserInterfaceActionType::RadioButton, FInputChord() );

	for (int32 TexCoordIndex = 0; TexCoordIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++TexCoordIndex)
	{
		const FName CommandName = *FString::Printf(TEXT("ShowUVChannel%d"), TexCoordIndex);
		const FText LocalizedName = FText::Format(LOCTEXT("ShowTexCoordCommands", "UV Channel {0}"), TexCoordIndex);
		const FText LocalizedTooltip = FText::Format(LOCTEXT("ShowTexCoordCommands_ToolTip","Visualize the size accuracy of UV density for channel {0}"), TexCoordIndex);

		TexStreamAccMeshUVDensitySingle[TexCoordIndex] = FUICommandInfoDecl(this->AsShared(), CommandName, LocalizedName, LocalizedTooltip).UserInterfaceType(EUserInterfaceActionType::RadioButton);
	}

	UI_COMMAND( TexStreamAccMaterialTextureScaleMode, "Material Texture Scales Accuracy View Mode", "Visualize the accuracy of the material texture scales used for texture streaming", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( TexStreamAccMaterialTextureScaleAll, "All Textures", "Visualize the scales accuracy of all textures", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( RequiredTextureResolutionMode, "Required Texture Resolution View Mode", "Visualize the ratio between the currently streamed texture resolution and the resolution wanted by the GPU", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( StationaryLightOverlapMode, "Stationary Light Overlap View Mode", "Visualizes overlap of stationary lights", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( LightmapDensityMode, "Lightmap Density View Mode", "Renders the scene with lightmap density visualization", EUserInterfaceActionType::RadioButton, FInputChord( EModifierKey::Alt, EKeys::Zero ) );

	UI_COMMAND( GroupLODColorationMode, "Level of Detail Coloration View Mode", "Renders the scene using Level of Detail visualization", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( LODColorationMode, "LOD Coloration View Mode", "Renders the scene using LOD color visualization", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( HLODColorationMode, "HLOD Coloration View Mode", "Renders the scene using HLOD color visualization", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( GroupStreamingDeficitMode, "Streaming Deficit View Mode", "Renders the scene using streaming deficit visualization", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( StreamingTextureDeficitMode, "Streaming Texture Deficit", "Colors primitives by texture streaming deficit (WantedMips vs ResidentMips)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( StreamingTextureResidencyMode, "Streaming Texture Residency", "Colors primitives by texture residency deficit (MaxLODs vs ResidentMips)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( StreamingMeshLODDeficitMode, "Streaming Mesh LOD Deficit", "Colors primitives by mesh LOD streaming deficit (WantedMips vs ResidentMips)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( StreamingMeshLODResidencyMode, "Streaming Mesh LOD Residency", "Colors primitives by mesh LOD residency deficit (MaxLODs vs ResidentLODs)", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( VisualizeGPUSkinCacheMode, "Skin Cache Visualization View Mode", "Visualizes various aspects of Skin ache", EUserInterfaceActionType::RadioButton, FInputChord());
	FGPUSkinCacheVisualizationMenuCommands::Register();

	UI_COMMAND( VisualizeLWCComplexity, "LWC Complexity Visualization View Mode", "Visualizes LWC usages in materials", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( VisualizeBufferMode, "Buffer Visualization View Mode", "Renders a set of selected post process materials, which visualize various intermediate render buffers (material attributes)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeNaniteMode, "Nanite Visualization View Mode", "Visualizes various rendering aspects of Nanite.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeLumenMode, "Lumen Visualization View Mode", "Visualizes Lumen debug views.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeMegaLightsMode, "MegaLights Visualization View Mode", "Visualizes MegaLights debug views.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeSubstrateMode, "Substrate Visualization View Mode", "Visualizes Substrate debug views.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeGroomMode, "Groom Visualization View Mode", "Visualizes Groom debug views.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeVirtualShadowMapMode, "Virtual Shadow Map Visualization View Mode", "Visualizes various rendering aspects of virtual shadow maps.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( VisualizeVirtualTextureMode, "Virtual Texture Visualization View Mode", "Visualizes various rendering aspects of virtual textures", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ReflectionOverrideMode, "Reflections View Mode", "Renders the scene with reflections only", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( CollisionPawn, "Player Collision", "Renders player collision visualization", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( CollisionVisibility, "Visibility Collision", "Renders visibility collision visualization", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( GeometryInspectionClayMode, "Geometry Inspection Clay Mode", "Displays using clay material to evaluate surface curvature and smoothness.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( GeometryInspectionZebraHorizontalMode, "Horizontal", "Displays simulated light reflections using stripes to evaluate surface curvature and smoothness.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( GeometryInspectionZebraVerticalMode, "Vertical", "Displays simulated light reflections using stripes to evaluate surface curvature and smoothness.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( GeometryInspectionFrontBackFaceMode, "Geometry Inspection Front/Back Face Mode", "Highlights front-facing and back-facing polygons differently to evaluate face normals.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( GeometryInspectionRandomColorMode, "Geometry Inspection Random Color Mode", "Assigns random colors to individual elements or components to improve visual distinction between overlapping or adjacent parts.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( GeometryInspectionShowWireframe, "Use Wireframe", "Use Wireframe in Geometry Inspection Modes.", EUserInterfaceActionType::Check, FInputChord());

	UI_COMMAND(AllChannelsMask, "RGB", "Show all color channels", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(RedChannelMask, "Red", "Show only the red color channel", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(GreenChannelMask, "Green", "Show only the green color channel", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(BlueChannelMask, "Blue", "Show only the blue color channel", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(AlphaChannelMask, "Alpha", "Show only the alpha color channel", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ToggleRealTime, "Realtime", "Toggles real time rendering in this viewport", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Control, EKeys::R ) );
	UI_COMMAND( ToggleStats, "Show Stats", "Toggles the ability to show stats in this viewport (enables realtime)", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::L ) );
	UI_COMMAND( ToggleFPS, "Show FPS", "Toggles showing frames per second in this viewport (enables realtime)", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Control|EModifierKey::Shift, EKeys::H ) );

	UI_COMMAND( ScreenCapture, "Screen Capture", "Take a screenshot of the active viewport.", EUserInterfaceActionType::Button, FInputChord(EKeys::F9) );
	UI_COMMAND( ScreenCaptureForProjectThumbnail, "Update Project Thumbnail", "Take a screenshot of the active viewport for use as the project thumbnail.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( IncrementPositionGridSize, "Grid Size (Position): Increment", "Increases the position grid size setting by one", EUserInterfaceActionType::Button, FInputChord( EKeys::RightBracket ) );
	UI_COMMAND( DecrementPositionGridSize, "Grid Size (Position): Decrement", "Decreases the position grid size setting by one", EUserInterfaceActionType::Button, FInputChord( EKeys::LeftBracket ) );
	UI_COMMAND( IncrementRotationGridSize, "Grid Size (Rotation): Increment", "Increases the rotation grid size setting by one", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::RightBracket ) );
	UI_COMMAND( DecrementRotationGridSize, "Grid Size (Rotation): Decrement", "Decreases the rotation grid size setting by one", EUserInterfaceActionType::Button, FInputChord( EModifierKey::Shift, EKeys::LeftBracket ) );

	UI_COMMAND(SelectMode, "Select Mode", "Select objects", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::Q));
	UI_COMMAND( TranslateMode, "Translate Mode", "Select and translate objects", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::W) );
	UI_COMMAND( RotateMode, "Rotate Mode", "Select and rotate objects", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::E) );
	UI_COMMAND( ScaleMode, "Scale Mode", "Select and scale objects", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::R) );
	UI_COMMAND( TranslateRotateMode, "Combined Translate and Rotate Mode", "Select and translate or rotate objects", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( TranslateRotate2DMode, "2D Mode", "Select and translate or rotate objects in 2D", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ShrinkTransformWidget, "Shrink Transform Widget", "Shrink the level editor transform widget", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::LeftBracket) );
	UI_COMMAND( ExpandTransformWidget, "Expand Transform Widget", "Expand the level editor transform widget", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::RightBracket) );

	UI_COMMAND(
		RelativeCoordinateSystem_World,
		"World Space",
		"Move and rotate objects relative to the cardinal world axes",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);
	UI_COMMAND(
		RelativeCoordinateSystem_Local,
		"Local Space",
		"Move and rotate objects relative to the object's local axes",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);
	UI_COMMAND(
		RelativeCoordinateSystem_Parent,
		"Parent Space",
		"Move and rotate objects relative to the parent's axes",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);
	UI_COMMAND(
		RelativeCoordinateSystem_Explicit,
		"Explicit Space",
		"Move objects relative to the parent's axes and rotate them using gimbal axes",
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

#if PLATFORM_MAC
	UI_COMMAND( CycleTransformGizmoCoordSystem, "Cycle Transform Coordinate System", "Cycles the transform gizmo coordinate systems between world and local (object) space", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Command, EKeys::Tilde));
#else
	UI_COMMAND( CycleTransformGizmoCoordSystem, "Cycle Transform Coordinate System", "Cycles the transform gizmo coordinate systems between world and local (object) space", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::Tilde));
#endif
	UI_COMMAND( CycleTransformGizmos, "Cycle Between Translate, Rotate, and Scale", "Cycles the transform gizmos between translate, rotate, and scale", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar) );
	
	UI_COMMAND( IncreaseGizmoSize, "Increase Gizmo Size", "Increases the size of the translation gizmo.", EUserInterfaceActionType::Button, FInputChord(EKeys::Equals, EModifierKey::Shift) );
	UI_COMMAND( DecreaseGizmoSize, "Decrease Gizmo Size", "Decreases the size of the translation gizmo.", EUserInterfaceActionType::Button, FInputChord(EKeys::Hyphen, EModifierKey::Shift) );
	UI_COMMAND( ResetGizmoSize, "Reset Gizmo Size", "Resets the size of the translation gizmo to its default state.", EUserInterfaceActionType::Button, FInputChord(EKeys::Equals) );
	
	UI_COMMAND( FocusAllViewportsToSelection, "Frame Selected Actors in All Viewports", "Centers all open viewports on the selected actors", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Shift) );
	UI_COMMAND( FocusViewportToSelection, "Frame Selected", "Centers the viewport on the selected actors", EUserInterfaceActionType::Button, FInputChord( EKeys::F ) );
	UI_COMMAND( FocusAndClipViewportToSelection, "Frame & Clip To Selected", "Centers the viewport on the selected actors and clips the near view plane to them", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control | EModifierKey::Alt) );
	UI_COMMAND( FocusOutlinerToSelection, "Frame Selected in Outliner", "Focuses the selected actors in all open outliners", EUserInterfaceActionType::Button, FInputChord() );
	UI_COMMAND( FocusOutlinerToContextFolder, "Frame Current Folder in Outliner", "Focuses the current folder (if it's set) in the outliner", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( UseOrthographicClippingPlanes, "Ortho Clipping Planes", "Whether or not the depth of the orthographic view should be clipped.", EUserInterfaceActionType::Check, FInputChord() );
	UI_COMMAND( DismissOrthographicClippingPlanes, "Dismiss Ortho Clipping Planes", "Resets the orthographic clipping planes and hides the visualization of viewport depth.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND( Snapping, "Snapping", "Enables or disables all snapping", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( LocationGridSnap, "Grid Snap", "Toggle whether objects snap to a grid when you move them.\nUse the submenu to choose the grid size.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( RotationGridSnap, "Rotation Snap", "Toggle whether objects snap to preset angles when you rotate them.\nUse the submenu to choose the angle increment.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( Layer2DSnap, "Layer2D Snap", "Enables or disables snapping objects to a 2D layer", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ScaleGridSnap, "Scale Snap", "Toggle whether objects resize by a preset ratio when you scale them.\nUse the submenu to choose the scaling increment.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( CurrentTRSSnapToggle, "Toggle Current TRS Snap", "Temporarily enables or disables snapping for the current transform mode while held; quick-press to toggle permanently.", EUserInterfaceActionType::Button, FInputChord( EKeys::X ) );
	UI_COMMAND( SurfaceSnapping, "Surface Snapping", "Toggle whether objects snap to surfaces when you drag them.\nUse the submenu to set up the snapping behavior.", EUserInterfaceActionType::ToggleButton, FInputChord( EKeys::Z ) );
	UI_COMMAND( RotateToSurfaceNormal, "Rotate to Surface Normal", "If enabled, actors rotation will match that of their snapping surface", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ToggleAutoExposure, "Auto", "If enabled, enables automatic exposure", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ToggleInGameExposure, "Game Settings", "If enabled, uses game settings", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND(ToggleOverrideViewportScreenPercentage, "Custom Override", "Overrides the screen percentage of this viewport with custom value", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(OpenEditorPerformanceProjectSettings, "Project Settings...", "Opens the project settings for viewport performance", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenEditorPerformanceEditorPreferences, "Editor Preferences...", "Opens the editor preferences for viewport performance", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleInViewportContextMenu, "In-Viewport Context Menu", "Shows a contextual menu of key properties and actions for the selected items in the viewport.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Tab));

	UI_COMMAND(ToggleDistanceBasedCameraSpeed, "Distance Based Camera Speed", "If enabled, scale the perspective camera speed based on the distance between the camera and its look-at position.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE

#define LOCTEXT_NAMESPACE "EditorViewModeOptionsMenu"

FText GetViewModeOptionsMenuLabel(EViewModeIndex ViewModeIndex)
{
	if (ViewModeIndex == VMI_MeshUVDensityAccuracy)
	{
		return LOCTEXT("ViewParamMenuTitle_UVChannels", "UV Channels");
	}
	else if (ViewModeIndex == VMI_MaterialTextureScaleAccuracy || ViewModeIndex == VMI_RequiredTextureResolution)
	{
		// Get form the content browser
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			TArray<FAssetData> SelectedAssetData;
			ContentBrowserModule.Get().GetSelectedAssets(SelectedAssetData);

			for ( auto AssetIt = SelectedAssetData.CreateConstIterator(); AssetIt; ++AssetIt )
			{
				// Grab the object if it is loaded
				if (AssetIt->IsAssetLoaded())
				{
					const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetIt->GetAsset());
					if (MaterialInterface)
					{
						return LOCTEXT("ViewParamMenuTitle_TexturesFromContentBrowser", "Textures (Content Browser)");
					}
				}
			}
		}

		// Get form the selection
		{
			TArray<UPrimitiveComponent*> SelectedComponents;
			GEditor->GetSelectedComponents()->GetSelectedObjects(SelectedComponents);

			TArray<AActor*> SelectedActors;
			GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
			for (AActor* Actor : SelectedActors)
			{
				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
					{
						SelectedComponents.Add(PrimComp);
					}
				}
			}

			for (const UPrimitiveComponent* SelectedComponent : SelectedComponents)
			{
				if (SelectedComponent)
				{
					const int32 NumMaterials = SelectedComponent->GetNumMaterials();
					for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
					{
						const UMaterialInterface* MaterialInterface = SelectedComponent->GetMaterial(MaterialIndex);
						if (MaterialInterface)
						{
							return LOCTEXT("ViewParamMenuTitle_TexturesFromSceneSelection", "Textures (Scene Selection)");
						}
					}
				}
			}
		}
		
		return LOCTEXT("ViewParamMenuTitle_Textures", "Textures");
	}
	else
	{

	}
	return LOCTEXT("ViewParamMenuTitle", "View Mode Options");
}

static void GetSelectedMaterials(TArray<const UMaterialInterface*>& SelectedMaterials)
{
	// Get selected materials form the content browser
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FAssetData> SelectedAssetData;
	ContentBrowserModule.Get().GetSelectedAssets(SelectedAssetData);

	for ( auto AssetIt = SelectedAssetData.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		// Grab the object if it is loaded
		if (AssetIt->IsAssetLoaded())
		{
			const UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(AssetIt->GetAsset());
			if (MaterialInterface)
			{
				SelectedMaterials.AddUnique(MaterialInterface);
			}
		}
	}

	// Get selected materials from component selection and actors
	if (!SelectedMaterials.Num())
	{
		TArray<UPrimitiveComponent*> SelectedComponents;
		GEditor->GetSelectedComponents()->GetSelectedObjects(SelectedComponents);

		TArray<AActor*> SelectedActors;
		GEditor->GetSelectedActors()->GetSelectedObjects(SelectedActors);
		for (AActor* Actor : SelectedActors)
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
				{
					SelectedComponents.Add(PrimComp);
				}
			}
		}

		for (const UPrimitiveComponent* SelectedComponent : SelectedComponents)
		{
			if (SelectedComponent)
			{
				const int32 NumMaterials = SelectedComponent->GetNumMaterials();
				for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
				{
					const UMaterialInterface* MaterialInterface = SelectedComponent->GetMaterial(MaterialIndex);
					if (MaterialInterface)
					{
						SelectedMaterials.AddUnique(MaterialInterface);
					}
				}
			}
		}
	}
}

static void AppendTextureStreamingInfoToMenu(const UMaterialInterface* MaterialInterface, bool bSingleMaterial, TMap<int32, TArray<FString>>& DataPerTextureIndex)
{
	check(MaterialInterface);
	for (const FMaterialTextureInfo& TextureData : MaterialInterface->GetTextureStreamingData())
	{
		if (TextureData.IsValid(true))
		{
			if (bSingleMaterial)
			{
				DataPerTextureIndex.FindOrAdd(TextureData.TextureIndex).AddUnique(FString::Printf(TEXT("%.2f X UV%d : %s"), TextureData.SamplingScale, TextureData.UVChannelIndex, *TextureData.TextureName.ToString()));
			}
			else
			{
				DataPerTextureIndex.FindOrAdd(TextureData.TextureIndex).AddUnique(FString::Printf(TEXT("%.2f X UV%d : %s.%s"), TextureData.SamplingScale, TextureData.UVChannelIndex, *MaterialInterface->GetName(), *TextureData.TextureName.ToString()));
			}
		}
	}

	for (const FMaterialTextureInfo& MissingEntry : MaterialInterface->TextureStreamingDataMissingEntries)
	{
		if (bSingleMaterial)
		{
			DataPerTextureIndex.FindOrAdd(MissingEntry.TextureIndex).AddUnique(FString::Printf(TEXT("Missing : %s"), *MissingEntry.TextureName.ToString()));
		}
		else
		{
			DataPerTextureIndex.FindOrAdd(MissingEntry.TextureIndex).AddUnique(FString::Printf(TEXT("Missing : %s.%s"), *MaterialInterface->GetName(), *MissingEntry.TextureName.ToString()));
		}
	}
}

struct FMenuDataPerTexture
{
	// Menu display name.
	FString DisplayName;
	// Menu tool tip lines.
	TArray<FString> ToolTips;
};

static void AppendMaterialInfoToMenu(const UMaterialInterface* MaterialInterface, ERHIFeatureLevel::Type FeatureLevel, const FString& MenuName, TMap<int32, TArray<FString> >& DataPerTextureIndex, TMap<FName, FMenuDataPerTexture>& DataPerTextureName)
{
	check(MaterialInterface);
	const FMaterialResource* Material = MaterialInterface->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(FeatureLevel));
	if (Material)
	{
		const FUniformExpressionSet& UniformExpressions = Material->GetUniformExpressions();
		EMaterialTextureParameterType TextureTypes[] = { EMaterialTextureParameterType::Standard2D, EMaterialTextureParameterType::Virtual };
		for (EMaterialTextureParameterType TextureType : TextureTypes)
		{
			for (int32 i = 0; i < UniformExpressions.GetNumTextures(TextureType); ++i)
			{
				UTexture* Texture = nullptr;
				UniformExpressions.GetGameThreadTextureValue(TextureType, i, MaterialInterface, *Material, Texture, true);

				const UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
				if (Texture2D)
				{
					const FMaterialTextureParameterInfo& Parameter = UniformExpressions.GetTextureParameter(TextureType, i);
					
					DataPerTextureIndex.FindOrAdd(Parameter.TextureIndex).AddUnique(FString::Printf(TEXT("%s.%s"), *MaterialInterface->GetName(), *Texture2D->GetName()));
					
					FMenuDataPerTexture& MenuData = DataPerTextureName.FindOrAdd(*Texture2D->GetName());
					if (MenuData.DisplayName.IsEmpty())
					{
						MenuData.DisplayName = *Texture2D->GetName();
						if (TextureType == EMaterialTextureParameterType::Virtual)
						{
							MenuData.DisplayName += TEXT(" [VT]");
						}
					}
					MenuData.ToolTips.AddUnique(FString::Printf(TEXT("%s %d : %s"), *MenuName, Parameter.TextureIndex, *MaterialInterface->GetName()));
				}
			}
		}
	}
}

TSharedRef<SWidget> BuildViewModeOptionsMenu(TSharedPtr<FUICommandList> CommandList, EViewModeIndex ViewModeIndex, ERHIFeatureLevel::Type FeatureLevel, TMap<int32, FName>& ParamNameMap)
{
	FMenuBuilder MenuBuilder( true, CommandList );
	const UE::UnrealEd::FViewModeOptionsArgs ViewModeArgs
	{
		.ViewModeIndex = ViewModeIndex,
		.FeatureLevel = FeatureLevel,
		.ParamNameMap = ParamNameMap,
		.ViewportClient = nullptr // Use UE::UnrealEd::BuildViewModeOptionsMenu instead
	};
	BuildViewModeOptionsMenu(MenuBuilder, ViewModeArgs);
	return MenuBuilder.MakeWidget();
}

void UE::UnrealEd::BuildViewModeOptionsMenu(FMenuBuilder& MenuBuilder, const FViewModeOptionsArgs& InArgs)
{
	const FEditorViewportCommands& Commands = FEditorViewportCommands::Get();
	const FString MenuName = LOCTEXT("TexStreamAccMaterialTextureScaleSingleDisplayName", "Texture").ToString();

	InArgs.ParamNameMap.Reset();

	if (InArgs.ViewModeIndex == VMI_MeshUVDensityAccuracy)
	{
		MenuBuilder.AddMenuEntry(Commands.TexStreamAccMeshUVDensityAll, NAME_None, LOCTEXT("TexStreamAccMeshUVDensityAllDisplayName", "All UV Channels"));
		for (int32 TexCoordIndex = 0; TexCoordIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++TexCoordIndex)
		{
			MenuBuilder.AddMenuEntry(Commands.TexStreamAccMeshUVDensitySingle[TexCoordIndex], NAME_None, FText::FromString(FString::Printf(TEXT("%s %d"), *MenuName, TexCoordIndex)));
		}
	}
	else if (InArgs.ViewModeIndex == VMI_MaterialTextureScaleAccuracy || InArgs.ViewModeIndex == VMI_RequiredTextureResolution)
	{
		TArray<const UMaterialInterface*> SelectedMaterials;
		GetSelectedMaterials(SelectedMaterials);

		TMap<int32, TArray<FString> > DataPerTextureIndex;
		TMap<FName, FMenuDataPerTexture> DataPerTextureName;

		if (InArgs.ViewModeIndex == VMI_MaterialTextureScaleAccuracy)
		{
			MenuBuilder.AddMenuEntry(Commands.TexStreamAccMaterialTextureScaleAll, NAME_None, LOCTEXT("TexStreamAccMaterialTextureScaleAllDisplayName", "All Textures"));

			for (const UMaterialInterface* MaterialInterface : SelectedMaterials)
			{
				AppendTextureStreamingInfoToMenu(MaterialInterface, SelectedMaterials.Num() == 1, DataPerTextureIndex);
			}
		}
		else // VMI_RequiredTextureResolution
		{
			for (const UMaterialInterface* MaterialInterface : SelectedMaterials)
			{
				AppendMaterialInfoToMenu(MaterialInterface, InArgs.FeatureLevel, MenuName, DataPerTextureIndex, DataPerTextureName);
			}
		}

		if (InArgs.ViewportClient.IsValid())
		{
			TSharedRef<FEditorViewportClient> ViewportClient = InArgs.ViewportClient.ToSharedRef();
			auto GetUIAction = [&ViewportClient](int32 InViewModeIndex)
			{
				return FUIAction(
					FExecuteAction::CreateSP( ViewportClient, &FEditorViewportClient::SetViewModeParam, InViewModeIndex ),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP( ViewportClient, &FEditorViewportClient::IsViewModeParam, InViewModeIndex )
				);
			};

			// If there is not too many textures, show the data per name.
			if (DataPerTextureName.Num() > 0 && DataPerTextureName.Num() < TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL)
			{
				TMap<FString, FName> SortedFNames;
				for (auto Ite = DataPerTextureName.CreateConstIterator(); Ite; ++Ite)
				{
					SortedFNames.Add(Ite.Key().ToString(), Ite.Key());
				}
				SortedFNames.KeySort(TLess<FString>());

				int32 CommandIndex = 0;
				for (auto Ite = SortedFNames.CreateConstIterator(); Ite; ++Ite)
				{ 
					const FMenuDataPerTexture& MaterialInfos = DataPerTextureName.FindOrAdd(Ite.Value());
					FString ToolTipOverride = MaterialInfos.ToolTips[0];
					for (int32 MaterialIndex = 1; MaterialIndex < MaterialInfos.ToolTips.Num(); ++MaterialIndex)
					{
						ToolTipOverride = FString::Printf(TEXT("%s\n%s"), *ToolTipOverride, *MaterialInfos.ToolTips[MaterialIndex]);
					}
					
					MenuBuilder.AddMenuEntry(FText::FromString(MaterialInfos.DisplayName), FText::FromString(ToolTipOverride), FSlateIcon(), GetUIAction(CommandIndex), NAME_None, EUserInterfaceActionType::RadioButton);
					
					InArgs.ParamNameMap.Add(CommandIndex, Ite.Value());
					++CommandIndex;
				}
			}
			else if (DataPerTextureIndex.Num() > 0) // Otherwise show the data per index, with extra info (also hidding entries with no info)
			{
				for (int32 TextureIndex = 0; TextureIndex < TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL; ++TextureIndex)
				{
					const TArray<FString>* TextureDataAtIndex = DataPerTextureIndex.Find(TextureIndex);
					if (TextureDataAtIndex && TextureDataAtIndex->Num() > 0)
					{
						FString NameOverride = FString::Printf(TEXT("%s %d (%s)"), *MenuName, TextureIndex, *(*TextureDataAtIndex)[0]);
						if (TextureDataAtIndex->Num() > 1)
						{
							NameOverride.Append(TEXT(" ..."));
						}
						FString ToolTipOverride = (*TextureDataAtIndex)[0];
						for (int32 Index = 1; Index < (*TextureDataAtIndex).Num(); ++Index)
						{
							ToolTipOverride = FString::Printf(TEXT("%s\n%s"), *ToolTipOverride, *(*TextureDataAtIndex)[Index]);
						}
						MenuBuilder.AddMenuEntry(FText::FromString(NameOverride), FText::FromString(ToolTipOverride), FSlateIcon(), GetUIAction(TextureIndex), NAME_None, EUserInterfaceActionType::RadioButton);
					}
				}
			}
			else // If nothing selected, just display a list of name
			{
				for (int32 TextureIndex = 0; TextureIndex < TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL; ++TextureIndex)
				{
					const FText Label = FText::FromString(FString::Printf(TEXT("%s %d"), *MenuName, TextureIndex));
					MenuBuilder.AddMenuEntry(Label, Label, FSlateIcon(), GetUIAction(TextureIndex), NAME_None, EUserInterfaceActionType::RadioButton);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
