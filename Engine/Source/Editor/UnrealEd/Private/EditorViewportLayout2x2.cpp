// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayout2x2.h"
#include "Framework/Docking/LayoutService.h"
#include "Editor.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SSplitter.h"

namespace ViewportLayout2x2Defs
{
	/** Default 2x2 splitters to equal 50/50 splits */
	static const FVector2D DefaultSplitterPercentages(0.5f, 0.5f);
}

// FLevelViewportLayout2x2 //////////////////////////////////////////

TSharedRef<SWidget> FEditorViewportLayout2x2::MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString)
{
	FString FrontConfigKey, PerspectiveConfigKey, RightConfigKey, TopConfigKey;

	FString RightTypeName = TEXT("Default"), PerspectiveTypeName = TEXT("Default"), FrontTypeName = TEXT("Default"), TopTypeName = TEXT("Default");

	TArray<FVector2D> SplitterPercentages;
	
	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
 	if (!SpecificLayoutString.IsEmpty())
	{
		// The Layout String only holds the unique ID of the Additional Layout Configs to use
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		FrontConfigKey = SpecificLayoutString + TEXT(".Viewport0");
		PerspectiveConfigKey = SpecificLayoutString + TEXT(".Viewport1");
		RightConfigKey = SpecificLayoutString + TEXT(".Viewport2");
		TopConfigKey = SpecificLayoutString + TEXT(".Viewport3");

		GConfig->GetString(*IniSection, *(FrontConfigKey + TEXT(".TypeWithinLayout")), FrontTypeName, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(PerspectiveConfigKey + TEXT(".TypeWithinLayout")), PerspectiveTypeName, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(RightConfigKey + TEXT(".TypeWithinLayout")), RightTypeName, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(TopConfigKey + TEXT(".TypeWithinLayout")), TopTypeName, GEditorPerProjectIni);

		for (int32 i = 0; i < 4; ++i)
		{
			FString PercentageString;
			FVector2D NewPercentage = ViewportLayout2x2Defs::DefaultSplitterPercentages;
			if (GConfig->GetString(*IniSection, *(SpecificLayoutString + FString::Printf(TEXT(".Percentages%i"), i)), PercentageString, GEditorPerProjectIni))
			{
				NewPercentage.InitFromString(PercentageString);
			}
			SplitterPercentages.Add(NewPercentage);
		}
	}

	// Set up the viewports
	FAssetEditorViewportConstructionArgs Args;
 	Args.ParentLayout = InParentLayout;

	// Front viewport
	Args.bRealtime = false;
	Args.ConfigKey = *FrontConfigKey;
	Args.ViewportType = LVT_OrthoFront;
	TSharedRef< SWidget > FrontViewport = InParentLayout->FactoryViewport(*FrontTypeName, Args);

	// Persp viewport
	Args.bRealtime = !FPlatformMisc::IsRemoteSession();
	Args.ConfigKey = *PerspectiveConfigKey;
	Args.ViewportType = LVT_Perspective;
	TSharedRef< SWidget > PerspectiveViewport = InParentLayout->FactoryViewport(*PerspectiveTypeName, Args);
	PerspectiveViewportConfigKey = *PerspectiveConfigKey;

	// Right viewport
	Args.bRealtime = false;
	Args.ConfigKey = *RightConfigKey;
	Args.ViewportType = LVT_OrthoRight;
	TSharedRef< SWidget > RightViewport = InParentLayout->FactoryViewport(*RightTypeName, Args);

	// Top Viewport
	Args.bRealtime = false;
	Args.ConfigKey = *TopConfigKey;
	Args.ViewportType = LVT_OrthoTop;
	TSharedRef< SWidget > TopViewport = InParentLayout->FactoryViewport(*TopTypeName, Args);

	// Set up the splitter
	SplitterWidget = 
	SNew( SSplitter2x2 )
	.TopLeft()
	[
		TopViewport
	]
	.BottomLeft()
	[
		FrontViewport
	]
	.TopRight()
	[
		PerspectiveViewport
	]
	.BottomRight()
	[
		RightViewport
	];
	
	if (SplitterPercentages.Num() > 0)
	{
		SplitterWidget->SetSplitterPercentages(SplitterPercentages);
	}


	return SplitterWidget.ToSharedRef();
}

void FEditorViewportLayout2x2::ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget)
{
	bool bWasFound = false;

	if (SplitterWidget->GetTopLeftContent() == OriginalWidget)
	{
		SplitterWidget->SetTopLeftContent(ReplacementWidget);
		bWasFound = true;
	}

	else if (SplitterWidget->GetBottomLeftContent() == OriginalWidget)
	{
		SplitterWidget->SetBottomLeftContent(ReplacementWidget);
		bWasFound = true;
	}

	else if (SplitterWidget->GetTopRightContent() == OriginalWidget)
	{
		SplitterWidget->SetTopRightContent(ReplacementWidget);
		bWasFound = true;
	}

	else if (SplitterWidget->GetBottomRightContent() == OriginalWidget)
	{
		SplitterWidget->SetBottomRightContent(ReplacementWidget);
		bWasFound = true;
	}

	// Source widget should have already been a content widget for the splitter
	check(bWasFound);
}

const FName& FEditorViewportLayout2x2::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::FourPanes2x2;
}

void FEditorViewportLayout2x2::SaveLayoutString(const FString& SpecificLayoutString) const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	TArray<FVector2D> Percentages;
	SplitterWidget->GetSplitterPercentages(Percentages);
	for (int32 i = 0; i < Percentages.Num(); ++i)
	{
		GConfig->SetString(*IniSection, *(SpecificLayoutString + FString::Printf(TEXT(".Percentages%i"), i)), *Percentages[i].ToString(), GEditorPerProjectIni);
	}
}
