// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayoutEntity.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Editor/UnrealEdEngine.h"
#include "LevelEditorViewport.h"
#include "UnrealEdGlobals.h"
#include "SAssetEditorViewport.h"
#include "EditorViewportCommands.h"
#include "EditorViewportTabContent.h"

FEditorViewportLayoutEntity::FEditorViewportLayoutEntity(TSharedPtr<SAssetEditorViewport>& InViewport)
	: AssetEditorViewport(InViewport)
{
	check(AssetEditorViewport.IsValid());
}


TSharedRef<SWidget> FEditorViewportLayoutEntity::AsWidget() const
{
	return AssetEditorViewport.ToSharedRef();
}

TSharedPtr<SAssetEditorViewport> FEditorViewportLayoutEntity::AsAssetEditorViewport() const
{
	return AssetEditorViewport;
}

FName FEditorViewportLayoutEntity::GetType() const
{
	static FName DefaultName("Default");
	return DefaultName;
}

TSharedPtr<FEditorViewportClient> FEditorViewportLayoutEntity::GetViewportClient() const
{
	return AssetEditorViewport->GetViewportClient();
}

void FEditorViewportLayoutEntity::SetKeyboardFocus()
{
}

void FEditorViewportLayoutEntity::OnLayoutDestroyed()
{
}

void FEditorViewportLayoutEntity::SaveConfig(const FString& ConfigSection)
{
}

void FEditorViewportLayoutEntity::TakeHighResScreenShot() const
{
	GetViewportClient()->TakeHighResScreenShot();
}
