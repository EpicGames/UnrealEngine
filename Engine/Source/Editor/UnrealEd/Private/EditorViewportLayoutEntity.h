// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "AssetEditorViewportLayout.h"
#include "SEditorViewport.h"

class FEditorViewportClient;

class FEditorViewportLayoutEntity : public IEditorViewportLayoutEntity
{
public:

	UNREALED_API FEditorViewportLayoutEntity(TSharedPtr<SAssetEditorViewport>& InViewport);
	UNREALED_API virtual TSharedRef<SWidget> AsWidget() const override;
	UNREALED_API virtual TSharedPtr<SAssetEditorViewport> AsAssetEditorViewport() const;

	UNREALED_API void SetKeyboardFocus() override;
	UNREALED_API void OnLayoutDestroyed() override;
	UNREALED_API void SaveConfig(const FString& ConfigSection) override;
	UNREALED_API TSharedPtr<FEditorViewportClient> GetViewportClient() const;
	UNREALED_API FName GetType() const override;
	UNREALED_API void TakeHighResScreenShot() const override;

protected:

	/** This entity's editor viewport */
	TSharedPtr<SAssetEditorViewport> AssetEditorViewport;
};
