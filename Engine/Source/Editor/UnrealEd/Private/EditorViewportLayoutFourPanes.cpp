// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayoutFourPanes.h"
#include "Framework/Docking/LayoutService.h"
#include "Editor.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"

namespace ViewportLayoutFourPanesDefs
{
	/** Default main splitter to equal 50/50 split */
	static const float DefaultPrimarySplitterPercentage = 0.5f;

	/** Default secondary splitter to equal three-way split */
	static const float DefaultSecondarySplitterPercentage = 0.333f;
}

// FEditorViewportLayoutFourPanes /////////////////////////////

TSharedRef<SWidget> FEditorViewportLayoutFourPanes::MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString)
{
	FString ViewportKey0, ViewportKey1, ViewportKey2, ViewportKey3;
	FString ViewportType0 = TEXT("Default"), ViewportType1 = TEXT("Default"), ViewportType2 = TEXT("Default"), ViewportType3 = TEXT("Default");

	float PrimarySplitterPercentage = ViewportLayoutFourPanesDefs::DefaultPrimarySplitterPercentage;
	float SecondarySplitterPercentage0 = ViewportLayoutFourPanesDefs::DefaultSecondarySplitterPercentage;
	float SecondarySplitterPercentage1 = ViewportLayoutFourPanesDefs::DefaultSecondarySplitterPercentage;

	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
	if (!SpecificLayoutString.IsEmpty())
	{
		// The Layout String only holds the unique ID of the Additional Layout Configs to use
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		ViewportKey0 = SpecificLayoutString + TEXT(".Viewport0");
		ViewportKey1 = SpecificLayoutString + TEXT(".Viewport1");
		ViewportKey2 = SpecificLayoutString + TEXT(".Viewport2");
		ViewportKey3 = SpecificLayoutString + TEXT(".Viewport3");

		GConfig->GetString(*IniSection, *(ViewportKey0 + TEXT(".TypeWithinLayout")), ViewportType0, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey1 + TEXT(".TypeWithinLayout")), ViewportType1, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey2 + TEXT(".TypeWithinLayout")), ViewportType2, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey3 + TEXT(".TypeWithinLayout")), ViewportType3, GEditorPerProjectIni);

		FString PercentageString;
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage0")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(PrimarySplitterPercentage, *PercentageString);
		}
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage1")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(SecondarySplitterPercentage0, *PercentageString);
		}
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage2")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(SecondarySplitterPercentage1, *PercentageString);
		}
	}

	// Set up the viewports
	FAssetEditorViewportConstructionArgs Args;
	Args.ParentLayout = InParentLayout;
	Args.IsEnabled = FSlateApplication::Get().GetNormalExecutionAttribute();

	Args.bRealtime = !FPlatformMisc::IsRemoteSession();
	Args.ConfigKey = *ViewportKey0;
	Args.ViewportType = LVT_Perspective;
	TSharedRef<SWidget> Viewport0 = InParentLayout->FactoryViewport(*ViewportType0, Args);

	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey1;
	Args.ViewportType = LVT_OrthoXY;
	TSharedRef<SWidget> Viewport1 = InParentLayout->FactoryViewport(*ViewportType1, Args);

	// Front viewport
	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey2;
	Args.ViewportType = LVT_OrthoXZ;
	TSharedRef<SWidget> Viewport2 = InParentLayout->FactoryViewport(*ViewportType2, Args);

	// Top Viewport
	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey3;
	Args.ViewportType = LVT_OrthoYZ;
	TSharedRef<SWidget> Viewport3 = InParentLayout->FactoryViewport(*ViewportType2, Args);

	TSharedRef<SWidget> LayoutWidget = MakeFourPanelWidget(
		Viewport0, Viewport1, Viewport2, Viewport3,
		PrimarySplitterPercentage, SecondarySplitterPercentage0, SecondarySplitterPercentage1);

	return LayoutWidget;
}

void FEditorViewportLayoutFourPanes::SaveLayoutString(const FString& SpecificLayoutString) const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	check(PrimarySplitterWidget->GetChildren()->Num() == 2);
	float PrimaryPercentage = PrimarySplitterWidget->SlotAt(0).GetSizeValue();
	check(SecondarySplitterWidget->GetChildren()->Num() == 3);
	float SecondaryPercentage0 = SecondarySplitterWidget->SlotAt(0).GetSizeValue();
	float SecondaryPercentage1 = SecondarySplitterWidget->SlotAt(1).GetSizeValue();

	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage0")), *TTypeToString<float>::ToString(PrimaryPercentage), GEditorPerProjectIni);
	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage1")), *TTypeToString<float>::ToString(SecondaryPercentage0), GEditorPerProjectIni);
	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage2")), *TTypeToString<float>::ToString(SecondaryPercentage1), GEditorPerProjectIni);
}

void FEditorViewportLayoutFourPanes::ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget)
{
	bool bWasFound = false;

	for (int32 SlotIdx = 0; SlotIdx < PrimarySplitterWidget->GetChildren()->Num(); SlotIdx++)
	{
		if (PrimarySplitterWidget->GetChildren()->GetChildAt(SlotIdx) == OriginalWidget)
		{
			PrimarySplitterWidget->SlotAt(SlotIdx)
				[
					ReplacementWidget
				];
			bWasFound = true;
			break;
		}
	}

	for (int32 SlotIdx = 0; SlotIdx < SecondarySplitterWidget->GetChildren()->Num(); SlotIdx++)
	{
		if (SecondarySplitterWidget->GetChildren()->GetChildAt(SlotIdx) == OriginalWidget)
		{
			SecondarySplitterWidget->SlotAt(SlotIdx)
				[
					ReplacementWidget
				];
			bWasFound = true;
			break;
		}
	}

	// Source widget should have already been a content widget for the splitter
	check(bWasFound);
}

// FEditorViewportLayoutFourPanesLeft /////////////////////////////
const FName& FEditorViewportLayoutFourPanesLeft::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::FourPanesLeft;
}

TSharedRef<SWidget> FEditorViewportLayoutFourPanesLeft::MakeFourPanelWidget(
	TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Horizontal)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			Viewport0
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Vertical)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage0)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage1)
			[
				Viewport2
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage0 - SecondarySplitterPercentage1)
			[
				Viewport3
			]
		];

	return Widget;
}


// FEditorViewportLayoutFourPanesRight /////////////////////////////
const FName& FEditorViewportLayoutFourPanesRight::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::FourPanesRight;
}

TSharedRef<SWidget> FEditorViewportLayoutFourPanesRight::MakeFourPanelWidget(
	TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Horizontal)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Vertical)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage0)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage1)
			[
				Viewport2
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage0 - SecondarySplitterPercentage1)
			[
				Viewport3
			]
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			Viewport0
		];

	return Widget;
}


// FEditorViewportLayoutFourPanesTop /////////////////////////////
const FName& FEditorViewportLayoutFourPanesTop::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::FourPanesTop;
}

TSharedRef<SWidget> FEditorViewportLayoutFourPanesTop::MakeFourPanelWidget(
	TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Vertical)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			Viewport0
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Horizontal)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage0)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage1)
			[
				Viewport2
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage0 - SecondarySplitterPercentage1)
			[
				Viewport3
			]
		];

	return Widget;
}


// FEditorViewportLayoutFourPanesBottom /////////////////////////////
const FName& FEditorViewportLayoutFourPanesBottom::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::FourPanesBottom;
}

TSharedRef<SWidget> FEditorViewportLayoutFourPanesBottom::MakeFourPanelWidget(
	TSharedRef<SWidget> Viewport0, TSharedRef<SWidget> Viewport1, TSharedRef<SWidget> Viewport2, TSharedRef<SWidget> Viewport3,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage0, float SecondarySplitterPercentage1)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Vertical)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Horizontal)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage0)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage1)
			[
				Viewport2
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage0 - SecondarySplitterPercentage1)
			[
				Viewport3
			]
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			Viewport0
		];

	return Widget;
}
