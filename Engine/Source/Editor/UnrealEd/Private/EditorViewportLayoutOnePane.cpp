// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayoutOnePane.h"
#include "ShowFlags.h"
#include "Editor.h"
#include "Framework/Docking/LayoutService.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"

TSharedRef<SWidget> FEditorViewportLayoutOnePane::MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString)
{
	FEngineShowFlags OrthoShowFlags(ESFIM_Editor);
	ApplyViewMode(VMI_Unlit, false, OrthoShowFlags);

	FEngineShowFlags PerspectiveShowFlags(ESFIM_Editor);
	ApplyViewMode(VMI_Lit, true, PerspectiveShowFlags);

	FString ViewportKey, ViewportType;

	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
	if (!SpecificLayoutString.IsEmpty())
	{
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		ViewportKey = SpecificLayoutString + TEXT(".Viewport0");
		GConfig->GetString(*IniSection, *(ViewportKey + TEXT(".TypeWithinLayout")), ViewportType, GEditorPerProjectIni);
	}

	// Set up the viewport
	FAssetEditorViewportConstructionArgs Args;
 	Args.ParentLayout = InParentLayout;
	Args.bRealtime = !FPlatformMisc::IsRemoteSession();
 	Args.ViewportType = LVT_Perspective;
	Args.ConfigKey = *ViewportKey;
	TSharedRef<SWidget> Viewport = InParentLayout->FactoryViewport(*ViewportType, Args);
	PerspectiveViewportConfigKey = *ViewportKey;

	ViewportBox =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			Viewport
		];

	return ViewportBox.ToSharedRef();
}

const FName& FEditorViewportLayoutOnePane::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::OnePane;
}

void FEditorViewportLayoutOnePane::SaveLayoutString(const FString& SpecificLayoutString) const
{
}

void FEditorViewportLayoutOnePane::ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget)
{
	check(ViewportBox->GetChildren()->Num() == 1)
		TSharedRef<SWidget> ViewportWidget = ViewportBox->GetChildren()->GetChildAt(0);

	check(ViewportWidget == OriginalWidget);
	ViewportBox->RemoveSlot(OriginalWidget);
	ViewportBox->AddSlot()
		[
			ReplacementWidget
		];
}
