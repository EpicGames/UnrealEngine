// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayoutThreePanes.h"
#include "Framework/Docking/LayoutService.h"
#include "ShowFlags.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SSplitter.h"

namespace ViewportLayoutThreePanesDefs
{
	/** Default splitters to equal 50/50 split */
	static const float DefaultSplitterPercentage = 0.5f;
}


// FEditorViewportLayoutThreePanes /////////////////////////////

TSharedRef<SWidget> FEditorViewportLayoutThreePanes::MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& LayoutString)
{
	FEngineShowFlags OrthoShowFlags(ESFIM_Editor);	
	ApplyViewMode(VMI_Unlit, false, OrthoShowFlags);

	FEngineShowFlags PerspectiveShowFlags(ESFIM_Editor);	
	ApplyViewMode(VMI_Lit, true, PerspectiveShowFlags);

	FString ViewportKey0, ViewportKey1, ViewportKey2;
	FString ViewportType0, ViewportType1, ViewportType2;
	float PrimarySplitterPercentage = ViewportLayoutThreePanesDefs::DefaultSplitterPercentage;
	float SecondarySplitterPercentage = ViewportLayoutThreePanesDefs::DefaultSplitterPercentage;

	FString SpecificLayoutString = GetTypeSpecificLayoutString(LayoutString);
	if (!SpecificLayoutString.IsEmpty())
	{
		// The Layout String only holds the unique ID of the Additional Layout Configs to use
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		ViewportKey0 = SpecificLayoutString + TEXT(".Viewport0");
		ViewportKey1 = SpecificLayoutString + TEXT(".Viewport1");
		ViewportKey2 = SpecificLayoutString + TEXT(".Viewport2");

		GConfig->GetString(*IniSection, *(ViewportKey0 + TEXT(".TypeWithinLayout")), ViewportType0, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey1 + TEXT(".TypeWithinLayout")), ViewportType1, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey2 + TEXT(".TypeWithinLayout")), ViewportType2, GEditorPerProjectIni);

		FString PercentageString;
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage0")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(PrimarySplitterPercentage, *PercentageString);
		}
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage1")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(SecondarySplitterPercentage, *PercentageString);
		}
	}

	// Set up the viewports
	FAssetEditorViewportConstructionArgs Args;
	Args.ParentLayout = InParentLayout;
	Args.IsEnabled = FSlateApplication::Get().GetNormalExecutionAttribute();

	Args.bRealtime = !FPlatformMisc::IsRemoteSession();
	Args.ConfigKey = *ViewportKey0;
	Args.ViewportType = LVT_Perspective;
	TSharedRef<SWidget> Viewport0 = InParentLayout->FactoryViewport(*ViewportType0, Args);
	PerspectiveViewportConfigKey = *ViewportKey0;

	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey1;
	Args.ViewportType = LVT_OrthoXY;
	TSharedRef<SWidget> Viewport1 = InParentLayout->FactoryViewport(*ViewportType1, Args);

	// Front viewport
	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey2;
	Args.ViewportType = LVT_OrthoXZ;
	TSharedRef<SWidget> Viewport2 = InParentLayout->FactoryViewport(*ViewportType2, Args);

	TSharedRef<SWidget> LayoutWidget = MakeThreePanelWidget(
		Viewport0, Viewport1, Viewport2,
		PrimarySplitterPercentage, SecondarySplitterPercentage);

	return LayoutWidget;
}

void FEditorViewportLayoutThreePanes::SaveLayoutString(const FString& SpecificLayoutString) const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	check(PrimarySplitterWidget->GetChildren()->Num() == 2);
	float PrimaryPercentage = PrimarySplitterWidget->SlotAt(0).GetSizeValue();
	check(SecondarySplitterWidget->GetChildren()->Num() == 2);
	float SecondaryPercentage = SecondarySplitterWidget->SlotAt(0).GetSizeValue();

	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage0")), *TTypeToString<float>::ToString(PrimaryPercentage), GEditorPerProjectIni);
	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage1")), *TTypeToString<float>::ToString(SecondaryPercentage), GEditorPerProjectIni);
}

void FEditorViewportLayoutThreePanes::ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget)
{
	bool bWasFound = false;

	for (int32 SlotIdx = 0; SlotIdx < PrimarySplitterWidget->GetChildren()->Num(); SlotIdx++)
	{
		if (PrimarySplitterWidget->GetChildren()->GetChildAt(SlotIdx) == OriginalWidget)
		{
			PrimarySplitterWidget->SlotAt(SlotIdx)
				[
					ReplacementWidget
				];
			bWasFound = true;
			break;
		}
	}

	for (int32 SlotIdx = 0; SlotIdx < SecondarySplitterWidget->GetChildren()->Num(); SlotIdx++)
	{
		if (SecondarySplitterWidget->GetChildren()->GetChildAt(SlotIdx) == OriginalWidget)
		{
			SecondarySplitterWidget->SlotAt(SlotIdx)
				[
					ReplacementWidget
				];
			bWasFound = true;
			break;
		}
	}

	// Source widget should have already been a content widget for the splitter
	check(bWasFound);
}

// FEditorViewportLayoutThreePanesLeft /////////////////////////////
const FName& FEditorViewportLayoutThreePanesLeft::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::ThreePanesLeft;
}

TSharedRef<SWidget> FEditorViewportLayoutThreePanesLeft::MakeThreePanelWidget(
	const TSharedRef<SWidget>& Viewport0, const TSharedRef<SWidget>& Viewport1, const TSharedRef<SWidget>& Viewport2,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Horizontal)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			Viewport0
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Vertical)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage)
			[
				Viewport2
			]
		];

	return Widget;
}


// FEditorViewportLayoutThreePanesRight /////////////////////////////
const FName& FEditorViewportLayoutThreePanesRight::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::ThreePanesRight;
}

TSharedRef<SWidget> FEditorViewportLayoutThreePanesRight::MakeThreePanelWidget(
	const TSharedRef<SWidget>& Viewport0, const TSharedRef<SWidget>& Viewport1, const TSharedRef<SWidget>& Viewport2,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Horizontal)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Vertical)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage)
			[
				Viewport2
			]
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			Viewport0
		];

	return Widget;
}


// FEditorViewportLayoutThreePanesTop /////////////////////////////
const FName& FEditorViewportLayoutThreePanesTop::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::ThreePanesTop;
}

TSharedRef<SWidget> FEditorViewportLayoutThreePanesTop::MakeThreePanelWidget(
	const TSharedRef<SWidget>& Viewport0, const TSharedRef<SWidget>& Viewport1, const TSharedRef<SWidget>& Viewport2,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Vertical)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			Viewport0
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Horizontal)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage)
			[
				Viewport2
			]
		];

	return Widget;
}


// FEditorViewportLayoutThreePanesBottom /////////////////////////////
const FName& FEditorViewportLayoutThreePanesBottom::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::ThreePanesBottom;
}

TSharedRef<SWidget> FEditorViewportLayoutThreePanesBottom::MakeThreePanelWidget(
	const TSharedRef<SWidget>& Viewport0, const TSharedRef<SWidget>& Viewport1, const TSharedRef<SWidget>& Viewport2,
	float PrimarySplitterPercentage, float SecondarySplitterPercentage)
{
	TSharedRef<SWidget> Widget = 
		SAssignNew( PrimarySplitterWidget, SSplitter )
		.Orientation(EOrientation::Orient_Vertical)
		+SSplitter::Slot()
		.Value(PrimarySplitterPercentage)
		[
			SAssignNew( SecondarySplitterWidget, SSplitter )
			.Orientation(EOrientation::Orient_Horizontal)
			+SSplitter::Slot()
			.Value(SecondarySplitterPercentage)
			[
				Viewport1
			]
			+SSplitter::Slot()
			.Value(1.0f - SecondarySplitterPercentage)
			[
				Viewport2
			]
		]
		+SSplitter::Slot()
		.Value(1.0f - PrimarySplitterPercentage)
		[
			Viewport0
		];

	return Widget;
}
