// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportLayoutTwoPanes.h"
#include "Framework/Docking/LayoutService.h"
#include "Editor.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSplitter.h"


namespace ViewportLayoutTwoPanesDefs
{
	/** Default splitters to equal 50/50 split */
	static const float DefaultSplitterPercentage = 0.5f;
}

template <EOrientation TOrientation>
TSharedRef<SWidget> TEditorViewportLayoutTwoPanes<TOrientation>::MakeViewportLayout(TSharedPtr<FAssetEditorViewportLayout> InParentLayout, const FString& SpecificLayoutString)
{
	FString ViewportKey0, ViewportKey1;
	FString ViewportType0, ViewportType1;
	float SplitterPercentage = ViewportLayoutTwoPanesDefs::DefaultSplitterPercentage;

	if (!SpecificLayoutString.IsEmpty())
	{
		// The Layout String only holds the unique ID of the Additional Layout Configs to use
		const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

		ViewportKey0 = SpecificLayoutString + TEXT(".Viewport0");
		ViewportKey1 = SpecificLayoutString + TEXT(".Viewport1");

		GConfig->GetString(*IniSection, *(ViewportKey0 + TEXT(".TypeWithinLayout")), ViewportType0, GEditorPerProjectIni);
		GConfig->GetString(*IniSection, *(ViewportKey1 + TEXT(".TypeWithinLayout")), ViewportType1, GEditorPerProjectIni);

		FString PercentageString;
		if (GConfig->GetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage")), PercentageString, GEditorPerProjectIni))
		{
			TTypeFromString<float>::FromString(SplitterPercentage, *PercentageString);
		}
	}

	// Set up the viewports
	FAssetEditorViewportConstructionArgs Args;
	Args.ParentLayout = InParentLayout;
	Args.IsEnabled = FSlateApplication::Get().GetNormalExecutionAttribute();

	Args.bRealtime = false;
	Args.ConfigKey = *ViewportKey0;
	Args.ViewportType = LVT_OrthoXY;
	TSharedRef<SWidget> Viewport0 = InParentLayout->FactoryViewport(*ViewportType0, Args);
	PerspectiveViewportConfigKey = *ViewportKey0;

	Args.bRealtime = !FPlatformMisc::IsRemoteSession();
	Args.ConfigKey = *ViewportKey1;
	Args.ViewportType = LVT_Perspective;
	TSharedRef<SWidget> Viewport1 = InParentLayout->FactoryViewport(*ViewportType1, Args);

	SplitterWidget =
		SNew(SSplitter)
		.Orientation(TOrientation)
		+ SSplitter::Slot()
		.Value(SplitterPercentage)
		[
			Viewport0
		]
	+ SSplitter::Slot()
		.Value(1.0f - SplitterPercentage)
		[
			Viewport1
		];

	return SplitterWidget.ToSharedRef();
}

template <EOrientation TOrientation>
void TEditorViewportLayoutTwoPanes<TOrientation>::ReplaceWidget(TSharedRef<SWidget> OriginalWidget, TSharedRef<SWidget> ReplacementWidget)
{
	bool bWasFound = false;

	for (int32 SlotIdx = 0; SlotIdx < SplitterWidget->GetChildren()->Num(); SlotIdx++)
	{
		if (SplitterWidget->GetChildren()->GetChildAt(SlotIdx) == OriginalWidget)
		{
			SplitterWidget->SlotAt(SlotIdx)
				[
					ReplacementWidget
				];
			bWasFound = true;
			break;
		}
	}

	// Source widget should have already been a content widget for the splitter
	check(bWasFound);
}

template <EOrientation TOrientation>
void TEditorViewportLayoutTwoPanes<TOrientation>::SaveLayoutString(const FString& SpecificLayoutString) const
{
	const FString& IniSection = FLayoutSaveRestore::GetAdditionalLayoutConfigIni();

	check(SplitterWidget->GetChildren()->Num() == 2);
	float Percentage = SplitterWidget->SlotAt(0).GetSizeValue();

	GConfig->SetString(*IniSection, *(SpecificLayoutString + TEXT(".Percentage")), *TTypeToString<float>::ToString(Percentage), GEditorPerProjectIni);
}

const FName& FEditorViewportLayoutTwoPanesVert::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::TwoPanesVert;
}

const FName& FEditorViewportLayoutTwoPanesHoriz::GetLayoutTypeName() const
{
	return EditorViewportConfigurationNames::TwoPanesHoriz;
}

template class TEditorViewportLayoutTwoPanes<EOrientation::Orient_Vertical>;
template class TEditorViewportLayoutTwoPanes<EOrientation::Orient_Horizontal>;
