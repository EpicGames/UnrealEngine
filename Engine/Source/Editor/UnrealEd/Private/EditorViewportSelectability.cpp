// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportSelectability.h"
#include "CanvasTypes.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Engine/Canvas.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Volume.h"
#include "HAL/CriticalSection.h"
#include "LevelEditorSubsystem.h"
#include "LevelEditorViewport.h"
#include "Math/Color.h"
#include "Misc/ScopeLock.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "UnrealWidget.h"
#include "UObject/WeakObjectPtr.h"

#define LOCTEXT_NAMESPACE "SequencerSelectabilityTool"

const FText FEditorViewportSelectability::DefaultLimitedSelectionText = LOCTEXT("DefaultSelectionLimitedHelp", "Viewport Selection Limited");

FDelegateHandle FEditorViewportSelectability::RegisterObjectSelectablePredicate(const FOnIsObjectSelectableInViewport& InPredicate)
{
	if (!InPredicate.IsBound())
	{
		return FDelegateHandle();
	}

	const FDelegateHandle NewHandle(FDelegateHandle::GenerateNewHandle);

	GetAdditionalObjectSelectablePredicates().Add(NewHandle, InPredicate);

	return NewHandle;
}

void FEditorViewportSelectability::UnregisterObjectSelectablePredicate(const FDelegateHandle InHandle)
{
	if (!InHandle.IsValid())
	{
		return;
	}

	GetAdditionalObjectSelectablePredicates().Remove(InHandle);
}

bool FEditorViewportSelectability::PassesAdditionalObjectSelectablePredicates(UObject* const InObject)
{
	for (const TPair<FDelegateHandle, FOnIsObjectSelectableInViewport>& Pair : GetAdditionalObjectSelectablePredicates())
	{
		if (Pair.Value.IsBound() && !Pair.Value.Execute(InObject))
		{
			return false;
		}
	}

	return true;
}

bool FEditorViewportSelectability::PassesAdditionalObjectSelectablePredicates(HHitProxy* const InHitProxy)
{
	const HActor* const ActorHitProxy = HitProxyCast<HActor>(InHitProxy);
	if (!ActorHitProxy)
	{
		return true;
	}

	if (ActorHitProxy->Actor)
	{
		return PassesAdditionalObjectSelectablePredicates(ActorHitProxy->Actor);
	}

	if (ActorHitProxy->PrimComponent)
	{
		return PassesAdditionalObjectSelectablePredicates(const_cast<UPrimitiveComponent*>(ActorHitProxy->PrimComponent.Get()));
	}

	return true;
}

TMap<FDelegateHandle, FOnIsObjectSelectableInViewport>& FEditorViewportSelectability::GetAdditionalObjectSelectablePredicates()
{
	static TMap<FDelegateHandle, FOnIsObjectSelectableInViewport> AdditionalPredicates;
	return AdditionalPredicates;
}

FEditorViewportSelectability::FEditorViewportSelectability(const FOnGetWorld& InOnGetWorld, const FOnIsObjectSelectableInViewport& InOnIsObjectSelectableInViewport)
	: OnGetWorld(InOnGetWorld)
	, OnIsObjectSelectableInViewportDelegate(InOnIsObjectSelectableInViewport)
{
}

void FEditorViewportSelectability::EnableLimitedSelection(const bool bInEnabled)
{
	bSelectionLimited = bInEnabled;

	if (bSelectionLimited)
	{
		DeselectNonSelectableActors();
	}

	UpdateSelectionLimitedVisuals(!bInEnabled);
}

bool FEditorViewportSelectability::IsObjectSelectableInViewport(UObject* const InObject) const
{
	if (OnIsObjectSelectableInViewportDelegate.IsBound()
		&& !OnIsObjectSelectableInViewportDelegate.Execute(InObject))
	{
		return false;
	}

	return PassesAdditionalObjectSelectablePredicates(InObject);
}

void FEditorViewportSelectability::UpdatePrimitiveVisuals(const bool bInSelectedLimited, UPrimitiveComponent* const InPrimitive, const TOptional<FColor>& InColor)
{
	if (bInSelectedLimited && InColor.IsSet())
	{
		// @TODO: Need to resolve rendering issue before this can be used
		//InPrimitive->SetOverlayColor(InColor.GetValue());
		InPrimitive->PushHoveredToProxy(true);
	}
	else
	{
		// @TODO: Need to resolve rendering issue before this can be used
		//InPrimitive->RemoveOverlayColor();
		InPrimitive->PushHoveredToProxy(false);
	}
}

bool FEditorViewportSelectability::UpdateHoveredPrimitive(const bool bInSelectedLimited
	, UPrimitiveComponent* const InPrimitiveComponent
	, TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& InOutHoveredPrimitiveComponents
	, const TFunctionRef<bool(UObject*)>& InSelectablePredicate)
{
	bool bValid = IsValid(InPrimitiveComponent);

	// Save the current overlay color to restore when unhovered
	TMap<UPrimitiveComponent*, TOptional<FColor>> PrimitiveComponentsToAdd;

	if (bValid && bInSelectedLimited && InSelectablePredicate(InPrimitiveComponent))
	{
		TOptional<FColor> UnhoveredColor;
		if (InPrimitiveComponent->bWantsEditorEffects)
		{
			UnhoveredColor = InPrimitiveComponent->OverlayColor;
		}
		PrimitiveComponentsToAdd.Add(const_cast<UPrimitiveComponent*>(InPrimitiveComponent), UnhoveredColor);

		bValid = true;
	}

	// Get the set of components to remove that aren't in the newly hovered set from the currently hovered compenents
	TMap<UPrimitiveComponent*, TOptional<FColor>> PrimitiveComponentsToRemove;
	for (const TPair<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& HoveredPair : InOutHoveredPrimitiveComponents)
	{
		UPrimitiveComponent* const PrimitiveComponent = HoveredPair.Key.Get();
		if (IsValid(PrimitiveComponent) && !PrimitiveComponentsToAdd.Contains(PrimitiveComponent))
		{
			PrimitiveComponentsToRemove.Add(PrimitiveComponent, HoveredPair.Value);
		}
	}

	// Hover new primitives, unhover old primitives
	InOutHoveredPrimitiveComponents.Empty();

	for (const TPair<UPrimitiveComponent*, TOptional<FColor>>& AddPair : PrimitiveComponentsToAdd)
	{
		InOutHoveredPrimitiveComponents.Add(AddPair);

		// Using white becaue we've stripped out the visual element until the rendering issue can be resolved
		UpdatePrimitiveVisuals(bInSelectedLimited, AddPair.Key, FColor::White);
	}

	for (const TPair<UPrimitiveComponent*, TOptional<FColor>>& RemovePair : PrimitiveComponentsToRemove)
	{
		UpdatePrimitiveVisuals(bInSelectedLimited, RemovePair.Key);
	}

	return bValid;
}

bool FEditorViewportSelectability::UpdateHoveredActorPrimitives(const bool bInSelectedLimited
	, AActor* const InActor
	, TMap<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& InOutHoveredPrimitiveComponents
	, const TFunctionRef<bool(UObject*)>& InSelectablePredicate)
{
	bool bValid = false;

	// Save the current overlay color to restore when unhovered
	TMap<UPrimitiveComponent*, TOptional<FColor>> PrimitiveComponentsToAdd;

	if (IsValid(InActor) && bInSelectedLimited)
	{
		if (InSelectablePredicate(InActor))
		{
			bValid = true;
		}
		InActor->ForEachComponent<UPrimitiveComponent>(/*bIncludeFromChildActors=*/true,
			[&InSelectablePredicate, &bValid, &PrimitiveComponentsToAdd](UPrimitiveComponent* const InPrimitiveComponent)
			{
				if (bValid || InSelectablePredicate(InPrimitiveComponent))
				{
					TOptional<FColor> UnhoveredColor;
					if (InPrimitiveComponent->bWantsEditorEffects)
					{
						UnhoveredColor = InPrimitiveComponent->OverlayColor;
					}
					PrimitiveComponentsToAdd.Add(InPrimitiveComponent, UnhoveredColor);

					bValid = true;
				}
			});
	}

	// Get the set of components to remove that aren't in the newly hovered set from the currently hovered compenents
	TMap<UPrimitiveComponent*, TOptional<FColor>> PrimitiveComponentsToRemove;
	for (const TPair<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& HoveredPair : InOutHoveredPrimitiveComponents)
	{
		UPrimitiveComponent* const PrimitiveComponent = HoveredPair.Key.Get();
		if (IsValid(PrimitiveComponent) && !PrimitiveComponentsToAdd.Contains(PrimitiveComponent))
		{
			PrimitiveComponentsToRemove.Add(PrimitiveComponent, HoveredPair.Value);
		}
	}

	// Hover new primitives, unhover old primitives
	InOutHoveredPrimitiveComponents.Empty();

	for (const TPair<UPrimitiveComponent*, TOptional<FColor>>& AddPair : PrimitiveComponentsToAdd)
	{
		InOutHoveredPrimitiveComponents.Add(AddPair);

		// Using white becaue we've stripped out the visual element until the rendering issue can be resolved
		UpdatePrimitiveVisuals(bInSelectedLimited, AddPair.Key, FColor::White);
	}

	for (const TPair<UPrimitiveComponent*, TOptional<FColor>>& RemovePair : PrimitiveComponentsToRemove)
	{
		UpdatePrimitiveVisuals(bInSelectedLimited, RemovePair.Key);
	}

	return bValid;
}

void FEditorViewportSelectability::UpdateHoveredActorPrimitives(AActor* const InActor)
{
	UpdateHoveredActorPrimitives(bSelectionLimited, InActor, HoveredPrimitiveComponents,
		[this](UObject* const InObject) -> bool
		{
			return IsObjectSelectableInViewport(InObject);
		});
}

void FEditorViewportSelectability::UpdateSelectionLimitedVisuals(const bool bInClearHovered)
{
	if (bInClearHovered)
	{
		UpdateHoveredActorPrimitives(nullptr);
	}

	for (const TPair<TWeakObjectPtr<UPrimitiveComponent>, TOptional<FColor>>& HoveredPair : HoveredPrimitiveComponents)
	{
		UPrimitiveComponent* const PrimitiveComponent = HoveredPair.Key.Get();
		if (IsValid(PrimitiveComponent))
		{
			if (bSelectionLimited
				&& (IsObjectSelectableInViewport(PrimitiveComponent) || IsObjectSelectableInViewport(PrimitiveComponent->GetOwner())))
			{
				UpdatePrimitiveVisuals(bSelectionLimited, PrimitiveComponent, HoveredPair.Value);
			}
			else
			{
				UpdatePrimitiveVisuals(bSelectionLimited, PrimitiveComponent);
			}
		}
	}
}

void FEditorViewportSelectability::DeselectNonSelectableActors()
{
	if (!bSelectionLimited)
	{
		return;
	}

	USelection* const ActorSelection = GEditor ? GEditor->GetSelectedActors() : nullptr;
	if (!ActorSelection || ActorSelection->Num() == 0)
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	UWorld* const World = OnGetWorld.IsBound() ? OnGetWorld.Execute() : nullptr;

	SelectActorsByPredicate(World, /*bInSelect=*/true, /*bInClearSelection=*/true
		, [this](AActor* const InActor) -> bool
		{
			return IsObjectSelectableInViewport(InActor);
		}
		, SelectedActors);
}

bool FEditorViewportSelectability::SelectActorsByPredicate(UWorld* const InWorld
	, const bool bInSelect
	, const bool bInClearSelection
	, const TFunctionRef<bool(AActor*)> InPredicate
	, const TArray<AActor*>& InActors)
{
	if (!GEditor || !IsValid(InWorld))
	{
		return false;
	}

	USelection* const ActorSelection = GEditor->GetSelectedActors();
	if (!ActorSelection)
	{
		return false;
	}

	const FText TransactionText = bInSelect ? LOCTEXT("SelectActors_Internal", "Select Actor(s)") : LOCTEXT("DeselectActors_Internal", "Deselect Actor(s)");
	FScopedTransaction ScopedTransaction(TransactionText, !GIsTransacting);

	bool bSomethingSelected = false;

	ActorSelection->BeginBatchSelectOperation();
	ActorSelection->Modify();

	if (bInClearSelection)
	{
		ActorSelection->DeselectAll();
	}

	// Early out for specific deselect case
	if (!bInSelect && bInClearSelection)
	{
		ActorSelection->EndBatchSelectOperation();
		GEditor->NoteSelectionChange();

		return true;
	}

	auto SelectIfPossible = [bInSelect, ActorSelection, &InPredicate, &bSomethingSelected](AActor* const InActor)
	{
		if (IsValid(InActor)
			&& ActorSelection->IsSelected(InActor) != bInSelect
			&& InPredicate(InActor))
		{
			bSomethingSelected = true;
			GEditor->SelectActor(InActor, bInSelect, true);
		}
	};

	if (InActors.IsEmpty())
	{
		for (FActorIterator Iter(InWorld); Iter; ++Iter)
		{
			AActor* const Actor = *Iter;
			SelectIfPossible(Actor);
		}
	}
	else
	{
		for (AActor* const Actor : InActors)
		{
			SelectIfPossible(Actor);
		}
	}

	ActorSelection->EndBatchSelectOperation();
	GEditor->NoteSelectionChange();

	if (!bSomethingSelected)
	{
		ScopedTransaction.Cancel();
	}

	return bSomethingSelected;
}

bool FEditorViewportSelectability::IsActorSelectableClass(const AActor& InActor)
{
	const bool bInvalidClass = InActor.IsA<AWorldSettings>();
	return !bInvalidClass;
}

bool FEditorViewportSelectability::IsActorInLevelHiddenLayer(const AActor& InActor, FLevelEditorViewportClient* const InLevelEditorViewportClient)
{
	if (!InLevelEditorViewportClient)
	{
		return false;
	}

	for (const FName Layer : InActor.Layers)
	{
		if (InLevelEditorViewportClient->ViewHiddenLayers.Contains(Layer))
		{
			return true;
		}
	}

	return false;
}

TTypedElement<ITypedElementWorldInterface> FEditorViewportSelectability::GetTypedWorldElementFromActor(const AActor& InActor)
{
	const FTypedElementHandle ActorElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(&InActor);
	if (!ActorElementHandle)
	{
		return TTypedElement<ITypedElementWorldInterface>();
	}

	const UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!TypedElementRegistry)
	{
		return TTypedElement<ITypedElementWorldInterface>();
	}

	return TypedElementRegistry->GetElement<ITypedElementWorldInterface>(ActorElementHandle);
}

bool FEditorViewportSelectability::GetCursorForHovered(EMouseCursor::Type& OutCursor) const
{
	if (bSelectionLimited && MouseCursor.IsSet())
	{
		OutCursor = MouseCursor.GetValue();
		return true;
	}

	return false;
}

void FEditorViewportSelectability::UpdateHoverFromHitProxy(HHitProxy* const InHitProxy)
{
	AActor* ActorHit = nullptr;
	bool bIsGizmoHit = false;
	bool bIsSelectableActorHit = false;

	if (InHitProxy)
	{
		if (InHitProxy->IsA(HWidgetAxis::StaticGetType()))
		{
			if (bSelectionLimited)
			{
				bIsGizmoHit = true;
			}
		}
		else if (InHitProxy->IsA(HActor::StaticGetType()))
		{
			const HActor* const ActorHitProxy = static_cast<HActor*>(InHitProxy);
			if (ActorHitProxy && IsValid(ActorHitProxy->Actor))
			{
				ActorHit = ActorHitProxy->Actor;

				if (bSelectionLimited)
				{
					bIsSelectableActorHit = IsObjectSelectableInViewport(ActorHitProxy->Actor);
				}
			}
		}
	}

	UpdateHoveredActorPrimitives(ActorHit);

	// Set mouse cursor after hovered primitive component list has been updated
	if (bIsGizmoHit)
	{
		MouseCursor = EMouseCursor::CardinalCross;
	}
	else if (bIsSelectableActorHit)
	{
		MouseCursor = EMouseCursor::Crosshairs;
	}
	else if (bSelectionLimited)
	{
		MouseCursor = EMouseCursor::Default;
	}
	else
	{
		MouseCursor.Reset();
	}
}

bool FEditorViewportSelectability::HandleClick(FEditorViewportClient* const InViewportClient, HHitProxy* const InHitProxy, const FViewportClick& InClick)
{
	if (!InViewportClient)
	{
		return false;
	}

	UWorld* const World = InViewportClient->GetWorld();
	if (!IsValid(World))
	{
		return false;
	}

	// Disable actor selection when sequencer is limiting selection
	const int32 HitX = InViewportClient->Viewport->GetMouseX();
	const int32 HitY = InViewportClient->Viewport->GetMouseY();
	const HHitProxy* const HitResult = InViewportClient->Viewport->GetHitProxy(HitX, HitY);
	if (!HitResult)
	{
		return false;
	}

	if (HitResult->IsA(HWidgetAxis::StaticGetType()) || !HitResult->IsA(HActor::StaticGetType()))
	{
		return false;
	}

	// Check for translucent actors if we don't want to allow them to be selected
	const UEditorPerProjectUserSettings* const Settings = GetDefault<UEditorPerProjectUserSettings>();
	if (!Settings->bAllowSelectTranslucent && HitResult->IsA(HTranslucentActor::StaticGetType()))
	{
		// Return true to disable selection of valid translucent actors
		const HTranslucentActor* const TranslucentActorHitProxy = static_cast<const HTranslucentActor*>(HitResult);
		return IsValid(TranslucentActorHitProxy->Actor);
	}

	const HActor* const ActorHitProxy = static_cast<const HActor*>(HitResult);
	if (!ActorHitProxy || !IsValid(ActorHitProxy->Actor))
	{
		return false;
	}

	const bool bNotSelectable = !IsObjectSelectableInViewport(ActorHitProxy->Actor);

	if (bNotSelectable)
	{
		SelectActorsByPredicate(World, false, true
			, [this](AActor* const InActor) -> bool
			{
				return false;
			});
	}

	return bNotSelectable;
}

void FEditorViewportSelectability::StartTracking(FEditorViewportClient* const InViewportClient, FViewport* const InViewport)
{
	FIntPoint MousePosition;
	InViewport->GetMousePos(MousePosition);

	DragStartPosition = FVector(MousePosition.X, MousePosition.Y, 0);
	DragEndPosition = DragStartPosition;
}

void FEditorViewportSelectability::EndTracking(FEditorViewportClient* const InViewportClient, FViewport* const InViewport)
{
	FIntPoint MousePosition;
	InViewport->GetMousePos(MousePosition);

	DragEndPosition = FVector(MousePosition);

	if (DragStartPosition.X > DragEndPosition.X)
	{
		Swap(DragStartPosition.X, DragEndPosition.X);
	}
	if (DragStartPosition.Y > DragEndPosition.Y)
	{
		Swap(DragStartPosition.Y, DragEndPosition.Y);
	}

	// Extend the endpoint of the rect to get the actual line
	const int32 MinX = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, DragStartPosition.X));
	const int32 MinY = UE::LWC::FloatToIntCastChecked<int32>(FMath::Max<double>(0.0, DragStartPosition.Y));
	const FIntPoint ViewportSize = InViewport->GetSizeXY();
	const int32 MaxX = FMath::Min(ViewportSize.X, FMath::TruncToInt32(DragEndPosition.X + 1.0));
	const int32 MaxY = FMath::Min(ViewportSize.Y, FMath::TruncToInt32(DragEndPosition.Y + 1.0));

	const FIntPoint Min { MinX, MinY };
	const FIntPoint Max { MaxX, MaxY };
	DragSelectionRect = { Min, Max };
}

bool FEditorViewportSelectability::BoxSelectWorldActors(FBox& InBox, FEditorViewportClient* const InEditorViewportClient, const bool bInSelect)
{
	if (!GEditor || !InEditorViewportClient || InEditorViewportClient->IsInGameView())
	{
		return false;
	}

	UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!ensure(TypedElementRegistry))
	{
		return false;
	}

	UTypedElementSelectionSet* const SelectionSet = GetLevelEditorSelectionSet();
	if (!IsValid(SelectionSet))
	{
		return false;
	}

	const ULevelEditorViewportSettings* const LevelEditorViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	if (!IsValid(LevelEditorViewportSettings))
	{
		return false;
	}

	SelectionSet->Modify();

	const bool bUseStrictSelection = LevelEditorViewportSettings->bStrictBoxSelection;

	FWorldSelectionElementArgs SelectionArgs
	{
		SelectionSet,
		ETypedElementSelectionMethod::Primary,
		FTypedElementSelectionOptions(),
		&InEditorViewportClient->EngineShowFlags,
		bUseStrictSelection,
		false
	};

	TArray<FTypedElementHandle> ElementsToSelect;

	auto AddToElementsToSelect = [this, SelectionSet, &ElementsToSelect](const FTypedElementHandle& InElement)
	{
		if (IsTypedElementSelectable(InElement))
		{
			ElementsToSelect.Add(SelectionSet->GetSelectionElement(InElement, ETypedElementSelectionMethod::Primary));
		}
	};

	if (LevelEditorViewportSettings->bTransparentBoxSelection)
	{
		// Get a list of box-culled elements
		for (FActorIterator It(InEditorViewportClient->GetWorld()); It; ++It)
		{
			AActor* const Actor = *It;
			GetSelectionElements(Actor, [this, &InBox, &SelectionArgs, &AddToElementsToSelect]
				(const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
				{
					for (const FTypedElementHandle& Element : InWorldElement.GetSelectionElementsInBox(InBox, SelectionArgs))
					{
						AddToElementsToSelect(Element);
					}
				});
		}
	}
	else
	{
		const FTypedElementListRef ElementList = TypedElementRegistry->CreateElementList();
		InEditorViewportClient->Viewport->GetElementHandlesInRect(DragSelectionRect, ElementList);

		if (bUseStrictSelection)
		{
			ElementList->ForEachElement<ITypedElementWorldInterface>([this, bUseStrictSelection, &InBox, &AddToElementsToSelect]
				(const TTypedElement<ITypedElementWorldInterface>& InElement)
				{
					if (InElement.IsElementInBox(InBox, bUseStrictSelection))
					{
						AddToElementsToSelect(InElement);
					}
					return true;
				});
		}
		else
		{
			ElementList->ForEachElementHandle([this, &AddToElementsToSelect]
				(const FTypedElementHandle& InElement)
				{
					AddToElementsToSelect(InElement);
					return true;
				});
		}
	}

	const bool bShiftDown = InEditorViewportClient->Viewport->KeyState(EKeys::LeftShift)
		|| InEditorViewportClient->Viewport->KeyState(EKeys::RightShift);

	if (!bShiftDown)
	{
		// If the user is selecting, but isn't hold down SHIFT, remove the previous selections
		SelectionSet->SetSelection(MoveTemp(ElementsToSelect), FTypedElementSelectionOptions());
	}
	else
	{
		SelectionSet->SelectElements(MoveTemp(ElementsToSelect), FTypedElementSelectionOptions());
	}

	return true;
}

bool FEditorViewportSelectability::FrustumSelectWorldActors(const FConvexVolume& InFrustum, FEditorViewportClient* const InEditorViewportClient, const bool bInSelect)
{
	if (!GEditor || !InEditorViewportClient || InEditorViewportClient->IsInGameView())
	{
		return false;
	}

	UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!ensure(TypedElementRegistry))
	{
		return false;
	}

	UTypedElementSelectionSet* const SelectionSet = GetLevelEditorSelectionSet();
	if (!IsValid(SelectionSet))
	{
		return false;
	}

	const ULevelEditorViewportSettings* const LevelEditorViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	if (!IsValid(LevelEditorViewportSettings))
	{
		return false;
	}

	SelectionSet->Modify();

	const bool bUseStrictSelection = LevelEditorViewportSettings->bStrictBoxSelection;

	FWorldSelectionElementArgs SelectionArgs
	{
		SelectionSet,
		ETypedElementSelectionMethod::Primary,
		FTypedElementSelectionOptions(),
		&InEditorViewportClient->EngineShowFlags,
		bUseStrictSelection,
		false
	};

	TArray<FTypedElementHandle> ElementsToSelect;

	auto AddToElementsToSelect = [this, SelectionSet, &ElementsToSelect](const FTypedElementHandle& InElement)
	{
		if (IsTypedElementSelectable(InElement))
		{
			ElementsToSelect.Add(SelectionSet->GetSelectionElement(InElement, ETypedElementSelectionMethod::Primary));
		}
	};

	if (LevelEditorViewportSettings->bTransparentBoxSelection)
	{
		// Get a list of frustum-culled elements
		for (FActorIterator It(InEditorViewportClient->GetWorld()); It; ++It)
		{
			AActor* Actor = *It;
			GetSelectionElements(Actor, [this, &InFrustum, &SelectionArgs, &AddToElementsToSelect]
				(const TTypedElement<ITypedElementWorldInterface>& InWorldElement)
				{
					for (const FTypedElementHandle& Element : InWorldElement.GetSelectionElementsInConvexVolume(InFrustum, SelectionArgs))
					{
						AddToElementsToSelect(Element);
					}
				});
		}
	}
	else
	{
		const FTypedElementListRef ElementList = TypedElementRegistry->CreateElementList();
		InEditorViewportClient->Viewport->GetElementHandlesInRect(DragSelectionRect, ElementList);

		if (bUseStrictSelection)
		{
			ElementList->ForEachElement<ITypedElementWorldInterface>([this, bUseStrictSelection, &InFrustum, &AddToElementsToSelect]
				(const TTypedElement<ITypedElementWorldInterface>& InElement)
				{
					if (InElement.IsElementInConvexVolume(InFrustum, bUseStrictSelection))
					{
						AddToElementsToSelect(InElement);
					}
					return true;
				});
		}
		else
		{
			ElementList->ForEachElementHandle([this, &AddToElementsToSelect]
				(const FTypedElementHandle& InElement)
				{
					AddToElementsToSelect(InElement);
					return true;
				});
		}
	}

	const bool bShiftDown = InEditorViewportClient->Viewport->KeyState(EKeys::LeftShift)
		|| InEditorViewportClient->Viewport->KeyState(EKeys::RightShift);

	if (!bShiftDown)
	{
		// If the user is selecting, but isn't hold down SHIFT, remove the previous selections
		SelectionSet->SetSelection(MoveTemp(ElementsToSelect), FTypedElementSelectionOptions());
	}
	else
	{
		SelectionSet->SelectElements(MoveTemp(ElementsToSelect), FTypedElementSelectionOptions());
	}

	return true;
}

void FEditorViewportSelectability::DrawEnabledTextNotice(FCanvas* const InCanvas, const FText& InText)
{
	const FStringView HelpString = *InText.ToString();

	FTextSizingParameters SizingParameters(GEngine->GetLargeFont(), 1.f, 1.f);
	UCanvas::CanvasStringSize(SizingParameters, HelpString);

	const float ViewWidth = InCanvas->GetViewRect().Width() / InCanvas->GetDPIScale();
	const float DrawX = FMath::FloorToFloat((ViewWidth - SizingParameters.DrawXL) * 0.5f);
	InCanvas->DrawShadowedString(DrawX, 34.f, HelpString, GEngine->GetLargeFont(), FLinearColor::White);
}

FText FEditorViewportSelectability::GetLimitedSelectionText(const TSharedPtr<FUICommandInfo>& InToggleAction, const FText& InDefaultText)
{
	if (!InToggleAction.IsValid())
	{
		return FText();
	}

	FText HelpText = InDefaultText.IsEmpty() ? DefaultLimitedSelectionText : InDefaultText;

	if (InToggleAction.IsValid())
	{
		const TSharedRef<const FInputChord> ActiveChord = InToggleAction->GetFirstValidChord();
		if (ActiveChord->IsValidChord())
		{
			HelpText = FText::Format(LOCTEXT("LimitedSelectionActionKeyHelp", "{0}  ({1} to toggle)"), HelpText, ActiveChord->GetInputText(true));
		}
	}

	return HelpText;
}

UTypedElementSelectionSet* FEditorViewportSelectability::GetLevelEditorSelectionSet()
{
	if (!ensure(GEditor))
	{
		return nullptr;
	}

	UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!ensure(TypedElementRegistry))
	{
		return nullptr;
	}
	
	ULevelEditorSubsystem* const EditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();
	if (!ensure(IsValid(EditorSubsystem)))
	{
		return nullptr;
	}

	return EditorSubsystem->GetSelectionSet();
}

bool FEditorViewportSelectability::IsTypedElementSelectable(const FTypedElementHandle& InElementHandle) const
{
	UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!ensure(TypedElementRegistry))
	{
		return false;
	}

	TTypedElement<ITypedElementObjectInterface> const ObjectInterface = TypedElementRegistry->GetElement<ITypedElementObjectInterface>(InElementHandle);
	if (!ObjectInterface)
	{
		return false;
	}

	UPrimitiveComponent* const PrimitiveComponent = ObjectInterface.GetObjectAs<UPrimitiveComponent>();
	if (!IsValid(PrimitiveComponent))
	{
		return false;
	}

	AActor* const Actor = PrimitiveComponent->GetOwner();
	if (!IsValid(Actor))
	{
		return false;
	}

	if (Actor->IsHiddenEd() || !IsActorSelectableClass(*Actor))
	{
		return false;
	}

	if (GCurrentLevelEditingViewportClient && IsActorInLevelHiddenLayer(*Actor, GCurrentLevelEditingViewportClient))
	{
		return false;
	}

	return IsObjectSelectableInViewport(Actor);
}

void FEditorViewportSelectability::GetSelectionElements(AActor* const InActor, const TFunctionRef<void(const TTypedElement<ITypedElementWorldInterface>&)> InPredicate)
{
	if (!IsValid(InActor))
	{
		return;
	}

	if (InActor->IsA(AVolume::StaticClass()))
	{
		if (!GCurrentLevelEditingViewportClient
			|| !GCurrentLevelEditingViewportClient->IsVolumeVisibleInViewport(*InActor))
		{
			return;
		}
	}

	UTypedElementRegistry* const TypedElementRegistry = UTypedElementRegistry::GetInstance();
	if (!ensure(TypedElementRegistry))
	{
		return;
	}

	if (const FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor))
	{
		if (const TTypedElement<ITypedElementWorldInterface> WorldElement = TypedElementRegistry->GetElement<ITypedElementWorldInterface>(ElementHandle))
		{
			InPredicate(WorldElement);
		}
	}
}

#undef LOCTEXT_NAMESPACE
