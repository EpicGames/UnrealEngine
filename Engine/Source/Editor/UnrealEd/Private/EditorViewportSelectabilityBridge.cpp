// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportSelectabilityBridge.h"

FEditorViewportSelectabilityBridge::FEditorViewportSelectabilityBridge(const TWeakPtr<FEditorViewportClient>& InEditorViewportClientWeak)
	: EditorViewportClientWeak(InEditorViewportClientWeak)
{
}

FOnIsViewportSelectionLimited& FEditorViewportSelectabilityBridge::OnIsViewportSelectionLimited()
{
	return IsViewportSelectionLimitedDelegate;
}

FOnIsObjectSelectableInViewport& FEditorViewportSelectabilityBridge::OnGetIsObjectSelectableInViewport()
{
	return GetIsObjectSelectableInViewportDelegate;
}

FOnGetViewportSelectionLimitedText& FEditorViewportSelectabilityBridge::OnGetViewportSelectionLimitedText()
{
	return GetViewportSelectionLimitedTextDelegate;
}

bool FEditorViewportSelectabilityBridge::IsViewportSelectionLimited() const
{
	if (IsViewportSelectionLimitedDelegate.IsBound())
	{
		return IsViewportSelectionLimitedDelegate.Execute();
	}
	return false;
}

bool FEditorViewportSelectabilityBridge::IsObjectSelectableInViewport(UObject* const InObject) const
{
	if (GetIsObjectSelectableInViewportDelegate.IsBound())
	{
		return GetIsObjectSelectableInViewportDelegate.Execute(InObject);
	}
	return true;
}

FText FEditorViewportSelectabilityBridge::GetViewportSelectionLimitedText() const
{
	if (GetViewportSelectionLimitedTextDelegate.IsBound())
	{
		return GetViewportSelectionLimitedTextDelegate.Execute();
	}
	return FText();
}
