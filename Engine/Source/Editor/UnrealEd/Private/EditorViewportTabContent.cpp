// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorViewportTabContent.h"
#include "SEditorViewport.h"
#include "Framework/Docking/LayoutService.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"

TSharedPtr< FEditorViewportLayout > FEditorViewportTabContent::ConstructViewportLayoutByTypeName(const FName& TypeName, bool bSwitchingLayouts)
{
	TSharedPtr<FEditorViewportLayout> ViewportLayout = FactoryViewportLayout(bSwitchingLayouts);
	ViewportLayout->FactoryPaneConfigurationFromTypeName(TypeName);
	return ViewportLayout;
}

void FEditorViewportTabContent::Initialize(AssetEditorViewportFactoryFunction Func, TSharedPtr<SDockTab> InParentTab, const FString& InLayoutString)
{
	check(!InLayoutString.IsEmpty());
	check(Func);

	ParentTab = InParentTab;
	LayoutString = InLayoutString;

	FName LayoutType = GetLayoutTypeNameFromLayoutString();
	ViewportCreationFactories.Add(AssetEditorViewportCreationFactories::ElementType(NAME_None, Func));
	SetViewportConfiguration(LayoutType);
}

TSharedPtr<SAssetEditorViewport> FEditorViewportTabContent::CreateSlateViewport(FName InTypeName, const FAssetEditorViewportConstructionArgs& ConstructionArgs) const
{
	if (const AssetEditorViewportFactoryFunction* CreateFunc = ViewportCreationFactories.Find(InTypeName))
	{
		return (*CreateFunc)(ConstructionArgs);
	}

	// CreateSlateViewport should not be called before Initialize
	check(ViewportCreationFactories.Find(NAME_None));
	return ViewportCreationFactories[NAME_None](ConstructionArgs);
}

void FEditorViewportTabContent::SetViewportConfiguration(const FName& ConfigurationName)
{
	bool bSwitchingLayouts = ActiveViewportLayout.IsValid();
	OnViewportTabContentLayoutStartChangeEvent.Broadcast(bSwitchingLayouts);

	if (bSwitchingLayouts)
	{
		SaveConfig();
		ActiveViewportLayout.Reset();
	}

	ActiveViewportLayout = ConstructViewportLayoutByTypeName(ConfigurationName, bSwitchingLayouts);
	check(ActiveViewportLayout.IsValid());

	UpdateViewportTabWidget();

	OnViewportTabContentLayoutChangedEvent.Broadcast();
}

void FEditorViewportTabContent::SaveConfig() const
{
	if (ActiveViewportLayout.IsValid())
	{
		ActiveViewportLayout->SaveConfig(LayoutString);
	}
}

TSharedPtr<SEditorViewport> FEditorViewportTabContent::GetFirstViewport()
{
 	const TMap< FName, TSharedPtr< IEditorViewportLayoutEntity > >& EditorViewports = ActiveViewportLayout->GetViewports();
 
	for (auto& Pair : EditorViewports)
	{
		TSharedPtr<SWidget> ViewportWidget = StaticCastSharedPtr<IEditorViewportLayoutEntity>(Pair.Value)->AsWidget();
		TSharedPtr<SEditorViewport> Viewport = StaticCastSharedPtr<SEditorViewport>(ViewportWidget);
		if (Viewport.IsValid())
		{
			return Viewport;
		}
	}

	return nullptr;
}


void FEditorViewportTabContent::UpdateViewportTabWidget()
{
	TSharedPtr<SDockTab> ParentTabPinned = ParentTab.Pin();
	if (ParentTabPinned.IsValid() && ActiveViewportLayout.IsValid())
	{
		TSharedRef<SWidget> LayoutWidget = StaticCastSharedPtr<FAssetEditorViewportLayout>(ActiveViewportLayout)->BuildViewportLayout(ParentTabPinned, SharedThis(this), LayoutString);
 		ParentTabPinned->SetContent(LayoutWidget);

		if (PreviouslyFocusedViewport.IsSet())
		{
			TSharedPtr<IEditorViewportLayoutEntity> ViewportToFocus = ActiveViewportLayout->GetViewports().FindRef(PreviouslyFocusedViewport.GetValue());
			if (ViewportToFocus.IsValid())
			{
				ViewportToFocus->SetKeyboardFocus();
			}
			PreviouslyFocusedViewport = TOptional<FName>();
		}
	}
}

void FEditorViewportTabContent::RefreshViewportConfiguration()
{
	if (!ActiveViewportLayout.IsValid())
	{
		return;
	}

	FName ConfigurationName = ActiveViewportLayout->GetActivePaneConfigurationTypeName();
	for (auto& Pair : ActiveViewportLayout->GetViewports())
	{
		if (Pair.Value->AsWidget()->HasFocusedDescendants())
		{
			PreviouslyFocusedViewport = Pair.Key;
			break;
		}
	}

	// Since we don't want config to save out, go ahead and clear out the active viewport layout before refreshing the current layout
	ActiveViewportLayout.Reset();
	SetViewportConfiguration(ConfigurationName);
}

const AssetEditorViewportFactoryFunction* FEditorViewportTabContent::FindViewportCreationFactory(FName InTypeName) const
{
	return ViewportCreationFactories.Find(InTypeName);
}

FName FEditorViewportTabContent::GetLayoutTypeNameFromLayoutString() const
{
	return *LayoutString;
}

TSharedPtr<FEditorViewportLayout> FEditorViewportTabContent::FactoryViewportLayout(bool bIsSwitchingLayouts)
{
	return MakeShareable(new FAssetEditorViewportLayout);
}
