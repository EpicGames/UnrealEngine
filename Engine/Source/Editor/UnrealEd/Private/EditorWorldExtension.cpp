// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorWorldExtension.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorWorldExtension)

/************************************************************************/
/* UEditorExtension		                                                */
/************************************************************************/
UEditorWorldExtension::UEditorWorldExtension() :
	Super(),
	OwningExtensionsCollection(nullptr),
	bActive(true)
{

}

UEditorWorldExtension::~UEditorWorldExtension()
{
	if (OwningExtensionsCollection)
	{
		OwningExtensionsCollection->RemoveExtension(this);
	}
	OwningExtensionsCollection = nullptr;
}

bool UEditorWorldExtension::InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event )
{
	return false;
}

bool UEditorWorldExtension::InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime )
{
	return false;
}

UWorld* UEditorWorldExtension::GetWorld() const
{
	if (OwningExtensionsCollection == nullptr)
	{
		return nullptr;
	}
	return OwningExtensionsCollection->GetWorld();
}

UWorld* UEditorWorldExtension::GetLastEditorWorld() const
{
	return OwningExtensionsCollection->GetLastEditorWorld();
}

AActor* UEditorWorldExtension::SpawnTransientSceneActor(TSubclassOf<AActor> ActorClass, const FString& ActorName, const bool bWithSceneComponent /*= false*/, const EObjectFlags InObjectFlags /*= EObjectFlags::RF_DuplicateTransient*/, const bool bValidForPIE /* = false */)
{
	UWorld* World = GetWorld();
	check(World != nullptr);

	// if currently in PIE, non-PIE actors should be spawned in LastEditorWorld if it exists.
	if (!bValidForPIE && !GEditor->bIsSimulatingInEditor && GEditor->PlayWorld != nullptr && GEditor->PlayWorld == World)
	{
		UWorld* LastEditorWorld = GetLastEditorWorld();
		if (LastEditorWorld != nullptr)
		{
			World = LastEditorWorld;
		}
	}

	const bool bWasWorldPackageDirty = World->GetOutermost()->IsDirty();

	FActorSpawnParameters ActorSpawnParameters;
	ActorSpawnParameters.Name = MakeUniqueObjectName(World->GetCurrentLevel(), ActorClass, *ActorName);
	ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	ActorSpawnParameters.ObjectFlags = InObjectFlags;
	ActorSpawnParameters.OverrideLevel = World->GetCurrentLevel();

	check(ActorClass != nullptr);
	AActor* NewActor = World->SpawnActor< AActor >(ActorClass, ActorSpawnParameters);
	NewActor->SetActorLabel(ActorName);

	FEditorWorldExtensionActorData ActorData;
	ActorData.Actor = NewActor;
	ActorData.bValidForPIE = bValidForPIE;

	// Keep track of this actor so that we can migrate it between worlds if needed
	ExtensionActors.Add( ActorData );

	if (bWithSceneComponent)
	{
		// Give the new actor a root scene component, so we can attach multiple sibling components to it
		USceneComponent* SceneComponent = NewObject<USceneComponent>(NewActor);
		NewActor->AddOwnedComponent(SceneComponent);
		NewActor->SetRootComponent(SceneComponent);
		SceneComponent->RegisterComponent();
	}

	// Don't dirty the level file after spawning a transient actor
	if (!bWasWorldPackageDirty)
	{
		World->GetOutermost()->SetDirtyFlag(false);
	}

	return NewActor;
}

void UEditorWorldExtension::DestroyTransientActor(AActor* Actor)
{
	if (Actor != nullptr)
	{
		for (int32 ActorIndex = 0; ActorIndex < ExtensionActors.Num(); ++ActorIndex)
		{
			FEditorWorldExtensionActorData ActorData = ExtensionActors[ActorIndex];
			if (ActorData.Actor == Actor)
			{
				ExtensionActors.RemoveAtSwap(ActorIndex--);
				break;
			}
		}
	}

	if (Actor != nullptr)
	{
		UWorld* World = Actor->GetWorld();
		const bool bWasWorldPackageDirty = World->GetOutermost()->IsDirty();

		const bool bNetForce = false;
		const bool bShouldModifyLevel = false;	// Don't modify level for transient actor destruction
		World->DestroyActor(Actor, bNetForce, bShouldModifyLevel);

		// Don't dirty the level file after destroying a transient actor
		if (!bWasWorldPackageDirty)
		{
			World->GetOutermost()->SetDirtyFlag(false);
		}
	}
}

void UEditorWorldExtension::SetActive(const bool bInActive)
{
	bActive = bInActive;
}

bool UEditorWorldExtension::IsActive() const
{
	return bActive;
}

UEditorWorldExtensionCollection* UEditorWorldExtension::GetOwningCollection()
{
	return OwningExtensionsCollection;
}

bool UEditorWorldExtension::ExecCommand(const FString& InCommand)
{
	bool bResult = false;

	UWorld* World = GetWorld();

	// @todo vreditor: The following should not be needed.  It's a workaround because the input preprocessor
	// in VREditor fires input events without setting GWorld to the PlayWorld during event processing.  This
	// is inconsistent with the normal editor.  We're working around it by only using this logic when
	// GIsPlayInEditorWorld is not set. (Which would be the case in Force VR Mode)
	if (!GIsPlayInEditorWorld && GEditor->bIsSimulatingInEditor && GEditor->PlayWorld != nullptr && GEditor->PlayWorld == World)
	{
		UWorld* OldWorld = World;

		// The play world needs to be selected if it exists
		OldWorld = SetPlayInEditorWorld(OldWorld);

		bResult = GUnrealEd->Exec(World, *InCommand);

		// Restore the old world if there was one
		if (OldWorld)
		{
			RestoreEditorWorld(OldWorld);
		}
	}
	else
	{
		bResult = GUnrealEd->Exec(World, *InCommand);
	}

	return bResult;
}

void UEditorWorldExtension::TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState)
{
	check(NewWorld != nullptr);

	for (int32 ActorIndex = 0; ActorIndex < ExtensionActors.Num(); ++ActorIndex)
	{
		FEditorWorldExtensionActorData ActorData = ExtensionActors[ ActorIndex ];
		if( ActorData.Actor != nullptr)
		{ 
			if (TransitionState == EEditorWorldExtensionTransitionState::TransitionAll ||
				(TransitionState == EEditorWorldExtensionTransitionState::TransitionPIEOnly && ActorData.bValidForPIE) ||
				(TransitionState == EEditorWorldExtensionTransitionState::TransitionNonPIEOnly && !ActorData.bValidForPIE))
			{
				ReparentActor(ActorData.Actor, NewWorld);
			}
		}
		else
		{
			ExtensionActors.RemoveAtSwap( ActorIndex-- );
		}
	}
}

void UEditorWorldExtension::ReparentActor(AActor* Actor, UWorld* NewWorld)
{
	// Do not try to reparent the actor if it is in the same world as the requested new world
	if(Actor->GetWorld() == NewWorld)
	{
		return;
	}

	ULevel* Level = NewWorld->PersistentLevel;
	Actor->Rename(nullptr, Level);

	// Are we transitioning into a live world?
	if( NewWorld->HasBegunPlay() )
	{
		// @todo vreditor simulate: Instead of doing all of this "fake finish spawn" work for actors that we've transitioned
		// to the PlayWorld, we could instead transition the actors BEFORE PlayWorld->InitializeActorsForPlay() is called.
		// Then, the level itself can finish getting these actors ready to play.  See UGameInstance::StartPlayInEditorGameInstance().

		// @todo vreditor simulate: When transition our actors that have had BeginPlay() called on them back to the regular
		// EditorWorld, we'll need to reset various state so that they behave correctly in both the editor world, and also
		// in a new game world after PIE is started again.  We also may need to deregister them for replication.

		// @todo vreditor simulate: Even though the actor might be set to replicate, until it's been moved into a world with BeginPlay() called on it,
		// it will never have had a chance to actually register itself with the networking system for replication.  So we'll
		// toggle replicated state to make sure it's registered here.
		if( Actor->GetIsReplicated() )
		{
			Actor->SetReplicates( false );
			Actor->SetReplicates( true );
		}

		// @todo vreditor simulate: This is needed because actors spawned into the EditorWorld never have PostActorConstruction()
		// called on them, even though the actor is considered fully initialized.  Actors only have PostActorConstruction()
		// called after being spawned into a level that has had BeginPlay() called on it, or when the actor already resided
		// in the level before the level had BeginPlay() called on it.  
		Actor->PostActorConstruction();		// @todo vreditor simulate: We had to make this AActor function PUBLIC instead of PRIVATE to be able to do this.  Not ideal.

		Actor->DispatchBeginPlay();
	}
}

void UEditorWorldExtension::InitInternal(UEditorWorldExtensionCollection* InOwningExtensionsCollection)
{
	OwningExtensionsCollection = InOwningExtensionsCollection;
}

/************************************************************************/
/* UEditorWorldExtensionCollection                                      */
/************************************************************************/
UEditorWorldExtensionCollection::UEditorWorldExtensionCollection() :
	Super(),
	Currentworld(nullptr),
	LastEditorWorld(nullptr)
{
	if( !IsTemplate() )
	{
		FEditorDelegates::PostPIEStarted.AddUObject( this, &UEditorWorldExtensionCollection::PostPIEStarted );
		FEditorDelegates::PrePIEEnded.AddUObject( this, &UEditorWorldExtensionCollection::OnPreEndPIE );
		FEditorDelegates::EndPIE.AddUObject( this, &UEditorWorldExtensionCollection::OnEndPIE );
		FEditorDelegates::OnSwitchBeginPIEAndSIE.AddUObject(this, &UEditorWorldExtensionCollection::SwitchPIEAndSIE);
	}
}

UEditorWorldExtensionCollection::~UEditorWorldExtensionCollection()
{
	FEditorDelegates::PostPIEStarted.RemoveAll( this );
	FEditorDelegates::PrePIEEnded.RemoveAll( this );
	FEditorDelegates::EndPIE.RemoveAll( this );
	FEditorDelegates::OnSwitchBeginPIEAndSIE.RemoveAll( this );

	for (const FEditorExtensionTuple& Extension : EditorExtensions)
	{
		Extension.Get<0>()->OwningExtensionsCollection = nullptr;
	}
	EditorExtensions.Empty();
	Currentworld.Reset();
	LastEditorWorld.Reset();
}

UWorld* UEditorWorldExtensionCollection::GetWorld() const
{
	return Currentworld.IsValid() ? Currentworld.Get() : nullptr;
}

UWorld* UEditorWorldExtensionCollection::GetLastEditorWorld() const
{
	return LastEditorWorld.IsValid() ? LastEditorWorld.Get() : nullptr;
}

UEditorWorldExtension* UEditorWorldExtensionCollection::AddExtension(TSubclassOf<UEditorWorldExtension> EditorExtensionClass)
{
	UEditorWorldExtension* Extension = nullptr;
	UEditorWorldExtension* FoundExtension = FindExtension(EditorExtensionClass);
	if (FoundExtension != nullptr)
	{
		Extension = FoundExtension;
	}
	else
	{
		Extension = NewObject<UEditorWorldExtension>(this, EditorExtensionClass);
	}

	AddExtension(Extension);
	return Extension;
}

void UEditorWorldExtensionCollection::AddExtension( UEditorWorldExtension* EditorExtension )
{
	check( EditorExtension != nullptr );

	const int32 ExistingExtensionIndex = EditorExtensions.IndexOfByPredicate(
		[ &EditorExtension ]( const FEditorExtensionTuple& Element ) -> bool
		{ 
			return Element.Get<0>() == EditorExtension;
		} 
	);

	if( ExistingExtensionIndex != INDEX_NONE )
	{
		FEditorExtensionTuple& EditorExtensionTuple = EditorExtensions[ ExistingExtensionIndex ];
		int32& RefCount = EditorExtensionTuple.Get<1>();
		++RefCount;
	}
	else
	{
		const int32 InitialRefCount = 1;
		EditorExtensions.Add( FEditorExtensionTuple( EditorExtension, InitialRefCount ) );

		EditorExtension->InitInternal(this);
		EditorExtension->Init();
	}
}

void UEditorWorldExtensionCollection::RemoveExtension( UEditorWorldExtension* EditorExtension )
{
	check( EditorExtension != nullptr );

	const int32 ExistingExtensionIndex = EditorExtensions.IndexOfByPredicate(
		[ &EditorExtension ]( const FEditorExtensionTuple& Element ) -> bool 
		{ 
			return Element.Get<0>() == EditorExtension;
		} 
	);
	if( ensure( ExistingExtensionIndex != INDEX_NONE ) )
	{
		check(EditorExtension->OwningExtensionsCollection == this);
		FEditorExtensionTuple& EditorExtensionTuple = EditorExtensions[ ExistingExtensionIndex ];
		int32& RefCount = EditorExtensionTuple.Get<1>();
		--RefCount;

		if( RefCount == 0 )
		{
			EditorExtensions.RemoveAt( ExistingExtensionIndex );
			EditorExtension->Shutdown();
			EditorExtension->OwningExtensionsCollection = nullptr;
		}
	}
}

UEditorWorldExtension* UEditorWorldExtensionCollection::FindExtension( TSubclassOf<UEditorWorldExtension> EditorExtensionClass )
{
	UEditorWorldExtension* ResultExtension = nullptr;
	for( FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions )
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		if( EditorExtension->GetClass() == EditorExtensionClass )
		{
			ResultExtension = EditorExtension;
			break;
		}
	}

	return ResultExtension;
}

void UEditorWorldExtensionCollection::Tick( float DeltaSeconds )
{
	for (FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions)
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		if (EditorExtension->IsActive())
		{
			EditorExtension->Tick(DeltaSeconds);
		}
	}
}

bool UEditorWorldExtensionCollection::InputKey( FEditorViewportClient* InViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event )
{
	bool bHandled = false;
	for( FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions )
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		bHandled |= EditorExtension->InputKey( InViewportClient, Viewport, Key, Event );
	}

	return bHandled;
}

bool UEditorWorldExtensionCollection::InputAxis( FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 ControllerId, FKey Key, float Delta, float DeltaTime )
{
	bool bHandled = false;

	for( FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions )
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		bHandled |= EditorExtension->InputAxis( InViewportClient, Viewport, ControllerId, Key, Delta, DeltaTime );
	}

	return bHandled;
}


void UEditorWorldExtensionCollection::ShowAllActors(const bool bShow)
{
	for (FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions)
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		for (FEditorWorldExtensionActorData ActorData : EditorExtension->ExtensionActors)
		{
			AActor* Actor = ActorData.Actor;
			if (Actor != nullptr)
			{
				TInlineComponentArray<USceneComponent*> ComponentArray;
				Actor->GetComponents(ComponentArray);
				for (USceneComponent* Component : ComponentArray)
				{
					Component->SetVisibility(bShow);
				}
				Actor->SetActorEnableCollision(bShow);
			}
		}
	}
}


void UEditorWorldExtensionCollection::PostPIEStarted( bool bIsSimulatingInEditor )
{
	if( GEditor->EditorWorld != nullptr && Currentworld.IsValid() && GEditor->EditorWorld == Currentworld.Get() && GEditor->PlayWorld != nullptr )
	{
		if (bIsSimulatingInEditor)
		{
			// Editor to SIE
			// Transition all actors to the play world.
			SetWorld(GEditor->PlayWorld, EEditorWorldExtensionTransitionState::TransitionAll);

			LastEditorWorld = GEditor->GetEditorWorldContext().World();

			for (FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions)
			{
				UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
				EditorExtension->EnteredSimulateInEditor();
			}
		}
		else
		{
			// Editor to PIE 
			// Transition PIE-valid actors to the play world.
			SetWorld(GEditor->PlayWorld, EEditorWorldExtensionTransitionState::TransitionPIEOnly);

			LastEditorWorld = GEditor->GetEditorWorldContext().World();
		}
	}
}

void UEditorWorldExtensionCollection::OnPreEndPIE(bool bWasSimulatingInEditor)
{
	if (!bWasSimulatingInEditor && LastEditorWorld.IsValid() && LastEditorWorld.Get() == GEditor->EditorWorld)
	{
		if (!IsEngineExitRequested())
		{
			// PIE to Editor 
			// Revert back to the editor world before closing the play world, otherwise actors and objects will be destroyed.
			// Transition PIE-valid extension actors back to the editor world.
			SetWorld(GEditor->EditorWorld, EEditorWorldExtensionTransitionState::TransitionPIEOnly);

			LastEditorWorld.Reset();
		}
	}
}

void UEditorWorldExtensionCollection::OnEndPIE( bool bWasSimulatingInEditor )
{
	if( bWasSimulatingInEditor && LastEditorWorld.IsValid() && LastEditorWorld.Get() == GEditor->EditorWorld )
	{
		if( !IsEngineExitRequested() )
		{
			UWorld* SimulateWorld = Currentworld.Get();

			// SIE to Editor 
			// Revert back to the editor world before closing the play world, otherwise actors and objects will be destroyed.
			// Transition all extension actors back to the editor world.
			SetWorld(GEditor->EditorWorld, EEditorWorldExtensionTransitionState::TransitionAll);

			LastEditorWorld.Reset();

			if (SimulateWorld != nullptr)
			{
				for (FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions)
				{
					UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
					EditorExtension->LeftSimulateInEditor(SimulateWorld);
				}
			}
		}
	}
}

void UEditorWorldExtensionCollection::SwitchPIEAndSIE(bool bIsSimulatingInEditor)
{
	if (GEditor->EditorWorld != nullptr && LastEditorWorld.IsValid() && LastEditorWorld.Get() == GEditor->EditorWorld && 
		GEditor->PlayWorld != nullptr && Currentworld.IsValid() && Currentworld.Get() == GEditor->PlayWorld)
	{
		if (!bIsSimulatingInEditor)
		{
			// Post SIE to PIE.
			// Transition non-PIE extension actors to the editor world while in PIE.
			TransitionWorld(GEditor->EditorWorld, EEditorWorldExtensionTransitionState::TransitionNonPIEOnly);
		}
		else
		{
			// Post PIE to SIE
			// Transition non-PIE extension actors back to simulate world from editor world where they were temporarily moved while in PIE.
			TransitionWorld(GEditor->PlayWorld, EEditorWorldExtensionTransitionState::TransitionNonPIEOnly);
		}
	}
}

void UEditorWorldExtensionCollection::TransitionWorld(UWorld* World, EEditorWorldExtensionTransitionState TransitionState)
{
	check(World != nullptr);

	for (FEditorExtensionTuple& EditorExtensionTuple : EditorExtensions)
	{
		UEditorWorldExtension* EditorExtension = EditorExtensionTuple.Get<0>();
		EditorExtension->TransitionWorld(World, TransitionState);
	}
}

void UEditorWorldExtensionCollection::SetWorld(UWorld* World, EEditorWorldExtensionTransitionState TransitionState /* = EEditorWorldExtensionTransitionState::TransitionAll */)
{
	check(World != nullptr);

	// First time setting the world on collection we don't want to transition because there is nothing yet to transition from.
	if (Currentworld.IsValid() && TransitionState != EEditorWorldExtensionTransitionState::TransitionNone)
	{
		TransitionWorld(World, TransitionState);
	}

	Currentworld = World;
}

/************************************************************************/
/* UEditorWorldExtensionManager                                         */
/************************************************************************/
UEditorWorldExtensionManager::UEditorWorldExtensionManager() :
	Super()
{
	if( GEngine )
	{
		GEngine->OnWorldContextDestroyed().AddUObject( this, &UEditorWorldExtensionManager::OnWorldContextRemove );
	}
}

UEditorWorldExtensionManager::~UEditorWorldExtensionManager()
{
	if( GEngine )
	{
		GEngine->OnWorldContextDestroyed().RemoveAll( this );
	}

	EditorWorldExtensionCollection.Empty();
}

UEditorWorldExtensionCollection* UEditorWorldExtensionManager::GetEditorWorldExtensions(UWorld* World, const bool bCreateIfNeeded /**= true*/)
{
	// Try to find this world in the map and return it or create and add one if nothing found
	UEditorWorldExtensionCollection* Result = nullptr;
	if (World)
	{
		UEditorWorldExtensionCollection* FoundExtensionCollection = FindExtensionCollection(World);
		if (FoundExtensionCollection != nullptr)
		{
			Result = FoundExtensionCollection;
		}
		else if(bCreateIfNeeded)
		{
			Result = OnWorldAdd(World);
		}
	}
	return Result;
}

UEditorWorldExtensionCollection* UEditorWorldExtensionManager::OnWorldAdd(UWorld* World)
{
	UEditorWorldExtensionCollection* Result = nullptr;
	if (World != nullptr)
	{
		UEditorWorldExtensionCollection* ExtensionCollection = NewObject<UEditorWorldExtensionCollection>();
		ExtensionCollection->SetWorld(World, EEditorWorldExtensionTransitionState::TransitionAll);
		Result = ExtensionCollection;
		EditorWorldExtensionCollection.Add(Result);
	}
	return Result;
}

void UEditorWorldExtensionManager::OnWorldContextRemove(FWorldContext& InWorldContext)
{
	UWorld* World = InWorldContext.World();
	if(World)
	{
		UEditorWorldExtensionCollection* ExtensionCollection = FindExtensionCollection(World);
		if (ExtensionCollection)
		{
			EditorWorldExtensionCollection.Remove(ExtensionCollection);
		}
	}
}

UEditorWorldExtensionCollection* UEditorWorldExtensionManager::FindExtensionCollection(const UWorld* InWorld)
{
	UEditorWorldExtensionCollection* ResultCollection = nullptr;
	for (UEditorWorldExtensionCollection* ExtensionCollection : EditorWorldExtensionCollection)
	{
		if (ExtensionCollection != nullptr && ExtensionCollection->GetWorld() == InWorld)
		{
			ResultCollection = ExtensionCollection;
			break;
		}
	}

	return ResultCollection;
}

void UEditorWorldExtensionManager::Tick( float DeltaSeconds )
{
	// Tick all the collections
	for(UEditorWorldExtensionCollection* ExtensionCollection : EditorWorldExtensionCollection)
	{
		check(ExtensionCollection != nullptr && ExtensionCollection->IsValidLowLevel());
		ExtensionCollection->Tick(DeltaSeconds);
	}
}
