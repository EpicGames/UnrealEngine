// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorWorldUtils.h"
#include "Editor.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

DEFINE_LOG_CATEGORY_STATIC(EditorWorldUtilsLog, Log, All);

FScopedEditorWorld::FScopedEditorWorld()
	: World(nullptr)
	, PrevGWorld(nullptr)
{
}

FScopedEditorWorld::FScopedEditorWorld(UWorld* InWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType)
	: FScopedEditorWorld()
{
	Init(InWorld, InInitializationValues, InWorldType);
}

FScopedEditorWorld::FScopedEditorWorld(const FStringView InLongPackageName, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType)
	: FScopedEditorWorld(TSoftObjectPtr<UWorld>(FSoftObjectPath(InLongPackageName)), InInitializationValues, InWorldType)
{
}

FScopedEditorWorld::FScopedEditorWorld(const TSoftObjectPtr<UWorld>& InSoftWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType)
	: FScopedEditorWorld()
{
	FSoftObjectPath WorldObjectPath(InSoftWorld.ToSoftObjectPath());
	// Handle cases where the SoftObjectPath doesn't have a valid AssetName
	if (WorldObjectPath.GetAssetFName().IsNone())
	{
		FString WorldObjectPathStr = WorldObjectPath.GetLongPackageName() + TEXT(".") + FPackageName::GetShortName(WorldObjectPath.GetLongPackageName());
		WorldObjectPath = FSoftObjectPath(WorldObjectPathStr);
	}
		
	if (UPackage* WorldPackage = LoadWorldPackageForEditor(WorldObjectPath.GetLongPackageName()))
	{
		if (UWorld* RuntimeWorld = UWorld::FindWorldInPackage(WorldPackage))
		{
			Init(RuntimeWorld, InInitializationValues, InWorldType);
		}
	}
}

void FScopedEditorWorld::Init(UWorld* InWorld, const UWorld::InitializationValues& InInitializationValues, EWorldType::Type InWorldType)
{
	check(InWorld);
	check(!InWorld->bIsWorldInitialized);

	World = InWorld;

	// Add to root
	World->AddToRoot();

	// Set current GWorld / WorldContext
	FWorldContext& WorldContext = GEditor->GetEditorWorldContext(true);
	WorldContext.SetCurrentWorld(World);
	PrevGWorld = GWorld;
	GWorld = World;

	// Initialize the world
	World->WorldType = InWorldType;
	World->InitWorld(InInitializationValues);
	World->PersistentLevel->UpdateModelComponents();
	World->UpdateWorldComponents(true /*bRerunConstructionScripts*/, false /*bCurrentLevelOnly*/);
	World->UpdateLevelStreaming();
}

UWorld* FScopedEditorWorld::GetWorld() const
{
	return World;
}

FScopedEditorWorld::~FScopedEditorWorld()
{
	// Ensure the editor's transaction system is reset to remove any lingering editor state changes when the ScopedEditorWorld is destroyed.
	// This prevents issues with persistent state data, such as in landscape mode tests, where the previous world state may interfere with consequent test runs.
	if (GEditor)
	{
		const FText TransReset = FText::FromString(TEXT("Resetting the Transaction System for ScopedEditorWorld destruction."));
		GEditor->Cleanse(true, true, TransReset);
	}
	if (World)
	{
		// Destroy world
		World->DestroyWorld(false /*bBroadcastWorldDestroyedEvent*/);

		// Unroot world
		World->RemoveFromRoot();

		// Restore previous GWorld / WorldContext
		FWorldContext& WorldContext = GEditor->GetEditorWorldContext(true);
		WorldContext.SetCurrentWorld(PrevGWorld);
		GWorld = PrevGWorld;
	}
}

UPackage* LoadWorldPackageForEditor(const FStringView InLongPackageName, EWorldType::Type InWorldType, uint32 InLoadFlags)
{
	FSoftObjectPath WorldPackagePath(InLongPackageName);
	UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldPackagePath);

	FString LongPackageName = WorldPackagePath.GetLongPackageName();
	FName WorldPackageFName(LongPackageName);
	UWorld::WorldTypePreLoadMap.FindOrAdd(WorldPackageFName) = InWorldType;
	UPackage* WorldPackage = LoadPackage(nullptr, *LongPackageName, InLoadFlags);
	UWorld::WorldTypePreLoadMap.Remove(WorldPackageFName);

	return WorldPackage;
}
