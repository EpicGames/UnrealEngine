// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementDetailsInterface.h"

#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementDetailsInterface)

class FActorTypedElementDetailsObject : public ITypedElementDetailsObject
{
public:
	explicit FActorTypedElementDetailsObject(AActor* InActor)
		: ActorPtr(InActor)
	{
	}

	virtual UObject* GetObject() override
	{
		return ActorPtr.Get();
	}

private:
	TWeakObjectPtr<AActor> ActorPtr;
};

TUniquePtr<ITypedElementDetailsObject> UActorElementDetailsInterface::GetDetailsObject(const FTypedElementHandle& InElementHandle)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		return MakeUnique<FActorTypedElementDetailsObject>(Actor);
	}
	return nullptr;
}
