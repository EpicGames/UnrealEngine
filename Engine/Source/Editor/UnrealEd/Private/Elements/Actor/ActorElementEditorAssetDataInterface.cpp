// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementEditorAssetDataInterface.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Blueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementEditorAssetDataInterface)

TArray<FAssetData> UActorElementEditorAssetDataInterface::GetAllReferencedAssetDatas(const FTypedElementHandle& InElementHandle, const FTypedElementAssetDataReferencedOptions& InOptions)
{
	TArray<FAssetData> AssetDatas;

	if (AActor* RawActorPtr = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		TArray<UObject*> ReferencedContentObjects;
		RawActorPtr->GetReferencedContentObjects(ReferencedContentObjects);

		if (InOptions.OnlyTopLevelAsset())
		{
			UObject** BPObject = ReferencedContentObjects.FindByPredicate([](const UObject* InObject)
			{
				if (InObject && (InObject->IsA<UBlueprint>() || InObject->IsA<UBlueprintGeneratedClass>()))
				{
					return true;
				}
				return false;
			});
			
			if (BPObject)
			{
				AssetDatas.Add(FAssetData(*BPObject, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering));
			}
		}

		if (AssetDatas.IsEmpty())
		{
			for (const UObject* ContentObject : ReferencedContentObjects)
			{
				FAssetData ObjectAssetData = FAssetData(ContentObject, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering);
				if (ObjectAssetData.IsValid())
				{
					AssetDatas.Emplace(ObjectAssetData);
				}
			}

			TArray<FSoftObjectPath> SoftObjects;
			RawActorPtr->GetSoftReferencedContentObjects(SoftObjects);
			if (SoftObjects.Num())
			{
				IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

				for (const FSoftObjectPath& SoftObject : SoftObjects)
				{
					FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(SoftObject);

					if (AssetData.IsValid())
					{
						AssetDatas.Add(AssetData);
					}
				}
			}
		}
	}

	return AssetDatas;
}
