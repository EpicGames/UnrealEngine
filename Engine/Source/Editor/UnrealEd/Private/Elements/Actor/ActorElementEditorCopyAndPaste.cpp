// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementEditorCopyAndPaste.h"

#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/GroupActor.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/World.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelUtils.h"
#include "Misc/Parse.h"
#include "Misc/FeedbackContext.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementEditorCopyAndPaste)

#define LOCTEXT_NAMESPACE "ActorElementCopyAndPaste"

UActorElementsExporterT3D::UActorElementsExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UActorElementsCopy::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal world text"));
}

bool UActorElementsExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type,
                                           FOutputDevice& Out, FFeedbackContext* Warn, uint32 PortFlags /*= 0*/)
{
	UActorElementsCopy* ActorElementCopy = Cast<UActorElementsCopy>(Object);
	if (!ActorElementCopy)
	{
		return false;
	}

	for (AActor* Actor : ActorElementCopy->ActorsToCopy)
	{
		if (!Actor->ShouldExport() || Actor->IsA<AGroupActor>())
		{
			continue;
		}

		// Temporarily unbind dynamic delegates so we don't export the bindings.
		UBlueprintGeneratedClass::UnbindDynamicDelegates(Actor->GetClass(), Actor);

		ULevel* Level = Actor->GetLevel();
		AActor* ParentActor = Actor->GetAttachParentActor();
		FName SocketName = Actor->GetAttachParentSocketName();

		// We set DetachmentTransformRules.bCallModify to false to match the level editor (don't need to modify on copy)
		FDetachmentTransformRules DetachmentTransformRules = FDetachmentTransformRules::KeepWorldTransform;
		DetachmentTransformRules.bCallModify = false;
		Actor->DetachFromActor(DetachmentTransformRules);

		FString ParentActorString = (ParentActor
			                             ? FString::Printf(TEXT(" ParentActor=%s"), *ParentActor->GetName())
			                             : TEXT(""));
		FString SocketNameString = ((ParentActor && SocketName != NAME_None)
			                            ? FString::Printf(TEXT(" SocketName=%s"), *SocketName.ToString())
			                            : TEXT(""));
		FString GroupActor = (Actor->GroupActor
			                      ? FString::Printf(TEXT(" GroupActor=%s"), *Actor->GroupActor->GetName())
			                      : TEXT(""));
		FString GroupFolder = (Actor->GroupActor
			                       ? FString::Printf(
				                       TEXT(" GroupFolder=%s"), *Actor->GroupActor->GetFolderPath().ToString())
			                       : TEXT(""));
		FString ActorFolderPath = (Level->IsUsingActorFolders()
			                           ? FString::Printf(
				                           TEXT(" ActorFolderPath=%s"), *Actor->GetFolderPath().ToString())
			                           : TEXT(""));
		Out.Logf(TEXT("%sBegin Actor Class=%s Name=%s Archetype=%s%s%s%s%s%s"),
		        FCString::Spc(TextIndent), *Actor->GetClass()->GetPathName(), *Actor->GetName(),
				*FObjectPropertyBase::GetExportPath(Actor->GetArchetype(), nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified),
		        *ParentActorString, *SocketNameString, *GroupActor, *GroupFolder, *ActorFolderPath);

		// When exporting for diffs, export paths can cause false positives. since diff files don't get imported, we can
		// skip adding this info the file.
		if (!(PortFlags & PPF_ForDiff))
		{
			// Emit the actor path
			Out.Logf(TEXT(" ExportPath=%s"), *FObjectPropertyBase::GetExportPath(Actor, nullptr, nullptr, (PortFlags | PPF_Delimited) & ~PPF_ExportsNotFullyQualified));
		}

		Out.Logf(LINE_TERMINATOR);

		ExportRootScope = Actor;
		ExportObjectInner(Context, Actor, Out, PortFlags);
		ExportRootScope = nullptr;

		Out.Logf(TEXT("%sEnd Actor\r\n"), FCString::Spc(TextIndent));
		Actor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform, SocketName);

		// Restore dynamic delegate bindings.
		UBlueprintGeneratedClass::BindDynamicDelegates(Actor->GetClass(), Actor);
	}

	return false;
}

void FActorElementEditorPasteImporter::Import(FContext& InContext)
{
	Context = InContext;
	FStringView InText = Context.Text;

	if (!Context.World || InText.IsEmpty())
	{
		return;
	}

	// Parse text of actors
	const TCHAR* TextStart = InText.GetData();
	const TCHAR** CurrentPos = &TextStart;
	const TCHAR* EndPos = InText.GetData() + InText.Len();
	FStringView Line;

	FImportedActor* CurrentImportedActor = nullptr;

	const TCHAR* StartOfCurrentActorInner = nullptr;
	const TCHAR* EndOfPreviousLine = *CurrentPos;

	while (*CurrentPos < EndPos && FParse::Line(CurrentPos, Line))
	{
		const TCHAR* LineStart = Line.GetData();
		const TCHAR** LinePos = &LineStart;
		if (Line.GetData() + Line.Len() < EndPos)
		{
			if (CurrentImportedActor)
			{
				if (!StartOfCurrentActorInner)
				{
					StartOfCurrentActorInner = Line.GetData();
				}

				if (GetEND(LinePos, TEXT("Actor")))
				{
					CurrentImportedActor->InnerText = FStringView(StartOfCurrentActorInner,
					                                              uint32(EndOfPreviousLine - StartOfCurrentActorInner));
					CurrentImportedActor = nullptr;
					StartOfCurrentActorInner = nullptr;
				}
			}
			else if (GetBEGIN(LinePos, TEXT("Actor")))
			{
				UClass* ActorClass = nullptr;
				if (ParseObject<UClass>(*LinePos, TEXT("CLASS="), ActorClass, nullptr))
				{
					CurrentImportedActor = &(ImportedActors.AddDefaulted_GetRef());
					CurrentImportedActor->Class = ActorClass;
					FParse::Value(*LinePos, TEXT("Name="), CurrentImportedActor->Name);
					FParse::Value(*LinePos, TEXT("Archetype="), CurrentImportedActor->Archetype);
					FParse::Value(*LinePos, TEXT("ParentActor="), CurrentImportedActor->ParentActor);
					FParse::Value(*LinePos, TEXT("SocketName="), CurrentImportedActor->SocketNameString);
					FParse::Value(*LinePos, TEXT("GroupActor="), CurrentImportedActor->GroupActor);
					FParse::Value(*LinePos, TEXT("GroupFolder="), CurrentImportedActor->GroupFolder);
					FParse::Value(*LinePos, TEXT("ActorFolderPath="), CurrentImportedActor->ActorFolderPath);
					FParse::Value(*LinePos, TEXT("ExportPath="), CurrentImportedActor->ExportedActorFullName);
				}
			}
		}

		EndOfPreviousLine = Line.GetData() + Line.Len();
	}

	if (CurrentImportedActor)
	{
		ImportedActors.Pop();
	}

	bPostImportProcessingNeeded = true;
}

TArray<FTypedElementHandle> FActorElementEditorPasteImporter::GetImportedElements()
{
	if (bPostImportProcessingNeeded)
	{
		PostImportProcess();
	}
	TArray<FTypedElementHandle> Handles;
	Handles.Reserve(ImportedActors.Num());

	for (FImportedActor& ImportedActor : ImportedActors)
	{
		AActor* Actor = ImportedActor.ActorPtr;
		if (IsValid(Actor))
		{
			if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
			{
				Handles.Add(MoveTemp(Handle));
			}
		}
	}

	for (TPair<FString, AGroupActor*>& GroupPair : NewGroups)
	{
		if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(GroupPair.Value))
		{
			Handles.Add(MoveTemp(Handle));
		}
	}

	return Handles;
}

void FActorElementEditorPasteImporter::PostImportProcess()
{
	TMap<FSoftObjectPath, UObject*> InExportedPathToObjectCreated;
	// CreateObjects
	{
		UWorld* World = Context.World;
		for (FImportedActor& ImportedActor : ImportedActors)
		{
			AActor* Found = nullptr;
			if (ImportedActor.Name != NAME_None)
			{
				// look in the current level for the same named actor
				Found = FindObjectFast<AActor>(World->GetCurrentLevel(), ImportedActor.Name);
			}

			// Make sure this name is unique. We need to do this upfront because we also want to potentially create the Associated BP class using the same name.
			bool bNeedGloballyUniqueName = (World->GetCurrentLevel()->IsUsingExternalActors() && CastChecked<AActor>(
				ImportedActor.Class->GetDefaultObject())->SupportsExternalPackaging()) || Found;
			FName ActorUniqueName = FActorSpawnUtils::MakeUniqueActorName(
				World->GetCurrentLevel(), ImportedActor.Class, FActorSpawnUtils::GetBaseName(ImportedActor.Name),
				bNeedGloballyUniqueName);


			AActor* Archetype = nullptr;
			{
				FString ObjectClass;
				FString ObjectPath;

				if (FPackageName::ParseExportTextPath(ImportedActor.Archetype, &ObjectClass, &ObjectPath))
				{
					// find the class
					UClass* ArchetypeClass = UClass::TryFindTypeSlow<UClass>(
						ObjectClass, EFindFirstObjectOptions::EnsureIfAmbiguous);
					if (ArchetypeClass)
					{
						if (ArchetypeClass->IsChildOf(AActor::StaticClass()))
						{
							// if we had the class, find the archetype
							Archetype = Cast<AActor>(StaticFindObject(ArchetypeClass, nullptr, *ObjectPath));
						}
						else
						{
							if (GWarn)
							{
								GWarn->Logf(ELogVerbosity::Warning,
								            TEXT(
									            "Invalid archetype specified in subobject definition '%s': %s is not a child of Actor"),
								            *(ImportedActor.Name.ToString()), *ObjectClass);
							}
						}
					}
				}
			}

			// Might not be needed need to validate if we can copy the level blueprint via the selection code
			if (FBlueprintEditorUtils::IsAnonymousBlueprintClass(ImportedActor.Class))
			{
				UBlueprint* NewBP = DuplicateObject(CastChecked<UBlueprint>(ImportedActor.Class->ClassGeneratedBy),
				                                    World->GetCurrentLevel(),
				                                    *FString::Printf(TEXT("%s_BPClass"), *ActorUniqueName.ToString()));
				if (NewBP)
				{
					NewBP->ClearFlags(RF_Standalone);

					FKismetEditorUtilities::CompileBlueprint(NewBP, EBlueprintCompileOptions::SkipGarbageCollection);

					ImportedActor.Class = NewBP->GeneratedClass;

					// Since we changed the class we can't use an Archetype,
					// however that is fine since we will have been editing the CDO anyways
					Archetype = nullptr;
				}
			}

			if (FLevelUtils::IsLevelLocked(World->GetCurrentLevel()))
			{
				if (GWarn)
				{
					// Todo 
					GWarn->Log(ELogVerbosity::Warning,
					           TEXT(
						           "Import actor: The requested operation could not be completed because the level is locked."));
				}
				return;
			}
			
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.Name = ActorUniqueName;
			SpawnInfo.Template = Archetype;
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			ImportedActor.ActorPtr = World->SpawnActor(ImportedActor.Class, nullptr, nullptr, SpawnInfo);

			// Todo investigate the support of moving some actor(or elements) to another level using elements instead of the old level Factory/Exporter code?
			const bool bIsMoveToStreamingLevel = false;

			if (ImportedActor.ActorPtr->ShouldImport(ImportedActor.InnerText, bIsMoveToStreamingLevel))
			{
				if (!ImportedActor.ExportedActorFullName.IsEmpty())
				{
					InExportedPathToObjectCreated.Add(ImportedActor.ExportedActorFullName, ImportedActor.ActorPtr);
				}

				EditorUtilities::FMultiStepsImportObjectParams Params;
				Params.DestData = static_cast<uint8*>(static_cast<void*>(ImportedActor.ActorPtr));
				Params.SourceText = ImportedActor.InnerText;
				Params.ObjectStruct = ImportedActor.ActorPtr->GetClass();
				Params.SubobjectRoot = ImportedActor.ActorPtr;
				Params.SubobjectOuter = ImportedActor.ActorPtr;
				Params.Warn = GWarn;
				Params.ObjectRemapper = &InExportedPathToObjectCreated;

				if (!EditorUtilities::ImportCreateObjectsStep(Params))
				{
					ImportedActor.ActorPtr->Destroy();
					ImportedActor.ActorPtr = nullptr;
				}
			}
			else
			{
				ImportedActor.ActorPtr->Destroy();
				ImportedActor.ActorPtr = nullptr;
			}
		}
	}
	// SerializeObjects
	{
		for (FImportedActor& ImportedActor : ImportedActors)
		{
			if (!ImportedActor.ActorPtr)
			{
				continue;
			}
			{
				TArray<UObject*> Objects;
				GetObjectsWithOuter(ImportedActor.ActorPtr, Objects);

				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						Object->PreEditChange(nullptr);
					}
				}
			}

			ImportedActor.ActorPtr->PreEditChange(nullptr);

			EditorUtilities::FMultiStepsImportObjectParams Params;
			Params.DestData = static_cast<uint8*>(static_cast<void*>(ImportedActor.ActorPtr));
			Params.SourceText = ImportedActor.InnerText;
			Params.ObjectStruct = ImportedActor.ActorPtr->GetClass();
			Params.SubobjectRoot = ImportedActor.ActorPtr;
			Params.SubobjectOuter = ImportedActor.ActorPtr;
			Params.Warn = GWarn;
			// This won't change the map
			Params.ObjectRemapper = (&InExportedPathToObjectCreated);

			if (!EditorUtilities::ImportObjectsPropertiesStep(Params))
			{
				ImportedActor.ActorPtr->Destroy();
				ImportedActor.ActorPtr = nullptr;
				continue;
			}

			if (!ImportedActor.ActorFolderPath.IsEmpty())
			{
				ImportedActor.ActorPtr->SetFolderPath(*ImportedActor.ActorFolderPath);
			}

			if (!ImportedActor.GroupActor.IsEmpty())
			{
				if (AGroupActor** ActorGroup = NewGroups.Find(ImportedActor.GroupActor))
				{
					(*ActorGroup)->Add(*ImportedActor.ActorPtr);
				}
				else
				{
					AGroupActor* SpawnedGroupActor = ImportedActor.ActorPtr->GetWorld()->SpawnActor<AGroupActor>();

					if (!ImportedActor.GroupFolder.IsEmpty())
					{
						SpawnedGroupActor->SetFolderPath(*ImportedActor.GroupFolder);
					}

					SpawnedGroupActor->Add(*ImportedActor.ActorPtr);
					NewGroups.Add(ImportedActor.GroupActor, SpawnedGroupActor);
					FActorLabelUtilities::SetActorLabelUnique(SpawnedGroupActor, ImportedActor.GroupActor);
				}
			}
			NameToImportedActors.Add(ImportedActor.Name, ImportedActor.ActorPtr);
		}
	}

	// NotifyCreatedObjects
	{
		for (FImportedActor& ImportedActor : ImportedActors)
		{
			AActor* Actor = ImportedActor.ActorPtr;
			if (IsValid(Actor))
			{
				TArray<UObject*> Objects;
				GetObjectsWithOuter(Actor, Objects);

				for (UObject* Object : Objects)
				{
					if (IsValid(Object))
					{
						Object->PostEditImport();
						Object->PostEditChange();
					}
				}

				// Let the actor deal with having been imported, if desired.
				Actor->PostEditImport();

				// Notify actor its properties have changed.
				Actor->PostEditChange();
			}
		}

		// Do the attachment of actors 
		for (FImportedActor& ImportedActor : ImportedActors)
		{
			if (ImportedActor.ActorPtr)
			{
				// Try to find the parent in our imported actors
				if (AActor** Parent = NameToImportedActors.Find(ImportedActor.ParentActor))
				{
					GEditor->ParentActors(*Parent, ImportedActor.ActorPtr, ImportedActor.SocketNameString);
				}
				// Otherwise try to find the parent in the world
				else if(AActor* ParentActor = FindObject<AActor>( ImportedActor.ActorPtr->GetWorld()->GetCurrentLevel(), *ImportedActor.ParentActor.ToString() ))
				{
					GEditor->ParentActors(ParentActor, ImportedActor.ActorPtr, ImportedActor.SocketNameString);
				}
			}
		}

		for (TPair<FString, AGroupActor*>& GroupPair : NewGroups)
		{
			GroupPair.Value->CenterGroupLocation();
			GroupPair.Value->Lock();
		}

		NameToImportedActors.Empty();
		NewGroups.Empty();
	}
}

#undef LOCTEXT_NAMESPACE
