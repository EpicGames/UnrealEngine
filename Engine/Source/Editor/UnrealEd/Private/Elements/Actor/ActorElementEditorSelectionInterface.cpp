// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Elements/Actor/ActorElementEditorSelectionInterface.h"

#include "DataStorage/Features.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Object/ObjectElementEditorSelectionInterface.h"
#include "GameFramework/Actor.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementEditorSelectionInterface)
 
class FActorElementTransactedElement : public ITypedElementTransactedElement
{
private:
	virtual TUniquePtr<ITypedElementTransactedElement> CloneImpl() const override
	{
		return MakeUnique<FActorElementTransactedElement>(*this);
	}
 
	virtual FTypedElementHandle GetElementImpl() const override
	{
		const AActor* Actor = ActorPtr.Get(/*bEvenIfPendingKill*/true);
		return Actor
			? UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor)
			: FTypedElementHandle();
	}
 
	virtual void SetElementImpl(const FTypedElementHandle& InElementHandle) override
	{
		AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
		if (!Actor)
		{
			UE_LOGF(LogTemp, Warning, "FActorElementTransactedElement::SetElementImpl: Failed to resolve actor from element handle");
		}
		ActorPtr = Actor;
	}
 
	virtual void SerializeImpl(FArchive& InArchive) override
	{
		InArchive << ActorPtr;
	}
 
	TWeakObjectPtr<AActor> ActorPtr;
};
 
bool UActorElementEditorSelectionInterface::SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace UE::Editor::DataStorage;

	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	// Add a selection column in TEDS
	if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		RowHandle Row = Compatibility->FindRowWithCompatibleObject(Actor);
		if (Row != InvalidRowHandle)
		{
			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				DataStorage->AddColumn(Row, FTypedElementSelectionColumn{ .SelectionSet = InSelectionOptions.GetNameForTEDSIntegration() });
			}
		}
	}
	
	return Super::SelectElement(InElementHandle, InSelectionSet, InSelectionOptions);
}
 
bool UActorElementEditorSelectionInterface::DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace UE::Editor::DataStorage;

	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		RowHandle Row = Compatibility->FindRowWithCompatibleObject(Actor);
		if (Row != InvalidRowHandle)
		{
			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
            {
            	DataStorage->RemoveColumn(Row, FTypedElementSelectionColumn::StaticStruct());
            }
		}
	}
	
	return Super::DeselectElement(InElementHandle, InSelectionSet, InSelectionOptions);
}
 
bool UActorElementEditorSelectionInterface::IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& SelectionSetPtr, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return SelectionSetPtr && Actor && IsActorSelected(Actor, SelectionSetPtr.ToSharedRef(), InSelectionOptions);
}
 
bool UActorElementEditorSelectionInterface::ShouldPreventTransactions(const FTypedElementHandle& InElementHandle)
{
	const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor && UObjectElementEditorSelectionInterface::ShouldObjectPreventTransactions(Actor);
}
 
TUniquePtr<ITypedElementTransactedElement> UActorElementEditorSelectionInterface::CreateTransactedElementImpl()
{
	return MakeUnique<FActorElementTransactedElement>();
}
 
bool UActorElementEditorSelectionInterface::IsActorSelected(const AActor* InActor, FTypedElementListConstRef InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	if (InSelectionSet->Num() == 0)
	{
		return false;
	}
 
	if (FTypedElementHandle ActorElement = UEngineElementsLibrary::AcquireEditorActorElementHandle(InActor, /*bAllowCreate*/false))
	{
		if (InSelectionSet->Contains(ActorElement))
		{
			return true;
		}
	}
 
	if (InSelectionOptions.AllowIndirect())
	{
		if (const AActor* RootSelectionActor = InActor->GetRootSelectionParent())
		{
			if (FTypedElementHandle RootSelectionElement = UEngineElementsLibrary::AcquireEditorActorElementHandle(RootSelectionActor, /*bAllowCreate*/false))
			{
				return InSelectionSet->Contains(RootSelectionElement);
			}
		}
	}
 
	return false;
}
