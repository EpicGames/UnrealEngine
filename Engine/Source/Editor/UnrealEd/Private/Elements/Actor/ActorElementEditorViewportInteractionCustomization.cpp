// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "Elements/Actor/ActorElementData.h"
#include "GameFramework/Actor.h"

#include "Editor.h"
#include "EditorModeManager.h"
#include "AI/NavigationSystemBase.h"
#include "Components/BrushComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "Engine/DocumentationActor.h"
#include "Particles/Emitter.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "Toolkits/IToolkitHost.h"

void FActorElementEditorViewportInteractionCustomization::GizmoManipulationStarted(
	const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle,
	const UE::Widget::EWidgetMode InWidgetMode)
{
	AActor* RootActor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!RootActor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementEditorViewportInteractionCustomization::GizmoManipulationStarted: Failed to resolve actor from element handle");
		return;
	}

	FTypedElementViewportInteractionCustomization::GizmoManipulationStarted(InElementWorldHandle, InWidgetMode);

	TArray<AActor*> AffectedActors;
	AffectedActors.Add(RootActor);
	
	// grab all attached actors to invalidate everybody 	
	RootActor->GetAttachedActors(AffectedActors, /*bResetArray*/ false, /*bRecursivelyIncludeAttachedActors*/ true );

	if (GEditor->IsDeltaModificationEnabled())
	{
		for (AActor* Actor : AffectedActors)
		{
			Actor->Modify();
		}
	}

	if (!GIsDemoMode)
	{
		for (AActor* Actor : AffectedActors)
		{
			// We don't know at this point if it is translation only
			constexpr bool bTranslationOnly = false;
			Actor->InvalidateLightingCacheDetailed(bTranslationOnly);
		}
	}
}

void FActorElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation)
{
	AActor* RootActor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!RootActor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate: Failed to resolve actor from element handle");
		return;
	}

	TArray<AActor*> AffectedActors;
	AffectedActors.Add(RootActor);
	
	// grab all attached actors to invalidate everybody 	
	RootActor->GetAttachedActors(AffectedActors, /*bResetArray*/ false, /*bRecursivelyIncludeAttachedActors*/ true );

	const FVector DeltaTranslation = InDeltaTransform.GetTranslation();
	const FRotator DeltaRotation = InDeltaTransform.Rotator();
	const FVector DeltaScale3D = InDeltaTransform.GetScale3D();
	
	const bool bIsSimulatingInEditor = GEditor->IsSimulatingInEditor();

	FNavigationLockContext LockNavigationUpdates(RootActor->GetWorld(), ENavigationLockReason::ContinuousEditorMove);

	///////////////////
	// Rotation
	
	const bool bRotatingActor = !DeltaRotation.IsZero();
	if (bRotatingActor)
	{			
		if (USceneComponent* RootComponent = RootActor->GetRootComponent())
		{
			const FRotator OriginalRotation = RootActor->GetRootComponent()->GetComponentRotation();

			auto ShouldApplyDeltaDirectly = [ToolkitHostPtr = GetToolkitHost()]()
			{
				if (ToolkitHostPtr)
				{
					const FEditorModeTools& ModeManager = ToolkitHostPtr->GetEditorModeManager();
					return ModeManager.GetRotationContext().bUseExplicitRotator;
				}
				return false;
			};

			if (ShouldApplyDeltaDirectly())
			{
				const FRotator RelativeRot = RootComponent->GetRelativeRotation();
				RootComponent->SetRelativeRotationExact(RelativeRot + DeltaRotation);
			}
			else
			{
				RootActor->EditorApplyRotation(DeltaRotation, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);
			}

			// Check to see if we should transform the rigid body
			UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(RootComponent);
			if (bIsSimulatingInEditor && GIsPlayInEditorWorld && RootPrimitiveComponent)
			{
				FRotator ActorRotWind, ActorRotRem;
				OriginalRotation.GetWindingAndRemainder(ActorRotWind, ActorRotRem);

				const FQuat ActorQ = ActorRotRem.Quaternion();
				const FQuat DeltaQ = DeltaRotation.Quaternion();
				const FQuat ResultQ = DeltaQ * ActorQ;

				const FRotator NewActorRotRem = FRotator(ResultQ);
				FRotator DeltaRot = NewActorRotRem - ActorRotRem;
				DeltaRot.Normalize();

				// @todo SIE: Not taking into account possible offset between root component and actor
				RootPrimitiveComponent->SetWorldRotation(OriginalRotation + DeltaRot);
			}
		}

		FVector NewActorLocation = RootActor->GetActorLocation();
		NewActorLocation -= InPivotLocation;
		NewActorLocation = FRotationMatrix(DeltaRotation).TransformPosition(NewActorLocation);
		NewActorLocation += InPivotLocation;
		NewActorLocation -= RootActor->GetActorLocation();
		RootActor->EditorApplyTranslation(NewActorLocation, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);
	}

	///////////////////
	// Translation
	if (RootActor->GetRootComponent())
	{
		const FVector OriginalLocation = RootActor->GetRootComponent()->GetComponentLocation();

		RootActor->EditorApplyTranslation(DeltaTranslation, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);

		// Check to see if we should transform the rigid body
		UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(RootActor->GetRootComponent());
		if (bIsSimulatingInEditor && GIsPlayInEditorWorld && RootPrimitiveComponent)
		{
			// @todo SIE: Not taking into account possible offset between root component and actor
			RootPrimitiveComponent->SetWorldLocation(OriginalLocation + DeltaTranslation);
		}
	}

	///////////////////
	// Scaling
	const bool bScalingActor = !DeltaScale3D.IsNearlyZero(0.000001f);
	if (bScalingActor)
	{
		FVector ModifiedScale = DeltaScale3D;

		// Note: With the new additive scaling method, this is handled in FLevelEditorViewportClient::ModifyScale
		if (GEditor->UsePercentageBasedScaling())
		{
			// Get actor box extents
			const FBox BoundingBox = RootActor->GetComponentsBoundingBox(true);
			const FVector BoundsExtents = BoundingBox.GetExtent();

			// Make sure scale on actors is clamped to a minimum and maximum size.
			const float MinThreshold = 1.0f;

			for (int32 Idx = 0; Idx < 3; Idx++)
			{
				if ((FMath::Pow(BoundsExtents[Idx], 2)) > BIG_NUMBER)
				{
					ModifiedScale[Idx] = 0.0f;
				}
				else if (SMALL_NUMBER < BoundsExtents[Idx])
				{
					const bool bBelowAllowableScaleThreshold = ((DeltaScale3D[Idx] + 1.0f) * BoundsExtents[Idx]) < MinThreshold;

					if (bBelowAllowableScaleThreshold)
					{
						ModifiedScale[Idx] = (MinThreshold / BoundsExtents[Idx]) - 1.0f;
					}
				}
			}
		}
		
		// Flag actors to use old-style scaling or not
		// @todo: Remove this hack once we have decided on the scaling method to use.
		AActor::bUsePercentageBasedScaling = GEditor->UsePercentageBasedScaling();

		RootActor->EditorApplyScale(
			ModifiedScale,
			&InPivotLocation,
			InInputState.bAltKeyDown,
			InInputState.bShiftKeyDown,
			InInputState.bCtrlKeyDown
			);
	}

	for (AActor* Actor : AffectedActors)
	{
		Actor->PostEditMove(false);
	}
}

void FActorElementEditorViewportInteractionCustomization::GizmoManipulationStopped(
	const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle,
	const UE::Widget::EWidgetMode InWidgetMode,
	const ETypedElementViewportInteractionGizmoManipulationType InManipulationType)
{
	FTypedElementViewportInteractionCustomization::GizmoManipulationStopped(
		InElementWorldHandle, InWidgetMode, InManipulationType);

	AActor* RootActor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!RootActor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementEditorViewportInteractionCustomization::GizmoManipulationStopped: Failed to resolve actor from element handle");
		return;
	}

	TArray<AActor*> AffectedActors;
	AffectedActors.Add(RootActor);
	
	// grab all attached actors to invalidate everybody 	
	RootActor->GetAttachedActors(AffectedActors, /*bResetArray*/ false, /*bRecursivelyIncludeAttachedActors*/ true );

	InElementWorldHandle.NotifyMovementEnded();
	// Update the actor before leaving.
	for (AActor* Actor : AffectedActors)
	{
		Actor->MarkPackageDirty();
	}
}

void FActorElementEditorViewportInteractionCustomization::MirrorElement(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const FVector& InMirrorScale, const FVector& InPivotLocation)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!Actor)
	{
		UE_LOGF(LogTemp, Warning, "FActorElementEditorViewportInteractionCustomization::MirrorElement: Failed to resolve actor from element handle");
		return;
	}

	Actor->Modify();
	Actor->EditorApplyMirror(InMirrorScale, InPivotLocation);

	if (ABrush* Brush = Cast<ABrush>(Actor))
	{
		if (UBrushComponent* BrushComponent = Brush->GetBrushComponent())
		{
			BrushComponent->RequestUpdateBrushCollision();
		}
	}

	Actor->InvalidateLightingCache();
	Actor->PostEditMove(true);

	Actor->MarkPackageDirty();
}

bool FActorElementEditorViewportInteractionCustomization::GetFocusBounds( const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementWorldHandle);
	if (!Actor)
	{
		return false;
	}

	// open the documentation of documentation actors
	if (ADocumentationActor* DocActor = Cast<ADocumentationActor>(Actor))
	{
		DocActor->OpenDocumentLink();
	}

	// Create a bounding volume of all of the sub-elements
	FBox BoundingBox(ForceInit);

	const bool bActorIsEmitter = (Cast<AEmitter>(Actor) != nullptr);

	if (bActorIsEmitter && GEditor && GEditor->bCustomCameraAlignEmitter)
	{
		const float CustomCameraAlignEmitterDistance = GEditor->CustomCameraAlignEmitterDistance;
		const FVector DefaultExtent(CustomCameraAlignEmitterDistance, CustomCameraAlignEmitterDistance,
		                            CustomCameraAlignEmitterDistance);
		const FBox DefaultSizeBox(Actor->GetActorLocation() - DefaultExtent, Actor->GetActorLocation() + DefaultExtent);
		BoundingBox += DefaultSizeBox;
	}
	else if (USceneComponent* RootComponent = Actor->GetRootComponent())
	{
		TSet<AActor*> Actors;
		Actor->EditorGetUnderlyingActors(Actors);
		Actors.Add(Actor);
		TSet<USceneComponent*> AllSceneComponents;

		for (AActor* CurrentActor : Actors)
		{
			if (USceneComponent* CurrentRootComponent = CurrentActor->GetRootComponent())
			{
				bool bIsAlreadyInSet = false;
				AllSceneComponents.Add(CurrentRootComponent, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					TArray<USceneComponent*> SceneComponents;
					CurrentRootComponent->GetChildrenComponents(true, SceneComponents);
					AllSceneComponents.Append(SceneComponents);
				}
			}
		}

		bool bHasAtLeastOnePrimitiveComponent = false;
		for (USceneComponent* SceneComponent : AllSceneComponents)
		{
			UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent);

			if (PrimitiveComponent && PrimitiveComponent->IsRegistered())
			{
				// Some components can have huge bounds but are not visible.  Ignore these components unless it is the only component on the actor 
				const bool bIgnore = AllSceneComponents.Num() > 1 && PrimitiveComponent->GetIgnoreBoundsForEditorFocus();

				if (!bIgnore)
				{
					// For components with an active deformer graph the true world-space vertex positions live in a GPU buffer
					// and are not yet reflected in PrimitiveComponent::Bounds. Their real bounds have to be computed via an async GPU readback.
					// We still mark bHasAtLeastOnePrimitiveComponent so the root-location fallback below does not fire for actors that are all-deformer.
#if WITH_EDITORONLY_DATA
					const USkinnedMeshComponent* SkinnedMeshComponent =  Cast<USkinnedMeshComponent>(PrimitiveComponent);
					if (SkinnedMeshComponent && SkinnedMeshComponent->HasMeshDeformer())
					{
						bHasAtLeastOnePrimitiveComponent = true;
						continue;
					}
#endif
					FBox LocalBox(ForceInit);
					if (GLevelEditorModeTools().ComputeBoundingBoxForViewportFocus(Actor, PrimitiveComponent, LocalBox))
					{
						BoundingBox += LocalBox;
					}
					else
					{
						BoundingBox += PrimitiveComponent->Bounds.GetBox();
					}

					bHasAtLeastOnePrimitiveComponent = true;
				}
			}
		}

		if (!bHasAtLeastOnePrimitiveComponent)
		{
			BoundingBox += RootComponent->GetComponentLocation();
		}

	}
	
	OutBounds = BoundingBox;
	return static_cast<bool>(BoundingBox.IsValid);
}

void FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(AActor* InActor, const bool InIsDelta, const FVector* InDeltaTranslationPtr, const FRotator* InDeltaRotationPtr, const FVector* InDeltaScalePtr, const FVector& InPivotLocation, const FInputDeviceState& InInputState)
{
	const bool bIsSimulatingInEditor = GEditor->IsSimulatingInEditor();

	TArray<AActor*> AffectedActors;
	AffectedActors.Add(InActor);
	
	// grab all attached actors to invalidate everybody 	
	InActor->GetAttachedActors(AffectedActors, /*bResetArray*/ false, /*bRecursivelyIncludeAttachedActors*/ true );

	if (GEditor->IsDeltaModificationEnabled())
	{
		for (AActor* Actor : AffectedActors)
		{
			Actor->Modify();
		}
	}

	FNavigationLockContext LockNavigationUpdates(InActor->GetWorld(), ENavigationLockReason::ContinuousEditorMove);

	bool bTranslationOnly = true;

	///////////////////
	// Rotation

	// Unfortunately this can't be moved into ABrush::EditorApplyRotation, as that would
	// create a dependence in Engine on Editor.
	if (InDeltaRotationPtr)
	{
		const FRotator& InDeltaRot = *InDeltaRotationPtr;
		const bool bRotatingActor = !InIsDelta || !InDeltaRot.IsZero();
		if (bRotatingActor)
		{
			bTranslationOnly = false;

			if (InIsDelta)
			{
				if (InActor->GetRootComponent())
				{
					const FRotator OriginalRotation = InActor->GetRootComponent()->GetComponentRotation();

					InActor->EditorApplyRotation(InDeltaRot, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);

					// Check to see if we should transform the rigid body
					UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(InActor->GetRootComponent());
					if (bIsSimulatingInEditor && GIsPlayInEditorWorld && RootPrimitiveComponent)
					{
						FRotator ActorRotWind, ActorRotRem;
						OriginalRotation.GetWindingAndRemainder(ActorRotWind, ActorRotRem);

						const FQuat ActorQ = ActorRotRem.Quaternion();
						const FQuat DeltaQ = InDeltaRot.Quaternion();
						const FQuat ResultQ = DeltaQ * ActorQ;

						const FRotator NewActorRotRem = FRotator(ResultQ);
						FRotator DeltaRot = NewActorRotRem - ActorRotRem;
						DeltaRot.Normalize();

						// @todo SIE: Not taking into account possible offset between root component and actor
						RootPrimitiveComponent->SetWorldRotation(OriginalRotation + DeltaRot);
					}
				}

				FVector NewActorLocation = InActor->GetActorLocation();
				NewActorLocation -= InPivotLocation;
				NewActorLocation = FRotationMatrix(InDeltaRot).TransformPosition(NewActorLocation);
				NewActorLocation += InPivotLocation;
				NewActorLocation -= InActor->GetActorLocation();
				InActor->EditorApplyTranslation(NewActorLocation, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);
			}
			else
			{
				InActor->SetActorRotation(InDeltaRot);
			}
		}
	}

	///////////////////
	// Translation
	if (InDeltaTranslationPtr)
	{
		if (InIsDelta)
		{
			if (InActor->GetRootComponent())
			{
				const FVector OriginalLocation = InActor->GetRootComponent()->GetComponentLocation();

				InActor->EditorApplyTranslation(*InDeltaTranslationPtr, InInputState.bAltKeyDown, InInputState.bShiftKeyDown, InInputState.bCtrlKeyDown);

				// Check to see if we should transform the rigid body
				UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(InActor->GetRootComponent());
				if (bIsSimulatingInEditor && GIsPlayInEditorWorld && RootPrimitiveComponent)
				{
					// @todo SIE: Not taking into account possible offset between root component and actor
					RootPrimitiveComponent->SetWorldLocation(OriginalLocation + *InDeltaTranslationPtr);
				}
			}
		}
		else
		{
			InActor->SetActorLocation(*InDeltaTranslationPtr, false);
		}
	}

	///////////////////
	// Scaling
	if (InDeltaScalePtr)
	{
		const FVector& InDeltaScale = *InDeltaScalePtr;
		const bool bScalingActor = !InIsDelta || !InDeltaScale.IsNearlyZero(0.000001f);
		if (bScalingActor)
		{
			bTranslationOnly = false;

			FVector ModifiedScale = InDeltaScale;

			// Note: With the new additive scaling method, this is handled in FLevelEditorViewportClient::ModifyScale
			if (GEditor->UsePercentageBasedScaling())
			{
				// Get actor box extents
				const FBox BoundingBox = InActor->GetComponentsBoundingBox(true);
				const FVector BoundsExtents = BoundingBox.GetExtent();

				// Make sure scale on actors is clamped to a minimum and maximum size.
				const float MinThreshold = 1.0f;

				for (int32 Idx = 0; Idx < 3; Idx++)
				{
					if ((FMath::Pow(BoundsExtents[Idx], 2)) > BIG_NUMBER)
					{
						ModifiedScale[Idx] = 0.0f;
					}
					else if (SMALL_NUMBER < BoundsExtents[Idx])
					{
						const bool bBelowAllowableScaleThreshold = ((InDeltaScale[Idx] + 1.0f) * BoundsExtents[Idx]) < MinThreshold;

						if (bBelowAllowableScaleThreshold)
						{
							ModifiedScale[Idx] = (MinThreshold / BoundsExtents[Idx]) - 1.0f;
						}
					}
				}
			}

			if (InIsDelta)
			{
				// Flag actors to use old-style scaling or not
				// @todo: Remove this hack once we have decided on the scaling method to use.
				AActor::bUsePercentageBasedScaling = GEditor->UsePercentageBasedScaling();

				InActor->EditorApplyScale(
					ModifiedScale,
					&InPivotLocation,
					InInputState.bAltKeyDown,
					InInputState.bShiftKeyDown,
					InInputState.bCtrlKeyDown
					);

			}
			else if (InActor->GetRootComponent())
			{
				InActor->GetRootComponent()->SetRelativeScale3D(InDeltaScale);
			}
		}
	}

	// Update the actor before leaving.
	for (AActor* Actor : AffectedActors)
	{
		Actor->MarkPackageDirty();
	}
	
	if (!GIsDemoMode)
	{
		for (AActor* Actor : AffectedActors)
		{
			Actor->InvalidateLightingCacheDetailed(bTranslationOnly);
		}
	}
	
	for (AActor* Actor : AffectedActors)
	{
		Actor->PostEditMove(false);
	}
}