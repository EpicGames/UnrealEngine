// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Actor/ActorElementEditorWorldInterface.h"

#include "Elements/Actor/ActorElementData.h"
#include "Elements/Actor/ActorElementEditorCopyAndPaste.h"
#include "GameFramework/Actor.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"

#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorElementEditorWorldInterface)

namespace UE::ActorElementEditorWorldInterface::Local
{
	bool bEnableTemporaryActorCopy = false;
	static FAutoConsoleVariableRef CVarEnableTemporaryActorCopy(TEXT("TypedElements.EnableTemporaryActorCopy"), bEnableTemporaryActorCopy,
		TEXT("Enable cut/copy/duplicate for temporary actors created during PIE. Experimental and can cause issues."));
}

bool UActorElementEditorWorldInterface::GetPivotOffset(const FTypedElementHandle& InElementHandle, FVector& OutPivotOffset)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		OutPivotOffset = Actor->GetPivotOffset();
		return true;
	}

	return false;
}

bool UActorElementEditorWorldInterface::SetPivotOffset(const FTypedElementHandle& InElementHandle, const FVector& InPivotOffset)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		Actor->Modify();
		Actor->SetPivotOffset(InPivotOffset);
		Actor->PostEditMove(true);
		return true;
	}

	return false;
}

void UActorElementEditorWorldInterface::NotifyMovementStarted(const FTypedElementHandle& InElementHandle)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		GEditor->BroadcastBeginObjectMovement(*Actor);
	}
}

void UActorElementEditorWorldInterface::NotifyMovementOngoing(const FTypedElementHandle& InElementHandle)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		Actor->PostEditMove(false);
	}
}

void UActorElementEditorWorldInterface::NotifyMovementEnded(const FTypedElementHandle& InElementHandle)
{
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{
		GEditor->BroadcastEndObjectMovement(*Actor);
		Actor->PostEditMove(true);

		Actor->InvalidateLightingCache();
		Actor->UpdateComponentTransforms();
		Actor->MarkPackageDirty();
	}
}

bool UActorElementEditorWorldInterface::CanDeleteElement(const FTypedElementHandle& InElementHandle)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
	return Actor && (GUnrealEd->CanDeleteActor(Actor) || Actor->bIsEditorPreviewActor);
}

bool UActorElementEditorWorldInterface::DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	const TArray<AActor*> ActorsToDelete = ActorElementDataUtil::GetActorsFromHandles(InElementHandles);
	return ActorsToDelete.Num() > 0
		&& GUnrealEd->DeleteActors(ActorsToDelete, InWorld, InSelectionSet, InDeletionOptions.VerifyDeletionCanHappen(), InDeletionOptions.WarnAboutReferences(), InDeletionOptions.WarnAboutSoftReferences());
}

bool UActorElementEditorWorldInterface::CanDuplicateElement(const FTypedElementHandle& InElementHandle)
{
	AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);

	if (!UE::ActorElementEditorWorldInterface::Local::bEnableTemporaryActorCopy)
	{
		if (Actor && Actor->GetLevel())
		{
			if (UWorld* ActorWorld = Actor->GetLevel()->GetWorld())
			{
				// If the actor is in a PIE world but doesn't have an editor counterpart it means it's a temporary
				// actor spawned to the world. These actors can cause issues when copied so have been disabled.
				if (ActorWorld->WorldType == EWorldType::PIE && !GEditor->ObjectsThatExistInEditorWorld.Get(Actor))
				{
					return false;
				}
			}
		}
	}
	return Actor != nullptr;
}

void UActorElementEditorWorldInterface::DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	const TArray<AActor*> ActorsToDuplicate = ActorElementDataUtil::GetActorsFromHandles(InElementHandles);

	if (ActorsToDuplicate.Num() > 0)
	{
		FEditorDelegates::OnDuplicateActorsBegin.Broadcast();

		TArray<AActor*> NewActors;
		ABrush::SetSuppressBSPRegeneration(true);
		ULevel* Level = InWorld->GetCurrentLevel();
		TGuardValue DisablePrompt(Level->bPromptWhenAddingToLevelOutsideBounds, false);
		GUnrealEd->DuplicateActors(ActorsToDuplicate, NewActors, Level, InLocationOffset);
		ABrush::SetSuppressBSPRegeneration(false);

		FEditorDelegates::OnDuplicateActorsEnd.Broadcast();

		bool bRebuildBSP = false;
		OutNewElements.Reserve(OutNewElements.Num() + NewActors.Num());
		for (AActor* NewActor : NewActors)
		{
			// Only rebuild if the new actors will change the BSP as this is expensive
			if (!bRebuildBSP)
			{
				if (ABrush* Brush = Cast<ABrush>(NewActor))
				{
					bRebuildBSP = Brush->IsStaticBrush();
				}
			}

			OutNewElements.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(NewActor));
		}

		if (bRebuildBSP)
		{
			GEditor->RebuildAlteredBSP();
		}
	}
}

bool UActorElementEditorWorldInterface::CanCopyElement(const FTypedElementHandle& InElementHandle)
{
	if (!UE::ActorElementEditorWorldInterface::Local::bEnableTemporaryActorCopy)
	{
		AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle);
		if (Actor && Actor->GetLevel())
		{
			if (UWorld* ActorWorld = Actor->GetLevel()->GetWorld())
			{
				// If the actor is in a PIE world but doesn't have an editor counterpart it means it's a temporary
				// actor spawned to the world. These actors can cause issues when copied so have been disabled.
				if (ActorWorld->WorldType == EWorldType::PIE && !GEditor->ObjectsThatExistInEditorWorld.Get(Actor))
				{
					return false;
				}
			}
		}
	}
	
	return true;
}

void UActorElementEditorWorldInterface::CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	TArray<AActor*> Actors = ActorElementDataUtil::GetActorsFromHandles(InElementHandles);

	if (Actors.IsEmpty())
	{
		return;
	}
	
	UActorElementsCopy* ActorElementsCopy = NewObject<UActorElementsCopy>();
	ActorElementsCopy->ActorsToCopy = Actors;

	constexpr int32 Indent = 3;
	UExporter::ExportToOutputDevice(nullptr, ActorElementsCopy, nullptr, Out, TEXT("copy"), Indent, PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified);
}

TSharedPtr<FWorldElementPasteImporter> UActorElementEditorWorldInterface::GetPasteImporter()
{
	return MakeShared<FActorElementEditorPasteImporter>();
}

bool UActorElementEditorWorldInterface::IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement)
{
	if (TTypedElement<ITypedElementHierarchyInterface> HierarchyElement = GetRegistry().GetElement<ITypedElementHierarchyInterface>(Handle))
	{
		TArray<FTypedElementHandle> ChildElementHandles;
		HierarchyElement.GetChildElements(ChildElementHandles);
		// An actor with no components has no spatial bounds and cannot be inside any volume.
		if (ChildElementHandles.IsEmpty())
		{
			return false;
		}
		for (const FTypedElementHandle& ChildHandle : ChildElementHandles)
		{
			if (TTypedElement<ITypedElementWorldInterface> ChildWorldElement = GetRegistry().GetElement<ITypedElementWorldInterface>(ChildHandle))
			{
				if (ChildWorldElement.IsElementInConvexVolume(InVolume, bMustEncompassEntireElement))
				{
					if (!bMustEncompassEntireElement)
					{
						return true;
					}
				}
				else if (bMustEncompassEntireElement)
				{
					return false;
				}
			}
		}
	}
	else
	{
		// We expect the actor elements to have a Hierarchy Interface implementation
		checkNoEntry();
	}

	return bMustEncompassEntireElement;
}

bool UActorElementEditorWorldInterface::IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement)
{
	if (TTypedElement<ITypedElementHierarchyInterface> HierarchyElement = GetRegistry().GetElement<ITypedElementHierarchyInterface>(Handle))
	{
		TArray<FTypedElementHandle> ChildElementHandles;
		HierarchyElement.GetChildElements(ChildElementHandles);
		// An actor with no components has no spatial bounds and cannot be inside any box.
		if (ChildElementHandles.IsEmpty())
		{
			return false;
		}
		for (const FTypedElementHandle& ChildHandle : ChildElementHandles)
		{
			if (TTypedElement<ITypedElementWorldInterface> ChildWorldElement = GetRegistry().GetElement<ITypedElementWorldInterface>(ChildHandle))
			{
				if (ChildWorldElement.IsElementInBox(InBox, bMustEncompassEntireElement))
				{
					if (!bMustEncompassEntireElement)
					{
						return true;
					}
				}
				else if (bMustEncompassEntireElement)
				{
					return false;
				}
			}
		}
	}
	else
	{
		// We expect the actor elements to have a Hierarchy Interface implementation
		checkNoEntry();
	}

	return bMustEncompassEntireElement;
}

TArray<FTypedElementHandle> UActorElementEditorWorldInterface::GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction)
{
	if (const AActor* Actor = ActorElementDataUtil::GetActorFromHandle(InElementHandle))
	{ 
		if (GetOwnerWorld(InElementHandle)->IsEditorWorld())
		{
			if (Actor->IsHiddenEd())
			{
				return {};
			}
		}
	}

	return  UActorElementWorldInterface::GetSelectionElementsFromSelectionFunction(InElementHandle, SelectionArgs, SelectionFunction);
}

