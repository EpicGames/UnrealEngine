// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementDetailsInterface.h"

#include "Components/ActorComponent.h"
#include "Elements/Component/ComponentElementData.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementDetailsInterface)

class FComponentTypedElementDetailsObject : public ITypedElementDetailsObject
{
public:
	explicit FComponentTypedElementDetailsObject(UActorComponent* InComponent)
		: ComponentPtr(InComponent)
	{
	}

	virtual UObject* GetObject() override
	{
		return ComponentPtr.Get();
	}

private:
	TWeakObjectPtr<UActorComponent> ComponentPtr;
};

TUniquePtr<ITypedElementDetailsObject> UComponentElementDetailsInterface::GetDetailsObject(const FTypedElementHandle& InElementHandle)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		return MakeUnique<FComponentTypedElementDetailsObject>(Component);
	}
	return nullptr;
}
