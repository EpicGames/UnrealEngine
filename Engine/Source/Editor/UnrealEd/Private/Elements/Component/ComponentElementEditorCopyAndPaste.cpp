// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementEditorCopyAndPaste.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Component/ComponentElementData.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/SMInstance/SMInstanceElementData.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementEditorCopyAndPaste)

namespace UE::ComponentEditorUtils
{
	USceneComponent* FindClosestParent(UActorComponent* ChildComponent, const TArray<TObjectPtr<UActorComponent>>& ComponentList)
	{
		// Find the most recent parent that is part of the ComponentList
		if (USceneComponent* ChildAsScene = Cast<USceneComponent>(ChildComponent))
		{
			for (USceneComponent* Parent = ChildAsScene->GetAttachParent(); Parent != nullptr; Parent = Parent->GetAttachParent())
			{
				if (ComponentList.Contains(Parent))
				{
					return Parent;
				}
			}
		}
		return nullptr;
	}
}

namespace UE::ElementCopyPasteUtil
{
	
FString ParseObjectText(FStringView Text, TArray<FTransform>* OutTransforms = nullptr)
{
	FString Result;
	FString FoundStaticMesh;
	TArray<FTransform> Transforms;

	const TCHAR* Buffer = Text.GetData();
	FString Line;

	while (FParse::Line(&Buffer, Line))
	{
		const TCHAR* Str = GetData(Line);
		if (GetBEGIN(&Str, TEXT("Object")))
		{
			FTransform Transform;
			bool bHasStaticMesh = false;
			do
			{
				FParse::Line(&Buffer, Line);
				Str = GetData(Line);
				FString StaticMesh;
				if (FParse::Value(Str, TEXT("StaticMesh="), StaticMesh))
				{
					if (FoundStaticMesh.IsEmpty())
					{
						FoundStaticMesh = MoveTemp(StaticMesh);
						bHasStaticMesh = true;
					}
					// will not paste partially matching selections
					else if (FoundStaticMesh != StaticMesh)
					{
						return Result;
					}
				}
				if (OutTransforms)
				{
					FString Value;
					if (FParse::Value(Str, TEXT("RelativeLocation="), Value, /*bShouldStopOnSeparator*/false))
					{
						FVector Location;
						if (Location.InitFromString(Value))
						{
							Transform.SetLocation(Location);
						}
					}
					if (FParse::Value(Str, TEXT("RelativeRotation="), Value))
					{
						FRotator Rotation;
						FParse::Value(Str, TEXT("Pitch="), Rotation.Pitch);
						FParse::Value(Str, TEXT("Yaw="), Rotation.Yaw);
						FParse::Value(Str, TEXT("Roll="), Rotation.Roll);
						Transform.SetRotation(FQuat::MakeFromRotator(Rotation));
					}
					if (FParse::Value(Str, TEXT("RelativeScale3D="), Value, /*bShouldStopOnSeparator*/false))
					{
						FVector Scale3D;
						if (Scale3D.InitFromString(Value))
						{
							Transform.SetScale3D(Scale3D);
						}
					}
				}
			}
			while (!GetEND(&Str, TEXT("Object")));
			if (bHasStaticMesh && OutTransforms)
			{
				Transforms.Add(MoveTemp(Transform));
			}
		}
	}

	OutTransforms->Append(MoveTemp(Transforms));
	Result = FoundStaticMesh;

	return Result;
}

bool TryPasteAsInstances(FStringView Text, UActorComponent* InComponent) 
{
	if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(InComponent))
	{
		TArray<FTransform> Transforms;
		// What static mesh (if any) is uniform among all pasted components
		FString PastedStaticMeshExportPath = ParseObjectText(Text, &Transforms);
		if (PastedStaticMeshExportPath.Len())
		{
			FString PastedStaticMeshPath = FPackageName::ExportTextPathToObjectPath(PastedStaticMeshExportPath).TrimQuotes();
			FString ComponentStaticMeshPath = ISMComponent->GetStaticMesh().GetPath();
			// check if static mesh of ISMC matches static mesh of pasted components
			if (PastedStaticMeshPath == ComponentStaticMeshPath)
			{
				for (const FTransform& Transform : Transforms)
				{
					// if so, create new instances on the selected component with the transforms of the pasted components
					ISMComponent->AddInstance(Transform.GetRelativeTransform(ISMComponent->GetComponentTransform()));
				}
				return true;
			}
		}
	}
	return false;
}

struct FPotentialSelection
{
	AActor* Actor = nullptr;
	TArray<UActorComponent*> ISMComponents;
};

FPotentialSelection GetPotentialSelection(FTypedElementListConstPtr InSelection) 
{
	FPotentialSelection PotentialSelection;
	if (!InSelection->IsValidIndex(0))
	{
		return PotentialSelection;
	}
	FTypedElementHandle FirstSelectedElement = InSelection->GetElementHandleAt(0);
	// ad hoc pattern matching the element type
	if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(FirstSelectedElement, true))
	{
		PotentialSelection.ISMComponents.Append(TInlineComponentArray<UInstancedStaticMeshComponent*>(Actor));
		PotentialSelection.Actor = Actor;
	}
	else if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(FirstSelectedElement, true))
	{
		if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(Component))
		{
			PotentialSelection.ISMComponents.Add(ISMComponent);
		}
		PotentialSelection.Actor = Component->GetOwner();
	}
	else if (FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(FirstSelectedElement, true))
	{
		PotentialSelection.ISMComponents.Add(SMInstance.GetISMComponent());
	}
	return PotentialSelection;
}
	
}

UComponentElementsExporterT3D::UComponentElementsExporterT3D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UComponentElementsCopy::StaticClass();
	bText = true;
	FormatExtension.Add(TEXT("COPY"));
	FormatDescription.Add(TEXT("Unreal world text"));
}

bool UComponentElementsExporterT3D::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Out, FFeedbackContext* Warn, uint32 PortFlags)
{
	UComponentElementsCopy* ComponentElementsCopy = Cast<UComponentElementsCopy>(Object);
	if (!ComponentElementsCopy)
	{
		return false;
	}

	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	// Duplicate the selected component templates into temporary objects that we can modify
	TMap<FName, FName> ParentMap;
	TMap<FName, UActorComponent*> ObjectMap;
	for (UActorComponent* Component : ComponentElementsCopy->ComponentsToCopy)
	{
		// Duplicate the component into a temporary object
		UObject* DuplicatedComponent = StaticDuplicateObject(Component, GetTransientPackage(), Component->GetFName());
		if (DuplicatedComponent)
		{
			// If the duplicated component is a scene component, wipe its attach parent (to prevent log warnings for referencing a private object in an external package)
			if (USceneComponent* DuplicatedCompAsSceneComp = Cast<USceneComponent>(DuplicatedComponent))
			{
				DuplicatedCompAsSceneComp->SetupAttachment(nullptr);

				AActor* Owner = Component->GetOwner();
				if (Owner && Component == Owner->GetRootComponent())
				{
					DuplicatedCompAsSceneComp->SetRelativeTransform_Direct(FTransform::Identity);
				}
			}

			// Find the closest parent component of the current component within the list of components to copy
			USceneComponent* ClosestSelectedParent = UE::ComponentEditorUtils::FindClosestParent(Component, ComponentElementsCopy->ComponentsToCopy);
			if (ClosestSelectedParent)
			{
				// If the parent is included in the list, record it into the node->parent map
				ParentMap.Add(Component->GetFName(), ClosestSelectedParent->GetFName());
			}

			// Record the temporary object into the name->object map
			ObjectMap.Add(Component->GetFName(), CastChecked<UActorComponent>(DuplicatedComponent));
		}
	}

	// Export the component object(s) to text for copying
	for (const TPair<FName, UActorComponent*>& ObjectPair : ObjectMap)
	{
		// Get the component object to be copied
		UActorComponent* ComponentToCopy = ObjectPair.Value;
		check(ComponentToCopy);

		// If this component object had a parent within the selected set
		if (ParentMap.Contains(ComponentToCopy->GetFName()))
		{
			// Get the name of the parent component
			FName ParentName = ParentMap[ComponentToCopy->GetFName()];
			if (ObjectMap.Contains(ParentName))
			{
				// Ensure that this component is a scene component
				USceneComponent* SceneComponent = Cast<USceneComponent>(ComponentToCopy);
				if (SceneComponent)
				{
					// Set the attach parent to the matching parent object in the temporary set. This allows us to preserve hierarchy in the copied set.
					SceneComponent->SetupAttachment(Cast<USceneComponent>(ObjectMap[ParentName]));
				}
			}
		}

		// Export the component object to the given string
		ExportToOutputDevice(Context, ComponentToCopy, nullptr, Out, TEXT("copy"), TextIndent, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ComponentToCopy->GetOuter());
	}

	return false;
}

void FComponentElementEditorPasteImporter::Import(FContext& Context)
{
	using namespace UE::ElementCopyPasteUtil;
	FTypedElementHandle FirstSelectedElement = Context.CurrentSelection->GetElementHandleAt(0);

	FPotentialSelection PotentialSelection = GetPotentialSelection(Context.CurrentSelection);
	
	if (PotentialSelection.ISMComponents.Num())
	{
		for (UActorComponent* ISMComponent : PotentialSelection.ISMComponents)
		{
			TryPasteAsInstances(Context.Text, ISMComponent);
		}
	}
	else if (PotentialSelection.Actor)
	{
		FString Text(Context.Text);
		GUnrealEd->PasteComponents(MutableView(ImportedComponents), PotentialSelection.Actor, true, &Text);
	}
}

TArray<FTypedElementHandle> FComponentElementEditorPasteImporter::GetImportedElements()
{
	TArray<FTypedElementHandle> Handles;
	Handles.Reserve(ImportedComponents.Num());

	for (UActorComponent* Component : ImportedComponents)
	{
		if (IsValid(Component))
		{
			if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component))
			{
				Handles.Add(MoveTemp(Handle));
			}
		}
	}

	return Handles;
}
