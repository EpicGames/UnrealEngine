// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementEditorSelectionInterface.h"

#include "DataStorage/Features.h"
#include "Components/ActorComponent.h"
#include "Components/ChildActorComponent.h"
#include "Components/SceneComponent.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Component/ComponentElementData.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "Elements/Object/ObjectElementEditorSelectionInterface.h"
#include "GameFramework/Actor.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementEditorSelectionInterface)

class FComponentElementTransactedElement : public ITypedElementTransactedElement
{
private:
	virtual TUniquePtr<ITypedElementTransactedElement> CloneImpl() const override
	{
		return MakeUnique<FComponentElementTransactedElement>(*this);
	}

	virtual FTypedElementHandle GetElementImpl() const override
	{
		const UActorComponent* Component = ComponentPtr.Get(/*bEvenIfPendingKill*/true);
		return Component
			? UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component)
			: FTypedElementHandle();
	}

	virtual void SetElementImpl(const FTypedElementHandle& InElementHandle) override
	{
		UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
		ComponentPtr = Component;
	}

	virtual void SerializeImpl(FArchive& InArchive) override
	{
		InArchive << ComponentPtr;
	}

	TWeakObjectPtr<UActorComponent> ComponentPtr;
};

bool UComponentElementEditorSelectionInterface::IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& SelectionSetPtr, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return SelectionSetPtr && Component && IsComponentSelected(Component, SelectionSetPtr.ToSharedRef(), InSelectionOptions);
}

bool UComponentElementEditorSelectionInterface::SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace UE::Editor::DataStorage;

	const UActorComponent* const Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);

	// Add a selection column in TEDS
	if (const ICompatibilityProvider* const Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		const RowHandle Row = Compatibility->FindRowWithCompatibleObject(Component);
		if (Row != InvalidRowHandle)
		{
			if (ICoreProvider* const DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				DataStorage->AddColumn<FTypedElementSelectionColumn>(Row, FTypedElementSelectionColumn{ .SelectionSet = InSelectionOptions.GetNameForTEDSIntegration() });
			}
		}
	}

	return Super::SelectElement(InElementHandle, InSelectionSet, InSelectionOptions);
}

bool UComponentElementEditorSelectionInterface::DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace UE::Editor::DataStorage;

	const UActorComponent* const Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);

	const ICompatibilityProvider* const Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	ICoreProvider* const DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (!Compatibility || !DataStorage)
	{
		return Super::DeselectElement(InElementHandle, InSelectionSet, InSelectionOptions);
	}

	const RowHandle Row = Compatibility->FindRowWithCompatibleObject(Component);
	if (DataStorage->IsRowAssigned(Row))
	{
		DataStorage->RemoveColumn(Row, FTypedElementSelectionColumn::StaticStruct());
	}

	return Super::DeselectElement(InElementHandle, InSelectionSet, InSelectionOptions);
}

bool UComponentElementEditorSelectionInterface::ShouldPreventTransactions(const FTypedElementHandle& InElementHandle)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component && UObjectElementEditorSelectionInterface::ShouldObjectPreventTransactions(Component);
}

TUniquePtr<ITypedElementTransactedElement> UComponentElementEditorSelectionInterface::CreateTransactedElementImpl()
{
	return MakeUnique<FComponentElementTransactedElement>();
}

bool UComponentElementEditorSelectionInterface::IsComponentSelected(const UActorComponent* InComponent, FTypedElementListConstRef InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	if (InSelectionSet->Num() == 0)
	{
		return false;
	}

	if (FTypedElementHandle ComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(InComponent, /*bAllowCreate*/false))
	{
		if (InSelectionSet->Contains(ComponentElement))
		{
			return true;
		}
	}

	if (InSelectionOptions.AllowIndirect())
	{
		const AActor* ConsideredActor = InComponent->GetOwner();
		const USceneComponent* ConsideredComponent = Cast<USceneComponent>(InComponent);
		if (ConsideredActor)
		{
			while (ConsideredActor->IsChildActor())
			{
				ConsideredActor = ConsideredActor->GetParentActor();
				ConsideredComponent = ConsideredActor->GetParentComponent();
			}

			while (ConsideredComponent && ConsideredComponent->IsVisualizationComponent())
			{
				ConsideredComponent = ConsideredComponent->GetAttachParent();
			}
		}

		if (ConsideredComponent)
		{
			if (FTypedElementHandle ConsideredElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(ConsideredComponent, /*bAllowCreate*/false))
			{
				return InSelectionSet->Contains(ConsideredElement);
			}
		}
	}

	return false;
}
