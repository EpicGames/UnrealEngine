// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementEditorViewportInteractionCustomization.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/SceneComponent.h"

#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorSupportDelegates.h"
#include "Components/SkinnedMeshComponent.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"

#include "Toolkits/IToolkitHost.h"

bool FComponentElementEditorViewportInteractionCustomization::GetGizmoPivotLocation(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, FVector& OutPivotLocation)
{
	const UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle);
	if (!Component)
	{
		UE_LOGF(LogTemp, Warning, "FComponentElementEditorViewportInteractionCustomization::GetGizmoPivotLocation: Failed to resolve component from element handle");
		return false;
	}

	if (const USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		FVector BasePivotLocation = FVector::ZeroVector;
		FTypedElementViewportInteractionCustomization::GetGizmoPivotLocation(InElementWorldHandle, InWidgetMode, BasePivotLocation);

		// If necessary, transform the editor pivot location to be relative to the component's parent
		const bool bIsRootComponent = SceneComponent->GetOwner()->GetRootComponent() == SceneComponent;
		const bool bIsComponentUsingAbsoluteLocation = SceneComponent->IsUsingAbsoluteLocation();
		OutPivotLocation = bIsRootComponent || bIsComponentUsingAbsoluteLocation || !SceneComponent->GetAttachParent() ? BasePivotLocation : SceneComponent->GetAttachParent()->GetComponentToWorld().Inverse().TransformPosition(BasePivotLocation);
		return true;
	}

	return false;
}

void FComponentElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, const UE::Widget::EWidgetMode InWidgetMode, const EAxisList::Type InDragAxis, const FInputDeviceState& InInputState, const FTransform& InDeltaTransform, const FVector& InPivotLocation)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle);
	if (!Component)
	{
		UE_LOGF(LogTemp, Warning, "FComponentElementEditorViewportInteractionCustomization::GizmoManipulationDeltaUpdate: Failed to resolve component from element handle");
		return;
	}

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		const FVector DeltaTranslation = InDeltaTransform.GetTranslation();
		const FRotator DeltaRotation = InDeltaTransform.Rotator();
		const FVector DeltaScale3D = InDeltaTransform.GetScale3D();

		auto ShouldApplyDeltaDirectly = [ToolkitHostPtr = GetToolkitHost()]()
		{
			if (ToolkitHostPtr)
			{
				const FEditorModeTools& ModeManager = ToolkitHostPtr->GetEditorModeManager();
				return ModeManager.GetRotationContext().bUseExplicitRotator;
			}
			return false;
		};

		const FRotator* DeltaRotationPtr = &DeltaRotation;
		if (ShouldApplyDeltaDirectly())
		{
			//
			const FRotator RelativeRot = SceneComponent->GetRelativeRotation();
			SceneComponent->SetRelativeRotationExact(RelativeRot + DeltaRotation);

			//
			FVector NewCompLocation = SceneComponent->GetRelativeLocation();
			NewCompLocation -= InPivotLocation;
			NewCompLocation = FRotationMatrix(DeltaRotation).TransformPosition(NewCompLocation);
			NewCompLocation += InPivotLocation;
			SceneComponent->SetRelativeLocation(NewCompLocation);

			//
			DeltaRotationPtr = nullptr;
		}
		
		FComponentElementEditorViewportInteractionCustomization::ApplyDeltaToComponent(SceneComponent, /*bDelta*/true, &DeltaTranslation, DeltaRotationPtr, &DeltaScale3D, InPivotLocation, InInputState);
	}
}

namespace EditorEngineDefs
{
/** Limit the minimum size of the bounding box when centering cameras on individual components to avoid extreme zooming */
static const float MinComponentBoundsForZoom = 50.0f;
}

bool FComponentElementEditorViewportInteractionCustomization::GetFocusBounds(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FBoxSphereBounds& OutBounds)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle))
	{
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
		{
			if (!PrimitiveComponent->IsRegistered())
			{
				return false;
			}

			// For components with an active deformer graph the true world-space vertex positions live in a GPU buffer
			// and are not yet reflected in PrimitiveComponent::Bounds. Their real bounds have to be computed via an async GPU readback.
#if WITH_EDITORONLY_DATA
			const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(PrimitiveComponent);
			if (SkinnedMeshComponent && SkinnedMeshComponent->HasMeshDeformer())
			{
				return false;
			}
#endif

			AActor* AlignActor = Component->GetOwner();
			if (!AlignActor)
			{
				return false;
			}

			if (USceneComponent* RootComponent = AlignActor->GetRootComponent())
			{
				TArray<USceneComponent*> SceneComponents;
				RootComponent->GetChildrenComponents(true, SceneComponents);
				SceneComponents.Add(RootComponent);
				// Some components can have huge bounds but are not visible.  Ignore these components unless it is the only component on the actor 
				const bool bIgnore = SceneComponents.Num() > 1 && PrimitiveComponent->GetIgnoreBoundsForEditorFocus();

				if (!bIgnore)
				{
					FBox LocalBox(ForceInit);
					if (!GLevelEditorModeTools().ComputeBoundingBoxForViewportFocus(
						AlignActor, PrimitiveComponent, LocalBox))
					{
						LocalBox = PrimitiveComponent->Bounds.GetBox();
						FVector Center;
						FVector Extents;
						LocalBox.GetCenterAndExtents(Center, Extents);

						// Apply a minimum size to the extents of the component's box to avoid the camera's zooming too close to small or zero-sized components
						if (Extents.SizeSquared() < EditorEngineDefs::MinComponentBoundsForZoom *
							EditorEngineDefs::MinComponentBoundsForZoom)
						{
							const FVector NewExtents(EditorEngineDefs::MinComponentBoundsForZoom, SMALL_NUMBER,
							                         SMALL_NUMBER);
							LocalBox = FBox(Center - NewExtents, Center + NewExtents);
						}
					}
					OutBounds = LocalBox;

					return true;
				}
			}
		}
	}
	return false;
}

bool FComponentElementEditorViewportInteractionCustomization::GetScaleForGizmo(const TTypedElement<ITypedElementWorldInterface>& InElementWorldHandle, FVector& OutScale)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementWorldHandle))
	{
		if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
		{
			if (!PrimitiveComponent->IsRegistered())
			{
				return false;
			}

			OutScale = PrimitiveComponent->GetRelativeScale3D();
			return true;
		}
	}

	return false;
}

void FComponentElementEditorViewportInteractionCustomization::ApplyDeltaToComponent(USceneComponent* InComponent, const bool InIsDelta, const FVector* InDeltaTranslationPtr, const FRotator* InDeltaRotationPtr, const FVector* InDeltaScalePtr, const FVector& InPivotLocation, const FInputDeviceState& InInputState)
{
	if (GEditor->IsDeltaModificationEnabled())
	{
		InComponent->Modify();
	}

	///////////////////
	// Rotation
	if (InDeltaRotationPtr)
	{
		const FRotator& InDeltaRot = *InDeltaRotationPtr;
		const bool bRotatingComp = !InIsDelta || !InDeltaRot.IsZero();
		if (bRotatingComp)
		{
			if (InIsDelta)
			{
				const FRotator Rot = InComponent->GetRelativeRotation();
				FRotator ActorRotWind, ActorRotRem;
				Rot.GetWindingAndRemainder(ActorRotWind, ActorRotRem);
				const FQuat ActorQ = ActorRotRem.Quaternion();
				const FQuat DeltaQ = InDeltaRot.Quaternion();
				const FQuat ResultQ = DeltaQ * ActorQ;

				FRotator NewActorRotRem = FRotator(ResultQ);
				ActorRotRem.SetClosestToMe(NewActorRotRem);
				FRotator DeltaRot = NewActorRotRem - ActorRotRem;
				DeltaRot.Normalize();
				InComponent->SetRelativeRotationExact(Rot + DeltaRot);
			}
			else
			{
				InComponent->SetRelativeRotationExact(InDeltaRot);
			}

			if (InIsDelta)
			{
				FVector NewCompLocation = InComponent->GetRelativeLocation();
				NewCompLocation -= InPivotLocation;
				NewCompLocation = FRotationMatrix(InDeltaRot).TransformPosition(NewCompLocation);
				NewCompLocation += InPivotLocation;
				InComponent->SetRelativeLocation(NewCompLocation);
			}
		}
	}

	///////////////////
	// Translation
	if (InDeltaTranslationPtr)
	{
		if (InIsDelta)
		{
			InComponent->SetRelativeLocation(InComponent->GetRelativeLocation() + *InDeltaTranslationPtr);
		}
		else
		{
			InComponent->SetRelativeLocation(*InDeltaTranslationPtr);
		}
	}

	///////////////////
	// Scaling
	if (InDeltaScalePtr)
	{
		const FVector& InDeltaScale = *InDeltaScalePtr;
		const bool bScalingComp = !InIsDelta || !InDeltaScale.IsNearlyZero(0.000001f);
		if (bScalingComp)
		{
			if (InIsDelta)
			{
				InComponent->SetRelativeScale3D(InComponent->GetRelativeScale3D() + InDeltaScale);

				FVector NewCompLocation = InComponent->GetRelativeLocation();
				NewCompLocation -= InPivotLocation;
				NewCompLocation += FScaleMatrix(InDeltaScale).TransformPosition(NewCompLocation);
				NewCompLocation += InPivotLocation;
				InComponent->SetRelativeLocation(NewCompLocation);
			}
			else
			{
				InComponent->SetRelativeScale3D(InDeltaScale);
			}
		}
	}

	// Update the actor before leaving.
	InComponent->MarkPackageDirty();

	InComponent->PostEditComponentMove(false);

	// Fire callbacks
	FEditorSupportDelegates::RefreshPropertyWindows.Broadcast();
	FEditorSupportDelegates::UpdateUI.Broadcast();
}
