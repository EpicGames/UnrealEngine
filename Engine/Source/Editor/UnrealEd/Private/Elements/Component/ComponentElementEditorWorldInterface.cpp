// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementEditorWorldInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Elements/Component/ComponentElementData.h"
#include "Components/ActorComponent.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Elements/Component/ComponentElementEditorCopyAndPaste.h"
#include "Kismet2/ComponentEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComponentElementEditorWorldInterface)

void UComponentElementEditorWorldInterface::NotifyMovementStarted(const FTypedElementHandle& InElementHandle)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		GEditor->BroadcastBeginObjectMovement(*Component);
	}
}

void UComponentElementEditorWorldInterface::NotifyMovementOngoing(const FTypedElementHandle& InElementHandle)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		if (AActor* Actor = Component->GetOwner())
		{
			Actor->PostEditMove(false);
		}
	}
}

void UComponentElementEditorWorldInterface::NotifyMovementEnded(const FTypedElementHandle& InElementHandle)
{
	if (UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle))
	{
		GEditor->BroadcastEndObjectMovement(*Component);
		if (AActor* Actor = Component->GetOwner())
		{
			Actor->PostEditMove(true);
			Actor->InvalidateLightingCache();
		}

		Component->MarkPackageDirty();
	}
}

bool UComponentElementEditorWorldInterface::CanDeleteElement(const FTypedElementHandle& InElementHandle)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component && GUnrealEd->CanDeleteComponent(Component);
}

bool UComponentElementEditorWorldInterface::DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	const TArray<UActorComponent*> ComponentsToDelete = ComponentElementDataUtil::GetComponentsFromHandles(InElementHandles);
	return ComponentsToDelete.Num() > 0
		&& GUnrealEd->DeleteComponents(ComponentsToDelete, InSelectionSet, InDeletionOptions.VerifyDeletionCanHappen());
}

bool UComponentElementEditorWorldInterface::CanDuplicateElement(const FTypedElementHandle& InElementHandle)
{
	UActorComponent* Component = ComponentElementDataUtil::GetComponentFromHandle(InElementHandle);
	return Component && FComponentEditorUtils::CanCopyComponent(Component); // If we can copy, we can duplicate
}

void UComponentElementEditorWorldInterface::DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	const TArray<UActorComponent*> ComponentsToDuplicate = ComponentElementDataUtil::GetComponentsFromHandles(InElementHandles);

	if (ComponentsToDuplicate.Num() > 0)
	{
		TArray<UActorComponent*> NewComponents;
		GUnrealEd->DuplicateComponents(ComponentsToDuplicate, NewComponents);

		OutNewElements.Reserve(OutNewElements.Num() + NewComponents.Num());
		for (UActorComponent* NewComponent : NewComponents)
		{
			OutNewElements.Add(UEngineElementsLibrary::AcquireEditorComponentElementHandle(NewComponent));
		}
	}
}

bool UComponentElementEditorWorldInterface::CanCopyElement(const FTypedElementHandle& InElementHandle)
{
	return true;
}

void UComponentElementEditorWorldInterface::CopyElements(TArrayView<const FTypedElementHandle> InElementHandles, FOutputDevice& Out)
{
	TArray<UActorComponent*> Components = ComponentElementDataUtil::GetComponentsFromHandles(InElementHandles);

	if (Components.IsEmpty())
	{
		return;
	}

	UComponentElementsCopy* ActorElementsCopy = NewObject<UComponentElementsCopy>();
	ActorElementsCopy->ComponentsToCopy = Components;

	int32 const Indent = 3;
	UExporter::ExportToOutputDevice(nullptr, ActorElementsCopy, nullptr, Out, TEXT("copy"), Indent, PPF_DeepCompareInstances);
}

TSharedPtr<FWorldElementPasteImporter> UComponentElementEditorWorldInterface::GetPasteImporter()
{
	return MakeShared<FComponentElementEditorPasteImporter>();
}

bool UComponentElementEditorWorldInterface::IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement)
{
	if (const UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(ComponentElementDataUtil::GetComponentFromHandle(Handle)))
	{
		return Component->ComponentIsTouchingSelectionFrustum(InVolume, false, bMustEncompassEntireElement);
	}

	return ITypedElementWorldInterface::IsElementInConvexVolume(Handle, InVolume, bMustEncompassEntireElement);
}

bool UComponentElementEditorWorldInterface::IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement)
{
	if (const UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(ComponentElementDataUtil::GetComponentFromHandle(Handle)))
	{
		return Component->ComponentIsTouchingSelectionBox(InBox, false, bMustEncompassEntireElement);
	}

	return ITypedElementWorldInterface::IsElementInBox(Handle, InBox, bMustEncompassEntireElement);
}

TArray<FTypedElementHandle> UComponentElementEditorWorldInterface::GetSelectionElementsFromSelectionFunction(const FTypedElementHandle& InElementHandle, const FWorldSelectionElementArgs& SelectionArgs, const TFunction<bool(const FTypedElementHandle&, const FWorldSelectionElementArgs&)>& SelectionFunction)
{
	if (const UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(ComponentElementDataUtil::GetComponentFromHandle(InElementHandle)))
	{
		bool bIsVisible = Component->IsRegistered();
		if (GetOwnerWorld(InElementHandle)->IsEditorWorld())
		{
			bIsVisible &= Component->IsVisibleInEditor();
		}
		else
		{
			bIsVisible &= Component->IsVisible();
		}

		if (SelectionArgs.ShowFlags)
		{
			bIsVisible &= Component->IsShown(*(SelectionArgs.ShowFlags));
		}
		
		if (bIsVisible)
		{
			return ITypedElementWorldInterface::GetSelectionElementsFromSelectionFunction(InElementHandle, SelectionArgs, SelectionFunction);
		}
	}

	return {};
}
