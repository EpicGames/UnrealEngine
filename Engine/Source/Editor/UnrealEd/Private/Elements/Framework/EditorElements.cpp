// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/EditorElements.h"

#include "Containers/UnrealString.h"
#include "EditorWidgetsModule.h"
#include "Elements/Actor/ActorElementDetailsInterface.h"
#include "Elements/Actor/ActorElementEditorAssetDataInterface.h"
#include "Elements/Actor/ActorElementEditorSelectionInterface.h"
#include "Elements/Actor/ActorElementEditorWorldInterface.h"
#include "Elements/Component/ComponentElementDetailsInterface.h"
#include "Elements/Component/ComponentElementEditorSelectionInterface.h"
#include "Elements/Component/ComponentElementEditorWorldInterface.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Elements/Object/ObjectElementDetailsInterface.h"
#include "Elements/Object/ObjectElementEditorSelectionInterface.h"
#include "Elements/SMInstance/SMInstanceElementDetailsInterface.h"
#include "Elements/SMInstance/SMInstanceElementDetailsProxyObject.h"
#include "Elements/SMInstance/SMInstanceElementEditorSelectionInterface.h"
#include "Elements/SMInstance/SMInstanceElementEditorWorldInterface.h"
#include "Elements/SMInstance/SMInstanceElementId.h"
#include "Elements/TypedElementEditorLog.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "ObjectNameEditSinkRegistry.h"
#include "Elements/Actor/ActorElementTedsTypedElementBridgeInterface.h"
#include "Elements/Interfaces/TedsTypedElementBridgeInterface.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

FSimpleMulticastDelegate OnRegisterEditorElementsDelegate;

void RegisterEditorObjectElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementInterface<ITypedElementDetailsInterface>(NAME_Object, NewObject<UObjectElementDetailsInterface>());
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Object, NewObject<UObjectElementEditorSelectionInterface>(), /*bAllowOverride*/true);
}

void RegisterEditorActorElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementInterface<ITypedElementDetailsInterface>(NAME_Actor, NewObject<UActorElementDetailsInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_Actor, NewObject<UActorElementEditorWorldInterface>(), /*bAllowOverride*/true);
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Actor, NewObject<UActorElementEditorSelectionInterface>(), /*bAllowOverride*/true);
	Registry->RegisterElementInterface<ITypedElementAssetDataInterface>(NAME_Actor, NewObject<UActorElementEditorAssetDataInterface>(), /*bAllowOverride*/true);
	Registry->RegisterElementInterface<ITedsTypedElementBridgeInterface>(NAME_Actor, NewObject<UActorElementTedsTypedElementBridgeInterface>());
}

void RegisterEditorComponentElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementInterface<ITypedElementDetailsInterface>(NAME_Components, NewObject<UComponentElementDetailsInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_Components, NewObject<UComponentElementEditorWorldInterface>(), /*bAllowOverride*/true);
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_Components, NewObject<UComponentElementEditorSelectionInterface>(), /*bAllowOverride*/true);
}

void RegisterEditorSMInstanceElements()
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

	Registry->RegisterElementInterface<ITypedElementDetailsInterface>(NAME_SMInstance, NewObject<USMInstanceElementDetailsInterface>());
	Registry->RegisterElementInterface<ITypedElementWorldInterface>(NAME_SMInstance, NewObject<USMInstanceElementEditorWorldInterface>(), /*bAllowOverride*/true);
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_SMInstance, NewObject<USMInstanceElementEditorSelectionInterface>(), /*bAllowOverride*/true);

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	EditorWidgetsModule.GetObjectNameEditSinkRegistry()->RegisterObjectNameEditSink(MakeShared<FSMInstanceElementDetailsProxyObjectNameEditSink>());
}

void RegisterEditorElements()
{
	RegisterEditorObjectElements();
	RegisterEditorActorElements();
	RegisterEditorComponentElements();
	RegisterEditorSMInstanceElements();

	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("TypedElements.OutputRegistredTypeElementsToClipboard"),
		TEXT("Output a debug string to the clipboard and to the log./n\
			It contains the names of the Typed Elements registred and their Interfaces./n\
			For each Interface it also provide the path of the class that implements it."),
			FConsoleCommandDelegate::CreateLambda([]()
				{
					FString DebugString = UTypedElementRegistry::GetInstance()->RegistredElementTypesAndInterfacesToString();
					FPlatformApplicationMisc::ClipboardCopy(*DebugString);
					UE_LOGF(LogTypedElementEditor, Log, "%ls", *DebugString);
				})
		);

	OnRegisterEditorElementsDelegate.Broadcast();
}
