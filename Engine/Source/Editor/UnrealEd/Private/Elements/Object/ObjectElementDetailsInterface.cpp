// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Object/ObjectElementDetailsInterface.h"

#include "Elements/Object/ObjectElementData.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectElementDetailsInterface)

class FObjectTypedElementDetailsObject : public ITypedElementDetailsObject
{
public:
	explicit FObjectTypedElementDetailsObject(UObject* InObject)
		: ObjectPtr(InObject)
	{
	}

	virtual UObject* GetObject() override
	{
		return ObjectPtr.Get();
	}

private:
	TWeakObjectPtr<UObject> ObjectPtr;
};

TUniquePtr<ITypedElementDetailsObject> UObjectElementDetailsInterface::GetDetailsObject(const FTypedElementHandle& InElementHandle)
{
	if (UObject* Object = ObjectElementDataUtil::GetObjectFromHandle(InElementHandle))
	{
		return MakeUnique<FObjectTypedElementDetailsObject>(Object);
	}
	return nullptr;
}
