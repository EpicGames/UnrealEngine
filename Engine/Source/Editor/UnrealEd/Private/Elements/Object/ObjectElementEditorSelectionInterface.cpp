// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Object/ObjectElementEditorSelectionInterface.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "Elements/Object/ObjectElementData.h"
#include "Serialization/Archive.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectElementEditorSelectionInterface)

class FObjectElementTransactedElement : public ITypedElementTransactedElement
{
private:
	virtual TUniquePtr<ITypedElementTransactedElement> CloneImpl() const override
	{
		return MakeUnique<FObjectElementTransactedElement>(*this);
	}

	virtual FTypedElementHandle GetElementImpl() const override
	{
		const UObject* Object = ObjectPtr.Get(/*bEvenIfPendingKill*/true);
		return Object
			? UEngineElementsLibrary::AcquireEditorObjectElementHandle(Object)
			: FTypedElementHandle();
	}

	virtual void SetElementImpl(const FTypedElementHandle& InElementHandle) override
	{
		UObject* Object = ObjectElementDataUtil::GetObjectFromHandleChecked(InElementHandle);
		ObjectPtr = Object;
	}

	virtual void SerializeImpl(FArchive& InArchive) override
	{
		InArchive << ObjectPtr;
	}

	TWeakObjectPtr<UObject> ObjectPtr;
};

bool UObjectElementEditorSelectionInterface::ShouldPreventTransactions(const FTypedElementHandle& InElementHandle)
{
	const UObject* Object = ObjectElementDataUtil::GetObjectFromHandle(InElementHandle);
	return Object && ShouldObjectPreventTransactions(Object);
}

TUniquePtr<ITypedElementTransactedElement> UObjectElementEditorSelectionInterface::CreateTransactedElementImpl()
{
	return MakeUnique<FObjectElementTransactedElement>();
}

bool UObjectElementEditorSelectionInterface::ShouldObjectPreventTransactions(const UObject* InObject)
{
	// If the selection currently contains any PIE objects we should not be including it in the transaction buffer
	return InObject->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor | PKG_ContainsScript | PKG_CompiledIn);
}
