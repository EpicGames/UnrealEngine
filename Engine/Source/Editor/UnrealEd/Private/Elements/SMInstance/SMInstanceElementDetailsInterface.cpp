// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementDetailsInterface.h"
#include "Elements/SMInstance/SMInstanceElementDetailsProxyObject.h"
#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Elements/SMInstance/SMInstanceManager.h"
#include "Components/InstancedStaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementDetailsInterface)

class FSMInstanceTypedElementDetailsObject : public ITypedElementDetailsObject
{
public:
	explicit FSMInstanceTypedElementDetailsObject(const FSMInstanceElementId& InSMInstanceElementId, const ISMInstanceManager* InInstanceManager)
	{
		USMInstanceProxyEditingObject* InstanceProxyObjectPtr = nullptr;
		if (UClass* ProxyObjectClass = InInstanceManager->GetSMInstanceEditingProxyClass())
		{
			InstanceProxyObjectPtr = NewObject<USMInstanceProxyEditingObject>((UObject*)GetTransientPackage(), ProxyObjectClass);
		}
		else
		{
			InstanceProxyObjectPtr = NewObject<USMInstanceElementDetailsProxyObject>();
		}
		InstanceProxyObjectPtr->Initialize(InSMInstanceElementId);

		InstanceProxyObject = InstanceProxyObjectPtr;
	}

	~FSMInstanceTypedElementDetailsObject()
	{
		if (USMInstanceProxyEditingObject* InstanceProxyObjectPtr = InstanceProxyObject.Get())
		{
			InstanceProxyObjectPtr->Shutdown();
		}
	}

	virtual UObject* GetObject() override
	{
		return InstanceProxyObject.Get();
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		if (USMInstanceProxyEditingObject* InstanceProxyObjectPtr = InstanceProxyObject.Get())
		{
			Collector.AddReferencedObject(InstanceProxyObject);
			InstanceProxyObject = InstanceProxyObjectPtr;
		}
	}

private:
	TWeakObjectPtr<USMInstanceProxyEditingObject> InstanceProxyObject;
};

TUniquePtr<ITypedElementDetailsObject> USMInstanceElementDetailsInterface::GetDetailsObject(const FTypedElementHandle& InElementHandle)
{
	if (const FSMInstanceElementData* SMInstanceElement = InElementHandle.GetData<FSMInstanceElementData>())
	{
		FSMInstanceManager InstanceManager = SMInstanceElementDataUtil::GetSMInstanceFromHandleChecked(InElementHandle);
		return MakeUnique<FSMInstanceTypedElementDetailsObject>(SMInstanceElement->InstanceElementId, InstanceManager.GetInstanceManager());
	}
	return nullptr;
}
