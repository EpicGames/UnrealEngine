// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementDetailsProxyObject.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Elements/SMInstance/SMInstanceElementData.h"

#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementDetailsProxyObject)

#define LOCTEXT_NAMESPACE "SMInstanceElementDetails"

void USMInstanceElementDetailsProxyObject::Initialize(const FSMInstanceElementId& InSMInstanceElementId)
{
	ISMComponent = InSMInstanceElementId.ISMComponent;
	ISMInstanceId = InSMInstanceElementId.InstanceId;

	TickHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("USMInstanceElementDetailsProxyObject"), 0.1f, [this](float)
	{
		const bool bDidSyncInstance = SyncProxyStateFromInstance();
		if (!bDidSyncInstance)
		{
			// The referenced instance is invalid, so mark this proxy as garbage so that the details panel stops using it
			MarkAsGarbage();
		}
		return bDidSyncInstance;
	});
	SyncProxyStateFromInstance();
}

void USMInstanceElementDetailsProxyObject::Shutdown()
{
	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	ISMComponent.Reset();
	ISMInstanceId = 0;

	Transform = FTransform::Identity;
}

void USMInstanceElementDetailsProxyObject::BeginDestroy()
{
	Super::BeginDestroy();

	Shutdown();
}

void USMInstanceElementDetailsProxyObject::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(USMInstanceElementDetailsProxyObject, Transform))
		{
			if (FSMInstanceManager SMInstance = GetSMInstance())
			{
				if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
				{
					if (!bIsWithinInteractiveTransformEdit)
					{
						SMInstance.NotifySMInstanceMovementStarted();
					}
					bIsWithinInteractiveTransformEdit = true;
				}

				// TODO: Need flag for local/world space, like FComponentTransformDetails
				SMInstance.SetSMInstanceTransform(Transform, /*bWorldSpace*/false, /*bMarkRenderStateDirty*/true);
				
				if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
				{
					check(bIsWithinInteractiveTransformEdit);
					SMInstance.NotifySMInstanceMovementOngoing();
				}
				else
				{
					if (bIsWithinInteractiveTransformEdit)
					{
						SMInstance.NotifySMInstanceMovementEnded();
					}
					bIsWithinInteractiveTransformEdit = false;
				}

				GUnrealEd->UpdatePivotLocationForSelection();
				GUnrealEd->RedrawLevelEditingViewports();
			}
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

bool USMInstanceElementDetailsProxyObject::SyncProxyStateFromInstance()
{
	if (FSMInstanceManager SMInstance = GetSMInstance())
	{
		// TODO: Need flag for local/world space, like FComponentTransformDetails
		SMInstance.GetSMInstanceTransform(Transform, /*bWorldSpace*/false);
		return true;
	}
	
	Transform = FTransform::Identity;
	return false;
}

FSMInstanceManager USMInstanceElementDetailsProxyObject::GetSMInstance() const
{
	const FSMInstanceId SMInstanceId = FSMInstanceElementIdMap::Get().GetSMInstanceIdFromSMInstanceElementId(FSMInstanceElementId{ ISMComponent.Get(), ISMInstanceId });
	return FSMInstanceManager(SMInstanceId, SMInstanceElementDataUtil::GetSMInstanceManager(SMInstanceId));
}

UClass* FSMInstanceElementDetailsProxyObjectNameEditSink::GetSupportedClass() const
{
	return USMInstanceElementDetailsProxyObject::StaticClass();
}

FText FSMInstanceElementDetailsProxyObjectNameEditSink::GetObjectDisplayName(UObject* Object) const
{
	USMInstanceElementDetailsProxyObject* ProxyObject = CastChecked<USMInstanceElementDetailsProxyObject>(Object);
	if (FSMInstanceManager SMInstance = ProxyObject->GetSMInstance())
	{
		FText DisplayName = SMInstance.GetSMInstanceDisplayName();
		if (DisplayName.IsEmpty())
		{
			const UStaticMesh* StaticMesh = SMInstance.GetISMComponent()->GetStaticMesh();
			const FText OwnerDisplayName = FText::FromString(StaticMesh ? StaticMesh->GetName() : SMInstance.GetISMComponent()->GetName());
			DisplayName = FText::Format(LOCTEXT("ProxyObjectDisplayNameFmt", "{0} - Instance {1}"), OwnerDisplayName, SMInstance.GetInstanceId().InstanceIndex);
		}
		return DisplayName;
	}
	return FText();
}

FText FSMInstanceElementDetailsProxyObjectNameEditSink::GetObjectNameTooltip(UObject* Object) const
{
	USMInstanceElementDetailsProxyObject* ProxyObject = CastChecked<USMInstanceElementDetailsProxyObject>(Object);
	if (FSMInstanceManager SMInstance = ProxyObject->GetSMInstance())
	{
		FText Tooltip = SMInstance.GetSMInstanceTooltip();
		if (Tooltip.IsEmpty())
		{
			const FText OwnerDisplayPath = FText::FromString(SMInstance.GetISMComponent()->GetPathName(SMInstance.GetISMComponent()->GetWorld())); // stops the path at the level of the world the object is in
			Tooltip = FText::Format(LOCTEXT("ProxyObjectTooltipFmt", "Instance {0} on {1}"), SMInstance.GetInstanceId().InstanceIndex, OwnerDisplayPath);
		}
		return Tooltip;
	}
	return FText();
}

#undef LOCTEXT_NAMESPACE
