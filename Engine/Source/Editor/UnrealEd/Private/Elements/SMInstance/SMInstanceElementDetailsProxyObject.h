// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Elements/SMInstance/SMInstanceManager.h"

#include "IObjectNameEditSink.h"

#include "Containers/Ticker.h"

#include "SMInstanceElementDetailsProxyObject.generated.h"

UCLASS(Transient)
class USMInstanceElementDetailsProxyObject : public USMInstanceProxyEditingObject
{
	GENERATED_BODY()

public:
	virtual void Initialize(const FSMInstanceElementId& InSMInstanceElementId) override;
	virtual void Shutdown() override;

	//~ UObject interface
	virtual void BeginDestroy() override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	UPROPERTY(EditAnywhere, Category="Transform", meta=(ShowOnlyInnerProperties))
	FTransform Transform;

private:
	bool SyncProxyStateFromInstance();

	FSMInstanceManager GetSMInstance() const;

	TWeakObjectPtr<UInstancedStaticMeshComponent> ISMComponent;
	uint64 ISMInstanceId = 0;

	FTSTicker::FDelegateHandle TickHandle;
	bool bIsWithinInteractiveTransformEdit = false;

	friend class FSMInstanceElementDetailsProxyObjectNameEditSink;
};

class FSMInstanceElementDetailsProxyObjectNameEditSink : public UE::EditorWidgets::IObjectNameEditSink
{
	virtual UClass* GetSupportedClass() const override;

	virtual FText GetObjectDisplayName(UObject* Object) const override;

	virtual FText GetObjectNameTooltip(UObject* Object) const override;
};