// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementEditorSelectionInterface.h"
#include "Elements/SMInstance/SMInstanceElementData.h"
#include "Components/InstancedStaticMeshComponent.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Object/ObjectElementEditorSelectionInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementEditorSelectionInterface)

class FSMInstanceElementTransactedElement : public ITypedElementTransactedElement
{
private:
	virtual TUniquePtr<ITypedElementTransactedElement> CloneImpl() const override
	{
		return MakeUnique<FSMInstanceElementTransactedElement>(*this);
	}

	virtual FTypedElementHandle GetElementImpl() const override
	{
		FSMInstanceId SMInstanceId = FSMInstanceElementIdMap::Get().GetSMInstanceIdFromSMInstanceElementId(FSMInstanceElementId{ ISMComponentPtr.Get(/*bEvenIfPendingKill*/true), InstanceId });
		return SMInstanceId
			? UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(SMInstanceId)
			: FTypedElementHandle();
	}

	virtual void SetElementImpl(const FTypedElementHandle& InElementHandle) override
	{
		const FSMInstanceElementData& SMInstanceElementData = InElementHandle.GetDataChecked<FSMInstanceElementData>();
		ISMComponentPtr = SMInstanceElementData.InstanceElementId.ISMComponent;
		InstanceId = SMInstanceElementData.InstanceElementId.InstanceId;
	}

	virtual void SerializeImpl(FArchive& InArchive) override
	{
		InArchive << ISMComponentPtr;
		InArchive << InstanceId;
	}

	TWeakObjectPtr<UInstancedStaticMeshComponent> ISMComponentPtr;
	uint64 InstanceId = 0;
};

bool USMInstanceElementEditorSelectionInterface::IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& SelectionSetPtr, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);

	if (SMInstance && SelectionSetPtr && SelectionSetPtr->Num() > 0)
	{
		// If any element within this group is selected, then the whole group is selected
		bool bIsSelected = false;
		SMInstance.ForEachSMInstanceInSelectionGroup([&InSelectionOptions, &SelectionSetPtr, &bIsSelected](FSMInstanceId GroupSMInstance)
		{
			if (FTypedElementHandle GroupSMInstanceElement = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(GroupSMInstance, /*bAllowCreate*/false))
			{
				if (SelectionSetPtr->Contains(GroupSMInstanceElement))
				{
					bIsSelected = true;
				}
				else if (InSelectionOptions.AllowIndirect())
				{
					if (FTypedElementHandle ISMComponentElement = UEngineElementsLibrary::AcquireEditorComponentElementHandle(GroupSMInstance.ISMComponent, /*bAllowCreate*/false))
					{
						bIsSelected = SelectionSetPtr->Contains(ISMComponentElement);
					}
				}
			}
			return !bIsSelected;
		});
		return bIsSelected;
	}

	return false;
}

bool USMInstanceElementEditorSelectionInterface::SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& SelectionSetPtr, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);

	if (SMInstance && SelectionSetPtr && SelectionSetPtr->Num() > 0)
	{
		// If any element within this group is already in the selection set, then we have nothing more to do
		bool bIsSelected = false;
		SMInstance.ForEachSMInstanceInSelectionGroup([&SelectionSetPtr, &bIsSelected](FSMInstanceId GroupSMInstance)
		{
			if (FTypedElementHandle GroupSMInstanceElement = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(GroupSMInstance, /*bAllowCreate*/false))
			{
				if (SelectionSetPtr->Contains(GroupSMInstanceElement))
				{
					bIsSelected = true;
				}
			}
			return !bIsSelected;
		});
		if (bIsSelected)
		{
			return false;
		}
	}

	if (SMInstance && SelectionSetPtr && SelectionSetPtr->Add(InElementHandle))
	{
		SMInstance.NotifySMInstanceSelectionChanged(/*bIsSelected*/true);
		return true;
	}

	return false;
}

bool USMInstanceElementEditorSelectionInterface::DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& SelectionSetPtr, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);

	if (SMInstance && SelectionSetPtr && SelectionSetPtr->Num() > 0)
	{
		bool bSelectionChanged = false;

		// Deselect every element within this group
		SMInstance.ForEachSMInstanceInSelectionGroup([&SelectionSetPtr, &bSelectionChanged](FSMInstanceId GroupSMInstance)
		{
			if (FTypedElementHandle GroupSMInstanceElement = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(GroupSMInstance, /*bAllowCreate*/false))
			{
				bSelectionChanged |= SelectionSetPtr->Remove(GroupSMInstanceElement);
			}
			return true;
		});

		if (bSelectionChanged)
		{
			SMInstance.NotifySMInstanceSelectionChanged(/*bIsSelected*/false);
		}

		return bSelectionChanged;
	}

	return false;
}

bool USMInstanceElementEditorSelectionInterface::ShouldPreventTransactions(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && UObjectElementEditorSelectionInterface::ShouldObjectPreventTransactions(SMInstance.GetISMComponent());
}

TUniquePtr<ITypedElementTransactedElement> USMInstanceElementEditorSelectionInterface::CreateTransactedElementImpl()
{
	return MakeUnique<FSMInstanceElementTransactedElement>();
}
