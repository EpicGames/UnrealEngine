// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/SMInstance/SMInstanceElementEditorWorldInterface.h"
#include "Elements/SMInstance/SMInstanceElementData.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

#include "ComponentReregisterContext.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SMInstanceElementEditorWorldInterface)

bool USMInstanceElementEditorWorldInterface::CanDeleteElement(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.CanDeleteSMInstance();
}

bool USMInstanceElementEditorWorldInterface::DeleteElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (SMInstance.CanDeleteSMInstance())
		{
			InSelectionSet->DeselectElement(InElementHandle, FTypedElementSelectionOptions());
			return SMInstance.DeleteSMInstance();
		}
	}

	return false;
}

bool USMInstanceElementEditorWorldInterface::DeleteElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, UTypedElementSelectionSet* InSelectionSet, const FTypedElementDeletionOptions& InDeletionOptions)
{
	const TArray<FSMInstanceManager> SMInstancesToDelete = SMInstanceElementDataUtil::GetSMInstancesFromHandles(InElementHandles);

	if (SMInstancesToDelete.Num() > 0)
	{
		// Batch by the ISM manager
		TMap<ISMInstanceManager*, TArray<FSMInstanceId>> BatchedInstancesToDelete;
		for (const FSMInstanceManager& SMInstance : SMInstancesToDelete)
		{
			if (SMInstance.CanDeleteSMInstance())
			{
				TArray<FSMInstanceId>& InstanceIds = BatchedInstancesToDelete.FindOrAdd(SMInstance.GetInstanceManager());
				InstanceIds.Add(SMInstance.GetInstanceId());
			}
		}

		bool bDidDelete = false;

		FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*InSelectionSet->GetElementList());
		for (TTuple<ISMInstanceManager*, TArray<FSMInstanceId>>& BatchedInstancesToDeletePair : BatchedInstancesToDelete)
		{
			for (const FSMInstanceId& InstanceId : BatchedInstancesToDeletePair.Value)
			{
				InSelectionSet->DeselectElement(UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(InstanceId), FTypedElementSelectionOptions());
			}
			bDidDelete |= BatchedInstancesToDeletePair.Key->DeleteSMInstances(BatchedInstancesToDeletePair.Value);
		}

		return bDidDelete;
	}

	return false;
}

bool USMInstanceElementEditorWorldInterface::CanDuplicateElement(const FTypedElementHandle& InElementHandle)
{
	const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle);
	return SMInstance && SMInstance.CanDuplicateSMInstance();
}

FTypedElementHandle USMInstanceElementEditorWorldInterface::DuplicateElement(const FTypedElementHandle& InElementHandle, UWorld* InWorld, const FVector& InLocationOffset)
{
	if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (SMInstance.CanDuplicateSMInstance())
		{
			FSMInstanceId NewInstanceId;
			if (SMInstance.DuplicateSMInstance(NewInstanceId))
			{
				const bool bOffsetIsZero = InLocationOffset.IsZero();

				if (!bOffsetIsZero)
				{
					ISMInstanceManager* InstanceManager = SMInstance.GetInstanceManager();

					FTransform NewInstanceTransform = FTransform::Identity;
					InstanceManager->GetSMInstanceTransform(NewInstanceId, NewInstanceTransform, /*bWorldSpace*/false);
					NewInstanceTransform.SetTranslation(NewInstanceTransform.GetTranslation() + InLocationOffset);
					InstanceManager->SetSMInstanceTransform(NewInstanceId, NewInstanceTransform, /*bWorldSpace*/false, /*bMarkRenderStateDirty*/true);
				}

				return UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(NewInstanceId);
			}
		}
	}

	return FTypedElementHandle();
}

void USMInstanceElementEditorWorldInterface::DuplicateElements(TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	const TArray<FSMInstanceManager> SMInstancesToDuplicate = SMInstanceElementDataUtil::GetSMInstancesFromHandles(InElementHandles);

	if (SMInstancesToDuplicate.Num() > 0)
	{
		// Batch by the ISM manager
		TMap<ISMInstanceManager*, TArray<FSMInstanceId>> BatchedInstancesToDuplicate;
		for (const FSMInstanceManager& SMInstance : SMInstancesToDuplicate)
		{
			if (SMInstance.CanDuplicateSMInstance())
			{
				TArray<FSMInstanceId>& InstanceIds = BatchedInstancesToDuplicate.FindOrAdd(SMInstance.GetInstanceManager());
				InstanceIds.Add(SMInstance.GetInstanceId());
			}
		}

		for (const TTuple<ISMInstanceManager*, TArray<FSMInstanceId>>& BatchedInstancesToDuplicatePair : BatchedInstancesToDuplicate)
		{
			TArray<FSMInstanceId> NewInstanceIds;
			if (BatchedInstancesToDuplicatePair.Key->DuplicateSMInstances(BatchedInstancesToDuplicatePair.Value, NewInstanceIds))
			{
				const bool bOffsetIsZero = InLocationOffset.IsZero();

				OutNewElements.Reserve(OutNewElements.Num() + NewInstanceIds.Num());
				for (const FSMInstanceId& NewInstanceId : NewInstanceIds)
				{
					if (!bOffsetIsZero)
					{
						FTransform NewInstanceTransform = FTransform::Identity;
						BatchedInstancesToDuplicatePair.Key->GetSMInstanceTransform(NewInstanceId, NewInstanceTransform, /*bWorldSpace*/false);
						NewInstanceTransform.SetTranslation(NewInstanceTransform.GetTranslation() + InLocationOffset);
						BatchedInstancesToDuplicatePair.Key->SetSMInstanceTransform(NewInstanceId, NewInstanceTransform, /*bWorldSpace*/false, /*bMarkRenderStateDirty*/true);
					}

					OutNewElements.Add(UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(NewInstanceId));
				}
			}
		}
	}
}

bool USMInstanceElementEditorWorldInterface::CanPromoteElement(const FTypedElementHandle& InElementHandle)
{
	if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
	{
		if (UWorld* ElementWorld = SMInstance.GetISMComponent()->GetWorld())
		{
			return !ElementWorld->IsGameWorld() && SMInstance.CanDeleteSMInstance();
		}
	}

	return false;
}

FTypedElementHandle USMInstanceElementEditorWorldInterface::PromoteElement(const FTypedElementHandle& InElementHandle, UWorld* OverrideWorld /* = nullptr */)
{
	UWorld* World = OverrideWorld;
	if (CanPromoteElement(InElementHandle))
	{
		if (const FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(InElementHandle))
		{
			UInstancedStaticMeshComponent* OldComponent = SMInstance.GetISMComponent();
			if (!World)
			{
				World = SMInstance.GetISMComponent()->GetWorld();
			}

			// Create Actor
			FActorSpawnParameters SpawnParams;
			SpawnParams.ObjectFlags = RF_Transactional;
			SpawnParams.OverrideLevel = World->GetCurrentLevel();

			AStaticMeshActor* StaticMeshActor = World->SpawnActor<AStaticMeshActor>(SpawnParams);

			UStaticMeshComponent* NewComponent = StaticMeshActor->GetStaticMeshComponent();

			// Copy editable properties
			FComponentReregisterContext ComponentRegisterContext(NewComponent);
			FProperty* Property = UStaticMeshComponent::StaticClass()->PropertyLink;
			while (Property)
			{
				if (Property->HasAllPropertyFlags(CPF_Edit))
				{
					void* DestinationAddress = Property->ContainerPtrToValuePtr<void>(NewComponent);
					void* SourceAddress = Property->ContainerPtrToValuePtr<void>(OldComponent);
					Property->CopyCompleteValue(DestinationAddress, SourceAddress);
				}

				Property = Property->PropertyLinkNext;
			}

			// Set the transform
			FTransform WorldTransform;
			SMInstance.GetSMInstanceTransform(WorldTransform, /*bWorldSpace*/true);
			StaticMeshActor->SetActorTransform(WorldTransform);

			// Move the custom data
			int32 CustomDataStart = OldComponent->NumCustomDataFloats * SMInstance.GetISMInstanceIndex();
			for (int32 Index = 0; Index < OldComponent->NumCustomDataFloats; ++Index)
			{
				const float CustomData = OldComponent->PerInstanceSMCustomData[CustomDataStart + Index];
				NewComponent->SetCustomPrimitiveDataFloat(Index, CustomData);
			}

			StaticMeshActor->PostEditChange();
			StaticMeshActor->PostEditMove(true);

			SMInstance.DeleteSMInstance();

			return UEngineElementsLibrary::AcquireEditorActorElementHandle(StaticMeshActor);
		}
	}

	return FTypedElementHandle();
}

bool USMInstanceElementEditorWorldInterface::IsElementInConvexVolume(const FTypedElementHandle& Handle, const FConvexVolume& InVolume, bool bMustEncompassEntireElement)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(Handle);

	if (UInstancedStaticMeshComponent* Component = SMInstance.GetISMComponent())
	{
		return Component->IsInstanceTouchingSelectionFrustum(SMInstance.GetISMInstanceIndex(), InVolume, bMustEncompassEntireElement);
	}

	return false;
}

bool USMInstanceElementEditorWorldInterface::IsElementInBox(const FTypedElementHandle& Handle, const FBox& InBox, bool bMustEncompassEntireElement)
{
	FSMInstanceManager SMInstance = SMInstanceElementDataUtil::GetSMInstanceFromHandle(Handle);

	if (UInstancedStaticMeshComponent* Component = SMInstance.GetISMComponent())
	{
		return Component->IsInstanceTouchingSelectionBox(SMInstance.GetISMInstanceIndex(), InBox, bMustEncompassEntireElement);
	}

	return false;
}
