// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ActorFactory.cpp: 
=============================================================================*/

#include "ActorFactories/ActorFactory.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Pawn.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Model.h"
#include "ActorFactories/ActorFactoryAmbientSound.h"
#include "ActorFactories/ActorFactorySkyAtmosphere.h"
#include "ActorFactories/ActorFactoryVolumetricCloud.h"
#include "ActorFactories/ActorFactoryBlueprint.h"
#include "ActorFactories/ActorFactoryBoxReflectionCapture.h"
#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "ActorFactories/ActorFactoryCameraActor.h"
#include "ActorFactories/ActorFactoryCharacter.h"
#include "ActorFactories/ActorFactoryClass.h"
#include "ActorFactories/ActorFactoryCylinderVolume.h"
#include "ActorFactories/ActorFactoryDeferredDecal.h"
#include "ActorFactories/ActorFactoryDirectionalLight.h"
#include "ActorFactories/ActorFactoryEmitter.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "ActorFactories/ActorFactoryPawn.h"
#include "ActorFactories/ActorFactoryExponentialHeightFog.h"
#include "ActorFactories/ActorFactoryLocalFogVolume.h"
#include "ActorFactories/ActorFactoryNote.h"
#include "ActorFactories/ActorFactoryPhysicsAsset.h"
#include "ActorFactories/ActorFactoryPlaneReflectionCapture.h"
#include "ActorFactories/ActorFactoryPlayerStart.h"
#include "ActorFactories/ActorFactoryPointLight.h"
#include "ActorFactories/ActorFactorySpotLight.h"
#include "ActorFactories/ActorFactoryRectLight.h"
#include "ActorFactories/ActorFactoryRuntimeVirtualTextureVolume.h"
#include "ActorFactories/ActorFactorySkyLight.h"
#include "ActorFactories/ActorFactorySkeletalMesh.h"
#include "ActorFactories/ActorFactoryAnimationAsset.h"
#include "ActorFactories/ActorFactorySphereReflectionCapture.h"
#include "ActorFactories/ActorFactorySphereVolume.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "ActorFactories/ActorFactoryInteractiveFoliage.h"
#include "ActorFactories/ActorFactoryTargetPoint.h"
#include "ActorFactories/ActorFactoryTextRender.h"
#include "ActorFactories/ActorFactoryTriggerBox.h"
#include "ActorFactories/ActorFactoryTriggerCapsule.h"
#include "ActorFactories/ActorFactoryTriggerSphere.h"
#include "ActorFactories/ActorFactoryVectorFieldVolume.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "Materials/Material.h"
#include "Animation/AnimSequenceBase.h"
#include "Engine/BrushBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Builders/CylinderBuilder.h"
#include "Builders/TetrahedronBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/EditorEngine.h"
#include "Animation/AnimBlueprint.h"
#include "Particles/ParticleSystem.h"
#include "Engine/Texture2D.h"
#include "Animation/SkeletalMeshActor.h"
#include "GameFramework/Character.h"
#include "Camera/CameraActor.h"
#include "GameFramework/PlayerStart.h"
#include "Particles/Emitter.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundBase.h"
#include "Sound/AmbientSound.h"
#include "GameFramework/Volume.h"
#include "Engine/DecalActor.h"
#include "Atmosphere/AtmosphericFog.h"
#include "Engine/ExponentialHeightFog.h"
#include "Engine/LocalFogVolume.h"
#include "Engine/SkyLight.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/SpotLight.h"
#include "Engine/RectLight.h"
#include "Engine/Note.h"
#include "Engine/BoxReflectionCapture.h"
#include "Engine/PlaneReflectionCapture.h"
#include "Engine/SphereReflectionCapture.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TargetPoint.h"
#include "VectorField/VectorFieldVolume.h"
#include "Components/DecalComponent.h"
#include "Components/BillboardComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/VolumetricCloudComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Engine/Polys.h"
#include "StaticMeshResources.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "BSPOps.h"
#include "InteractiveFoliageActor.h"
#include "VT/RuntimeVirtualTextureVolume.h"

#include "AssetRegistry/AssetRegistryModule.h"



#include "VectorField/VectorField.h"

#include "Engine/TriggerBox.h"
#include "Engine/TriggerSphere.h"
#include "Engine/TriggerCapsule.h"
#include "Engine/TextRenderActor.h"

#include "Components/SkeletalMeshComponent.h"
#include "Particles/ParticleSystemComponent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Components/AudioComponent.h"
#include "Components/BrushComponent.h"
#include "Components/VectorFieldComponent.h"
#include "ActorFactories/ActorFactoryPlanarReflection.h"
#include "Engine/PlanarReflection.h"

#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "Factories/ActorFactoryLevelSequence.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Subsystems/PlacementSubsystem.h"

#include "LevelEditorViewport.h"
#include "Editor.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Misc/NamePermissionList.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactory)

DEFINE_LOG_CATEGORY(LogActorFactory);

#define LOCTEXT_NAMESPACE "ActorFactory"

/**
 * Find am alignment transform for the specified actor rotation, given a model-space axis to align, and a world space normal to align to.
 * This function attempts to find a 'natural' looking rotation by rotating around a local pitch axis, and a world Z. Rotating in this way
 * should retain the roll around the model space axis, removing rotation artifacts introduced by a simpler quaternion rotation.
 */
FQuat FindActorAlignmentRotation(const FQuat& InActorRotation, const FVector& InModelAxis, const FVector& InWorldNormal, FQuat* OutDeltaRotation)
{
	FVector TransformedModelAxis = InActorRotation.RotateVector(InModelAxis);

	const auto InverseActorRotation = InActorRotation.Inverse();
	const auto DestNormalModelSpace = InverseActorRotation.RotateVector(InWorldNormal);

	FQuat DeltaRotation = FQuat::Identity;
	if (OutDeltaRotation)
	{
		*OutDeltaRotation = FQuat::Identity;
	}

	const double VectorDot = InWorldNormal | TransformedModelAxis;
	if (1.f - FMath::Abs(VectorDot) <= KINDA_SMALL_NUMBER)
	{
		if (VectorDot < 0.f)
		{
			// Anti-parallel
			return InActorRotation * FQuat::FindBetween(InModelAxis, DestNormalModelSpace);
		}
	}
	else
	{
		const FVector Z(0.f, 0.f, 1.f);

		// Find a reference axis to measure the relative pitch rotations between the source axis, and the destination axis.
		FVector PitchReferenceAxis = InverseActorRotation.RotateVector(Z);
		if (FMath::Abs(FVector::DotProduct(InModelAxis, PitchReferenceAxis)) > 0.7f)
		{
			PitchReferenceAxis = DestNormalModelSpace;
		}
		
		// Find a local 'pitch' axis to rotate around
		const FVector OrthoPitchAxis = FVector::CrossProduct(PitchReferenceAxis, InModelAxis);
		const double Pitch = FMath::Acos(PitchReferenceAxis | DestNormalModelSpace) - FMath::Acos(PitchReferenceAxis | InModelAxis);//FMath::Asin(OrthoPitchAxis.Size());

		DeltaRotation = FQuat(OrthoPitchAxis.GetSafeNormal(), Pitch);
		DeltaRotation.Normalize();

		// Transform the model axis with this new pitch rotation to see if there is any need for yaw
		TransformedModelAxis = (InActorRotation * DeltaRotation).RotateVector(InModelAxis);

		const float ParallelDotThreshold = 0.98f; // roughly 11.4 degrees (!)
		if (!FVector::Coincident(InWorldNormal, TransformedModelAxis, ParallelDotThreshold))
		{
			const double Yaw = FMath::Atan2(InWorldNormal.X, InWorldNormal.Y) - FMath::Atan2(TransformedModelAxis.X, TransformedModelAxis.Y);

			// Rotation axis for yaw is the Z axis in world space
			const FVector WorldYawAxis = (InActorRotation * DeltaRotation).Inverse().RotateVector(Z);
			DeltaRotation *= FQuat(WorldYawAxis, -Yaw);
		}

		if (OutDeltaRotation)
		{
			*OutDeltaRotation = DeltaRotation;
		}
	}

	return InActorRotation * DeltaRotation;
}

/*-----------------------------------------------------------------------------
UActorFactory
-----------------------------------------------------------------------------*/
UActorFactory::UActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DefaultName","Actor");
	bShowInEditorQuickMenu = false;
	bUsePlacementExtent = true;
}

bool UActorFactory::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UActorFactory::CanCreateActorFrom);

	if (!AssetData.IsValid())
	{
		return true;
	}

	UObject* DefaultActor = GetDefaultActor(AssetData);
	FSoftObjectPath DefaultActorPath(DefaultActor);
	FSoftObjectPath DefaultActorClassPath(DefaultActor->GetClass());

	// By Default we assume the factory can't work with existing asset data
	return AssetData.GetSoftObjectPath() == DefaultActorPath || 
		AssetData.GetSoftObjectPath() == DefaultActorClassPath;
}

AActor* UActorFactory::GetDefaultActor( const FAssetData& AssetData )
{
	UClass* DefaultActorClass = GetDefaultActorClass( AssetData );
	return (DefaultActorClass ? DefaultActorClass->GetDefaultObject<AActor>() : nullptr);
}

UClass* UActorFactory::GetDefaultActorClass( const FAssetData& AssetData )
{
	if (!NewActorClassName.IsEmpty())
	{
		UE_LOGF(LogActorFactory, Log, "Loading ActorFactory Class %ls", *NewActorClassName);
		UClass* NewActorClassCandidate = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, *NewActorClassName, nullptr, LOAD_NoWarn, nullptr));
		NewActorClassName.Empty();
		if (NewActorClassCandidate == nullptr)
		{
			UE_LOGF(LogActorFactory, Warning, "ActorFactory Class LOAD FAILED - falling back to %ls", *GetNameSafe(*NewActorClass));
		}
		else if (!NewActorClassCandidate->IsChildOf(AActor::StaticClass()))
		{
			UE_LOGF(LogActorFactory, Warning, "ActorFactory Class loaded class %ls doesn't derive from Actor - falling back to %ls", *NewActorClassCandidate->GetName(), *GetNameSafe(*NewActorClass));
		}
		else
		{
			NewActorClass = NewActorClassCandidate;
		}
	}
	return NewActorClass;
}

UObject* UActorFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	return nullptr;
}

FQuat UActorFactory::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	if (bUseSurfaceOrientation)
	{
		// By default we align the X axis with the inverse of the surface normal (so things look at the surface)
		return FindActorAlignmentRotation(ActorRotation, FVector(-1.f, 0.f, 0.f), InSurfaceNormal);
	}
	else
	{
		return FQuat::Identity;
	}
}

bool UActorFactory::CanPlaceElementsFromAssetData(const FAssetData& InAssetData)
{
	if (!InAssetData.IsValid())
	{
		return false;
	}

	UClass* Class = InAssetData.GetClass();
	if (Class && (Class == GetDefaultActorClass(InAssetData)))
	{
		return true;
	}

	FText TmpMsg;
	return CanCreateActorFrom(InAssetData, TmpMsg);
}

bool UActorFactory::PrePlaceAsset(FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	return PreSpawnActor(InPlacementInfo.AssetToPlace.GetAsset(), InPlacementInfo.FinalizedTransform);
}

TArray<FTypedElementHandle> UActorFactory::PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	TArray<FTypedElementHandle> PlacedActorHandles;
	FActorSpawnParameters SpawnParams;
	SpawnParams.Name = InPlacementInfo.NameOverride;
	SpawnParams.ObjectFlags = InPlacementOptions.bIsCreatingPreviewElements ? EObjectFlags::RF_Transient : EObjectFlags::RF_Transactional;
	SpawnParams.bTemporaryEditorActor = InPlacementOptions.bIsCreatingPreviewElements;

	AActor* NewActor = CreateActor(InPlacementInfo.AssetToPlace.GetAsset(), InPlacementInfo.PreferredLevel.Get(), InPlacementInfo.FinalizedTransform, SpawnParams);
	if (NewActor)
	{
		PlacedActorHandles.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(NewActor));

		// Disable collision for preview actors
		if (InPlacementOptions.bIsCreatingPreviewElements)
		{
			NewActor->SetActorEnableCollision(false);
		}
	}

	return PlacedActorHandles;
}

void UActorFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InElementHandles, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	for (const FTypedElementHandle& PlacedElement : InElementHandles)
	{
		if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(PlacedElement))
		{
			if (AActor* CreatedActor = ObjectInterface.GetObjectAs<AActor>())
			{
				UObject* Asset = InPlacementInfo.AssetToPlace.GetAsset();

				// Only do this if the actor wasn't already given a name
				if (InPlacementInfo.NameOverride.IsNone())
				{
					FActorLabelUtilities::SetActorLabelUnique(CreatedActor, GetDefaultActorLabel(Asset));
				}

				PostSpawnActor(Asset, CreatedActor);

				CreatedActor->PostEditChange();
				CreatedActor->PostEditMove(true);
			}
		}
	}
}

FAssetData UActorFactory::GetAssetDataFromElementHandle(const FTypedElementHandle& InHandle)
{
	if (!InHandle.IsSet())
	{
		return FAssetData();
	}

	// Check if the handle is the type of actor created by this factory, and use the factory to find the wrapped asset data if possible.
	if (TTypedElement<ITypedElementObjectInterface> ElementObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(InHandle))
	{
		if (AActor* RawElementActorPtr = ElementObjectInterface.GetObjectAs<AActor>(GetDefaultActorClass(FAssetData())))
		{
			if (UObject* WrappedAssetObject = GetAssetFromActorInstance(RawElementActorPtr))
			{
				return FAssetData(WrappedAssetObject);
			}
		}
	}

	// Check if any of the referenced content is created by this factory.
	if (TTypedElement<ITypedElementAssetDataInterface> AssetDataInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementAssetDataInterface>(InHandle))
	{
		TArray<FAssetData> ContentAssetDatas = AssetDataInterface.GetAllReferencedAssetDatas();
		for (FAssetData& AssetData : ContentAssetDatas)
		{
			if (CanPlaceElementsFromAssetData(AssetData))
			{
				return MoveTemp(AssetData);
			}
		}
	}

	return FAssetData();
}

void UActorFactory::BeginPlacement(const FPlacementOptions& InPlacementOptions)
{
}

void UActorFactory::EndPlacement(TArrayView<const FTypedElementHandle> InPlacedElements, const FPlacementOptions& InPlacementOptions)
{
}

UInstancedPlacemenClientSettings* UActorFactory::FactorySettingsObjectForPlacement(const FAssetData& InAssetData, const FPlacementOptions& InPlacementOptions)
{
	return nullptr;
}

AActor* UActorFactory::CreateActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AActor* NewActor = nullptr;

	FTransform Transform(InTransform);
	if (PreSpawnActor(InAsset, Transform))
	{
		NewActor = SpawnActor(InAsset, InLevel, Transform, InSpawnParams);

		if (NewActor)
		{
			// Only do this if the actor wasn't already given a name
			if ((InAsset != nullptr) && (InSpawnParams.Name == NAME_None))
			{
				FActorLabelUtilities::SetActorLabelUnique(NewActor, GetDefaultActorLabel(InAsset));
			}

			PostSpawnActor(InAsset, NewActor);
			NewActor->PostEditChange();
			NewActor->PostEditMove(true);
		}
	}

	return NewActor;
}

ULevel* UActorFactory::ValidateSpawnActorLevel(ULevel* InLevel, const FActorSpawnParameters& InSpawnParams) const
{
	ULevel* LocalLevel = InLevel;
	if (InLevel != nullptr)
	{
		// If InLevel is passed, then InSpawnParams.OverrideLevel shouldn't be used or be identical to that level, because in the end, InLevel will be the one we'll use :
		ensure((InSpawnParams.OverrideLevel == nullptr) || (InSpawnParams.OverrideLevel == InLevel));
	}
	else
	{
		// If InLevel is not passed then the level should at least be specified in InSpawnParams.OverrideLevel :
		ensure(InSpawnParams.OverrideLevel != nullptr);
		LocalLevel = InSpawnParams.OverrideLevel;
	}

	return LocalLevel;
}

bool UActorFactory::PreSpawnActor( UObject* Asset, FTransform& InOutLocation)
{
	UE_LOGF(LogActorFactory, Log, "Actor Factory attempting to spawn %ls", *Asset->GetFullName());

	// Subclasses may implement this to set up a spawn or to adjust the spawn location or rotation.
	return true;
}

AActor* UActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	// If not already set, provide an override spawning level mount point object using the asset used to spawn the actor
	TUniquePtr<FScopedOverrideSpawningLevelMountPointObject> ScopeSpawningMountPoint;
	if (!ULevel::GetOverrideSpawningLevelMountPointObject())
	{
		ScopeSpawningMountPoint = MakeUnique<FScopedOverrideSpawningLevelMountPointObject>(InAsset);
	}
	
	ULevel* LocalLevel = ValidateSpawnActorLevel(InLevel, InSpawnParams);

	AActor* DefaultActor = GetDefaultActor(FAssetData(InAsset));
	if ((DefaultActor != nullptr) && (LocalLevel != nullptr))
	{
		FActorSpawnParameters SpawnParamsCopy(InSpawnParams);
		SpawnParamsCopy.OverrideLevel = LocalLevel;

		const bool bIsCreatingPreviewElements = FLevelEditorViewportClient::IsDroppingPreviewActor();
		bool bIsPlacementSystemCreatingPreviewElements = false;
		if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
		{
			bIsPlacementSystemCreatingPreviewElements = PlacementSubsystem->IsCreatingPreviewElements();
		}
		SpawnParamsCopy.bTemporaryEditorActor |= bIsCreatingPreviewElements || bIsPlacementSystemCreatingPreviewElements;
		SpawnParamsCopy.bHideFromSceneOutliner |= bIsPlacementSystemCreatingPreviewElements;

		return LocalLevel->OwningWorld->SpawnActor(DefaultActor->GetClass(), &InTransform, SpawnParamsCopy);
	}

	return nullptr;
}

void UActorFactory::PostSpawnActor( UObject* Asset, AActor* NewActor)
{
	UE_LOGF(LogActorFactory, Log, "Actor Factory spawned %ls as actor: %ls", *Asset->GetFullName(), *NewActor->GetFullName());
}

FString UActorFactory::GetDefaultActorLabel(UObject* Asset) const
{
	UClass* PotentialActorClass = nullptr;
	if (UBlueprint* Blueprint = Cast<UBlueprint>(Asset))
	{
		PotentialActorClass = Blueprint->GeneratedClass;
	}
	else
	{
		PotentialActorClass = Cast<UClass>(Asset);
	}

	if (PotentialActorClass && PotentialActorClass->IsChildOf<AActor>())
	{
		return GetDefault<AActor>(PotentialActorClass)->GetDefaultActorLabel();
	}

	return Asset->GetName();
}


/*-----------------------------------------------------------------------------
UActorFactoryStaticMesh
-----------------------------------------------------------------------------*/
UActorFactoryStaticMesh::UActorFactoryStaticMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("StaticMeshDisplayName", "Static Mesh");
	NewActorClass = AStaticMeshActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryStaticMesh::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UStaticMesh::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoStaticMesh", "A valid static mesh must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryStaticMesh::PostSpawnActor( UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UStaticMesh* StaticMesh = CastChecked<UStaticMesh>(Asset);

	// Change properties
	AStaticMeshActor* StaticMeshActor = CastChecked<AStaticMeshActor>( NewActor );
	UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();
	check(StaticMeshComponent);

	StaticMeshComponent->UnregisterComponent();

	StaticMeshComponent->SetStaticMesh(StaticMesh);
	if (StaticMesh->GetRenderData())
	{
		StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
	}

	// Init Component
	StaticMeshComponent->RegisterComponent();
}

UObject* UActorFactoryStaticMesh::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	AStaticMeshActor* SMA = CastChecked<AStaticMeshActor>(Instance);

	check(SMA->GetStaticMeshComponent());
	return SMA->GetStaticMeshComponent()->GetStaticMesh();
}

FQuat UActorFactoryStaticMesh::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	// Meshes align the Z (up) axis with the surface normal
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

/*-----------------------------------------------------------------------------
UActorFactoryBasicShape
-----------------------------------------------------------------------------*/

const FSoftObjectPath UActorFactoryBasicShape::BasicCube(FTopLevelAssetPath("/Engine/BasicShapes/Cube", "Cube"));
const FSoftObjectPath UActorFactoryBasicShape::BasicSphere(FTopLevelAssetPath("/Engine/BasicShapes/Sphere", "Sphere"));
const FSoftObjectPath UActorFactoryBasicShape::BasicCylinder(FTopLevelAssetPath("/Engine/BasicShapes/Cylinder", "Cylinder"));
const FSoftObjectPath UActorFactoryBasicShape::BasicCone(FTopLevelAssetPath("/Engine/BasicShapes/Cone", "Cone"));
const FSoftObjectPath UActorFactoryBasicShape::BasicPlane(FTopLevelAssetPath("/Engine/BasicShapes/Plane", "Plane"));

UActorFactoryBasicShape::UActorFactoryBasicShape(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("UActorFactoryBasicShapeDisplayName", "Basic Shape");
	NewActorClass = AStaticMeshActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryBasicShape::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	FSoftObjectPath AssetPath = AssetData.GetSoftObjectPath();
	if(AssetData.IsValid() && (AssetPath == BasicCube || AssetPath == BasicSphere || AssetPath == BasicCone || AssetPath == BasicCylinder || AssetPath == BasicPlane) )
	{
		return true;
	}

	return false;
}

void UActorFactoryBasicShape::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	// HACK 4.24 crash fix
	// You CAN end up in this code with a redirector! so you can't CastChecked()
	// Ideally this wouldn't be possible, but unfortunately that is a much bigger refactor.
	// You can chase the redirector here and it causes this to be functional at first, BUT when you restart the editor this no longer works because the initial load chases the redirector then the CanCreateActorFrom() fails all together.
	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Asset))
	{
		Super::PostSpawnActor(Asset, NewActor);

		// Change properties
		AStaticMeshActor* StaticMeshActor = CastChecked<AStaticMeshActor>(NewActor);
		UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

		if (StaticMeshComponent)
		{
			StaticMeshComponent->UnregisterComponent();

			StaticMeshComponent->SetStaticMesh(StaticMesh);
			StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
			StaticMeshComponent->SetMaterial(0, LoadObject<UMaterial>(nullptr, TEXT("/Engine/BasicShapes/BasicShapeMaterial.BasicShapeMaterial")));
			// Init Component
			StaticMeshComponent->RegisterComponent();
		}
	}
}


/*-----------------------------------------------------------------------------
UActorFactoryDeferredDecal
-----------------------------------------------------------------------------*/
UActorFactoryDeferredDecal::UActorFactoryDeferredDecal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ 
	DisplayName = LOCTEXT("DecalDisplayName", "Decal");
	NewActorClass = ADecalActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UActorFactoryDeferredDecal::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	//We can create a DecalActor without an existing asset
	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	//But if an asset is specified it must be based-on a deferred decal umaterial
	if ( !AssetData.IsInstanceOf( UMaterialInterface::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoMaterial", "A valid material must be specified.");
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	uint32 SanityCheck = 0;
	FAssetData CurrentAssetData = AssetData;
	while( SanityCheck < 1000 && !CurrentAssetData.IsInstanceOf( UMaterial::StaticClass() ) )
	{
		const FString ObjectPath = CurrentAssetData.GetTagValueRef<FString>( "Parent" );
		if ( ObjectPath.IsEmpty() )
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoMaterial", "A valid material must be specified.");
			return false;
		}

		CurrentAssetData = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(ObjectPath) );
		if ( !CurrentAssetData.IsValid() )
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoMaterial", "A valid material must be specified.");
			return false;
		}

		++SanityCheck;
	}

	if ( SanityCheck >= 1000 )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "RecursiveParentMaterial", "The specified material must not have a recursive parent.");
		return false;
	}

	if ( !CurrentAssetData.IsInstanceOf( UMaterial::StaticClass() ) )
	{
		return false;
	}

	const FString MaterialDomain = CurrentAssetData.GetTagValueRef<FString>( "MaterialDomain" );
	if ( MaterialDomain != TEXT("MD_DeferredDecal") )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NotDecalMaterial", "Only materials with a material domain of DeferredDecal can be specified.");
		return false;
	}

	return true;
}

void UActorFactoryDeferredDecal::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UMaterialInterface* Material = GetMaterial( Asset );

	if (Material != nullptr )
	{
		// Change properties
		UDecalComponent* DecalComponent = nullptr;
		for (UActorComponent* Component : NewActor->GetComponents())
		{
			if (UDecalComponent* DecalComp = Cast<UDecalComponent>(Component))
			{
				DecalComponent = DecalComp;
				break;
			}
		}

		check(DecalComponent);

		DecalComponent->UnregisterComponent();

		DecalComponent->SetDecalMaterial(Material);

		// Init Component
		DecalComponent->RegisterComponent();
	}
}

UMaterialInterface* UActorFactoryDeferredDecal::GetMaterial( UObject* Asset ) const
{
	UMaterialInterface* TargetMaterial = Cast<UMaterialInterface>( Asset );

	return TargetMaterial 
		&& TargetMaterial->GetMaterial() 
		&& TargetMaterial->GetMaterial()->MaterialDomain == MD_DeferredDecal ? 
TargetMaterial : 
	nullptr;
}

/*-----------------------------------------------------------------------------
UActorFactoryTextRender
-----------------------------------------------------------------------------*/
UActorFactoryTextRender::UActorFactoryTextRender(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Property initialization
	DisplayName = LOCTEXT("TextRenderDisplayName", "Text Render");
	NewActorClass = ATextRenderActor::StaticClass();
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactoryEmitter
-----------------------------------------------------------------------------*/
UActorFactoryEmitter::UActorFactoryEmitter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("EmitterDisplayName", "Emitter");
	NewActorClass = AEmitter::StaticClass();
}

bool UActorFactoryEmitter::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UParticleSystem::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoParticleSystem", "A valid particle system must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryEmitter::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UParticleSystem* ParticleSystem = CastChecked<UParticleSystem>(Asset);
	AEmitter* NewEmitter = CastChecked<AEmitter>(NewActor);

	// Term Component
	NewEmitter->GetParticleSystemComponent()->UnregisterComponent();

	// Change properties
	NewEmitter->SetTemplate(ParticleSystem);

	// if we're created by Kismet on the server during gameplay, we need to replicate the emitter
	if (NewEmitter->GetWorld()->HasBegunPlay() && NewEmitter->GetWorld()->GetNetMode() != NM_Client)
	{
		NewEmitter->SetReplicates(true);
		NewEmitter->bAlwaysRelevant = true;
		NewEmitter->SetNetUpdateFrequency(0.1f); // could also set bNetTemporary but LD might further trigger it or something
		// call into gameplay code with template so it can set up replication
		NewEmitter->SetTemplate(ParticleSystem);
	}

	// Init Component
	NewEmitter->GetParticleSystemComponent()->RegisterComponent();
}

UObject* UActorFactoryEmitter::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	AEmitter* Emitter = CastChecked<AEmitter>(Instance);
	if (Emitter->GetParticleSystemComponent())
	{
		return Emitter->GetParticleSystemComponent()->Template;
	}
	else
	{
		return nullptr;
	}
}


/*-----------------------------------------------------------------------------
UActorFactoryPlayerStart
-----------------------------------------------------------------------------*/
UActorFactoryPlayerStart::UActorFactoryPlayerStart(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PlayerStartDisplayName", "Player Start");
	NewActorClass = APlayerStart::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryTargetPoint
-----------------------------------------------------------------------------*/
UActorFactoryTargetPoint::UActorFactoryTargetPoint( const FObjectInitializer& ObjectInitializer )
: Super( ObjectInitializer )
{
	DisplayName = LOCTEXT( "TargetPointDisplayName", "Target Point" );
	NewActorClass = ATargetPoint::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryNote
-----------------------------------------------------------------------------*/
UActorFactoryNote::UActorFactoryNote( const FObjectInitializer& ObjectInitializer )
: Super( ObjectInitializer )
{
	DisplayName = LOCTEXT( "NoteDisplayName", "Note" );
	NewActorClass = ANote::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryPhysicsAsset
-----------------------------------------------------------------------------*/
UActorFactoryPhysicsAsset::UActorFactoryPhysicsAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PhysicsAssetDisplayName", "Skeletal Physics");
	NewActorClass = ASkeletalMeshActor::StaticClass();
}

bool UActorFactoryPhysicsAsset::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UPhysicsAsset::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoPhysicsAsset", "A valid physics asset must be specified.");
		return false;
	}

	return true;
}

bool UActorFactoryPhysicsAsset::PreSpawnActor(UObject* Asset, FTransform& InOutLocation)
{
	UPhysicsAsset* PhysicsAsset = CastChecked<UPhysicsAsset>(Asset);
	USkeletalMesh* UseSkelMesh = PhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();

	if(!UseSkelMesh)
	{
		return false;
	}

	return true;
}

void UActorFactoryPhysicsAsset::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UPhysicsAsset* PhysicsAsset = CastChecked<UPhysicsAsset>(Asset);
	USkeletalMesh* UseSkelMesh = PhysicsAsset->PreviewSkeletalMesh.Get();

	ASkeletalMeshActor* NewSkelActor = CastChecked<ASkeletalMeshActor>(NewActor);

	// Term Component
	NewSkelActor->GetSkeletalMeshComponent()->UnregisterComponent();

	// Change properties
	NewSkelActor->GetSkeletalMeshComponent()->SetSkeletalMeshAsset(UseSkelMesh);
	if (NewSkelActor->GetWorld()->IsPlayInEditor())
	{
		NewSkelActor->ReplicatedMesh = UseSkelMesh;
		NewSkelActor->ReplicatedPhysAsset = PhysicsAsset;
	}
	NewSkelActor->GetSkeletalMeshComponent()->PhysicsAssetOverride = PhysicsAsset;

	// set physics setup
	NewSkelActor->GetSkeletalMeshComponent()->KinematicBonesUpdateType = EKinematicBonesUpdateToPhysics::SkipSimulatingBones;
	NewSkelActor->GetSkeletalMeshComponent()->BodyInstance.bSimulatePhysics = true;
	NewSkelActor->GetSkeletalMeshComponent()->bBlendPhysics = true;

	NewSkelActor->bAlwaysRelevant = true;
	NewSkelActor->SetReplicatingMovement(true);
	NewSkelActor->SetReplicates(true);

	// Init Component
	NewSkelActor->GetSkeletalMeshComponent()->RegisterComponent();
}


/*-----------------------------------------------------------------------------
UActorFactoryAnimationAsset
-----------------------------------------------------------------------------*/
UActorFactoryAnimationAsset::UActorFactoryAnimationAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("SingleAnimSkeletalDisplayName", "Single Animation Skeletal");
	NewActorClass = ASkeletalMeshActor::StaticClass();
}

bool UActorFactoryAnimationAsset::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{ 
	if ( !AssetData.IsValid() 
		|| !AssetData.GetClass()
		|| !AssetData.GetClass()->IsChildOf(UAnimSequenceBase::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoAnimData", "A valid anim data must be specified.");
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	if ( AssetData.IsInstanceOf( UAnimSequenceBase::StaticClass() ) )
	{
		const FString SkeletonPath = AssetData.GetTagValueRef<FString>("Skeleton");
		if ( SkeletonPath.IsEmpty() ) 
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSkeleton", "UAnimationAssets must have a valid Skeleton.");
			return false;
		}

		FAssetData SkeletonData = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(SkeletonPath) );

		if ( !SkeletonData.IsValid() )
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSkeleton", "UAnimationAssets must have a valid Skeleton.");
			return false;
		}

		// skeleton should be loaded by this time. If not, we have problem
		// so I'm changing this to load directly not using tags and values
		USkeleton* Skeleton = Cast<USkeleton>(SkeletonData.GetAsset());
		if (Skeleton)
		{
			USkeletalMesh * PreviewMesh = Skeleton->GetPreviewMesh(true);
			if (PreviewMesh)
			{ 
				return true;
			}
			else
			{
				OutErrorMsg = NSLOCTEXT("CanCreateActor", "UAnimationAssetNoSkeleton", "UAnimationAssets must have a valid Skeleton with a valid preview skeletal mesh.");
				return false;
			}			
		}
		else
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSkeleton", "UAnimationAssets must have a valid Skeleton.");
			return false;
		}
	}

	return true;
}

USkeletalMesh* UActorFactoryAnimationAsset::GetSkeletalMeshFromAsset( UObject* Asset )
{
	USkeletalMesh* SkeletalMesh = nullptr;
	
	if(UAnimSequenceBase* AnimationAsset = Cast<UAnimSequenceBase>(Asset))
	{
		// base it on preview skeletal mesh, just to have something
		SkeletalMesh = AnimationAsset->GetSkeleton() ? AnimationAsset->GetSkeleton()->GetAssetPreviewMesh(AnimationAsset) : nullptr;
	}
	else if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Asset))
	{
		SkeletalMesh = AnimBlueprint->TargetSkeleton ? AnimBlueprint->TargetSkeleton->GetAssetPreviewMesh(AnimBlueprint) : nullptr;
	}

	// Check to see if we are using a custom factory in which case this should probably be ignored. This seems kind of wrong...
	if( SkeletalMesh && SkeletalMesh->HasCustomActorFactory())
	{
		SkeletalMesh = nullptr;
	}

	check( SkeletalMesh != nullptr );
	return SkeletalMesh;
}

void UActorFactoryAnimationAsset::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	Super::PostSpawnActor( Asset, NewActor );
	UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(Asset);

	ASkeletalMeshActor* NewSMActor = CastChecked<ASkeletalMeshActor>(NewActor);
	USkeletalMeshComponent* NewSASComponent = (NewSMActor->GetSkeletalMeshComponent());

	if( NewSASComponent )
	{
		if( AnimationAsset )
		{
			NewSASComponent->SetAnimationMode(EAnimationMode::Type::AnimationSingleNode);
			NewSASComponent->AnimationData.AnimToPlay = AnimationAsset;
			
			// set runtime data
			NewSASComponent->SetAnimation(AnimationAsset);

			if (UAnimSequenceBase* AnimSeq = Cast<UAnimSequenceBase>(AnimationAsset))
			{
				//If we have a negative play rate, default initial position to sequence end
				if (AnimSeq->RateScale < 0.f)
				{
					NewSASComponent->AnimationData.SavedPosition = AnimSeq->GetPlayLength();
					NewSASComponent->SetPosition(AnimSeq->GetPlayLength(), false);
				}
			}
			
		}
	}
}

UObject* UActorFactoryAnimationAsset::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if (ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(ActorInstance))
	{
		if (USkeletalMeshComponent* NewSASComponent = (SkeletalMeshActor->GetSkeletalMeshComponent()))
		{
			if (NewSASComponent->GetAnimationMode() == EAnimationMode::Type::AnimationSingleNode)
			{
				return NewSASComponent->AnimationData.AnimToPlay;
			}
		}
	}
	return nullptr;
}


/*-----------------------------------------------------------------------------
UActorFactorySkeletalMesh
-----------------------------------------------------------------------------*/

// static storage
TMap<UClass*, FGetSkeletalMeshFromAssetDelegate> UActorFactorySkeletalMesh::GetSkeletalMeshDelegates;
TMap<UClass*, FPostSkeletalMeshActorSpawnedDelegate> UActorFactorySkeletalMesh::PostSkeletalMeshActorSpawnedDelegates;

UActorFactorySkeletalMesh::UActorFactorySkeletalMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{ 
	DisplayName = LOCTEXT("SkeletalMeshDisplayName", "Skeletal Mesh");
	NewActorClass = ASkeletalMeshActor::StaticClass();
	bUseSurfaceOrientation = true;
	ClassUsedForDelegate = nullptr;
}

bool UActorFactorySkeletalMesh::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	FAssetData SkeletalMeshData;

	UClass* AssetClass = AssetData.GetClass();
	if ( AssetClass && AssetClass->IsChildOf( USkeletalMesh::StaticClass() ) )
	{
		SkeletalMeshData = AssetData;
	}

	if ( !SkeletalMeshData.IsValid() && AssetClass && (AssetClass->IsChildOf(UAnimBlueprint::StaticClass()) || AssetClass->IsChildOf(UAnimBlueprintGeneratedClass::StaticClass())))
	{
		const FString TargetSkeletonPath = AssetData.GetTagValueRef<FString>( "TargetSkeleton" );
		if ( TargetSkeletonPath.IsEmpty() )
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoAnimBPTargetSkeleton", "UAnimBlueprints must have a valid Target Skeleton.");
			return false;
		}

		FAssetData TargetSkeleton = AssetRegistry.GetAssetByObjectPath( FSoftObjectPath(TargetSkeletonPath) );
		if ( !TargetSkeleton.IsValid() )
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoAnimBPTargetSkeleton", "UAnimBlueprints must have a valid Target Skeleton.");
			return false;
		}

		// skeleton should be loaded by this time. If not, we have problem
		// so I'm changing this to load directly not using tags and values
		USkeleton* Skeleton = Cast<USkeleton>(TargetSkeleton.GetAsset());
		if(Skeleton)
		{
			USkeletalMesh * PreviewMesh = Skeleton->GetPreviewMesh(true);
			if(PreviewMesh)
			{
				return true;
			}
			else
			{
				OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoPreviewSkeletalMesh", "The Target Skeleton of the UAnimBlueprint must have a valid Preview Skeletal Mesh.");
				return false;
			}
		}
		else
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoAnimBPTargetSkeleton", "UAnimBlueprints must have a valid Target Skeleton.");
		}
	}

	if ( !SkeletalMeshData.IsValid() && AssetClass && AssetClass->IsChildOf( USkeleton::StaticClass() ) )
	{
		// so I'm changing this to load directly not using tags and values
		USkeleton* Skeleton = Cast<USkeleton>(AssetData.GetAsset());
		if(Skeleton)
		{
			USkeletalMesh * PreviewMesh = Skeleton->GetPreviewMesh(true);
			if(PreviewMesh)
			{
				return true;
			}
			else
			{
				OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoPreviewSkeletalMesh", "The Target Skeleton of the UAnimBlueprint must have a valid Preview Skeletal Mesh.");
				return false;
			}
		}
	}

	if (!SkeletalMeshData.IsValid())
	{
		for (const auto& Pair : UActorFactorySkeletalMesh::GetSkeletalMeshDelegates)
		{
			if (Pair.Value.Execute(AssetData) != nullptr)
			{
				return true;
			}
		}
	}

	if ( !SkeletalMeshData.IsValid() )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSkeletalMeshAss", "No valid skeletal mesh was found associated with the animation sequence.");
		return false;
	}

	check(AssetClass);
	if (USkeletalMesh* SkeletalMeshCDO = Cast<USkeletalMesh>(AssetClass->GetDefaultObject()))
	{
		if (SkeletalMeshCDO->HasCustomActorFactory())
		{
			return false;
		}
	}

	return true;
}

USkeletalMesh* UActorFactorySkeletalMesh::GetSkeletalMeshFromAsset( UObject* Asset )
{
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>( Asset );
	USkeleton* Skeleton = Cast<USkeleton>( Asset );

	if(SkeletalMesh == nullptr)
	{
		// base it on preview skeletal mesh, just to have something
		if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Asset))
		{
			if(AnimBlueprint->TargetSkeleton)
			{
				SkeletalMesh = AnimBlueprint->TargetSkeleton->GetPreviewMesh(true);
			}
		}
		if(UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = Cast<UAnimBlueprintGeneratedClass>(Asset))
		{
			if(AnimBlueprintGeneratedClass->TargetSkeleton)
			{
				SkeletalMesh = AnimBlueprintGeneratedClass->TargetSkeleton->GetPreviewMesh(true);
			}
		}
	}

	if( SkeletalMesh == nullptr && Skeleton != nullptr )
	{
		SkeletalMesh = Skeleton->GetPreviewMesh(true);
	}

	if (SkeletalMesh == nullptr)
	{
		for (const auto& Pair : UActorFactorySkeletalMesh::GetSkeletalMeshDelegates)
		{
			if (USkeletalMesh* SkeletalMeshFromDelegate = Pair.Value.Execute(Asset))
			{
				ClassUsedForDelegate = Pair.Key;
				SkeletalMesh = SkeletalMeshFromDelegate;
				break;
			}
		}
	}

	check( SkeletalMesh != nullptr );
	return SkeletalMesh;
}

void UActorFactorySkeletalMesh::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	USkeletalMesh* SkeletalMesh = GetSkeletalMeshFromAsset(Asset);
	ASkeletalMeshActor* NewSMActor = CastChecked<ASkeletalMeshActor>(NewActor);

	Super::PostSpawnActor(SkeletalMesh, NewActor);

	// Term Component
	NewSMActor->GetSkeletalMeshComponent()->UnregisterComponent();

	// Change properties
	NewSMActor->GetSkeletalMeshComponent()->SetSkeletalMeshAsset(SkeletalMesh);
	if (NewSMActor->GetWorld()->IsGameWorld())
	{
		NewSMActor->ReplicatedMesh = SkeletalMesh;
	}

	// Init Component
	NewSMActor->GetSkeletalMeshComponent()->RegisterComponent();
	if( UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>( Asset ) )
	{
		NewSMActor->GetSkeletalMeshComponent()->SetAnimInstanceClass(AnimBlueprint->GeneratedClass);
	}
	else if( UAnimBlueprintGeneratedClass* AnimBlueprintGeneratedClass = Cast<UAnimBlueprintGeneratedClass>(Asset) )
	{
		NewSMActor->GetSkeletalMeshComponent()->SetAnimInstanceClass(AnimBlueprintGeneratedClass);
	}

	if (ClassUsedForDelegate != nullptr)
	{
		UActorFactorySkeletalMesh::PostSkeletalMeshActorSpawnedDelegates.FindChecked(ClassUsedForDelegate).Execute(NewSMActor, Asset);
	}

}

FQuat UActorFactorySkeletalMesh::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
	// Meshes align the Z (up) axis with the surface normal
	return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

void UActorFactorySkeletalMesh::RegisterDelegatesForAssetClass(
	UClass* InAssetClass,
	FGetSkeletalMeshFromAssetDelegate GetSkeletalMeshFromAssetDelegate,
	FPostSkeletalMeshActorSpawnedDelegate PostSkeletalMeshActorSpawnedDelegate
)
{
	UActorFactorySkeletalMesh::GetSkeletalMeshDelegates.Add(InAssetClass, GetSkeletalMeshFromAssetDelegate);
	UActorFactorySkeletalMesh::PostSkeletalMeshActorSpawnedDelegates.Add(InAssetClass, PostSkeletalMeshActorSpawnedDelegate);
}

void UActorFactorySkeletalMesh::UnregisterDelegatesForAssetClass(UClass* InAssetClass)
{
	UActorFactorySkeletalMesh::GetSkeletalMeshDelegates.Remove(InAssetClass);
	UActorFactorySkeletalMesh::PostSkeletalMeshActorSpawnedDelegates.Remove(InAssetClass);
}


/*-----------------------------------------------------------------------------
UActorFactoryCameraActor
-----------------------------------------------------------------------------*/
UActorFactoryCameraActor::UActorFactoryCameraActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("CameraDisplayName", "Camera");
	NewActorClass = ACameraActor::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryEmptyActor
-----------------------------------------------------------------------------*/
UActorFactoryEmptyActor::UActorFactoryEmptyActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ActorFactoryEmptyActorDisplayName", "Empty Actor");
	NewActorClass = AActor::StaticClass();
	bVisualizeActor = true;
}

bool UActorFactoryEmptyActor::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	return AssetData.ToSoftObjectPath() == FSoftObjectPath(AActor::StaticClass());
}

AActor* UActorFactoryEmptyActor::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AActor* NewActor = nullptr;
	{
		// Spawn a temporary actor for dragging around
		NewActor = Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams);

		USceneComponent* RootComponent = NewObject<USceneComponent>(NewActor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
		RootComponent->Mobility = EComponentMobility::Movable;
		RootComponent->bVisualizeComponent = bVisualizeActor;
		RootComponent->SetWorldTransform(InTransform);

		NewActor->SetRootComponent(RootComponent);
		NewActor->AddInstanceComponent(RootComponent);

		RootComponent->RegisterComponent();
	}

	return NewActor;
}



/*-----------------------------------------------------------------------------
UActorFactoryCharacter
-----------------------------------------------------------------------------*/
UActorFactoryCharacter::UActorFactoryCharacter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ActorFactoryCharacterDisplayName", "Empty Character");
	NewActorClass = ACharacter::StaticClass();
}

bool UActorFactoryCharacter::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	return AssetData.ToSoftObjectPath() == FSoftObjectPath(ACharacter::StaticClass());
}

/*-----------------------------------------------------------------------------
UActorFactoryPawn
-----------------------------------------------------------------------------*/
UActorFactoryPawn::UActorFactoryPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ActorFactoryPawnDisplayName", "Empty Pawn");
	NewActorClass = APawn::StaticClass();
}

bool UActorFactoryPawn::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	return AssetData.GetSoftObjectPath() == FSoftObjectPath(APawn::StaticClass());
}

/*-----------------------------------------------------------------------------
UActorFactoryAmbientSound
-----------------------------------------------------------------------------*/
UActorFactoryAmbientSound::UActorFactoryAmbientSound(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("AmbientSoundDisplayName", "Ambient Sound");
	NewActorClass = AAmbientSound::StaticClass();
}

bool UActorFactoryAmbientSound::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{

	if(!CanImportAmbientSounds())
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "AssetNotAllowed", "Ambient Sound Actors are disabled in this environment.");
		return false;
	}

	//We allow creating AAmbientSounds without an existing sound asset
	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf( USoundBase::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoSoundAsset", "A valid sound asset must be specified.");
		return false;
	}

	return true;
}

void UActorFactoryAmbientSound::PostSpawnActor( UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	USoundBase* AmbientSound = Cast<USoundBase>(Asset);

	if ( AmbientSound != nullptr )
	{
		AAmbientSound* NewSound = CastChecked<AAmbientSound>( NewActor );
		NewSound->GetAudioComponent()->SetSound(AmbientSound);
	}
}

UObject* UActorFactoryAmbientSound::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	AAmbientSound* SoundActor = CastChecked<AAmbientSound>(Instance);

	check(SoundActor->GetAudioComponent());
	return SoundActor->GetAudioComponent()->Sound;
}

bool UActorFactoryAmbientSound::CanImportAmbientSounds()
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
	if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		if (!AssetClassPermissionList->PassesFilter(AAmbientSound::StaticClass()->GetPathName()))
		{
			return false;
		}
	}

	return true;
}

/*-----------------------------------------------------------------------------
UActorFactoryClass
-----------------------------------------------------------------------------*/
UActorFactoryClass::UActorFactoryClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ClassDisplayName", "Class");
}

bool UActorFactoryClass::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( AssetData.IsValid() && AssetData.IsInstanceOf( UClass::StaticClass() ) )
	{
		UClass* ActualClass = Cast<UClass>(AssetData.GetAsset());
		if ( (nullptr != ActualClass) && ActualClass->IsChildOf(AActor::StaticClass()) )
		{
			return true;
		}
	}

	OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoClass", "The specified Blueprint must be Actor based.");
	return false;
}

AActor* UActorFactoryClass::GetDefaultActor( const FAssetData& AssetData )
{
	if ( AssetData.IsValid() && AssetData.IsInstanceOf( UClass::StaticClass() ) )
	{
		UClass* ActualClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, *AssetData.GetObjectPathString(), nullptr, LOAD_NoWarn, nullptr));
			
		//Cast<UClass>(AssetData.GetAsset());
		if ( (nullptr != ActualClass) && ActualClass->IsChildOf(AActor::StaticClass()) )
		{
			return ActualClass->GetDefaultObject<AActor>();
		}
	}

	return nullptr;
}

bool UActorFactoryClass::PreSpawnActor( UObject* Asset, FTransform& InOutLocation)
{
	UClass* ActualClass = Cast<UClass>(Asset);

	if ( (nullptr != ActualClass) && ActualClass->IsChildOf(AActor::StaticClass()) )
	{
		return Super::PreSpawnActor(Asset, InOutLocation);
	}

	return false;
}

AActor* UActorFactoryClass::SpawnActor( UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams)
{
	UClass* ActualClass = Cast<UClass>(Asset);

	if ( (nullptr != ActualClass) && ActualClass->IsChildOf(AActor::StaticClass()) )
	{
		FActorSpawnParameters SpawnInfo(InSpawnParams);
		SpawnInfo.OverrideLevel = InLevel;
		return InLevel->OwningWorld->SpawnActor(ActualClass, &Transform, SpawnInfo);
	}

	return nullptr;
}


/*-----------------------------------------------------------------------------
UActorFactoryBlueprint
-----------------------------------------------------------------------------*/
UActorFactoryBlueprint::UActorFactoryBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("BlueprintDisplayName", "Blueprint");
}

bool UActorFactoryBlueprint::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UBlueprint::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoBlueprint", "No Blueprint was specified, or the specified Blueprint needs to be compiled.");
		return false;
	}

	const FString ParentClassPath = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
	if ( ParentClassPath.IsEmpty() )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoBlueprint", "No Blueprint was specified, or the specified Blueprint needs to be compiled.");
		return false;
	}

	UClass* ParentClass = FindObject<UClass>(nullptr, *ParentClassPath);

	bool bIsActorBased = false;
	if ( ParentClass != nullptr )
	{
		// The parent class is loaded. Make sure it is derived from AActor
		bIsActorBased = ParentClass->IsChildOf(AActor::StaticClass());
	}
	else
	{
		// The parent class does not exist or is not loaded.
		// Ask the asset registry for the ancestors of this class to see if it is an unloaded blueprint generated class.
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		const FTopLevelAssetPath ObjectPath(FPackageName::ExportTextPathToObjectPath(ParentClassPath));
		const FName ParentClassPathFName = ObjectPath.GetAssetName();
		TArray<FTopLevelAssetPath> AncestorClassNames;
		AssetRegistry.GetAncestorClassNames(ObjectPath, AncestorClassNames);

		bIsActorBased = AncestorClassNames.Contains(AActor::StaticClass()->GetClassPathName());
	}

	if ( !bIsActorBased )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NotActor", "The specified Blueprint must be Actor based.");
		return false;
	}

	return true;
}

AActor* UActorFactoryBlueprint::GetDefaultActor( const FAssetData& AssetData )
{
	UClass* GeneratedClass = GetDefaultActorClass(AssetData);
	return GeneratedClass ? GeneratedClass->GetDefaultObject<AActor>() : nullptr;
}

UClass* UActorFactoryBlueprint::GetDefaultActorClass(const FAssetData& AssetData)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UBlueprint::StaticClass()))
	{
		return nullptr;
	}

	const FString GeneratedClassPath = AssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
	if (GeneratedClassPath.IsEmpty())
	{
		return nullptr;
	}

	UClass* GeneratedClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, *GeneratedClassPath, nullptr, LOAD_NoWarn, nullptr));

	if (GeneratedClass == nullptr)
	{
		return nullptr;
	}

	return GeneratedClass;
}

bool UActorFactoryBlueprint::PreSpawnActor( UObject* Asset, FTransform& InOutLocation)
{
	UBlueprint* Blueprint = CastChecked<UBlueprint>(Asset);

	// Invalid if there is no generated class, or this is not actor based
	if (Blueprint == nullptr || Blueprint->GeneratedClass == nullptr || !FBlueprintEditorUtils::IsActorBased(Blueprint))
	{
		return false;
	}

	UE_LOGF(LogActorFactory, Log, "Actor Factory attempting to spawn %ls", *Blueprint->GeneratedClass->GetFullName());

	return true;
}

void UActorFactoryBlueprint::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	AVolume* VolumeActor = Cast<AVolume>(NewActor);
	if ( VolumeActor != nullptr )
	{
		const UBrushBuilder* ActorBrushBuilder = VolumeActor->GetBrushBuilder();
		UBrushBuilder* BrushBuilder = ActorBrushBuilder ? DuplicateObject<UBrushBuilder>(ActorBrushBuilder, this) : NewObject<UCubeBuilder>();
		CreateBrushForVolumeActor( VolumeActor, BrushBuilder );
	}
}

/*-----------------------------------------------------------------------------
UActorFactoryDirectionalLight
-----------------------------------------------------------------------------*/
UActorFactoryDirectionalLight::UActorFactoryDirectionalLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("DirectionalLightDisplayName", "Directional Light");
	NewActorClass = ADirectionalLight::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactorySpotLight
-----------------------------------------------------------------------------*/
UActorFactorySpotLight::UActorFactorySpotLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("SpotLightDisplayName", "Spot Light");
	NewActorClass = ASpotLight::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactoryPointLight
-----------------------------------------------------------------------------*/
UActorFactoryPointLight::UActorFactoryPointLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PointLightDisplayName", "Point Light");
	NewActorClass = APointLight::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactoryRectLight
-----------------------------------------------------------------------------*/
UActorFactoryRectLight::UActorFactoryRectLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("RectLightDisplayName", "Rect Light");
	NewActorClass = ARectLight::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactorySkyLight
-----------------------------------------------------------------------------*/
UActorFactorySkyLight::UActorFactorySkyLight( const FObjectInitializer& ObjectInitializer )
: Super( ObjectInitializer )
{
	DisplayName = LOCTEXT( "SkyLightDisplayName", "Sky Light" );
	NewActorClass = ASkyLight::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactorySphereReflectionCapture
-----------------------------------------------------------------------------*/
UActorFactorySphereReflectionCapture::UActorFactorySphereReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ReflectionCaptureSphereDisplayName", "Sphere Reflection Capture");
	NewActorClass = ASphereReflectionCapture::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactoryBoxReflectionCapture
-----------------------------------------------------------------------------*/
UActorFactoryBoxReflectionCapture::UActorFactoryBoxReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ReflectionCaptureBoxDisplayName", "Box Reflection Capture");
	NewActorClass = ABoxReflectionCapture::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactoryPlanarReflection
-----------------------------------------------------------------------------*/
UActorFactoryPlanarReflection::UActorFactoryPlanarReflection(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("PlanarReflectionDisplayName", "Planar Reflection");
	NewActorClass = APlanarReflection::StaticClass();
	SpawnPositionOffset = FVector(0, 0, 0);
	bUseSurfaceOrientation = false;
}

/*-----------------------------------------------------------------------------
UActorFactoryPlaneReflectionCapture
-----------------------------------------------------------------------------*/
UActorFactoryPlaneReflectionCapture::UActorFactoryPlaneReflectionCapture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ReflectionCapturePlaneDisplayName", "Plane Reflection Capture");
	NewActorClass = APlaneReflectionCapture::StaticClass();
	SpawnPositionOffset = FVector(50, 0, 0);
	bUseSurfaceOrientation = true;
}

/*-----------------------------------------------------------------------------
UActorFactorySkyAtmosphere
-----------------------------------------------------------------------------*/
UActorFactorySkyAtmosphere::UActorFactorySkyAtmosphere(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("SkyAtmosphereDisplayName", "Sky Atmosphere");
	NewActorClass = ASkyAtmosphere::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryVolumetricCloud
-----------------------------------------------------------------------------*/
UActorFactoryVolumetricCloud::UActorFactoryVolumetricCloud(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("VolumetricCloudDisplayName", "Volumetric Cloud");
	NewActorClass = AVolumetricCloud::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryExponentialHeightFog
-----------------------------------------------------------------------------*/
UActorFactoryExponentialHeightFog::UActorFactoryExponentialHeightFog(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ExponentialHeightFogDisplayName", "Exponential Height Fog");
	NewActorClass = AExponentialHeightFog::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryLocalFogVolume
-----------------------------------------------------------------------------*/
UActorFactoryLocalFogVolume::UActorFactoryLocalFogVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("LocalFogVolumeDisplayName", "Local Height Fog");
	NewActorClass = ALocalFogVolume::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryInteractiveFoliage
-----------------------------------------------------------------------------*/
UActorFactoryInteractiveFoliage::UActorFactoryInteractiveFoliage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("InteractiveFoliageDisplayName", "Interactive Foliage");
	NewActorClass = AInteractiveFoliageActor::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryTriggerBox
-----------------------------------------------------------------------------*/
UActorFactoryTriggerBox::UActorFactoryTriggerBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("TriggerBoxDisplayName", "Box Trigger");
	NewActorClass = ATriggerBox::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryTriggerCapsule
-----------------------------------------------------------------------------*/
UActorFactoryTriggerCapsule::UActorFactoryTriggerCapsule(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("TriggerCapsuleDisplayName", "Capsule Trigger");
	NewActorClass = ATriggerCapsule::StaticClass();
}

/*-----------------------------------------------------------------------------
UActorFactoryTriggerSphere
-----------------------------------------------------------------------------*/
UActorFactoryTriggerSphere::UActorFactoryTriggerSphere(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("TriggerSphereDisplayName", "Sphere Trigger");
	NewActorClass = ATriggerSphere::StaticClass();
}


/*-----------------------------------------------------------------------------
UActorFactoryVectorField
-----------------------------------------------------------------------------*/
UActorFactoryVectorFieldVolume::UActorFactoryVectorFieldVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("VectorFieldVolumeDisplayName", "Vector Field Volume");
	NewActorClass = AVectorFieldVolume::StaticClass();
}

bool UActorFactoryVectorFieldVolume::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( !AssetData.IsValid() || !AssetData.IsInstanceOf( UVectorField::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoVectorField", "No vector field was specified.");
		return false;
	}

	return true;
}

void UActorFactoryVectorFieldVolume::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	Super::PostSpawnActor(Asset, NewActor);

	UVectorField* VectorField = CastChecked<UVectorField>(Asset);
	AVectorFieldVolume* VectorFieldVolumeActor = CastChecked<AVectorFieldVolume>(NewActor);

	if ( VectorFieldVolumeActor && VectorFieldVolumeActor->GetVectorFieldComponent() )
	{
		VectorFieldVolumeActor->GetVectorFieldComponent()->VectorField = VectorField;
		VectorFieldVolumeActor->PostEditChange();
	}
}

/*-----------------------------------------------------------------------------
CreateBrushForVolumeActor
-----------------------------------------------------------------------------*/
// Helper function for the volume actor factories
void UActorFactory::CreateBrushForVolumeActor( AVolume* NewActor, UBrushBuilder* BrushBuilder )
{
	if ( NewActor != nullptr )
	{
		// this code builds a brush for the new actor
		NewActor->PreEditChange(nullptr);

		// Use the same object flags as the owner volume
		EObjectFlags ObjectFlags = NewActor->GetFlags() & (RF_Transient | RF_Transactional);

		NewActor->PolyFlags = 0;
		NewActor->Brush = NewObject<UModel>(NewActor, NAME_None, ObjectFlags);
		NewActor->Brush->Initialize(nullptr, true);
		NewActor->Brush->Polys = NewObject<UPolys>(NewActor->Brush, NAME_None, ObjectFlags);
		NewActor->GetBrushComponent()->Brush = NewActor->Brush;
		if(BrushBuilder != nullptr)
		{
			NewActor->BrushBuilder = DuplicateObject<UBrushBuilder>(BrushBuilder, NewActor);
		}

		BrushBuilder->Build( NewActor->GetWorld(), NewActor );

		FBSPOps::csgPrepMovingBrush( NewActor );

		// Set the texture on all polys to nullptr.  This stops invisible textures
		// dependencies from being formed on volumes.
		if ( NewActor->Brush )
		{
			for ( int32 poly = 0 ; poly < NewActor->Brush->Polys->Element.Num() ; ++poly )
			{
				FPoly* Poly = &(NewActor->Brush->Polys->Element[poly]);
				Poly->Material = nullptr;
			}
		}

		NewActor->PostEditChange();
	}
}

/*-----------------------------------------------------------------------------
UActorFactoryBoxVolume
-----------------------------------------------------------------------------*/
UActorFactoryBoxVolume::UActorFactoryBoxVolume( const FObjectInitializer& ObjectInitializer )
: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT( "BoxVolumeDisplayName", "Box Volume" );
	NewActorClass = AVolume::StaticClass();
}

bool UActorFactoryBoxVolume::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf( AVolume::StaticClass() ) )
	{
		return false;
	}

	return true;
}

void UActorFactoryBoxVolume::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	Super::PostSpawnActor(Asset, NewActor);

	AVolume* VolumeActor = CastChecked<AVolume>(NewActor);
	if ( VolumeActor != nullptr )
	{
		UCubeBuilder* Builder = NewObject<UCubeBuilder>();
		CreateBrushForVolumeActor( VolumeActor, Builder );
	}
}

/*-----------------------------------------------------------------------------
UActorFactorySphereVolume
-----------------------------------------------------------------------------*/
UActorFactorySphereVolume::UActorFactorySphereVolume( const FObjectInitializer& ObjectInitializer )
: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT( "SphereVolumeDisplayName", "Sphere Volume" );
	NewActorClass = AVolume::StaticClass();
}

bool UActorFactorySphereVolume::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf( AVolume::StaticClass() ) )
	{
		return false;
	}

	return true;
}

void UActorFactorySphereVolume::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	Super::PostSpawnActor(Asset, NewActor);

	AVolume* VolumeActor = CastChecked<AVolume>(NewActor);
	if ( VolumeActor != nullptr )
	{
		UTetrahedronBuilder* Builder = NewObject<UTetrahedronBuilder>();
		Builder->SphereExtrapolation = 2;
		Builder->Radius = 192.0f;
		CreateBrushForVolumeActor( VolumeActor, Builder );
	}
}

/*-----------------------------------------------------------------------------
UActorFactoryCylinderVolume
-----------------------------------------------------------------------------*/
UActorFactoryCylinderVolume::UActorFactoryCylinderVolume( const FObjectInitializer& ObjectInitializer )
: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT( "CylinderVolumeDisplayName", "Cylinder Volume" );
	NewActorClass = AVolume::StaticClass();
}
void UActorFactoryCylinderVolume::PostSpawnActor( UObject* Asset, AActor* NewActor )
{
	Super::PostSpawnActor(Asset, NewActor);

	AVolume* VolumeActor = CastChecked<AVolume>(NewActor);
	if ( VolumeActor != nullptr )
	{
		UCylinderBuilder* Builder = NewObject<UCylinderBuilder>();
		Builder->OuterRadius = 128.0f;
		CreateBrushForVolumeActor( VolumeActor, Builder );
	}
}

/*-----------------------------------------------------------------------------
UActorFactoryLevelSequence
-----------------------------------------------------------------------------*/
UActorFactoryLevelSequence::UActorFactoryLevelSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("LevelSequenceDisplayName", "Level Sequence");
	NewActorClass = ALevelSequenceActor::StaticClass();
}

bool UActorFactoryLevelSequence::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
	TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
	FClassViewerInitializationOptions ClassViewerOptions = {};

	if (GlobalClassFilter.IsValid())
	{
		if (!GlobalClassFilter->IsClassAllowed(ClassViewerOptions, ALevelSequenceActor::StaticClass(), ClassFilterFuncs))
		{
			return false;
		}
	}

	if ( UActorFactory::CanCreateActorFrom( AssetData, OutErrorMsg ) )
	{
		return true;
	}

	if ( AssetData.IsValid() && !AssetData.IsInstanceOf( ULevelSequence::StaticClass() ) )
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoLevelSequenceAsset", "A valid sequencer asset must be specified.");
		return false;
	}

	return true;
}

AActor* UActorFactoryLevelSequence::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	ALevelSequenceActor* NewActor = Cast<ALevelSequenceActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));

	if (NewActor)
	{
		if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(InAsset))
		{
			NewActor->SetSequence(LevelSequence);
		}
	}

	return NewActor;
}

UObject* UActorFactoryLevelSequence::GetAssetFromActorInstance(AActor* Instance)
{
	if (ALevelSequenceActor* LevelSequenceActor = Cast<ALevelSequenceActor>(Instance))
	{
		return LevelSequenceActor->GetSequence();
	}

	return nullptr;
}

/*-----------------------------------------------------------------------------
UActorFactoryRuntimeVirtualTextureVolume
-----------------------------------------------------------------------------*/
UActorFactoryRuntimeVirtualTextureVolume::UActorFactoryRuntimeVirtualTextureVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("VirtualTextureVolume_DisplayName", "Runtime Virtual Texture Volume");
	NewActorClass = ARuntimeVirtualTextureVolume::StaticClass();
	bShowInEditorQuickMenu = 1;
}

void UActorFactoryRuntimeVirtualTextureVolume::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	FText ActorName = LOCTEXT("VirtualTextureVolume_DefaultActorName", "Runtime Virtual Texture Volume");
	NewActor->SetActorLabel(ActorName.ToString());

	// Good default size to see object in editor
	NewActor->SetActorScale3D(FVector(100.f, 100.f, 1.f));

	Super::PostSpawnActor(Asset, NewActor);
}

#undef LOCTEXT_NAMESPACE
