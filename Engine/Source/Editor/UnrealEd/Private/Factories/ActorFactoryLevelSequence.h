// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryLevelSequence.generated.h"

class AActor;
struct FActorSpawnParameters;
struct FAssetData;
class ULevel;

UCLASS()
class UActorFactoryLevelSequence : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	// Begin UActorFactory Interface
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	// End UActorFactory Interface
};



