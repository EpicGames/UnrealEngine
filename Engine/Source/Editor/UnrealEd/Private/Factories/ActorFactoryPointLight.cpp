// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompositeFactory.cpp: Factory for AnimComposite
=============================================================================*/

#include "ActorFactories/ActorFactoryPointLight.h"
#include "GameFramework/Actor.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryPointLight)

void UActorFactoryPointLight::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	// Make all spawned actors use the candela units.
	TArray<UPointLightComponent*> PointLightComponents;
	NewActor->GetComponents(PointLightComponents);
	for (UPointLightComponent* Component : PointLightComponents)
	{
		if (Component && Component->CreationMethod == EComponentCreationMethod::Native)
		{
			static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
			ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnAnyThread();

			Component->Intensity *= UPointLightComponent::GetUnitsConversionFactor(Component->IntensityUnits, DefaultUnits, -1.f);
			Component->IntensityUnits = DefaultUnits;
		}
	}
}

