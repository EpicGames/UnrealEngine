// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ActorFactoryRectLight.cpp: Factory for RectLight
=============================================================================*/

#include "ActorFactories/ActorFactoryRectLight.h"
#include "GameFramework/Actor.h"
#include "Components/RectLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryRectLight)

void UActorFactoryRectLight::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	// Make all spawned actors use the candela units.
	TArray<URectLightComponent*> RectLightComponents;
	NewActor->GetComponents(RectLightComponents);
	for (URectLightComponent* Component : RectLightComponents)
	{
		if (Component && Component->CreationMethod == EComponentCreationMethod::Native)
		{
			static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
			ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnAnyThread();

			// Passing .5 as CosHalfConeAngle  since URectLightComponent::SetLightBrightness() lumens conversion use only PI
			Component->Intensity *= URectLightComponent::GetUnitsConversionFactor(Component->IntensityUnits, DefaultUnits, .5f);
			Component->IntensityUnits = DefaultUnits;
		}
	}
}

