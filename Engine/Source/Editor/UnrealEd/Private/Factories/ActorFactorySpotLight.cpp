// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ActorFactorySpotLight.cpp: Factory for SpotLight
=============================================================================*/

#include "ActorFactories/ActorFactorySpotLight.h"
#include "GameFramework/Actor.h"
#include "Components/SpotLightComponent.h"
#include "Components/RectLightComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactorySpotLight)

void UActorFactorySpotLight::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	// Make all spawned actors use the candela units.
	TArray<USpotLightComponent*> SpotLightComponents;
	NewActor->GetComponents(SpotLightComponents);
	for (USpotLightComponent* Component : SpotLightComponents)
	{
		if (Component && Component->CreationMethod == EComponentCreationMethod::Native)
		{
			static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
			ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnAnyThread();

			Component->Intensity *= UPointLightComponent::GetUnitsConversionFactor(Component->IntensityUnits, DefaultUnits, Component->GetCosHalfConeAngle());
			Component->IntensityUnits = DefaultUnits;
		}
	}
}

