// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/AnimBankFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimBank.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBankFactory)

#define LOCTEXT_NAMESPACE "AnimBankFactory"

UAnimBankFactory::UAnimBankFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimBank::StaticClass();
}

bool UAnimBankFactory::ConfigureProperties()
{
	// Null the skeleton so we can check for selection later
	TargetSkeleton = nullptr;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UAnimBankFactory::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateAnimBankOptions", "Pick Skeleton"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != nullptr;
}

UObject* UAnimBankFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAnimBank* AnimBank = NewObject<UAnimBank>(InParent, Class, Name, Flags);
	
	if (TargetSkeleton)
	{
		AnimBank->SetSkeleton(TargetSkeleton);
	}

	if (PreviewSkeletalMesh)
	{
		AnimBank->SetPreviewMesh(PreviewSkeletalMesh);
	}

	return AnimBank;
}

void UAnimBankFactory::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

UAnimBankDataFactory::UAnimBankDataFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SupportedClass = UAnimBankData::StaticClass();
	ProviderDataClass = UAnimBankData::StaticClass();
}

bool UAnimBankDataFactory::ConfigureProperties()
{
	return true;
}

#undef LOCTEXT_NAMESPACE
