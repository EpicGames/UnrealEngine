// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimBlueprintFactory.cpp: Factory for Anim Blueprints
=============================================================================*/

#include "Factories/AnimBlueprintFactory.h"
#include "InputCoreTypes.h"
#include "UObject/Interface.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimInstance.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Animation/AnimLayerInterface.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBlueprintFactory)

#define LOCTEXT_NAMESPACE "AnimBlueprintFactory"

static bool CanCreateAnimBlueprint(const FAssetData& Skeleton, UClass const * ParentClass)
{
	if (Skeleton.IsValid() && ParentClass != nullptr)
	{
		if (UAnimBlueprintGeneratedClass const * GeneratedParent = Cast<const UAnimBlueprintGeneratedClass>(ParentClass))
		{
			if (GeneratedParent->GetTargetSkeleton() != nullptr && Skeleton.GetExportTextName() != FAssetData(GeneratedParent->GetTargetSkeleton()).GetExportTextName())
			{
				return false;
			}
		}
	}
	return true;
}

/*------------------------------------------------------------------------------
	Dialog to configure creation properties
------------------------------------------------------------------------------*/
class SAnimBlueprintCreateDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimBlueprintCreateDialog ){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs )
	{
		bOkClicked = false;
		ParentClass = UAnimInstance::StaticClass();

		const FText TemplateDesc = LOCTEXT("TemplateDesc", "A Template Animation Blueprint has no target skeleton.\nTemplates cannot have direct references to animation assets placed inside of their animation graphs."); 
		
		ChildSlot
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("ChildWindow.Background"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10.0f)
					[
						SNew(SBox)
						.Visibility(EVisibility::Visible)
						.WidthOverride(400.0f)
						.HeightOverride(400.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NewAnimBlueprintDialog.AreaBorder"))
							[	
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 5.0f)
								[
									SNew(SSegmentedControl<bool>)
									.UniformPadding(FMargin(25.0f,5.0f))
									.OnValueChanged_Lambda([this](bool bInNewValue)
									{
										bTemplate = bInNewValue;
										RefreshSkeletonPicker();
									})
									.Value_Lambda([this](){ return bTemplate; })	
									+SSegmentedControl<bool>::Slot(false)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("SpecificSkeleton", "Specific Skeleton"))
										.TextStyle( FAppStyle::Get(), "NormalText" )
										.ToolTipText(LOCTEXT("SpecificSkeletonTooltip", "Choose a specific skeleton to bind your new Animation Blueprint to. The Blueprint will be able to use assets that are compatible with this skeleton."))
									]
									+SSegmentedControl<bool>::Slot(true)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("Template", "Template"))
										.TextStyle(FAppStyle::Get(), "NormalText")
										.ToolTipText(TemplateDesc)
									]
								]
								+SVerticalBox::Slot()
								.FillHeight(1.0f)
								[
									SNew(SWidgetSwitcher)
									.WidgetIndex_Lambda([this](){ return bTemplate ? 1 : 0; })
									+SWidgetSwitcher::Slot()
									[
										MakeSkeletonPickerArea()
									]
									+SWidgetSwitcher::Slot()
									.Padding(5.0f)
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.Justification(ETextJustify::Center)
										.AutoWrapText(true)
										.Text(TemplateDesc)
										.TextStyle(FAppStyle::Get(), "NormalText")
									]
								]
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(10.0f, 0.0f, 10.0f, 0.0f)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NewAnimBlueprintDialog.AreaBorder"))
						[
							MakeParentClassPicker()
						]
					]
				]

				// Ok/Cancel buttons
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(10.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton)
						.ToolTipText(LOCTEXT("CreateAnimBlueprintCreate_Tooltip", "Create a new animation Blueprint.\nSelect a target skeleton or whether the animation Blueprint should be a template.\nOptionally select a parent class."))
						.IsEnabled_Lambda([this]()
						{
							return ParentClass.Get() != nullptr && (bTemplate || TargetSkeleton.IsValid());
						})
						.HAlign(HAlign_Center)
						.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
						.OnClicked(this, &SAnimBlueprintCreateDialog::OkClicked)
						.Text(LOCTEXT("CreateAnimBlueprintCreate", "Create"))
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
						.OnClicked(this, &SAnimBlueprintCreateDialog::CancelClicked)
						.Text(LOCTEXT("CreateAnimBlueprintCancel", "Cancel"))
					]
				]
			]
		];
	}
	
	/** Sets properties for the supplied AnimBlueprintFactory */
	bool ConfigureProperties(TWeakObjectPtr<UAnimBlueprintFactory> InAnimBlueprintFactory)
	{
		AnimBlueprintFactory = InAnimBlueprintFactory;

		TSharedRef<SWindow> Window = SNew(SWindow)
		.Title( LOCTEXT("CreateAnimBlueprintOptions", "Create Animation Blueprint") )
		.SizingRule(ESizingRule::Autosized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			AsShared()
		];

		PickerWindow = Window;

		GEditor->EditorAddModalWindow(Window);
		AnimBlueprintFactory.Reset();

		return bOkClicked;
	}

private:
	class FAnimBlueprintParentFilter : public IClassViewerFilter
	{
	public:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet< const UClass* > AllowedChildrenOfClasses;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
		{
			// If it appears on the allowed child-of classes list (or there is nothing on that list)
			return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			// If it appears on the allowed child-of classes list (or there is nothing on that list)
			return InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
		}
	};

	/** Creates the combo menu for the parent class */
	TSharedRef<SWidget> MakeParentClassPicker()
	{
		// Load the classviewer module to display a class picker
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		// Fill in options
		FClassViewerInitializationOptions Options;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.DisplayMode = EClassViewerDisplayMode::TreeView;
		Options.InitiallySelectedClass = UAnimInstance::StaticClass();
		
		// Only allow parenting to base blueprints.
		Options.bIsBlueprintBaseOnly = true;

		TSharedPtr<FAnimBlueprintParentFilter> Filter = MakeShared<FAnimBlueprintParentFilter>();
		Options.ClassFilters.Add(Filter.ToSharedRef());

		// All child child classes of UAnimInstance are valid.
		Filter->AllowedChildrenOfClasses.Add(UAnimInstance::StaticClass());

		return
			SNew(SExpandableArea)
			.ToolTipText(LOCTEXT("ParentClass_Tooltip", "Optionally choose a parent class for your Animation Blueprint"))
			.Padding(10.0f)
			.InitiallyCollapsed(true)
			.MaxHeight(200)
			.HeaderContent()
			[
				SNew(SBox)
				.Padding(5.0f)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return FText::Format(LOCTEXT("ParentClassFormat", "Parent Class: {0}"), FText::FromString(ParentClass->GetName()));
					})
					.TextStyle( FAppStyle::Get(), "NormalText" )
				]
			]
			.BodyContent()
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SAnimBlueprintCreateDialog::OnClassPicked))
			];
	}

	/** Handler for when a parent class is selected */
	void OnClassPicked(UClass* ChosenClass)
	{
		ParentClass = ChosenClass;
		RefreshSkeletonPicker();
	}

	void RefreshSkeletonPicker()
	{
		RefreshSkeletonViewDelegate.ExecuteIfBound(true);
	}
	
	/** Creates the widgets for the target skeleton area */
	TSharedRef<SWidget> MakeSkeletonPickerArea()
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshSkeletonViewDelegate);
		AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAnimBlueprintCreateDialog::OnSkeletonSelected);
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SAnimBlueprintCreateDialog::FilterSkeletonBasedOnParentClass);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.InitialAssetSelection = TargetSkeleton;
		AssetPickerConfig.HiddenColumnNames =
		{
			"DiskSize",
			"AdditionalPreviewSkeletalMeshes",
			"PreviewSkeletalMesh"
		};
		AssetPickerConfig.bShowPathInColumnView = false;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
	
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(5.0f)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			];
	}

	bool FilterSkeletonBasedOnParentClass(const FAssetData& AssetData)
	{
		return ! CanCreateAnimBlueprint(AssetData, ParentClass.Get());
	}

	/** Handler for when a skeleton is selected */
	void OnSkeletonSelected(const FAssetData& AssetData)
	{
		TargetSkeleton = AssetData;
	}

	/** Handler for when ok is clicked */
	FReply OkClicked()
	{
		if (bTemplate)
		{
			TargetSkeleton = FAssetData();
		}

		if ( AnimBlueprintFactory.IsValid() )
		{
			AnimBlueprintFactory->BlueprintType = BPTYPE_Normal;
			AnimBlueprintFactory->ParentClass = ParentClass.Get();
			AnimBlueprintFactory->bTemplate = bTemplate;
			AnimBlueprintFactory->TargetSkeleton = Cast<USkeleton>(TargetSkeleton.GetAsset());
		}

		if (! CanCreateAnimBlueprint(TargetSkeleton, ParentClass.Get()))
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("NeedCompatibleSkeleton", "Selected skeleton has to be compatible with selected parent class."));
			return FReply::Handled();
		}
		CloseDialog(true);

		return FReply::Handled();
	}

	void CloseDialog(bool bWasPicked=false)
	{
		bOkClicked = bWasPicked;
		if ( PickerWindow.IsValid() )
		{
			PickerWindow.Pin()->RequestDestroyWindow();
		}
	}

	/** Handler for when cancel is clicked */
	FReply CancelClicked()
	{
		CloseDialog();
		return FReply::Handled();
	}

	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			CloseDialog();
			return FReply::Handled();
		}
		return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

private:
	/** The factory for which we are setting up properties */
	TWeakObjectPtr<UAnimBlueprintFactory> AnimBlueprintFactory;

	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> PickerWindow;

	/** The container for the Parent Class picker */
	TSharedPtr<SVerticalBox> ParentClassContainer;

	/** The container for the target skeleton picker*/
	TSharedPtr<SVerticalBox> SkeletonContainer;

	/** The selected class */
	TWeakObjectPtr<UClass> ParentClass;

	/** The selected skeleton */
	FAssetData TargetSkeleton;

	/** Delegate called to refresh the skeleton view */
	FRefreshAssetViewDelegate RefreshSkeletonViewDelegate;
	
	/** Whether we have a template selected or not */
	bool bTemplate = false;
	
	/** True if Ok was clicked */
	bool bOkClicked;
};


/*------------------------------------------------------------------------------
	UAnimBlueprintFactory implementation.
------------------------------------------------------------------------------*/

UAnimBlueprintFactory::UAnimBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAnimBlueprint::StaticClass();
	ParentClass = UAnimInstance::StaticClass();
}

FText UAnimBlueprintFactory::GetDisplayName() const
{
	return LOCTEXT("AnimationBlueprintFactoryDescription", "Animation Blueprint");
}

bool UAnimBlueprintFactory::ConfigureProperties()
{
	TSharedRef<SAnimBlueprintCreateDialog> Dialog = SNew(SAnimBlueprintCreateDialog);
	return Dialog->ConfigureProperties(this);
};

UObject* UAnimBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a Anim Blueprint, then create and init one
	check(Class->IsChildOf(UAnimBlueprint::StaticClass()));

	if (BlueprintType != BPTYPE_Interface && ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf(UAnimInstance::StaticClass())))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateAnimBlueprint", "Cannot create an Anim Blueprint based on the class '{ClassName}'."), Args ) );
		return nullptr;
	}
	else
	{
		UClass* ClassToUse = BlueprintType == BPTYPE_Interface ? UAnimLayerInterface::StaticClass() : ParentClass.Get();
		UAnimBlueprint* NewBP = CastChecked<UAnimBlueprint>(FKismetEditorUtilities::CreateBlueprint(ClassToUse, InParent, Name, BlueprintType, UAnimBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext));
		
		// Inherit any existing overrides in parent class
		if (NewBP->ParentAssetOverrides.Num() > 0)
		{
			// We've inherited some overrides from the parent graph and need to recompile the blueprint.
			FKismetEditorUtilities::CompileBlueprint(NewBP);
		}
		
		if(bTemplate)
		{
			NewBP->bIsTemplate = true;
			NewBP->TargetSkeleton = nullptr;
		}
		else
		{
			NewBP->bIsTemplate = false;
			NewBP->TargetSkeleton = TargetSkeleton;
		}

		// Because the BP itself didn't have the skeleton set when the initial compile occured, it's not set on the generated classes either
		if (UAnimBlueprintGeneratedClass* TypedNewClass = Cast<UAnimBlueprintGeneratedClass>(NewBP->GeneratedClass))
		{
			TypedNewClass->TargetSkeleton = TargetSkeleton;
		}
		if (UAnimBlueprintGeneratedClass* TypedNewClass_SKEL = Cast<UAnimBlueprintGeneratedClass>(NewBP->SkeletonGeneratedClass))
		{
			TypedNewClass_SKEL->TargetSkeleton = TargetSkeleton;
		}

		if (TargetSkeleton && PreviewSkeletalMesh)
		{
			NewBP->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return NewBP;
	}
}

UObject* UAnimBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

/*------------------------------------------------------------------------------
	UAnimLayerInterfaceFactory implementation.
------------------------------------------------------------------------------*/

UAnimLayerInterfaceFactory::UAnimLayerInterfaceFactory()
{
	BlueprintType = BPTYPE_Interface;
}

bool UAnimLayerInterfaceFactory::ConfigureProperties()
{
	return true;
}

FText UAnimLayerInterfaceFactory::GetDisplayName() const
{
	return LOCTEXT("AnimationLayerInterfaceFactoryDescription", "Animation Layer Interface");
}

FName UAnimLayerInterfaceFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("ClassThumbnail.BlueprintInterface");
}

uint32 UAnimLayerInterfaceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UAnimLayerInterfaceFactory::GetToolTip() const
{
	return LOCTEXT("AnimationLayerInterfaceTooltip", "An Animation Layer Interface is a collection of one or more animation graphs - name only, no implementation - that can be added to other Animation Blueprints. These other Blueprints are then expected to implement the graphs of the Animation Layer Interface in a unique manner.");
}

FString UAnimLayerInterfaceFactory::GetToolTipDocumentationExcerpt() const
{
	return TEXT("UAnimationBlueprint_LayerInterface");
}

FString UAnimLayerInterfaceFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewAnimLayerInterface"));
}

#undef LOCTEXT_NAMESPACE
