// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AnimBoneCompressionSettingsFactory.cpp: Factory for animation bone compression settings assets
=============================================================================*/

#include "Factories/AnimBoneCompressionSettingsFactory.h"

#include "Animation/AnimBoneCompressionSettings.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionSettingsFactory)

class FFeedbackContext;
class UClass;
class UObject;

UAnimBoneCompressionSettingsFactory::UAnimBoneCompressionSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimBoneCompressionSettings::StaticClass();
}

UObject* UAnimBoneCompressionSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimBoneCompressionSettings>(InParent, Class, Name, Flags);
}
