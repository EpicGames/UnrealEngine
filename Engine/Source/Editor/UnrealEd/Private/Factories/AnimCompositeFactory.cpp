// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompositeFactory.cpp: Factory for AnimComposite
=============================================================================*/

#include "Factories/AnimCompositeFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimCompositeBase.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Animation/AnimComposite.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCompositeFactory)

#define LOCTEXT_NAMESPACE "AnimCompositeFactory"

UAnimCompositeFactory::UAnimCompositeFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimComposite::StaticClass();
}

bool UAnimCompositeFactory::ConfigureProperties()
{
	// Null the skeleton so we can check for selection later
	TargetSkeleton = NULL;
	SourceAnimation = NULL;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UAnimCompositeFactory::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateAnimCompositeOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != NULL;
}

UObject* UAnimCompositeFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (TargetSkeleton || SourceAnimation)
	{
		UAnimComposite* AnimComposite = NewObject<UAnimComposite>(InParent, Class, Name, Flags);

		if(SourceAnimation)
		{
			USkeleton* SourceSkeleton = SourceAnimation->GetSkeleton();
			//Make sure we haven't asked to create an AnimComposite with mismatching skeletons
			check(TargetSkeleton == NULL || TargetSkeleton == SourceSkeleton);
			TargetSkeleton = SourceSkeleton;

			FAnimSegment NewSegment;
			NewSegment.SetAnimReference(SourceAnimation, true);

			AnimComposite->AnimationTrack.AnimSegments.Add(NewSegment);
			AnimComposite->SetCompositeLength(AnimComposite->AnimationTrack.GetLength());
		}
		
		AnimComposite->SetSkeleton( TargetSkeleton );
		if (PreviewSkeletalMesh)
		{
			AnimComposite->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return AnimComposite;
	}

	return NULL;
}

void UAnimCompositeFactory::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE
