// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
AnimCurveCompressionSettingsFactory.cpp: Factory for animation curve compression settings assets
=============================================================================*/

#include "Factories/AnimCurveCompressionSettingsFactory.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "Animation/AnimCurveCompressionCodec_CompressedRichCurve.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionSettingsFactory)

UAnimCurveCompressionSettingsFactory::UAnimCurveCompressionSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimCurveCompressionSettings::StaticClass();
}

UObject* UAnimCurveCompressionSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAnimCurveCompressionSettings* Settings = NewObject<UAnimCurveCompressionSettings>(InParent, Class, Name, Flags);
	Settings->Codec = NewObject<UAnimCurveCompressionCodec_CompressedRichCurve>(Settings);
	Settings->Codec->SetFlags(RF_Transactional);

	return Settings;
}
