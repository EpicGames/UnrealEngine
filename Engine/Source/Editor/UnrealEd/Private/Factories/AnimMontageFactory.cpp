// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimMontageFactory.cpp: Factory for AnimMontages
=============================================================================*/

#include "Factories/AnimMontageFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimCompositeBase.h"
#include "Animation/AnimMontage.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimMontageFactory)

#define LOCTEXT_NAMESPACE "AnimMontageFactory"

UAnimMontageFactory::UAnimMontageFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimMontage::StaticClass();
}

bool UAnimMontageFactory::ConfigureProperties()
{
	// Null the skeleton so we can check for selection later
	TargetSkeleton = NULL;
	SourceAnimation = NULL;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UAnimMontageFactory::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateAnimMontageOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != NULL;
}

UObject* UAnimMontageFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (TargetSkeleton || SourceAnimation)
	{
		UAnimMontage* AnimMontage = NewObject<UAnimMontage>(InParent, Class, Name, Flags);

		if(SourceAnimation)
		{
			USkeleton* SourceSkeleton = SourceAnimation->GetSkeleton();
			//Make sure we haven't asked to create an AnimComposite with mismatching skeletons
			check(TargetSkeleton == NULL || TargetSkeleton == SourceSkeleton);
			TargetSkeleton = SourceSkeleton;

			FAnimSegment NewSegment;
			NewSegment.SetAnimReference(SourceAnimation, true);

			FSlotAnimationTrack& NewTrack = AnimMontage->SlotAnimTracks[0];
			NewTrack.AnimTrack.AnimSegments.Add(NewSegment);

			AnimMontage->SetCompositeLength(SourceAnimation->GetPlayLength());

			AnimMontage->UpdateCommonTargetFrameRate();
		}

		AnimMontage->SetSkeleton(TargetSkeleton);
		if (PreviewSkeletalMesh)
		{
			AnimMontage->SetPreviewMesh(PreviewSkeletalMesh);
		}

		EnsureStartingSection(AnimMontage);

		return AnimMontage;
	}

	return NULL;
}

void UAnimMontageFactory::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

bool UAnimMontageFactory::EnsureStartingSection(UAnimMontage* Montage)
{
	bool bModified = false;
	if (Montage->CompositeSections.Num() <= 0)
	{
		FCompositeSection NewSection;
		NewSection.SetTime(0.0f);
		NewSection.SectionName = FName(TEXT("Default"));
		Montage->CompositeSections.Add(NewSection);
		bModified = true;
	}

	check(Montage->CompositeSections.Num() > 0);
	if (Montage->CompositeSections[0].GetTime() > 0.0f)
	{
		Montage->CompositeSections[0].SetTime(0.0f);
		bModified = true;
	}

	return bModified;
}

#undef LOCTEXT_NAMESPACE
