// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimSequenceFactory.cpp: Factory for AnimSequence
=============================================================================*/

#include "Factories/AnimSequenceFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceFactory)

#define LOCTEXT_NAMESPACE "AnimSequenceFactory"

UAnimSequenceFactory::UAnimSequenceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimSequence::StaticClass();
}

bool UAnimSequenceFactory::ConfigureProperties()
{
	// Null the skeleton so we can check for selection later
	TargetSkeleton = nullptr;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UAnimSequenceFactory::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateAnimSequenceOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != nullptr;
}

UObject* UAnimSequenceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAnimSequence* AnimSequence = NewObject<UAnimSequence>(InParent, Class, Name, Flags);
	
	if (TargetSkeleton)
	{
		AnimSequence->SetSkeleton(TargetSkeleton);
	}

	AnimSequence->GetController().InitializeModel();
	AnimSequence->GetController().SetNumberOfFrames(1);
	if (PreviewSkeletalMesh)
	{
		AnimSequence->SetPreviewMesh(PreviewSkeletalMesh);
	}
	AnimSequence->GetController().NotifyPopulated();

	return AnimSequence;
}

void UAnimSequenceFactory::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE
