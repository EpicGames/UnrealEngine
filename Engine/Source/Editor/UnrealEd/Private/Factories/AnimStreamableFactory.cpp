// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimCompositeFactory.cpp: Factory for AnimComposite
=============================================================================*/

#include "Factories/AnimStreamableFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimStreamable.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimStreamableFactory)

#define LOCTEXT_NAMESPACE "AnimStreamableFactory"

UAnimStreamableFactory::UAnimStreamableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = UAnimStreamable::StaticClass();
}

bool UAnimStreamableFactory::ConfigureProperties()
{
	// Null the skeleton so we can check for selection later
	/*TargetSkeleton = NULL;
	SourceAnimation = NULL;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetFName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	// The delegate that fires when an asset was selected
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UAnimCompositeFactory::OnTargetSkeletonSelected);

	//The default view mode should be a list view
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateAnimCompositeOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != NULL;*/
	return true;
}

UObject* UAnimStreamableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (SourceAnimation)
	{
		UAnimStreamable* StreamableAnim = NewObject<UAnimStreamable>(InParent, Class, Name, Flags);

		StreamableAnim->InitFrom(SourceAnimation);
		
		return StreamableAnim;
	}

	return NULL;
}

#undef LOCTEXT_NAMESPACE
