// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CSVImportFactory.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "Factories/ReimportCurveFactory.h"
#include "Factories/ReimportCurveTableFactory.h"
#include "Factories/ReimportDataTableFactory.h"
#include "EditorFramework/AssetImportData.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Engine/CurveTable.h"
#include "Engine/DataTable.h"
#include "Editor.h"

#include "Interfaces/IMainFrameModule.h"
#include "SCSVImportOptions.h"
#include "DataTableEditorUtils.h"
#include "JsonObjectConverter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CSVImportFactory)

DEFINE_LOG_CATEGORY(LogCSVImportFactory);

#define LOCTEXT_NAMESPACE "CSVImportFactory"

//////////////////////////////////////////////////////////////////////////

FCSVImportSettings::FCSVImportSettings()
{
	ImportRowStruct = nullptr;
	ImportType = ECSVImportType::ECSV_DataTable;
	ImportCurveInterpMode = ERichCurveInterpMode::RCIM_Linear;
}


static UClass* GetCurveClass( ECSVImportType ImportType )
{
	switch( ImportType )
	{
	case ECSVImportType::ECSV_CurveFloat:
		return UCurveFloat::StaticClass();
	case ECSVImportType::ECSV_CurveVector:
		return UCurveVector::StaticClass();
	case ECSVImportType::ECSV_CurveLinearColor:
		return UCurveLinearColor::StaticClass();
	default:
		return UCurveVector::StaticClass();
	}
}


UCSVImportFactory::UCSVImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UDataTable::StaticClass();

	bEditorImport = true;
	bText = true;

	// Give this factory a lower than normal import priority, as CSV and JSON can be commonly used and we'd like to give the other import factories a shot first
	--ImportPriority;

	Formats.Add(TEXT("csv;Comma-separated values"));
}

bool UCSVImportFactory::IsAutomatedImport() const
{
	return Super::IsAutomatedImport()
		|| AutomatedImportSettings.bForceAutomatedImport;
}

FText UCSVImportFactory::GetDisplayName() const
{
	return LOCTEXT("CSVImportFactoryDescription", "Comma Separated Values");
}

bool UCSVImportFactory::DoesSupportClass(UClass * Class)
{
	return (Class == UDataTable::StaticClass() || Class == UCurveTable::StaticClass() || Class == UCurveFloat::StaticClass() || Class == UCurveVector::StaticClass() || Class == UCurveLinearColor::StaticClass() );
}

bool UCSVImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if (Extension == TEXT("csv"))
	{
		return true;
	}
	return false;
}

IImportSettingsParser* UCSVImportFactory::GetImportSettingsParser()
{
	return this;
}

void UCSVImportFactory::CleanUp()
{
	Super::CleanUp();
	
	bImportAll = false;
	DataTableImportOptions = nullptr;
}

UObject* UCSVImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, 
	const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	// ScriptFactoryCreateFile not implemented. We do not support blueprint/python subclasses of CSVImportFactory
	FString FileExtension = FPaths::GetExtension(Filename);

	// load as text
	check(bText); // Set in constructor, so we do not need to support load as binary
	{
		FString Data;
		if (!FFileHelper::LoadFileToString(Data, *Filename, FFileHelper::EHashOptions::None, FILEREAD_AllowWrite))
		{
			UE_LOGF(LogCSVImportFactory, Error, "Failed to load file '%ls' to string", *Filename);
			return nullptr;
		}

		ParseParms(Parms);
		const TCHAR* Ptr = *Data;

		return FactoryCreateText(InClass, InParent, InName, Flags, nullptr, *FileExtension, Ptr, Ptr + Data.Len(),
			Warn, bOutOperationCanceled);
	}
}

UObject* UCSVImportFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	bOutOperationCanceled = false;

	// See if table/curve already exists
	UDataTable* ExistingTable = FindObject<UDataTable>(InParent, *InName.ToString());
	UCurveTable* ExistingCurveTable = FindObject<UCurveTable>(InParent, *InName.ToString());
	UCurveBase* ExistingCurve = FindObject<UCurveBase>(InParent, *InName.ToString());

	FCSVImportSettings ImportSettings;

	bool bHaveInfo = false;
	bool bReimportingCurveTable = false;

	if (IsAutomatedImport())
	{
		ImportSettings = AutomatedImportSettings;

		// For automated import to work a row struct must be specified for a datatable type or a curve type must be specified
		bHaveInfo = ImportSettings.ImportRowStruct != nullptr || ImportSettings.ImportType != ECSVImportType::ECSV_DataTable;
	}
	else if (ExistingTable != nullptr)
	{
		ImportSettings.ImportType = ECSVImportType::ECSV_DataTable;
		ImportSettings.ImportRowStruct = ExistingTable->RowStruct;
		
		bHaveInfo = true;
	}
	else if (ExistingCurveTable != nullptr)
	{
		ImportSettings.ImportType = ECSVImportType::ECSV_CurveTable;
		bReimportingCurveTable = true;

		if (ExistingCurveTable->ImportCurveInterpMode != RCIM_None)
		{
			ImportSettings.ImportCurveInterpMode = ExistingCurveTable->ImportCurveInterpMode;
			bHaveInfo = true;
		}
	}
	else if (ExistingCurve != nullptr)
	{
		if (ExistingCurve->IsA(UCurveFloat::StaticClass()))
		{
			ImportSettings.ImportType = ECSVImportType::ECSV_CurveFloat;
		}
		else if (ExistingCurve->IsA(UCurveLinearColor::StaticClass()))
		{
			ImportSettings.ImportType = ECSVImportType::ECSV_CurveLinearColor;
		}
		else
		{
			ImportSettings.ImportType = ECSVImportType::ECSV_CurveVector;
		}
		bHaveInfo = true;
	}
	else if (bImportAll)
	{
		ImportSettings = AutomatedImportSettings;
		bHaveInfo = true;
	}

	ImportSettings.bDataIsJson = FString(Type).Equals(TEXT("json"), ESearchCase::IgnoreCase);
	bool bDoImport = true;

	// If we do not have the info we need, pop up window to ask for things
	if (!bHaveInfo && !IsAutomatedImport())
	{
		TSharedPtr<SWindow> ParentWindow;
		// Check if the main frame is loaded.  When using the old main frame it may not be.
		if (FModuleManager::Get().IsModuleLoaded( "MainFrame" ))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
			ParentWindow = MainFrame.GetParentWindow();
		}

		TSharedPtr<SCSVImportOptions> ImportOptionsWindow;

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title( LOCTEXT("DataTableOptionsWindowTitle", "DataTable Options" ))
			.SizingRule( ESizingRule::Autosized );
		
		FString ParentFullPath;

		if (InParent)
		{
			ParentFullPath = InParent->GetPathName();
		}

		DataTableImportOptions = NewObject<UDataTable>(this);

		Window->SetContent
		(
			SAssignNew(ImportOptionsWindow, SCSVImportOptions)
				.WidgetWindow(Window)
				.FullPath(FText::FromString(ParentFullPath))
				.TempImportDataTable(DataTableImportOptions)
				.bReimportingCurveTable(bReimportingCurveTable)
		);

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		ImportSettings.ImportType = ImportOptionsWindow->GetSelectedImportType();
		ImportSettings.ImportRowStruct = ImportOptionsWindow->GetSelectedRowStruct();
		ImportSettings.ImportCurveInterpMode = ImportOptionsWindow->GetSelectedCurveIterpMode();

		bDoImport = ImportOptionsWindow->ShouldImport();
		bImportAll = ImportOptionsWindow->ShouldImportAll();
		bOutOperationCanceled = !bDoImport;

		if (bImportAll)
		{
			AutomatedImportSettings = ImportSettings;
		}
	}
	else if (!bHaveInfo && IsAutomatedImport())
	{
		if (ImportSettings.ImportType == ECSVImportType::ECSV_DataTable && !ImportSettings.ImportRowStruct)
		{
			UE_LOGF(LogCSVImportFactory, Error, "A Data table row type must be specified in the import settings json file for automated import");
		}
		bDoImport = false;
	}

	UObject* NewAsset = nullptr;
	if (bDoImport)
	{
		// Convert buffer to an FString (will this be slow with big tables?)
		int32 NumChars = UE_PTRDIFF_TO_INT32(BufferEnd - Buffer);
		TArray<TCHAR, FString::AllocatorType>& StringChars = ImportSettings.DataToImport.GetCharArray();
		StringChars.AddUninitialized(NumChars+1);
		FMemory::Memcpy(StringChars.GetData(), Buffer, NumChars*sizeof(TCHAR));
		StringChars.Last() = 0;

		TArray<FString> Problems;

		if (ImportSettings.ImportType == ECSVImportType::ECSV_DataTable)
		{
			UDataTable* TempImportDataTable = NewObject<UDataTable>(this);
			if (DataTableImportOptions)
			{
				TempImportDataTable->CopyImportOptions(DataTableImportOptions);
			}
			
			TMap<FName, TUniquePtr<uint8[]>> TempRowMap;

			// If there is an existing table, need to call this to free data memory before recreating object
			UDataTable::FOnDataTableChanged OldOnDataTableChanged;
			UClass* DataTableClass = UDataTable::StaticClass();
			if (ExistingTable != nullptr)
			{
				TempImportDataTable->CopyImportOptions(ExistingTable);
				OldOnDataTableChanged = MoveTemp(ExistingTable->OnDataTableChanged());
				ExistingTable->OnDataTableChanged().Clear();
				DataTableClass = ExistingTable->GetClass();
				
				TMap<FName, uint8*> ExistingTableRowMap;
				ExistingTableRowMap = ExistingTable->GetRowMap();

				if (ExistingTable->bPreserveExistingValues)
				{
					// Copy each existing row to the temporary map
					for (const TPair<FName, uint8*>& TableRow : ExistingTableRowMap)
					{
						TempRowMap.Add(TableRow.Key, CopyRow(TableRow.Value, ExistingTable->RowStruct));
					}
				}
				
				ExistingTable->EmptyTable();
			}

			// Create/reset table
			UDataTable* NewTable = NewObject<UDataTable>(InParent, DataTableClass, InName, Flags);
			TempImportDataTable->RowStruct = ImportSettings.ImportRowStruct;
			
			NewTable->CopyImportOptions(TempImportDataTable);
			if (!CurrentFilename.IsEmpty())
			{
				NewTable->AssetImportData->Update(CurrentFilename);
			}

			if (NewTable->bPreserveExistingValues && ExistingTable)
			{
				ensure(NewTable->RowStruct == ExistingTable->RowStruct);
				
				for (TPair<FName, TUniquePtr<uint8[]>>& Row : TempRowMap)
				{
					if (const FTableRowBase* RowBase = reinterpret_cast<FTableRowBase*>(Row.Value.Get()))
					{
						NewTable->AddRow(Row.Key, *RowBase);
					}
				}
			}

			// Go ahead and create table from string
			Problems = DoImportDataTable(ImportSettings, NewTable);

			// hook delegates back up and inform listeners of changes
			NewTable->OnDataTableChanged() = MoveTemp(OldOnDataTableChanged);
			NewTable->OnDataTableChanged().Broadcast();

			// Print out
			UE_LOGF(LogCSVImportFactory, Log, "Imported DataTable '%ls' - %d Problems", *InName.ToString(), Problems.Num());
			NewAsset = NewTable;
		}
		else if (ImportSettings.ImportType == ECSVImportType::ECSV_CurveTable)
		{
			UClass* CurveTableClass = UCurveTable::StaticClass();

			// If there is an existing table, need to call this to free data memory before recreating object
			UCurveTable::FOnCurveTableChanged OldOnCurveTableChanged;
			if (ExistingCurveTable != nullptr)
			{
				OldOnCurveTableChanged = MoveTemp(ExistingCurveTable->OnCurveTableChanged());
				ExistingCurveTable->OnCurveTableChanged().Clear();
				CurveTableClass = ExistingCurveTable->GetClass();
				ExistingCurveTable->EmptyTable();
			}

			// Create/reset table
			UCurveTable* NewTable = NewObject<UCurveTable>(InParent, CurveTableClass, InName, Flags);
			if (!CurrentFilename.IsEmpty())
			{
				NewTable->AssetImportData->Update(CurrentFilename);
			}

			// Go ahead and create table from string
			Problems = DoImportCurveTable(ImportSettings, NewTable);

			// hook delegates back up and inform listeners of changes
			NewTable->OnCurveTableChanged() = MoveTemp(OldOnCurveTableChanged);
			NewTable->OnCurveTableChanged().Broadcast();

			// Print out
			UE_LOGF(LogCSVImportFactory, Log, "Imported CurveTable '%ls' - %d Problems", *InName.ToString(), Problems.Num());
			NewAsset = NewTable;
		}
		else if (ImportSettings.ImportType == ECSVImportType::ECSV_CurveFloat || ImportSettings.ImportType == ECSVImportType::ECSV_CurveVector || ImportSettings.ImportType == ECSVImportType::ECSV_CurveLinearColor)
		{
			// Create/reset curve
			UCurveBase* NewCurve = NewObject<UCurveBase>(InParent, GetCurveClass(ImportSettings.ImportType), InName, Flags);
			Problems = DoImportCurve(ImportSettings, NewCurve);

			UE_LOGF(LogCSVImportFactory, Log, "Imported Curve '%ls' - %d Problems", *InName.ToString(), Problems.Num());
			NewCurve->AssetImportData->Update(CurrentFilename);
			NewAsset = NewCurve;
		}
		
		if (Problems.Num() > 0)
		{
			FString AllProblems;

			for (int32 ProbIdx=0; ProbIdx<Problems.Num(); ProbIdx++)
			{
				// Output problems to log
				UE_LOGF(LogCSVImportFactory, Log, "%d:%ls", ProbIdx, *Problems[ProbIdx]);
				AllProblems += Problems[ProbIdx];
				AllProblems += TEXT("\n");
			}

			if (!IsAutomatedImport())
			{
				// Pop up any problems for user
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(AllProblems));
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NewAsset);

	return NewAsset;
}

EReimportResult::Type UCSVImportFactory::ReimportCSV(UObject* Obj)
{
	EReimportResult::Type Result = EReimportResult::Failed;

	if (UCurveBase* Curve = Cast<UCurveBase>(Obj))
	{
		Result = Reimport(Curve, Curve->AssetImportData->GetFirstFilename());
	}
	else if (UCurveTable* CurveTable = Cast<UCurveTable>(Obj))
	{
		Result = Reimport(CurveTable, CurveTable->AssetImportData->GetFirstFilename());
	}
	else if (UDataTable* DataTable = Cast<UDataTable>(Obj))
	{
		FDataTableEditorUtils::BroadcastPreChange(DataTable, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
		Result = Reimport(DataTable, DataTable->AssetImportData->GetFirstFilename());
		FDataTableEditorUtils::BroadcastPostChange(DataTable, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
	}
	return Result;
}

void UCSVImportFactory::ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson)
{	
	FJsonObjectConverter::JsonObjectToUStruct(ImportSettingsJson, FCSVImportSettings::StaticStruct(), &AutomatedImportSettings, 0, 0);
}

EReimportResult::Type UCSVImportFactory::Reimport(UObject* Obj, const FString& Path)
{
	if (Path.IsEmpty() == false)
	{ 
		FString FilePath = IFileManager::Get().ConvertToRelativePath(*Path);

		FString Data;
		if ( FFileHelper::LoadFileToString( Data, *FilePath, FFileHelper::EHashOptions::None, FILEREAD_AllowWrite) )
		{
			const TCHAR* Ptr = *Data;
			CurrentFilename = FilePath; //not thread safe but seems to be how it is done..
			bool bWasCancelled = false;
			UObject* Result = FactoryCreateText(Obj->GetClass(), Obj->GetOuter(), Obj->GetFName(), Obj->GetFlags(), nullptr, *FPaths::GetExtension(FilePath), Ptr, Ptr+Data.Len(), nullptr, bWasCancelled);
			if (bWasCancelled)
			{
				return EReimportResult::Cancelled;
			}
			return Result ? EReimportResult::Succeeded : EReimportResult::Failed;
		}
	}
	return EReimportResult::Failed;
}

TUniquePtr<uint8[]> UCSVImportFactory::CopyRow(const uint8* RowData, const UScriptStruct* RowStruct) const
{
	const int32 RowSize = RowStruct->GetStructureSize();
	TUniquePtr<uint8[]> NewRowData = MakeUnique<uint8[]>(RowSize);
	RowStruct->InitializeStruct(NewRowData.Get());
	RowStruct->CopyScriptStruct(NewRowData.Get(), RowData);
	return NewRowData;
}

TArray<FString> UCSVImportFactory::DoImportDataTable(const FCSVImportSettings& InImportSettings, UDataTable* TargetDataTable)
{
	if (InImportSettings.bDataIsJson)
	{
		return TargetDataTable->CreateTableFromJSONString(InImportSettings.DataToImport);
	}

	return TargetDataTable->CreateTableFromCSVString(InImportSettings.DataToImport);
}

TArray<FString> UCSVImportFactory::DoImportCurveTable(const FCSVImportSettings& InImportSettings, UCurveTable* TargetCurveTable)
{
	if (InImportSettings.bDataIsJson)
	{
		return TargetCurveTable->CreateTableFromJSONString(InImportSettings.DataToImport, InImportSettings.ImportCurveInterpMode);
	}

	return TargetCurveTable->CreateTableFromCSVString(InImportSettings.DataToImport, InImportSettings.ImportCurveInterpMode);
}

TArray<FString> UCSVImportFactory::DoImportCurve(const FCSVImportSettings& InImportSettings, UCurveBase* TargetCurve)
{
	if (InImportSettings.bDataIsJson)
	{
		TArray<FString> Result;
		Result.Add(LOCTEXT("Error_CannotImportCurveFromJSON", "Cannot import a curve from JSON. Please use CSV instead.").ToString());
		return Result;
	}

	return TargetCurve->CreateCurveFromCSVString(InImportSettings.DataToImport);
}

//////////////////////////////////////////////////////////////////////////

UReimportDataTableFactory::UReimportDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("json;JavaScript Object Notation"));
}

bool UReimportDataTableFactory::FactoryCanImport( const FString& Filename )
{
	return true;
}

bool UReimportDataTableFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UDataTable* DataTable = Cast<UDataTable>(Obj);
	if (DataTable)
	{
		DataTable->AssetImportData->ExtractFilenames(OutFilenames);
		
		// Always allow reimporting a data table as it is common to convert from manual to CSV-driven tables
		return true;
	}
	return false;
}

void UReimportDataTableFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UDataTable* DataTable = Cast<UDataTable>(Obj);
	if (DataTable && ensure(NewReimportPaths.Num() == 1))
	{
		DataTable->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportDataTableFactory::Reimport( UObject* Obj )
{	
	EReimportResult::Type Result = EReimportResult::Failed;
	if (UDataTable* DataTable = Cast<UDataTable>(Obj))
	{
		Result = UCSVImportFactory::ReimportCSV(DataTable) ? EReimportResult::Succeeded : EReimportResult::Failed;
	}
	return Result;
}

int32 UReimportDataTableFactory::GetPriority() const
{
	return ImportPriority;
}

////////////////////////////////////////////////////////////////////////////
//
UReimportCurveTableFactory::UReimportCurveTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("json;JavaScript Object Notation"));
}

bool UReimportCurveTableFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UCurveTable* CurveTable = Cast<UCurveTable>(Obj);
	if (CurveTable)
	{
		CurveTable->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UReimportCurveTableFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UCurveTable* CurveTable = Cast<UCurveTable>(Obj);
	if (CurveTable && ensure(NewReimportPaths.Num() == 1))
	{
		CurveTable->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportCurveTableFactory::Reimport( UObject* Obj )
{	
	if (Cast<UCurveTable>(Obj))
	{
		return UCSVImportFactory::ReimportCSV(Obj) ? EReimportResult::Succeeded : EReimportResult::Failed;
	}
	return EReimportResult::Failed;
}

int32 UReimportCurveTableFactory::GetPriority() const
{
	return ImportPriority;
}

////////////////////////////////////////////////////////////////////////////
//
UReimportCurveFactory::UReimportCurveFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveBase::StaticClass();
}

bool UReimportCurveFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UCurveBase* CurveBase = Cast<UCurveBase>(Obj);
	if (CurveBase)
	{
		CurveBase->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UReimportCurveFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UCurveBase* CurveBase = Cast<UCurveBase>(Obj);
	if (CurveBase && ensure(NewReimportPaths.Num() == 1))
	{
		CurveBase->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportCurveFactory::Reimport( UObject* Obj )
{	
	if (Cast<UCurveBase>(Obj))
	{
		return UCSVImportFactory::ReimportCSV(Obj) ? EReimportResult::Succeeded : EReimportResult::Failed;
	}
	return EReimportResult::Failed;
}

int32 UReimportCurveFactory::GetPriority() const
{
	return ImportPriority;
}


#undef LOCTEXT_NAMESPACE

