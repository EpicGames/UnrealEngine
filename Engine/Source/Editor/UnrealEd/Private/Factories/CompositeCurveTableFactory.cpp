// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CompositeCurveTableFactory.h"

#include "Engine/CompositeCurveTable.h"
#include "Engine/CurveTable.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCurveTableFactory)

class FFeedbackContext;
class UClass;
class UObject;

UCompositeCurveTableFactory::UCompositeCurveTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCompositeCurveTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UCompositeCurveTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCurveTable* CurveTable = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		CurveTable = MakeNewCurveTable(InParent, Name, Flags);
	}
	return CurveTable;
}

UCurveTable* UCompositeCurveTableFactory::MakeNewCurveTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	return NewObject<UCompositeCurveTable>(InParent, Name, Flags);
}
