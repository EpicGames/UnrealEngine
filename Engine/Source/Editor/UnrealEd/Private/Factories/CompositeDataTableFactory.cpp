// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CompositeDataTableFactory.h"

#include "Engine/CompositeDataTable.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeDataTableFactory)

class UDataTable;
class UObject;

UCompositeDataTableFactory::UCompositeDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCompositeDataTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UDataTable* UCompositeDataTableFactory::MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	return NewObject<UCompositeDataTable>(InParent, Name, Flags);
}
