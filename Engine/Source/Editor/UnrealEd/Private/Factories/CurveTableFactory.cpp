// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/CurveTableFactory.h"
#include "Engine/CurveTable.h"
#include "Editor.h"
#include "SCurveTableOptions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveTableFactory)

#define LOCTEXT_NAMESPACE "CurveTableFactory"

UCurveTableFactory::UCurveTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UCurveTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCurveTable* CurveTable = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		CurveTable = MakeNewCurveTable(InParent, Name, Flags);
	}
	return CurveTable;
}


bool UCurveTableFactory::ConfigureProperties()
{

	ERichCurveInterpMode SelectedInterpMode = ERichCurveInterpMode::RCIM_Linear;
	bool bDoCreation = false;

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.Title( LOCTEXT("NewCurveTableWindowTitle", "Curve Table Options" ))
		.SizingRule( ESizingRule::Autosized );

	Window->SetContent( 
		SNew(SCurveTableOptions)
		.OnCancelClicked_Lambda([Window, &bDoCreation]() 
		{
			bDoCreation = false;
			Window->RequestDestroyWindow();	
		})
		.OnCreateClicked_Lambda([Window, &bDoCreation, &SelectedInterpMode] (ERichCurveInterpMode InInterpMode) 
		{
			SelectedInterpMode = InInterpMode;
			bDoCreation = true;
			Window->RequestDestroyWindow();
		})
	);

	GEditor->EditorAddModalWindow(Window.ToSharedRef());

	// Store which interpolation mode was selected
	InterpMode = SelectedInterpMode;

	return bDoCreation;
}

UCurveTable* UCurveTableFactory::MakeNewCurveTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	UCurveTable* NewTable = NewObject<UCurveTable>(InParent, Name, Flags);
	if (InterpMode != ERichCurveInterpMode::RCIM_Cubic)
	{
		FSimpleCurve& NewSimpleCurve = NewTable->AddSimpleCurve(FName("Curve"));
		NewSimpleCurve.SetKeyInterpMode(InterpMode);
	}
	else
	{
		FRichCurve& NewRichCurve = NewTable->AddRichCurve(FName("Curve"));
	}

	return NewTable;
}

#undef LOCTEXT_NAMESPACE 
