// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DataTableFactory.h"
#include "DataTableEditorUtils.h"
#include "Engine/DataTable.h"
#include "Editor.h"

#include "Modules/ModuleManager.h"
#include "StructViewerModule.h"
#include "StructViewerFilter.h"

#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Input/Reply.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataTableFactory)

#define LOCTEXT_NAMESPACE "DataTableFactory"

UDataTableFactory::UDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UDataTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UDataTableFactory::ConfigureProperties()
{
	class FDataTableStructFilter : public IStructViewerFilter
	{
	public:
		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			return FDataTableEditorUtils::IsValidTableStruct(InStruct);
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			// Unloaded structs are always User Defined Structs, and User Defined Structs are always allowed
			// They will be re-validated by IsStructAllowed once loaded during the pick
			return true;
		}
	};

	class FDataTableFactoryUI : public TSharedFromThis<FDataTableFactoryUI>
	{
	public:
		FReply OnCreate()
		{
			check(ResultStruct);
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		FReply OnCancel()
		{
			ResultStruct = nullptr;
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		bool IsStructSelected() const
		{
			return ResultStruct != nullptr;
		}

		void OnPickedStruct(const UScriptStruct* ChosenStruct)
		{
			ResultStruct = ChosenStruct;
			StructPickerAnchor->SetIsOpen(false);
		}

		FText OnGetComboTextValue() const
		{
			return ResultStruct
				? FText::AsCultureInvariant(ResultStruct->GetName())
				: LOCTEXT("None", "None");
		}

		TSharedRef<SWidget> GenerateStructPicker()
		{
			FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

			// Fill in options
			FStructViewerInitializationOptions Options;
			Options.Mode = EStructViewerMode::StructPicker;
			Options.StructFilter = MakeShared<FDataTableStructFilter>();

			return
				SNew(SBox)
				.WidthOverride(330.0f)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					.MaxHeight(500)
					[
						SNew(SBorder)
						.Padding(4)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateSP(this, &FDataTableFactoryUI::OnPickedStruct))
						]
					]
				];
		}

		const UScriptStruct* OpenStructSelector()
		{
			FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");
			ResultStruct = nullptr;

			// Fill in options
			FStructViewerInitializationOptions Options;
			Options.Mode = EStructViewerMode::StructPicker;
			Options.StructFilter = MakeShared<FDataTableStructFilter>();

			PickerWindow = SNew(SWindow)
				.Title(LOCTEXT("DataTableFactoryOptions", "Pick Row Structure"))
				.ClientSize(FVector2D(350, 100))
				.SupportsMinimize(false)
				.SupportsMaximize(false)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					.Padding(10)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(StructPickerAnchor, SComboButton)
							.ContentPadding(FMargin(2,2,2,1))
							.MenuPlacement(MenuPlacement_BelowAnchor)
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(this, &FDataTableFactoryUI::OnGetComboTextValue)
							]
							.OnGetMenuContent(this, &FDataTableFactoryUI::GenerateStructPicker)
						]
						+SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("OK", "OK"))
								.IsEnabled(this, &FDataTableFactoryUI::IsStructSelected)
								.OnClicked(this, &FDataTableFactoryUI::OnCreate)
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("Cancel", "Cancel"))
								.OnClicked(this, &FDataTableFactoryUI::OnCancel)
							]
						]
					]
				];

			GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
			PickerWindow.Reset();

			return ResultStruct;
		}

	private:
		TSharedPtr<SWindow> PickerWindow;
		TSharedPtr<SComboButton> StructPickerAnchor;
		const UScriptStruct* ResultStruct = nullptr;
	};

	TSharedRef<FDataTableFactoryUI> StructSelector = MakeShareable(new FDataTableFactoryUI());
	Struct = StructSelector->OpenStructSelector();

	return Struct != nullptr;
}

UObject* UDataTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UDataTable* DataTable = nullptr;
	if (Struct && ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		DataTable = MakeNewDataTable(InParent, Name, Flags);
		if (DataTable)
		{
			DataTable->RowStruct = const_cast<UScriptStruct*>(ToRawPtr(Struct));
		}
	}
	return DataTable;
}

UDataTable* UDataTableFactory::MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	return NewObject<UDataTable>(InParent, Name, Flags);
}

#undef LOCTEXT_NAMESPACE // "DataTableFactory"
