// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorFactories.cpp: Editor class factories.
=============================================================================*/

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/Interface.h"
#include "Misc/PackageName.h"
#include "Fonts/FontBulkData.h"
#include "Fonts/CompositeFont.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Internationalization/Regex.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/AppStyle.h"
#include "Engine/EngineTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Model.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Curves/KeyHandle.h"
#include "MaterialExpressionIO.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionMaterialLayer.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/Material.h"
#include "Animation/AnimSequence.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "Engine/Font.h"
#include "Animation/AnimInstance.h"
#include "Engine/Brush.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Factories/Factory.h"
#include "Factories/BlendSpaceFactory1D.h"
#include "Factories/AimOffsetBlendSpaceFactory1D.h"
#include "Factories/BlendSpaceFactoryNew.h"
#include "Factories/AimOffsetBlendSpaceFactoryNew.h"
#include "Factories/BlueprintFactory.h"
#include "Factories/BlueprintFunctionLibraryFactory.h"
#include "Factories/BlueprintMacroFactory.h"
#include "Factories/BlueprintInterfaceFactory.h"
#include "Factories/CurveFactory.h"
#include "Factories/CurveImportFactory.h"
#include "Factories/DataAssetFactory.h"
#include "Factories/DialogueVoiceFactory.h"
#include "Factories/DialogueWaveFactory.h"
#include "Factories/EnumFactory.h"
#include "Factories/ReimportFbxAnimSequenceFactory.h"
#include "Factories/ReimportFbxSkeletalMeshFactory.h"
#include "Factories/ReimportFbxStaticMeshFactory.h"
#include "Factories/FontFactory.h"
#include "Factories/FontFileImportFactory.h"
#include "Factories/ForceFeedbackEffectFactory.h"
#include "Factories/HapticFeedbackEffectCurveFactory.h"
#include "Factories/HapticFeedbackEffectBufferFactory.h"
#include "Factories/HapticFeedbackEffectSoundWaveFactory.h"
#include "Factories/LevelFactory.h"
#include "Factories/MaterialFactoryNew.h"
#include "Factories/UIMaterialFactoryNew.h"
#include "Factories/MaterialFunctionFactoryNew.h"
#include "Factories/MaterialFunctionMaterialLayerFactory.h"
#include "Factories/MaterialFunctionMaterialLayerBlendFactory.h"
#include "Factories/MaterialFunctionInstanceFactory.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Factories/MaterialParameterCollectionFactoryNew.h"
#include "Factories/ModelFactory.h"
#include "Factories/ObjectLibraryFactory.h"
#include "Factories/PackageFactory.h"
#include "Factories/ParticleSystemFactoryNew.h"
#include "Factories/PhysicalMaterialFactoryNew.h"
#include "Factories/PhysicalMaterialMaskFactory.h"
#include "Factories/PolysFactory.h"
#include "Factories/ReverbEffectFactory.h"
#include "Factories/SoundAttenuationFactory.h"
#include "Factories/SoundConcurrencyFactory.h"
#include "Factories/SoundClassFactory.h"
#include "Factories/SoundCueFactoryNew.h"
#include "Factories/ReimportSoundFactory.h"
#include "Factories/SoundMixFactory.h"
#include "Factories/StructureFactory.h"
#include "Factories/StringTableFactory.h"
#include "Factories/SubsurfaceProfileFactory.h"
#include "Factories/SpecularProfileFactory.h"
#include "Factories/NeuralProfileFactory.h"
#include "Factories/ToonProfileFactory.h"
#include "Factories/Texture2dFactoryNew.h"
#include "Engine/Texture.h"
#include "Factories/TextureFactory.h"
#include "Factories/ReimportTextureFactory.h"
#include "Factories/TextureRenderTarget2DArrayFactoryNew.h"
#include "Factories/TextureRenderTargetCubeFactoryNew.h"
#include "Factories/TextureRenderTargetVolumeFactoryNew.h"
#include "Factories/TextureRenderTargetFactoryNew.h"
#include "Factories/TouchInterfaceFactory.h"
#include "Factories/FbxAssetImportData.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxImportUI.h"
#include "Editor/GroupActor.h"
#include "Particles/ParticleSystem.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureLightProfile.h"
#include "SoundCueGraph/SoundCueGraphNode.h"
#include "Exporters/TextureCubeExporterHDR.h"
#include "Exporters/TextureExporterGeneric.h"
#include "Exporters/TextureExporterDDS.h"
#include "Exporters/TextureExporterBMP.h"
#include "Exporters/TextureExporterHDR.h"
#include "Exporters/RenderTargetExporterHDR.h"
#include "Exporters/TextureExporterTGA.h"
#include "Exporters/TextureExporterPNG.h"
#include "Exporters/TextureExporterEXR.h"
#include "EngineGlobals.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "Engine/StaticMesh.h"
#include "Sound/SoundWave.h"
#include "GameFramework/DefaultPhysicsVolume.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "Engine/NeuralProfile.h"
#include "Engine/ToonProfile.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/DataAsset.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Engine/DataTable.h"
#include "Sound/DialogueVoice.h"
#include "Sound/DialogueWave.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialParameterCollection.h"
#include "Engine/ObjectLibrary.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicalMaterials/PhysicalMaterialMask.h"
#include "PhysicalMaterialMaskImport.h"
#include "Engine/Polys.h"
#include "Sound/ReverbEffect.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundMix.h"
#include "Engine/TextureCube.h"
#include "Engine/Texture2DArray.h"
#include "Engine/VolumeTexture.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/TextureRenderTarget.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "Engine/TextureRenderTargetVolume.h"
#include "GameFramework/TouchInterface.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Internationalization/StringTable.h"
#include "Editor.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundNodeAttenuation.h"
#include "Sound/SoundNodeModulator.h"
#include "Factories.h"
#include "NormalMapIdentification.h"
#include "AudioDeviceManager.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ScopedTransaction.h"
#include "BSPOps.h"
#include "LevelUtils.h"
#include "PackageTools.h"
#include "SSkeletonWidget.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "WorldPartition/ContentBundle/ContentBundleActivationScope.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "ActorEditorContext/ScopedActorEditorContextSetExternalDataLayerAsset.h"

#include "DDSFile.h"
#include "IESConverter.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "TgaImageSupport.h"
#include "ImageCore.h"
#include "ImageCoreUtils.h"

#include "FbxImporter.h"
#include "Misc/FbxErrors.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Logging/MessageLog.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Kismet2/StructureEditorUtils.h"

#include "InstancedFoliageActor.h"

#include "Animation/DebugSkelMeshComponent.h"

#include "VT/VirtualTexture.h"
#include "VT/VirtualTextureBuilder.h"

#include "CubemapUnwrapUtils.h"

#include "Components/BrushComponent.h"
#include "EngineUtils.h"
#include "Engine/AssetUserData.h"
#include "Animation/BlendSpace1D.h"
#include "Engine/FontFace.h"
#include "Components/AudioComponent.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "Animation/BlendSpace.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "GameFramework/ForceFeedbackAttenuation.h"
#include "Haptics/HapticFeedbackEffect_Curve.h"
#include "Haptics/HapticFeedbackEffect_Buffer.h"
#include "Haptics/HapticFeedbackEffect_SoundWave.h"
#include "DataTableEditorUtils.h"
#include "KismetCompilerModule.h"
#include "Factories/SubUVAnimationFactory.h"
#include "Particles/SubUVAnimation.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Factories/CanvasRenderTarget2DFactoryNew.h"
#include "ImageUtils.h"
#include "Engine/PreviewMeshCollection.h"
#include "Factories/PreviewMeshCollectionFactory.h"
#include "Factories/ForceFeedbackAttenuationFactory.h"
#include "Misc/FileHelper.h"
#include "ActorGroupingUtils.h"

#include "Editor/EditorPerProjectUserSettings.h"
#include "JsonObjectConverter.h"
#include "MaterialEditorModule.h"
#include "Factories/CurveLinearColorAtlasFactory.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Rendering/SkeletalMeshModel.h"

#include "Misc/App.h"
#include "Subsystems/ImportSubsystem.h"

#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "TextureImportSettings.h"
#include "TextureImportUserSettings.h"
#include "AssetImportTask.h"
#include "ObjectTools.h"

#include "SkinWeightsUtilities.h"
#include "TextureImportUtils.h"
#include "TextureReferenceResolver.h"
#include "UDIMUtilities.h"
#include "FileHelpers.h"
#include "MaterialEditorSettings.h"
#include "StaticMeshOperations.h"
#include "UObject/OverridableManager.h"


DEFINE_LOG_CATEGORY(LogEditorFactories);

#define LOCTEXT_NAMESPACE "EditorFactories"

static TAutoConsoleVariable<int32> CVarAutomaticallySetMaterialUsageInEditorDefault(
	TEXT("r.Material.DefaultAutoMaterialUsage"),
	1,
	TEXT("Whether new Materials should automatically set usage flags in the Editor."),
	ECVF_RenderThreadSafe);

/*------------------------------------------------------------------------------
	Shared - used by multiple factories
------------------------------------------------------------------------------*/

void GetReimportPathFromUser(const FText& TitleLabel, TArray<FString>& InOutFilenames)
{
	FString FileTypes;
	FString AllExtensions;
	// Determine whether we will allow multi select and clear old filenames
	bool bAllowMultiSelect = false;
	InOutFilenames.Empty();

	FileTypes = TEXT("FBX Files (*.fbx)|*.fbx");

	FString DefaultFolder;
	FString DefaultFile;

	// Prompt the user for the filenames
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if (DesktopPlatform)
	{
		void* ParentWindowWindowHandle = NULL;

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if (MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid())
		{
			ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		const FString Title = FString::Printf(TEXT("%s %s"), *NSLOCTEXT("FBXReimport", "ImportContentTypeDialogTitle", "Add import source file for").ToString(), *TitleLabel.ToString());
		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			Title,
			*DefaultFolder,
			*DefaultFile,
			FileTypes,
			bAllowMultiSelect ? EFileDialogFlags::Multiple : EFileDialogFlags::None,
			OpenFilenames
		);
	}

	if (bOpened)
	{
		for (int32 FileIndex = 0; FileIndex < OpenFilenames.Num(); ++FileIndex)
		{
			InOutFilenames.Add(OpenFilenames[FileIndex]);
		}
	}
}


class FAssetClassParentFilter : public IClassViewerFilter
{
public:
	FAssetClassParentFilter()
		: DisallowedClassFlags(CLASS_None), bDisallowBlueprintBase(false)
	{}

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	/** Disallow blueprint base classes. */
	bool bDisallowBlueprintBase;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		bool bAllowed= !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InClass->CanCreateAssetOfClass()
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;

		if (bAllowed && bDisallowBlueprintBase)
		{
			if (FKismetEditorUtilities::CanCreateBlueprintOfClass(InClass))
			{
				return false;
			}
		}

		return bAllowed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if (bDisallowBlueprintBase)
		{
			return false;
		}

		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

namespace UE::EditorFactories::Private
{
	bool CanImportClass(UClass* Class)
	{
		IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
		TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
		if (Class && AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
		{
			return AssetClassPermissionList->PassesFilter(Class->GetPathName());
		}
		return true;
	}

}
/*------------------------------------------------------------------------------
	UTexture2DFactoryNew implementation.
------------------------------------------------------------------------------*/
UTexture2DFactoryNew::UTexture2DFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass		= UTexture2D::StaticClass();
	bCreateNew			= true;
	bEditAfterNew		= true;

	Width = 256;
	Height = 256;
}

bool UTexture2DFactoryNew::ShouldShowInNewMenu() const
{
	// You may not create texture2d assets in the content browser
	return false;
}

UObject* UTexture2DFactoryNew::FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn )
{
	// Do not create a texture with bad dimensions.
	if((Width & (Width - 1)) || (Height & (Height - 1)))
	{
		return nullptr;
	}

	UTexture2D* Object = NewObject<UTexture2D>(InParent, InClass, InName, Flags);

	Object->Source.Init2DWithMipChain(Width, Height, TSF_BGRA8);

	//Set the source art to be white as default.
	if( Object->Source.IsValid() )
	{
		// scope MipLock
		{
			FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadWrite,&Object->Source,0);

			if ( MipLock.IsValid() )
			{
				FMemory::Memset(MipLock.Image.RawData, 255, MipLock.Image.GetImageSizeBytes());
			}
		}

		Object->PostEditChange();
	}
	return Object;
}

/*------------------------------------------------------------------------------
	UMaterialInstanceConstantFactoryNew implementation.
------------------------------------------------------------------------------*/
UMaterialInstanceConstantFactoryNew::UMaterialInstanceConstantFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UMaterialInstanceConstant::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialInstanceConstantFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	auto MIC = NewObject<UMaterialInstanceConstant>(InParent, Class, Name, Flags);
	
	if ( MIC )
	{
		MIC->InitResources();

		if ( InitialParent )
		{
			MIC->SetParentEditorOnly(InitialParent);
		}
	}

	return MIC;
}

bool UMaterialInstanceConstantFactoryNew::ShouldShowInNewMenu() const
{
	return Super::ShouldShowInNewMenu() && GetDefault<UMaterialEditorSettings>()->MaterialInstanceWizardConfiguration == nullptr;
}

/*------------------------------------------------------------------------------
	UMaterialFactoryNew implementation.
------------------------------------------------------------------------------*/
UMaterialFactoryNew::UMaterialFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UMaterial::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


UObject* UMaterialFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UMaterial* NewMaterial = NewObject<UMaterial>(InParent, Class, Name, Flags);

	NewMaterial->bAutomaticallySetUsageInEditor = CVarAutomaticallySetMaterialUsageInEditorDefault.GetValueOnAnyThread() != 0;

	if ( InitialTexture != nullptr )
	{
		// An initial texture was specified, add it and assign it to the BaseColor
		UMaterialExpressionTextureSample* TextureSampler = NewObject<UMaterialExpressionTextureSample>(NewMaterial);
		{
			TextureSampler->MaterialExpressionEditorX = -250;
			TextureSampler->Texture = InitialTexture;
			TextureSampler->AutoSetSampleType();
		}

		NewMaterial->GetExpressionCollection().AddExpression(TextureSampler);

		UMaterialEditorOnlyData* NewMaterialEditorOnly = NewMaterial->GetEditorOnlyData();

		FExpressionOutput& Output = TextureSampler->GetOutputs()[0];
		FExpressionInput& Input = (TextureSampler->SamplerType == SAMPLERTYPE_Normal)
			? (FExpressionInput&)NewMaterialEditorOnly->Normal
			: (FExpressionInput&)NewMaterialEditorOnly->BaseColor;

		Input.Expression = TextureSampler;
		Input.Mask = Output.Mask;
		Input.MaskR = Output.MaskR;
		Input.MaskG = Output.MaskG;
		Input.MaskB = Output.MaskB;
		Input.MaskA = Output.MaskA;

		NewMaterial->PostEditChange();
	}

	return NewMaterial;
}

bool UMaterialFactoryNew::ShouldShowInNewMenu() const
{
	return Super::ShouldShowInNewMenu() && GetDefault<UMaterialEditorSettings>()->MaterialWizardConfiguration == nullptr;
}

/*------------------------------------------------------------------------------
	UUIMaterialFactoryNew implementation.
------------------------------------------------------------------------------*/
UUIMaterialFactoryNew::UUIMaterialFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterial::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

FText UUIMaterialFactoryNew::GetDisplayName() const
{
	return LOCTEXT("UIMaterialFactoryDescription", "UI Material");
}

uint32 UUIMaterialFactoryNew::GetMenuCategories() const
{
	return EAssetTypeCategories::Type::Materials;
}

FText UUIMaterialFactoryNew::GetToolTip() const
{
	return LOCTEXT("UIMaterialFactoryTooltip", "Creates a new Material with the UI Material Domain and turns off Used With Static Mesh usage flag.");
}

UObject* UUIMaterialFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMaterial* NewMaterial = NewObject<UMaterial>(InParent, Class, Name, Flags);
	NewMaterial->bAutomaticallySetUsageInEditor = false;
	NewMaterial->MaterialDomain = EMaterialDomain::MD_UI;
	NewMaterial->SetUsageByFlag(MATUSAGE_StaticMesh, false);
	NewMaterial->CacheShaders(EMaterialShaderPrecompileMode::Background);

	return NewMaterial;
}

bool UUIMaterialFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}

FString UUIMaterialFactoryNew::GetDefaultNewAssetName() const
{
	return FString(TEXT("M_UI_Material"));
}

/*------------------------------------------------------------------------------
	UMaterialFunctionFactoryNew implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionFactoryNew::UMaterialFunctionFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunction::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	return NewObject<UObject>(InParent, Class, Name, Flags);
}

/*------------------------------------------------------------------------------
	UMaterialFunctionMaterialLayerFactory implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionMaterialLayerFactory::UMaterialFunctionMaterialLayerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunctionMaterialLayer::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionMaterialLayerFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UMaterialFunctionMaterialLayer* Function = NewObject<UMaterialFunctionMaterialLayer>(InParent, UMaterialFunctionMaterialLayer::StaticClass(), Name, Flags);
	if (Function)
	{
		Function->SetMaterialFunctionUsage(EMaterialFunctionUsage::MaterialLayer);
	}
	return Function;
}

/*------------------------------------------------------------------------------
	UMaterialFunctionMaterialLayerBlendFactory implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionMaterialLayerBlendFactory::UMaterialFunctionMaterialLayerBlendFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunctionMaterialLayerBlend::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionMaterialLayerBlendFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UMaterialFunctionMaterialLayerBlend* Function = NewObject<UMaterialFunctionMaterialLayerBlend>(InParent, UMaterialFunctionMaterialLayerBlend::StaticClass(), Name, Flags);
	if (Function)
	{
		Function->SetMaterialFunctionUsage(EMaterialFunctionUsage::MaterialLayerBlend);
	}
	return Function;
}


/*------------------------------------------------------------------------------
	UMaterialFunctionInstanceFactory implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionInstanceFactory::UMaterialFunctionInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunctionInstance::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionInstanceFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	auto MFI = NewObject<UMaterialFunctionInstance>(InParent, Class, Name, Flags);

	if (MFI && InitialParent)
	{
		MFI->SetParent(InitialParent);
	}

	return MFI;
}

/*------------------------------------------------------------------------------
UMaterialFunctionMaterialLayerInstanceFactory implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionMaterialLayerInstanceFactory::UMaterialFunctionMaterialLayerInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunctionMaterialLayerInstance::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionMaterialLayerInstanceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto MFI = NewObject<UMaterialFunctionMaterialLayerInstance>(InParent, Class, Name, Flags);

	if (MFI && InitialParent)
	{
		MFI->SetParent(InitialParent);
	}

	return MFI;
}

/*------------------------------------------------------------------------------
UMaterialFunctionMaterialLayerBlendInstanceFactory implementation.
------------------------------------------------------------------------------*/
UMaterialFunctionMaterialLayerBlendInstanceFactory::UMaterialFunctionMaterialLayerBlendInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterialFunctionMaterialLayerBlendInstance::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialFunctionMaterialLayerBlendInstanceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto MFI = NewObject<UMaterialFunctionMaterialLayerBlendInstance>(InParent, Class, Name, Flags);

	if (MFI && InitialParent) 
	{
		MFI->SetParent(InitialParent);
	}

	return MFI;
}

/*------------------------------------------------------------------------------
	UMaterialParameterCollectionFactoryNew implementation.
------------------------------------------------------------------------------*/
UMaterialParameterCollectionFactoryNew::UMaterialParameterCollectionFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UMaterialParameterCollection::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UMaterialParameterCollectionFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UMaterialParameterCollection* MaterialParameterCollection = NewObject<UMaterialParameterCollection>(InParent, Class, Name, Flags);

	if (MaterialParameterCollection)
	{
		MaterialParameterCollection->SetupWorldParameterCollectionInstances();
	}

	return MaterialParameterCollection;
}

/*------------------------------------------------------------------------------
	ULevelFactory.
------------------------------------------------------------------------------*/

ULevelFactory::ULevelFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UWorld::StaticClass();
	Formats.Add(TEXT("t3d;Unreal World"));

	bCreateNew = false;
	bText = true;
	bEditorImport = false;
}

UObject* ULevelFactory::FactoryCreateText(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type,
	const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	return LevelFactoryCreateText(Class, InParent, Name, Flags, Context, Type, Buffer, BufferEnd, Warn, nullptr);
}

UObject* ULevelFactory::LevelFactoryCreateText
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		Type,
	const TCHAR*&		Buffer,
	const TCHAR*		BufferEnd,
	FFeedbackContext*	Warn,
	TArray<AActor*>*    OutActors
)
{
	if (OutActors)
	{
		OutActors->Reset();
	}
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	UWorld* World = GWorld;
	//@todo locked levels - if lock state is persistent, do we need to check for whether the level is locked?
#ifdef MULTI_LEVEL_IMPORT
	// this level is the current level for pasting. If we get a named level, not for pasting, we will look up the level, and overwrite this
	ULevel*				OldCurrentLevel = World->GetCurrentLevel();
	check(OldCurrentLevel);
#endif

	UPackage* RootMapPackage = Cast<UPackage>(InParent);
	TMap<FString, UPackage*> MapPackages;
	// Assumes data is being imported over top of a new, valid map.
	FParse::Next( &Buffer );
	if (GetBEGIN(&Buffer, TEXT("MAP")))
	{
		if (ULevel* Level = Cast<ULevel>(InParent))
		{
			World = Level->GetWorld();
		}

		if (RootMapPackage)
		{
			FString MapName;
			if (FParse::Value(Buffer, TEXT("Name="), MapName))
			{
				// Advance the buffer
				Buffer += FCString::Strlen(TEXT("Name="));
				Buffer += MapName.Len();
				// Check to make sure that there are no naming conflicts
				if( RootMapPackage->Rename(*MapName, nullptr, REN_Test | REN_AllowPackageLinkerMismatch) )
				{
					// Rename it!
					RootMapPackage->Rename(*MapName, nullptr, REN_AllowPackageLinkerMismatch);
				}
				else
				{
					Warn->Logf(ELogVerbosity::Warning, TEXT("The Root map package name : '%s', conflicts with the existing object : '%s'"), *RootMapPackage->GetFullName(), *MapName);
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr );
					return nullptr;
				}
				
				// Stick it in the package map
				MapPackages.Add(MapName, RootMapPackage);
			}
		}
	}
	else
	{
		return World;
	}

	bool bIsExpectingNewMapTag = false;

	// Unselect all actors.
	if (GWorld == World)
	{
		GEditor->SelectNone( false, false );

		// Mark us importing a T3D (only from a file, not from copy/paste).
		GEditor->IsImportingT3D = (FCString::Stricmp(Type,TEXT("paste")) != 0) && (FCString::Stricmp(Type,TEXT("move")) != 0);
		GIsImportingT3D = GEditor->IsImportingT3D;
	}

	// We need to detect if the .t3d file is the entire level or just selected actors, because we
	// don't want to replace the WorldSettings and BuildBrush if they already exist. To know if we
	// can skip the WorldSettings and BuilderBrush (which will always be the first two actors if the entire
	// level was exported), we make sure the first actor is a WorldSettings, if it is, and we already had
	// a WorldSettings, then we skip the builder brush
	// In other words, if we are importing a full level into a full level, we don't want to import
	// the WorldSettings and BuildBrush
	bool bShouldSkipImportSpecialActors = false;
	bool bHitLevelToken = false;

	FString MapPackageText;

	int32 ActorIndex = 0;

	//@todo locked levels - what needs to happen here?


	// Maintain a list of a new actors and the text they were created from.
	TMap<AActor*,FString> NewActorMap;
	TMap< FString, AGroupActor* > NewGroups; // Key=The orig actor's group's name, Value=The new actor's group.
	
	// Maintain a lookup for the new actors, keyed by their source FName.
	TMap<FName, AActor*> NewActorsFNames;

	// Maintain a lookup from existing to new objects, typically actors, used when replacing internal references when copy+pasting / duplicating
	TMap<FSoftObjectPath, UObject*> ExistingToNewMap;

	// Maintain a lookup of the new actors to their parent and socket attachment if provided.
	struct FAttachmentDetail
	{
		const FName ParentName;
		const FName SocketName;
		FAttachmentDetail(const FName InParentName, const FName InSocketName) : ParentName(InParentName), SocketName(InSocketName) {}
	};
	TMap<AActor*,FAttachmentDetail> NewActorsAttachmentMap;

	FString StrLine;
	while( FParse::Line(&Buffer,StrLine) )
	{
		const TCHAR* Str = *StrLine;

		// If we're still waiting to see a 'MAP' tag, then check for that
		if( bIsExpectingNewMapTag )
		{
			if( GetBEGIN( &Str, TEXT("MAP")) )
			{
				bIsExpectingNewMapTag = false;
			}
			else
			{
				// Not a new map tag, so continue on
			}
		}
		else if( GetEND(&Str,TEXT("MAP")) )
		{
			// End of brush polys.
			bIsExpectingNewMapTag = true;
		}
		else if( GetBEGIN(&Str,TEXT("LEVEL")) )
		{
			bHitLevelToken = true;
#ifdef MULTI_LEVEL_IMPORT
			// try to look up the named level. if this fails, we will need to create a new level
			if (ParseObject<ULevel>(Str, TEXT("NAME="), World->GetCurrentLevel(), World->GetOuter()) == false)
			{
				// get the name
				FString LevelName;
				// if there is no name, that means we are pasting, so just put this object into the CurrentLevel - don't make a new one
				if (FParse::Value(Str, TEXT("NAME="), LevelName))
				{
					// create a new named level
					World->SetCurrentLevel( new(World->GetOuter(), *LevelName)ULevel(FObjectInitializer(),FURL(nullptr)) );
				}
			}
#endif
		}
		else if( GetEND(&Str,TEXT("LEVEL")) )
		{
#ifdef MULTI_LEVEL_IMPORT
			// any actors outside of a level block go into the current level
			World->SetCurrentLevel( OldCurrentLevel );
#endif
		}
		else if( GetBEGIN(&Str,TEXT("ACTOR")) )
		{
			UClass* TempClass;
			if( ParseObject<UClass>( Str, TEXT("CLASS="), TempClass, nullptr, EParseObjectLoadingPolicy::FindOrLoad ) )
			{
				// Get actor name.
				FName ActorUniqueName(NAME_None);
				FName ActorSourceName(NAME_None);
				FParse::Value( Str, TEXT("NAME="), ActorSourceName );
				ActorUniqueName = ActorSourceName;
				AActor* Found = nullptr;

				// Make sure this name is unique. We need to do this upfront because we also want to potentially create the Associated BP class using the same name.
				bool bNeedGloballyUniqueName = World->GetCurrentLevel()->IsUsingExternalActors() && CastChecked<AActor>(TempClass->GetDefaultObject())->SupportsExternalPackaging();
				ActorUniqueName = FActorSpawnUtils::MakeUniqueActorName(World->GetCurrentLevel(), TempClass, FActorSpawnUtils::GetBaseName(ActorUniqueName), bNeedGloballyUniqueName);

				// Get the full ExportPath for this actor (used to redirection if the existing actor cannot be found)
				FString ExportedActorFullName;
				FParse::Value(Str, TEXT("ExportPath="), ExportedActorFullName);

				// Get parent name for attachment.
				FName ActorParentName(NAME_None);
				FParse::Value( Str, TEXT("ParentActor="), ActorParentName );

				// Get socket name for attachment.
				FName ActorParentSocket(NAME_None);
				FParse::Value( Str, TEXT("SocketName="), ActorParentSocket );

				// if an archetype was specified in the Begin Object block, use that as the template for the ConstructObject call.
				FString ArchetypeName;
				AActor* Archetype = nullptr;
				if (FParse::Value(Str, TEXT("Archetype="), ArchetypeName))
				{
					// if given a name, break it up along the ' so separate the class from the name
					FString ObjectClass;
					FString ObjectPath;
					if ( FPackageName::ParseExportTextPath(ArchetypeName, &ObjectClass, &ObjectPath) )
					{
						// find the class
						UClass* ArchetypeClass = UClass::TryFindTypeSlow<UClass>(ObjectClass, EFindFirstObjectOptions::EnsureIfAmbiguous);
						if ( ArchetypeClass )
						{
							if ( ArchetypeClass->IsChildOf(AActor::StaticClass()) )
							{
								// if we had the class, find the archetype
								Archetype = Cast<AActor>(StaticFindObject(ArchetypeClass, nullptr, *ObjectPath));
							}
							else
							{
								Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid archetype specified in subobject definition '%s': %s is not a child of Actor"),
									Str, *ObjectClass);
							}
						}
					}
				}


				FGuid ActorContentBundleGuid;
				FParse::Value(Str, TEXT("ActorContentBundleGuid="), ActorContentBundleGuid);
				FContentBundleActivationScope Scope(ActorContentBundleGuid);

				FString ExternalDataLayerAssetPathStr;
				FParse::Value(Str, TEXT("ExternalDataLayerAsset="), ExternalDataLayerAssetPathStr);
				const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(FSoftObjectPath(ExternalDataLayerAssetPathStr).TryLoad());
				FScopedOverrideSpawningLevelMountPointObject ScopeOverrideLevelMountPoint(ExternalDataLayerAsset);

				// If we're pasting from a class that belongs to a map we need to duplicate the class and use that instead
				if (FBlueprintEditorUtils::IsAnonymousBlueprintClass(TempClass))
				{
					UBlueprint* NewBP = DuplicateObject(CastChecked<UBlueprint>(TempClass->ClassGeneratedBy), World->GetCurrentLevel(), *FString::Printf(TEXT("%s_BPClass"), *ActorUniqueName.ToString()));
					if (NewBP)
					{
						NewBP->ClearFlags(RF_Standalone);

						FKismetEditorUtilities::CompileBlueprint(NewBP, EBlueprintCompileOptions::SkipGarbageCollection);

						TempClass = NewBP->GeneratedClass;

						// Since we changed the class we can't use an Archetype,
						// however that is fine since we will have been editing the CDO anyways
						Archetype = nullptr;
					}
				}

				if (TempClass->IsChildOf(AWorldSettings::StaticClass()))
				{
					// if we see a WorldSettings, then we are importing an entire level, so if we
					// are importing into an existing level, then we should not import the next actor
					// which will be the builder brush
					check(ActorIndex == 0);

					// if we have any actors, then we are importing into an existing level
					if (World->GetCurrentLevel()->Actors.Num())
					{
						check(World->GetCurrentLevel()->Actors[0]->IsA(AWorldSettings::StaticClass()));

						// full level into full level, skip the first two actors
						bShouldSkipImportSpecialActors = true;
					}
				}

				// Get property text.
				FString PropText, PropertyLine;
				while
				(	GetEND( &Buffer, TEXT("ACTOR") )==0
				&&	FParse::Line( &Buffer, PropertyLine ) )
				{
					PropText += *PropertyLine;
					PropText += TEXT("\r\n");
				}

				// If we need to skip the WorldSettings and BuilderBrush, skip the first two actors.  Note that
				// at this point, we already know that we have a WorldSettings and BuilderBrush in the .t3d.
				if ( FLevelUtils::IsLevelLocked(World->GetCurrentLevel()) )
				{
					UE_LOGF(LogEditorFactories, Warning, "Import actor: The requested operation could not be completed because the level is locked.");
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr );
					return nullptr;
				}
				else if ( !(bShouldSkipImportSpecialActors && ActorIndex < 2) )
				{
					// Don't import the default physics volume, as it doesn't have a UModel associated with it
					// and thus will not import properly.
					if ( !TempClass->IsChildOf(ADefaultPhysicsVolume::StaticClass()) )
					{
						// Create a new actor.
						FActorSpawnParameters SpawnInfo;
						SpawnInfo.Name = ActorUniqueName;
						SpawnInfo.Template = Archetype;
						SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
						AActor* NewActor = World->SpawnActor( TempClass, nullptr, nullptr, SpawnInfo );
						
						if( NewActor )
						{
							if( UActorGroupingUtils::IsGroupingActive() && !Cast<AGroupActor>(NewActor) )
							{
								bool bGrouped = false;

								AGroupActor** tmpNewGroup = nullptr;
								// We need to add all the objects we selected into groups with new objects that were in their group before.
								FString GroupName;
								FString GroupFolder;
								if(FParse::Value(Str, TEXT("GroupActor="), GroupName))
								{
									tmpNewGroup = NewGroups.Find(GroupName);
									bGrouped = true;
									FParse::Value(Str, TEXT("GroupFolder="), GroupFolder);
								}

								// Don't overwrite if its already been set during actor creation
								// (i.e. by UActorEditorContextSubsystem::ApplyContext).
								// It mirrors the behavior of the task "Importing Actors" below.
								if (NewActor->GetFolder().IsNone())
								{
									FString ActorFolderPath;
									if (FParse::Value(Str, TEXT("ActorFolderPath="), ActorFolderPath))
									{
										NewActor->SetFolderPath(*ActorFolderPath);
									}
								}

								uint32 CopyPasteId;
								if (FParse::Value(Str, TEXT("CopyPasteId="), CopyPasteId))
								{
									check(CopyPasteId != INDEX_NONE);
									NewActor->CopyPasteId = CopyPasteId;

									// Use ActorSourceName + CopyPasteId to find matching source actor
									if (ActorSourceName != NAME_None)
									{
										for (ULevel* Level : World->GetLevels())
										{
											if (Level && IsValid(Level))
											{
												if (AActor* Potential = FindObject<AActor>(Level, *ActorSourceName.ToString()))
												{
													if (Potential->CopyPasteId == CopyPasteId)
													{
														Found = Potential;
														break;
													}
												}
											}
										}
									}
								}
								else if(ActorSourceName != NAME_None)
								{
									// look in the current level for the same named actor
									Found = FindObject<AActor>(World->GetCurrentLevel(), *ActorSourceName.ToString());
								}

								// Does the group exist?
								if(tmpNewGroup)
								{
									AGroupActor* NewActorGroup = *tmpNewGroup;

									// Add it to the group.
									NewActorGroup->Add(*NewActor);
								}
								else if(bGrouped)
								{
									// Create a new group and add the actor.
									AGroupActor* SpawnedGroupActor = NewActor->GetWorld()->SpawnActor<AGroupActor>();
									SpawnedGroupActor->SetFolderPath(*GroupFolder);
									SpawnedGroupActor->Add(*NewActor);

									// Place the group in the map so we can find it later.
									NewGroups.Add( GroupName, SpawnedGroupActor);
									FActorLabelUtilities::SetActorLabelUnique(SpawnedGroupActor, GroupName);
								}

								// If we're copying a sub-group, add add duplicated group to original parent
								// If we're just copying an actor, only append it to the original parent group if unlocked
								if (Found)
								{
									AGroupActor* FoundParent = AGroupActor::GetParentForActor(Found);
									if(FoundParent && (Found->IsA(AGroupActor::StaticClass()) || !FoundParent->IsLocked()) )
									{
										FoundParent->Add(*NewActor);
									}
								}
							}

							// Store the new actor and the text it should be initialized with.
							NewActorMap.Add( NewActor, *PropText );

							// Store the new actor against its source actor name (not the one that may have been made unique)
							if( ActorSourceName!=NAME_None )
							{
								NewActorsFNames.Add( ActorSourceName, NewActor );
								if (Found)
								{
									ExistingToNewMap.Add(Found, NewActor);
								}
								if (!ExportedActorFullName.IsEmpty())
								{
									ExistingToNewMap.Add(ExportedActorFullName, NewActor);
								}
							}

							// Store the new actor with its parent's FName, and socket FName if applicable
							if( ActorParentName!=NAME_None )
							{
								NewActorsAttachmentMap.Add( NewActor, FAttachmentDetail( ActorParentName, ActorParentSocket ) );
							}
						}
					}
				}

				// increment the number of actors we imported
				ActorIndex++;
			}
		}
		else if( GetBEGIN(&Str,TEXT("SURFACE")) )
		{
			UMaterialInterface* SrcMaterial = nullptr;
			FVector SrcBase, SrcTextureU, SrcTextureV, SrcNormal;
			uint32 SrcPolyFlags = PF_DefaultFlags;
			int32 SurfacePropertiesParsed = 0;

			SrcBase = FVector::ZeroVector;
			SrcTextureU = FVector::ZeroVector;
			SrcTextureV = FVector::ZeroVector;
			SrcNormal = FVector::ZeroVector;

			bool bJustParsedTextureName = false;
			bool bFoundSurfaceEnd = false;
			bool bParsedLineSuccessfully = false;

			do
			{
				if( GetEND( &Buffer, TEXT("SURFACE") ) )
				{
					bFoundSurfaceEnd = true;
					bParsedLineSuccessfully = true;
				}
				else if( FParse::Command(&Buffer,TEXT("TEXTURE")) )
				{
					Buffer++;	// Move past the '=' sign

					FString TextureName;
					bParsedLineSuccessfully = FParse::Line(&Buffer, TextureName, true);
					if ( TextureName != TEXT("None") )
					{
						SrcMaterial = Cast<UMaterialInterface>(StaticLoadObject( UMaterialInterface::StaticClass(), nullptr, *TextureName, nullptr, LOAD_NoWarn, nullptr ));
					}
					bJustParsedTextureName = true;
					SurfacePropertiesParsed++;
				}
				else if( FParse::Command(&Buffer,TEXT("BASE")) )
				{
					GetFVECTOR( Buffer, SrcBase );
					SurfacePropertiesParsed++;
				}
				else if( FParse::Command(&Buffer,TEXT("TEXTUREU")) )
				{
					GetFVECTOR( Buffer, SrcTextureU );
					SurfacePropertiesParsed++;
				}
				else if( FParse::Command(&Buffer,TEXT("TEXTUREV")) )
				{
					GetFVECTOR( Buffer, SrcTextureV );
					SurfacePropertiesParsed++;
				}
				else if( FParse::Command(&Buffer,TEXT("NORMAL")) )
				{
					GetFVECTOR( Buffer, SrcNormal );
					SurfacePropertiesParsed++;
				}
				else if( FParse::Command(&Buffer,TEXT("POLYFLAGS")) )
				{
					FParse::Value( Buffer, TEXT("="), SrcPolyFlags );
					SurfacePropertiesParsed++;
				}

				// Parse to the next line only if the texture name wasn't just parsed or if the 
				// end of surface isn't parsed. Don't parse to the next line for the texture 
				// name because a FParse::Line() is called when retrieving the texture name. 
				// Doing another FParse::Line() would skip past a necessary surface property.
				if( !bJustParsedTextureName && !bFoundSurfaceEnd )
				{
					FString DummyLine;
					bParsedLineSuccessfully = FParse::Line( &Buffer, DummyLine );
				}

				// Reset this bool so that we can parse lines starting during next iteration.
				bJustParsedTextureName = false;
			}
			while( !bFoundSurfaceEnd && bParsedLineSuccessfully );

			// There are 6 BSP surface properties exported via T3D. If there wasn't 6 properties 
			// successfully parsed, the parsing failed. This surface isn't valid then.
			if( SurfacePropertiesParsed == 6 )
			{
				const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PasteTextureToSurface", "Paste Texture to Surface") );

				for( int32 j = 0; j < World->GetNumLevels(); ++j )
				{
					ULevel* CurrentLevel = World->GetLevel(j);
					for( int32 i = 0 ; i < CurrentLevel->Model->Surfs.Num() ; i++ )
					{
						FBspSurf* DstSurf = &CurrentLevel->Model->Surfs[i];

						if( DstSurf->PolyFlags & PF_Selected )
						{
							CurrentLevel->Model->ModifySurf( i, 1 );

							const FVector DstNormal = (FVector)CurrentLevel->Model->Vectors[ DstSurf->vNormal ];

							// Need to compensate for changes in the polygon normal.
							const FRotator SrcRot = SrcNormal.Rotation();
							const FRotator DstRot = DstNormal.Rotation();
							const FRotationMatrix RotMatrix( DstRot - SrcRot );

							FVector NewBase	= RotMatrix.TransformPosition( SrcBase );
							FVector NewTextureU = RotMatrix.TransformVector( SrcTextureU );
							FVector NewTextureV = RotMatrix.TransformVector( SrcTextureV );

							DstSurf->Material = SrcMaterial;
							DstSurf->pBase = FBSPOps::bspAddPoint( CurrentLevel->Model, &NewBase, 1 );
							DstSurf->vTextureU = FBSPOps::bspAddVector( CurrentLevel->Model, &NewTextureU, 0 );
							DstSurf->vTextureV = FBSPOps::bspAddVector( CurrentLevel->Model, &NewTextureV, 0 );
							DstSurf->PolyFlags = SrcPolyFlags;

							DstSurf->PolyFlags &= ~PF_Selected;

							CurrentLevel->MarkPackageDirty();

							const bool bUpdateTexCoords = true;
							const bool bOnlyRefreshSurfaceMaterials = false;
							if (GWorld == World)
							{
								GEditor->polyUpdateBrush(CurrentLevel->Model, i, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
							}
						}
					}
				}
			}
		}
		else if (GetBEGIN(&Str,TEXT("MAPPACKAGE")))
		{
			// Get all the text.
			while ((GetEND(&Buffer, TEXT("MAPPACKAGE") )==0) && FParse::Line(&Buffer, StrLine))
			{
				MapPackageText += *StrLine;
				MapPackageText += TEXT("\r\n");
			}
		}
	}

	// Import actor properties.
	// We do this after creating all actors so that actor references can be matched up.
	AWorldSettings* WorldSettings = World->GetWorldSettings();

	if (GIsImportingT3D && (MapPackageText.Len() > 0))
	{
		UPackageFactory* PackageFactory = NewObject<UPackageFactory>();
		check(PackageFactory);

		FName NewPackageName(*(RootMapPackage->GetName()));

		const TCHAR* MapPkg_BufferStart = *MapPackageText;
		const TCHAR* MapPkg_BufferEnd = MapPkg_BufferStart + MapPackageText.Len();
		PackageFactory->FactoryCreateText(UPackage::StaticClass(), nullptr, NewPackageName, RF_NoFlags, 0, TEXT("T3D"), MapPkg_BufferStart, MapPkg_BufferEnd, Warn);
	}

	// Pass 1: Sort out all the properties on the individual actors
	bool bIsMoveToStreamingLevel =(FCString::Stricmp(Type, TEXT("move")) == 0);
	{
		FScopedSlowTask SlowTask(NewActorMap.Num(), LOCTEXT("Importing Actors", "Importing Actors"));
		SlowTask.MakeDialogDelayed(1.f);

		for (auto& ActorMapElement : NewActorMap)
		{
			AActor* Actor = ActorMapElement.Key;

			// Import properties if the new actor is 
			bool		bActorChanged = false;
			const FString&	PropText = ActorMapElement.Value;
			if ( Actor->ShouldImport(FStringView(PropText), bIsMoveToStreamingLevel) )
			{
				const FName OldPath = Actor->GetFolderPath();

				Actor->PreEditChange(nullptr);

				// Remapping of sub-objects across actors is currently dependant on the order in which we import those actors 
				// so for now make it unsupported for consistency by copying the ExistingToNewMap for every imported actor.
				TMap<FSoftObjectPath, UObject*> CopyExistingToNewMap(ExistingToNewMap);
				ImportObjectProperties( (uint8*)Actor, *PropText, Actor->GetClass(), Actor, Actor, Warn, 0, INDEX_NONE, NULL, &CopyExistingToNewMap);
				bActorChanged = true;

				// Path might have been set through OnLevelActorAdded (i.e. by UActorEditorContextSubsystem).
				// If that's the case, then let's restore it here, as ImportObjectProperties might have
				// overridden that with an imported value that we'd like to discard.
				if (!OldPath.IsNone())
				{
					Actor->SetFolderPath(OldPath);
				}

				if (GWorld == World)
				{
					GEditor->SelectActor( Actor, true, false, true );
				}
			}
			else // This actor is new, but rejected to import its properties, so just delete...
			{
				Actor->Destroy();
				continue;
			}

			// If this is a newly imported brush, validate it.  If it's a newly imported dynamic brush, rebuild it first.
			// Previously, this just called bspValidateBrush.  However, that caused the dynamic brushes which require a valid BSP tree
			// to be built to break after being duplicated.  Calling RebuildBrush will rebuild the BSP tree from the imported polygons.
			ABrush* Brush = Cast<ABrush>(Actor);
			if( bActorChanged && Brush && Brush->Brush )
			{
				const bool bIsStaticBrush = Brush->IsStaticBrush();
				if( !bIsStaticBrush )
				{
					FBSPOps::RebuildBrush( Brush->Brush );
				}

				FBSPOps::bspValidateBrush(Brush->Brush, true, false);
			}

			// Copy brushes' model pointers over to their BrushComponent, to keep compatibility with old T3Ds.
			if( Brush && bActorChanged )
			{
				if( Brush->GetBrushComponent() ) // Should always be the case, but not asserting so that old broken content won't crash.
				{
					Brush->GetBrushComponent()->Brush = Brush->Brush;

					// We need to avoid duplicating default/ builder brushes. This is done by destroying all brushes that are CSG_Active and are not
					// the default brush in their respective levels.
					if( Brush->IsStaticBrush() && Brush->BrushType==Brush_Default )
					{
						bool bIsDefaultBrush = false;
						
						// Iterate over all levels and compare current actor to the level's default brush.
						for( int32 LevelIndex=0; LevelIndex<World->GetNumLevels(); LevelIndex++ )
						{
							ULevel* Level = World->GetLevel(LevelIndex);
							if(Level->GetDefaultBrush() == Brush)
							{
								bIsDefaultBrush = true;
								break;
							}
						}

						// Destroy actor if it's a builder brush but not the default brush in any of the currently loaded levels.
						if( !bIsDefaultBrush )
						{
							World->DestroyActor( Brush );

							// Since the actor has been destroyed, skip the rest of this iteration of the loop.
							continue;
						}
					}
				}
			}
			
			// If the actor was imported . . .
			if( bActorChanged )
			{
				// Let the actor deal with having been imported, if desired.
				Actor->PostEditImport();

				// Notify actor its properties have changed.
				Actor->PostEditChange();
			}
			SlowTask.EnterProgressFrame();
		}
	}

	// Pass 2: Sort out any attachment parenting on the new actors now that all actors have the correct properties set
	for( auto It = NewActorMap.CreateIterator(); It; ++It )
	{
		AActor* const Actor = It.Key();

		// Fixup parenting
		FAttachmentDetail* ActorAttachmentDetail = NewActorsAttachmentMap.Find( Actor );
		if( ActorAttachmentDetail != nullptr )
		{
			AActor* ActorParent = nullptr;
			// Try to find the new copy of the parent
			AActor** NewActorParent = NewActorsFNames.Find( ActorAttachmentDetail->ParentName );
			if ( NewActorParent != nullptr )
			{
				ActorParent = *NewActorParent;
			}
			// Try to find an already existing parent
			if( ActorParent == nullptr )
			{
				ActorParent = FindObject<AActor>( World->GetCurrentLevel(), *ActorAttachmentDetail->ParentName.ToString() );
			}
			// Parent the actors
			if( GWorld == World && ActorParent != nullptr )
			{
				// Make sure our parent isn't selected (would cause GEditor->ParentActors to fail)
				const bool bParentWasSelected = ActorParent->IsSelected();
				if(bParentWasSelected)
				{
					GEditor->SelectActor(ActorParent, false, false, true);
				}

				GEditor->ParentActors( ActorParent, Actor, ActorAttachmentDetail->SocketName );

				if(bParentWasSelected)
				{
					GEditor->SelectActor(ActorParent, true, false, true);
				}
			}
		}
	}

	// Go through all the groups we added and finalize them.
	for(TMap< FString, AGroupActor* >::TIterator It(NewGroups); It; ++It)
	{
		It.Value()->CenterGroupLocation();
		It.Value()->Lock();
	}

	// Mark us as no longer importing a T3D.
	if (GWorld == World)
	{
		GEditor->IsImportingT3D = 0;
		GIsImportingT3D = false;

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, World );
	}

	if (OutActors)
	{
		OutActors->SetNumUninitialized(NewActorMap.Num());
		
		int32 Index = 0;
		for (const TPair<AActor*, FString>& Pair : NewActorMap)
		{
			(*OutActors)[Index++] = Pair.Key;
		}
	}

	return World;
}

/*-----------------------------------------------------------------------------
	UPackageFactory.
-----------------------------------------------------------------------------*/
UPackageFactory::UPackageFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPackage::StaticClass();
	Formats.Add(TEXT("T3DPKG;Unreal Package"));

	bCreateNew = false;
	bText = true;
	bEditorImport = false;
}

UObject* UPackageFactory::FactoryCreateText( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn )
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	bool bSavedImportingT3D = GIsImportingT3D;
	// Mark us as importing a T3D.
	GEditor->IsImportingT3D = true;
	GIsImportingT3D = true;

	if (InParent != nullptr)
	{
		return nullptr;
	}

	TMap<FString, UPackage*> MapPackages;
	bool bImportingMapPackage = false;

	UPackage* TopLevelPackage = nullptr;
	UPackage* RootMapPackage = nullptr;
	UWorld* World = GWorld;
	if (World)
	{
		RootMapPackage = World->GetOutermost();
	}

	if (RootMapPackage)
	{
		if (RootMapPackage->GetName() == Name.ToString())
		{
			// Loading into the Map package!
			MapPackages.Add(RootMapPackage->GetName(), RootMapPackage);
			TopLevelPackage = RootMapPackage;
			bImportingMapPackage = true;
		}
	}

	// Unselect all actors.
	GEditor->SelectNone( false, false );

	// Mark us importing a T3D (only from a file, not from copy/paste).
	GEditor->IsImportingT3D = FCString::Stricmp(Type,TEXT("paste")) != 0;
	GIsImportingT3D = GEditor->IsImportingT3D;

	// Maintain a list of a new package objects and the text they were created from.
	TMap<UObject*,FString> NewPackageObjectMap;

	FString StrLine;
	while( FParse::Line(&Buffer,StrLine) )
	{
		const TCHAR* Str = *StrLine;

		if (GetBEGIN(&Str, TEXT("TOPLEVELPACKAGE")) && !bImportingMapPackage)
		{
			//Begin TopLevelPackage Class=Package Name=ExportTest_ORIG Archetype=Package'Core.Default__Package'
			UClass* TempClass;
			if( ParseObject<UClass>( Str, TEXT("CLASS="), TempClass, nullptr ) )
			{
				// Get actor name.
				FName PackageName(NAME_None);
				FParse::Value( Str, TEXT("NAME="), PackageName );

				if (FindObject<UPackage>(nullptr, *(PackageName.ToString())))
				{
					UE_LOGF(LogEditorFactories, Warning, "Package factory can only handle the map package or new packages!");
					return nullptr;
				}
				TopLevelPackage = CreatePackage( *(PackageName.ToString()));
				TopLevelPackage->SetFlags(RF_Standalone|RF_Public);
				MapPackages.Add(TopLevelPackage->GetName(), TopLevelPackage);

				// if an archetype was specified in the Begin Object block, use that as the template for the ConstructObject call.
				FString ArchetypeName;
				AActor* Archetype = nullptr;
				if (FParse::Value(Str, TEXT("Archetype="), ArchetypeName))
				{
				}
			}
		}
		else if (GetBEGIN(&Str,TEXT("PACKAGE")))
		{
			FString ParentPackageName;
			FParse::Value(Str, TEXT("PARENTPACKAGE="), ParentPackageName);
			UClass* PkgClass;
			if (ParseObject<UClass>(Str, TEXT("CLASS="), PkgClass, nullptr))
			{
				// Get the name of the object.
				FName NewPackageName(NAME_None);
				FParse::Value(Str, TEXT("NAME="), NewPackageName);

				// if an archetype was specified in the Begin Object block, use that as the template for the ConstructObject call.
				FString ArchetypeName;
				UPackage* Archetype = nullptr;
				if (FParse::Value(Str, TEXT("Archetype="), ArchetypeName))
				{
					// if given a name, break it up along the ' so separate the class from the name
					FString ObjectClass;
					FString ObjectPath;
					if ( FPackageName::ParseExportTextPath(ArchetypeName, &ObjectClass, &ObjectPath) )
					{
						// find the class
						UClass* ArchetypeClass = UClass::TryFindTypeSlow<UClass>(ObjectClass, EFindFirstObjectOptions::EnsureIfAmbiguous);
						if (ArchetypeClass)
						{
							if (ArchetypeClass->IsChildOf(UPackage::StaticClass()))
							{
								// if we had the class, find the archetype
								Archetype = Cast<UPackage>(StaticFindObject(ArchetypeClass, nullptr, *ObjectPath));
							}
							else
							{
								Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid archetype specified in subobject definition '%s': %s is not a child of Package"),
									Str, *ObjectClass);
							}
						}
					}

					UPackage* ParentPkg = nullptr;
					UPackage** ppParentPkg = MapPackages.Find(ParentPackageName);
					if (ppParentPkg)
					{
						ParentPkg = *ppParentPkg;
					}
					check(ParentPkg);

					auto NewPackage = NewObject<UPackage>(ParentPkg, NewPackageName, RF_NoFlags, Archetype);
					check(NewPackage);
					NewPackage->SetFlags(RF_Standalone|RF_Public);
					MapPackages.Add(NewPackageName.ToString(), NewPackage);
				}
			}
		}
	}

	for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
	{
		UObject* LoadObject = *ObjIt;

		if (LoadObject)
		{
			bool bModifiedObject = false;

			FString* PropText = NewPackageObjectMap.Find(LoadObject);
			if (PropText)
			{
				LoadObject->PreEditChange(nullptr);
				ImportObjectProperties((uint8*)LoadObject, **PropText, LoadObject->GetClass(), LoadObject, LoadObject, Warn, 0 );
				bModifiedObject = true;
			}

			if (bModifiedObject)
			{
				// Let the actor deal with having been imported, if desired.
				LoadObject->PostEditImport();
				// Notify actor its properties have changed.
				LoadObject->PostEditChange();
				LoadObject->SetFlags(RF_Standalone | RF_Public);
				LoadObject->MarkPackageDirty();
			}
		}
	}

	// Mark us as no longer importing a T3D.
	GEditor->IsImportingT3D = bSavedImportingT3D;
	GIsImportingT3D = bSavedImportingT3D;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, TopLevelPackage );

	return TopLevelPackage;
}

/*-----------------------------------------------------------------------------
	UPolysFactory.
-----------------------------------------------------------------------------*/

UPolysFactory::UPolysFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UPolys::StaticClass();
	Formats.Add(TEXT("t3d;Unreal brush text"));
	bCreateNew = false;
	bText = true;
}

UObject* UPolysFactory::FactoryCreateText
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		Type,
	const TCHAR*&		Buffer,
	const TCHAR*		BufferEnd,
	FFeedbackContext*	Warn
)
{
	FVector3f PointPool[4096] = {};
	int32 NumPoints = 0;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	// Create polys.	
	UPolys* Polys = Context ? CastChecked<UPolys>(Context) : NewObject<UPolys>(InParent, Name, Flags);

	// Eat up if present.
	GetBEGIN( &Buffer, TEXT("POLYLIST") );

	// Parse all stuff.
	int32 First=1, GotBase=0;
	FString StrLine, ExtraLine;
	FPoly Poly;
	while( FParse::Line( &Buffer, StrLine ) )
	{
		const TCHAR* Str = *StrLine;
		if( GetEND(&Str,TEXT("POLYLIST")) )
		{
			// End of brush polys.
			break;
		}
		//
		//
		// AutoCad - DXF File
		//
		//
		else if( FCString::Strstr(Str,TEXT("ENTITIES")) && First )
		{
			UE_LOGF(LogEditorFactories, Log, "Reading Autocad DXF file");
			int32 Started=0, IsFace=0;
			FPoly NewPoly; NewPoly.Init();
			NumPoints = 0;

			while
			(	FParse::Line( &Buffer, StrLine, 1 )
			&&	FParse::Line( &Buffer, ExtraLine, 1 ) )
			{
				// Handle the line.
				Str = *ExtraLine;
				int32 Code = FCString::Atoi(*StrLine);
				if( Code==0 )
				{
					// Finish up current poly.
					if( Started )
					{
						if( NewPoly.Vertices.Num() == 0 )
						{
							// Got a vertex definition.
							NumPoints++;
						}
						else if( NewPoly.Vertices.Num()>=3 )
						{
							// Got a poly definition.
							if( IsFace ) NewPoly.Reverse();
							NewPoly.Base = NewPoly.Vertices[0];
							NewPoly.Finalize(nullptr,0);
							new(Polys->Element)FPoly( NewPoly );
						}
						else
						{
							// Bad.
							Warn->Logf( TEXT("DXF: Bad vertex count %i"), NewPoly.Vertices.Num() );
						}
						
						// Prepare for next.
						NewPoly.Init();
					}
					Started=0;

					if( FParse::Command(&Str,TEXT("VERTEX")) )
					{
						// Start of new vertex.
						PointPool[NumPoints] = FVector3f::ZeroVector;
						Started = 1;
						IsFace  = 0;
					}
					else if( FParse::Command(&Str,TEXT("3DFACE")) )
					{
						// Start of 3d face definition.
						Started = 1;
						IsFace  = 1;
					}
					else if( FParse::Command(&Str,TEXT("SEQEND")) )
					{
						// End of sequence.
						NumPoints=0;
					}
					else if( FParse::Command(&Str,TEXT("EOF")) )
					{
						// End of file.
						break;
					}
				}
				else if( Started )
				{
					// Replace commas with periods to handle european dxf's.
					//for( TCHAR* Stupid = FCString::Strchr(*ExtraLine,','); Stupid; Stupid=FCString::Strchr(Stupid,',') )
					//	*Stupid = '.';

					// Handle codes.
					if( Code>=10 && Code<=19 )
					{
						// X coordinate.
						int32 VertexIndex = Code-10;
						if( IsFace && VertexIndex >= NewPoly.Vertices.Num() )
						{
							NewPoly.Vertices.AddZeroed(VertexIndex - NewPoly.Vertices.Num() + 1);
						}
						NewPoly.Vertices[VertexIndex].X = PointPool[NumPoints].X = FCString::Atof(*ExtraLine);
					}
					else if( Code>=20 && Code<=29 )
					{
						// Y coordinate.
						int32 VertexIndex = Code-20;
						NewPoly.Vertices[VertexIndex].Y = PointPool[NumPoints].Y = FCString::Atof(*ExtraLine);
					}
					else if( Code>=30 && Code<=39 )
					{
						// Z coordinate.
						int32 VertexIndex = Code-30;
						NewPoly.Vertices[VertexIndex].Z = PointPool[NumPoints].Z = FCString::Atof(*ExtraLine);
					}
					else if( Code>=71 && Code<=79 && (Code-71)==NewPoly.Vertices.Num() )
					{
						int32 iPoint = FMath::Abs(FCString::Atoi(*ExtraLine));
						if (iPoint > 0 && iPoint <= NumPoints)
						{
							NewPoly.Vertices.Emplace(PointPool[iPoint - 1]);
						}
						else UE_LOGF(LogEditorFactories, Warning, "DXF: Invalid point index %i/%i", iPoint, NumPoints );
					}
				}
			}
		}
		//
		//
		// 3D Studio MAX - ASC File
		//
		//
		else if( FCString::Strstr(Str,TEXT("Tri-mesh,")) && First )
		{
			UE_LOGF(LogEditorFactories, Log,  "Reading 3D Studio ASC file" );
			NumPoints = 0;

			AscReloop:
			int32 TempNumPolys=0, TempVerts=0;
			while( FParse::Line( &Buffer, StrLine ) )
			{
				Str = *StrLine;

				FString VertText = FString::Printf( TEXT("Vertex %i:"), NumPoints );
				FString FaceText = FString::Printf( TEXT("Face %i:"), TempNumPolys );
				if( FCString::Strstr(Str,*VertText) )
				{
					PointPool[NumPoints].X = FCString::Atof(FCString::Strstr(Str,TEXT("X:"))+2);
					PointPool[NumPoints].Y = FCString::Atof(FCString::Strstr(Str,TEXT("Y:"))+2);
					PointPool[NumPoints].Z = FCString::Atof(FCString::Strstr(Str,TEXT("Z:"))+2);
					NumPoints++;
					TempVerts++;
				}
				else if( FCString::Strstr(Str,*FaceText) )
				{
					Poly.Init();
					new(Poly.Vertices)FVector3f(PointPool[FCString::Atoi(FCString::Strstr(Str,TEXT("A:"))+2)]);
					new(Poly.Vertices)FVector3f(PointPool[FCString::Atoi(FCString::Strstr(Str,TEXT("B:"))+2)]);
					new(Poly.Vertices)FVector3f(PointPool[FCString::Atoi(FCString::Strstr(Str,TEXT("C:"))+2)]);
					Poly.Base = Poly.Vertices[0];
					Poly.Finalize(nullptr,0);
					new(Polys->Element)FPoly(Poly);
					TempNumPolys++;
				}
				else if( FCString::Strstr(Str,TEXT("Tri-mesh,")) )
					goto AscReloop;
			}
			UE_LOGF(LogEditorFactories, Log,  "Imported %i vertices, %i faces", TempVerts, Polys->Element.Num() );
		}
		//
		//
		// T3D FORMAT
		//
		//
		else if( GetBEGIN(&Str,TEXT("POLYGON")) )
		{
			// Init to defaults and get group/item and texture.
			Poly.Init();
			FParse::Value( Str, TEXT("LINK="), Poly.iLink );
			FParse::Value( Str, TEXT("ITEM="), Poly.ItemName );
			FParse::Value( Str, TEXT("FLAGS="), Poly.PolyFlags );
			FParse::Value( Str, TEXT("LightMapScale="), Poly.LightMapScale );
			Poly.PolyFlags &= ~PF_NoImport;

			FString TextureName;
			// only load the texture if it was present
			if (FParse::Value( Str, TEXT("TEXTURE="), TextureName ))
			{
				Poly.Material = FindFirstObject<UMaterialInterface>( *TextureName, EFindFirstObjectOptions::EnsureIfAmbiguous );
/***
				if (Poly.Material == nullptr)
				{
					Poly.Material = Cast<UMaterialInterface>(StaticLoadObject( UMaterialInterface::StaticClass(), nullptr, *TextureName, nullptr,  LOAD_NoWarn, nullptr ) );
				}
***/
			}
		}
		else if( FParse::Command(&Str,TEXT("PAN")) )
		{
			int32	PanU = 0,
				PanV = 0;

			FParse::Value( Str, TEXT("U="), PanU );
			FParse::Value( Str, TEXT("V="), PanV );

			Poly.Base += Poly.TextureU * PanU;
			Poly.Base += Poly.TextureV * PanV;
		}
		else if( FParse::Command(&Str,TEXT("ORIGIN")) )
		{
			GotBase=1;
			FVector Base;
			GetFVECTOR( Str, Base );
			Poly.Base = (FVector3f)Base;
		}
		else if( FParse::Command(&Str,TEXT("VERTEX")) )
		{
			FVector TempVertex;
			GetFVECTOR( Str, TempVertex );
			new(Poly.Vertices) FVector3f(TempVertex);
		}
		else if( FParse::Command(&Str,TEXT("TEXTUREU")) )
		{
			FVector TextureU;
			GetFVECTOR( Str, TextureU );
			Poly.TextureU = (FVector3f)TextureU;
		}
		else if( FParse::Command(&Str,TEXT("TEXTUREV")) )
		{
			FVector TextureV;
			GetFVECTOR(Str, TextureV);
			Poly.TextureV = (FVector3f)TextureV;
		}
		else if( GetEND(&Str,TEXT("POLYGON")) )
		{
			if( !GotBase )
				Poly.Base = Poly.Vertices[0];
			if( Poly.Finalize(nullptr,1)==0 )
				new(Polys->Element)FPoly(Poly);
			GotBase=0;
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Polys );

	// Success.
	return Polys;
}

/*-----------------------------------------------------------------------------
	UModelFactory.
-----------------------------------------------------------------------------*/
UModelFactory::UModelFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UModel::StaticClass();
	Formats.Add(TEXT("t3d;Unreal model text"));
	bCreateNew = false;
	bText = true;
}

UObject* UModelFactory::FactoryCreateText
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		Type,
	const TCHAR*&		Buffer,
	const TCHAR*		BufferEnd,
	FFeedbackContext*	Warn
)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	ABrush* TempOwner = (ABrush*)Context;
	UModel* Model = NewObject<UModel>(InParent, Name, Flags);
	Model->Initialize(TempOwner, true);

	const TCHAR* StrPtr;
	FString StrLine;
	if( TempOwner )
	{
		TempOwner->InitPosRotScale();
		GEditor->GetSelectedActors()->Deselect( TempOwner );
	}
	while( FParse::Line( &Buffer, StrLine ) )
	{
		StrPtr = *StrLine;
		if( GetEND(&StrPtr,TEXT("BRUSH")) )
		{
			break;
		}
		else if( GetBEGIN (&StrPtr,TEXT("POLYLIST")) )
		{
			UPolysFactory* PolysFactory = NewObject<UPolysFactory>();
			Model->Polys = (UPolys*)PolysFactory->FactoryCreateText(UPolys::StaticClass(),Model,NAME_None,RF_Transactional,nullptr,Type,Buffer,BufferEnd,Warn);
			check(Model->Polys);
		}
		if( TempOwner )
		{
			if(FParse::Command(&StrPtr,TEXT("PREPIVOT"	))) 
			{
				FVector TempPrePivot(0.f);
				GetFVECTOR 	(StrPtr,TempPrePivot);
				TempOwner->SetPivotOffset(TempPrePivot);
			}
			else if (FParse::Command(&StrPtr,TEXT("LOCATION"	))) 
			{
				FVector NewLocation(0.f);
				GetFVECTOR	(StrPtr,NewLocation);
				TempOwner->SetActorLocation(NewLocation, false);
			}
			else if (FParse::Command(&StrPtr,TEXT("ROTATION"	))) 
			{
				FRotator NewRotation;
				GetFROTATOR  (StrPtr,NewRotation,1);
				TempOwner->SetActorRotation(NewRotation);
			}
			if( FParse::Command(&StrPtr,TEXT("SETTINGS")) )
			{
				uint8 BrushType = (uint8)TempOwner->BrushType;
				FParse::Value( StrPtr, TEXT("BRUSHTYPE="), BrushType);
				TempOwner->BrushType = EBrushType(BrushType);
				FParse::Value( StrPtr, TEXT("POLYFLAGS="), TempOwner->PolyFlags );
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Model );

	return Model;
}

/*------------------------------------------------------------------------------
	UParticleSystemFactoryNew.
------------------------------------------------------------------------------*/
UParticleSystemFactoryNew::UParticleSystemFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UParticleSystem::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UParticleSystemFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	return NewObject<UObject>(InParent, Class, Name, Flags);
}

USubUVAnimationFactory::USubUVAnimationFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = USubUVAnimation::StaticClass();
}

UObject* USubUVAnimationFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USubUVAnimation* NewAsset = NewObject<USubUVAnimation>(InParent, Class, Name, Flags | RF_Transactional);

	if ( InitialTexture != nullptr )
	{
		//@todo - auto-detect SubImages_Horizontal and SubImages_Vertical from texture contents
		NewAsset->SubUVTexture = InitialTexture;
		NewAsset->PostEditChange();
	}

	return NewAsset;
}

uint32 USubUVAnimationFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

/*------------------------------------------------------------------------------
	UPhysicalMaterialFactoryNew.
------------------------------------------------------------------------------*/
UPhysicalMaterialFactoryNew::UPhysicalMaterialFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UPhysicalMaterial::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UPhysicalMaterialFactoryNew::ConfigureProperties()
{
	// nullptr the DataAssetClass so we can check for selection
	PhysicalMaterialClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UPhysicalMaterial::StaticClass());

	const FText TitleText = LOCTEXT("CreatePhysicalMaterial", "Pick Physical Material Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UPhysicalMaterial::StaticClass());

	if (bPressedOk)
	{
		PhysicalMaterialClass = ChosenClass;
	}

	return bPressedOk;
}
UObject* UPhysicalMaterialFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	EObjectFlags MaterialFlags = PhysicalMaterialClass.Get() ? Flags | RF_Transactional : Flags;
	UClass* ClassToUse = PhysicalMaterialClass.Get() ? PhysicalMaterialClass.Get() : Class;
	
	check(ClassToUse->IsChildOf(UPhysicalMaterial::StaticClass()));

	UPhysicalMaterial* NewMaterial = NewObject<UPhysicalMaterial>(InParent, ClassToUse, Name, MaterialFlags);
	// Assign a random debug color in the factory. The user can then change it at will : 
	NewMaterial->DebugColor = FLinearColor::MakeRandomSeededColor(GetTypeHash(NewMaterial->GetPathName()));

	// A call to get will ensure any physics-engine specific data is built
	NewMaterial->GetPhysicsMaterial();
	return NewMaterial;
}

/*------------------------------------------------------------------------------
	UPhysicalMaterialMaskFactory.
------------------------------------------------------------------------------*/
UPhysicalMaterialMaskFactory::UPhysicalMaterialMaskFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPhysicalMaterialMask::StaticClass();

	bCreateNew = true;
	bText = false;
	bEditAfterNew = false;
	bEditorImport = true;

	// Required to allow texture factory to take priority when importing new image files
	ImportPriority = DefaultImportPriority - 1;
}

bool UPhysicalMaterialMaskFactory::ConfigureProperties()
{
	// nullptr the DataAssetClass so we can check for selection
	PhysicalMaterialMaskClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UPhysicalMaterialMask::StaticClass());

	const FText TitleText = LOCTEXT("CreatePhysicalMaterialMask", "Pick Physical Material Mask Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UPhysicalMaterialMask::StaticClass());

	if (bPressedOk)
	{
		PhysicalMaterialMaskClass = ChosenClass;
	}

	return bPressedOk;
}
UObject* UPhysicalMaterialMaskFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (PhysicalMaterialMaskClass != nullptr)
	{
		return NewObject<UPhysicalMaterialMask>(InParent, PhysicalMaterialMaskClass, Name, Flags | RF_Transactional);
	}
	else
	{
		// if we have no data asset class, use the passed-in class instead
		check(Class->IsChildOf(UPhysicalMaterialMask::StaticClass()));
		return NewObject<UPhysicalMaterialMask>(InParent, Class, Name, Flags);
	}
}

bool UPhysicalMaterialMaskFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(Obj);
	if (PhysMatMask)
	{
		if (PhysMatMask->AssetImportData)
		{
			FString FileExtension = FPaths::GetExtension(PhysMatMask->AssetImportData->GetFirstFilename());
			if (FileExtension.Equals(TEXT("png"), ESearchCase::IgnoreCase) || FileExtension.Equals("jpg", ESearchCase::IgnoreCase))
			{
				OutFilenames.Add(PhysMatMask->AssetImportData->GetFirstFilename());
			}
		}

		return true;
	}
	return false;
}

void UPhysicalMaterialMaskFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(Obj);
	if (PhysMatMask && ensure(NewReimportPaths.Num() == 1))
	{
		PhysMatMask->Modify();

		if (!PhysMatMask->AssetImportData)
		{
			PhysMatMask->AssetImportData = NewObject<UAssetImportData>(PhysMatMask, TEXT("AssetImportData"), RF_NoFlags);
		}

		PhysMatMask->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UPhysicalMaterialMaskFactory::Reimport( UObject* Obj )
{
	if (!Obj || !Obj->IsA(UPhysicalMaterialMask::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	UPhysicalMaterialMask* PhysMatMask = Cast<UPhysicalMaterialMask>(Obj);

	return FPhysicalMaterialMaskImport::ReimportMaskTexture(PhysMatMask);
}

int32 UPhysicalMaterialMaskFactory::GetPriority() const
{
	return ImportPriority;
}

/*-----------------------------------------------------------------------------
	UTextureRenderTargetFactoryNew
-----------------------------------------------------------------------------*/

UTextureRenderTargetFactoryNew::UTextureRenderTargetFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTextureRenderTarget2D::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 256;
	Height = 256;
	Format = 0;
}


UObject* UTextureRenderTargetFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	// create the new object
	UTextureRenderTarget2D* Result = NewObject<UTextureRenderTarget2D>(InParent, Class, Name, Flags);
	// initialize the resource
	Result->InitAutoFormat( Width, Height );
	return( Result );
}

/*-----------------------------------------------------------------------------
	UCanvasRenderTargetFactoryNew
-----------------------------------------------------------------------------*/

UCanvasRenderTarget2DFactoryNew::UCanvasRenderTarget2DFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCanvasRenderTarget2D::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 256;
	Height = 256;
	Format = 0;
}


UObject* UCanvasRenderTarget2DFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	// create the new object
	UCanvasRenderTarget2D* Result = NewObject<UCanvasRenderTarget2D>(InParent, Class, Name, Flags);
	check(Result);
	// initialize the resource
	Result->InitAutoFormat( Width, Height );
	return( Result );
}

/*-----------------------------------------------------------------------------
UCurveLinearColorAtlasFactory
-----------------------------------------------------------------------------*/

UCurveLinearColorAtlasFactory::UCurveLinearColorAtlasFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveLinearColorAtlas::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 64;
}

FText UCurveLinearColorAtlasFactory::GetDisplayName() const
{
	return LOCTEXT("CurveAtlas", "Curve Atlas");
}

UObject* UCurveLinearColorAtlasFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCurveLinearColorAtlas* Object = NewObject<UCurveLinearColorAtlas>(InParent, Class, Name, Flags);
	
	Object->TextureSize = FMath::Max(Width,2);

	Object->UpdateTextures();

	return Object;
}

/*-----------------------------------------------------------------------------
	UTextureRenderTarget2DArrayFactoryNew
-----------------------------------------------------------------------------*/
UTextureRenderTarget2DArrayFactoryNew::UTextureRenderTarget2DArrayFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UTextureRenderTarget2DArray::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 256;
	Height = 256;
	Slices = 1;
	Format = 0;
}

UObject* UTextureRenderTarget2DArrayFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// create the new object
	UTextureRenderTarget2DArray* Result = NewObject<UTextureRenderTarget2DArray>(InParent, Class, Name, Flags);

	// initialize the resource
	Result->InitAutoFormat(Width, Height, Slices);

	return (Result);
}

/*-----------------------------------------------------------------------------
	UTextureRenderTargetCubeFactoryNew
-----------------------------------------------------------------------------*/
UTextureRenderTargetCubeFactoryNew::UTextureRenderTargetCubeFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UTextureRenderTargetCube::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 256;
	Format = 0;
}

UObject* UTextureRenderTargetCubeFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// create the new object
	UTextureRenderTargetCube* Result = NewObject<UTextureRenderTargetCube>(InParent, Class, Name, Flags);

	// initialize the resource
	Result->InitAutoFormat(Width);

	return (Result);
}

/*-----------------------------------------------------------------------------
	UTextureRenderTargetVolumeFactoryNew
-----------------------------------------------------------------------------*/
UTextureRenderTargetVolumeFactoryNew::UTextureRenderTargetVolumeFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UTextureRenderTargetVolume::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;

	Width = 64;
	Height = 64;
	Depth = 64;
	Format = 0;
}

UObject* UTextureRenderTargetVolumeFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// create the new object
	UTextureRenderTargetVolume* Result = NewObject<UTextureRenderTargetVolume>(InParent, Class, Name, Flags);

	// initialize the resource
	Result->InitAutoFormat(Width, Height, Depth);

	return Result;
}

/*-----------------------------------------------------------------------------
	UTextureFactory.
-----------------------------------------------------------------------------*/

// .PCX file header.
#pragma pack(push,1)

struct FTGAFileFooter
{
	uint32 ExtensionAreaOffset;
	uint32 DeveloperDirectoryOffset;
	uint8 Signature[16];
	uint8 TrailingPeriod;
	uint8 NullTerminator;
};

struct FPSDFileHeader
{                                                           
	int32     Signature;      // 8BPS
	int16   Version;        // Version
	int16   nChannels;      // Number of Channels (3=RGB) (4=RGBA)
	int32     Height;         // Number of Image Rows
	int32     Width;          // Number of Image Columns
	int16   Depth;          // Number of Bits per Channel
	int16   Mode;           // Image Mode (0=Bitmap)(1=Grayscale)(2=Indexed)(3=RGB)(4=CYMK)(7=Multichannel)
	uint8    Pad[6];         // Padding

	/**
	 * @return Whether file has a valid signature
	 */
	bool IsValid( void )
	{
		// Fail on bad signature
		if (Signature != 0x38425053)
			return false;

		return true;
	}

	/**
	 * @return Whether file has a supported version
	 */
	bool IsSupported( void )
	{
		// Fail on bad version
		if( Version != 1 )
			return false;   
		// Fail on anything other than 1, 3 or 4 channels
		if ((nChannels!=1) && (nChannels!=3) && (nChannels!=4))
			return false;
		// Fail on anything other than 8 Bits/channel or 16 Bits/channel  
		if ((Depth != 8) && (Depth != 16))
			return false;
		// Fail on anything other than Grayscale and RGB
		// We can add support for indexed later if needed.
		if (Mode!=1 && Mode!=3)
			return false;

		return true;
	}
};

#pragma pack(pop)


static bool psd_ReadData( uint8* pOut, const uint8*& pBuffer, FPSDFileHeader& Info )
{
	const uint8* pPlane = nullptr;
	const uint8* pRowTable = nullptr;
	int32         iPlane;
	int16       CompressionType;
	int32         iPixel;
	int32         iRow;
	int32         CompressedBytes;
	int32         iByte;
	int32         Count;
	uint8        Value;

	// Double check to make sure this is a valid request
	if (!Info.IsValid() || !Info.IsSupported())
	{
		return false;
	}

	const uint8* pCur = pBuffer + sizeof(FPSDFileHeader);
	int32         NPixels = Info.Width * Info.Height;

	int32  ClutSize =  ((int32)pCur[ 0] << 24) +
		((int32)pCur[ 1] << 16) +
		((int32)pCur[ 2] <<  8) +
		((int32)pCur[ 3] <<  0);
	pCur+=4;
	pCur += ClutSize;    

	// Skip Image Resource Section
	int32 ImageResourceSize = ((int32)pCur[ 0] << 24) +
		((int32)pCur[ 1] << 16) +
		((int32)pCur[ 2] <<  8) +
		((int32)pCur[ 3] <<  0);
	pCur += 4+ImageResourceSize;

	// Skip Layer and Mask Section
	int32 LayerAndMaskSize =  ((int32)pCur[ 0] << 24) +
		((int32)pCur[ 1] << 16) +
		((int32)pCur[ 2] <<  8) +
		((int32)pCur[ 3] <<  0);
	pCur += 4+LayerAndMaskSize;

	// Determine number of bytes per pixel
	int32 BytesPerPixel = 3;
	const int32 BytesPerChannel = (Info.Depth / 8);
	switch( Info.Mode )
	{
	case 1: // 'GrayScale'
		BytesPerPixel = BytesPerChannel;
		break;
	case 2:        
		BytesPerPixel = 1;        
		return false;  // until we support indexed...
	case 3: // 'RGBColor'
		if (Info.nChannels == 3)
			BytesPerPixel = 3 * BytesPerChannel;
		else                   
			BytesPerPixel = 4 * BytesPerChannel;       
		break;
	default:
		return false;
	}

	// Get Compression Type
	CompressionType = ((int32)pCur[0] <<  8) + ((int32)pCur[1] <<  0);    
	pCur += 2;

	// Fail on 16 Bits/channel with RLE. This can occur when the file is not saved with 'Maximize Compatibility'. Compression doesn't appear to be standard.
	if(CompressionType == 1 && Info.Depth == 16)
	{
		return false;
	}

	// If no alpha channel, set alpha to opaque (255 or 65536).
	if( Info.nChannels != 4)
	{
		if(Info.Depth == 8)
		{
			const uint32 Channels = 4;
			const uint32 BufferSize = Info.Width * Info.Height * Channels * sizeof(uint8);
			FMemory::Memset(pOut, 0xff, BufferSize);
		}
		else if(Info.Depth == 16)
		{
			const uint32 Channels = 4;
			const uint32 BufferSize = Info.Width * Info.Height * Channels * sizeof(uint16);
			FMemory::Memset(pOut, 0xff, BufferSize);
		}
	}

	// Uncompressed?
	if( CompressionType == 0 )
	{
		if(Info.Depth == 8)
		{
			FColor* Dest = (FColor*)pOut;
			for(int32 Pixel=0; Pixel < NPixels; Pixel++ )
			{
				if (Info.nChannels == 1)
				{
					Dest[Pixel].R = pCur[Pixel];
					Dest[Pixel].G = pCur[Pixel];
					Dest[Pixel].B = pCur[Pixel];
				}
				else
				{
					// Each channel live in a separate plane
					Dest[Pixel].R = pCur[Pixel];
					Dest[Pixel].G = pCur[NPixels + Pixel];
					Dest[Pixel].B = pCur[NPixels * 2 + Pixel];
					if (Info.nChannels == 4)
					{
						Dest[Pixel].A = pCur[NPixels * 3 + Pixel];
					}
				}
			}
		}
		else if (Info.Depth == 16)
		{
			uint32 SrcOffset = 0;
			
			if (Info.nChannels == 1)
			{
				uint16* Dest = (uint16*)pOut;
				uint32 ChannelOffset = 0;

				for (int32 Pixel = 0; Pixel < NPixels; Pixel++)
				{
					Dest[ChannelOffset + 0] = ((pCur[SrcOffset] << 8) + (pCur[SrcOffset + 1] << 0));
					Dest[ChannelOffset + 1] = ((pCur[SrcOffset] << 8) + (pCur[SrcOffset + 1] << 0));
					Dest[ChannelOffset + 2] = ((pCur[SrcOffset] << 8) + (pCur[SrcOffset + 1] << 0));

					//Increment offsets
					ChannelOffset += 4;
					SrcOffset += BytesPerChannel;
				}
			}
			else
			{
				// Loop through the planes	
				for (iPlane = 0; iPlane < Info.nChannels; iPlane++)
				{
					uint16* Dest = (uint16*)pOut;
					uint32 ChannelOffset = iPlane;

					for (int32 Pixel = 0; Pixel < NPixels; Pixel++)
					{
						Dest[ChannelOffset] = ((pCur[SrcOffset] << 8) + (pCur[SrcOffset + 1] << 0));

						//Increment offsets
						ChannelOffset += 4;
						SrcOffset += BytesPerChannel;
					}
				}
			}
		}
	}
	// RLE?
	else if( CompressionType == 1 )
	{
		// Setup RowTable
		pRowTable = pCur;
		pCur += Info.nChannels*Info.Height*2;

		FColor* Dest = (FColor*)pOut;

		// Loop through the planes
		for( iPlane=0 ; iPlane<Info.nChannels ; iPlane++ )
		{
			int32 iWritePlane = iPlane;
			if( iWritePlane > BytesPerPixel-1 ) iWritePlane = BytesPerPixel-1;

			// Loop through the rows
			for( iRow=0 ; iRow<Info.Height ; iRow++ )
			{
				// Load a row
				CompressedBytes = (pRowTable[(iPlane*Info.Height+iRow)*2  ] << 8) +
					(pRowTable[(iPlane*Info.Height+iRow)*2+1] << 0);

				// Setup Plane
				pPlane = pCur;
				pCur += CompressedBytes;

				// Decompress Row
				iPixel = 0;
				iByte = 0;
				while( (iPixel < Info.Width) && (iByte < CompressedBytes) )
				{
					int8 code = (int8)pPlane[iByte++];

					// Is it a repeat?
					if( code < 0 )
					{
						Count = -(int32)code + 1;
						Value = pPlane[iByte++];
						while( Count-- > 0 )
						{
							int32 idx = (iPixel) + (iRow*Info.Width);
							if (Info.nChannels == 1)
							{
								Dest[idx].R = Value;
								Dest[idx].G = Value;
								Dest[idx].B = Value;
							}
							else
							{
								switch (iWritePlane)
								{
								case 0: Dest[idx].R = Value; break;
								case 1: Dest[idx].G = Value; break;
								case 2: Dest[idx].B = Value; break;
								case 3: Dest[idx].A = Value; break;
								}
							}
							iPixel++;
						}
					}
					// Must be a literal then
					else
					{
						Count = (int32)code + 1;
						while( Count-- > 0 )
						{
							Value = pPlane[iByte++];
							int32 idx = (iPixel) + (iRow*Info.Width);

							if (Info.nChannels == 1)
							{
								Dest[idx].R = Value;
								Dest[idx].G = Value;
								Dest[idx].B = Value;
							}
							else
							{
								switch (iWritePlane)
								{
								case 0: Dest[idx].R = Value; break;
								case 1: Dest[idx].G = Value; break;
								case 2: Dest[idx].B = Value; break;
								case 3: Dest[idx].A = Value; break;
								}
							}
							iPixel++;
						}
					}
				}

				// Confirm that we decoded the right number of bytes
				check( iByte  == CompressedBytes );
				check( iPixel == Info.Width );
			}
		}
	}
	else
		return false;

	// Success!
	return( true );
}

static void psd_GetPSDHeader( const uint8* Buffer, FPSDFileHeader& Info )
{
	Info.Signature      =   ((int32)Buffer[ 0] << 24) +
		((int32)Buffer[ 1] << 16) +
		((int32)Buffer[ 2] <<  8) +
		((int32)Buffer[ 3] <<  0);
	Info.Version        =   ((int32)Buffer[ 4] <<  8) +
		((int32)Buffer[ 5] <<  0);
	Info.nChannels      =   ((int32)Buffer[12] <<  8) +
		((int32)Buffer[13] <<  0);
	Info.Height         =   ((int32)Buffer[14] << 24) +
		((int32)Buffer[15] << 16) +
		((int32)Buffer[16] <<  8) +
		((int32)Buffer[17] <<  0);
	Info.Width          =   ((int32)Buffer[18] << 24) +
		((int32)Buffer[19] << 16) +
		((int32)Buffer[20] <<  8) +
		((int32)Buffer[21] <<  0);
	Info.Depth          =   ((int32)Buffer[22] <<  8) +
		((int32)Buffer[23] <<  0);
	Info.Mode           =   ((int32)Buffer[24] <<  8) +
		((int32)Buffer[25] <<  0);
}

bool UTextureFactory::bSuppressImportOverwriteDialog = false;
bool UTextureFactory::bForceOverwriteExistingSettings = false;

UTextureFactory::UTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTexture::StaticClass();

	Formats.Add( TEXT( "bmp;Texture" ) );
	Formats.Add( TEXT( "pcx;Texture" ) );
	Formats.Add( TEXT( "tga;Texture" ) );
	//Formats.Add( TEXT( "float;Texture" ) ); // no such thing?
	Formats.Add( TEXT( "psd;Texture" ) );
	Formats.Add( TEXT( "dds;Texture (Cubemap or 2D)" ) );
	Formats.Add( TEXT( "hdr;Texture (HDR) (LongLat unwrap or 2D)" ) );
	Formats.Add( TEXT( "ies;IES Texture (Standard light profiles)" ) );
	Formats.Add( TEXT( "png;Texture" ) );
	Formats.Add( TEXT( "jpg;Texture" ) );
	Formats.Add( TEXT( "jpeg;Texture" ) );
	// note: conflicts with .exr ImgMedia import in the import dialog
	// alphabetic order affects the select when you enter "*.exr" in the import dialog box
	Formats.Add( TEXT( "exr;EXR image HDR Texture2d" ) );
	Formats.Add( TEXT( "tif;Texture (TIFF)" ) );
	Formats.Add( TEXT( "tiff;Texture (TIFF)" ) );

	bCreateNew = false;
	bEditorImport = true;

	UdimRegexPattern = UE::TextureUtilitiesCommon::DefaultUdimRegexPattern;

	ColorSpaceMode = ETextureSourceColorSpace::Auto;
}

bool UTextureFactory::FactoryCanImport(const FString& Filename)
{
	FString Extension = FPaths::GetExtension(Filename);

	return (Formats.ContainsByPredicate(
		[&Extension](const FString& Format)
		{
			return Format.StartsWith(Extension);
		}));
}



void UTextureFactory::PostInitProperties()
{
	Super::PostInitProperties();
	MipGenSettings = TextureMipGenSettings(0);
	check( MipGenSettings == TMGS_FromTextureGroup );
	bool bFlipNormalMapGreenChannelSetting = false;
	GConfig->GetBool(TEXT("/Script/UnrealEd.EditorEngine"), TEXT("FlipNormalMapGreenChannel"), bFlipNormalMapGreenChannelSetting, GEngineIni);
	bFlipNormalMapGreenChannel = bFlipNormalMapGreenChannelSetting;
}

UTexture2D* UTextureFactory::CreateTexture2D( UObject* InParent, FName Name, EObjectFlags Flags )
{
	UObject* NewObject = CreateOrOverwriteAsset(UTexture2D::StaticClass(), InParent, Name, Flags);
	UTexture2D* NewTexture = nullptr;
	if(NewObject)
	{
		NewTexture = CastChecked<UTexture2D>(NewObject);
	}
	
	return NewTexture;
}

UTextureCube* UTextureFactory::CreateTextureCube( UObject* InParent, FName Name, EObjectFlags Flags )
{
	// CreateOrOverwriteAsset could fail if this cubemap replaces an asset that still has references.
	UObject* NewObject = CreateOrOverwriteAsset(UTextureCube::StaticClass(), InParent, Name, Flags);
	return NewObject ? CastChecked<UTextureCube>(NewObject) : nullptr;
}

UTexture2DArray* UTextureFactory::CreateTexture2DArray(UObject* InParent, FName Name, EObjectFlags Flags) 
{
	UObject* NewObject = CreateOrOverwriteAsset(UTexture2DArray::StaticClass(), InParent, Name, Flags);
	return NewObject ? CastChecked<UTexture2DArray>(NewObject) : nullptr;
}

void UTextureFactory::SuppressImportOverwriteDialog(bool bOverwriteExistingSettings)
{
	bSuppressImportOverwriteDialog = true;
    bForceOverwriteExistingSettings = bOverwriteExistingSettings;
}


extern ENGINE_API bool GUseBilinearLightmaps;

void FImportImage::Init2DWithParams(int32 InSizeX, int32 InSizeY, ETextureSourceFormat InFormat, bool InSRGB)
{
	SizeX = InSizeX;
	SizeY = InSizeY;
	NumMips = 1;
	Format = InFormat;
	SRGB = InSRGB;
}

int64 FImportImage::GetMipSize(int32 InMipIndex) const
{
	check(InMipIndex >= 0);
	check(InMipIndex < NumMips);
	const int32 MipSizeX = FMath::Max(SizeX >> InMipIndex, 1);
	const int32 MipSizeY = FMath::Max(SizeY >> InMipIndex, 1);
	return (int64)MipSizeX * MipSizeY * FTextureSource::GetBytesPerPixel(Format);
}

void* FImportImage::GetMipData(int32 InMipIndex)
{
	int64 Offset = 0;
	for (int32 MipIndex = 0; MipIndex < InMipIndex; ++MipIndex)
	{
		Offset += GetMipSize(MipIndex);
	}
	int64 CurMipSize = GetMipSize(InMipIndex);
	check( Offset + CurMipSize <= RawData.Num() );
	return &RawData[Offset];
}

bool UTextureFactory::ImportImage(const uint8* Buffer, int64 Length, FFeedbackContext* Warn, EImageImportFlags Flags, FImportImage& OutImage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTextureFactory::ImportImage)

	const bool bAllowNonPowerOfTwo = EnumHasAllFlags(Flags, EImageImportFlags::AllowNonPowerOfTwo);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));

	// Use the magic bytes when possible to avoid calling inefficient code to check if the image is of the right format
	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(Buffer, Length);

	if ( ImageFormat != EImageFormat::Invalid )
	{
		// Generic ImageWrapper loader :
		// for PNG,EXR,BMP,TGA :
		FImage LoadedImage;
		if ( ImageWrapperModule.DecompressImage(Buffer, Length,LoadedImage) )
		{
			if (!IsImportResolutionValid(LoadedImage.SizeX, LoadedImage.SizeY, bAllowNonPowerOfTwo, Warn))
			{
				return false;
			}
		
			// branch for JPEG, if retaining the jpeg compressed data
			// this is inside the DecompressImage branch even though we don't use the LoadedImage at all
			//	 just to ensure that the jpeg will decode successfully
			if (ImageFormat == EImageFormat::JPEG || ImageFormat == EImageFormat::UEJPEG)
			{
				// unusual loader, retains jpeg
				bool bRetainJpegFormat = false;
				if (EnumHasAllFlags(Flags, EImageImportFlags::AllowReturnOfCompressedData))
				{
					// For now this option is opt in via the config files once there is no technical risk this will become the default path.
					GConfig->GetBool(TEXT("TextureImporter"), TEXT("RetainJpegFormat"), bRetainJpegFormat, GEditorIni);
				}
				if ( bRetainJpegFormat)
				{
					// does not decode jpeg, just to get width/height :
					TSharedPtr<IImageWrapper> JpegImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);
					if (JpegImageWrapper.IsValid() && JpegImageWrapper->SetCompressed(Buffer, Length))
					{
						check( JpegImageWrapper->GetWidth() == LoadedImage.SizeX );
						check( JpegImageWrapper->GetHeight() == LoadedImage.SizeY );

						ERawImageFormat::Type RawFormat = JpegImageWrapper->GetClosestRawImageFormat();
						check( RawFormat != ERawImageFormat::Invalid );
						ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);
						bool bSRGB = JpegImageWrapper->GetSRGB();

						OutImage.Init2DWithParams(
							JpegImageWrapper->GetWidth(),
							JpegImageWrapper->GetHeight(),
							TextureFormat,
							bSRGB
						);

						OutImage.RawData.Append(Buffer, Length);

						if (ImageFormat == EImageFormat::JPEG) 
						{
							OutImage.RawDataCompressionFormat = ETextureSourceCompressionFormat::TSCF_JPEG;
							UE_LOGF(LogEditorFactories,Display,"JPEG imported and retained as JPEG in uasset.");
						}
						else
						{
							OutImage.RawDataCompressionFormat = ETextureSourceCompressionFormat::TSCF_UEJPEG;
							UE_LOGF(LogEditorFactories,Display,"UEJPEG imported and retained as UEJPEG in uasset.");
						}

						return true;
					}
				}
				// if not bRetainJpegFormat , jpeg will continue with generic loader
			}

			if ( UE::TextureUtilitiesCommon::AutoDetectAndChangeGrayScale(LoadedImage) )
			{
				UE_LOGF(LogEditorFactories,Display,"Auto-detected grayscale, image changed to G8");
			}

			ETextureSourceFormat TextureFormat = FImageCoreUtils::ConvertToTextureSourceFormat(LoadedImage.Format);
			bool bSRGB = LoadedImage.GammaSpace != EGammaSpace::Linear;

			OutImage.Init2DWithParams(
				LoadedImage.SizeX,
				LoadedImage.SizeY,
				TextureFormat,
				bSRGB
			);

			// move LoadedImage bits to OutImage :
			Swap(OutImage.RawData,LoadedImage.RawData);
		
			if ( ERawImageFormat::IsHDR(LoadedImage.Format) )
			{
				OutImage.CompressionSettings = TC_HDR;
				check( bSRGB == false );
			}

			// do per-format processing to match legacy behavior :

			if ( ImageFormat == EImageFormat::PNG )
			{
				ETextureImportPNGInfill PNGInfill = UE::TextureUtilitiesCommon::GetPNGInfillSetting();

				if (PNGInfill != ETextureImportPNGInfill::Never)
				{
					bool bDoOnComplexAlphaNotJustBinaryTransparency = ( PNGInfill == ETextureImportPNGInfill::Always );

					// Replace the pixels with 0.0 alpha with a color value from the nearest neighboring color which has a non-zero alpha
					UE::TextureUtilitiesCommon::FillZeroAlphaPNGData(OutImage.SizeX, OutImage.SizeY, OutImage.Format, OutImage.RawData.GetData(), bDoOnComplexAlphaNotJustBinaryTransparency);
				}
			}
			else if ( ImageFormat == EImageFormat::TGA )
			{
				const FTGAFileHeader* TGA = (FTGAFileHeader*)Buffer;

				if (TGA->ColorMapType == 1 && TGA->ImageTypeCode == 1 && TGA->BitsPerPixel == 8)
				{
					// Notes: The Scaleform GFx exporter (dll) strips all font glyphs into a single 8-bit texture.
					// The targa format uses this for a palette index; GFx uses a palette of (i,i,i,i) so the index
					// is also the alpha value.
					//
					// We store the image as PF_G8, where it will be used as alpha in the Glyph shader.
				
					// ?? check or convert? or neither?
					//check( TextureFormat == TSF_G8 );

					OutImage.CompressionSettings = TC_Grayscale;
				}
				else if(TGA->ColorMapType == 0 && TGA->ImageTypeCode == 3 && TGA->BitsPerPixel == 8)
				{
					// standard grayscale images
				
					// ?? check or convert? or neither?
					//check( TextureFormat == TSF_G8 );

					OutImage.CompressionSettings = TC_Grayscale;
				}

				if ( OutImage.CompressionSettings == TC_Grayscale && TGA->ImageTypeCode == 3)
				{
					// default grayscales to linear as they wont get compression otherwise and are commonly used as masks
					// -> this is wrong way to do this
					//	Image.SRGB should be the setting of the import image data
					//	not contain information about how we want the texture platform data to be set up
					OutImage.SRGB = false;
				}
			}

			return true;
		}
		else
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("Image Import failed."));
			// do not return false, let loaders below try
		}
	}


	//
	// PSD File
	//
	FPSDFileHeader			 psdhdr;
	if (Length > sizeof(FPSDFileHeader))
	{
		psd_GetPSDHeader(Buffer, psdhdr);
	}
	if (psdhdr.IsValid())
	{
		// Check the resolution of the imported texture to ensure validity
		if (!IsImportResolutionValid(psdhdr.Width, psdhdr.Height, bAllowNonPowerOfTwo, Warn))
		{
			return false;
		}
		if (!psdhdr.IsSupported())
		{
			Warn->Logf(TEXT("Format of this PSD is not supported. Only Grayscale and RGBColor PSD images are currently supported, in 8-bit or 16-bit."));
			return false;
		}

		// Select the texture's source format
		ETextureSourceFormat TextureFormat = TSF_Invalid;
		if (psdhdr.Depth == 8)
		{
			TextureFormat = TSF_BGRA8;
		}
		else if (psdhdr.Depth == 16)
		{
			TextureFormat = TSF_RGBA16;
		}

		if (TextureFormat == TSF_Invalid)
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("PSD file contains data in an unsupported format."));
			return false;
		}

		// The psd is supported. Load it up.        
		bool bSRGB = true;
		OutImage.Init2DWithParams(
			psdhdr.Width,
			psdhdr.Height,
			TextureFormat,
			bSRGB
		);
		
		OutImage.RawData.SetNumUninitialized((int64)psdhdr.Width * psdhdr.Height * FTextureSource::GetBytesPerPixel(TextureFormat));

		uint8* Dst = (uint8*)OutImage.RawData.GetData();

		// @todo : dangerous : psd_ReadData doesn't take Dst end, may overrun
		//	-> just delete the PSD reader from here entirely, it should be in ImageWrapper instead

		if (!psd_ReadData(Dst, Buffer, psdhdr))
		{
			Warn->Logf(TEXT("Failed to read this PSD"));
			return false;
		}

		return true;
	}
	
	return false;
}

UTexture* UTextureFactory::ImportTextureUDIM(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const TCHAR* Type, const TMap<int32, FString>& UDIMIndexToFile, FFeedbackContext* Warn)
{
	TArray64<uint8> TextureData;
	TArray<FImportImage> SourceImages;
	TArray<FTextureSourceBlock> SourceBlocks;
	TArray<FString> SourceFileNames;
	SourceImages.Reserve(UDIMIndexToFile.Num());
	SourceBlocks.Reserve(UDIMIndexToFile.Num());
	SourceFileNames.Reserve(UDIMIndexToFile.Num());

	ETextureSourceFormat Format = TSF_Invalid;
	TextureCompressionSettings TCSettings = TC_MAX;
	bool bSRGB = false;
	
	// UDIM requires each page to be power-of-2 and be uncompressed
	const EImageImportFlags ImportFlags = EImageImportFlags::None;

	bool bMismatchedFormats = false;
	bool bMismatchedGammaSpace = false;

	for (const auto& It : UDIMIndexToFile)
	{
		const FString& TexturePath = It.Value;
		if (FFileHelper::LoadFileToArray(TextureData, *TexturePath))
		{
			FImportImage Image;
			if (ImportImage(TextureData.GetData(), TextureData.Num(), Warn, ImportFlags, Image))
			{
				if (Format == TSF_Invalid)
				{
					Format = Image.Format;
					bSRGB = Image.SRGB;
				}

				if (TCSettings == TC_MAX)
				{
					// Should we somehow try to combine different compression settings? Is that ever useful/needed?
					TCSettings = Image.CompressionSettings;
				}

				if (bSRGB != Image.SRGB)
				{
					bMismatchedGammaSpace = true;
					bSRGB = false;
					Format = TSF_RGBA32F;
				}

				if (Format != Image.Format)
				{
					bMismatchedFormats = true;
					Format = FImageCoreUtils::GetCommonSourceFormat(Format, Image.Format);
				}

				const int32 UDIMIndex = It.Key;
				FTextureSourceBlock* Block = new(SourceBlocks) FTextureSourceBlock();
				UE::TextureUtilitiesCommon::ExtractUDIMCoordinates(UDIMIndex, Block->BlockX, Block->BlockY);
				Block->SizeX = Image.SizeX;
				Block->SizeY = Image.SizeY;
				Block->NumSlices = 1;
				Block->NumMips = Image.NumMips;

				SourceImages.Emplace(MoveTemp(Image));
				SourceFileNames.Add(TexturePath);
			}
		}
	}

	if (SourceImages.Num() < 1)
	{
		return nullptr;
	}

	if (bMismatchedGammaSpace || bMismatchedFormats)
	{
		Warn->Logf(ELogVerbosity::Warning, TEXT("Mismatched UDIM image %s, converting all to %s/%s ..."), bMismatchedGammaSpace ? TEXT("gamma spaces") : TEXT("pixel formats"),
			ERawImageFormat::GetName(FImageCoreUtils::ConvertToRawImageFormat(Format)), bSRGB ? TEXT("sRGB") : TEXT("Linear"));

		for( FImportImage & Image : SourceImages )
		{
			if (Image.SRGB != bSRGB || Image.Format != Format)
			{
				ERawImageFormat::Type ImageRawFormat = FImageCoreUtils::ConvertToRawImageFormat(Image.Format);
				FImageView SourceImage(Image.RawData.GetData(), Image.SizeX, Image.SizeY, 1, ImageRawFormat, Image.SRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
				FImage DestImage(Image.SizeX, Image.SizeY, FImageCoreUtils::ConvertToRawImageFormat(Format), bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
				FImageCore::CopyImage(SourceImage, DestImage);

				Image.RawData = MoveTemp(DestImage.RawData);
				Image.Format = Format;
				Image.SRGB = bSRGB;
			}				
		}
	}

	TArray<const uint8*> SourceImageData;
	SourceImageData.Reserve(SourceImages.Num());
	for (const FImportImage& Image : SourceImages)
	{
		SourceImageData.Add(Image.RawData.GetData());
	}

	UTexture2D* Texture = CreateTexture2D(InParent, Name, Flags);
	Texture->Source.InitBlocked(&Format, SourceBlocks.GetData(), 1, SourceBlocks.Num(), SourceImageData.GetData());
	Texture->CompressionSettings = TCSettings;

	if (ColorSpaceMode == ETextureSourceColorSpace::Auto)
	{
		Texture->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(TCSettings,Format,bSRGB);
	}
	else if (ColorSpaceMode == ETextureSourceColorSpace::Linear)
	{
		Texture->SRGB = false;
	}
	else 
	{
		check(ColorSpaceMode == ETextureSourceColorSpace::SRGB);
		Texture->SRGB = true;
	}

	for (int32 FileIndex = 0; FileIndex < SourceFileNames.Num(); ++FileIndex)
	{
		Texture->AssetImportData->AddFileName(SourceFileNames[FileIndex], FileIndex);
	}

	return Texture;
}

UTexture * UTextureFactory::ImportDDS(const uint8* Buffer,int64 Length,UObject* InParent,FName Name, EObjectFlags Flags,EImageImportFlags ImportFlags,FFeedbackContext* Warn)
{
	UE::DDS::EDDSError Error;
	UE::DDS::FDDSFile* DDS = UE::DDS::FDDSFile::CreateFromDDSInMemory(Buffer,Length,&Error);
	if ( DDS == nullptr )
	{
		// NotADds is okay/expected , we try this on all image buffers
		// IoError means buffer is too small to be a DDS
		check( Error != UE::DDS::EDDSError::OK ); 
		if ( Error != UE::DDS::EDDSError::NotADds && Error != UE::DDS::EDDSError::IoError )
		{
			UE_LOGF(LogEditorFactories,Warning,"Failed to load DDS (Error=%d)",(int)Error);			
		}
		return nullptr;
	}

	// delete DDS at scope exit :
	TUniquePtr<UE::DDS::FDDSFile> DDSPtr(DDS);

	if (!IsImportResolutionValid(DDS->Width, DDS->Height, !!(ImportFlags & EImageImportFlags::AllowNonPowerOfTwo), Warn))
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("DDS resolution not supported"));
		return nullptr;
	}

	// change X8 formats to A8 :	
	DDS->ConvertRGBXtoRGBA();
	// change RGBA8 to BGRA8 before DXGIFormatGetClosestRawFormat :
	DDS->ConvertChannelOrder(UE::DDS::EChannelOrder::BGRA);

	// map format to RawFormat and ETextureSourceFormat
	ERawImageFormat::Type RawFormat = UE::DDS::DXGIFormatGetClosestRawFormat(DDS->DXGIFormat);
	if ( RawFormat == ERawImageFormat::Invalid )
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("DDS DXGIFormat not supported : %d : %s"), (int)DDS->DXGIFormat, UE::DDS::DXGIFormatGetName(DDS->DXGIFormat) );
		return nullptr;
	}

	UTexture * Texture;
	int NumSlices;
	if ( DDS->IsValidTexture2D() )
	{
		Texture = CreateTexture2D(InParent, Name, Flags);
		NumSlices = 1;

		UE_LOGF(LogEditorFactories,Display,"DDS imported as 2d Texture");
	}
	else if ( DDS->IsValidTextureCube() )
	{
		if ( DDS->ArraySize > 6 )
		{
			UObject* NewObject = CreateOrOverwriteAsset(UTextureCubeArray::StaticClass(), InParent, Name, Flags);
			Texture = NewObject ? CastChecked<UTextureCubeArray>(NewObject) : nullptr;
			
			UE_LOGF(LogEditorFactories,Display,"DDS imported as Cube Texture Array");
		}
		else
		{
			Texture = CreateTextureCube(InParent, Name, Flags);
			
			UE_LOGF(LogEditorFactories,Display,"DDS imported as Cube Texture");
		}

		NumSlices = DDS->ArraySize;
	}
	else if ( DDS->IsValidTextureArray() )
	{
		Texture = CreateTexture2DArray(InParent, Name, Flags);
		NumSlices = DDS->ArraySize;

		UE_LOGF(LogEditorFactories,Display,"DDS imported as Texture Array");
	}
	else if ( DDS->IsValidTextureVolume() )
	{
		UObject* NewObject = CreateOrOverwriteAsset(UVolumeTexture::StaticClass(), InParent, Name, Flags);
		Texture = NewObject ? CastChecked<UVolumeTexture>(NewObject) : nullptr;
		NumSlices = DDS->Depth;

		UE_LOGF(LogEditorFactories,Display,"DDS imported as Volume");
	}
	else
	{
		UE_LOGF(LogEditorFactories,Error,"DDS is not a valid Unreal type");
		return nullptr;
	}
	if ( Texture == nullptr )
	{
		UE_LOGF(LogEditorFactories,Error,"DDS could not create texture");
		return nullptr;
	}

	ETextureSourceFormat TSFormat = FImageCoreUtils::ConvertToTextureSourceFormat(RawFormat);

	int MipCount = (int)DDS->MipCount;

	if ( MipCount > MAX_TEXTURE_MIP_COUNT )
	{
		// resolution can be above MAX_TEXTURE_MIP_COUNT for VT
		UE_LOGF(LogEditorFactories,Warning,"DDS exceeds MAX_TEXTURE_MIP_COUNT");
		MipCount = MAX_TEXTURE_MIP_COUNT;
	}

	Texture->Source.Init(DDS->Width,DDS->Height,NumSlices,MipCount,TSFormat);

	for(int MipIndex=0;MipIndex<MipCount;MipIndex++)
	{
		FTextureSource::FMipLock MipLock(FTextureSource::ELockState::ReadWrite,&(Texture->Source),MipIndex);

		if ( DDS->Dimension == 3 )
		{
			check( DDS->Mips.Num() == DDS->MipCount );
			check( DDS->Mips[MipIndex].Depth == MipLock.Image.NumSlices );

			if ( ! DDS->GetMipImage( MipLock.Image, MipIndex ) )
			{
				// Texture is not returned; it will be deleted by GC
				Texture->MarkAsGarbage();
				UE_LOGF(LogEditorFactories,Error,"DDS could not convert pixel format");
				return nullptr;
			}
		}
		else
		{
			// DDS->Mips[] contains both mips and arrays
			check( DDS->Mips.Num() == DDS->MipCount * NumSlices );

			for(int SliceIndex=0;SliceIndex<NumSlices;SliceIndex++)
			{
				FImageView DestSlice = MipLock.Image.GetSlice(SliceIndex);
				
				// DDS Mips[] array has whole mip chain of each slice, then next slice
				// we have the opposite (all slices of top mip first, then next mip)
				int DDSMipIndex = SliceIndex * DDS->MipCount + MipIndex;

				if ( ! DDS->GetMipImage( DestSlice, DDSMipIndex ) )
				{
					// Texture is not returned; it will be deleted by GC
					Texture->MarkAsGarbage();
					UE_LOGF(LogEditorFactories,Error,"DDS could not convert pixel format");
					return nullptr;
				}
			}
		}
	}

	bool bSRGB = false;

	if ( ERawImageFormat::GetFormatNeedsGammaSpace(RawFormat) )
	{
		if ( DDS->CreateFlags & UE::DDS::FDDSFile::CREATE_FLAG_WAS_D3D9 )
		{
			// no SRGB info in Dx9 format
			// assume SRGB yes
			bSRGB = true;
		}
		else if ( UE::DDS::DXGIFormatHasLinearAndSRGBForm(DDS->DXGIFormat) )
		{
			// Dx10 file with format that has linear/srgb pair
			//	( _UNORM when _UNORM_SRGB)
			bSRGB = UE::DDS::DXGIFormatIsSRGB(DDS->DXGIFormat);
		}
		else
		{
			// Dx10 format that doesn't have linear/srgb pairs
			
			// R8G8_UNORM and R8_UNORM have no _SRGB pair
			// so no way to clearly indicate SRGB or Linear for them
			// assume SRGB yes
			bSRGB = true;
		}
	}

	if ( ERawImageFormat::IsHDR(RawFormat) )
	{
		Texture->CompressionSettings = TC_HDR;
		check( bSRGB == false );
	}

	Texture->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(Texture->CompressionSettings,TSFormat,bSRGB);
	
	if ( MipCount > 1)
	{
		// if the source has mips we keep the mips by default, unless the user changes that
		Texture->MipGenSettings = TMGS_LeaveExistingMips;
	}
	
	return Texture;
}

UTexture* UTextureFactory::ImportTexture(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ImportTexture);

	bool bAllowNonPowerOfTwo = false;
	GConfig->GetBool( TEXT("TextureImporter"), TEXT("AllowNonPowerOfTwoTextures"), bAllowNonPowerOfTwo, GEditorIni );

	// Validate it.
	const int64 Length = BufferEnd - Buffer;

	// We want to allow certain texture types to potentially return the compressed data rather than raw uncompressed
	EImageImportFlags ImportFlags = EImageImportFlags::AllowReturnOfCompressedData;
	if (bAllowNonPowerOfTwo)
	{
		ImportFlags |= EImageImportFlags::AllowNonPowerOfTwo;
	}

	// Do unusual loaders first before Generic

	UTexture * DDSTexture = ImportDDS(Buffer,Length,InParent,Name,Flags,ImportFlags,Warn);
	if ( DDSTexture != nullptr )
	{
		return DDSTexture;
	}

	//
	// HDR File -> longlat cube map
	//
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	if (ImageWrapperModule.DetectImageFormat(Buffer, Length) == EImageFormat::HDR)
	{
		// do longlat or 2d or ask ?
		
		// note: HDRImportShouldBeLongLatCubeMap member variable is not preserved across imports
		// because the TextureFactory is reconstructed each time
		// it is saved across multiple files in a single import
		
		// if we don't have an All answer, pop the question :
		if ( HDRImportShouldBeLongLatCubeMap != EAppReturnType::YesAll &&
			HDRImportShouldBeLongLatCubeMap != EAppReturnType::NoAll )
		{
			// trinary bool
			// 0 = no
			// 1 = yes
			// other = ask
			int LoadHdrAsLongLatCubemap = 2;
			GConfig->GetInt(TEXT("TextureImporter"), TEXT("LoadHdrAsLongLatCubemap"), LoadHdrAsLongLatCubemap, GEditorIni);
			if ( LoadHdrAsLongLatCubemap == 0 )
			{
				HDRImportShouldBeLongLatCubeMap = EAppReturnType::NoAll;
			}
			else if ( LoadHdrAsLongLatCubemap == 1 )
			{
				HDRImportShouldBeLongLatCubeMap = EAppReturnType::YesAll;
			}
			else
			{
				// ask

				if ( GIsRunningUnattendedScript )
				{
					// not allowed to messagebox
					// just default to legacy behavior = yes latlong cubemap
					HDRImportShouldBeLongLatCubeMap = EAppReturnType::Yes;
				}
				else
				{
					HDRImportShouldBeLongLatCubeMap = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, NSLOCTEXT("UnrealEd", "Import_HDRLongLat", "Should HDR texture be imported as longlat cubemap instead of 2d texture?"));
				}
			}
		}

		// branch if they want longlat cubemap :
		if (HDRImportShouldBeLongLatCubeMap == EAppReturnType::Yes ||
			HDRImportShouldBeLongLatCubeMap == EAppReturnType::YesAll)
		{
			if (Class && !UTextureCube::StaticClass()->IsChildOf(Class))
			{
				//The source use to import this texture is not compatible with the
				//existing texture type.
				Warn->Logf(ELogVerbosity::Error, TEXT("Cannot import/reimport a [%s] with a UTextureCube source file"), *Class->GetName());
				return nullptr;
			}

			TSharedPtr<IImageWrapper> HdrImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::HDR);
			if (HdrImageWrapper->SetCompressed(Buffer, Length))
			{
				TArray64<uint8> UnCompressedData;
				if (HdrImageWrapper->GetRaw(ERGBFormat::BGRE, 8, UnCompressedData))
				{
					// create the cube texture
					UTextureCube* TextureCube = CreateTextureCube(InParent, Name, Flags);
					if ( TextureCube )
					{
						TextureCube->Source.Init(
							HdrImageWrapper->GetWidth(),
							HdrImageWrapper->GetHeight(),
							/*NumSlices=*/ 1,
							/*NumMips=*/ 1,
							TSF_BGRE8,
							UnCompressedData.GetData()
							);
						// the loader can suggest a compression setting
						TextureCube->CompressionSettings = TC_HDR;
						TextureCube->SRGB = false;
						
						UE_LOGF(LogEditorFactories,Display,"HDR Image imported as LongLat cube map." );

						return TextureCube;
					}
				}
			}

			Warn->Log(ELogVerbosity::Error, TEXT("HDR import failed"));
			return nullptr;
		}
		// else didn't want cubemap
		// will fall down to generic importer and make a 2d texture
	}

	//
	// IES File (usually measured real world light profiles)
	//
	if( FCString::Stricmp(Type, TEXT("ies")) == 0)
	{
		// checks for .IES extension to avoid wasting loading large assets just to reject them during header parsing
		FIESConverter IESConverter(Buffer, IntCastChecked<uint32>(Length));

		if(IESConverter.IsValid())
		{
			if (Class && !UTextureLightProfile::StaticClass()->IsChildOf(Class))
			{
				//The source use to import this texture is not compatible with the
				//existing texture type.
				Warn->Logf(ELogVerbosity::Error, TEXT("Cannot import/reimport a [%s] with a UTextureLightProfile source file"), *Class->GetName());
				return nullptr;
			}

			UTextureLightProfile* Texture = Cast<UTextureLightProfile>( CreateOrOverwriteAsset(UTextureLightProfile::StaticClass(), InParent, Name, Flags) );
			if ( Texture )
			{
				Texture->Source.Init(
					IESConverter.GetWidth(),
					IESConverter.GetHeight(),
					/*NumSlices=*/ 1,
					1,
					TSF_RGBA16F,
					IESConverter.GetRawData().GetData()
					);

				Texture->AddressX = TA_Clamp;
				Texture->AddressY = TA_Clamp;
				Texture->CompressionSettings = TC_HDR;
				Texture->SRGB = false;
				MipGenSettings = TMGS_NoMipmaps;
				Texture->Brightness = IESConverter.GetBrightness();
				Texture->TextureMultiplier = IESConverter.GetMultiplier();
			}

			return Texture;
		}
		else
		{
			Warn->Log(ELogVerbosity::Error, TEXT("IES import failed"));
			return nullptr;
		}
	}

	//
	// Generic 2D Image
	//
	FImportImage Image;
	if (ImportImage(Buffer, Length, Warn, ImportFlags, Image))
	{
		if (Class && !UTexture2D::StaticClass()->IsChildOf(Class))
		{
			//The source use to import this texture is not compatible with the
			//existing texture type.
			Warn->Logf(ELogVerbosity::Error, TEXT("Cannot import/reimport a [%s] with a UTexture2D source file"), *Class->GetName());
			return nullptr;
		}

		UTexture2D* Texture = CreateTexture2D(InParent, Name, Flags);
		if (Texture)
		{
			UE_LOGF(LogEditorFactories,Display,"Image imported as : %ls", *(StaticEnum<ETextureSourceFormat>()->GetDisplayNameTextByValue(Image.Format).ToString()) );

			if (Image.RawDataCompressionFormat == ETextureSourceCompressionFormat::TSCF_None)
			{
				Texture->Source.Init(
					Image.SizeX,
					Image.SizeY,
					/*NumSlices=*/ 1,
					Image.NumMips,
					Image.Format,
					Image.RawData.GetData()
				);
			}
			else
			{
				Texture->Source.InitWithCompressedSourceData(
					Image.SizeX,
					Image.SizeY,
					Image.NumMips,
					Image.Format,
					Image.RawData,
					Image.RawDataCompressionFormat
				);
			}

			Texture->CompressionSettings = Image.CompressionSettings;

			if (ColorSpaceMode == ETextureSourceColorSpace::Auto)
			{
				Texture->SRGB = UE::TextureUtilitiesCommon::GetDefaultSRGB(Image.CompressionSettings,Image.Format,Image.SRGB);
			}
			else if (ColorSpaceMode == ETextureSourceColorSpace::Linear)
			{
				Texture->SRGB = false;
			}
			else if (ColorSpaceMode == ETextureSourceColorSpace::SRGB)
			{
				Texture->SRGB = true;
			}
			else
			{
				check(0);
			}
		}
		return Texture;
	}

	return nullptr;
}

bool UTextureFactory::DoesSupportClass(UClass* Class)
{
	return Class == UTexture2D::StaticClass() || Class == UTextureCube::StaticClass();
}

UObject* UTextureFactory::FactoryCreateBinary
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		Type,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FactoryCreateBinary);
	check(Type);

	FName TextureName = Name;

	// Check to see if we should import a series of textures as UDIM
	// Need to do this first, as this step affects the final name of the created texture asset
	TMap<int32, FString> UDIMIndexToFile;
	{
		const FString FilenameNoExtension = FPaths::GetBaseFilename(CurrentFilename);

		FString PreUDIMName;
		FString PostUDIMName;
		const int32 BaseUDIMIndex = UE::TextureUtilitiesCommon::ParseUDIMName(FilenameNoExtension, UdimRegexPattern, PreUDIMName, PostUDIMName);

		const FString BaseUDIMName = PreUDIMName + PostUDIMName;
		if (BaseUDIMIndex != INDEX_NONE)
		{
			UDIMIndexToFile.Add(BaseUDIMIndex, CurrentFilename);

			// Filter for other potential UDIM pages, with the same base name and file extension
			const FString Path = FPaths::GetPath(CurrentFilename);
			const FString UDIMFilter = (Path / PreUDIMName) + TEXT("*") + PostUDIMName + FPaths::GetExtension(CurrentFilename, true);

			TArray<FString> UDIMFiles;
			IFileManager::Get().FindFiles(UDIMFiles, *UDIMFilter, true, false);

			for (const FString& UDIMFile : UDIMFiles)
			{
				if (!CurrentFilename.EndsWith(UDIMFile) && FactoryCanImport(UDIMFile))
				{
					const int32 UDIMIndex = UE::TextureUtilitiesCommon::ParseUDIMName(FPaths::GetBaseFilename(UDIMFile), UdimRegexPattern, PreUDIMName, PostUDIMName);
					if (UDIMIndex != INDEX_NONE)
					{
						const FString UDIMName = PreUDIMName + PostUDIMName;
						if (!UDIMIndexToFile.Contains(UDIMIndex) && UDIMName == BaseUDIMName)
						{
							UDIMIndexToFile.Add(UDIMIndex, Path / UDIMFile);
						}
					}
				}
			}

			// Don't try to rename the texture if we are doing a reimport.
			if (!bIsDoingAReimport)
			{ 
				// Exclude UDIM number from the name of the UE texture asset we create
				// We do this even in the case where we're only importing a single 1001 image, which won't technically be a UDIM
				// We still want to strip the UDIM suffix in this case, as industry standard still considers this a UDIM imageset
				const FString ShortPackageName = ObjectTools::SanitizeInvalidChars(BaseUDIMName, INVALID_LONGPACKAGE_CHARACTERS);

				// Don't try to rename if the name of texture differ from the source file 
				// Some factory provide a name for the texture that is different form the source file Ex:FBX.
				const FString TextureNameAsString = TextureName.ToString();
				if (TextureNameAsString.Equals(ObjectTools::SanitizeInvalidChars(FilenameNoExtension, INVALID_LONGPACKAGE_CHARACTERS)))
				{
					// Don't try to rename the package if its the transient package
					if ( InParent != GetTransientPackage() && InParent->IsA<UPackage>() )
					{
						// Need to rename the package to match the new texture name, since package was already created
						// Package name will be the same as the object name, except will contain additional path information,
						// so we take the existing package name, then extract the UDIM index in order to preserve the path
						FString PackageName;
						InParent->GetName(PackageName);

						const int32 PackageUDIMIndex = UE::TextureUtilitiesCommon::ParseUDIMName(PackageName, UdimRegexPattern, PreUDIMName, PostUDIMName);
						const FString PackageUDIMName = PreUDIMName + PostUDIMName;

						// If we're re-importing UDIM texture, the package will already be correctly named after the UDIM base name
						// In this case we'll fail to parse the UDIM name, but the package should already have the proper name
						// Note that the package name may not match the asset name in this case, if we're reimporting new files
						if (PackageUDIMIndex != -1)
						{
							check(PackageUDIMIndex == BaseUDIMIndex);
							check(PackageUDIMName.EndsWith(ShortPackageName, ESearchCase::CaseSensitive));

							// Rename the texture object
							TextureName = *ShortPackageName;

							// In normal case, higher level code would have already checked for duplicate package name
							// But since we're changing package name here, check to see if package with the new name already exists...
							// If it does, code later in this method will prompt user to overwrite the existing asset
							UPackage* ExistingPackage = FindPackage(InParent->GetOuter(), *PackageUDIMName);
							if (ExistingPackage)
							{
								InParent = ExistingPackage;
							}
							else
							{
								verify(InParent->Rename(*PackageUDIMName, nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch));
							}
						}
					}
				}
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, TextureName, Type);

	// if the texture already exists, remember the user settings
	UTexture* ExistingTexture = FindObject<UTexture>( InParent, *TextureName.ToString() );
	UTexture2D* ExistingTexture2D = FindObject<UTexture2D>( InParent, *TextureName.ToString() );

	TextureAddress						ExistingAddressX	= TA_Wrap;
	TextureAddress						ExistingAddressY	= TA_Wrap;
	TextureFilter						ExistingFilter		= TF_Default;
	TextureGroup						ExistingLODGroup	= TEXTUREGROUP_World;
	TextureCompressionSettings			ExistingCompressionSettings = TC_Default;
	int32								ExistingLODBias		= 0;
	int32								ExistingNumCinematicMipLevels = 0;
	bool								ExistingNeverStream = false;
	bool								ExistingSRGB		= false;
	bool								ExistingPreserveBorder = false;
	bool								ExistingNoCompression = false;
	bool								ExistingNoAlpha = false;
	bool								ExistingDeferCompression = false;
	bool 								ExistingDitherMipMapAlpha = false;
	bool 								ExistingFlipGreenChannel = false;
	float								ExistingAdjustBrightness = 1.0f;
	float								ExistingAdjustBrightnessCurve = 1.0f;
	float								ExistingAdjustVibrance = 0.0f;
	float								ExistingAdjustSaturation = 1.0f;
	float								ExistingAdjustRGBCurve = 1.0f;
	float								ExistingAdjustHue = 0.0f;
	float								ExistingAdjustMinAlpha = 0.0f;
	float								ExistingAdjustMaxAlpha = 1.0f;
	bool								ExistingbDoScaleMipsForAlphaCoverage = false;
	FVector4							ExistingAlphaCoverageThresholds = FVector4(0, 0, 0, 0);
	bool								ExistingbUseNewMipFilter = false;
	TextureMipGenSettings				ExistingMipGenSettings = TextureMipGenSettings(0);
	check( ExistingMipGenSettings == TMGS_FromTextureGroup );
	bool								ExistingVirtualTextureStreaming = false;

	if (bForceOverwriteExistingSettings)
	{
		bUsingExistingSettings = false;
	}
	else if (AssetImportTask && AssetImportTask->bAutomated)
	{
		if (ExistingTexture)
		{
			if (!AssetImportTask->bReplaceExisting)
			{
				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
				return nullptr;
			}
			else
			{
				bUsingExistingSettings = !AssetImportTask->bReplaceExistingSettings;
			}
		}
		else
		{
			bUsingExistingSettings = false;
		}
	}
	else
	{
		bUsingExistingSettings = bSuppressImportOverwriteDialog;

		if (ExistingTexture && !bSuppressImportOverwriteDialog)
		{
			DisplayOverwriteOptionsDialog(FText::Format(
				NSLOCTEXT("TextureFactory", "ImportOverwriteWarning", "You are about to import '{0}' over an existing texture."),
				FText::FromName(TextureName)));

			switch (OverwriteYesOrNoToAllState)
			{

			case EAppReturnType::Yes:
			case EAppReturnType::YesAll:
				{
					// Overwrite existing settings
					bUsingExistingSettings = false;
					break;
				}
				case EAppReturnType::No:
				case EAppReturnType::NoAll:
				{
					// Preserve existing settings
					bUsingExistingSettings = true;
					break;
				}
				case EAppReturnType::Cancel:
				default:
				{
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
					return nullptr;
				}
			}
		}
	}

	// Don't suppress future textures from checking for overwrites unless the calling code explicitly asks for it
	bSuppressImportOverwriteDialog = false;
	bForceOverwriteExistingSettings = false;

	if (ExistingTexture && bUsingExistingSettings)
	{
		// save settings
		if(ExistingTexture2D)
		{
			ExistingAddressX	= ExistingTexture2D->AddressX;
			ExistingAddressY	= ExistingTexture2D->AddressY;
		}
		ExistingFilter		= ExistingTexture->Filter;
		ExistingLODGroup	= ExistingTexture->LODGroup;
		ExistingCompressionSettings = ExistingTexture->CompressionSettings;
		ExistingLODBias		= ExistingTexture->LODBias;
		ExistingNumCinematicMipLevels = ExistingTexture->NumCinematicMipLevels;
		ExistingNeverStream = ExistingTexture->NeverStream;
		ExistingSRGB		= ExistingTexture->SRGB;
		ExistingPreserveBorder = ExistingTexture->bPreserveBorder;
		ExistingNoCompression = ExistingTexture->CompressionNone;
		ExistingNoAlpha = ExistingTexture->CompressionNoAlpha;
		ExistingDeferCompression = ExistingTexture->DeferCompression;
		ExistingFlipGreenChannel = ExistingTexture->bFlipGreenChannel;
		ExistingbDoScaleMipsForAlphaCoverage = ExistingTexture->bDoScaleMipsForAlphaCoverage;
		ExistingAlphaCoverageThresholds = ExistingTexture->AlphaCoverageThresholds;
		ExistingbUseNewMipFilter = ExistingTexture->bUseNewMipFilter;
		ExistingAdjustBrightness = ExistingTexture->AdjustBrightness;
		ExistingAdjustBrightnessCurve = ExistingTexture->AdjustBrightnessCurve;
		ExistingAdjustVibrance = ExistingTexture->AdjustVibrance;
		ExistingAdjustSaturation = ExistingTexture->AdjustSaturation;
		ExistingAdjustRGBCurve = ExistingTexture->AdjustRGBCurve;
		ExistingAdjustHue = ExistingTexture->AdjustHue;
		ExistingAdjustMinAlpha = ExistingTexture->AdjustMinAlpha;
		ExistingAdjustMaxAlpha = ExistingTexture->AdjustMaxAlpha;
		ExistingMipGenSettings = ExistingTexture->MipGenSettings;
		ExistingVirtualTextureStreaming = ExistingTexture->VirtualTextureStreaming;
	}

	if (ExistingTexture2D)
	{
		// Update with new settings, which should disable streaming...
		// -> pretty sure this is unnecessary
		//	the combination of WaitForPendingInitOrStreaming and PreEditChange do all the waiting necessary
		ExistingTexture2D->UpdateResource();
	}
	if(ExistingTexture)
	{
		// Wait for InitRHI() to complete before the FTextureReferenceReplacer calls ReleaseRHI() to follow the workflow.
		// Static texture needs to avoid having pending InitRHI() before enqueuing ReleaseRHI() to safely track access of the PlatformData on the renderthread.
		ExistingTexture->WaitForPendingInitOrStreaming();

		// Make sure the changes are part of the transaction when reimporting over an existing texture
		ExistingTexture->PreEditChange(nullptr);
	}

	FTextureReferenceReplacer RefReplacer(ExistingTexture);

	UTexture* Texture = nullptr;
	if (UDIMIndexToFile.Num() > 1)
	{
		// Import UDIM texture
		Texture = ImportTextureUDIM(Class, InParent, TextureName, Flags, Type, UDIMIndexToFile, Warn);
	}
	else
	{
		// Not a UDIM, import a regular texture
		Texture = ImportTexture(Class, InParent, TextureName, Flags, Type, Buffer, BufferEnd, Warn);
		if (Texture)
		{
			Texture->AssetImportData->Update(CurrentFilename, FileHash.IsValid() ? &FileHash : nullptr);
		}
	}

	if(!Texture)
	{
		if (ExistingTexture)
		{
			// We failed to import over the existing texture. Make sure the resource is ready in the existing texture.
			ExistingTexture->PostEditChange();
		}

		Warn->Logf(ELogVerbosity::Error, TEXT("Texture import failed") );
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport( this, nullptr );
		return nullptr;
	}

	// this is automatic now and redundant;
	//	Source.Init does UseHashAsGuid
	if (bUseHashAsGuid)
	{
		Texture->Source.UseHashAsGuid();
	}

	//Replace the reference for the new texture with the existing one so that all current users still have valid references.
	RefReplacer.Replace(Texture);

	// Start with the value that the loader suggests.
	CompressionSettings = Texture->CompressionSettings;

	// note: LODGroup is a member var on TextureFactory not pulled from Texture->LODGroup ?
	//	notice above that CompressionSettings is primed from Texture->CompressionSettings
	//	but LODGroup is not pulled from Texture->LODGroup

	// If the TextureFactory LODGroup is set to a normal map LOD group, then set CompressionSettings to Normalmap
	bool bIsNormalMapLODGroup = ( LODGroup == TEXTUREGROUP_WorldNormalMap 
		||	LODGroup == TEXTUREGROUP_CharacterNormalMap
		||	LODGroup == TEXTUREGROUP_VehicleNormalMap
		||	LODGroup == TEXTUREGROUP_WeaponNormalMap );

	if ( bIsNormalMapLODGroup )
	{
		// Change from default to normal map.
		if( CompressionSettings == TC_Default )
		{
			CompressionSettings = TC_Normalmap;
		}
	}
		
	// Propagate options.
	Texture->CompressionSettings	= CompressionSettings;
	Texture->LODGroup				= LODGroup;
	
	if(!FCString::Stricmp(Type, TEXT("ies")))
	{
		Texture->LODGroup = TEXTUREGROUP_IESLightProfile;
	}

	// note that NormalmapIdentification has not run yet
	//	so I think the only way you get in this branch is if bIsNormalMapLODGroup == true
	if( Texture->IsNormalMap() ) // TC == TC_Normalmap
	{
		Texture->SRGB = 0;
		if( !bIsNormalMapLODGroup )
		{
			Texture->LODGroup = TEXTUREGROUP_WorldNormalMap;
		}
	}

	Texture->CompressionNone				= NoCompression;
	Texture->CompressionNoAlpha				= NoAlpha;
	Texture->DeferCompression				= bDeferCompression;
	Texture->bDoScaleMipsForAlphaCoverage	= bDoScaleMipsForAlphaCoverage;
	Texture->AlphaCoverageThresholds		= AlphaCoverageThresholds;
	Texture->bUseNewMipFilter				= bUseNewMipFilter;
	// these get changed by SetModernSettingsForNewOrChangedTexture anyway

	if(Texture->MipGenSettings == TMGS_FromTextureGroup)
	{
		// unless the loader suggest a different setting
		Texture->MipGenSettings = MipGenSettings;
	}
	
	Texture->bPreserveBorder		= bPreserveBorder;
					
	if ( UE::TextureUtilitiesCommon::ShouldTextureBeVirtualByAutoImportSize(Texture) )
	{
		// only UTexture2D can be VT
		if ( Texture->GetTextureClass() == ETextureClass::TwoD )
		{
			Texture->VirtualTextureStreaming = true;
		}
		else
		{
			UE_LOGF(LogEditorFactories, Warning, "Texture [%ls] is too large for non-VT (%lld x %lld) but is not a UTexture2D.",
				*Texture->GetName(),
				Texture->Source.GetSizeX() , Texture->Source.GetSizeY());
		}
	}
	else
	{
		static const auto CVarVirtualTexturesEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTextures"));
		check(CVarVirtualTexturesEnabled);

		// if Texture is too large and should be VT but VT is not enabled, warn about that	
		if ( ! CVarVirtualTexturesEnabled->GetValueOnAnyThread() )
		{
			int32 MaxDimension = FMath::Max( Texture->Source.GetSizeX() , Texture->Source.GetSizeY() );
			bool bLargeTextureMustBeVT = MaxDimension > UTexture::GetMaximumDimensionOfNonVT();
			if ( bLargeTextureMustBeVT )
			{
				UE_LOGF(LogEditorFactories, Warning, "Texture is too large for non-VT (%lld x %lld) but VT is not enabled in this project.",
					Texture->Source.GetSizeX() , Texture->Source.GetSizeY());
			}
			else if ( Texture->Source.GetNumBlocks() > 1 )
			{
				UE_LOGF(LogEditorFactories, Warning, "Texture has blocks which requires VT but VT is not enabled in this project.");
			}
		}
	}

	UTexture2D * Texture2D = Cast<UTexture2D>(Texture);

	// Restore user set options
	if (ExistingTexture && bUsingExistingSettings)
	{
		if(Texture2D)
		{
			Texture2D->AddressX		= ExistingAddressX;
			Texture2D->AddressY		= ExistingAddressY;
		}

		Texture->Filter			= ExistingFilter;
		Texture->LODGroup		= ExistingLODGroup;
		Texture->CompressionSettings = ExistingCompressionSettings;
		Texture->LODBias		= ExistingLODBias;
		Texture->NumCinematicMipLevels = ExistingNumCinematicMipLevels;
		Texture->NeverStream	= ExistingNeverStream;
		Texture->SRGB			= ExistingSRGB;
		Texture->bPreserveBorder = ExistingPreserveBorder;
		Texture->CompressionNone = ExistingNoCompression;
		Texture->CompressionNoAlpha = ExistingNoAlpha;
		Texture->DeferCompression = ExistingDeferCompression;
		Texture->bDoScaleMipsForAlphaCoverage = ExistingbDoScaleMipsForAlphaCoverage;
		Texture->AlphaCoverageThresholds = ExistingAlphaCoverageThresholds;
		Texture->bUseNewMipFilter = ExistingbUseNewMipFilter;
		Texture->bFlipGreenChannel = ExistingFlipGreenChannel;
		Texture->AdjustBrightness = ExistingAdjustBrightness;
		Texture->AdjustBrightnessCurve = ExistingAdjustBrightnessCurve;
		Texture->AdjustVibrance = ExistingAdjustVibrance;
		Texture->AdjustSaturation = ExistingAdjustSaturation;
		Texture->AdjustRGBCurve = ExistingAdjustRGBCurve;
		Texture->AdjustHue = ExistingAdjustHue;
		Texture->AdjustMinAlpha = ExistingAdjustMinAlpha;
		Texture->AdjustMaxAlpha = ExistingAdjustMaxAlpha;
		Texture->MipGenSettings = ExistingMipGenSettings;
		Texture->VirtualTextureStreaming = ExistingVirtualTextureStreaming;
	}
	else
	{
		Texture->bFlipGreenChannel = (bFlipNormalMapGreenChannel && Texture->IsNormalMap());
		// save user option
		GConfig->SetBool( TEXT("/Script/UnrealEd.EditorEngine"), TEXT("FlipNormalMapGreenChannel"), bFlipNormalMapGreenChannel, GEngineIni );
	}

	// Automatically detect if the texture is a normal map and configure its properties accordingly

	if (!bUsingExistingSettings)
	{
		FTextureSource::FMipLock LockedMip(FTextureSource::ELockState::ReadOnly, &Texture->Source, 0);
		if (LockedMip.IsValid() && UE::NormalMapIdentification::HandleAssetPostImport(Texture, LockedMip.Image))
		{
			UE_LOGF(LogEditorFactories,Display,"Auto-detected normal map");

			Texture->bFlipGreenChannel = bFlipNormalMapGreenChannel;
		}
	}
	
	// Texture property setup is almost done
	//  ApplyDefaultsForNewlyImportedTextures before ApplyAutoImportSettings so user settings can override
	bool bIsReimport = ExistingTexture && bUsingExistingSettings;
	UE::TextureUtilitiesCommon::ApplyDefaultsForNewlyImportedTextures(Texture,bIsReimport);

	if(IsAutomatedImport())
	{
		// Apply Auto import settings 
		// Should be applied before post edit change
		ApplyAutoImportSettings(Texture);
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, Texture);

	// PostEditChange calls ValidateSettingsAfterImportOrEdit();

	// Invalidate any materials using the newly imported texture. (occurs if you import over an existing texture)
	Texture->PostEditChange();

	if (Texture2D)
	{
		// Invalidate any volume texture that was built on this texture.
		for (TObjectIterator<UVolumeTexture> It; It; ++It)
		{
			UVolumeTexture* VolumeTexture = *It;
			if (VolumeTexture && VolumeTexture->Source2DTexture == Texture2D)
			{
				VolumeTexture->UpdateSourceFromSourceTexture();
				VolumeTexture->UpdateResource();
			}
		}

		// Invalidate any Texture2DArrays that use the updated texture.
		for (TObjectIterator<UTexture2DArray> It; It; ++It) 
		{
			UTexture2DArray* TextureArray = *It;
			if (TextureArray) 
			{
				for (int32 SourceIndex = 0; SourceIndex < TextureArray->SourceTextures.Num(); ++SourceIndex) 
				{
					if (TextureArray->SourceTextures[SourceIndex] == Texture2D) 
					{
						// Update the entire texture array.
						TextureArray->UpdateSourceFromSourceTextures(false);
						// UpdateSourceFromSourceTextures calls UpdateResource
						break;
					}
				}
			}
		}
	}

	{
		// Invalidate any TextureCubeArrays that use the updated texture.
		for (TObjectIterator<UTextureCubeArray> It; It; ++It) 
		{
			UTextureCubeArray* TextureArray = *It;
			if (TextureArray) 
			{
				for (int32 SourceIndex = 0; SourceIndex < TextureArray->SourceTextures.Num(); ++SourceIndex) 
				{
					if (TextureArray->SourceTextures[SourceIndex] == Texture) 
					{
						// Update the entire texture array.
						TextureArray->UpdateSourceFromSourceTextures(false);
						// UpdateSourceFromSourceTextures calls UpdateResource
						break;
					}
				}
			}
		}
	}


	// If we are automatically creating a material for this texture...
	if( bCreateMaterial )
	{
		// Create the package for the material
		const FString MaterialName = FString::Printf( TEXT("%s_Mat"), *TextureName.ToString() );
		const FString MaterialPackageName = FPackageName::GetLongPackagePath(InParent->GetName()) + TEXT("/") + MaterialName;
		UPackage* MaterialPackage = CreatePackage( *MaterialPackageName);

		// Create the material
		UMaterialFactoryNew* Factory = NewObject<UMaterialFactoryNew>();
		UMaterial* Material = (UMaterial*)Factory->FactoryCreateNew( UMaterial::StaticClass(), MaterialPackage, *MaterialName, Flags, Context, Warn );

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(Material);

		UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();

		// Create a texture reference for the texture we just imported and hook it up to the diffuse channel
		UMaterialExpression* Expression = NewObject<UMaterialExpression>(Material, UMaterialExpressionTextureSample::StaticClass());
		Material->GetExpressionCollection().AddExpression( Expression );
		TArray<FExpressionOutput> Outputs;

		// If the user hasn't turned on any of the link checkboxes, default "bRGBToBaseColor" to being on.
		if( !bRGBToBaseColor && !bRGBToEmissive && !bAlphaToRoughness && !bAlphaToEmissive && !bAlphaToOpacity && !bAlphaToOpacityMask )
		{
			bRGBToBaseColor = 1;
		}

		// Set up the links the user asked for
		if( bRGBToBaseColor )
		{
			MaterialEditorOnly->BaseColor.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->BaseColor.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->BaseColor.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->BaseColor.Mask = Output->Mask;
			MaterialEditorOnly->BaseColor.MaskR = Output->MaskR;
			MaterialEditorOnly->BaseColor.MaskG = Output->MaskG;
			MaterialEditorOnly->BaseColor.MaskB = Output->MaskB;
			MaterialEditorOnly->BaseColor.MaskA = Output->MaskA;
		}

		if( bRGBToEmissive )
		{
			MaterialEditorOnly->EmissiveColor.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->EmissiveColor.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->EmissiveColor.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->EmissiveColor.Mask = Output->Mask;
			MaterialEditorOnly->EmissiveColor.MaskR = Output->MaskR;
			MaterialEditorOnly->EmissiveColor.MaskG = Output->MaskG;
			MaterialEditorOnly->EmissiveColor.MaskB = Output->MaskB;
			MaterialEditorOnly->EmissiveColor.MaskA = Output->MaskA;
		}

		if( bAlphaToRoughness )
		{
			MaterialEditorOnly->Roughness.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->Roughness.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->Roughness.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->Roughness.Mask = Output->Mask;
			MaterialEditorOnly->Roughness.MaskR = 0;
			MaterialEditorOnly->Roughness.MaskG = 0;
			MaterialEditorOnly->Roughness.MaskB = 0;
			MaterialEditorOnly->Roughness.MaskA = 1;
		}

		if( bAlphaToEmissive )
		{
			MaterialEditorOnly->EmissiveColor.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->EmissiveColor.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->EmissiveColor.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->EmissiveColor.Mask = Output->Mask;
			MaterialEditorOnly->EmissiveColor.MaskR = 0;
			MaterialEditorOnly->EmissiveColor.MaskG = 0;
			MaterialEditorOnly->EmissiveColor.MaskB = 0;
			MaterialEditorOnly->EmissiveColor.MaskA = 1;
		}

		if( bAlphaToOpacity )
		{
			MaterialEditorOnly->Opacity.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->Opacity.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->Opacity.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->Opacity.Mask = Output->Mask;
			MaterialEditorOnly->Opacity.MaskR = 0;
			MaterialEditorOnly->Opacity.MaskG = 0;
			MaterialEditorOnly->Opacity.MaskB = 0;
			MaterialEditorOnly->Opacity.MaskA = 1;
		}

		if( bAlphaToOpacityMask )
		{
			MaterialEditorOnly->OpacityMask.Expression = Expression;
			((UMaterialExpressionTextureSample*)MaterialEditorOnly->OpacityMask.Expression)->Texture = Texture;

			Outputs = MaterialEditorOnly->OpacityMask.Expression->GetOutputs();
			FExpressionOutput* Output = Outputs.GetData();
			MaterialEditorOnly->OpacityMask.Mask = Output->Mask;
			MaterialEditorOnly->OpacityMask.MaskR = 0;
			MaterialEditorOnly->OpacityMask.MaskG = 0;
			MaterialEditorOnly->OpacityMask.MaskB = 0;
			MaterialEditorOnly->OpacityMask.MaskA = 1;
		}

		Material->TwoSided	= bTwoSided;
		Material->BlendMode = Blending;
		Material->SetShadingModel(ShadingModel);

		Material->PostEditChange();
	}
	return Texture;
}

void UTextureFactory::ApplyAutoImportSettings(UTexture* Texture)
{
	if ( AutomatedImportSettings.IsValid() )
	{
		FJsonObjectConverter::JsonObjectToUStruct(AutomatedImportSettings.ToSharedRef(), Texture->GetClass(), Texture, 0, CPF_InstancedReference);
	}
}

bool UTextureFactory::IsImportResolutionValid(int64 Width, int64 Height, bool bAllowNonPowerOfTwo, FFeedbackContext* Warn)
{
	FText ErrorMessage;
	
	if ( UE::TextureUtilitiesCommon::IsImportResolutionValid(Width,Height,bAllowNonPowerOfTwo,&ErrorMessage) )
	{
		return true;
	}

	if ( ! ErrorMessage.IsEmpty() )
	{
		Warn->Log(ELogVerbosity::Error, ErrorMessage.ToString());
	}

	return false;
}

IImportSettingsParser* UTextureFactory::GetImportSettingsParser()
{
	return this;
}

void UTextureFactory::ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson)
{
	// Store these settings to be applied to the texture later
	AutomatedImportSettings = ImportSettingsJson;

	// Try to apply any import time options now 
	FJsonObjectConverter::JsonObjectToUStruct(ImportSettingsJson, GetClass(), this, 0, CPF_InstancedReference);
}


/*------------------------------------------------------------------------------
	UTextureExporterDDS implementation.
	UTextureExporterDDS does use UTextureExporterGeneric, but overrides its methods to do its own writing
	because it can work on more complex textures, not just simple 2D like the base class
------------------------------------------------------------------------------*/

UTextureExporterDDS::UTextureExporterDDS(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// note UTexture not UTexture2D , picks up cube etc.
	SupportedClass = UTexture::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("DDS"));
	FormatDescription.Add(TEXT("DDS DirectDraw Surface"));
}

bool UTextureExporterDDS::SupportsObject(UObject* Object) const
{
	// don't call Super (UTextureExporterGeneric), skip to UExporter
	// this checks we are "SupportedClass"
	if ( ! UExporter::SupportsObject(Object) )
	{
		return false;
	}

	UTexture * Texture = GetExportTexture( Object );
	if ( Texture == nullptr )
	{
		return false;
	}
	
	if ( !Texture->Source.IsValid() )
	{
		if ( Texture->GetTextureClass() == ETextureClass::RenderTarget )
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	if ( Texture->Source.GetFormat() == TSF_BGRE8 )
	{
		// exclude BGRE export from DDS, use HDR files if possible
		if ( Texture->Source.GetNumSlices() == 1 )
		{
			// HDR can handle it, say I won't
			return false;
		}
		// BGRE format, but something HDR can't handle, so I'll do it...
	}
	
	return true;
}

bool UTextureExporterDDS::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	UTexture * Texture = GetExportTexture( Object );
	check( Texture != nullptr );

	UTextureRenderTarget * RT = Cast<UTextureRenderTarget>(Texture);
	
	if ( RT != nullptr )
	{
		TArray64<uint8> OutArray;
		if ( ! FImageUtils::ExportRenderTargetToDDS(OutArray,RT) )
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("ExportRenderTargetToDDS failed"));
			return false;
		}

		Ar.Serialize(OutArray.GetData(),OutArray.Num());

		return true;		
	}
	else
	{

		// FileIndex for layers for VT :
	
		int NumBlocks = Texture->Source.GetNumBlocks();
		int NumLayers = Texture->Source.GetNumLayers();

		int NumFiles = NumBlocks * NumLayers;
		check( FileIndex < NumFiles );
		int BlockIndex = FileIndex / NumLayers;
		int LayerIndex = FileIndex % NumLayers; 

		UE_LOGF(LogEditorFactories, Display, "Exporting DDS from Layer %d/%d Block %d/%d", LayerIndex,NumLayers, BlockIndex,NumBlocks);

		// Type == Image extension
	
		TArray64<uint8> OutArray;
		if ( ! FImageUtils::ExportTextureSourceToDDS(OutArray,Texture,BlockIndex,LayerIndex) )
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("ExportTextureSourceToDDS failed"));
			return false;
		}

		Ar.Serialize(OutArray.GetData(),OutArray.Num());

		return true;
	}
}

UVirtualTextureBuilderExporterDDS::UVirtualTextureBuilderExporterDDS(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterDDS(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
}

/*------------------------------------------------------------------------------
	UTextureExporterGeneric implementation.

	UTextureExporterGeneric base class for all standard 2d texture exporters
------------------------------------------------------------------------------*/

static bool ExportTextureSourceGeneric( UObject* Object,FArchive& Ar, const TCHAR * ImageFormat, FFeedbackContext* Warn, int FileIndex=0)
{
	// supports export of the source texture for Texture2D and TextureCube generated from long-lat image
	if (!Object->IsA(UTexture2D::StaticClass()) && !(Object->IsA(UTextureCube::StaticClass()) && Cast<UTextureCube>(Object)->Source.IsLongLatCubemap()))
	{
		return false;
	}
	UTexture* Texture = CastChecked<UTexture>( Object );
	if ( Texture == nullptr )
	{
		return false;
	}

	if ( !Texture->Source.IsValid() )
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("ExportTexture failed : source not valid"));
		return false;
	}
	
	int NumBlocks = Texture->Source.GetNumBlocks();
	int NumLayers = Texture->Source.GetNumLayers();

	int NumFiles = NumBlocks * NumLayers;
	check( FileIndex < NumFiles );
	int BlockIndex = FileIndex / NumLayers;
	int LayerIndex = FileIndex % NumLayers; 

	UE_LOGF(LogEditorFactories, Display, "Exporting Texture as %ls Layer %d/%d Block %d/%d", ImageFormat, LayerIndex,NumLayers, BlockIndex,NumBlocks);

	FImage Image;
	if ( ! Texture->Source.GetMipImage(Image,BlockIndex,LayerIndex,0) )
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("ExportTexture failed : could not GetMipImage"));
		return false;
	}
	
	TArray64<uint8> OutArray;
	if ( ! FImageUtils::CompressImage(OutArray,ImageFormat,Image) )
	{
		Warn->Logf(ELogVerbosity::Error, TEXT("ExportTexture failed : could not CompressImage to format %s"),ImageFormat);
		return false;
	}

	Ar.Serialize(OutArray.GetData(),OutArray.Num());

	return true;
}

UTextureExporterGeneric::UTextureExporterGeneric(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTexture2D::StaticClass();
	PreferredFormatIndex = 0;
}

UTexture* UTextureExporterGeneric::GetExportTexture(UObject* Object) const
{
	// for UVirtualTextureBuilderExporter
	UVirtualTextureBuilder* VirtualTextureBuilder = Cast<UVirtualTextureBuilder>(Object);
	if (VirtualTextureBuilder != nullptr)
	{
		return VirtualTextureBuilder->Texture;
	}
	return Cast<UTexture>(Object);
}

// note that UTextureExporterGeneric will actually be registered as an exporter itself
// but will return false for SupportsObject because it has SupportsTexture == false

bool UTextureExporterGeneric::SupportsObject(UObject* Object) const
{
	bool bSupportsObject = false;
	if (Super::SupportsObject(Object))
	{
		UTexture* Texture = GetExportTexture(Object);
		// only Texture2D's for generic :
		UTexture2D * Texture2D = Cast<UTexture2D>(Texture);
		if ( Texture2D != nullptr )
		{
			if ( ! Texture->Source.IsValid() )
			{
				// should not get here
				check(0);
				return false;
			}

			return SupportsTexture(Texture);
		}
	}
	return false;
}

int32 UTextureExporterGeneric::GetFileCount(UObject* Object) const
{
	UTexture* Texture = GetExportTexture(Object);
	check(Texture != nullptr);

	if ( Texture->Source.IsValid() )
	{
		// standard textures will have NumBlocks == NumLayers == 1
		// VT can have them > 1
		// UDIM gives you NumBlocks
		int NumBlocks = Texture->Source.GetNumBlocks();
		int NumLayers = Texture->Source.GetNumLayers();
		int FileCount = NumBlocks * NumLayers;
		check( FileCount > 0 );
		return FileCount;
	}
	else
	{
		// eg. render targets
		return 1;
	}
}

FString UTextureExporterGeneric::GetUniqueFilename( UObject* Object, const TCHAR* Filename, int32 FileIndex, int32 FileCount ) const
{
	check( FileCount > 0 );
	if ( FileCount == 1 )
	{
		// standard single texture
		return Filename;
	}
	else
	{
		//	for VT with layers/blocks

		UTexture* Texture = GetExportTexture(Object);
		check( Texture != nullptr );
		
		int NumBlocks = Texture->Source.GetNumBlocks();
		int NumLayers = Texture->Source.GetNumLayers();
		check( FileCount == NumBlocks * NumLayers );
		
		// this must match ExportTexture2DGeneric :
		int BlockIndex = FileIndex / NumLayers;
		int LayerIndex = FileIndex % NumLayers; 
		
		// make a UDIM name like MyTexture.1001.png

		FString OutName = FPaths::GetBaseFilename(Filename, false);
		// if Filename already has a .1001 in it, remove it
		OutName.RemoveFromEnd( FString(TEXT(".1001")) );

		if ( NumLayers > 1 )
		{
			OutName += FString::Printf(TEXT(".L%d"), LayerIndex);
		}

		if ( NumBlocks > 1 )
		{
			FTextureSourceBlock Block;
			Texture->Source.GetBlock(BlockIndex,Block);
			int UDIMCoordinate = 1001 + Block.BlockX + Block.BlockY*10;
			OutName += FString::Printf(TEXT(".%d"), UDIMCoordinate);
		}

		OutName += FPaths::GetExtension(Filename, true);

		return OutName;
	}
}

bool UTextureExporterGeneric::ExportBinary( UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags )
{
	UTexture* Texture = GetExportTexture(Object);
	check( Texture != nullptr );
		
	UTexture2D * Texture2D = Cast<UTexture2D>(Texture);
	check( Texture2D != nullptr );
	check( SupportsTexture(Texture) );
	
	// FileIndex is for layers for VT
	// Type == Image extension
	return ExportTextureSourceGeneric(Texture2D,Ar,Type,Warn,FileIndex);
}

//===========================================================
// formats derived from UTextureExporterGeneric
// just need to add their FormatExtension
// and implement SupportsTexture

UTextureExporterBMP::UTextureExporterBMP(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterGeneric(ObjectInitializer)
{
	FormatExtension.Add(TEXT("BMP"));
	FormatDescription.Add(TEXT("Windows Bitmap"));
}

bool UTextureExporterBMP::SupportsTexture(UTexture* Texture) const
{
	// U8 formats only :
	return Texture->Source.GetFormat() == TSF_BGRA8 || Texture->Source.GetFormat() == TSF_G8;
}

UVirtualTextureBuilderExporterBMP::UVirtualTextureBuilderExporterBMP(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterBMP(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
}

//========================================================

UTextureExporterHDR::UTextureExporterHDR(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterGeneric(ObjectInitializer)
{
	FormatExtension.Add(TEXT("HDR"));
	FormatDescription.Add(TEXT("HDR Radiance RGBE texture"));
}

bool UTextureExporterHDR::SupportsTexture(UTexture* Texture) const
{
	// only BGRE is allowed to go out to HDR now (lossless)
	bool bSupportsObject = Texture->Source.GetFormat() == TSF_BGRE8;
	return bSupportsObject;
}

UVirtualTextureBuilderExporterHDR::UVirtualTextureBuilderExporterHDR(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterHDR(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
}

//========================================================

UTextureExporterPNG::UTextureExporterPNG(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterGeneric(ObjectInitializer)
{
	FormatExtension.Add(TEXT("PNG"));
	FormatDescription.Add(TEXT("PNG"));
}

bool UTextureExporterPNG::SupportsTexture(UTexture* Texture) const
{
	ETextureSourceFormat TSF = Texture->Source.GetFormat();
	ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(TSF);
	// supports all non-HDR formats :
	return ! ERawImageFormat::IsHDR(RawFormat);
}

UVirtualTextureBuilderExporterPNG::UVirtualTextureBuilderExporterPNG(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterPNG(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
}

//========================================================

UTextureExporterEXR::UTextureExporterEXR(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterGeneric(ObjectInitializer)
{
	FormatExtension.Add(TEXT("EXR"));
	FormatDescription.Add(TEXT("EXR HDR float texture"));
}

bool UTextureExporterEXR::SupportsTexture(UTexture* Texture) const
{
	ETextureSourceFormat TSF = Texture->Source.GetFormat();
	ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(TSF);
	// supports all HDR formats :
	return ERawImageFormat::IsHDR(RawFormat);
}

UVirtualTextureBuilderExporterEXR::UVirtualTextureBuilderExporterEXR(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterEXR(ObjectInitializer)
{
	SupportedClass = UVirtualTextureBuilder::StaticClass();
}


/*------------------------------------------------------------------------------
	URenderTargetExporterHDR implementation.
	Exports render targets.
------------------------------------------------------------------------------*/
// This is not for screenshots it's for "rendertarget" assets in levels
//	when you do asset action export on them

static bool ExportRenderTarget2DGeneric(UObject* Object, FArchive& Ar, const TCHAR * ImageExtension)
{
	UTextureRenderTarget2D* TexRT2D = Cast<UTextureRenderTarget2D>(Object);

	if (TexRT2D == nullptr)
	{
		return false;
	}
	
	FImage Image;
	if ( ! FImageUtils::GetRenderTargetImage(TexRT2D,Image) )
	{
		return false;
	}

	TArray64<uint8> CompressedData;
	if ( ! FImageUtils::CompressImage(CompressedData,ImageExtension,Image) )
	{
		return false;
	}

	Ar.Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());

	return true;
}

URenderTargetExporterPNG::URenderTargetExporterPNG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTextureRenderTarget2D::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("PNG"));
	FormatDescription.Add(TEXT("PNG"));
}

static int GetBitsPerComponent(EPixelFormat Format)
{
	if ( Format == PF_A2B10G10R10 ) return 10; // doesn't handle heterogenous bit counts well

	const FPixelFormatInfo & Info = GPixelFormats[Format];
	// rounds down
	int BitsPerComponent = ( Info.BlockBytes * 8 ) / ( Info.BlockSizeX * Info.BlockSizeY * Info.BlockSizeZ * Info.NumComponents );
	return BitsPerComponent;
}

bool URenderTargetExporterPNG::SupportsObject(UObject* Object) const
{
	if (Super::SupportsObject(Object))
	{
		UTextureRenderTarget2D* TexRT2D = Cast<UTextureRenderTarget2D>(Object);

		if (TexRT2D)
		{
			EPixelFormat PixelFormat = TexRT2D->GetFormat();

			return GetBitsPerComponent(PixelFormat) <= 8;
		}
	}
	return false;
}

bool URenderTargetExporterPNG::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	return ExportRenderTarget2DGeneric(Object,Ar,TEXT("PNG"));
}

URenderTargetExporterEXR::URenderTargetExporterEXR(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTextureRenderTarget2D::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("EXR"));
	FormatDescription.Add(TEXT("EXR HDR float rendertarget"));
}

bool URenderTargetExporterEXR::SupportsObject(UObject* Object) const
{
	if (Super::SupportsObject(Object))
	{
		UTextureRenderTarget2D* TexRT2D = Cast<UTextureRenderTarget2D>(Object);

		if (TexRT2D)
		{
			EPixelFormat PixelFormat = TexRT2D->GetFormat();
			
			return GetBitsPerComponent(PixelFormat) > 8;
		}
	}
	return false;
}

bool URenderTargetExporterEXR::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	return ExportRenderTarget2DGeneric(Object,Ar,TEXT("EXR"));
}

/*------------------------------------------------------------------------------
	UTextureCubeExporterHDR implementation.
	Export UTextureCubes as .HDR
------------------------------------------------------------------------------*/
UTextureCubeExporterHDR::UTextureCubeExporterHDR(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTextureCube::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("HDR"));
	FormatDescription.Add(TEXT("HDR cube LongLat"));
	
	// this is the only exporter registered for UTextureCube
	// this does GenerateLongLatUnwrap
}

bool UTextureCubeExporterHDR::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	UTextureCube* TexCube = Cast<UTextureCube>(Object);
	// for URenderTargetCubeExporterHDR
	UTextureRenderTargetCube* TexRTCube = Cast<UTextureRenderTargetCube>(Object);

	if (TexCube != nullptr)
	{
		if (TexCube->Source.IsLongLatCubemap() && TexCube->Source.GetFormat() == TSF_BGRE8)
		{
			// export the source HDR long-lat texture if available
			return ExportTextureSourceGeneric(TexCube, Ar, Type, Warn, FileIndex);
		}
		else
		{
			// this does GenerateLongLatUnwrap
			return FImageUtils::ExportTextureCubeAsHDR(TexCube, Ar);
		}
	}
	else if (TexRTCube != nullptr)
	{
		// -> this can continue to use HDR for the longlat unwraps
		// this does GenerateLongLatUnwrap
		return FImageUtils::ExportRenderTargetCubeAsHDR(TexRTCube, Ar);
	}

	return false;
}

URenderTargetCubeExporterHDR::URenderTargetCubeExporterHDR(const FObjectInitializer& ObjectInitializer)
	: UTextureCubeExporterHDR(ObjectInitializer)
{
	SupportedClass = UTextureRenderTargetCube::StaticClass();
}

//-------------
// UTextureExporterJPEG does not let you compress data to JPEG
//	it only writes out existing JPEG data
// do NOT use UTextureExporterGeneric here, that would go to ImageWrapper and compress to JPEG
// note: does not support UDIM blocks like UTextureExporterGeneric

UTextureExporterJPEG::UTextureExporterJPEG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTexture2D::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("JPG"));
	FormatDescription.Add(TEXT("JPEG original imported into uasset"));
}

bool UTextureExporterJPEG::SupportsObject(UObject* Object) const
{
	if (Super::SupportsObject(Object))
	{
		UTexture2D* Texture = Cast<UTexture2D>(Object);

		if (Texture)
		{
			// we do NOT do lossy recompression
			
			if (Texture->Source.GetNumBlocks() > 1 )
			{
				// does not support UDIM
				return false;
			}

			// Check it has JPEG BulkData :
			if ( (Texture->Source.GetSourceCompression() == TSCF_JPEG || Texture->Source.GetSourceCompression() == TSCF_UEJPEG) &&
				Texture->Source.GetSizeOnDisk() > 0 )
			{
				ETextureSourceFormat TSF = Texture->Source.GetFormat();
				ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(TSF);
				check( RawFormat == ERawImageFormat::G8 || RawFormat == ERawImageFormat::BGRA8 );

				return true;
			}
		}
	}
	return false;
}

bool UTextureExporterJPEG::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	UTexture2D* Texture = Cast<UTexture2D>(Object);
	check(Texture != nullptr);

	check((Texture->Source.GetSourceCompression() == TSCF_JPEG || Texture->Source.GetSourceCompression() == TSCF_UEJPEG) &&
			Texture->Source.GetSizeOnDisk() > 0 );

	// just write the JPEG data we already have :
	
	UE_LOGF(LogEditorFactories, Display, "Exporting Texture as JPEG stored bits (no lossy decompress or recompress).");

	Texture->Source.OperateOnLoadedBulkData( [&](const FSharedBuffer& BulkDataBuffer) {
		if (Texture->Source.GetSourceCompression() == TSCF_JPEG)
		{
			Ar.Serialize(const_cast<void*>(BulkDataBuffer.GetData()), BulkDataBuffer.GetSize());
		}
		else
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::UEJPEG);
			ImageWrapper->SetCompressed(BulkDataBuffer.GetData(), BulkDataBuffer.GetSize());
			TArray64<uint8> ExportData = ImageWrapper->GetExportData();
			Ar.Serialize(ExportData.GetData(), ExportData.Num());
		}
	} );
	return true;
}

//-------------
// UTextureExporterUEJPEG does not let you compress data to UEJPEG
//	it only writes out existing UEJPEG data
// do NOT use UTextureExporterGeneric here, that would go to ImageWrapper and compress to UEJPEG
// note: does not support UDIM blocks like UTextureExporterGeneric

UTextureExporterUEJPEG::UTextureExporterUEJPEG(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTexture2D::StaticClass();
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("UEJ"));
	FormatDescription.Add(TEXT("UE-JPEG original imported into uasset"));
}

bool UTextureExporterUEJPEG::SupportsObject(UObject* Object) const
{
	if (Super::SupportsObject(Object))
	{
		UTexture2D* Texture = Cast<UTexture2D>(Object);

		if (Texture)
		{
			// we do NOT do lossy recompression
			
			if (Texture->Source.GetNumBlocks() > 1 )
			{
				// does not support UDIM
				return false;
			}

			// Check it has JPEG BulkData :
			if ( Texture->Source.GetSourceCompression() == TSCF_UEJPEG && Texture->Source.GetSizeOnDisk() > 0 )
			{
				ETextureSourceFormat TSF = Texture->Source.GetFormat();
				ERawImageFormat::Type RawFormat = FImageCoreUtils::ConvertToRawImageFormat(TSF);
				check( RawFormat == ERawImageFormat::G8 || RawFormat == ERawImageFormat::BGRA8 );

				return true;
			}
		}
	}
	return false;
}

bool UTextureExporterUEJPEG::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
	UTexture2D* Texture = Cast<UTexture2D>(Object);
	check(Texture != nullptr);

	check(Texture->Source.GetSourceCompression() == TSCF_UEJPEG && Texture->Source.GetSizeOnDisk() > 0 );

	// just write the JPEG data we already have :
	
	UE_LOGF(LogEditorFactories, Display, "Exporting Texture as UE-JPEG stored bits (no lossy decompress or recompress).");

	Texture->Source.OperateOnLoadedBulkData( [&](const FSharedBuffer& BulkDataBuffer) {
		Ar.Serialize(const_cast<void*>(BulkDataBuffer.GetData()), BulkDataBuffer.GetSize());
	} );
	return true;
}
/*------------------------------------------------------------------------------
	UTextureExporterTGA implementation.
------------------------------------------------------------------------------*/
UTextureExporterTGA::UTextureExporterTGA(const FObjectInitializer& ObjectInitializer)
	: UTextureExporterGeneric(ObjectInitializer)
{
	FormatExtension.Add(TEXT("TGA"));
	FormatDescription.Add(TEXT("Targa"));
}

bool UTextureExporterTGA::SupportsTexture(UTexture* Texture) const
{
	ETextureSourceFormat TSF = Texture->Source.GetFormat();
	
	// U8 formats only :
	return TSF == TSF_BGRA8 || TSF == TSF_G8;
}

/*------------------------------------------------------------------------------
	UFontFactory.
------------------------------------------------------------------------------*/

UFontFactory::UFontFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFont::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UFontFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	UFont* const Font = NewObject<UFont>(InParent, InClass, InName, InFlags);
	if(Font)
	{
		Font->FontCacheType = EFontCacheType::Runtime;
	}
	return Font;
}

/*------------------------------------------------------------------------------
	UFontFileImportFactory.
------------------------------------------------------------------------------*/

UFontFileImportFactory::UFontFileImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFontFace::StaticClass();

	bEditorImport = true;

	Formats.Add(TEXT("ttf;TrueType Font"));
	Formats.Add(TEXT("ttc;TrueType Font"));
	Formats.Add(TEXT("otf;OpenType Font"));
	Formats.Add(TEXT("otc;OpenType Font"));
	
	BatchCreateFontAsset = EBatchCreateFontAsset::Unknown;
}

bool UFontFileImportFactory::ConfigureProperties()
{
	BatchCreateFontAsset = EBatchCreateFontAsset::Unknown;
	return true;
}

UObject* UFontFileImportFactory::FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, const TCHAR* InType, const uint8*& InBuffer, const uint8* InBufferEnd, FFeedbackContext* InWarn)
{
	const FString CreatedFontPackageName = FString::Printf(TEXT("%s/%s_Font"), *FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetName()), *InName.ToString());

	// Should we create a font asset alongside our font face?
	bool bCreateFontAsset = false;
	{
		const bool bIsAutomated = IsAutomatedImport();
		const bool bShowImportDialog = BatchCreateFontAsset == EBatchCreateFontAsset::Unknown && !bIsAutomated;
		if (bShowImportDialog)
		{
			const FText DlgTitle = LOCTEXT("ImportFont_OptionsDlgTitle", "Font Face Import Options");
			const FText DlgMsg = LOCTEXT("ImportFont_OptionsDlgMsg", "Would you like to create a new Font asset using the imported Font Face as its default font?");
			switch (FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel, DlgMsg, DlgTitle))
			{
			case EAppReturnType::Yes:
				bCreateFontAsset = true;
				break;
			case EAppReturnType::YesAll:
				bCreateFontAsset = true;
				BatchCreateFontAsset = EBatchCreateFontAsset::Yes;
				break;
			case EAppReturnType::No:
				break;
			case EAppReturnType::NoAll:
				BatchCreateFontAsset = EBatchCreateFontAsset::No;
				break;
			default:
				BatchCreateFontAsset = EBatchCreateFontAsset::Cancel;
				break;
			}
		}
		else
		{
			bCreateFontAsset = BatchCreateFontAsset == EBatchCreateFontAsset::Yes;
			if (BatchCreateFontAsset == EBatchCreateFontAsset::CreateIfNoFontExists)
			{
				FAssetData Unused;
				const FString CreatedFontObjectName = FString::Printf(TEXT("%s.%s_Font"), *CreatedFontPackageName, *InName.ToString());
				if (IAssetRegistryInterface::GetPtr()->TryGetAssetByObjectPath(FSoftObjectPath(CreatedFontObjectName), Unused) != UE::AssetRegistry::EExists::Exists)
				{
					bCreateFontAsset = true;
				}
			}
		}
	}

	if (BatchCreateFontAsset == EBatchCreateFontAsset::Cancel)
	{
		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, InType);

	// Create the font face
	UFontFace* const FontFace = NewObject<UFontFace>(InParent, InClass, InName, InFlags);
	if (FontFace)
	{
		FontFace->SourceFilename = GetCurrentFilename();

		TArray<uint8> FontData;
		int32 BufferSize = IntCastChecked<int32>(InBufferEnd - InBuffer);
		FontData.Append(InBuffer, BufferSize);
		FontFace->FontFaceData->SetData(MoveTemp(FontData));
		FontFace->CacheSubFaces();
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, FontFace);
	
	// Create the font (if requested)
	if (FontFace && bCreateFontAsset)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		FString FontPackageName;
		FString FontAssetName;
		AssetToolsModule.Get().CreateUniqueAssetName(CreatedFontPackageName, FString(), FontPackageName, FontAssetName);

		UFontFactory* FontFactory = NewObject<UFontFactory>();
		FontFactory->bEditAfterNew = false;

		UPackage* FontPackage = CreatePackage( *FontPackageName);
		UFont* Font = Cast<UFont>(FontFactory->FactoryCreateNew(UFont::StaticClass(), FontPackage, *FontAssetName, InFlags, InContext, InWarn));
		if (Font)
		{
			Font->FontCacheType = EFontCacheType::Runtime;

			// Add a default typeface referencing the newly created font face
			FTypefaceEntry& DefaultTypefaceEntry = Font->GetMutableInternalCompositeFont().DefaultTypeface.Fonts.AddDefaulted_GetRef();
			DefaultTypefaceEntry.Name = "Default";
			DefaultTypefaceEntry.Font = FFontData(FontFace);

			FAssetRegistryModule::AssetCreated(Font);
			FontPackage->MarkPackageDirty();
		}
	}

	return FontFace;
}

bool UFontFileImportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UFontFace* FontFaceToReimport = Cast<UFontFace>(Obj);
	if (FontFaceToReimport)
	{
		OutFilenames.Add(FontFaceToReimport->SourceFilename);
		return true;
	}
	return false;
}

void UFontFileImportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UFontFace* FontFaceToReimport = Cast<UFontFace>(Obj);
	if (FontFaceToReimport && ensure(NewReimportPaths.Num() == 1))
	{
		FontFaceToReimport->SourceFilename = NewReimportPaths[0];
	}
}

EReimportResult::Type UFontFileImportFactory::Reimport(UObject* InObject)
{
	UFontFace* FontFaceToReimport = Cast<UFontFace>(InObject);

	if (!FontFaceToReimport)
	{
		return EReimportResult::Failed;
	}

	if (FontFaceToReimport->SourceFilename.IsEmpty() || !FPaths::FileExists(FontFaceToReimport->SourceFilename))
	{
		return EReimportResult::Failed;
	}

	// Never create font assets when reimporting
	BatchCreateFontAsset = EBatchCreateFontAsset::No;

	bool OutCanceled = false;
	if (ImportObject(InObject->GetClass(), InObject->GetOuter(), *InObject->GetName(), RF_Public | RF_Standalone, FontFaceToReimport->SourceFilename, nullptr, OutCanceled))
	{
		return EReimportResult::Succeeded;
	}

	return OutCanceled ? EReimportResult::Cancelled : EReimportResult::Failed;
}

int32 UFontFileImportFactory::GetPriority() const
{
	return ImportPriority;
}

/*------------------------------------------------------------------------------
FCustomizableTextObjectFactory implementation.
------------------------------------------------------------------------------*/

// Util to ensure that InName is a valid name for a new object within InParent. Will rename any existing object within InParent if it is called InName.
void FCustomizableTextObjectFactory::ClearObjectNameUsage(UObject* InParent, FName InName)
{
	// Make sure this name is unique within the scope of InParent.
	UObject* Found=nullptr;
	if( (InName != NAME_None) && (InParent != nullptr) )
	{
		Found = FindObject<UObject>( InParent, *InName.ToString() );
	}

	// If there is already another object in the same scope with this name, rename it.
	while (Found)
	{
		check(Found->GetOuter() == InParent);

		Found->Rename(nullptr, nullptr, REN_DontCreateRedirectors);

		// It's possible after undo for there to be multiple objects with the same name in the way, rename all of them
		Found = FindObject<UObject>(InParent, *InName.ToString());
	}
}

// Constructor for the factory; takes a context for emitting warnings such as GWarn
FCustomizableTextObjectFactory::FCustomizableTextObjectFactory(FFeedbackContext* InWarningContext)
	: WarningContext(InWarningContext)
{
}

// Parses a text buffer and factories objects from it, subject to the restrictions imposed by CanCreateClass()
void FCustomizableTextObjectFactory::ProcessBuffer(UObject* InParent, EObjectFlags Flags, const FString& TextBuffer)
{
	ProcessBuffer(InParent, Flags, *TextBuffer);
}

void FCustomizableTextObjectFactory::ProcessBuffer(UObject* InParent, EObjectFlags Flags, const TCHAR* Buffer)
{
	// We keep a mapping of new, empty sequence objects to their property text.
	// We want to create all new SequenceObjects first before importing their properties (which will create links)
	TArray<UObject*>			NewObjects;
	TMap<UObject*,FString>		PropMap;

	FParse::Next( &Buffer );

	int32 NestedDepth     = 0;
	int32 OmittedOuterObj = 0; // zero signifies "nothing omitted"

	const bool bT3DOverrideEnabled = FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization);

	// Local graphs to use for objects that require dedicated instancing graphs.
	TMap<UObject*, FObjectInstancingGraph> DedicatedGraphs;

	FString StrLine;
	while( FParse::Line(&Buffer,StrLine) )
	{
		const TCHAR* Str = *StrLine;
		if( GetBEGIN(&Str, TEXT("OBJECT")) || (NestedDepth == 0 && GetBEGIN(&Str, TEXT("ACTOR"))) )
		{
			++NestedDepth;
			if (OmittedOuterObj > 0)
			{
				if (NestedDepth > OmittedOuterObj)
				{
					continue;
				}
				ensure(OmittedOuterObj == NestedDepth);
				// clear the omitted outer, we've parsed passed it
				OmittedOuterObj = 0;
			}

			UClass* ObjClass;
			if( ParseObject<UClass>( Str, TEXT("CLASS="), ObjClass, nullptr ) )
			{
				bool bOmitSubObjects = false;
				if (!CanCreateClass(ObjClass, bOmitSubObjects))
				{
					if (bOmitSubObjects)
					{
						OmittedOuterObj = NestedDepth;
					}
					continue;
				}

				FName ObjName(NAME_None);
				FParse::Value( Str, TEXT("NAME="), ObjName );

				// Setup archetype
				UObject* ObjArchetype = nullptr;
				FString ObjArchetypeName;
				if (FParse::Value(Str, TEXT("ARCHETYPE="), ObjArchetypeName))
				{
					ObjArchetype = LoadObject<UObject>(nullptr, *ObjArchetypeName, nullptr, LOAD_None, nullptr);
				}

				FName OverriddenOperationName = NAME_None;
				TOptional<EOverriddenPropertyOperation> OverriddenOperation;
				const bool bParsed = FParse::Value(Str,TEXT("OverriddenOperation="),OverriddenOperationName);
				if (bParsed && bT3DOverrideEnabled)
				{
					OverriddenOperation = GetOverriddenOperationFromName(OverriddenOperationName);
				}

				UObject* ObjectParent = InParent ? InParent : GetParentForNewObject(ObjClass);

				// Make sure we are allowed to create this object
				if(!CanCreateObject(ObjectParent, ObjClass, ObjName))
				{
					continue;
				}

				// Make sure this name is not used by anything else. Will rename other stuff if necessary
				UpdateObjectName(ObjClass, ObjectParent, ObjName);
				ClearObjectNameUsage(ObjectParent, ObjName);

				// Spawn the object and reset it's archetype
				FObjectInstancingGraph TempGraph;
				bool bUseDedicatedInstancingGraph = FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraphForNewObject(ObjClass, Flags, ObjArchetype, bT3DOverrideEnabled && OverriddenOperation);
				UObject* CreatedObject = NewObject<UObject>(ObjectParent, ObjClass, ObjName, Flags, ObjArchetype, !!ObjectParent, bUseDedicatedInstancingGraph ? &TempGraph : &InstanceGraph);
				if (bUseDedicatedInstancingGraph)
				{
					if (!TempGraph.HasDestinationRoot())
					{
						TempGraph.SetDestinationRoot(CreatedObject);
					}
					DedicatedGraphs.Emplace(CreatedObject, MoveTemp(TempGraph));
				}

				// Overridable handling
				if (bT3DOverrideEnabled)
				{
					if (OverriddenOperation.IsSet())
					{
						FOverridableManager::Get().RestoreOverrideOperation(CreatedObject, OverriddenOperation.GetValue(), /*bNeedsSubObjectTemplateInstantiation*/true);
					}
					else
					{
						FOverridableManager::Get().Disable(CreatedObject);
					}
				}

				// Get property text for the new object.
				FString PropText, PropLine;
				FString SubObjText;
				int32 ObjDepth = 1;
				while ( FParse::Line( &Buffer, PropLine ) )
				{
					const TCHAR* PropStr = *PropLine;

					// Track how deep we are in contained sets of sub-objects.
					bool bEndLine = false;
					if( GetBEGIN(&PropStr, TEXT("OBJECT")) )
					{
						ObjDepth++;
					}
					else if( GetEND(&PropStr, TEXT("OBJECT")) || (ObjDepth == 1 && GetEND(&PropStr, TEXT("ACTOR"))) )
					{
						bEndLine = true;

						// When close out our initial BEGIN OBJECT, we are done with this object.
						if(ObjDepth == 1)
						{
							break;
						}
					}

					PropText += *PropLine;
					PropText += TEXT("\r\n");

					if(bEndLine)
					{
						ObjDepth--;
					}
				}

				// Save property text and possibly sub-object text.
				PropMap.Add(CreatedObject, *PropText);
				NewObjects.Add(CreatedObject);
			}
		}
		else if (GetEND(&Str, TEXT("OBJECT")) || (NestedDepth == 1 && GetEND(&Str, TEXT("ACTOR"))))
		{
			--NestedDepth;
		}
		else
		{
			ProcessUnidentifiedLine(StrLine);
		}
	}

	// Apply the property text to each of the created objects
	for (int32 i = 0; i < NewObjects.Num(); ++i)
	{
		UObject* CreatedObject = NewObjects[i];
		const FString& PropText = PropMap.FindChecked(CreatedObject);

		FObjectInstancingGraph* InstanceGraphToUse = DedicatedGraphs.Find(CreatedObject);
		if (!InstanceGraphToUse)
		{
			InstanceGraphToUse = &InstanceGraph;
		}

		// Import the properties and give the derived factory a shot at it
		if (ImportObjectProperties((uint8*)CreatedObject, *PropText, CreatedObject->GetClass(), CreatedObject, CreatedObject, WarningContext, 0, 0, InstanceGraphToUse))
		{
			ProcessConstructedObject(CreatedObject);
		}
	}
	PostProcessConstructedObjects();
}


bool FCustomizableTextObjectFactory::CanCreateObjectsFromText( const FString& TextBuffer ) const
{
	bool bCanCreate = false;

	const TCHAR* Buffer = *TextBuffer;
	const TCHAR* BufferEnd = Buffer + FCString::Strlen(Buffer);

	FParse::Next( &Buffer );

	int32 NestedDepth     = 0;
	int32 OmittedOuterObj = 0; // zero signifies "nothing omitted"

	FString StrLine;
	while( FParse::Line(&Buffer,StrLine) )
	{
		const TCHAR* Str = *StrLine;
		if( GetBEGIN(&Str,TEXT("OBJECT")) || (NestedDepth == 0 && GetBEGIN(&Str, TEXT("ACTOR"))) )
		{
			++NestedDepth;
			if (OmittedOuterObj > 0)
			{
				if (NestedDepth > OmittedOuterObj)
				{
					continue;
				}
				ensure(OmittedOuterObj == NestedDepth);
				// clear the omitted outer, we've parsed passed it
				OmittedOuterObj = 0;
			}

			UClass* ObjClass;
			if (ParseObject<UClass>(Str, TEXT("CLASS="), ObjClass, nullptr))
			{
				bool bOmitSubObjects = false;;
				if (CanCreateClass(ObjClass, bOmitSubObjects))
				{
					bCanCreate = true;
					break;
				}
				else if (bOmitSubObjects)
				{
					OmittedOuterObj = NestedDepth;
				}
			}
		}
		else if ( GetEND(&Str, TEXT("OBJECT")) || (NestedDepth == 1 && GetEND(&Str, TEXT("ACTOR"))) )
		{
			--NestedDepth;
		}
	}
	return bCanCreate;
}

/** Return true if the an object of type ObjectClass is allowed to be created; If false is returned, the object and subobjects will be ignored. */
bool FCustomizableTextObjectFactory::CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const
{
	return false;
}

bool FCustomizableTextObjectFactory::CanCreateObject(UObject* InParent, UClass* ObjectClass,
	const FName& InDesiredName) const
{
	return true;
}

/** This is called on each created object after PreEditChange and the property text is imported, but before PostEditChange */
void FCustomizableTextObjectFactory::ProcessConstructedObject(UObject* CreatedObject)
{
}

/*-----------------------------------------------------------------------------
UReimportTextureFactory.
-----------------------------------------------------------------------------*/
UReimportTextureFactory::UReimportTextureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UTexture::StaticClass();

	bCreateNew = false;
	
	// Required to allow texture factory to take priority when importing new image files
	ImportPriority = DefaultImportPriority - 1;
}

UTexture2D* UReimportTextureFactory::CreateTexture2D( UObject* InParent, FName Name, EObjectFlags Flags )
{
	UTexture2D* pTex2D = Cast<UTexture2D>(pOriginalTex);

	if (pTex2D)
	{
		// Release the existing resource so the new texture can get a fresh one. Otherwise if the next call to Init changes the format
		// of the texture and UpdateResource is called the editor will crash in RenderThread
		pTex2D->ReleaseResource();
		return pTex2D;
	}
	else
	{
		return Super::CreateTexture2D( InParent, Name, Flags );
	}
}

UTextureCube* UReimportTextureFactory::CreateTextureCube( UObject* InParent, FName Name, EObjectFlags Flags )
{
	UTextureCube* pTexCube = Cast<UTextureCube>(pOriginalTex);
	if (pTexCube)
	{
		// Release the existing resource so the new texture can get a fresh one. Otherwise if the next call to Init changes the format
		// of the texture and UpdateResource is called the editor will crash in RenderThread
		pTexCube->ReleaseResource();
		return pTexCube;
	}
	else
	{
		return Super::CreateTextureCube( InParent, Name, Flags );
	}
}

bool UReimportTextureFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UTexture* pTex = Cast<UTexture>(Obj);
	if( pTex && !pTex->IsA<UTextureRenderTarget>() && !pTex->IsA<UCurveLinearColorAtlas>())
	{
		pTex->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UReimportTextureFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UTexture* pTex = Cast<UTexture>(Obj);
	if(pTex && ensure(NewReimportPaths.Num() == 1))
	{
		pTex->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

/**
* Reimports specified texture from its source material, if the meta-data exists
*/
EReimportResult::Type UReimportTextureFactory::Reimport( UObject* Obj )
{
	TGuardValue<bool> IsDoingAReimportGuard(bIsDoingAReimport, true);

	if(!Obj || !Obj->IsA(UTexture::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	UTexture* pTex = Cast<UTexture>(Obj);
	
	TGuardValue<decltype(pOriginalTex)> OriginalTexGuardValue(pOriginalTex, pTex);

	const FString ResolvedSourceFilePath = pTex->AssetImportData->GetFirstFilename();
	if (!ResolvedSourceFilePath.Len())
	{
		// Since this is a new system most textures don't have paths, so logging has been commented out
		//UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: texture resource does not have path stored.");
		return EReimportResult::Failed;
	}

	UTexture2D* pTex2D = Cast<UTexture2D>(Obj);
	// Check if this texture has been modified by the paint tool.
	// If so, prompt the user to see if they'll continue with reimporting, returning if they decline.
	if( pTex2D && pTex2D->bHasBeenPaintedInEditor && !IsAutomatedImport() && EAppReturnType::Yes != FMessageDialog::Open( EAppMsgType::YesNo,
		FText::Format(NSLOCTEXT("UnrealEd", "Import_TextureHasBeenPaintedInEditor", "The texture '{0}' has been painted on by the Mesh Paint tool.\nReimporting it will override any changes.\nWould you like to continue?"),
			FText::FromString(pTex2D->GetName()))) )
	{
		return EReimportResult::Failed;
	}

	UE_LOGF(LogEditorFactories, Log, "Performing atomic reimport of [%ls]",*ResolvedSourceFilePath);

	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*ResolvedSourceFilePath) == INDEX_NONE)
	{
		UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: source file cannot be found.");
		return EReimportResult::Failed;
	}

	// We use this reimport factory to skip the object creation process
	// which obliterates all of the properties of the texture.
	// Also preset the factory with the settings of the current texture.
	// These will be used during the import and compression process.        
	CompressionSettings   = pTex->CompressionSettings;
	NoCompression         = pTex->CompressionNone;
	NoAlpha               = pTex->CompressionNoAlpha;
	bDeferCompression     = pTex->DeferCompression;
	MipGenSettings		  = pTex->MipGenSettings;

	float Brightness = 0.f;
	float TextureMultiplier = 1.f;

	UTextureLightProfile* pTexLightProfile = Cast<UTextureLightProfile>(Obj);
	if ( pTexLightProfile )
	{
		Brightness = pTexLightProfile->Brightness;
		TextureMultiplier = pTexLightProfile->TextureMultiplier;
	}

	// Suppress the import overwrite dialog because we know that for explicitly re-importing we want to preserve existing settings
	UTextureFactory::SuppressImportOverwriteDialog();

	bool OutCanceled = false;

	if (ImportObject(pTex->GetClass(), pTex->GetOuter(), *pTex->GetName(), RF_Public | RF_Standalone, ResolvedSourceFilePath, nullptr, OutCanceled) != nullptr)
	{
		if ( pTexLightProfile )
		{
			// We don't update the Brightness and TextureMultiplier during reimport.
			// The reason is that the IESLoader has changed and calculates these values differently.
			// Since existing lights have been calibrated, we don't want to screw with those values.
			pTexLightProfile->Brightness = Brightness;
			pTexLightProfile->TextureMultiplier = TextureMultiplier;
		}

		UE_LOGF(LogEditorFactories, Log, "-- imported successfully" );

		pTex->AssetImportData->Update(ResolvedSourceFilePath);

		// Try to find the outer package so we can dirty it up
		if (pTex->GetOuter())
		{
			pTex->GetOuter()->MarkPackageDirty();
		}
		else
		{
			pTex->MarkPackageDirty();
		}
	}
	else if (OutCanceled)
	{
		UE_LOGF(LogEditorFactories, Warning, "-- import canceled");
		return EReimportResult::Cancelled;
	}
	else
	{
		UE_LOGF(LogEditorFactories, Warning, "-- import failed");
		return EReimportResult::Failed;
	}
	
	return EReimportResult::Succeeded;
}

int32 UReimportTextureFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportTextureFactory::IsAutomatedImport() const
{
	return Super::IsAutomatedImport() || IsAutomatedReimport();
}


/*-----------------------------------------------------------------------------
UReimportFbxStaticMeshFactory.
-----------------------------------------------------------------------------*/
UReimportFbxStaticMeshFactory::UReimportFbxStaticMeshFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UStaticMesh::StaticClass();
	Formats.Add(TEXT("fbx;FBX static meshes"));

	bCreateNew = false;
	bText = false;
	bShowOption = true;

	// Required to allow other StaticMesh re importers to do their CanReimport checks first, and if they fail the FBX will catch it
	ImportPriority = DefaultImportPriority - 1;
}

bool UReimportFbxStaticMeshFactory::FactoryCanImport(const FString& Filename)
{
	// Return false, we are a reimport only factory
	return false;
}

bool UReimportFbxStaticMeshFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{
	if (!UE::EditorFactories::Private::CanImportClass(UStaticMesh::StaticClass()))
	{
		return false;
	}

	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if(Mesh)
	{
		if ( Mesh->GetAssetImportData() )
		{
			UFbxAssetImportData *FbxAssetImportData = Cast<UFbxAssetImportData>(Mesh->GetAssetImportData());
			if (FbxAssetImportData != nullptr && FbxAssetImportData->bImportAsScene)
			{
				//This mesh was import with a scene import, we cannot reimport it
				return false;
			}

			FString FileExtension = FPaths::GetExtension(Mesh->GetAssetImportData()->GetFirstFilename());
			const bool bIsValidFile = FileExtension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase) || FileExtension.Equals("obj", ESearchCase::IgnoreCase);
			if (!bIsValidFile)
			{
				return false;
			}
			OutFilenames.Add(Mesh->GetAssetImportData()->GetFirstFilename());
		}
		else
		{
			OutFilenames.Add(TEXT(""));
		}
		return true;
	}
	return false;
}

void UReimportFbxStaticMeshFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if(Mesh && ensure(NewReimportPaths.Num() == 1))
	{
		Mesh->Modify();
		UFbxStaticMeshImportData* ImportData = UFbxStaticMeshImportData::GetImportDataForStaticMesh(Mesh, ImportUI->StaticMeshImportData);

		check( ImportData != nullptr );
		ImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportFbxStaticMeshFactory::Reimport( UObject* Obj )
{
	if (!UE::EditorFactories::Private::CanImportClass(UStaticMesh::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	UStaticMesh* Mesh = Cast<UStaticMesh>(Obj);
	if( !Mesh )
	{
		return EReimportResult::Failed;
	}
	
	UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();
	
	//Pop the message log in case of error
	UnFbx::FFbxLoggerSetter Logger(FFbxImporter, true);

	//Clean up the options
	UnFbx::FBXImportOptions::ResetOptions(ImportOptions);

	UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(Mesh->GetAssetImportData());
	
	UFbxImportUI* ReimportUI;
	UFbxImportUI* OverrideImportUI = AssetImportTask ? Cast<UFbxImportUI>(AssetImportTask->Options) : nullptr;
	if (OverrideImportUI)
	{
		ReimportUI = OverrideImportUI;
	}
	else
	{
		if (AssetImportTask && AssetImportTask->Options)
		{
			UE_LOGF(LogFbx, Display, "The options set in the Asset Import Task are not of type UFbxImportUI and will be ignored");
		}
		ReimportUI = NewObject<UFbxImportUI>();
	}

	ReimportUI->MeshTypeToImport = FBXIT_StaticMesh;
	ReimportUI->StaticMeshImportData->bCombineMeshes = true;

	if (!ImportUI)
	{
		ImportUI = NewObject<UFbxImportUI>(this, NAME_None, RF_Public);
	}
	//Prevent any UI for automation, unattended and commandlet
	const bool IsUnattended = IsAutomatedImport() || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
	const bool ShowImportDialogAtReimport = (GetDefault<UEditorPerProjectUserSettings>()->bShowImportDialogAtReimport || bForceShowDialog) && !IsUnattended;

	if (ImportData == nullptr)
	{
		// An existing import data object was not found, make one here and show the options dialog
		ImportData = UFbxStaticMeshImportData::GetImportDataForStaticMesh(Mesh, ImportUI->StaticMeshImportData);
		check( ImportData != nullptr );

		Mesh->SetAssetImportData(ImportData);
	}

	//Get the re-import filename
	const FString Filename = ImportData->GetFirstFilename();
	const FString FileExtension = FPaths::GetExtension(Filename);
	const bool bIsValidFile = FileExtension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase) || FileExtension.Equals("obj", ESearchCase::IgnoreCase);
	if (!bIsValidFile)
	{
		return EReimportResult::Failed;
	}
	if (!(Filename.Len()))
	{
		// Since this is a new system most static meshes don't have paths, so logging has been commented out
		//UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: static mesh resource does not have path stored.");
		return EReimportResult::Failed;
	}
	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
	{
		UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: source file cannot be found.");
		return EReimportResult::Failed;
	}
	CurrentFilename = Filename;

	check( ImportData != nullptr );
	if( !ShowImportDialogAtReimport)
	{
		// Import data already exists, apply it to the fbx import options
		ReimportUI->StaticMeshImportData = ImportData;
		ApplyImportUIToImportOptions(ReimportUI, *ImportOptions);
	}
	else
	{
		ReimportUI->bIsReimport = true;
		ReimportUI->ReimportMesh = Mesh;

		//Make sure the outer is the ImportUI, because there is some logic in the meta data needing this outer
		check( ImportData != nullptr );
		UObject* OriginalOuter = ImportData->GetOuter();
		ReimportUI->StaticMeshImportData = ImportData;
		if ( OriginalOuter)
		{
			ReimportUI->StaticMeshImportData->Rename(nullptr, ReimportUI);
		}
		
		//Force the bAutoGenerateCollision to false if the Mesh Customize collision is true
		bool bOldAutoGenerateCollision = ReimportUI->StaticMeshImportData->bAutoGenerateCollision;
		if (Mesh->GetCustomizedCollision())
		{
			ReimportUI->StaticMeshImportData->bAutoGenerateCollision = false;
		}
		
		bool bImportOperationCanceled = false;
		bool bForceImportType = true;
		bool bShowOptionDialog = true;
		bool bOutImportAll = false;
		bool bIsObjFormat = false;
		bool bIsAutomated = false;

		GetImportOptions( FFbxImporter, ReimportUI, bShowOptionDialog, bIsAutomated, Obj->GetPathName(), bOperationCanceled, bOutImportAll, bIsObjFormat, Filename, bForceImportType, FBXIT_StaticMesh);
		
		//Put back the original bAutoGenerateCollision settings since the user cancel the re-import
		check( ReimportUI->StaticMeshImportData != nullptr );
		if ( bOperationCanceled && Mesh->GetCustomizedCollision())
		{
			ReimportUI->StaticMeshImportData->bAutoGenerateCollision = bOldAutoGenerateCollision;
		}

		//Put back the original SM outer
		if ( OriginalOuter)
		{
			ReimportUI->StaticMeshImportData->Rename(nullptr, OriginalOuter);
		}
	}
	ImportOptions->bCanShowDialog = !IsUnattended;
	//We do not touch bAutoComputeLodDistances when we re-import, setting it to true will make sure we do not change anything.
	//We set the LODDistance only when the value is false.
	ImportOptions->bAutoComputeLodDistances = true;
	ImportOptions->LodNumber = 0;
	ImportOptions->MinimumLodNumber = 0;
	//Make sure the LODGroup do not change when re-importing a mesh
	ImportOptions->StaticMeshLODGroup = Mesh->GetLODGroup();

	if( !bOperationCanceled && ensure(ImportData) )
	{
		UE_LOGF(LogEditorFactories, Log, "Performing atomic reimport of [%ls]", *Filename);

		bool bImportSucceed = true;
		if ( FFbxImporter->ImportFromFile( *Filename, FPaths::GetExtension( Filename ), true ) )
		{
			FFbxImporter->ApplyTransformSettingsToFbxNode(FFbxImporter->Scene->GetRootNode(), ImportData);

			// preserve the user data by doing a copy
			const TArray<UAssetUserData*>* UserData = Mesh->GetAssetUserDataArray();
			TMap<UAssetUserData*, bool> UserDataCopy;
			if (UserData)
			{
				for (int32 Idx = 0; Idx < UserData->Num(); Idx++)
				{
					if ((*UserData)[Idx] != nullptr)
					{
						UAssetUserData* DupObject = (UAssetUserData*)StaticDuplicateObject((*UserData)[Idx], GetTransientPackage());
						bool bAddDupToRoot = !(DupObject->IsRooted());
						if (bAddDupToRoot)
						{
							DupObject->AddToRoot();
						}
						UserDataCopy.Add(DupObject, bAddDupToRoot);
					}
				}
			}

			// preserve settings in navcollision subobject
			UNavCollisionBase* NavCollision = Mesh->GetNavCollision() ? 
				(UNavCollisionBase*)StaticDuplicateObject(Mesh->GetNavCollision(), GetTransientPackage()) :
				nullptr;

			bool bAddedNavCollisionDupToRoot = false;
			if (NavCollision && !NavCollision->IsRooted())
			{
				bAddedNavCollisionDupToRoot = true;
				NavCollision->AddToRoot();
			}

			// preserve extended bound settings
			const FVector PositiveBoundsExtension = Mesh->GetPositiveBoundsExtension();
			const FVector NegativeBoundsExtension = Mesh->GetNegativeBoundsExtension();

			if (FFbxImporter->ReimportStaticMesh(Mesh, ImportData))
			{
				UE_LOGF(LogEditorFactories, Log, "-- imported successfully" );

				// Copy user data to newly created mesh
				for (auto Kvp : UserDataCopy)
				{
					UAssetUserData* UserDataObject = Kvp.Key;
					if (Kvp.Value)
					{
						//if the duplicated temporary UObject was add to root, we must remove it from the root
						UserDataObject->RemoveFromRoot();
					}
					UserDataObject->Rename(nullptr, Mesh, REN_DontCreateRedirectors | REN_DoNotDirty);
					Mesh->AddAssetUserData(UserDataObject);
				}

				if (NavCollision)
				{
					if (bAddedNavCollisionDupToRoot)
					{
						//if the duplicated temporary UObject was add to root, we must remove it from the root
						NavCollision->RemoveFromRoot();
					}
					Mesh->SetNavCollision(NavCollision);
					NavCollision->Rename(NULL, Mesh, REN_DontCreateRedirectors | REN_DoNotDirty);
				}

				// Restore bounds extension settings
				Mesh->SetPositiveBoundsExtension(PositiveBoundsExtension);
				Mesh->SetNegativeBoundsExtension(NegativeBoundsExtension);

				Mesh->GetAssetImportData()->Update(Filename);

				// Try to find the outer package so we can dirty it up
				if (Mesh->GetOuter())
				{
					Mesh->GetOuter()->MarkPackageDirty();
				}
				else
				{
					Mesh->MarkPackageDirty();
				}

				FFbxImporter->ImportStaticMeshGlobalSockets(Mesh);

				if (Mesh->IsHiResMeshDescriptionValid())
				{
					FMeshDescription* HiresMeshDescription = Mesh->GetHiResMeshDescription();
					FMeshDescription* Lod0MeshDescription = Mesh->GetMeshDescription(0);
					if (HiresMeshDescription && Lod0MeshDescription)
					{
						Mesh->ModifyHiResMeshDescription();
						FString MaterialNameConflictMsg = TEXT("[Asset ") + Mesh->GetPathName() + TEXT("] Nanite hi - res import have some material name that differ from the LOD 0 material name.Your nanite hi - res should use the same material names the LOD 0 use to ensure we can remap the section in the same order.");
						FString MaterialCountConflictMsg = TEXT("[Asset ") + Mesh->GetPathName() + TEXT("] Nanite hi-res import dont have the same material count then LOD 0. Your nanite hi-res should have equal number of material.");
						FStaticMeshOperations::ReorderMeshDescriptionPolygonGroups(*Lod0MeshDescription, *HiresMeshDescription, MaterialNameConflictMsg, MaterialCountConflictMsg);
						Mesh->CommitHiResMeshDescription();
					}
				}
			}
			else
			{
				UE_LOGF(LogEditorFactories, Warning, "-- import failed" );
				bImportSucceed = false;
			}
		}
		else
		{
			UE_LOGF(LogEditorFactories, Warning, "-- import failed" );
			bImportSucceed = false;
		}

		FFbxImporter->ReleaseScene(); 

		return bImportSucceed ? EReimportResult::Succeeded : EReimportResult::Failed;
	}
	else
	{
		FFbxImporter->ReleaseScene();
		return EReimportResult::Cancelled;
	}
}

int32 UReimportFbxStaticMeshFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportFbxStaticMeshFactory::IsAutomatedImport() const
{
	return Super::IsAutomatedImport() || IsAutomatedReimport();
}


/*-----------------------------------------------------------------------------
UReimportFbxSkeletalMeshFactory
-----------------------------------------------------------------------------*/ 
UReimportFbxSkeletalMeshFactory::UReimportFbxSkeletalMeshFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = USkeletalMesh::StaticClass();
	Formats.Add(TEXT("fbx;FBX skeletal meshes"));

	bCreateNew = false;
	bText = false;
	bShowOption = true;
}

bool UReimportFbxSkeletalMeshFactory::FactoryCanImport(const FString& Filename)
{
	// Return false, we are a reimport only factory
	return false;
}

bool UReimportFbxSkeletalMeshFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{
	if (!UE::EditorFactories::Private::CanImportClass(USkeletalMesh::StaticClass()))
	{
		return false;
	}

	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj);
	TArray<FString> FactoryExtensions;
	GetSupportedFileExtensions(FactoryExtensions);

	if (SkeletalMesh && !SkeletalMesh->HasCustomActorReimportFactory())
	{
		if (UAssetImportData* AssetImportData = SkeletalMesh->GetAssetImportData())
		{
			UFbxAssetImportData *FbxAssetImportData = Cast<UFbxAssetImportData>(AssetImportData);
			if (FbxAssetImportData != nullptr && FbxAssetImportData->bImportAsScene)
			{
				//This skeletal mesh was import with a scene import, we cannot reimport it here
				return false;
			}
			else if (FPaths::GetExtension(AssetImportData->GetFirstFilename()) != TEXT("fbx")
				  && FPaths::GetExtension(AssetImportData->GetFirstFilename()) != TEXT("obj"))
			{
				//Fbx factory only support fbx and obj files
				return false;
			}
			else if (!PreferredReimportPath.IsEmpty() && !FactoryExtensions.Contains(FPaths::GetExtension(PreferredReimportPath)))
			{
				// Unsupported extensions shouldnt be considered for reimport
				return false;
			}
			AssetImportData->ExtractFilenames(OutFilenames);
		}
		else
		{
			OutFilenames.Add(TEXT(""));
		}
		return true;
	}
	return false;
}

void UReimportFbxSkeletalMeshFactory::SetReimportPaths( UObject* Obj, const FString& NewReimportPath, const int32 SourceFileIndex )
{	
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Obj);
	if(SkeletalMesh)
	{
		SkeletalMesh->Modify();
		UFbxSkeletalMeshImportData* ImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, ImportUI->SkeletalMeshImportData);
		int32 RealSourceFileIndex = SourceFileIndex == INDEX_NONE ? 0 : SourceFileIndex;
		if (RealSourceFileIndex < ImportData->GetSourceFileCount())
		{
			ImportData->UpdateFilenameOnly(NewReimportPath, SourceFileIndex);
		}
		else
		{
			//Create a source file entry, this case happen when user import a specific content for the first time
			FString SourceIndexLabel = USkeletalMesh::GetSourceFileLabelFromIndex(RealSourceFileIndex).ToString();
			ImportData->AddFileName(NewReimportPath, RealSourceFileIndex, SourceIndexLabel);
		}
	}
}

EReimportResult::Type UReimportFbxSkeletalMeshFactory::Reimport( UObject* Obj, int32 SourceFileIndex)
{
	if (!UE::EditorFactories::Private::CanImportClass(USkeletalMesh::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	// Only handle valid skeletal meshes
	if( !Obj || !Obj->IsA( USkeletalMesh::StaticClass() ))
	{
		return EReimportResult::Failed;
	}

	USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>( Obj );

	if(SkeletalMesh->HasCustomActorReimportFactory())
	{
		return EReimportResult::Failed;
	}

	UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();

	//Pop the message log in case of error
	UnFbx::FFbxLoggerSetter Logger(FFbxImporter, true);

	//Clean up the options
	UnFbx::FBXImportOptions::ResetOptions(ImportOptions);

	UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(SkeletalMesh->GetAssetImportData());
	
	// Prepare the import options
	UFbxImportUI* ReimportUI;
	UFbxImportUI* OverrideImportUI = AssetImportTask ? Cast<UFbxImportUI>(AssetImportTask->Options) : nullptr;
	if (OverrideImportUI)
	{
		ReimportUI = OverrideImportUI;
	}
	else
	{
		if (AssetImportTask && AssetImportTask->Options)
		{
			UE_LOGF(LogFbx, Display, "The options set in the Asset Import Task are not of type UFbxImportUI and will be ignored");
		}
		ReimportUI = NewObject<UFbxImportUI>();
	}

	ReimportUI->MeshTypeToImport = FBXIT_SkeletalMesh;
	ReimportUI->Skeleton = SkeletalMesh->GetSkeleton();
	ReimportUI->bCreatePhysicsAsset = false;
	ReimportUI->PhysicsAsset = SkeletalMesh->GetPhysicsAsset();
	ReimportUI->bImportAnimations = false;
	ReimportUI->OverrideAnimationName = TEXT("");
	ReimportUI->bImportRigidMesh = false;

	if (!ImportUI)
	{
		ImportUI = NewObject<UFbxImportUI>(this, NAME_None, RF_Public);
	}

	bool bSuccess = false;
	//Prevent any UI for automation, unattended and commandlet
	const bool IsUnattended = IsAutomatedImport() || FApp::IsUnattended() || IsRunningCommandlet() || GIsRunningUnattendedScript;
	const bool ShowImportDialogAtReimport = (GetDefault<UEditorPerProjectUserSettings>()->bShowImportDialogAtReimport || bForceShowDialog) && !IsUnattended;

	if (ImportData == nullptr)
	{
		// An existing import data object was not found, make one here and show the options dialog
		ImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, ImportUI->SkeletalMeshImportData);
		SkeletalMesh->SetAssetImportData(ImportData);
	}
	check(ImportData != nullptr);

	auto GetSourceFileName = [](UFbxSkeletalMeshImportData* ImportDataPtr, FString& OutFilename, bool bUnattended)->bool
	{
		if (ImportDataPtr == nullptr)
		{
			return false;
		}
		EFBXImportContentType ContentType = ImportDataPtr->ImportContentType;
		TArray<FString> AbsoluteFilenames;
		ImportDataPtr->ExtractFilenames(AbsoluteFilenames);
		
		auto InternalGetSourceFileName = [&ImportDataPtr, &AbsoluteFilenames, &bUnattended, &OutFilename](const int32 SourceIndex, const FText& SourceLabel)->bool
		{
			if (AbsoluteFilenames.Num() > SourceIndex)
			{
				OutFilename = AbsoluteFilenames[SourceIndex];
			}
			else if (!bUnattended)
			{
				GetReimportPathFromUser(SourceLabel, AbsoluteFilenames);
				if (AbsoluteFilenames.Num() < 1)
				{
					return false;
				}
				OutFilename = AbsoluteFilenames[0];
			}
			//Make sure the source file data is up to date
			if (SourceIndex == 0)
			{
				//When we re-import the All content we just update the 
				ImportDataPtr->AddFileName(OutFilename, SourceIndex, SourceLabel.ToString());
			}
			else
			{
				//Refresh the absolute filenames
				AbsoluteFilenames.Reset();
				ImportDataPtr->ExtractFilenames(AbsoluteFilenames);
				//Set both geo and skinning filepath. Reuse existing file path if possible. Use the first filename(geo and skin) if it has to be create.
				FString FilenameToAdd = SourceIndex == 1 ? OutFilename : AbsoluteFilenames.IsValidIndex(1) ? AbsoluteFilenames[1] : AbsoluteFilenames[0];
				ImportDataPtr->AddFileName(FilenameToAdd, 1, NSSkeletalMeshSourceFileLabels::GeometryText().ToString());
				FilenameToAdd = SourceIndex == 2 ? OutFilename : AbsoluteFilenames.IsValidIndex(2) ? AbsoluteFilenames[2] : AbsoluteFilenames[0];
				ImportDataPtr->AddFileName(FilenameToAdd, 2, NSSkeletalMeshSourceFileLabels::SkinningText().ToString());
			}
			return true;
		};

		switch (ContentType)
		{
			case FBXICT_All:
			{
				if (!InternalGetSourceFileName(0, NSSkeletalMeshSourceFileLabels::GeoAndSkinningText()))
				{
					return false;
				}
			}
			break;
			case FBXICT_Geometry:
			{
				if (!InternalGetSourceFileName(1, NSSkeletalMeshSourceFileLabels::GeometryText()))
				{
					return false;
				}
			}
			break;
			case FBXICT_SkinningWeights:
			{
				if (!InternalGetSourceFileName(2, NSSkeletalMeshSourceFileLabels::SkinningText()))
				{
					return false;
				}
			}
			break;
			default:
			{
				if (!InternalGetSourceFileName(0, NSSkeletalMeshSourceFileLabels::GeoAndSkinningText()))
				{
					return false;
				}
			}
		}
		return IFileManager::Get().FileSize(*OutFilename) != INDEX_NONE;
	};

	FString Filename = ImportData->GetFirstFilename();

	ReimportUI->SkeletalMeshImportData = ImportData;
	const FSkeletalMeshModel* SkeletalMeshModel = SkeletalMesh->GetImportedModel();

	const bool bUserDefinedGeometryExists = SkeletalMesh->HasMeshDescription(0);
	
	//Manage the content type from the source file index
	ReimportUI->bAllowContentTypeImport = SkeletalMeshModel && SkeletalMeshModel->LODModels.Num() > 0 && bUserDefinedGeometryExists;
	if (!ReimportUI->bAllowContentTypeImport)
	{
		//No content type allow reimport All (legacy)
		ImportData->ImportContentType = EFBXImportContentType::FBXICT_All;
	}
	else if (SourceFileIndex != INDEX_NONE)
	{
		//Reimport a specific source file index
		TArray<FString> SourceFilenames;
		ImportData->ExtractFilenames(SourceFilenames);
		if (SourceFilenames.IsValidIndex(SourceFileIndex))
		{
			ImportData->ImportContentType = SourceFileIndex == 0 ? EFBXImportContentType::FBXICT_All : SourceFileIndex == 1 ? EFBXImportContentType::FBXICT_Geometry : EFBXImportContentType::FBXICT_SkinningWeights;
			Filename = SourceFilenames[SourceFileIndex];
		}
	}
	else
	{
		//No source index is provided. Reimport the last imported content.
		int32 LastSourceFileIndex = ImportData->LastImportContentType == EFBXImportContentType::FBXICT_All ? 0 : ImportData->LastImportContentType == EFBXImportContentType::FBXICT_Geometry ? 1 : 2;
		TArray<FString> SourceFilenames;
		ImportData->ExtractFilenames(SourceFilenames);
		if (SourceFilenames.IsValidIndex(LastSourceFileIndex))
		{
			ImportData->ImportContentType = ImportData->LastImportContentType;
			Filename = SourceFilenames[LastSourceFileIndex];
		}
		else
		{
			ImportData->ImportContentType = EFBXImportContentType::FBXICT_All;
		}
	}


	if( !ShowImportDialogAtReimport)
	{
		// Import data already exists, apply it to the fbx import options
		//Some options not supported with skeletal mesh
		ImportData->bBakePivotInVertex = false;
		ImportData->bTransformVertexToAbsolute = true;

		if (!GetSourceFileName(ImportData, Filename, true))
		{
			UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: source file cannot be found.");
			return EReimportResult::Failed;
		}

		ApplyImportUIToImportOptions(ReimportUI, *ImportOptions);
	}
	else
	{
		ReimportUI->bIsReimport = true;
		ReimportUI->ReimportMesh = Obj;

		bool bImportOperationCanceled = false;
		bool bShowOptionDialog = true;
		bool bForceImportType = true;
		bool bOutImportAll = false;
		bool bIsObjFormat = false;
		bool bIsAutomated = false;
		// @hack to make sure skeleton is set before opening the dialog
		ImportOptions->SkeletonForAnimation = SkeletalMesh->GetSkeleton();
		ImportOptions->bCreatePhysicsAsset = false;
		ImportOptions->PhysicsAsset = SkeletalMesh->GetPhysicsAsset();
		
		EFBXImportContentType BeforeUIContentType = ReimportUI->SkeletalMeshImportData->ImportContentType;
		ImportOptions = GetImportOptions( FFbxImporter, ReimportUI, bShowOptionDialog, bIsAutomated, Obj->GetPathName(), bOperationCanceled, bOutImportAll, bIsObjFormat, Filename, bForceImportType, FBXIT_SkeletalMesh);
		
		//If the import type has change from the UI, assign the filename to the file source index
		if (bShowOptionDialog && ReimportUI->SkeletalMeshImportData->ImportContentType != BeforeUIContentType && ReimportUI->SkeletalMeshImportData->ImportContentType != EFBXImportContentType::FBXICT_All)
		{
			TArray<FString> ExtractedFilenames;
			SkeletalMesh->GetAssetImportData()->ExtractFilenames(ExtractedFilenames);
			//By default add the original file
			if (!ExtractedFilenames.IsValidIndex(1))
			{
				SkeletalMesh->GetAssetImportData()->AddFileName(ExtractedFilenames[0], 1, NSSkeletalMeshSourceFileLabels::GeometryText().ToString());
			}
			if (!ExtractedFilenames.IsValidIndex(2))
			{
				SkeletalMesh->GetAssetImportData()->AddFileName(ExtractedFilenames[0], 2, NSSkeletalMeshSourceFileLabels::SkinningText().ToString());
			}
		}

		if (!GetSourceFileName(ImportData, Filename, false))
		{
			UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: source file cannot be found.");
			return EReimportResult::Failed;
		}
	}

	//Set the build option to reflect the user choice in the dialog
	UFbxSkeletalMeshImportData* SKImportData = UFbxSkeletalMeshImportData::GetImportDataForSkeletalMesh(SkeletalMesh, ReimportUI->SkeletalMeshImportData);
	if (SKImportData)
	{
		FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(0);
		if (LODInfo && SkeletalMesh->GetImportedModel() && SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(0))
		{
			const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[0];
			
			if (bUserDefinedGeometryExists)
			{
				//Set the build settings
				LODInfo->BuildSettings.bComputeWeightedNormals = SKImportData->bComputeWeightedNormals;
				LODInfo->BuildSettings.bRecomputeNormals = SKImportData->NormalImportMethod == EFBXNormalImportMethod::FBXNIM_ComputeNormals;
				LODInfo->BuildSettings.bRecomputeTangents = SKImportData->NormalImportMethod != EFBXNormalImportMethod::FBXNIM_ImportNormalsAndTangents;
				LODInfo->BuildSettings.bUseMikkTSpace = SKImportData->NormalGenerationMethod == EFBXNormalGenerationMethod::MikkTSpace;
				LODInfo->BuildSettings.ThresholdPosition = SKImportData->ThresholdPosition;
				LODInfo->BuildSettings.ThresholdTangentNormal = SKImportData->ThresholdTangentNormal;
				LODInfo->BuildSettings.ThresholdUV = SKImportData->ThresholdUV;
				LODInfo->BuildSettings.MorphThresholdPosition = SKImportData->MorphThresholdPosition;
			}
		}
	}

	UE_LOGF(LogEditorFactories, Log, "Performing atomic reimport of [%ls]", *Filename);
	CurrentFilename = Filename;

	if (!bOperationCanceled)
	{
		// Replacing the skeleton requires setting up a fresh editor
		const bool bReplacingSkeleton = ImportOptions->SkeletonForAnimation != SkeletalMesh->GetSkeleton();
		bool bShouldReopenAsset = false;
		if (bReplacingSkeleton)
		{
			if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
			{
				bShouldReopenAsset = AssetEditorSubsystem->CloseAllEditorsForAsset(SkeletalMesh) != 0;
			}
		}
		
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinWeightPreview(SkeletalMesh);
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

			ImportOptions->bCanShowDialog = !IsUnattended;

			if (ImportOptions->bImportAsSkeletalSkinning)
			{
				ImportOptions->bImportMaterials = false;
				ImportOptions->bImportTextures = false;
				ImportOptions->bImportLOD = false;
				ImportOptions->bImportSkeletalMeshLODs = false;
				ImportOptions->bImportAnimations = false;
				ImportOptions->bImportMorph = false;
				ImportOptions->VertexColorImportOption = EVertexColorImportOption::Ignore;
			}
			else if (ImportOptions->bImportAsSkeletalGeometry)
			{
				ImportOptions->bImportAnimations = false;
				ImportOptions->bUpdateSkeletonReferencePose = false;
			}

			//Save all skinweight profile infos (need a copy, because they will be removed)
			const TArray<FSkinWeightProfileInfo> ExistingSkinWeightProfileInfos = SkeletalMesh->GetSkinWeightProfiles();

			if (FFbxImporter->ImportFromFile(*Filename, FPaths::GetExtension(Filename), true))
			{
				if (FFbxImporter->ReimportSkeletalMesh(SkeletalMesh, ImportData))
				{
					UE_LOGF(LogEditorFactories, Log, "-- imported successfully");

					// Try to find the outer package so we can dirty it up
					if (SkeletalMesh->GetOuter())
					{
						SkeletalMesh->GetOuter()->MarkPackageDirty();
					}
					else
					{
						SkeletalMesh->MarkPackageDirty();
					}

					bSuccess = true;
				}
				else
				{
					UE_LOGF(LogEditorFactories, Warning, "-- import failed");
				}
			}
			else
			{
				UE_LOGF(LogEditorFactories, Warning, "-- import failed");
			}
			FFbxImporter->ReleaseScene();

			CleanUp();

			if (bSuccess && ExistingSkinWeightProfileInfos.Num() > 0)
			{
				//Restore skin weight profile infos, then reimport affected LODs
				TArray<FSkinWeightProfileInfo>&SkinWeightsProfile = SkeletalMesh->GetSkinWeightProfiles();
				SkinWeightsProfile = ExistingSkinWeightProfileInfos;
				FSkinWeightsUtilities::ReimportAlternateSkinWeight(SkeletalMesh, 0);
			}

			// Reopen asset if it was open previously
			if (bShouldReopenAsset)
			{
				if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					AssetEditorSubsystem->OpenEditorForAsset(SkeletalMesh);
				}
			}

			// Reimporting can have dangerous effects if the mesh is still in the transaction buffer.  Reset the transaction buffer if this is the case
			if (!IsRunningCommandlet() && GEditor->IsObjectInTransactionBuffer(SkeletalMesh))
			{
				GEditor->ResetTransaction(LOCTEXT("ReimportSkeletalMeshTransactionReset", "Reimporting a skeletal mesh which was in the undo buffer"));
			}

			return bSuccess ? EReimportResult::Succeeded : EReimportResult::Failed;
		}
	}
	else
	{
		FFbxImporter->ReleaseScene();
		return EReimportResult::Cancelled;
	}
}

int32 UReimportFbxSkeletalMeshFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportFbxSkeletalMeshFactory::IsAutomatedImport() const
{
	return Super::IsAutomatedImport() || IsAutomatedReimport();
}

/*-----------------------------------------------------------------------------
UReimportFbxAnimSequenceFactory
-----------------------------------------------------------------------------*/ 
USkeleton* ChooseSkeleton() 
{
	TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
		.Title(LOCTEXT("ChooseSkeletonWindowTitle", "Choose Skeleton"))
		.ClientSize(FVector2D(500, 600));

	TSharedRef<SSkeletonSelectorWindow> SkeletonSelectorWindow = SNew(SSkeletonSelectorWindow) .WidgetWindow(WidgetWindow);

	WidgetWindow->SetContent(SkeletonSelectorWindow);

	GEditor->EditorAddModalWindow(WidgetWindow);
	return SkeletonSelectorWindow->GetSelectedSkeleton();
}

UReimportFbxAnimSequenceFactory::UReimportFbxAnimSequenceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UAnimSequence::StaticClass();
	Formats.Empty(1);
	Formats.Add(TEXT("fbx;FBX animation"));

	bCreateNew = false;
	bText = false;
	bShowOption = true;
}

bool UReimportFbxAnimSequenceFactory::FactoryCanImport(const FString& Filename)
{
	// Return false, we are a reimport only factory
	return false;
}

bool UReimportFbxAnimSequenceFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{
	if (!UE::EditorFactories::Private::CanImportClass(UAnimSequence::StaticClass()))
	{
		return false;
	}
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Obj);
	if(AnimSequence)
	{
		if (AnimSequence->AssetImportData)
		{
			AnimSequence->AssetImportData->ExtractFilenames(OutFilenames);

			UFbxAssetImportData *FbxAssetImportData = Cast<UFbxAssetImportData>(AnimSequence->AssetImportData);
			if (FbxAssetImportData != nullptr && FbxAssetImportData->bImportAsScene)
			{
				//This mesh was import with a scene import, we cannot reimport it
				return false;
			}
			else if (FPaths::GetExtension(AnimSequence->AssetImportData->GetFirstFilename()) != TEXT("fbx")
				&& FPaths::GetExtension(AnimSequence->AssetImportData->GetFirstFilename()) != TEXT("obj"))
			{
				return false;
			}
		}
		else
		{
			OutFilenames.Add(TEXT(""));
		}
		return true;
	}
	return false;
}

void UReimportFbxAnimSequenceFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Obj);
	if(AnimSequence && ensure(NewReimportPaths.Num() == 1))
	{
		UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(AnimSequence, ImportUI->AnimSequenceImportData);

		ImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportFbxAnimSequenceFactory::Reimport( UObject* Obj )
{
	if (!UE::EditorFactories::Private::CanImportClass(UAnimSequence::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	// Only handle valid skeletal meshes
	if( !Obj || !Obj->IsA( UAnimSequence::StaticClass() ) )
	{
		return EReimportResult::Failed;
	}

	UAnimSequence* AnimSequence = Cast<UAnimSequence>( Obj );
	UFbxAnimSequenceImportData* ImportData = UFbxAnimSequenceImportData::GetImportDataForAnimSequence(AnimSequence, ImportUI->AnimSequenceImportData);
	if ( !ensure(ImportData) )
	{
		return EReimportResult::Failed;
	}

	const FString Filename = ImportData->GetFirstFilename();
	const FString FileExtension = FPaths::GetExtension(Filename);
	const bool bIsNotFBXFile = ( FileExtension.Len() > 0  && FCString::Stricmp( *FileExtension, TEXT("FBX") ) != 0 );

	// Only handle FBX files
	if ( bIsNotFBXFile )
	{
		return EReimportResult::Failed;
	}

	UE_LOGF(LogEditorFactories, Log, "Performing atomic reimport of [%ls]", *Filename);

	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*Filename) == INDEX_NONE)
	{
		UE_LOGF(LogEditorFactories, Warning, "-- cannot reimport: source file cannot be found." );
		return EReimportResult::Failed;
	}

	UnFbx::FFbxImporter* Importer = UnFbx::FFbxImporter::GetInstance();

	//Pop the message log in case of error
	UnFbx::FFbxLoggerSetter Logger(Importer, false);

	CurrentFilename = Filename;

	USkeleton* Skeleton = AnimSequence->GetSkeleton();
	if (!Skeleton)
	{
		// if it does not exist, ask for one
		Skeleton = !IsAutomatedImport() ? ChooseSkeleton() : nullptr;
		if (!Skeleton)
		{
			// If skeleton wasn't found or the user canceled out of the dialog, we cannot proceed, but this reimport factory 
			// has still technically "handled" the reimport, so return true instead of false
			UE_LOGF(LogEditorFactories, Warning, "-- import failed" );
			Importer->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_CouldNotFindSkeleton", "Cannot re-import animation with no skeleton.\nImport failed.")), FFbxErrors::SkeletalMesh_NoBoneFound);
			Importer->ReleaseScene();
			return EReimportResult::Succeeded;
		}
		//Set the selected skeleton in the anim sequence
		AnimSequence->SetSkeleton(Skeleton);
	}

	UFbxImportUI* OverrideImportUI = AssetImportTask ? Cast<UFbxImportUI>(AssetImportTask->Options) : nullptr;
	if (!OverrideImportUI && AssetImportTask && AssetImportTask->Options)
	{
		UE_LOGF(LogFbx, Display, "The options set in the Asset Import Task are not of type UFbxImportUI and will be ignored");
	}

	bool bOutImportAll = false;
	if ( UEditorEngine::ReimportFbxAnimation(Skeleton, AnimSequence, ImportData, *Filename, bOutImportAll, (bShowOption || bForceShowDialog) && !IsAutomatedImport(), OverrideImportUI) )
	{
		if (bOutImportAll)
		{
			// If the user chose to import all, we don't show the dialog again and use the same settings for each object until importing another set of files
			bShowOption = false;
		}
		UE_LOGF(LogEditorFactories, Log, "-- imported successfully" );
		// update the data in case the file source has changed
		ImportData->Update(UFactory::CurrentFilename);
		AnimSequence->ImportFileFramerate = Importer->GetOriginalFbxFramerate();

		// Try to find the outer package so we can dirty it up
		if (AnimSequence->GetOuter())
		{
			AnimSequence->GetOuter()->MarkPackageDirty();
		}
		else
		{
			AnimSequence->MarkPackageDirty();
		}
	}
	else
	{
		UE_LOGF(LogEditorFactories, Warning, "-- import failed" );
		Importer->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("Error_CouldNotReimportAnimation", "Cannot re-import animation.")), FFbxErrors::Generic_ReimportingObjectFailed);
		Importer->ReleaseScene();
		return EReimportResult::Failed;
	}

	Importer->ReleaseScene(); 

	return EReimportResult::Succeeded;
}

int32 UReimportFbxAnimSequenceFactory::GetPriority() const
{
	return ImportPriority;
}

bool UReimportFbxAnimSequenceFactory::IsAutomatedImport() const
{
	return Super::IsAutomatedImport() || IsAutomatedReimport();
}


/*------------------------------------------------------------------------------
	FBlueprintParentFilter implementation.
------------------------------------------------------------------------------*/

class FBlueprintParentFilter : public IClassViewerFilter
{
public:
	/** Classes to not allow any children of into the Class Viewer/Picker. */
	TSet< const UClass* > DisallowedChildrenOfClasses;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		return InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InClass) != EFilterReturn::Passed && !InClass->HasAnyClassFlags(CLASS_Deprecated);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InFilterFuncs->IfInChildOfClassesSet(DisallowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Passed && !InUnloadedClassData->HasAnyClassFlags(CLASS_Deprecated);
	}
};


/*------------------------------------------------------------------------------
	UBlueprintFactory implementation.
------------------------------------------------------------------------------*/
UBlueprintFactory::UBlueprintFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	// Look in the config file to determine what the default base class is, if any
	FString ClassPath;
	GConfig->GetString(TEXT("/Script/Engine.Engine"), TEXT("DefaultBlueprintBaseClassName"), /*out*/ ClassPath, GEngineIni);
	UClass* DefaultParentClass = !ClassPath.IsEmpty() 
		? LoadClass<UObject>(nullptr, *ClassPath, nullptr, LOAD_None, nullptr) 
		: nullptr;
	
	if( !DefaultParentClass || !FKismetEditorUtilities::CanCreateBlueprintOfClass(DefaultParentClass) )
	{
		DefaultParentClass = AActor::StaticClass();
	}

	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = DefaultParentClass;
}

bool UBlueprintFactory::ConfigureProperties()
{
	if (bSkipClassPicker)
	{
		check(ParentClass);
		return true;
	}

	// Null the parent class to ensure one is selected
	ParentClass = nullptr;

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bShowObjectRootClass = true;

	// Only want blueprint actor base classes.
	Options.bIsBlueprintBaseOnly = true;

	// This will allow unloaded blueprints to be shown.
	Options.bShowUnloadedBlueprints = true;

	// Enable Class Dynamic Loading
	Options.bEnableClassDynamicLoading = true;

	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::Dynamic;

	// Prevent creating blueprints of classes that require special setup (they'll be allowed in the corresponding factories / via other means)
	TSharedPtr<FBlueprintParentFilter> Filter = MakeShareable(new FBlueprintParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());
	if (BlueprintType == EBlueprintType::BPTYPE_MacroLibrary)
	{
		Filter->DisallowedChildrenOfClasses.Add(ALevelScriptActor::StaticClass());
		Filter->DisallowedChildrenOfClasses.Add(UAnimInstance::StaticClass());
	}

	// Filter out interfaces in all cases; they can never contain code, so it doesn't make sense to use them as a macro basis
	Filter->DisallowedChildrenOfClasses.Add(UInterface::StaticClass());
	
	// Allow overriding properties
	OnConfigurePropertiesDelegate.ExecuteIfBound(&Options);

	const FText TitleText = LOCTEXT("CreateBlueprintOptions", "Pick Parent Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UBlueprint::StaticClass());

	if ( bPressedOk )
	{
		ParentClass = ChosenClass;

		FEditorDelegates::OnFinishPickingBlueprintClass.Broadcast(ParentClass);
	}

	return bPressedOk;
};

UObject* UBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UBlueprint::StaticClass()));

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args ) );
		return nullptr;
	}
	else
	{
		UClass* BlueprintClass = nullptr;
		UClass* BlueprintGeneratedClass = nullptr;

		IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		KismetCompilerModule.GetBlueprintTypesForClass(ParentClass, BlueprintClass, BlueprintGeneratedClass);

		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, BlueprintClass, BlueprintGeneratedClass, CallingContext);
	}
}

UObject* UBlueprintFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}


/*------------------------------------------------------------------------------
	UBlueprintMacroFactory implementation.
------------------------------------------------------------------------------*/
UBlueprintMacroFactory::UBlueprintMacroFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = AActor::StaticClass();
	BlueprintType = BPTYPE_MacroLibrary;
}

FText UBlueprintMacroFactory::GetDisplayName() const
{
	return LOCTEXT("BlueprintMacroLibraryFactoryDescription", "Blueprint Macro Library");
}

FName UBlueprintMacroFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("ClassThumbnail.BlueprintMacroLibrary");
}

uint32 UBlueprintMacroFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Blueprint;
}

FText UBlueprintMacroFactory::GetToolTip() const
{
	return LOCTEXT("BlueprintMacroLibraryTooltip", "Blueprint Macro Libraries are containers of macros to be used in other blueprints. They cannot contain variables, inherit from other blueprints, or be placed in levels. Changes to macros in a Blueprint Macro Library will not take effect until client blueprints are recompiled.");
}

FString UBlueprintMacroFactory::GetToolTipDocumentationExcerpt() const
{
	return TEXT("UBlueprint_Macro");
}

UObject* UBlueprintMacroFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UBlueprint::StaticClass()));

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args ) );
		return nullptr;
	}
	else
	{
		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
	}
}

FString UBlueprintMacroFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewMacroLibrary"));
}

/*------------------------------------------------------------------------------
BlueprintFunctionLibraryFactory implementation.
------------------------------------------------------------------------------*/

UBlueprintFunctionLibraryFactory::UBlueprintFunctionLibraryFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	static FBoolConfigValueHelper CanCreateNewHelper(TEXT("CustomBlueprintFunctionLibrary"), TEXT("bCanCreateNew"));
	bCreateNew = CanCreateNewHelper;
	bEditAfterNew = true;
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = UBlueprintFunctionLibrary::StaticClass();
	BlueprintType = BPTYPE_FunctionLibrary;
}

FText UBlueprintFunctionLibraryFactory::GetDisplayName() const
{
	return LOCTEXT("BlueprintFunctionLibraryFactoryDescription", "Blueprint Function Library");
}

FName UBlueprintFunctionLibraryFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("ClassThumbnail.BlueprintFunctionLibrary");
}

uint32 UBlueprintFunctionLibraryFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Blueprint;
}

FText UBlueprintFunctionLibraryFactory::GetToolTip() const
{
	return LOCTEXT("BlueprintFunctionLibraryTooltip", "Blueprint Function Libraries are containers of functions to be used in other blueprints. They cannot contain variables, inherit from other blueprints, or be placed in levels. Changes to functions in a Blueprint Function Library will take effect without recompiling the client blueprints.");
}

FString UBlueprintFunctionLibraryFactory::GetToolTipDocumentationExcerpt() const
{
	return TEXT("UBlueprint_FunctionLibrary");
}

UObject* UBlueprintFunctionLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UBlueprint::StaticClass()));

	if (ParentClass != UBlueprintFunctionLibrary::StaticClass())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : LOCTEXT("Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
	}
}

bool UBlueprintFunctionLibraryFactory::ConfigureProperties()
{
	return true;
}

FString UBlueprintFunctionLibraryFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewFunctionLibrary"));
}

/*------------------------------------------------------------------------------
	UBlueprintInterfaceFactory implementation.
------------------------------------------------------------------------------*/
UBlueprintInterfaceFactory::UBlueprintInterfaceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = UInterface::StaticClass();
	BlueprintType = BPTYPE_Interface;
}

FText UBlueprintInterfaceFactory::GetDisplayName() const
{
	return LOCTEXT("BlueprintInterfaceFactoryDescription", "Blueprint Interface");
}

FName UBlueprintInterfaceFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("ClassThumbnail.BlueprintInterface");
}

uint32 UBlueprintInterfaceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Blueprint;
}

FText UBlueprintInterfaceFactory::GetToolTip() const
{
	return LOCTEXT("BlueprintInterfaceTooltip", "A Blueprint Interface is a collection of one or more functions - name only, no implementation - that can be added to other Blueprints. These other Blueprints are then expected to implement the functions of the Blueprint Interface in a unique manner.");
}

FString UBlueprintInterfaceFactory::GetToolTipDocumentationExcerpt() const
{
	return TEXT("UBlueprint_Interface");
}

UObject* UBlueprintInterfaceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UBlueprint::StaticClass()));

	if ((ParentClass == nullptr) || !FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args ) );
		return nullptr;
	}
	else
	{
		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
	}
}

bool UBlueprintInterfaceFactory::ConfigureProperties()
{
	return true;
}

FString UBlueprintInterfaceFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("NewInterface"));
}

/*------------------------------------------------------------------------------
	UCurveFactory implementation.
------------------------------------------------------------------------------*/

UCurveFactory::UCurveFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UCurveBase::StaticClass();

	CurveClass = nullptr;
}

bool UCurveFactory::ConfigureProperties()
{
	// Null the CurveClass so we can get a clean class
	CurveClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UCurveBase::StaticClass());

	const FText TitleText = LOCTEXT("CreateCurveOptions", "Pick Curve Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UCurveBase::StaticClass());

	if ( bPressedOk )
	{
		CurveClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UCurveFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCurveBase* NewCurve = nullptr;
	if(CurveClass != nullptr)
	{
		NewCurve = NewObject<UCurveBase>(InParent, CurveClass, Name, Flags);
	}

	return NewCurve;
}

/*------------------------------------------------------------------------------
	UCurveFloatFactory implementation.
------------------------------------------------------------------------------*/

UCurveFloatFactory::UCurveFloatFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveFloat::StaticClass();
	CurveClass = UCurveFloat::StaticClass();
}

bool UCurveFloatFactory::ConfigureProperties()
{
	return true;
}

/*------------------------------------------------------------------------------
	UCurveLinearColorFactory implementation.
------------------------------------------------------------------------------*/

UCurveLinearColorFactory::UCurveLinearColorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveLinearColor::StaticClass();
	CurveClass = UCurveLinearColor::StaticClass();
}

bool UCurveLinearColorFactory::ConfigureProperties()
{
	return true;
}

/*------------------------------------------------------------------------------
	UCurveVectorFactory implementation.
------------------------------------------------------------------------------*/

UCurveVectorFactory::UCurveVectorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UCurveVector::StaticClass();
	CurveClass = UCurveVector::StaticClass();
}

bool UCurveVectorFactory::ConfigureProperties()
{
	return true;
}

/*------------------------------------------------------------------------------
	UCurveImportFactory implementation.
------------------------------------------------------------------------------*/

UCurveImportFactory::UCurveImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bCreateNew = false;
	SupportedClass = UCurveBase::StaticClass();

	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("as;Audio amplitude curve"));
}

// @note jf: for importing a curve from a text format.  this is experimental code for a prototype feature and not fully fleshed out
UObject* UCurveImportFactory::FactoryCreateText( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn )
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	if(	FCString::Stricmp( Type, TEXT( "AS" ) ) == 0 )
	{
		FString Str;
		TCHAR const* BufRead = Buffer;

		// first line is faFile="", we can ignore
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);

		// 2nd line is fps=X
		float KeyFrameHz = 0.f;
		if ( !FParse::Value(BufRead, TEXT("fps="), KeyFrameHz) ) return nullptr;
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);

		// next line is scale=X, we can ignore?
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);
		// next line is smoothing=X, we can ignore?
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);
		// next line is dBValues=X, we can ignore?
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);
		// next line is stereo=X, we can ignore?
		if ( !FParse::Line(&BufRead, Str) ) return nullptr;
		FParse::Next(&BufRead);

		// next line is amplitude=[, then list of CSV floats
		if ( !FParse::Value(BufRead, TEXT("amplitude=["), Str) ) return nullptr;
		BufRead += FCString::Strlen(TEXT("amplitude=["));

		TArray<float> FloatKeys;

		while (1)
		{
			if (!FParse::AlnumToken(BufRead, Str)) break;

			float Key = (float) FCString::Atoi(*Str);
			FloatKeys.Add(Key);

			if (*BufRead == ',')
			{
				BufRead++;
				FParse::Next(&BufRead);
			}
			else
			{
				break;
			}
		}

		// make the curve object and set up the keys
		if (FloatKeys.Num() > 0)
		{
			UCurveFloat* NewCurve = NewObject<UCurveFloat>(InParent,InName,Flags);

			if (NewCurve)
			{
				for (int32 KeyIdx=0; KeyIdx<FloatKeys.Num(); ++KeyIdx)
				{
					float const KeyTime = KeyIdx / KeyFrameHz;
					float const KeyValue = FloatKeys[KeyIdx];
					FKeyHandle const KeyHandle = NewCurve->FloatCurve.AddKey(KeyTime, KeyValue);
					NewCurve->FloatCurve.SetKeyInterpMode(KeyHandle, RCIM_Cubic);
				}
			}

			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NewCurve);

			return NewCurve;
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport( this, nullptr );
	return nullptr;
}


/*------------------------------------------------------------------------------
	UArchetypeLibraryFactory implementation.
------------------------------------------------------------------------------*/
UObjectLibraryFactory::UObjectLibraryFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UObjectLibrary::StaticClass();
}

UObject* UObjectLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UObjectLibrary>(InParent, Class, Name, Flags);
}

/*------------------------------------------------------------------------------
	UDataAssetFactory implementation.
------------------------------------------------------------------------------*/

UDataAssetFactory::UDataAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UDataAsset::StaticClass();
}

bool UDataAssetFactory::ConfigureProperties()
{
	// nullptr the DataAssetClass so we can check for selection
	DataAssetClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	TSharedPtr<FAssetClassParentFilter> Filter = MakeShareable(new FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown;
	Filter->AllowedChildrenOfClasses.Add(UDataAsset::StaticClass());

	const FText TitleText = LOCTEXT("CreateDataAssetOptions", "Pick Class For Data Asset Instance");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UDataAsset::StaticClass());

	if ( bPressedOk )
	{
		DataAssetClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UDataAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (DataAssetClass != nullptr)
	{
		return NewObject<UDataAsset>(InParent, DataAssetClass, Name, Flags | RF_Transactional);
	}
	else
	{
		// if we have no data asset class, use the passed-in class instead
		check(Class->IsChildOf(UDataAsset::StaticClass()));
		return NewObject<UDataAsset>(InParent, Class, Name, Flags);
	}
}

/*------------------------------------------------------------------------------
	UBlendSpaceFactoryNew.
------------------------------------------------------------------------------*/
UBlendSpaceFactoryNew::UBlendSpaceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UBlendSpace::StaticClass();
	bCreateNew = true;
}

bool UBlendSpaceFactoryNew::ConfigureProperties()
{
	// Null the parent class so we can check for selection later
	TargetSkeleton = nullptr;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;

	/** The asset picker will only show skeletal meshes */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UBlendSpaceFactoryNew::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;


	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateBlendSpaceOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];


	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != nullptr;
}

UObject* UBlendSpaceFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	if (TargetSkeleton)
	{
		UBlendSpace * BlendSpace = NewObject<UBlendSpace>(InParent, Class, Name, Flags);

		BlendSpace->SetSkeleton(TargetSkeleton);
		if (PreviewSkeletalMesh)
		{
			BlendSpace->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return BlendSpace;
	}

	return nullptr;
}

void UBlendSpaceFactoryNew::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

/*------------------------------------------------------------------------------
	UBlendSpaceFactory1D.
------------------------------------------------------------------------------*/
UBlendSpaceFactory1D::UBlendSpaceFactory1D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UBlendSpace1D::StaticClass();
	bCreateNew = true;
}

bool UBlendSpaceFactory1D::ConfigureProperties()
{
	// Null the parent class so we can check for selection later
	TargetSkeleton = nullptr;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;

	/** The asset picker will only show skeletal meshes */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UBlendSpaceFactory1D::OnTargetSkeletonSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreateBlendSpaceOptions", "Pick Skeleton"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];


	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeleton != nullptr;
}

UObject* UBlendSpaceFactory1D::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	if (TargetSkeleton)
	{
		UBlendSpace1D * BlendSpace = NewObject<UBlendSpace1D>(InParent, Class, Name, Flags);

		BlendSpace->SetSkeleton(TargetSkeleton);
		if (PreviewSkeletalMesh)
		{
			BlendSpace->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return BlendSpace;
	}

	return nullptr;
}

void UBlendSpaceFactory1D::OnTargetSkeletonSelected(const FAssetData& SelectedAsset)
{
	TargetSkeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

/*------------------------------------------------------------------------------
	UAimOffsetBlendSpaceFactoryNew.
------------------------------------------------------------------------------*/
UAimOffsetBlendSpaceFactoryNew::UAimOffsetBlendSpaceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAimOffsetBlendSpace::StaticClass();
	bCreateNew = true;
}

UObject* UAimOffsetBlendSpaceFactoryNew::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	if (TargetSkeleton)
	{
		UAimOffsetBlendSpace * BlendSpace = NewObject<UAimOffsetBlendSpace>(InParent, Class, Name, Flags);

		BlendSpace->SetSkeleton(TargetSkeleton);
		if (PreviewSkeletalMesh)
		{
			BlendSpace->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return BlendSpace;
	}

	return nullptr;
}

/*------------------------------------------------------------------------------
	UAimOffsetBlendSpaceFactory1D.
------------------------------------------------------------------------------*/
UAimOffsetBlendSpaceFactory1D::UAimOffsetBlendSpaceFactory1D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAimOffsetBlendSpace1D::StaticClass();
	bCreateNew = true;
}

UObject* UAimOffsetBlendSpaceFactory1D::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	if (TargetSkeleton)
	{
		UAimOffsetBlendSpace1D * BlendSpace = NewObject<UAimOffsetBlendSpace1D>(InParent, Class, Name, Flags);

		BlendSpace->SetSkeleton(TargetSkeleton);
		if (PreviewSkeletalMesh)
		{
			BlendSpace->SetPreviewMesh(PreviewSkeletalMesh);
		}

		return BlendSpace;
	}

	return nullptr;
}

/*------------------------------------------------------------------------------
	UEnumFactory implementation.
------------------------------------------------------------------------------*/
UEnumFactory::UEnumFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UUserDefinedEnum::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UEnumFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	ensure(UUserDefinedEnum::StaticClass() == Class);

	if(!FEnumEditorUtils::IsNameAvailebleForUserDefinedEnum(Name))
	{
		const FText Message = FText::Format( LOCTEXT("EnumWithNameAlreadyExists", "Enum '{0}' already exists. The name must be unique."), FText::FromName( Name ) );
		if(Warn)
		{
			Warn->Log( Message );
		}
		FMessageDialog::Open( EAppMsgType::Ok, Message);
		return nullptr;
	}

	return FEnumEditorUtils::CreateUserDefinedEnum(InParent, Name, Flags);
}

/*------------------------------------------------------------------------------
	UStructureFactory implementation.
------------------------------------------------------------------------------*/
UStructureFactory::UStructureFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UUserDefinedStruct::StaticClass();
	bCreateNew = FStructureEditorUtils::UserDefinedStructEnabled();
	bEditAfterNew = true;
}

UObject* UStructureFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	ensure(UUserDefinedStruct::StaticClass() == Class);
	return FStructureEditorUtils::CreateUserDefinedStruct(InParent, Name, Flags);
}

/*-----------------------------------------------------------------------------
UForceFeedbackAttenuationFactory implementation.
-----------------------------------------------------------------------------*/
UForceFeedbackAttenuationFactory::UForceFeedbackAttenuationFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UForceFeedbackAttenuation::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UForceFeedbackAttenuationFactory::FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn )
{
	return NewObject<UForceFeedbackAttenuation>(InParent, InName, Flags);
}

/*-----------------------------------------------------------------------------
	UForceFeedbackEffectFactory implementation.
-----------------------------------------------------------------------------*/
UForceFeedbackEffectFactory::UForceFeedbackEffectFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UForceFeedbackEffect::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UForceFeedbackEffectFactory::FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn )
{
	return NewObject<UForceFeedbackEffect>(InParent, InName, Flags);
}

/*-----------------------------------------------------------------------------
	UHapticFeedbackEffectCurveFactory implementation.
-----------------------------------------------------------------------------*/
UHapticFeedbackEffectCurveFactory::UHapticFeedbackEffectCurveFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

	SupportedClass = UHapticFeedbackEffect_Curve::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UHapticFeedbackEffectCurveFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHapticFeedbackEffect_Curve>(InParent, InName, Flags);
}

/*-----------------------------------------------------------------------------
UHapticFeedbackEffectBufferFactory implementation.
-----------------------------------------------------------------------------*/
UHapticFeedbackEffectBufferFactory::UHapticFeedbackEffectBufferFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UHapticFeedbackEffect_Buffer::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UHapticFeedbackEffectBufferFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHapticFeedbackEffect_Buffer>(InParent, InName, Flags);
}

/*-----------------------------------------------------------------------------
UHapticFeedbackEffectSoundWaveFactory implementation.
-----------------------------------------------------------------------------*/
UHapticFeedbackEffectSoundWaveFactory::UHapticFeedbackEffectSoundWaveFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UHapticFeedbackEffect_SoundWave::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UHapticFeedbackEffectSoundWaveFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UHapticFeedbackEffect_SoundWave>(InParent, InName, Flags);
}

/*-----------------------------------------------------------------------------
USubsurfaceProfileFactory implementation.
-----------------------------------------------------------------------------*/
USubsurfaceProfileFactory::USubsurfaceProfileFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

	SupportedClass = USubsurfaceProfile::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USubsurfaceProfileFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USubsurfaceProfile* Object = NewObject<USubsurfaceProfile>(InParent, InName, Flags);
	
	// Enable smooth upgrading from Separable to Burley. This flag change
	// allows all newly created subsurface profile in Burley, while allowing
	// the profiles loaded from files to be converted into burley if they
	// are separable.
	Object->Settings.bEnableBurley = true;

	// Enable smooth upgrading from DMFP to MFP. This flag change allows
	// all newly created subsurface profile in MFP, while allowing old profiles
	// loaded from files to be automatically converted to MFP.
	Object->Settings.bEnableMeanFreePath = true;

	// Assign a GUID to the asset
	Object->Guid = FGuid::NewGuid();

	return Object;
}


/*-----------------------------------------------------------------------------
	USpecularProfileFactory implementation.
	-----------------------------------------------------------------------------*/
USpecularProfileFactory::USpecularProfileFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{

	SupportedClass = USpecularProfile::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* USpecularProfileFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	USpecularProfile* Object = NewObject<USpecularProfile>(InParent, InName, Flags);	
	Object->Guid = FGuid::NewGuid();
	return Object;
}

/*-----------------------------------------------------------------------------
	UNeuralProfileFactory implementation.
	-----------------------------------------------------------------------------*/
UNeuralProfileFactory::UNeuralProfileFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNeuralProfile::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UNeuralProfileFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UNeuralProfile* Object = NewObject<UNeuralProfile>(InParent, InName, Flags);
	Object->Guid = FGuid::NewGuid();
	return Object;
}


/*-----------------------------------------------------------------------------
	UToonProfileFactory implementation.
-----------------------------------------------------------------------------*/
UToonProfileFactory::UToonProfileFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SupportedClass = UToonProfile::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UToonProfileFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UToonProfile* Object = NewObject<UToonProfile>(InParent, InName, Flags);	
	Object->Guid = FGuid::NewGuid();
	return Object;
}

/*-----------------------------------------------------------------------------
	UTouchInterfaceFactory implementation.
-----------------------------------------------------------------------------*/
UTouchInterfaceFactory::UTouchInterfaceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UTouchInterface::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UTouchInterfaceFactory::FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn )
{
	return NewObject<UTouchInterface>(InParent, InName, Flags);
}

/*------------------------------------------------------------------------------
UStringTableFactory implementation.
------------------------------------------------------------------------------*/
UStringTableFactory::UStringTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UStringTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UStringTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UStringTable>(InParent, Name, Flags);
}

/*------------------------------------------------------------------------------
 UPreviewMeshCollectionFactory implementation.
------------------------------------------------------------------------------*/

UPreviewMeshCollectionFactory::UPreviewMeshCollectionFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPreviewMeshCollection::StaticClass();
	bCreateNew = true;
}

FText UPreviewMeshCollectionFactory::GetDisplayName() const
{
	return LOCTEXT("PreviewMeshCollection", "Preview Mesh Collection");
}

FText UPreviewMeshCollectionFactory::GetToolTip() const
{
	return LOCTEXT("PreviewMeshCollection_Tooltip", "Preview Mesh Collections are used to build collections of related skeletal meshes that are animated together (such as components of a character)");
}

bool UPreviewMeshCollectionFactory::ConfigureProperties()
{
	if (CurrentSkeleton.IsValid())
	{
		return true;
	}

	USkeleton* Skeleton = ChooseSkeleton();
	if (Skeleton != nullptr)
	{
		CurrentSkeleton = Skeleton;
		return true;
	}

	return false;
}

UObject* UPreviewMeshCollectionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPreviewMeshCollection* NewCollection = NewObject<UPreviewMeshCollection>(InParent, Name, Flags);
	NewCollection->Skeleton = CurrentSkeleton.Get();

	return NewCollection;
}

UUDIMTextureFunctionLibrary::UUDIMTextureFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{}

struct FScopedTextureSourceExtracts
{
	const TArray<UTexture2D*>& Textures;
	TArray<const uint8*> DataArray;
	TArray<FTextureSourceBlock> Blocks;
	ETextureSourceFormat Format;

	FScopedTextureSourceExtracts(const TArray<UTexture2D*>& InTextures, const TArray<FIntPoint>& BlockCoords)
		: Textures(InTextures)
		, Format(TSF_Invalid)
	{
		bool bMismatchedFormat = false;

		DataArray.Reserve(Textures.Num());
		Blocks.Reserve(Textures.Num());

		for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
		{
			UTexture2D* Texture = Textures[TextureIndex];
			const FIntPoint& BlockXY = BlockCoords[TextureIndex];

			if (Texture)
			{
				check(Texture->Source.GetNumLayers() == 1 && Texture->Source.GetNumBlocks() == 1);

				if (Format == TSF_Invalid)
				{
					Format = Texture->Source.GetFormat();
				}

				if (Format != Texture->Source.GetFormat())
				{
					bMismatchedFormat = true;
				}

				const uint8* DataPtr = Texture->Source.LockMipReadOnly(0);
				if (DataPtr)
				{
					DataArray.Add(DataPtr);
					Blocks.AddDefaulted();

					FTextureSourceBlock& Block = Blocks.Last();
					Block.BlockX = BlockXY.X;
					Block.BlockY = BlockXY.Y;
					Block.SizeX = Texture->Source.GetSizeX();
					Block.SizeY = Texture->Source.GetSizeY();
					Block.NumSlices = 1;
					Block.NumMips = Texture->Source.GetNumMips();
				}
			}
		}

		if (bMismatchedFormat)
		{
			UE_LOGF(LogBlueprintUserMessages, Error, "Cannot create a UDIM texture from Texture2Ds with different formats.");
			DataArray.Empty();
			Blocks.Empty();
		}
	}

	~FScopedTextureSourceExtracts()
	{
		for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); ++TextureIndex)
		{
			UTexture2D* Texture = Textures[TextureIndex];
			if (Texture)
			{
				Texture->Source.UnlockMip(0);
			}
		}
	}

	bool IsValid() const
	{
		return DataArray.Num() > 0 && Blocks.Num() > 0;
	}
};

UTexture2D* UUDIMTextureFunctionLibrary::MakeUDIMVirtualTextureFromTexture2Ds(FString OutputPathName, const TArray<UTexture2D*>& SourceTextures, const TArray<FIntPoint>& BlockCoords, bool bKeepExistingSettings, bool bCheckOutAndSave)
{
	static const TConsoleVariableData<int32>* CVarVirtualTexturesEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTextures"));
	check(CVarVirtualTexturesEnabled);

	if (!CVarVirtualTexturesEnabled || !CVarVirtualTexturesEnabled->GetValueOnAnyThread())
	{
		UE_LOGF(LogBlueprintUserMessages, Error, "Cannot create virtual textures when virtual texture support (r.VirtualTextures) is disabled.");
		return nullptr;
	}
	if (SourceTextures.Num() <= 1)
	{
		UE_LOGF(LogBlueprintUserMessages, Error, "Not enough source textures to create a UDIM texture.");
		return nullptr;
	}
	if (SourceTextures.Num() != BlockCoords.Num())
	{
		UE_LOGF(LogBlueprintUserMessages, Error, "The numbers of source textures and block coordinates don't match.");
		return nullptr;
	}
#if DO_CHECK
	for (int32 Index = 0; Index < SourceTextures.Num(); ++Index)
	{
		UTexture2D* Texture = SourceTextures[Index];
		const FIntPoint& Coords = BlockCoords[Index];
		if (Texture && (Coords.X < 0 || Coords.Y < 0))
		{
			UE_LOGF(LogBlueprintUserMessages, Error, "BlockCoords[%d] contains negative coordinate(s).", Index);
			return nullptr;
		}
	}
#endif

	OutputPathName = UPackageTools::SanitizePackageName(OutputPathName);

	FName TextureName;
	{
		int32 SlashIndex;
		int32 BackSlashIndex;
		OutputPathName.FindLastChar(TEXT('/'), SlashIndex);
		OutputPathName.FindLastChar(TEXT('\\'), BackSlashIndex);
		const int32 TextureNameStart = FMath::Max(SlashIndex, BackSlashIndex) + 1;
		TextureName = *OutputPathName.RightChop(TextureNameStart);
	}

	UTexture2D* ExistingTexture = LoadObject<UTexture2D>(nullptr, *OutputPathName, nullptr, LOAD_Quiet | LOAD_NoWarn);
	UPackage* TexturePackage;

	if (!ExistingTexture)
	{
		TexturePackage = CreatePackage(*OutputPathName);
	}
	else
	{
		TexturePackage = ExistingTexture->GetOutermost();
	}

	FScopedTextureSourceExtracts SourceExtracts(SourceTextures, BlockCoords);
	UTextureFactory* TextureFactory = NewObject<UTextureFactory>();

	if (!SourceExtracts.IsValid() || !TextureFactory)
	{
		return nullptr;
	}

	TextureFactory->AddToRoot();
	TextureFactory->SuppressImportOverwriteDialog();

	UTexture2D* NewTexture = TextureFactory->CreateTexture2D(TexturePackage, TextureName, RF_Public | RF_Standalone);

	TextureFactory->RemoveFromRoot();

	if (!NewTexture)
	{
		return nullptr;
	}

	ETextureSourceFormat Format = SourceExtracts.Format;
	NewTexture->Source.InitBlocked(&Format, SourceExtracts.Blocks.GetData(), 1, SourceExtracts.Blocks.Num(), SourceExtracts.DataArray.GetData());

	UTexture2D* FirstSourceTexture = SourceTextures[0];
	UTexture2D* SettingsSource = ExistingTexture && bKeepExistingSettings ? ExistingTexture : FirstSourceTexture;

	NewTexture->AddressX = SettingsSource->AddressX;
	NewTexture->AddressY = SettingsSource->AddressY;
	NewTexture->Filter = SettingsSource->Filter;
	NewTexture->LODGroup = SettingsSource->LODGroup;
	NewTexture->CompressionSettings = SettingsSource->CompressionSettings;
	NewTexture->LODBias = SettingsSource->LODBias;
	NewTexture->NumCinematicMipLevels = SettingsSource->NumCinematicMipLevels;
	NewTexture->NeverStream = SettingsSource->NeverStream;
	NewTexture->SRGB = SettingsSource->SRGB;
	NewTexture->bPreserveBorder = SettingsSource->bPreserveBorder;
	NewTexture->CompressionNone = SettingsSource->CompressionNone;
	NewTexture->CompressionNoAlpha = SettingsSource->CompressionNoAlpha;
	NewTexture->DeferCompression = SettingsSource->DeferCompression;
	NewTexture->bFlipGreenChannel = SettingsSource->bFlipGreenChannel;
	NewTexture->bDoScaleMipsForAlphaCoverage = SettingsSource->bDoScaleMipsForAlphaCoverage;
	NewTexture->AlphaCoverageThresholds = SettingsSource->AlphaCoverageThresholds;
	NewTexture->bUseNewMipFilter = SettingsSource->bUseNewMipFilter;
	NewTexture->AdjustBrightness = SettingsSource->AdjustBrightness;
	NewTexture->AdjustBrightnessCurve = SettingsSource->AdjustBrightnessCurve;
	NewTexture->AdjustVibrance = SettingsSource->AdjustVibrance;
	NewTexture->AdjustSaturation = SettingsSource->AdjustSaturation;
	NewTexture->AdjustRGBCurve = SettingsSource->AdjustRGBCurve;
	NewTexture->AdjustHue = SettingsSource->AdjustHue;
	NewTexture->AdjustMinAlpha = SettingsSource->AdjustMinAlpha;
	NewTexture->AdjustMaxAlpha = SettingsSource->AdjustMaxAlpha;
	NewTexture->MipGenSettings = SettingsSource->MipGenSettings;
	NewTexture->VirtualTextureStreaming = true;

	if (ExistingTexture)
	{
		// Disable streaming
		ExistingTexture->UpdateResource();
		// Wait for InitRHI() to complete before the FTextureReferenceReplacer calls ReleaseRHI() to follow the workflow.
		// Static texture needs to avoid having pending InitRHI() before enqueuing ReleaseRHI() to safely track access of the PlatformData on the renderthread.
		ExistingTexture->WaitForPendingInitOrStreaming();
	}

	FTextureReferenceReplacer(ExistingTexture).Replace(NewTexture);

	// Invalidate any materials using the new texture. (occurs if overwriting an existing texture)
	NewTexture->PostEditChange();

	FAssetRegistryModule::AssetCreated(NewTexture);

	TexturePackage->MarkPackageDirty();

	if (bCheckOutAndSave)
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(TexturePackage);
		UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);
	}

	return NewTexture;
}

#undef LOCTEXT_NAMESPACE
