// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorMorphFactory.cpp: Morph target mesh factory import code.
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Animation/MorphTarget.h"
#include "Factories.h"

#define SET_MORPH_ERROR( ErrorType ) { if( Error ) *Error = ErrorType; }

/** 
* Constructor
* 
* @param	InSrcMesh - source skeletal mesh that the new morph mesh morphs
* @param	LODIndex - lod entry of base mesh
* @param	InWarn - for outputing warnings
*/
FMorphTargetBinaryImport::FMorphTargetBinaryImport( USkeletalMesh* InSrcMesh, int32 LODIndex, FFeedbackContext* InWarn )
:	Warn(InWarn)
,	BaseLODIndex(LODIndex)
,   BaseMesh(InSrcMesh)
{	
}

/** 
* Constructor
* 
* @param	InSrcMesh - source static mesh that the new morph mesh morphs
* @param	LODIndex - lod entry of base mesh
* @param	InWarn - for outputing warnings
*/
FMorphTargetBinaryImport::FMorphTargetBinaryImport( UStaticMesh* InSrcMesh, int32 LODIndex, FFeedbackContext* InWarn )
:	Warn(InWarn)
,	BaseLODIndex(LODIndex)
,   BaseMesh(InSrcMesh)
{	
}


