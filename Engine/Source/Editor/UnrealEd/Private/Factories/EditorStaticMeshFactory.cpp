// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/EditorStaticMeshFactory.h"

#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Instances/InstancedPlacementPartitionActor.h"
#include "Instances/EditorPlacementSettings.h"

#include "LevelEditorSubsystem.h"

#include "ActorPartition/ActorPartitionSubsystem.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "UObject/Class.h"
#include "Editor.h"

#include "Subsystems/PlacementSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorStaticMeshFactory)

bool UEditorStaticMeshFactory::PrePlaceAsset(FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	if (!Super::PrePlaceAsset(InPlacementInfo, InPlacementOptions))
	{
		return false;
	}

	if (!InPlacementInfo.PreferredLevel.IsValid())
	{
		return false;
	}

	if (!InPlacementInfo.SettingsObject)
	{
		return false;
	}

	// Make a good known client GUID out of the placed asset's package if one was not given to us.
	if (!InPlacementInfo.ItemGuid.IsValid())
	{
		InPlacementInfo.ItemGuid = InPlacementInfo.AssetToPlace.GetAsset()->GetPackage()->GetPersistentGuid();
	}

	return true;
}

TArray<FTypedElementHandle> UEditorStaticMeshFactory::PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	// If we're disallowing instanced placement, or creating preview elements, don't use the ISM placement.
	if (!ShouldPlaceInstancedStaticMeshes(InPlacementOptions))
	{
		return Super::PlaceAsset(InPlacementInfo, InPlacementOptions);
	}

	if (UActorPartitionSubsystem* PartitionSubsystem = UWorld::GetSubsystem<UActorPartitionSubsystem>(InPlacementInfo.PreferredLevel->GetWorld()))
	{
		// Create or find the placement partition actor
		auto OnActorCreated = [InPlacementOptions](APartitionActor* CreatedPartitionActor)
		{
			if (AInstancedPlacementPartitionActor* ElementPartitionActor = Cast<AInstancedPlacementPartitionActor>(CreatedPartitionActor))
			{
				ElementPartitionActor->SetGridGuid(InPlacementOptions.InstancedPlacementGridGuid);
			}
		};

		constexpr bool bCreatePartitionActorIfMissing = true;
		FActorPartitionGetParams PartitionActorFindParams(
			AInstancedPlacementPartitionActor::StaticClass(),
			bCreatePartitionActorIfMissing,
			InPlacementInfo.PreferredLevel.Get(),
			InPlacementInfo.FinalizedTransform.GetLocation(),
			0,
			InPlacementOptions.InstancedPlacementGridGuid,
			true,
			OnActorCreated
		);
		AInstancedPlacementPartitionActor* PlacedElementsActor = Cast<AInstancedPlacementPartitionActor>(PartitionSubsystem->GetActor(PartitionActorFindParams));

		auto RegisterISMDefinitionFunc = [&InPlacementInfo](AInstancedPlacementPartitionActor* PartitionActor, TSortedMap<int32, TArray<FTransform>>& ISMDefinition)
		{
			FISMComponentDescriptor ComponentDescriptor = InPlacementInfo.SettingsObject->InstancedComponentSettings;
			ComponentDescriptor.StaticMesh = Cast<UStaticMesh>(InPlacementInfo.AssetToPlace.GetAsset());
			ComponentDescriptor.ComputeHash();

			int32 DescriptorIndex = PartitionActor->RegisterISMComponentDescriptor(ComponentDescriptor);
			ISMDefinition.Emplace(DescriptorIndex, TArray<FTransform>({ FTransform() }));
		};

		FString ClientDisplayName = InPlacementInfo.NameOverride.ToString();
		if (ClientDisplayName.IsEmpty())
		{
			ClientDisplayName = InPlacementInfo.AssetToPlace.GetFullName();
		}

		// Create an info or find it based on the given client guid
		FClientPlacementInfo* PlacementInfo = PlacedElementsActor->PreAddClientInstances(InPlacementInfo.ItemGuid, ClientDisplayName, RegisterISMDefinitionFunc);
		if (!PlacementInfo || !PlacementInfo->IsInitialized())
		{
			return TArray<FTypedElementHandle>();
		}

		ModifiedPartitionActors.Add(PlacedElementsActor);

		FPlacementInstance InstanceToPlace;
		InstanceToPlace.SetInstanceWorldTransform(InPlacementInfo.FinalizedTransform);
		// todo: z offset and align to normal
		TArray<FTypedElementHandle> PlacedInstanceHandles;
		TArray<FSMInstanceId> PlacedInstances = PlacementInfo->AddInstances(MakeArrayView({ InstanceToPlace }));
		for (const FSMInstanceId PlacedInstanceID : PlacedInstances)
		{
			if (FTypedElementHandle PlacedHandle = UEngineElementsLibrary::AcquireEditorSMInstanceElementHandle(PlacedInstanceID))
			{
				PlacedInstanceHandles.Emplace(PlacedHandle);
			}
		}

		return PlacedInstanceHandles;
	}

	return TArray<FTypedElementHandle>();
}

FAssetData UEditorStaticMeshFactory::GetAssetDataFromElementHandle(const FTypedElementHandle& InHandle)
{
	FAssetData FoundAssetData;
	if (TTypedElement<ITypedElementAssetDataInterface> AssetDataInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementAssetDataInterface>(InHandle))
	{
		FoundAssetData = AssetDataInterface.GetAssetData();
	}

	if (!FoundAssetData.IsValid())
	{
		UInstancedStaticMeshComponent* ISMComponent = nullptr;
		if (TTypedElement<ITypedElementObjectInterface> ObjectInterface = UTypedElementRegistry::GetInstance()->GetElement<ITypedElementObjectInterface>(InHandle))
		{
			// Try to pull from component handle
			if (UInstancedStaticMeshComponent* RawComponentPtr = ObjectInterface.GetObjectAs<UInstancedStaticMeshComponent>())
			{
				ISMComponent = RawComponentPtr;
			}
			else if (AActor* RawActorPtr = ObjectInterface.GetObjectAs<AActor>())
			{
				ISMComponent = RawActorPtr->FindComponentByClass<UInstancedStaticMeshComponent>();
			}
		}

		if (ISMComponent)
		{
			FoundAssetData = FAssetData(ISMComponent->GetStaticMesh());
		}
	}

	if (CanPlaceElementsFromAssetData(FoundAssetData))
	{
		return FoundAssetData;
	}

	return Super::GetAssetDataFromElementHandle(InHandle);
}

UInstancedPlacemenClientSettings* UEditorStaticMeshFactory::FactorySettingsObjectForPlacement(const FAssetData& InAssetData, const FPlacementOptions& InPlacementOptions)
{
	if (!ShouldPlaceInstancedStaticMeshes(InPlacementOptions))
	{
		return Super::FactorySettingsObjectForPlacement(InAssetData, InPlacementOptions);
	}

	UEditorInstancedPlacementSettings* PlacementSettingsObject = NewObject<UEditorInstancedPlacementSettings>(this);
	if (PlacementSettingsObject)
	{
		UObject* AssetToPlaceAsObject = InAssetData.GetAsset();
		FISMComponentDescriptor& ComponentDescriptor = PlacementSettingsObject->InstancedComponentSettings;
		if (UStaticMesh* StaticMeshObject = Cast<UStaticMesh>(AssetToPlaceAsObject))
		{
			// If this is a Nanite mesh, prefer to use ISM over HISM, as HISM duplicates many features/bookkeeping that Nanite already handles for us.
			if (StaticMeshObject->HasValidNaniteData())
			{
				ComponentDescriptor.InitFrom(UInstancedStaticMeshComponent::StaticClass()->GetDefaultObject<UInstancedStaticMeshComponent>());
			}
			ComponentDescriptor.StaticMesh = StaticMeshObject;
		}
		else if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(AssetToPlaceAsObject))
		{
			if (UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent())
			{
				ComponentDescriptor.StaticMesh = StaticMeshComponent->GetStaticMesh();
			}
		}

		// Go ahead and compute the descriptor now, in case we do not go through a place cycle or edit any properties
		ComponentDescriptor.ComputeHash();
	}

	return PlacementSettingsObject;
}

bool UEditorStaticMeshFactory::ShouldPlaceInstancedStaticMeshes(const FPlacementOptions& InPlacementOptions) const
{
	return !InPlacementOptions.bIsCreatingPreviewElements && InPlacementOptions.InstancedPlacementGridGuid.IsValid();
}

void UEditorStaticMeshFactory::EndPlacement(TArrayView<const FTypedElementHandle> InPlacedElements, const FPlacementOptions& InPlacementOptions)
{
	for (TWeakObjectPtr<AInstancedPlacementPartitionActor> ISMPartitionActor : ModifiedPartitionActors)
	{
		if (ISMPartitionActor.IsValid())
		{
			ISMPartitionActor->PostAddClientInstances();
		}
	}

	ModifiedPartitionActors.Empty();
}
