// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/AssetFactoryInterface.h"
#include "ActorFactories/ActorFactoryStaticMesh.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "Instances/InstancedPlacementClientInfo.h"

#include "EditorStaticMeshFactory.generated.h"

class AInstancedPlacementPartitionActor;
class UInstancedPlacemenClientSettings;

UCLASS(Transient)
class UEditorStaticMeshFactory : public UActorFactoryStaticMesh
{
	GENERATED_BODY()

public:
	// Begin IAssetFactoryInterface
	virtual bool PrePlaceAsset(FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	virtual TArray<FTypedElementHandle> PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	virtual FAssetData GetAssetDataFromElementHandle(const FTypedElementHandle& InHandle) override;
	virtual void EndPlacement(TArrayView<const FTypedElementHandle> InPlacedElements, const FPlacementOptions& InPlacementOptions) override;
	virtual UInstancedPlacemenClientSettings* FactorySettingsObjectForPlacement(const FAssetData& InAssetData, const FPlacementOptions& InPlacementOptions) override;
	// End IAssetFactoryInterface

protected:
	bool ShouldPlaceInstancedStaticMeshes(const FPlacementOptions& InPlacementOptions) const;

	TSet<TWeakObjectPtr<AInstancedPlacementPartitionActor>> ModifiedPartitionActors;
};