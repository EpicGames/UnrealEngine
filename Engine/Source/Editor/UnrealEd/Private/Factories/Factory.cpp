// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/Factory.h"

#include "AssetDefinitionRegistry.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/BulkData.h"
#include "Engine/Level.h"
#include "ObjectTools.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "EditorClassUtils.h"
#include "AutomatedAssetImportData.h"
#include "AssetImportTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Factory)

DEFINE_LOG_CATEGORY_STATIC(LogFactory, Log, All);


FString UFactory::CurrentFilename(TEXT(""));

//@third party BEGIN SIMPLYGON
FMD5Hash UFactory::FileHash;
//@third party END SIMPLYGON

// This needs to be greater than 0 to allow factories to have both higher and lower priority than the default
const int32 UFactory::DefaultImportPriority = 100;


UFactory::UFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ImportPriority = DefaultImportPriority;
}


void UFactory::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{	
	UFactory* This = CastChecked<UFactory>(InThis);
	Collector.AddReferencedObject(This->SupportedClass.GetGCPtr(), This);
	Collector.AddReferencedObject(This->ContextClass.GetGCPtr(), This);

	Super::AddReferencedObjects(This, Collector);
}

TArray<FAssetCategoryPath> UFactory::GetAssetMenuPaths(const UClass* InClass, FName InCategory) const
{
	const UClass* LocalSupportedClass = InClass;

	if (UAssetDefinitionRegistry::Get() && LocalSupportedClass)
	{
		const UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get();
		const UAssetDefinition* AssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForClass(LocalSupportedClass);
		if (AssetDefinition)
		{
			TArray<FAssetCategoryPath> AssetCategoryPaths;
			for (const FAssetCategoryPath& AssetCategoryPath : AssetDefinition->GetAssetCategories())
			{
				if (AssetCategoryPath.GetCategory() == InCategory)
				{
					AssetCategoryPaths.Add(AssetCategoryPath);
				}
			}
			return AssetCategoryPaths;
		}
	}

	const static TArray<FAssetCategoryPath> DefaultSubCategories;
	return DefaultSubCategories;
}


UObject* UFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	AdditionalImportedObjects.Empty();
	UAssetImportTask* Task = AssetImportTask;
	if (Task == nullptr)
	{
		Task = NewObject<UAssetImportTask>();
		Task->Filename = Filename;
	}

	if (ScriptFactoryCreateFile(Task))
	{
		TArray<UObject*> TaskResults = Task->GetObjects();
		if (TaskResults.Num() == 0)
		{
			return nullptr;
		}
		else
		{
			for (int32 ResultIndex = 1; ResultIndex < TaskResults.Num(); ++ResultIndex)
			{
				AdditionalImportedObjects.Add(TaskResults[ResultIndex]);
			}

			return TaskResults[0];
		}
	}

	FString FileExtension = FPaths::GetExtension(Filename);

	// load as text
	if (bText)
	{
		FString Data;
		if (!FFileHelper::LoadFileToString(Data, *Filename))
		{
			UE_LOGF(LogFactory, Error, "Failed to load file '%ls' to string", *Filename);
			return nullptr;
		}

		ParseParms(Parms);
		const TCHAR* Ptr = *Data;
			
		return FactoryCreateText(InClass, InParent, InName, Flags, nullptr, *FileExtension, Ptr, Ptr + Data.Len(), Warn, bOutOperationCanceled);
	}

	// load as binary
	{
		TArray64<uint8> Data;
		if (!FFileHelper::LoadFileToArray(Data, *Filename))
		{
			UE_LOGF(LogFactory, Error, "Failed to load file '%ls' to array", *Filename);
			return nullptr;
		}

		// adds an extra null byte
		// data loaders must be able to ignore an unexpected extra zero byte at the end of the file
		Data.Add(0);
		ParseParms(Parms);
		const uint8* Ptr = &Data[0];

		return FactoryCreateBinary(InClass, InParent, InName, Flags, nullptr, *FileExtension, Ptr, Ptr + Data.Num() - 1, Warn, bOutOperationCanceled);
	}
}


bool UFactory::FactoryCanImport( const FString& Filename )
{
	if (ScriptFactoryCanImport(Filename))
	{
		return true;
	}

	if (IsSupportedFileExtension(FPathViews::GetExtension(Filename)))
	{
		return true;
	}

	// T3D support
	if (FPaths::GetExtension(Filename) == TEXT("t3d"))
	{
		// open file
		FString Data;

		if (FFileHelper::LoadFileToString(Data, *Filename))
		{
			const TCHAR* Str= *Data;
			if (FParse::Command(&Str, TEXT("BEGIN")) && FParse::Command(&Str, TEXT("OBJECT")))
			{
				FString strClass;
				if (FParse::Value(Str, TEXT("CLASS="), strClass))
				{
					//we found the right syntax, so no error if we don't match
					if (strClass == SupportedClass->GetName())
					{
						return true;
					}

					return false;
				}
			}

			UE_LOGF(LogFactory, Warning, "Factory import failed due to invalid format: %ls", *Filename);
		}
		else
		{
			UE_LOGF(LogFactory, Warning, "Factory import failed due to inability to load file %ls", *Filename);
		}
	}

	return false;
}


UObject* UFactory::ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags InFlags, const FString& Filename, const TCHAR* Parms, bool& OutCanceled)
{
	UObject* Result = nullptr;
	AdditionalImportedObjects.Empty();
	CurrentFilename = Filename;
	

	// sanity check the file size of the impending import and prompt
	// the user if they wish to continue if the file size is very large
	const int64 FileSize = IFileManager::Get().FileSize(*CurrentFilename);

	//@third party BEGIN SIMPLYGON
	// Hack do not hash files that are huge.  They take forever anway
	const int32 Gigabyte = 1024 * 1024 * 1024;
	if(FileSize < Gigabyte)
	{
		FileHash = FMD5Hash::HashFile(*CurrentFilename);
	}

	//@third party END SIMPLYGON


	if (CanCreateNew())
	{
		UE_LOGF(LogFactory, Log, "FactoryCreateNew: %ls with %ls (%i %i %ls)", *InClass->GetName(), *GetClass()->GetName(), bCreateNew, bText, *Filename);
		ParseParms(Parms);

		Result = FactoryCreateNew(InClass, InOuter, InName, InFlags, nullptr, GWarn);
	}
	else if (!Filename.IsEmpty())
	{
		if (FileSize == INDEX_NONE)
		{
			UE_LOGF(LogFactory, Error, "Can't find file '%ls' for import", *Filename);
		}
		else
		{
			UE_LOGF(LogFactory, Log, "FactoryCreateFile: %ls with %ls (%i %i %ls)", *InClass->GetName(), *GetClass()->GetName(), bCreateNew, bText, *Filename);

			Result = FactoryCreateFile(InClass, InOuter, InName, InFlags, *Filename, Parms, GWarn, OutCanceled);
		}
	}

	if (Result != nullptr)
	{
		Result->MarkPackageDirty();
		ULevel::LevelDirtiedEvent.Broadcast();
		Result->PostEditChange();
	}

	CurrentFilename = TEXT("");

	return Result;
}


bool UFactory::ShouldShowInNewMenu() const
{
	return CanCreateNew();
}


FName UFactory::GetNewAssetThumbnailOverride() const
{
	// @note: this typically refers to a 64x64 thumbnail
	return NAME_None;
}

FName UFactory::GetNewAssetIconOverride() const
{
	// @note: this typically refers to a 16x16 icon
	return NAME_None;
}

FText UFactory::GetDisplayName() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	UClass* LocalSupportedClass = GetSupportedClass();
	if ( LocalSupportedClass )
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(LocalSupportedClass);
		if ( AssetTypeActions.IsValid() )
		{
			FText Name = AssetTypeActions.Pin()->GetName();

			if (!Name.IsEmpty())
			{
				return Name;
			}
		}

		// Factories whose classes do not have asset type actions should just display the sanitized class name
		return FText::FromString( FName::NameToDisplayString(*LocalSupportedClass->GetName(), false) );
	}

	// Factories that have no supported class have no display name.
	return FText();
}


uint32 UFactory::GetMenuCategories() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	UClass* LocalSupportedClass = GetSupportedClass();

	if (LocalSupportedClass)
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(LocalSupportedClass);
		if (AssetTypeActions.IsValid())
		{
			return AssetTypeActions.Pin()->GetCategories();
		}
	}

	// Factories whose classes do not have asset type actions fall in the misc category
	return EAssetTypeCategories::Misc;
}

const TArray<FText>& UFactory::GetMenuCategorySubMenus() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	UClass* LocalSupportedClass = GetSupportedClass();

	if (LocalSupportedClass)
	{
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(LocalSupportedClass);
		if (AssetTypeActions.IsValid())
		{
			return AssetTypeActions.Pin()->GetSubMenus();
		}
	}

	const static TArray<FText> DefaultSubCategories;
	return DefaultSubCategories;
}

TArray<FAssetCategoryPath> UFactory::GetAssetMenuPathsForCategory(FName InCategory) const
{
	return GetAssetMenuPaths(GetSupportedClass(), InCategory);
}

FText UFactory::GetToolTip() const
{
	return GetSupportedClass()->GetToolTipText();
}


FString UFactory::GetToolTipDocumentationPage() const
{
	return FEditorClassUtils::GetDocumentationPage(GetSupportedClass());
}


FString UFactory::GetToolTipDocumentationExcerpt() const
{
	return FEditorClassUtils::GetDocumentationExcerpt(GetSupportedClass());
}


UClass* UFactory::GetSupportedClass() const
{
	return SupportedClass;
}


bool UFactory::DoesSupportClass(UClass* Class)
{
	return (Class == GetSupportedClass());
}


UClass* UFactory::ResolveSupportedClass()
{
	// This check forces factories which support multiple classes to overload this method.
	// In other words, you can't have a SupportedClass of nullptr and not overload this method.
	check( SupportedClass );
	return SupportedClass;
}


void UFactory::ResetState()
{
	// Resets the state of the 'Yes To All / No To All' prompt for overwriting existing objects on import.
	// After the reset, the next import collision will always display the prompt.
	OverwriteYesOrNoToAllState = -1;
}


void UFactory::DisplayOverwriteOptionsDialog(const FText& Message)
{
	// if the asset importing is automated, get the override state from the automated settings from there because we cannot prompt
	if(AutomatedImportData)
	{
		OverwriteYesOrNoToAllState =  AutomatedImportData->bReplaceExisting ? EAppReturnType::YesAll : EAppReturnType::NoAll;
	}
	else if (AssetImportTask && AssetImportTask->bAutomated)
	{
		OverwriteYesOrNoToAllState = AssetImportTask->bReplaceExisting ? EAppReturnType::YesAll : EAppReturnType::NoAll;
	}
	else if (OverwriteYesOrNoToAllState != EAppReturnType::YesAll && OverwriteYesOrNoToAllState != EAppReturnType::NoAll)
	{
		OverwriteYesOrNoToAllState = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel, FText::Format(
			NSLOCTEXT("UnrealEd", "ImportedAssetAlreadyExists", "{0}\n\nWould you also like to overwrite its existing settings?"),
			Message));
	}
}


bool UFactory::SortFactoriesByPriority(const UFactory& A, const UFactory& B)
{
	// First sort so that higher priorities are earlier in the list
	if (A.ImportPriority > B.ImportPriority)
	{
		return true;
	}

	if (A.ImportPriority < B.ImportPriority)
	{
		return false;
	}

	// Then sort so that factories that only create new assets are tried after those that actually import the file data (when they have an equivalent priority)
	const bool bFactoryAImportsFiles = !A.CanCreateNew();
	const bool bFactoryBImportsFiles = !B.CanCreateNew();

	if (bFactoryAImportsFiles && !bFactoryBImportsFiles)
	{
		return true;
	}

	return false;
}


UObject* UFactory::StaticImportObject
(
	UClass*				Class,
	UObject*			InOuter,
	FName				Name,
	EObjectFlags		Flags,
	const TCHAR*		Filename,
	UObject*			Context,
	UFactory*			InFactory,
	const TCHAR*		Parms,
	FFeedbackContext*	Warn,
	int32				MaxImportFileSize
)
{
	bool bOperationCanceled = false;
	return StaticImportObject(Class, InOuter, Name, Flags, bOperationCanceled, Filename, Context, InFactory, Parms, Warn, MaxImportFileSize);
}


UObject* UFactory::StaticImportObject
(
	UClass*				Class,
	UObject*			InOuter,
	FName				Name,
	EObjectFlags		Flags,
	bool&				bOutOperationCanceled,
	const TCHAR*		Filename,
	UObject*			Context,
	UFactory*			InFactory,
	const TCHAR*		Parms,
	FFeedbackContext*	Warn,
	int32				MaxImportFileSize
)
{
	check(Class);

	UObject* Result = nullptr;
	FString Extension = FPaths::GetExtension(Filename);
	TArray<UFactory*> Factories;

	// Make list of all applicable factories.
	if (InFactory != nullptr)
	{
		// Use just the specified factory.
		if (ensureMsgf( !InFactory->SupportedClass || Class->IsChildOf(InFactory->SupportedClass), 
			TEXT("Factory is (%s), SupportedClass is (%s) and Class name is (%s)"), *InFactory->GetName(), (InFactory->SupportedClass)? *InFactory->SupportedClass->GetName() : TEXT("None"), *Class->GetName() ))
		{
			Factories.Add( InFactory );
		}
	}
	else
	{
		auto TransientPackage = GetTransientPackage();

		// try all automatic factories, sorted by priority
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (It->IsChildOf(UFactory::StaticClass()))
			{
				UFactory* Default = It->GetDefaultObject<UFactory>();

				if (Class->IsChildOf(Default->SupportedClass) && Default->ImportPriority >= 0)
				{
					//Add the factory if there is no extension or the factory don't have any supported file extension or the factory support this file extension.
					//Its ok to add CanCreateNew factory (even if there is an extension) since they will be less prioritize (if there priority is equal to) when sorting by priority.
					//See UFactory::SortFactoriesByPriority
					TArray<FString> FactoryExtension;
					Default->GetSupportedFileExtensions(FactoryExtension);

					if (Extension.IsEmpty() || (FactoryExtension.Num() == 0) || (FactoryExtension.Contains(Extension)))
					{
						Factories.Add(NewObject<UFactory>(TransientPackage, *It));
					}
				}
			}
		}

		Factories.Sort(&UFactory::SortFactoriesByPriority);
	}

	// Try each factory in turn.
	for (auto& Factory : Factories)
	{
		Result = Factory->ImportObject(Class, InOuter, Name, Flags, Filename, Parms, bOutOperationCanceled);

		if (Result != nullptr)
		{
			break;
		}
	}

	if ((Result == nullptr) && !bOutOperationCanceled)
	{
		Warn->Log(*FText::Format(NSLOCTEXT("UnrealEd", "ImportFailed", "Failed to import file '{0}'"), FText::FromString(FString(Filename))).ToString());
	}

	return Result;
}


void UFactory::GetSupportedFileExtensions(TArray<FString>& OutExtensions) const
{
	for (const auto& Format : Formats)
	{
		const int32 DelimiterIdx = Format.Find(TEXT(";"));

		if (DelimiterIdx != INDEX_NONE)
		{
			OutExtensions.Add(Format.Left(DelimiterIdx));
		}
	}
}


bool UFactory::IsSupportedFileExtension(FStringView InExtension) const
{
	if (InExtension.StartsWith(TEXT('.')))
	{
		InExtension.RightChopInline(1);
	}

	for (const FString& Format : Formats)
	{
		const int32 DelimiterIdx = Format.Find(TEXT(";"));

		if (DelimiterIdx != INDEX_NONE)
		{
			const FStringView FormatExtension(*Format, DelimiterIdx);
			if (FormatExtension == InExtension)
			{
				return true;
			}
		}
	}

	return false;
}


bool UFactory::ImportUntypedBulkDataFromText(const TCHAR*& Buffer, FBulkData& BulkData)
{
	FString StrLine;
	int32 ElementCount = 0;
	int32 ElementSize = 0;
	bool bBulkDataIsLocked = false;

	while(FParse::Line(&Buffer,StrLine))
	{
		FString ParsedText;
		const TCHAR* Str = *StrLine;

		if (FParse::Value(Str, TEXT("ELEMENTCOUNT="), ParsedText))
		{
			/** Number of elements in bulk data array */
			ElementCount = FCString::Atoi(*ParsedText);
		}
		else if (FParse::Value(Str, TEXT("ELEMENTSIZE="), ParsedText))
		{
			/** Serialized flags for bulk data */
			ElementSize = FCString::Atoi(*ParsedText);
		}
		else if (FParse::Value(Str, TEXT("BEGIN "), ParsedText) && (ParsedText.ToUpper() == TEXT("BINARYBLOB")))
		{
			uint8* RawData = nullptr;

			/** The bulk data... */
			while(FParse::Line(&Buffer,StrLine))
			{
				Str = *StrLine;

				if (FParse::Value(Str, TEXT("SIZE="), ParsedText))
				{
					int32 Size = FCString::Atoi(*ParsedText);

					check(Size == (ElementSize *ElementCount));

					BulkData.Lock(LOCK_READ_WRITE);
					void* RawBulkData = BulkData.Realloc(ElementCount, ElementSize);
					RawData = (uint8*)RawBulkData;
					bBulkDataIsLocked = true;
				}
				else if (FParse::Value(Str, TEXT("BEGIN "), ParsedText) && (ParsedText.ToUpper() == TEXT("BINARY")))
				{
					check(RawData);
					uint8* BulkDataPointer = RawData;
					while(FParse::Line(&Buffer,StrLine))
					{
						Str = *StrLine;
						TCHAR* ParseStr = (TCHAR*)(Str);

						if (FParse::Value(Str, TEXT("END "), ParsedText) && (ParsedText.ToUpper() == TEXT("BINARY")))
						{
							break;
						}

						// Clear whitespace
						while ((*ParseStr == L' ') || (*ParseStr == L'\t'))
						{
							ParseStr++;
						}

						// Parse the values into the bulk data...
						while ((*ParseStr != L'\n') && (*ParseStr != L'\r') && (*ParseStr != 0))
						{
							int32 Value;
							if (!FCString::Strnicmp(ParseStr, TEXT("0x"), 2))
							{
								ParseStr +=2;
							}
							Value = FParse::HexDigit(ParseStr[0]) * 16 + FParse::HexDigit(ParseStr[1]);
							*BulkDataPointer = (uint8)Value;
							BulkDataPointer++;
							ParseStr += 2;
							ParseStr++;
						}
					}
				}
				else if (FParse::Value(Str, TEXT("END "), ParsedText) && (ParsedText.ToUpper() == TEXT("BINARYBLOB")))
				{
					BulkData.Unlock();
					bBulkDataIsLocked = false;
					break;
				}
			}
		}
		else if (FParse::Value(Str, TEXT("END "), ParsedText) && (ParsedText.ToUpper() == TEXT("UNTYPEDBULKDATA")))
		{
			break;
		}
	}

	if (bBulkDataIsLocked == true)
	{
		BulkData.Unlock();
	}

	return true;
}


UObject* UFactory::CreateOrOverwriteAsset(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InTemplate) const
{
	// create an asset if it doesn't exist
	UObject* ExistingAsset = StaticFindObject(nullptr, InParent, *InName.ToString());
	
	if (!ExistingAsset)
	{
		return NewObject<UObject>(InParent, InClass, InName, InFlags, InTemplate);
	}

	// overwrite existing asset, if possible
	if (ExistingAsset->GetClass()->IsChildOf(InClass))
	{
		return NewObject<UObject>(InParent, InClass, InName, InFlags, InTemplate);
	}
	
	// otherwise delete and replace
	if (!ObjectTools::DeleteSingleObject(ExistingAsset))
	{
		UE_LOGF(LogFactory, Warning, "Could not delete existing asset %ls", *ExistingAsset->GetFullName());
		return nullptr;
	}

	// keep InPackage alive through the GC, in case ExistingAsset was the only reason it was around
	const bool bRootedPackage = InParent->IsRooted();

	if (!bRootedPackage)
	{
		InParent->AddToRoot();
	}

	// force GC so we can cleanly create a new asset (and not do an 'in place' replacement)
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	if (!bRootedPackage)
	{
		InParent->RemoveFromRoot();
	}

	// try to find the existing asset again now that the GC has occurred
	ExistingAsset = StaticFindObject(nullptr, InParent, *InName.ToString());

	// if the object is still around after GC, fail this operation
	if (ExistingAsset)
	{
		return nullptr;
	}

	// create the asset in the package
	return NewObject<UObject>(InParent, InClass, InName, InFlags, InTemplate);
}


FString UFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("New")) + GetSupportedClass()->GetName();
}

void UFactory::SetAutomatedAssetImportData(const UAutomatedAssetImportData* Data)
{
	AutomatedImportData = Data;
}

void UFactory::SetAssetImportTask(class UAssetImportTask* Task)
{
	AssetImportTask = Task;
}

bool UFactory::IsAutomatedImport() const
{
	return GIsAutomationTesting || AutomatedImportData || (AssetImportTask && AssetImportTask->bAutomated);
}
