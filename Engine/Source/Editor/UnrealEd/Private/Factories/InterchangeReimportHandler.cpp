// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/InterchangeReimportHandler.h"

#include "Animation/AnimSequence.h"
#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Factories/Factory.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeManager.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeReimportHandler)

#define LOCTEXT_NAMESPACE "InterchangeReimportHandler"

UInterchangeReimportHandler::UInterchangeReimportHandler(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//~ Begin FReimportHandler Interface
bool UInterchangeReimportHandler::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	const UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	return InterchangeManager.CanReimport(Obj, OutFilenames);
}

void UInterchangeReimportHandler::SetReimportPaths(UObject* Obj, const FString& NewReimportPath, const int32 SourceFileIndex)
{
	const UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	if (const UClass* FactoryClass = InterchangeManager.GetRegisteredFactoryClass(Obj->GetClass()))
	{
		UInterchangeFactoryBase* FactoryBase = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
		FactoryBase->BackupSourceData(Obj);
		FactoryBase->SetSourceFilename(Obj, NewReimportPath, SourceFileIndex);
	}
}

void UInterchangeReimportHandler::SetReimportSourceIndex(UObject* Obj, const int32 SourceIndex)
{
	const UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	if (const UClass* FactoryClass = InterchangeManager.GetRegisteredFactoryClass(Obj->GetClass()))
	{
		UInterchangeFactoryBase* FactoryBase = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
		FactoryBase->SetReimportSourceIndex(Obj, SourceIndex);
	}
}

EReimportResult::Type UInterchangeReimportHandler::Reimport(UObject* Obj, int32 SourceFileIndex)
{
	return EReimportResult::Failed;
}

int32 UInterchangeReimportHandler::GetPriority() const
{
	//We want a high priority to surpass other legacy re-import handlers
	return UFactory::GetDefaultImportPriority() + 10;
}

#undef LOCTEXT_NAMESPACE
