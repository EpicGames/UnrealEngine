// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/LightWeightInstanceFactory.h"
#include "GameFramework/LightWeightInstanceManager.h"
#include "Editor.h"

#include "KismetCompilerModule.h"
#include "ClassViewerFilter.h"

#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LightWeightInstanceFactory)

#define LOCTEXT_NAMESPACE "LightWeightInstanceFactory"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

class FLWIFilter : public IClassViewerFilter
{
public:
	/** Classes to not allow any children of into the Class Viewer/Picker. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) == EFilterReturn::Passed && !InClass->HasAnyClassFlags(CLASS_Deprecated);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) == EFilterReturn::Passed && !InUnloadedClassData->HasAnyClassFlags(CLASS_Deprecated);
	}
};

UDEPRECATED_LightWeightInstanceFactory::UDEPRECATED_LightWeightInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = ADEPRECATED_LightWeightInstanceManager::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UDEPRECATED_LightWeightInstanceFactory::ConfigureProperties()
{
	// Null the parent class to ensure one is selected
	ParentClass = nullptr;

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bShowObjectRootClass = true;
	Options.bIsBlueprintBaseOnly = false;

	// This will allow unloaded blueprints to be shown.
	Options.bShowUnloadedBlueprints = true;

	// Enable Class Dynamic Loading
	Options.bEnableClassDynamicLoading = true;

	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::Dynamic;

	// Prevent creating blueprints of classes that require special setup (they'll be allowed in the corresponding factories / via other means)
	TSharedPtr<FLWIFilter> Filter = MakeShareable(new FLWIFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->AllowedChildrenOfClasses.Add(ADEPRECATED_LightWeightInstanceManager::StaticClass());

	const FText TitleText = LOCTEXT("CreateBlueprintOptions", "Pick Parent Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, ADEPRECATED_LightWeightInstanceManager::StaticClass());

	if (bPressedOk)
	{
		ParentClass = ChosenClass;

		FEditorDelegates::OnFinishPickingBlueprintClass.Broadcast(ChosenClass);
	}

	return bPressedOk;
};

UObject* UDEPRECATED_LightWeightInstanceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UObject* LWIManager = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));

		UClass* BlueprintClass = nullptr;
		UClass* BlueprintGeneratedClass = nullptr;

		IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
		KismetCompilerModule.GetBlueprintTypesForClass(ParentClass, BlueprintClass, BlueprintGeneratedClass);

		LWIManager = FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BPTYPE_Normal, BlueprintClass, BlueprintGeneratedClass);
	}
	return LWIManager;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE // "LightWeightInstanceFactory"
