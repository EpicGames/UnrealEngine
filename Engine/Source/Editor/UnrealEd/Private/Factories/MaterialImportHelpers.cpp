// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MaterialImportHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialImportHelpers)

UMaterialInterface* UMaterialImportHelpers::FindExistingMaterialFromSearchLocation(const FString& MaterialFullName, const FString& BasePackagePath, EMaterialSearchLocation SearchLocation, FText& OutError)
{
	if (SearchLocation == EMaterialSearchLocation::DoNotSearch)
	{
		return nullptr;
	}

	//Search in memory
	UMaterialInterface* FoundMaterial = FindObject<UMaterialInterface>(nullptr, *MaterialFullName, EFindObjectFlags::None);

	if (FoundMaterial == nullptr)
	{
		FString SearchPath = FPaths::GetPath(BasePackagePath);
		
		// Search in asset's local folder
		FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, false, OutError);
		
		// Search recursively in asset's folder
		if (FoundMaterial == nullptr &&
			(SearchLocation != EMaterialSearchLocation::Local))
		{
			FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, true, OutError);
		}

		if (FoundMaterial == nullptr &&
			(	SearchLocation == EMaterialSearchLocation::UnderParent ||
				SearchLocation == EMaterialSearchLocation::UnderRoot ||
				SearchLocation == EMaterialSearchLocation::AllAssets))
		{
			// Search recursively in parent's folder
			SearchPath = FPaths::GetPath(SearchPath);
			if (!SearchPath.IsEmpty())
			{
				FoundMaterial = FindExistingMaterial(SearchPath, MaterialFullName, true, OutError);
			}
		}
		if (FoundMaterial == nullptr &&
			(	SearchLocation == EMaterialSearchLocation::UnderRoot ||
				SearchLocation == EMaterialSearchLocation::AllAssets))
		{
			// Search recursively in root folder of asset
			FString OutPackageRoot, OutPackagePath, OutPackageName;
			FPackageName::SplitLongPackageName(SearchPath, OutPackageRoot, OutPackagePath, OutPackageName);
			if (!SearchPath.IsEmpty())
			{
				FoundMaterial = FindExistingMaterial(OutPackageRoot, MaterialFullName, true, OutError);
			}
		}
		if (FoundMaterial == nullptr &&
			SearchLocation == EMaterialSearchLocation::AllAssets)
		{
			// Search everywhere
			FoundMaterial = FindExistingMaterial(TEXT("/"), MaterialFullName, true, OutError);
		}
	}

	return FoundMaterial;
}

UMaterialInterface* UMaterialImportHelpers::FindExistingMaterial(const FString& BasePath, const FString& MaterialFullName, const bool bRecursivePaths, FText& OutError)
{
	UMaterialInterface* Material = nullptr;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Finish/update any scans
	TArray<FString> ScanPaths;
	if (BasePath.IsEmpty() || BasePath == TEXT("/"))
	{
		FPackageName::QueryRootContentPaths(ScanPaths);
	}
	else
	{ 
		ScanPaths.Add(BasePath);
	}
	const bool bForceRescan = false;
	AssetRegistry.ScanPathsSynchronous(ScanPaths, bForceRescan);


	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.bRecursivePaths = bRecursivePaths;
	Filter.ClassPaths.Add(UMaterialInterface::StaticClass()->GetClassPathName());
	Filter.PackagePaths.Add(FName(*BasePath));

	TArray<FAssetData> AssetData;
	AssetRegistry.GetAssets(Filter, AssetData);

	TArray<UMaterialInterface*> FoundMaterials;
	for (const FAssetData& Data : AssetData)
	{
		if (Data.AssetName == FName(*MaterialFullName))
		{
			Material = Cast<UMaterialInterface>(Data.GetAsset());
			if (Material != nullptr)
			{
				FoundMaterials.Add(Material);
			}
		}
	}

	if (FoundMaterials.Num() > 1)
	{
		check(Material != nullptr);
		OutError =
			FText::Format(NSLOCTEXT("MaterialImportHelpers", "MultipleMaterialsFound", "Found {0} materials matching name '{1}'. Using '{2}'."),
				FText::FromString(FString::FromInt(FoundMaterials.Num())),
				FText::FromString(MaterialFullName),
				FText::FromString(Material->GetOutermost()->GetName()));
	}
	return Material;
}
