// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MeshDeformerCollectionFactory.h"
#include "AssetTypeCategories.h"
#include "Animation/MeshDeformerCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshDeformerCollectionFactory)


#define LOCTEXT_NAMESPACE "MeshDeformerCollectionFactory"

UMeshDeformerCollectionFactory::UMeshDeformerCollectionFactory()
{
	SupportedClass = UMeshDeformerCollection::StaticClass();
	bEditAfterNew = true;
	bCreateNew = true;
}



UObject* UMeshDeformerCollectionFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* Context,
                                                          FFeedbackContext* Warn)
{
	UMeshDeformerCollection* MeshDeformerCollection = NewObject<UMeshDeformerCollection>(InParent, InClass, InName, InFlags);
	return MeshDeformerCollection;
}

FText UMeshDeformerCollectionFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Mesh Deformer Collection");
}

FText UMeshDeformerCollectionFactory::GetToolTip() const
{
	return LOCTEXT("ToolTip", "A simple collection of Mesh Deformers primarily used by Skeletal Mesh Asset to determined if extra deformer specific data should be built");
}

uint32 UMeshDeformerCollectionFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

const TArray<FText>& UMeshDeformerCollectionFactory::GetMenuCategorySubMenus() const
{
	static TArray<FText> SubMenus { LOCTEXT("SubMenuDeformers", "Deformers") };
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
