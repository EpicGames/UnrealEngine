// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/MirrorDataTableFactory.h"
#include "Animation/Skeleton.h"
#include "DataTableEditorUtils.h"
#include "Animation/MirrorDataTable.h"
#include "Animation/AnimationSettings.h"
#include "Editor.h"
#include "StructViewerModule.h"
#include "StructViewerFilter.h"

#include "PropertyEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MirrorDataTableFactory)

#define LOCTEXT_NAMESPACE "MirrorDataTableFactory"

DECLARE_DELEGATE_FourParams(FOnMirrorTableConfigureUserAction, bool /*bCreate*/, USkeleton* /*InSkeleton*/, UMirrorTableFindReplaceExpressions* /*InRegularExpressions*/, const UScriptStruct* /*InPoseNames*/);

class FDataTableStructFilter : public IStructViewerFilter
{
public:
	virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		return FDataTableEditorUtils::IsValidTableStruct(InStruct) && InStruct->FindPropertyByName(TEXT("MirroredName")) != nullptr;
	}

	virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		// Unloaded structs are always User Defined Structs, and User Defined Structs are always allowed
		// They will be re-validated by IsStructAllowed once loaded during the pick
		//return FDataTableEditorUtils::IsValidTableStruct(InStruct) && InStruct->FindPropertyByName(TEXT("MirroredName")) != nullptr;

		return false;
	}
};

class SMirrorDataTableFactoryWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SMirrorDataTableFactoryWindow)
	{}

	SLATE_ARGUMENT(USkeleton*, Skeleton)
	SLATE_ARGUMENT(UMirrorTableFindReplaceExpressions*, FindReplaceExpressions)
	SLATE_ARGUMENT(FOnMirrorTableConfigureUserAction, UserActionHandler)
	SLATE_ARGUMENT(FSimpleDelegate, OnCreateCanceled)
		
	SLATE_END_ARGS()
public:
	FReply OnCreate()
	{
		check(ResultStruct);
		if (PickerWindow.IsValid())
		{
			PickerWindow->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		if (UserActionHandler.IsBound())
		{
			UserActionHandler.Execute(false, nullptr,nullptr, nullptr);
		}

		RequestDestroyWindow();
		return FReply::Handled();
	}

	bool IsStructSelected() const
	{
		return ResultStruct != nullptr;
	}

	void OnPickedStruct(const UScriptStruct* ChosenStruct)
	{
		ResultStruct = ChosenStruct;
		StructPickerAnchor->SetIsOpen(false);
	}

	FText OnGetComboTextValue() const
	{
		return ResultStruct
			? FText::AsCultureInvariant(ResultStruct->GetName())
			: LOCTEXT("None", "None");
	}

	TSharedRef<SWidget> GenerateStructPicker()
	{
		FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

		// Fill in options
		FStructViewerInitializationOptions Options;
		Options.Mode = EStructViewerMode::StructPicker;
		Options.StructFilter = MakeShared<FDataTableStructFilter>();
		return
			SNew(SBox)
			.WidthOverride(330.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.MaxHeight(500)
				[
					SNew(SBorder)
					.Padding(4.0f)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateSP(this, &SMirrorDataTableFactoryWindow::OnPickedStruct))
					]
				]
			];
	}


	void Construct(const FArguments& InArgs)
	{
		UserActionHandler = InArgs._UserActionHandler;
		Skeleton = InArgs._Skeleton;
		FindReplaceExpressions = InArgs._FindReplaceExpressions;
		ResultStruct = FMirrorTableRow::StaticStruct();
		FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");
		

		// Load the content browser module to display an asset picker
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		FAssetPickerConfig AssetPickerConfig;
		/** The asset picker will only show skeletons */
		AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
		/** The delegate that fires when an asset was selected */
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &SMirrorDataTableFactoryWindow::OnSkeletonSelected);
		if (Skeleton != nullptr)
		{
			AssetPickerConfig.InitialAssetSelection = FAssetData(Skeleton);
		}

		// Create a property view
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bShowOptions = false;

		PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
		PropertyView->SetObject(FindReplaceExpressions);
		PropertyView->SetClipping(EWidgetClipping::ClipToBounds);

		/** The default view mode should be a list view */
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

		SWindow::Construct(SWindow::FArguments()
			.Title(LOCTEXT("CreateMirrorDataTableOptions", "Create Mirror Data Table Asset"))
			.SizingRule(ESizingRule::UserSized)
			.ClientSize(FVector2D(500, 600))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("Menu.Background") )
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(0.6)
				.Padding(3, 3)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 2)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Select Skeleton")))
					]
					+SVerticalBox::Slot()
					.Padding(2, 2)
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(300)
				[
					SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("CreateMirrorDataTableAdvancedOptions", "Advanced Options"))
					.Padding(2.0f)
					.InitiallyCollapsed(true)
					.BodyContent()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("MirrorRowStructure", "Row Structure"))
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SAssignNew(StructPickerAnchor, SComboButton)
								.ContentPadding(FMargin(2, 2, 2, 1))
								.MenuPlacement(MenuPlacement_BelowAnchor)
								.ButtonContent()
								[
									SNew(STextBlock)
									.Text(this, &SMirrorDataTableFactoryWindow::OnGetComboTextValue)
								]
								.OnGetMenuContent(this, &SMirrorDataTableFactoryWindow::GenerateStructPicker)
							]
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0)
						[
							SNew(SScrollBox)
							.Orientation(Orient_Vertical)
							+SScrollBox::Slot()
							[
								PropertyView.ToSharedRef()
							]
						]
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(3,3)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.Text(LOCTEXT("Accept", "Accept"))
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.IsEnabled(this, &SMirrorDataTableFactoryWindow::CanAccept)
						.OnClicked_Raw(this, &SMirrorDataTableFactoryWindow::OnAccept)
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked_Raw(this, &SMirrorDataTableFactoryWindow::OnCancel)
					]
				]
			]
		]);
	}

	bool CanAccept() const
	{
		return Skeleton != nullptr &&  ResultStruct != nullptr && UserActionHandler.IsBound();
	}

	FReply OnAccept()
	{
		if (CanAccept())
		{
			UserActionHandler.Execute(true, Skeleton, FindReplaceExpressions, ResultStruct);
		}

		RequestDestroyWindow();
		return FReply::Handled();
	}

	void OnSkeletonSelected(const FAssetData& SelectedAsset)
	{
		Skeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
	}


	USkeleton*					Skeleton;
	TSharedPtr<SWindow>			PickerWindow;
	TSharedPtr<SComboButton>	StructPickerAnchor;
	FOnMirrorTableConfigureUserAction	UserActionHandler;
	TSharedPtr<class IDetailsView> PropertyView;
	UMirrorTableFindReplaceExpressions* FindReplaceExpressions;
	const UScriptStruct* ResultStruct = nullptr;
};

UMirrorTableFindReplaceExpressions::UMirrorTableFindReplaceExpressions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UAnimationSettings* AnimationSettings = UAnimationSettings::Get(); 
	if (AnimationSettings)
	{
		FindReplaceExpressions = AnimationSettings->MirrorFindReplaceExpressions;
	}
}

UMirrorDataTableFactory::UMirrorDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), Struct(nullptr), Skeleton(nullptr)
{
	SupportedClass = UMirrorDataTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	MirrorFindReplaceExpressions = NewObject<UMirrorTableFindReplaceExpressions>();
}

UMirrorDataTable* UMirrorDataTableFactory::MakeNewMirrorDataTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	return NewObject<UMirrorDataTable>(InParent, Name, Flags);
}

UObject* UMirrorDataTableFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMirrorDataTable* DataTable = nullptr;
	if (Struct && ensure(SupportedClass == Class) && Skeleton)
	{
		ensure(0 != (RF_Public & Flags));
		DataTable = MakeNewMirrorDataTable(InParent, Name, Flags);
		if (DataTable)
		{
			DataTable->RowStruct = const_cast<UScriptStruct*>(ToRawPtr(Struct));
			DataTable->Skeleton = Skeleton; 
			if (MirrorFindReplaceExpressions)
			{
				DataTable->MirrorFindReplaceExpressions = MirrorFindReplaceExpressions->FindReplaceExpressions;
			}
			if (Skeleton)
			{
				// Sync fully so the table is considered up to date on first open.
				DataTable->UpdateFromFindReplaceExpressions(UMirrorDataTable::FFindReplaceOptions::Sync());
			}
		}
	}
	return DataTable;
}

FText UMirrorDataTableFactory::GetDisplayName() const
{
	return LOCTEXT("MirrorDataTableFactoryDescription", "Mirror Data Table");
}

uint32 UMirrorDataTableFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

bool UMirrorDataTableFactory::ConfigureProperties()
{
	TSharedPtr<SWindow> PickerWindow = SNew(SMirrorDataTableFactoryWindow)
		.Skeleton(Skeleton)
		.FindReplaceExpressions(MirrorFindReplaceExpressions)
		.UserActionHandler(FOnMirrorTableConfigureUserAction::CreateUObject(this, &UMirrorDataTableFactory::OnWindowUserActionDelegate));

	// have to clear after setting it because you could use close button without clicking cancel
	Skeleton = nullptr;

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return Skeleton != nullptr && Struct != nullptr;
}

void  UMirrorDataTableFactory::OnWindowUserActionDelegate(bool bCreate, USkeleton* InSkeleton, UMirrorTableFindReplaceExpressions* InFindReplaceExpressions, const UScriptStruct* InResultStruct)
{
	if (bCreate)
	{
		Skeleton = InSkeleton;
		MirrorFindReplaceExpressions = InFindReplaceExpressions;
		Struct = InResultStruct; 
	}
}

#undef LOCTEXT_NAMESPACE // "MirrorDataTableFactory"
