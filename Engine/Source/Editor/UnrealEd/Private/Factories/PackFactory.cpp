// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackFactory.cpp: Factory for importing asset and feature packs
=============================================================================*/

#include "Factories/PackFactory.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "Math/GuardedInt.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Compression.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/LinkerLoad.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "GameFramework/PlayerInput.h"
#include "GameFramework/InputSettings.h"
#include "IPlatformFilePak.h"
#include "SourceCodeNavigation.h"
#include "Misc/HotReloadInterface.h"
#include "Misc/AES.h"
#include "Hash/xxhash.h"
#include "GameProjectGenerationModule.h"
#include "Dialogs/SOutputLogDialog.h"
#include "Templates/UniquePtr.h"
#include "Logging/MessageLog.h"
#include "Misc/CoreDelegates.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackFactory)

DEFINE_LOG_CATEGORY_STATIC(LogPackFactory, Log, All);

UPackFactory::UPackFactory(const FObjectInitializer& PCIP)
	: Super(PCIP)
{
	// Since this factory can output multiple and any number of class it doesn't really have a 
	// SupportedClass per say, but one must be defined, so we just reference ourself
	SupportedClass = UPackFactory::StaticClass();

	Formats.Add(TEXT("upack;Asset Pack"));
	Formats.Add(TEXT("upack;Feature Pack"));

	bEditorImport = true;
}

namespace PackFactoryHelper
{
	// Gate for extracting C++ source from a .upack into the user's project.
	// When true, the user is still prompted with a security warning before any file is
	// written, and no auto-compile is performed.
	static bool GAllowSourceCodeImport = false;
	static FAutoConsoleVariableRef CVarAllowSourceCodeImport(
		TEXT("PackFactory.AllowSourceCodeImport"),
		GAllowSourceCodeImport,
		TEXT("If true, .upack imports may extract C++ source into the project."),
		ECVF_Default);

	// Utility function to copy a single pak entry out of the Source archive and in to the Destination archive using Buffer as temporary space
	bool BufferedCopyFile(FArchive& DestAr, FArchive& Source, const FPakEntry& Entry, TArray<uint8>& Buffer, const FPakFile& PakFile)
	{	
		// Align down
		const int64 BufferSize = Buffer.Num() & ~(FAES::AESBlockSize - 1);
		int64 RemainingSizeToCopy = Entry.Size;
		while (RemainingSizeToCopy > 0)
		{
			const int64 SizeToCopy = FMath::Min(BufferSize, RemainingSizeToCopy);
			// If file is encrypted so we need to account for padding
			int64 SizeToRead = Entry.IsEncrypted() ? Align(SizeToCopy, FAES::AESBlockSize) : SizeToCopy;

			Source.Serialize(Buffer.GetData(), SizeToRead);
			if (Entry.IsEncrypted())
			{
				FAES::FAESKey Key;
				FPakPlatformFile::GetPakEncryptionKey(Key, PakFile.GetInfo().EncryptionKeyGuid);
				checkf(Key.IsValid(), TEXT("Trying to copy an encrypted file between pak files, but no decryption key is available"));
				FAES::DecryptData(Buffer.GetData(), IntCastChecked<uint32>(SizeToRead), Key);
			}
			DestAr.Serialize(Buffer.GetData(), SizeToCopy);
			RemainingSizeToCopy -= SizeToRead;
		}
		return true;
	}

	// Utility function to uncompress and copy a single pak entry out of the Source archive and in to the Destination archive using PersistentBuffer as temporary space
	bool UncompressCopyFile(FArchive& DestAr, FArchive& Source, const FPakEntry& Entry, TArray<uint8>& PersistentBuffer, const FPakFile& PakFile)
	{
		// Entry is untrusted data.
		if (Entry.UncompressedSize <= 0)
		{
			return false;
		}

		TOptional<FName> CompressionMethod = PakFile.GetInfo().TryGetCompressionMethod(Entry.CompressionMethodIndex);
		if (CompressionMethod.IsSet() == false)
		{
			return false;
		}

		FGuardedInt32 GuardedWorkingSize = FGuardedInt32(Entry.CompressionBlockSize);
		int64 MaxCompressionBlockSize64 = 0;
		if (!FCompression::GetMaximumCompressedSize(CompressionMethod.GetValue(), MaxCompressionBlockSize64, GuardedWorkingSize.Get(0)))
		{
			return false;
		}

		if (MaxCompressionBlockSize64 < 0)
		{
			return false;
		}

		GuardedWorkingSize += MaxCompressionBlockSize64;
		int32 WorkingSize = GuardedWorkingSize.Get(0);
		if (WorkingSize <= 0)
		{
			return false;
		}

		// WorkingSize is now a sanitized int32 value.
		if (PersistentBuffer.Num() < WorkingSize)
		{
			PersistentBuffer.SetNumUninitialized(WorkingSize);
		}

		for (uint32 BlockIndex = 0, BlockIndexNum = Entry.CompressionBlocks.Num(); BlockIndex < BlockIndexNum; ++BlockIndex)
		{
			FGuardedInt64 CompressedBlockSize64 = FGuardedInt64(Entry.CompressionBlocks[BlockIndex].CompressedEnd) - Entry.CompressionBlocks[BlockIndex].CompressedStart;
			if (CompressedBlockSize64.ValidAndGreaterThan(0) == false ||
				IntFitsIn<int32>(CompressedBlockSize64.Get(0)) == false)
			{
				return false;
			}

			int32 CompressedBlockSize = static_cast<int32>(CompressedBlockSize64.Get(0));
			// CompressedBlockSize now sanitized


			FGuardedInt64 UncompressedBlockSize64 = FGuardedInt64(Entry.UncompressedSize) - FGuardedInt64(Entry.CompressionBlockSize) * BlockIndex;
			if (UncompressedBlockSize64.ValidAndGreaterThan(0) == false ||
				IntFitsIn<uint32>(UncompressedBlockSize64.Get(0)) == false)
			{
				return false;
			}

			// CompressionBlockSize is guaranteed to fit in int32 from earlier, so we know after the Min() we can fit in uint32
			uint32 UncompressedBlockSize = (uint32)FMath::Min<int64>(UncompressedBlockSize64.Get(0), Entry.CompressionBlockSize);

			if (Entry.Offset < 0)
			{
				return false;
			}

			FGuardedInt64 OffsetInPak = FGuardedInt64(Entry.CompressionBlocks[BlockIndex].CompressedStart) + (PakFile.GetInfo().HasRelativeCompressedChunkOffsets() ? Entry.Offset : 0);
			if (OffsetInPak.IsValid() == false)
			{
				return false;
			}

			Source.Seek(OffsetInPak.Get(0));
			int32 SizeToRead = Entry.IsEncrypted() ? Align(CompressedBlockSize, FAES::AESBlockSize) : CompressedBlockSize;
			if (SizeToRead > PersistentBuffer.Num())
			{
				// Shouldn't ever happen but I think this is possible if we are uncompressed and for some reason the block size isn't aligned to AESBlockSize.
				return false;
			}
			Source.Serialize(PersistentBuffer.GetData(), SizeToRead);

			if (Entry.IsEncrypted())
			{
				FAES::FAESKey Key;
				FPakPlatformFile::GetPakEncryptionKey(Key, PakFile.GetInfo().EncryptionKeyGuid);
				checkf(Key.IsValid(), TEXT("Trying to copy an encrypted file between pak files, but no decryption key is available"));
				FAES::DecryptData(PersistentBuffer.GetData(), SizeToRead, Key);
			}

			uint8* UncompressedBuffer = PersistentBuffer.GetData() + MaxCompressionBlockSize64;
			if (!FCompression::UncompressMemory(CompressionMethod.GetValue(), UncompressedBuffer, UncompressedBlockSize, PersistentBuffer.GetData(), CompressedBlockSize))
			{
				return false;
			}
			DestAr.Serialize(UncompressedBuffer, UncompressedBlockSize);
		}

		return true;
	}

	// Utility function to extract a pak entry out of the memory reader containing the pak file and place in the destination archive.
	// Uses Buffer or PersistentCompressionBuffer depending on whether the entry is compressed or not.
	void ExtractFile(const FPakEntry& Entry, FBufferReader& PakReader, TArray<uint8>& Buffer, TArray<uint8>& PersistentCompressionBuffer, FArchive& DestAr, const FPakFile& PakFile)
	{
		// 0 is uncompressed
		if (Entry.CompressionMethodIndex == 0)
		{
			PackFactoryHelper::BufferedCopyFile(DestAr, PakReader, Entry, Buffer, PakFile);
		}
		else
		{
			PackFactoryHelper::UncompressCopyFile(DestAr, PakReader, Entry, PersistentCompressionBuffer, PakFile);
		}
	}

	// Utility function to extract a pak entry out of the memory reader containing the pak file and place in a string.
	// Uses Buffer or PersistentCompressionBuffer depending on whether the entry is compressed or not.
	void ExtractFileToString(const FPakEntry& Entry, FBufferReader& PakReader, TArray<uint8>& Buffer, TArray<uint8>& PersistentCompressionBuffer, FString& FileContents, const FPakFile& PakFile)
	{
		TArray<uint8> Contents;
		FMemoryWriter MemWriter(Contents);

		ExtractFile(Entry, PakReader, Buffer, PersistentCompressionBuffer, MemWriter, PakFile);

		// Add a line feed at the end because the FString archive read will consume the last byte
		Contents.Add('\n');

		// Insert the length of the string to the front of the memory chunk so we can use FString archive read
		const int32 StringLength = Contents.Num();
		Contents.InsertUninitialized(0,sizeof(int32));
		*(reinterpret_cast<int32*>(Contents.GetData())) = StringLength;

		FMemoryReader MemReader(Contents);
		MemReader << FileContents;
	}

	// Utility function to return a safe destination path for extracting a pak entry.
	// By default it just returns the normalized combined path of NormalizedDestRoot and RelativeDestFilename.
	// If the normalized path would escape NormalizedDestRoot (e.g. via path traversal), the file is redirected to a quarantine subfolder
	// generated from a hash of the original EntryFileName. 
	FString MakeSafeDestPath(const FString& NormalizedDestRoot, const FString& RelativeDestFilename, const FString& EntryFilename)
	{
		FString DestFilename = FPaths::ConvertRelativePathToFull(NormalizedDestRoot / RelativeDestFilename);

		if (!FPaths::IsUnderDirectory(DestFilename, NormalizedDestRoot))
		{
			FString EntryPath = FPaths::GetPath(EntryFilename);
			uint64 EntryHash = FXxHash64::HashBuffer(*EntryPath, EntryPath.Len() * sizeof(TCHAR)).Hash;
			FString HashDir = FString::Printf(TEXT("%016llx"), EntryHash);
			FString SafeDestFilename = FPaths::ConvertRelativePathToFull(NormalizedDestRoot / TEXT("Quarantined") / HashDir / FPaths::GetCleanFilename(EntryFilename));
			FMessageLog("AssetTools").Error(FText::Format(
				NSLOCTEXT("PackFactory", "PathTraversalDetected", "Path traversal detected for entry \"{0}\": \"{1}\" is outside destination root. Quarantined to \"{2}\"."),
				FText::FromString(EntryFilename), FText::FromString(DestFilename), FText::FromString(SafeDestFilename)));
			DestFilename = SafeDestFilename;
		}

		return DestFilename;
	}

	struct FPackConfigParameters
	{
		FPackConfigParameters()
			: bContainsSource(false)
			, bCompileSource(false)
		{
		}

		uint8 bContainsSource:1;
		uint8 bCompileSource:1;
		FString GameName;
		FString InstallMessage;
		TArray<FString> AdditionalFilesToAdd;
	};

	// Takes a string that represents the contents of a config file and sets up the supported config properties based on it
	// Currently we support Action and Axis Mappings and a GameName (for setting up redirects)
	void ProcessPackConfig(const FString& ConfigString, FPackConfigParameters& ConfigParameters)
	{
		FConfigFile PackConfig;
		PackConfig.ProcessInputFileContents(ConfigString, TEXT("Unknown (see PackFactoryHelper::ProcessPackConfig)"));

		// Input Settings
		static FArrayProperty* ActionMappingsProp = FindFieldChecked<FArrayProperty>(UInputSettings::StaticClass(), UInputSettings::GetActionMappingsPropertyName());
		static FArrayProperty* AxisMappingsProp = FindFieldChecked<FArrayProperty>(UInputSettings::StaticClass(), UInputSettings::GetAxisMappingsPropertyName());

		UInputSettings* InputSettingsCDO = GetMutableDefault<UInputSettings>();
		bool bCheckedOut = false;

		const FConfigSection* InputSettingsSection = PackConfig.FindSection("InputSettings");
		if (InputSettingsSection)
		{
			TArray<FInputActionKeyMapping> ActionMappingsToAdd;
			TArray<FInputAxisKeyMapping> AxisMappingsToAdd;

			for (auto SettingPair : *InputSettingsSection)
			{
				

				if (SettingPair.Key.ToString().Contains("ActionMappings"))
				{
					FInputActionKeyMapping ActionKeyMapping;
					ActionMappingsProp->Inner->ImportText_Direct(*SettingPair.Value.GetValue(), &ActionKeyMapping, nullptr, PPF_None);

					if (!InputSettingsCDO->DoesActionExist(ActionKeyMapping.ActionName))
					{
						ActionMappingsToAdd.Add(ActionKeyMapping);
					}
				}
				else if (SettingPair.Key.ToString().Contains("AxisMappings"))
				{
					FInputAxisKeyMapping AxisKeyMapping;
					AxisMappingsProp->Inner->ImportText_Direct(*SettingPair.Value.GetValue(), &AxisKeyMapping, nullptr, PPF_None);

					if (!InputSettingsCDO->DoesAxisExist(AxisKeyMapping.AxisName))
					{
						AxisMappingsToAdd.Add(AxisKeyMapping);
					}
				}
			}

			if (ActionMappingsToAdd.Num() > 0 || AxisMappingsToAdd.Num() > 0)
			{
				if (ISourceControlModule::Get().IsEnabled())
				{
					FText ErrorMessage;

					const FString InputSettingsFilename = FPaths::ConvertRelativePathToFull(InputSettingsCDO->GetDefaultConfigFilename());
					if (!SourceControlHelpers::CheckoutOrMarkForAdd(InputSettingsFilename, FText::FromString(InputSettingsFilename), NULL, ErrorMessage))
					{
						UE_LOGF(LogPackFactory, Error, "%ls", *ErrorMessage.ToString());
					}
				}

				for (const FInputActionKeyMapping& ActionKeyMapping : ActionMappingsToAdd)
				{
					InputSettingsCDO->AddActionMapping(ActionKeyMapping);
				}
				for (const FInputAxisKeyMapping& AxisKeyMapping : AxisMappingsToAdd)
				{
					InputSettingsCDO->AddAxisMapping(AxisKeyMapping);
				}
					
				InputSettingsCDO->SaveKeyMappings();
				InputSettingsCDO->TryUpdateDefaultConfigFile();
			}
		}

		const FConfigSection* RedirectsSection = PackConfig.FindSection("Redirects");
		if (RedirectsSection)
		{	
			if (const FConfigValue* GameName = RedirectsSection->Find("GameName"))
			{
				ConfigParameters.GameName = GameName->GetValue();
			}
		}

		const FConfigSection* AdditionalFilesSection = PackConfig.FindSection("AdditionalFilesToAdd");
		if (AdditionalFilesSection)
		{
			for (auto FilePair : *AdditionalFilesSection)
			{
				if (FilePair.Key.ToString().Contains("Files"))
				{
					FString Filename = FPaths::GetCleanFilename(FilePair.Value.GetValue());
					FString Directory = FPaths::RootDir() / FPaths::GetPath(FilePair.Value.GetValue());
					FPaths::MakeStandardFilename(Directory);
					FPakFile::MakeDirectoryFromPath(Directory);

					if (Filename.Contains(TEXT("*")))
					{
						TArray<FString> FoundFiles;
						IFileManager::Get().FindFilesRecursive(FoundFiles, *Directory, *Filename, true, false);
						ConfigParameters.AdditionalFilesToAdd.Append(FoundFiles);
						
					}
					else
					{
						ConfigParameters.AdditionalFilesToAdd.Add(Directory / Filename);
					}
				}
			}
		}

		const FConfigSection* FeaturePackSettingsSection = PackConfig.FindSection("FeaturePackSettings");
		if (FeaturePackSettingsSection)
		{
			if (const FConfigValue* CompileSource = FeaturePackSettingsSection->Find("CompileSource"))
			{
				UE_LOGF(LogPackFactory, Warning, "Modifying CompileSource setting directly from the pack import is deprecated");
			}
			if (const FConfigValue* InstallMessage = FeaturePackSettingsSection->Find("InstallMessage"))
			{
				ConfigParameters.InstallMessage = InstallMessage->GetValue();
			}
		}
	}
}

UObject* UPackFactory::FactoryCreateBinary
(
	UClass*				Class,
	UObject*			InParent,
	FName				Name,
	EObjectFlags		Flags,
	UObject*			Context,
	const TCHAR*		FileType,
	const uint8*&		Buffer,
	const uint8*		BufferEnd,
	FFeedbackContext*	Warn
)
{ 
	FBufferReader PakReader((void*)Buffer, BufferEnd-Buffer, false);
	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&PakReader);
	FPakFile& PakFile = *PakFilePtr;

	UObject* ReturnAsset = nullptr;

	if (PakFile.IsValid() && PakFile.HasFilenames())
	{
		static FString ContentFolder(TEXT("/Content/"));
		FString ContentDestinationRoot = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());

		const int32 ChopIndex = PakFile.GetMountPoint().Find(ContentFolder);
		if (ChopIndex != INDEX_NONE)
		{
			FString RelativeRoot = PakFile.GetMountPoint().RightChop(ChopIndex + ContentFolder.Len());
			FString NewDestinationRoot = FPaths::ConvertRelativePathToFull(ContentDestinationRoot / RelativeRoot);
			if (FPaths::IsUnderDirectory(NewDestinationRoot, ContentDestinationRoot))
			{
				ContentDestinationRoot = MoveTemp(NewDestinationRoot);
			}
			else
			{
				ContentDestinationRoot /= TEXT("Quarantined");
				FMessageLog("AssetTools").Error(FText::Format(
					NSLOCTEXT("PackFactory", "MountPointPathTraversalDetected", "Path traversal detected for mount point \"{0}\": \"{1}\" is outside the content folder. Quarantined to \"{2}\"."),
					FText::FromString(PakFile.GetMountPoint()), FText::FromString(NewDestinationRoot), FText::FromString(ContentDestinationRoot)));
			}
		}

		TArray<uint8> CopyBuffer;
		TArray<uint8> PersistentCompressionBuffer;
		CopyBuffer.AddUninitialized(8 * 1024 * 1024); // 8MB buffer for extracting
		int32 ErrorCount = 0;
		int32 FileCount = 0;
		int32 SkipCount = 0;

		FModuleContextInfo SourceModuleInfo;
		PackFactoryHelper::FPackConfigParameters ConfigParameters;

		TArray<FString> WrittenFiles;
		TArray<FString> WrittenSourceFiles;

		// Process the config files and identify if we have source files
		for (FPakFile::FPakEntryIterator It(PakFile); It; ++It)
		{
			const FString* EntryFilename = It.TryGetFilename();
			check(EntryFilename);
			if (EntryFilename->StartsWith(TEXT("Config/")) || EntryFilename->Contains(TEXT("/Config/")))
			{
				const FPakEntry& Entry = It.Info();
				PakReader.Seek(Entry.Offset);
				FPakEntry EntryInfo;
				EntryInfo.Serialize(PakReader, PakFile.GetInfo().Version);

				if (EntryInfo.IndexDataEquals(Entry))
				{
					FString ConfigString;
					PackFactoryHelper::ExtractFileToString(Entry, PakReader, CopyBuffer, PersistentCompressionBuffer, ConfigString, PakFile);
					PackFactoryHelper::ProcessPackConfig(ConfigString, ConfigParameters);
				}
				else
				{
					UE_LOGF(LogPackFactory, Error, "Index data mismatch for entry: \"%ls\".", **EntryFilename);
					ErrorCount++;
				}
			}
			else if (EntryFilename->StartsWith(TEXT("Source/")) || EntryFilename->Contains(TEXT("/Source/")))
			{
				ConfigParameters.bContainsSource = true;
				// Use written source file list to present to information to user prior to actually writing the files
				WrittenSourceFiles.Add(*EntryFilename);
			}
		}
		for (const FString& File : ConfigParameters.AdditionalFilesToAdd)
		{
			if (File.StartsWith(TEXT("Source/")) || File.Contains(TEXT("/Source/")))
			{
				ConfigParameters.bContainsSource = true;
				WrittenSourceFiles.Add(File);
			}
		}

		if (ConfigParameters.bContainsSource)
		{
			if (!PackFactoryHelper::GAllowSourceCodeImport)
			{
				UE_LOGF(LogPackFactory, Warning,
					"Pack \"%ls\" contains C++ source files, but PackFactory.AllowSourceCodeImport is disabled. Source entries will be skipped.",
					*Name.ToString());
				ConfigParameters.bContainsSource = false;
			}
			else
			{
				constexpr int32 MaxListed = 10;
				TStringBuilder<512> SourceList;
				const int32 NumToList = FMath::Min(WrittenSourceFiles.Num(), MaxListed);
				for (int32 Index = 0; Index < NumToList; ++Index)
				{
					SourceList.Appendf(TEXT("\n    %s"), *WrittenSourceFiles[Index]);
				}
				if (WrittenSourceFiles.Num() > MaxListed)
				{
					SourceList.Appendf(TEXT("\n    ... and %d more"), WrittenSourceFiles.Num() - MaxListed);
				}

				const FText WarningText = FText::Format(
					NSLOCTEXT("PackFactory", "SourceImportConfirmPrompt",
						"The .upack \"{0}\" contains {1} C++ source file(s) that will be written into your project. Only proceed if you trust the origin of this pack.\n\nFiles to be written:{2}\n\nAllow these source files to be written into the project?"),
					FText::FromName(Name),
					FText::AsNumber(WrittenSourceFiles.Num()),
					FText::FromString(SourceList.ToString()));

				if (FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::No, WarningText) != EAppReturnType::Yes)
				{
					UE_LOGF(LogPackFactory, Warning,
						"User declined source-code import for pack \"%ls\". Source entries skipped.",
						*Name.ToString());
					ConfigParameters.bContainsSource = false;
				}
			}
			WrittenSourceFiles.Empty();
		}

		bool bProjectHadSourceFiles = false;

		// If we have source files, set up the project files if necessary and the game name redirects for blueprints saved with class
		// references to the module name from the source template
		if (ConfigParameters.bContainsSource)
		{
			FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
			bProjectHadSourceFiles = GameProjectModule.Get().ProjectHasCodeFiles();

			if (!bProjectHadSourceFiles)
			{
				TArray<FString> StartupModuleNames;
				TArray<FString> CreatedFiles;
				FText OutFailReason;
				if ( GameProjectModule.Get().GenerateBasicSourceCode(CreatedFiles, OutFailReason) )
				{
					WrittenFiles.Append(CreatedFiles);
				}
				else
				{
					UE_LOGF(LogPackFactory, Error, "Unable to create basic source code: '%ls'", *OutFailReason.ToString());
				}
			}

			for (const FModuleContextInfo& ModuleInfo : GameProjectModule.Get().GetCurrentProjectModules())
			{
				// Pick the module to insert the code in.  For now always pick the first Runtime module
				if (ModuleInfo.ModuleType == EHostType::Runtime)
				{
					SourceModuleInfo = ModuleInfo;

					// Setup the game name redirect
					if (!ConfigParameters.GameName.IsEmpty())
					{
						const FString EngineIniFilename = FPaths::ConvertRelativePathToFull(GetDefault<UEngine>()->GetDefaultConfigFilename());

						if (ISourceControlModule::Get().IsEnabled())
						{
							FText ErrorMessage;

							if (!SourceControlHelpers::CheckoutOrMarkForAdd(EngineIniFilename, FText::FromString(EngineIniFilename), NULL, ErrorMessage))
							{
								UE_LOGF(LogPackFactory, Error, "%ls", *ErrorMessage.ToString());
							}
						}

						const FString RedirectsSection(TEXT("/Script/Engine.Engine"));
						const FString LongOldGameName = FString::Printf(TEXT("/Script/%s"), *ConfigParameters.GameName);
						const FString LongNewGameName = FString::Printf(TEXT("/Script/%s"), *ModuleInfo.ModuleName);
						
						FConfigCacheIni Config(EConfigCacheType::Temporary);
						FConfigFile& NewFile = Config.Add(EngineIniFilename, FConfigFile());
						FConfigCacheIni::LoadLocalIniFile(NewFile, TEXT("DefaultEngine"), false);

						NewFile.AddToSection(*RedirectsSection, TEXT("+ActiveGameNameRedirects"), FString::Printf(TEXT("(OldGameName=\"%s\",NewGameName=\"%s\")"), *LongOldGameName, *LongNewGameName));
						NewFile.AddToSection(*RedirectsSection, TEXT("+ActiveGameNameRedirects"), FString::Printf(TEXT("(OldGameName=\"%s\",NewGameName=\"%s\")"), *ConfigParameters.GameName, *LongNewGameName));

						NewFile.UpdateSections(*EngineIniFilename, *RedirectsSection);

						FConfigContext::ForceReloadIntoGConfig().Load(*RedirectsSection);

						FLinkerLoad::AddGameNameRedirect(*LongOldGameName, *LongNewGameName);
						FLinkerLoad::AddGameNameRedirect(*ConfigParameters.GameName, *LongNewGameName);
					}
					break;
				}
			}
		}

		const FString NormalizedSourceRoot = SourceModuleInfo.ModuleSourcePath.IsEmpty()
			? FString()
			: FPaths::ConvertRelativePathToFull(SourceModuleInfo.ModuleSourcePath);

		// Process everything else and copy out to disk
		for (FPakFile::FPakEntryIterator It(PakFile); It; ++It, ++FileCount)
		{
			const FString* EntryFilename = It.TryGetFilename();
			check(EntryFilename);
			// config files already handled
			if (EntryFilename->StartsWith(TEXT("Config/")) || EntryFilename->Contains(TEXT("/Config/")))
			{
				continue;
			}

			// Media and manifest files don't get written out as part of the install
			if (EntryFilename->Contains(TEXT("manifest.json")) || EntryFilename->StartsWith(TEXT("Media/")) || EntryFilename->Contains(TEXT("/Media/")))
			{
				continue;
			}

			const FPakEntry& Entry = It.Info();
			PakReader.Seek(Entry.Offset);
			FPakEntry EntryInfo;
			EntryInfo.Serialize(PakReader, PakFile.GetInfo().Version);

			if (EntryInfo.IndexDataEquals(Entry))
			{
				if (EntryFilename->StartsWith(TEXT("Source/")) || EntryFilename->Contains(TEXT("/Source/")))
				{
					// if config parameter "contains source" flag is off at this point, source extraction has been denied, so skip entry
					if (!ConfigParameters.bContainsSource)
					{
						UE_LOGF(LogPackFactory, Log, "Skipping source entry \"%ls\": source extraction disabled.", **EntryFilename);
						++SkipCount;
						continue;
					}

					if (NormalizedSourceRoot.IsEmpty())
					{
						UE_LOGF(LogPackFactory, Error, "Skipping source entry \"%ls\": no Runtime module source path available.", **EntryFilename);
						++ErrorCount;
						continue;
					}

					FString DestFilename = *EntryFilename;
					if (DestFilename.StartsWith(TEXT("Source/")))
					{
						DestFilename.RightChopInline(7, EAllowShrinking::No);
					}
					else
					{
						const int32 SourceIndex = DestFilename.Find(TEXT("/Source/"));
						if (SourceIndex != INDEX_NONE)
						{
							DestFilename.RightChopInline(SourceIndex + 8, EAllowShrinking::No);
						}
					}

					DestFilename = PackFactoryHelper::MakeSafeDestPath(NormalizedSourceRoot, DestFilename, *EntryFilename);
					UE_LOGF(LogPackFactory, Log, "%ls (%lld) -> %ls", **EntryFilename, Entry.Size, *DestFilename);

					FString SourceContents;
					PackFactoryHelper::ExtractFileToString(Entry, PakReader, CopyBuffer, PersistentCompressionBuffer, SourceContents, PakFile);

					FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));

					// Add the PCH for the project above the default pack include
					const FString StringToReplace = FString::Printf(TEXT("%s.h"),*ConfigParameters.GameName);
					const FString StringToReplaceWith = FString::Printf(TEXT("%s\"%s#include \"%s"),
						*GameProjectModule.Get().DetermineModuleIncludePath(SourceModuleInfo, DestFilename),
						LINE_TERMINATOR,
						*StringToReplace);

					if (FFileHelper::SaveStringToFile(SourceContents, *DestFilename))
					{
						WrittenFiles.Add(*DestFilename);
						WrittenSourceFiles.Add(*DestFilename);
					}
					else
					{
						UE_LOGF(LogPackFactory, Error, "Unable to write file \"%ls\".", *DestFilename);
						++ErrorCount;
					}
				}
				else
				{
					FString DestFilename = *EntryFilename;
					if (DestFilename.StartsWith(TEXT("Content/")))
					{
						DestFilename.RightChopInline(8, EAllowShrinking::No);
					}
					else
					{
						const int32 ContentIndex = DestFilename.Find(ContentFolder);
						if (ContentIndex != INDEX_NONE)
						{
							DestFilename.RightChopInline(ContentIndex + 9, EAllowShrinking::No);
						}
					}
					DestFilename = PackFactoryHelper::MakeSafeDestPath(ContentDestinationRoot, DestFilename, *EntryFilename);
					UE_LOGF(LogPackFactory, Log, "%ls (%lld) -> %ls", **EntryFilename, Entry.Size, *DestFilename);

					TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileWriter(*DestFilename));

					if (FileHandle)
					{
						PackFactoryHelper::ExtractFile(Entry, PakReader, CopyBuffer, PersistentCompressionBuffer, *FileHandle, PakFile);
						WrittenFiles.Add(*DestFilename);
					}
					else
					{
						UE_LOGF(LogPackFactory, Error, "Unable to create file \"%ls\".", *DestFilename);
						++ErrorCount;
					}
				}
			}
			else
			{
				UE_LOGF(LogPackFactory, Error, "Index data mismatch for entry: \"%ls\".", **EntryFilename);
				ErrorCount++;
			}
		}

		if (ConfigParameters.AdditionalFilesToAdd.Num() > 0)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			for (const FString& FileToCopy : ConfigParameters.AdditionalFilesToAdd)
			{
				if (FileToCopy.StartsWith(TEXT("Source/")) || FileToCopy.Contains(TEXT("/Source/")))
				{
					// if config parameter "contains source" flag is off at this point, source extraction has been denied, so skip entry
					if (!ConfigParameters.bContainsSource)
					{
						UE_LOGF(LogPackFactory, Log, "Skipping source file \"%ls\": source extraction disabled.", *FileToCopy);
						++SkipCount;
						continue;
					}

					if (NormalizedSourceRoot.IsEmpty())
					{
						UE_LOGF(LogPackFactory, Error, "Skipping source file \"%ls\": no Runtime module source path available.", *FileToCopy);
						++ErrorCount;
						continue;
					}

					FString DestFilename = FileToCopy;
					if (DestFilename.StartsWith(TEXT("Source/")))
					{
						DestFilename.RightChopInline(7, EAllowShrinking::No);
					}
					else 
					{
						const int32 SourceIndex = DestFilename.Find(TEXT("/Source/"));
						if (SourceIndex != INDEX_NONE)
						{
							DestFilename.RightChopInline(SourceIndex + 8, EAllowShrinking::No);
						}
					}
					DestFilename = PackFactoryHelper::MakeSafeDestPath(NormalizedSourceRoot, DestFilename, FileToCopy);

					FString DestDirectory = FPaths::GetPath(DestFilename);

					if (PlatformFile.CreateDirectoryTree(*DestDirectory))
					{
						FString SourceContents;
						if (FFileHelper::LoadFileToString(SourceContents, *FileToCopy))
						{
							FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
							
							// Add the PCH for the project above the default pack include
							const FString StringToReplace = FString::Printf(TEXT("%s.h"),*ConfigParameters.GameName);
							const FString StringToReplaceWith = FString::Printf(TEXT("%s\"%s#include \"%s"),
								*GameProjectModule.Get().DetermineModuleIncludePath(SourceModuleInfo, DestFilename),
								LINE_TERMINATOR,
								*StringToReplace);

							SourceContents = SourceContents.Replace(*StringToReplace, *StringToReplaceWith, ESearchCase::CaseSensitive);

							if (FFileHelper::SaveStringToFile(SourceContents, *DestFilename))
							{
								WrittenFiles.Add(*DestFilename);
								WrittenSourceFiles.Add(*DestFilename);
							}
							else
							{
								UE_LOGF(LogPackFactory, Error, "Unable to write file \"%ls\".", *DestFilename);
								++ErrorCount;
							}
						}
						else
						{
							UE_LOGF(LogPackFactory, Error, "Unable to read file \"%ls\".", *FileToCopy);
						}
					}
				}
				else
				{
					FString DestFilename = FileToCopy;
					if (DestFilename.StartsWith(TEXT("Content/")))
					{
						DestFilename.RightChopInline(8, EAllowShrinking::No);
					}
					else
					{
						const int32 ContentIndex = DestFilename.Find(ContentFolder);
						if (ContentIndex != INDEX_NONE)
						{
							DestFilename.RightChopInline(ContentIndex + 9, EAllowShrinking::No);
						}
					}
					DestFilename = PackFactoryHelper::MakeSafeDestPath(ContentDestinationRoot, DestFilename, FileToCopy);

					FString DestDirectory = FPaths::GetPath(DestFilename);

					if (PlatformFile.CreateDirectoryTree(*DestDirectory))
					{
						if (PlatformFile.CopyFile(*DestFilename, *FileToCopy))
						{
							WrittenFiles.Add(DestFilename);
							UE_LOGF(LogPackFactory, Log, "Copied \"%ls\" to \"%ls\"", *FileToCopy, *DestFilename);
						}
						else
						{
							UE_LOGF(LogPackFactory, Error, "Unable to copy file \"%ls\" to \"%ls\".", *FileToCopy, *DestFilename);
						}
					}
					else
					{
						UE_LOGF(LogPackFactory, Error, "Unable to create directory \"%ls\".", *DestDirectory);
					}
				}
			}
		}
		UE_LOGF(LogPackFactory, Log, "Finished extracting %d files (including %d skipped, %d errors).", FileCount, SkipCount, ErrorCount);

		if (WrittenFiles.Num() > 0)
		{
			if (WrittenSourceFiles.Num() > 0)
			{
				// Update the game projects so the user's IDE reflects the new files.
				FGameProjectGenerationModule& GameProjectModule = FModuleManager::LoadModuleChecked<FGameProjectGenerationModule>(TEXT("GameProjectGeneration"));
				FText FailReason, FailLog;
				if (!GameProjectModule.UpdateCodeProject(FailReason, FailLog))
				{
					SOutputLogDialog::Open(NSLOCTEXT("PackFactory", "CreateBinary", "Create binary"), FailReason, FailLog, FText::GetEmpty());
				}

				// If the project had no prior code, configure the UBT target so a future user-initiated
				// recompile (editor toolbar button, IDE build) builds the correct target.
				if (!bProjectHadSourceFiles)
				{
					FPlatformMisc::SetUBTTargetName(*(FString(FApp::GetProjectName()) + TEXT("Editor")));
				}

				// Intentionally do NOT invoke hot-reload, live-coding, or RecompileModule from here.
				// Auto-compiling unstrusted C++ extracted from an imported .upack is an attack vector
				FMessageDialog::Open(EAppMsgType::Ok,
					NSLOCTEXT("PackFactory", "SourceAddedBuildManually",
						"Source files from the pack have been written to your project. Close the editor and build from your IDE to complete installation."));

				// Ask about editing code where applicable
				if (FSlateApplication::Get().SupportsSourceAccess())
				{
					// Code successfully added, notify the user and ask about opening the IDE now
					const FText Message = NSLOCTEXT("PackFactory", "CodeAdded", "Added source file(s). Would you like to edit the code now?");
					if (FMessageDialog::Open(EAppMsgType::YesNo, Message) == EAppReturnType::Yes)
					{
						FSourceCodeNavigation::OpenSourceFilesAsync(WrittenSourceFiles);
					}
				}
			}
			
			// Find an asset to return (It will be marked as dirty)
			for (const FString& Filename : WrittenFiles)
			{
				static const FString AssetExtension(TEXT(".uasset"));
				if (Filename.EndsWith(AssetExtension))
				{
					FString GameFileName = Filename;
					if (FPaths::MakePathRelativeTo(GameFileName, *FPaths::ProjectContentDir()))
					{
						int32 SlashIndex = INDEX_NONE;
						GameFileName = FString(TEXT("/Game/")) / GameFileName.LeftChop(AssetExtension.Len());
						if (GameFileName.FindLastChar(TEXT('/'), SlashIndex))
						{
							const FString AssetName = GameFileName.RightChop(SlashIndex + 1);
							ReturnAsset = LoadObject<UObject>(nullptr, *(GameFileName + TEXT(".") + AssetName));
							if (ReturnAsset)
							{
								break;
							}
						}
					}
				}
			}

			// If source control is enabled mark all the added files for checkout/add
			if (ISourceControlModule::Get().IsEnabled() && GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles)
			{
				for (const FString& Filename : WrittenFiles)
				{
					FText ErrorMessage;
					if (!SourceControlHelpers::CheckoutOrMarkForAdd(Filename, FText::FromString(Filename), NULL, ErrorMessage))
					{
						UE_LOGF(LogPackFactory, Error, "%ls", *ErrorMessage.ToString());
					}
				}
			}
		}

		if (!ConfigParameters.InstallMessage.IsEmpty())
		{
			FMessageLog("AssetTools").Warning(FText::FromString(ConfigParameters.InstallMessage));
			FMessageLog("AssetTools").Open();
		}
	}
	else
	{
		if (!PakFile.IsValid())
		{
			UE_LOGF(LogPackFactory, Warning, "Invalid pak file.");
		}
		else
		{
			UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, creation aborted.");
		}
	}

	return ReturnAsset;
}
