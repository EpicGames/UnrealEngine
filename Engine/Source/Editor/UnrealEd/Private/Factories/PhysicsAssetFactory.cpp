// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/PhysicsAssetFactory.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "PhysicsAssetEditorModule.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "AnimationEditorUtils.h"
#include "PhysicsAssetUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PhysicsAssetGenerationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetFactory)

#define LOCTEXT_NAMESPACE "PhysicsAssetFactory"

UPhysicsAssetFactory::UPhysicsAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UPhysicsAsset::StaticClass();
	TargetSkeletalMesh = nullptr;
}

bool UPhysicsAssetFactory::ConfigureProperties()
{
	// nullptr the skeletal mesh so we can check for selection later
	TargetSkeletalMesh = nullptr;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;

	/** The asset picker will only show skeletal meshes */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = false;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UPhysicsAssetFactory::OnTargetSkeletalMeshSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title(LOCTEXT("CreatePhysicsAssetOptions", "Pick Skeletal Mesh"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeletalMesh != nullptr;
}

UObject* UPhysicsAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Make sure we are trying to factory a phyiscs asset, then create and init one
	check(Class->IsChildOf(UPhysicsAsset::StaticClass()));

	if (TargetSkeletalMesh == nullptr)
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidSkeletalMesh", "Must specify a valid skeletal mesh for the physics asset to target.") );
		return nullptr;
	}

	return CreatePhysicsAssetFromMesh(Name, InParent, TargetSkeletalMesh, true);
}

void UPhysicsAssetFactory::OnTargetSkeletalMeshSelected(const FAssetData& SelectedAsset)
{
	TargetSkeletalMesh = Cast<USkeletalMesh>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

UObject* UPhysicsAssetFactory::CreatePhysicsAssetFromMesh(FName InAssetName, UObject* InParent, USkeletalMesh* SkelMesh, bool bSetToMesh)
{
	FString Name = InAssetName.ToString();
	FString PackageName;

	if(InAssetName == NAME_None)
	{
		// Get a unique package and asset name
		AnimationEditorUtils::CreateUniqueAssetName(SkelMesh->GetOutermost()->GetName(), TEXT("_Physics"), PackageName, Name);
	}

	UPackage* Package = Cast<UPackage>(InParent);
	if(InParent == nullptr && !PackageName.IsEmpty())
	{
		// Then find/create it.
		Package = CreatePackage( *PackageName);
		if ( !ensure(Package) )
		{
			// There was a problem creating the package
			return nullptr;
		}
	}

	IPhysicsAssetEditorModule* PhysicsAssetEditorModule = &FModuleManager::LoadModuleChecked<IPhysicsAssetEditorModule>( "PhysicsAssetEditor" );
	
	EAppReturnType::Type NewBodyResponse;

	// Now show the 'asset creation' options dialog
	PhysicsAssetEditorModule->OpenNewBodyDlg(&NewBodyResponse);
	bool bWasOkClicked = (NewBodyResponse == EAppReturnType::Ok);
	
	if( bWasOkClicked )
	{
		UPhysicsAsset* NewAsset = NewObject<UPhysicsAsset>(Package, *Name, RF_Public | RF_Standalone | RF_Transactional);
		if(NewAsset)
		{
			// Do automatic asset generation.
			FText ErrorMessage;
			const FPhysAssetCreateParams& NewBodyData = GetDefault<UPhysicsAssetGenerationSettings>()->CreateParams;
			bool bSuccess = FPhysicsAssetUtils::CreateFromSkeletalMesh(NewAsset, SkelMesh, NewBodyData, ErrorMessage, bSetToMesh);
			//Reset any property that should not be reconduct next time the dialog is open
			//	- the LodIndex must be reset to zero
			if (UPhysicsAssetGenerationSettings* MutablePhysicsAssetGenerationSettings = GetMutableDefault<UPhysicsAssetGenerationSettings>())
			{
				MutablePhysicsAssetGenerationSettings->CreateParams.LodIndex = 0;
				MutablePhysicsAssetGenerationSettings->SaveConfig();
			}
			if(bSuccess)
			{
				NewAsset->MarkPackageDirty();

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewAsset);

				if(bSetToMesh)
				{
					// auto-link source skelmesh to the new physasset and recreate physics state if needed
					RefreshSkelMeshOnPhysicsAssetChange(SkelMesh);
					SkelMesh->MarkPackageDirty();
				}

				return NewAsset;
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage);
				NewAsset->ClearFlags( RF_Public| RF_Standalone );
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
