// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PoseAssetFactory.cpp: Factory for PoseAsset
=============================================================================*/

#include "Factories/PoseAssetFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Animation/PoseAsset.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PoseAssetFactory)

#define LOCTEXT_NAMESPACE "PoseAssetFactory"

DECLARE_DELEGATE_ThreeParams(FOnPoseConfigureUserAction, bool /*bCreate*/, UAnimSequence* /*InSequence*/, const TArray<FString>& /*InPoseNames*/);

class SPoseConfigureWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SPoseConfigureWindow)
	{}

	SLATE_ARGUMENT(UAnimSequence*, SourceSequence)
	SLATE_ARGUMENT(FOnPoseConfigureUserAction, UserActionHandler)
	SLATE_ARGUMENT(FSimpleDelegate, OnCreateCanceled)
		
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs)
	{
		UserActionHandler = InArgs._UserActionHandler;
		SourceSequence = InArgs._SourceSequence;

		// Load the content browser module to display an asset picker
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		FAssetPickerConfig AssetPickerConfig;
		/** The asset picker will only show sequences */
		AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
		/** The delegate that fires when an asset was selected */
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &SPoseConfigureWindow::OnSourceAnimationSelected);
		if (SourceSequence != nullptr)
		{
			AssetPickerConfig.InitialAssetSelection = FAssetData(SourceSequence);
		}

		/** The default view mode should be a list view */
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

		SWindow::Construct(SWindow::FArguments()
			.Title(LOCTEXT("CreatePoseAssetOptions", "Create Pose Asset"))
			.SizingRule(ESizingRule::UserSized)
			.ClientSize(FVector2D(500, 600))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("Menu.Background") )
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(0.4)
				.Padding(3, 3)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 2)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectSourceAnimation", "Select Source Animation"))
					]

					+SVerticalBox::Slot()
					.Padding(2, 2)
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

				+SVerticalBox::Slot()
				.FillHeight(0.4)
				.Padding(3, 3)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2, 2)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("PoseNamesDescription", "[OPTIONAL] Pose Names (one name for each line)"))
					]

					+ SVerticalBox::Slot()
					.Padding(2, 2)
					[
						SAssignNew(TextBlock, SMultiLineEditableTextBox)
						.HintText(LOCTEXT("PoseNamesPlaceholder", "Type one pose name for each line..."))
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.Text(LOCTEXT("Accept", "Accept"))
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.IsEnabled(this, &SPoseConfigureWindow::CanAccept)
						.OnClicked_Raw(this, &SPoseConfigureWindow::OnAccept)
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked_Raw(this, &SPoseConfigureWindow::OnCancel)
					]

				]
			]
		]);
	}

	bool CanAccept() const
	{
		return SourceSequence != nullptr && SourceSequence->GetSkeleton() && UserActionHandler.IsBound();
	}

	FReply OnAccept()
	{
		if (CanAccept())
		{
			// parse posenames
			FText InputText = TextBlock->GetPlainText();

			FString InputString = InputText.ToString();
			PoseNames.Reset();
			InputString.ParseIntoArrayLines(PoseNames);
			UserActionHandler.Execute(true, SourceSequence, PoseNames);
		}

		RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		if (UserActionHandler.IsBound())
		{
			UserActionHandler.Execute(false, nullptr, PoseNames);
		}

		RequestDestroyWindow();
		return FReply::Handled();
	}

	void OnSourceAnimationSelected(const FAssetData& SelectedAsset)
	{
		SourceSequence = Cast<UAnimSequence>(SelectedAsset.GetAsset());
	}

private:
	UAnimSequence*						SourceSequence;
	TArray<FString>						PoseNames;
	FOnPoseConfigureUserAction			UserActionHandler;
	TSharedPtr<SMultiLineEditableTextBox> TextBlock;
};

UPoseAssetFactory::UPoseAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UPoseAsset::StaticClass();
}

bool UPoseAssetFactory::ConfigureProperties()
{
	TSharedPtr<SWindow> PickerWindow = SNew(SPoseConfigureWindow)
		.SourceSequence(SourceAnimation)
		.UserActionHandler(FOnPoseConfigureUserAction::CreateUObject(this, &UPoseAssetFactory::OnWindowUserActionDelegate));

	// have to clear after setting it because
	// you could use close button without clicking cancel
	SourceAnimation = nullptr;

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return SourceAnimation != nullptr;
}

UObject* UPoseAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (SourceAnimation)
	{
		// Use the skeleton from the source animation
		TargetSkeleton = SourceAnimation->GetSkeleton();

		UPoseAsset* PoseAsset = NewObject<UPoseAsset>(InParent, Class, Name, Flags);
		TArray<FName> InputPoseNames;
		for (int32 Index = 0; Index < PoseNames.Num(); ++Index)
		{
			InputPoseNames.AddUnique(FName(*PoseNames[Index]));
		}

		PoseAsset->CreatePoseFromAnimation( SourceAnimation, &InputPoseNames);
		PoseAsset->SetSkeleton(TargetSkeleton);
		return PoseAsset;
	}

	return nullptr;
}

void UPoseAssetFactory::OnWindowUserActionDelegate(bool bCreate, UAnimSequence* InSequence, const TArray<FString>& InPoseNames)
{
	if (bCreate && InSequence)
	{
		SourceAnimation = InSequence;
		PoseNames = InPoseNames;
	}
	else
	{
		SourceAnimation = nullptr;
		PoseNames.Reset();
	}
}
#undef LOCTEXT_NAMESPACE
