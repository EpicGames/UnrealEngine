// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkeletonFactory.cpp: Factory for Skeletons
=============================================================================*/

#include "Factories/SkeletonFactory.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Editor.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonFactory)

#define LOCTEXT_NAMESPACE "SkeletonFactory"


/*------------------------------------------------------------------------------
	USkeletonFactory implementation.
------------------------------------------------------------------------------*/

USkeletonFactory::USkeletonFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = USkeleton::StaticClass();
}

bool USkeletonFactory::ConfigureProperties()
{
	// Null the parent class so we can check for selection later
	TargetSkeletalMesh = NULL;

	// Load the content browser module to display an asset picker
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	
	/** The asset picker will only show skeletal meshes */
	AssetPickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &USkeletonFactory::OnTargetSkeletalMeshSelected);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	PickerWindow = SNew(SWindow)
	.Title( LOCTEXT("CreateSkeletonOptions", "Pick Skeletal Mesh") )
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return TargetSkeletalMesh != NULL;
}

UObject* USkeletonFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Make sure we are trying to factory a skeleton, then create and init one
	check(Class->IsChildOf(USkeleton::StaticClass()));

	if (TargetSkeletalMesh == NULL)
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("InvalidSkeletalMesh", "Must specify a valid skeletal mesh for the skeleton to target.") );
		return NULL;
	}

	USkeleton* NewAsset = NewObject<USkeleton>(InParent, Class, Name, Flags);
	if(NewAsset)
	{
		// this should not fail. If so there is problem.
		if ( !NewAsset->MergeAllBonesToBoneTree( TargetSkeletalMesh ) )
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("CreateNewSkeletonFailed_BoneMerge", "Failed to create Skeleton : Could not merge bone.") );
			NewAsset->ClearFlags( RF_Public| RF_Standalone );
			NewAsset = NULL;
		}

		if (TargetSkeletalMesh->GetSkeleton() != NewAsset)
		{
			TargetSkeletalMesh->SetSkeleton(NewAsset);
			TargetSkeletalMesh->MarkPackageDirty();
		}
	}
	
	return NewAsset;
}

void USkeletonFactory::OnTargetSkeletalMeshSelected(const FAssetData& SelectedAsset)
{
	TargetSkeletalMesh = Cast<USkeletalMesh>(SelectedAsset.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE
