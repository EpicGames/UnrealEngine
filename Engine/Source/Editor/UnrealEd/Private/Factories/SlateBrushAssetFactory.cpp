// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SlateBrushAssetFactory.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Engine/Texture2D.h"
#include "Slate/SlateBrushAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateBrushAssetFactory)

#define LOCTEXT_NAMESPACE "SlateBrushAssetFactory"

USlateBrushAssetFactory::USlateBrushAssetFactory( const FObjectInitializer& ObjectInitializer )
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = USlateBrushAsset::StaticClass();
}


FText USlateBrushAssetFactory::GetDisplayName() const
{
	return LOCTEXT("SlateBrushAssetFactoryDescription", "Slate Brush");
}


bool USlateBrushAssetFactory::ConfigureProperties()
{
	return true;
}


UObject* USlateBrushAssetFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	USlateBrushAsset* NewSlateBrushAsset = NewObject<USlateBrushAsset>(InParent, Name, Flags);
	NewSlateBrushAsset->Brush = InitialTexture != NULL ? FSlateDynamicImageBrush( InitialTexture, FVector2D( InitialTexture->GetImportedSize() ), NAME_None ) : FSlateBrush();
	return NewSlateBrushAsset;
}

#undef LOCTEXT_NAMESPACE
