// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SlateWidgetStyleAssetFactory.h"

#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateWidgetStyleAssetFactory)

class FFeedbackContext;
class UObject;

class FClassFilter : public IClassViewerFilter
{
public:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	TSet< const UClass* > DisallowedClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags) && InFilterFuncs->IfInClassesSet( DisallowedClasses , InClass ) == EFilterReturn::Failed
			&& InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags) && InFilterFuncs->IfInClassesSet( DisallowedClasses , InUnloadedClassData ) == EFilterReturn::Failed
			&& InFilterFuncs->IfInChildOfClassesSet( AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};


USlateWidgetStyleAssetFactory::USlateWidgetStyleAssetFactory( const FObjectInitializer& ObjectInitializer )
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = USlateWidgetStyleAsset::StaticClass();
	StyleType = NULL;
}

FText USlateWidgetStyleAssetFactory::GetDisplayName() const
{
	return NSLOCTEXT("SlateWidgetStyleAssetFactory", "SlateWidgetStyleAssetFactoryDescription", "Slate Widget Style");
}

bool USlateWidgetStyleAssetFactory::ConfigureProperties()
{
	// NULL the DataAssetClass so we can check for selection
	StyleType = NULL;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FClassFilter> Filter = MakeShareable(new FClassFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add( USlateWidgetStyleContainerBase::StaticClass() );
	Filter->DisallowedClasses.Add( USlateWidgetStyleContainerBase::StaticClass() );

	const FText TitleText = NSLOCTEXT("SlateWidgetStyleAssetFactory", "CreateSlateWidgetStyleAssetOptions", "Pick Slate Widget Style Class");
	UClass* ChosenClass = NULL;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, USlateWidgetStyleAsset::StaticClass());

	if ( bPressedOk )
	{
		StyleType = ChosenClass;
	}

	return bPressedOk;
}


UObject* USlateWidgetStyleAssetFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	USlateWidgetStyleAsset* NewUSlateStyle = NewObject<USlateWidgetStyleAsset>(InParent, Name, Flags);

	//initialize
	NewUSlateStyle->CustomStyle = NewObject<USlateWidgetStyleContainerBase>(NewUSlateStyle, StyleType, Name);

	return NewUSlateStyle;
}
