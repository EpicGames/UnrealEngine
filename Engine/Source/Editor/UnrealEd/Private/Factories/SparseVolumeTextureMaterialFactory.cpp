// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/SparseVolumeTextureMaterialFactory.h"

#include "Engine/RendererSettings.h"
#include "Materials/MaterialExpressionSparseVolumeTextureSample.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SparseVolumeTextureMaterialFactory)

#if WITH_EDITOR

USparseVolumeTextureMaterialFactoryNew::USparseVolumeTextureMaterialFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UMaterial::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
}

UObject* USparseVolumeTextureMaterialFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UMaterial* NewMaterial = NewObject<UMaterial>(InParent, Class, Name, Flags);
	NewMaterial->MaterialDomain = EMaterialDomain::MD_Volume;
	NewMaterial->BlendMode = BLEND_Additive;

	UMaterialExpressionSparseVolumeTextureSample* TextureSampler = NewObject<UMaterialExpressionSparseVolumeTextureSample>(NewMaterial);
	{
		TextureSampler->SparseVolumeTexture = InitialTexture;
	}

	NewMaterial->GetExpressionCollection().AddExpression(TextureSampler);

	UMaterialEditorOnlyData* NewMaterialEditorOnly = NewMaterial->GetEditorOnlyData();
	{
		FExpressionOutput& Output = TextureSampler->GetOutputs()[0];
		FExpressionInput& Input = NewMaterialEditorOnly->EmissiveColor;
		Input.Expression = TextureSampler;
		Input.Mask = Output.Mask;
		Input.MaskR = Output.MaskR;
		Input.MaskG = Output.MaskG;
		Input.MaskB = Output.MaskB;
		Input.MaskA = Output.MaskA;
	}
	{
		FExpressionOutput& Output = TextureSampler->GetOutputs()[1];
		FExpressionInput& Input = NewMaterialEditorOnly->SubsurfaceColor;
		Input.Expression = TextureSampler;
		Input.Mask = Output.Mask;
		Input.MaskR = Output.MaskR;
		Input.MaskG = Output.MaskG;
		Input.MaskB = Output.MaskB;
		Input.MaskA = Output.MaskA;
	}

	NewMaterial->PostEditChange();
	
	NewMaterial->bAutomaticallySetUsageInEditor = GetDefault<URendererSettings>()->bAutomaticallySetMaterialUsageInEditorDefault;

	return NewMaterial;
}


USparseVolumeTextureMaterialInstanceFactoryNew::USparseVolumeTextureMaterialInstanceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultSVTMaterial(FSoftObjectPath(TEXT("/Engine/EngineMaterials/SparseVolumeMaterial.SparseVolumeMaterial")))
{
	SupportedClass = UMaterialInstanceConstant::StaticClass();
	bCreateNew = false;
	bEditAfterNew = true;
}

UObject* USparseVolumeTextureMaterialInstanceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (InitialParent == nullptr)
	{
		InitialParent = DefaultSVTMaterial.LoadSynchronous();
	}

	UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(UMaterialInstanceConstantFactoryNew::FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn));
	if (MIC && InitialTexture)
	{
		MIC->SetSparseVolumeTextureParameterValueEditorOnly(FMaterialParameterInfo("SparseVolumeTexture"), InitialTexture);
	}

	return MIC;
}

#endif //WITH_EDITOR
