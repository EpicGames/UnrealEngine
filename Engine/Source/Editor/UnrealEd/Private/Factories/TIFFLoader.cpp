// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TIFFLoader.h"

DEFINE_LOG_CATEGORY_STATIC(LogTIFFLoader, Log, All);


// disable warnings about myself :
PRAGMA_DISABLE_DEPRECATION_WARNINGS


#if WITH_FREEIMAGE_LIB

#include "Misc/Paths.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h" // This is needed to make sure FreeImage is not typedeffing BOOL
#endif

THIRD_PARTY_INCLUDES_START
#include "FreeImage.h"
THIRD_PARTY_INCLUDES_END

class FFreeImageWrapper
{
public:
	static bool IsValid() { return FreeImageDllHandle != nullptr; }

	static void FreeImage_Initialise(bool bLoadLocalPluginsOnly); // Loads and inits FreeImage on first call

private:
	static void* FreeImageDllHandle; // Lazy init on first use, never release for now
};

void* FFreeImageWrapper::FreeImageDllHandle = nullptr;

void FFreeImageWrapper::FreeImage_Initialise(bool bLoadLocalPluginsOnly)
{
	if (FreeImageDllHandle != nullptr)
	{
		return;
	}

	if (FreeImageDllHandle == nullptr)
	{
		FString FreeImageDir = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/FreeImage"), FPlatformProcess::GetBinariesSubdirectory());
		FString FreeImageLibDir = FPaths::Combine( FreeImageDir, TEXT(FREEIMAGE_LIB_FILENAME));
		FPlatformProcess::PushDllDirectory(*FreeImageDir);
		FreeImageDllHandle = FPlatformProcess::GetDllHandle(*FreeImageLibDir);
		FPlatformProcess::PopDllDirectory(*FreeImageDir);
	}

	if (FreeImageDllHandle)
	{
		::FreeImage_Initialise((BOOL)bLoadLocalPluginsOnly);
	}
}

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

FTiffLoadHelper::FTiffLoadHelper()
{
	FFreeImageWrapper::FreeImage_Initialise(false);
	if (!FFreeImageWrapper::IsValid())
	{
		SetError(TEXT("Can't initialize FreeImage"));
		return;
	}

	bIsValid = true;
}

FTiffLoadHelper::~FTiffLoadHelper()
{
	if (Memory)
	{
		FreeImage_CloseMemory(Memory);
	}

	if (Bitmap)
	{
		FreeImage_Unload(Bitmap);
	}
}

bool FTiffLoadHelper::Load(const uint8 * Buffer, uint32 Length)
{
	FREE_IMAGE_FORMAT FileType = FIF_TIFF;

	Memory = FreeImage_OpenMemory(const_cast<uint8*>(Buffer), Length);
	Bitmap = FreeImage_LoadFromMemory(FileType, Memory, 0);

	if (!Bitmap)
	{
		return false;
	}

	// FreeImage keeps all images upside-down
	if (!ensure(FreeImage_FlipVertical(Bitmap)))
	{
		UE_LOGF(LogTIFFLoader, Error, "Can't flip TIFF image");
	}

	int32 BitsPerPixel = FreeImage_GetBPP(Bitmap);

	bool bIsSourceSupported = true;
	// source is grayscale(one channel), although this doesn't mean that all grayscale formats will be converted to G8
	// high-bpp formats will use 16-bit-per-channel 
	int32 bIsSourceGrayScale = false;
	bool bIsSource16BitsPerChannel = false;
	// some FreeImage formats(signed integers) have limited support for conversion
	bool bShouldConvertToByte = false;
	bool bIsSourceFloatingPoint = false;

	int32 ImageType = FreeImage_GetImageType(Bitmap);
	switch (ImageType)
	{
	case FIT_RGB16:
	case FIT_RGBA16:
	{
		bIsSource16BitsPerChannel = true;
		break;
	}

	case FIT_INT16:
	{
		bShouldConvertToByte = true;
		bIsSourceGrayScale = (BitsPerPixel / 16) == 1;
		break;
	}
	case FIT_INT32:
	{
		bShouldConvertToByte = true;
		bIsSourceGrayScale = (BitsPerPixel / 32) == 1;
		break;
	}
	case FIT_FLOAT:
	case FIT_DOUBLE:
	case FIT_RGBAF:
	case FIT_RGBF:
	{
		bIsSourceFloatingPoint = true;
		break;
	}
	case FIT_COMPLEX:
	{
		bIsSourceSupported = false;
		break;
	}


	case FIT_UINT16:
	{
		bIsSource16BitsPerChannel = true;
		bIsSourceGrayScale = (BitsPerPixel / 16) == 1;
		break;
	}
	case FIT_UINT32:
	{
		bIsSource16BitsPerChannel = true;
		bIsSourceGrayScale = (BitsPerPixel / 32) == 1;
		break;
	}

	case FIT_BITMAP:
	{
		// treat 1-bit dib as grayscale
		bIsSourceGrayScale = BitsPerPixel == 1;
		break;
	}
	case FIT_UNKNOWN:
	default:
	{
		break;
	}
	}

	if (!bIsSourceSupported)
	{
		UE_LOGF(LogTIFFLoader, Error, "Unsupported TIFF format");
		return false;
	}

	Width = FreeImage_GetWidth(Bitmap);
	Height = FreeImage_GetHeight(Bitmap);

	if (bIsSourceFloatingPoint)
	{
		// Floating point images converted to RGBA16F
		RawData.SetNumUninitialized(Height * Width * 4 * sizeof(FFloat16));

		TextureSourceFormat = TSF_RGBA16F;
		CompressionSettings = TC_HDR_Compressed;
		bSRGB = false;

		FIBITMAP* ConvertedBitmap = FreeImage_ConvertToType(Bitmap, FIT_RGBAF, true);
		if (ConvertedBitmap)
		{
			BYTE* Bits = FreeImage_GetBits(ConvertedBitmap);
			int32 Pitch = FreeImage_GetPitch(ConvertedBitmap);
			for (int Y = 0; Y < Height; Y++)
			{
				BYTE* ScanLine = Bits + Pitch * Y;
				FIRGBAF* Pixels = (FIRGBAF*)ScanLine;
				for (int X = 0; X < Width; X++)
				{
					FIRGBAF P = Pixels[X];
					FFloat16* TargetPixel = ((FFloat16*)RawData.GetData()) + (X + Y * Width) * 4;
					TargetPixel[0].Set(P.red);
					TargetPixel[1].Set(P.green);
					TargetPixel[2].Set(P.blue);
					TargetPixel[3].Set(P.alpha);
				}
			}

			FreeImage_Unload(ConvertedBitmap);
		}
	}
	else if (bIsSourceGrayScale)
	{
		// Grayscale images converted to either G8 or RGBA16
		if (bIsSource16BitsPerChannel)
		{
			ConvertToRGBA16();
		}
		{
			RawData.SetNumUninitialized(Height * Width * 4);

			TextureSourceFormat = TSF_G8;
			CompressionSettings = TC_Grayscale;
			bSRGB = false;

			FIBITMAP* ConvertedBitmap;
			if (bShouldConvertToByte) {
				ConvertedBitmap = FreeImage_ConvertToStandardType(Bitmap, true);
			}
			else
			{
				ConvertedBitmap = FreeImage_ConvertToGreyscale(Bitmap);
			}

			if (ConvertedBitmap)
			{

				BYTE* Bits = FreeImage_GetBits(ConvertedBitmap);
				int32 Pitch = FreeImage_GetPitch(ConvertedBitmap);
				for (int Y = 0; Y < Height; Y++)
				{
					BYTE* ScanLine = Bits + Pitch * Y;
					uint8* TargetPixels = ((uint8*)RawData.GetData()) + Y * Width;
					for (int X = 0; X < Width; X++)
					{
						uint8 P = ScanLine[X];
						TargetPixels[X] = P;
					}
				}
				FreeImage_Unload(ConvertedBitmap);
			}
		}
	}
	else // rgb(a)
	{
		if (bIsSource16BitsPerChannel)
		{
			ConvertToRGBA16();
		}
		else
		{
			// Convert to RGBA(8-bit)

			RawData.SetNumUninitialized(Height * Width * 4);

			TextureSourceFormat = TSF_BGRA8;
			CompressionSettings = TC_Default;
			bSRGB = true;

			FIBITMAP* ConvertedBitmap = FreeImage_ConvertTo32Bits(Bitmap);
			if (ConvertedBitmap)
			{

				BYTE* Bits = FreeImage_GetBits(ConvertedBitmap);
				int32 Pitch = FreeImage_GetPitch(ConvertedBitmap);
				for (int Y = 0; Y < Height; Y++)
				{
					BYTE* ScanLine = Bits + Pitch * Y;
					for (int X = 0; X < Width; X++)
					{
						uint8* P = ScanLine + X * 4;
						uint8* TargetPixel = ((uint8*)RawData.GetData()) + (X + Y * Width) * 4;
						// FI_RGBA_X - cross-platform way to retrieve channels
						TargetPixel[0] = P[FI_RGBA_BLUE];
						TargetPixel[1] = P[FI_RGBA_GREEN];
						TargetPixel[2] = P[FI_RGBA_RED];
						TargetPixel[3] = P[FI_RGBA_ALPHA];
					}
				}

				FreeImage_Unload(ConvertedBitmap);
			}
		}
	}

	return true;
}

bool FTiffLoadHelper::ConvertToRGBA16()
{
	RawData.SetNumUninitialized(Height * Width * 4 * 2);

	TextureSourceFormat = TSF_RGBA16;
	CompressionSettings = TC_Default;
	bSRGB = false;

	FIBITMAP* ConvertedBitmap = FreeImage_ConvertToType(Bitmap, FIT_RGBA16, true);
	if (ConvertedBitmap)
	{
		BYTE* Bits = FreeImage_GetBits(ConvertedBitmap);
		int32 Pitch = FreeImage_GetPitch(ConvertedBitmap);
		for (int Y = 0; Y < Height; Y++)
		{
			BYTE* ScanLine = Bits + Pitch * Y;
			FIRGBA16* Pixels = (FIRGBA16*)ScanLine;
			uint16* TargetScanLine = ((uint16*)RawData.GetData()) + Y * Width * 4;
			for (int X = 0; X < Width; X++)
			{
				FIRGBA16 P = Pixels[X];
				uint16* TargetPixel = TargetScanLine + X * 4;
				TargetPixel[0] = P.red;
				TargetPixel[1] = P.green;
				TargetPixel[2] = P.blue;
				TargetPixel[3] = P.alpha;
			}
		}

		FreeImage_Unload(ConvertedBitmap);
		return true;
	}
	return false;
}

void FTiffLoadHelper::SetError(const FString& InErrorMessage)
{
	ErrorMessage = InErrorMessage;
}

FString FTiffLoadHelper::GetError()
{
	return ErrorMessage;
}

bool FTiffLoadHelper::IsValid()
{
	return bIsValid;
}

#else // WITH_FREEIMAGE_LIB 

FTiffLoadHelper::FTiffLoadHelper() 
{
}

FTiffLoadHelper::~FTiffLoadHelper() 
{
}

bool FTiffLoadHelper::Load(const uint8 * Buffer, uint32 Length)
{
    return false;
}

bool FTiffLoadHelper::ConvertToRGBA16()
{
    return true;
}

void FTiffLoadHelper::SetError(const FString& InErrorMessage)
{
}

FString FTiffLoadHelper::GetError()
{
	return TEXT("FreeImage not supported on this platform");
}

bool FTiffLoadHelper::IsValid()
{
    return false;
}

#endif // WITH_FREEIMAGE_LIB

PRAGMA_ENABLE_DEPRECATION_WARNINGS
