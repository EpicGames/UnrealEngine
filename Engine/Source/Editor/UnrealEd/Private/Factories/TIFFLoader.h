// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture.h"

// DEPRECATED
// use Tiff ImageWrapper instead
// delete me in UE 5.4

class UTexture2D;

struct FIBITMAP;
struct FIMEMORY;

class UE_DEPRECATED(5.3,"Use Tiff ImageWrapper instead") FTiffLoadHelper
{
public:

	FTiffLoadHelper();

	~FTiffLoadHelper();

	bool Load(const uint8 * Buffer, uint32 Length);

	bool ConvertToRGBA16();

	void SetError(const FString& InErrorMessage);

	FString GetError();

	bool IsValid();

	// Resulting image data and properties
	TArray<uint8> RawData;
	int32 Width;
	int32 Height;
	ETextureSourceFormat TextureSourceFormat = TSF_Invalid;
	TextureCompressionSettings CompressionSettings = TC_Default;
	bool bSRGB = true;

private:
	bool bIsValid = false;
	FIBITMAP* Bitmap = nullptr;
	FIMEMORY* Memory = nullptr;

	FString ErrorMessage;
};
