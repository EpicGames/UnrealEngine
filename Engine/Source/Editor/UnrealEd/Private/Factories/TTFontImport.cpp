// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TTFontImport.cpp: True-type Font Importing
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Engine/FontImportOptions.h"
#include "Engine/Font.h"
#include "Factories/TrueTypeFontFactory.h"
#include "RHI.h"
#include "Engine/Texture2D.h"
#include "Interfaces/IMainFrameModule.h"
#include "IDesktopPlatform.h"
#include "DesktopPlatformModule.h"

#ifndef WITH_FREETYPE
	#define WITH_FREETYPE	0
#endif // WITH_FREETYPE

#if PLATFORM_MAC
#include "Apple/PreAppleSystemHeaders.h"
#include <CoreText/CoreText.h>
#include "Apple/PostAppleSystemHeaders.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
namespace TTFConstants
{
	uint32 WIN_SRCCOPY = SRCCOPY;
}
#include "Windows/HideWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

#define USE_FREETYPE (!PLATFORM_WINDOWS && WITH_FREETYPE) // @todo: Enable for Windows when support for bitmap fonts is fixed

#if USE_FREETYPE
#include "ft2build.h"
// Freetype style include
#include FT_FREETYPE_H
#include FT_GLYPH_H
#endif // USE_FREETYPE

DEFINE_LOG_CATEGORY_STATIC(LogTTFontImport, Log, All);

#define LOCTEXT_NAMESPACE "TTFontImport"

namespace
{
	int32 FromHex(TCHAR Ch)
	{
		if (Ch >= '0' && Ch <= '9')
			return Ch - '0';
		else if (Ch >= 'a' && Ch <= 'f')
			return 10 + Ch - 'a';
		else if (Ch >= 'A' && Ch <= 'F')
			return 10 + Ch - 'A';
		UE_LOGF(LogTTFontImport, Fatal, "Expecting digit, got character %c", Ch);
		return 0;
	}

	int32 RoundUpToMultipleOf4(int32 X)
	{
		return (X + 3) & ~3;
	}
}

UTrueTypeFontFactory::UTrueTypeFontFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFont::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	ImportPriority = -1;
	LODGroup = TEXTUREGROUP_UI;
}

void UTrueTypeFontFactory::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad))
	{
		SetupFontImportOptions();
	}
}

#if !PLATFORM_WINDOWS // !!! FIXME
#define FW_NORMAL 400
#endif

void UTrueTypeFontFactory::SetupFontImportOptions()
{
	// Allocate our import options object if it hasn't been created already!
	ImportOptions = NewObject<UFontImportOptions>(this, NAME_None);
}

bool UTrueTypeFontFactory::ConfigureProperties()
{
	// Set bFontSelected to false so we can see test selection
	bFontSelected = false;

	// Show the dialog to let them choose the font
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		void* ParentWindowWindowHandle = NULL;

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if ( MainFrameParentWindow.IsValid() && MainFrameParentWindow->GetNativeWindow().IsValid() )
		{
			ParentWindowWindowHandle = MainFrameParentWindow->GetNativeWindow()->GetOSWindowHandle();
		}

		if( ImportOptions == NULL )
		{
			SetupFontImportOptions();
		}

		EFontImportFlags FontFlags;
		bFontSelected = DesktopPlatform->OpenFontDialog(
			ParentWindowWindowHandle,
			ImportOptions->Data.FontName,
			ImportOptions->Data.Height,
			FontFlags
			);

		if ( bFontSelected )
		{
			if( !!(FontFlags & EFontImportFlags::EnableUnderline) )
			{
				ImportOptions->Data.bEnableUnderline = true;
			}
			if( !!(FontFlags & EFontImportFlags::EnableItalic) )
			{
				ImportOptions->Data.bEnableItalic = true;
			}
			if( !!(FontFlags & EFontImportFlags::EnableBold) )
			{
				ImportOptions->Data.bEnableBold = true;
			}
		}
	}

	bPropertiesConfigured = true;

	return bFontSelected;
}

UObject* UTrueTypeFontFactory::FactoryCreateNew(
	UClass* Class,
	UObject* InParent,
	FName Name,
	EObjectFlags Flags,
	UObject* Context,
	FFeedbackContext*	Warn )
{
#if !PLATFORM_WINDOWS && !PLATFORM_MAC && !PLATFORM_LINUX
	STUBBED("Windows/Mac TTF code");
	return NULL;
#else
	check(Class==UFont::StaticClass());

	if ( bPropertiesConfigured && !bFontSelected )
	{
		// If the font dialog was shown but no font was selected, don't create a font object.
		return NULL;
	}

	// Create font and its texture.
	auto Font = NewObject<UFont>(InParent, Name, Flags);
	
	if (ImportOptions->Data.bUseDistanceFieldAlpha)
	{
		// high res font bitmap should only contain a mask
		ImportOptions->Data.bEnableAntialiasing = false;
		// drop shadows can be generated dynamically during rendering of distance fields
		ImportOptions->Data.bEnableDropShadow = false;
		// scale factor should always be a power of two
		ImportOptions->Data.DistanceFieldScaleFactor = FMath::RoundUpToPowerOfTwo(FMath::Max<int32>(ImportOptions->Data.DistanceFieldScaleFactor,2));
		ImportOptions->Data.DistanceFieldScanRadiusScale = FMath::Clamp<float>(ImportOptions->Data.DistanceFieldScanRadiusScale,0.f,8.f);
		// need a minimum padding of 4,4 to prevent bleeding of distance values across characters
		ImportOptions->Data.XPadding = FMath::Max<int32>(4,ImportOptions->Data.XPadding);
		ImportOptions->Data.YPadding = FMath::Max<int32>(4,ImportOptions->Data.YPadding);
	}

	ImportOptions->Data.TexturePageWidth = RoundUpToMultipleOf4(ImportOptions->Data.TexturePageWidth);
	ImportOptions->Data.TexturePageMaxHeight = RoundUpToMultipleOf4(ImportOptions->Data.TexturePageMaxHeight);

	// Copy the import settings into the font for later reference
	Font->ImportOptions = ImportOptions->Data;
	

	// For a single-resolution font, we'll create a one-element array and pass that along to our import function
	TArray< float > ResHeights;
	ResHeights.Add( ImportOptions->Data.Height );

	GWarn->BeginSlowTask( NSLOCTEXT("UnrealEd", "FontFactory_ImportingTrueTypeFont", "Importing TrueType Font..." ), true );

	// Import the font
	const bool bSuccess = ImportTrueTypeFont( Font, Warn, ResHeights.Num(), ResHeights );
	if (!bSuccess)
	{
		Font = NULL;
	}

	GWarn->EndSlowTask();

	return bSuccess ? Font : NULL;
#endif
}

bool UTrueTypeFontFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UFont* FontToReimport = Cast<UFont>(Obj);
	if(FontToReimport && FontToReimport->FontCacheType == EFontCacheType::Offline)
	{
		OutFilenames.Add(TEXT("None"));
		return true;
	}
	return false;
}

void UTrueTypeFontFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	// No path is needed
}

EReimportResult::Type UTrueTypeFontFactory::Reimport( UObject* InObject )
{
	UFont* FontToReimport = Cast<UFont>(InObject);
	
	if (FontToReimport == nullptr)
	{
		return EReimportResult::Failed;
	}

	SetupFontImportOptions();
	this->ImportOptions->Data = FontToReimport->ImportOptions;

	bool OutCanceled = false;

	if (ImportObject(InObject->GetClass(), InObject->GetOuter(), *InObject->GetName(), RF_Public | RF_Standalone, TEXT(""), nullptr, OutCanceled) != nullptr)
	{
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

int32 UTrueTypeFontFactory::GetPriority() const
{
	return ImportPriority;
}


/**
* Utility to convert texture alpha mask to a signed distance field
*
* Based on the following paper:
* http://www.valvesoftware.com/publications/2007/SIGGRAPH2007_AlphaTestedMagnification.pdf
*/
class FTextureAlphaToDistanceField
{
	/** Container for the input image from which we build the distance field */
	struct FTaskSrcData
	{
		/**
		 * Creates a source data with the following parameters:
		 *
		 * @param SrcTexture  A pointer to the raw data
		 * @param InSrcSizeX  The Width of the data in pixels
		 * @param InSrcSizeY  The Height of the data in pixels
		 * @param InSrcFormat The format of each pixel
		 */
		FTaskSrcData(const uint8* InSrcTexture, int32 InSrcSizeX, int32 InSrcSizeY, uint8 InSrcFormat)
			: SrcSizeX(InSrcSizeX)
			, SrcSizeY(InSrcSizeY)
			, SrcTexture(InSrcTexture)
			, SrcFormat(InSrcFormat)
		{ 
			check(SrcFormat == PF_B8G8R8A8);
		}

	/**
	 * Get the color for the source texture at the specified coordinates
	 *
	 * @param PointX - x coordinate
	 * @param PointY - y coordinate
	 * @return texel color
	 */
	FORCEINLINE FColor GetSourceColor(int32 PointX, int32 PointY) const
	{
		checkSlow(PointX < SrcSizeX && PointY < SrcSizeY);
		return FColor(
			SrcTexture[4 * (PointX + PointY * SrcSizeX) + 2],
			SrcTexture[4 * (PointX + PointY * SrcSizeX) + 1],
			SrcTexture[4 * (PointX + PointY * SrcSizeX) + 0],
			SrcTexture[4 * (PointX + PointY * SrcSizeX) + 3]
		);
	}
		
	/**
	 * Get just the alpha for the source texture at the specified coordinates
	 *
	 * @param PointX - x coordinate
	 * @param PointY - y coordinate
	 * @return texel alpha
	 */
	FORCEINLINE uint8 GetSourceAlpha(int32 PointX, int32 PointY) const
	{
		checkSlow(PointX < SrcSizeX && PointY < SrcSizeY);
		return SrcTexture[4 * (PointX + PointY * SrcSizeX) + 3];
	}

	/** Width of the source texture */
	const int32 SrcSizeX;
	/** Height of the source texture */
	const int32 SrcSizeY;

	private:
		/** Source texture used for silhouette determination. Alpha channel contains mask */
		const uint8* SrcTexture;
		/** Format of the source texture. Assumed to be PF_B8G8R8A8 */
		const uint8 SrcFormat;
	};
	
	/** The source data */
	const FTaskSrcData TaskData;

	/** Downsampled destination texture. Populated by Generate().  Alpha channel contains distance field */
	TArray<uint8> DstTexture;
	/** Width of the destination texture */
	int32 DstSizeX;
	/** Height of the destination texture */
	int32 DstSizeY;
	/** Format of the destination texture */
	uint8 DstFormat;

	/** A task that builds the distance field for a strip of the image. */
	class FBuildDistanceFieldTask : public FNonAbandonableTask
	{
	public:
		/**
		 * Initialize the task as follows:
		 *
		 * @param InThreadScaleCounter  Used by the invoker to track when tasks are completed.
		 * @param InTaskData            Source data from which to build the distance field
		 * @param InStartRow            Row on which this task should start.
		 * @param InNumRowsToProcess    How many rows this task should process
		 * @param InScanRadius          Radius of area to scan for nearest border
		 * @param InScaleFactor         The input image and resulting distance field
		 */
		FBuildDistanceFieldTask(
			FThreadSafeCounter* InThreadScaleCounter,
			TArray<float>* OutSignedDistances,
			const FTaskSrcData* InTaskData,
			int32 InStartRow,
			int32 InDstRowWidth,
			int32 InNumRowsToProcess,
			int32 InScanRadius,
			int32 InScaleFactor)
			: ThreadScaleCounter(InThreadScaleCounter)
			, SignedDistances(*OutSignedDistances)
			, TaskData(InTaskData)
			, StartRow(InStartRow)
			, DstRowWidth(InDstRowWidth)
			, NumRowsToProcess(InNumRowsToProcess)
			, ScanRadius(InScanRadius)
			, ScaleFactor(InScaleFactor)
		{
		}

		/**
		 * Calculate the signed distance at the given coordinate to the 
		 * closest silhouette edge of the source texture.  
		 * 
		 * If the current point is solid then the closest non-solid 
		 * pixel is the edge, and if the current point is non-solid
		 * then the closest solid pixel is the edge.
		 *
		 * @param PointX - x coordinate
		 * @param PointY - y coordinate
		 * @param ScanRadius - radius (in texels) to scan the source texture for a silhouette edge
		 * @return distance to silhouette edge
		 */
		float CalcSignedDistanceToSrc(int32 PointX, int32 PointY, int32 ScanRadius) const;

		/** Called by the thread pool to do the work in this task */
		void DoWork(void);

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FBuildDistanceFieldTask, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		FThreadSafeCounter* ThreadScaleCounter;
		TArray<float>& SignedDistances;
		const FTaskSrcData *TaskData;
		const int32 StartRow;
		const int32 DstRowWidth;
		const int32 NumRowsToProcess;
		const int32 ScanRadius;
		const int32 ScaleFactor;
	};
	
public:

	/**
	* Constructor
	*
	* @param InSrcTexture - source texture data
	* @param InSrcSizeX - source texture width
	* @param InSrcSizeY - source texture height
	* @param InSrcFormat - source texture format
	*/	
	FTextureAlphaToDistanceField(const uint8* InSrcTexture, int32 InSrcSizeX, int32 InSrcSizeY, uint8 InSrcFormat)
	:	TaskData(InSrcTexture, InSrcSizeX, InSrcSizeY, InSrcFormat)
	,	DstSizeX(0)
	,	DstSizeY(0)
	,	DstFormat(PF_B8G8R8A8)
	{
	}
	
	// Accessors
	
	const uint8* GetResultTexture() const
	{
		return DstTexture.GetData();
	}
	const int32 GetResultTextureSize() const
	{
		return DstTexture.Num();
	}	
	const int32 GetResultSizeX() const
	{
		return DstSizeX;
	}	
	const int32 GetResultSizeY() const
	{
		return DstSizeY;
	}
	
	/**
	* Generates the destination texture from the source texture.
	* The alpha channel of the destination texture contains the
	* signed distance field.
	*
	* The destination texture size is scaled based on the ScaleFactor.
	* Eg. a scale factor of 4 creates a destination texture 4x smaller.
	*
	* @param ScaleFactor - downsample scale from source to destination texture
	* @param ScanRadius - distance in texels to scan high res font for the silhouette
	*/
	void Generate(int32 ScaleFactor, int32 ScanRadius);

	/**
	 * Calculate the distance between 2 coordinates
	 *
	 * @param X1 - 1st x coordinate
	 * @param Y1 - 1st y coordinate
	 * @param X2 - 2nd x coordinate
	 * @param Y2 - 2nd y coordinate
	 * @return 2d distance between the coordinates 
	 */
	static FORCEINLINE float CalcDistance(int32 X1, int32 Y1, int32 X2, int32 Y2)
	{
		int32 DX = X1 - X2;
		int32 DY = Y1 - Y2;
		return FMath::Sqrt(static_cast<float>(DX*DX + DY*DY));
	}
	
};




/**
 * Generates the destination texture from the source texture.
 * The alpha channel of the destination texture contains the
 * signed distance field.
 *
 * The destination texture size is scaled based on the ScaleFactor.
 * Eg. a scale factor of 4 creates a destination texture 4x smaller.
 *
 * @param ScaleFactor - downsample scale from source to destination texture
 * @param ScanRadius - distance in texels to scan high res font for the silhouette
 */
void FTextureAlphaToDistanceField::Generate(int32 ScaleFactor, int32 ScanRadius)
{
	// restart progress bar for distance field generation as this can be slow
	GWarn->StatusUpdate(0,0, NSLOCTEXT("TextureAlphaToDistanceField", "BeginGeneratingDistanceFieldTask", "Generating Distance Field"));

	// need to maintain pow2 sizing for textures
	ScaleFactor = FMath::RoundUpToPowerOfTwo(ScaleFactor);
	DstSizeX = TaskData.SrcSizeX / ScaleFactor;
	DstSizeY = TaskData.SrcSizeY / ScaleFactor;

	// destination texture
	// note that destination format can be different from source format	
	int32 NumBytes = IntCastChecked<int32>(CalculateImageBytes(DstSizeX,DstSizeY,0,DstFormat));
	DstTexture.Empty(NumBytes);	
	DstTexture.AddZeroed(NumBytes);
	
	// array of signed distance values for the downsampled texture
	TArray<float> SignedDistance;
	SignedDistance.Empty(DstSizeX*DstSizeY);
	SignedDistance.AddUninitialized(DstSizeX*DstSizeY);
	
	// We want to run the distance field computation as a maximum of 16 tasks for a speed boost on multi-core machines.
	const int32 NumTasks = FMath::Min(DstSizeY, 16); //ensure we don't split the work so much that we end up with a null DstStripHeight... and an infinite loop.
	FThreadSafeCounter BuildTasksCounter;
	int32 DstStripHeight = DstSizeY / NumTasks;
	
	// We need to report the progress, and all the threads must be able to update it.
	const int32 TotalDistFieldWork = DstStripHeight * NumTasks;


	int32 TotalProgress = DstSizeY-1;	
	// calculate distances
	for (int32 y=0; y < DstSizeY; y+=DstStripHeight)
	{
		DstStripHeight = FMath::Min(DstStripHeight, DstSizeY - y); //Don't go out of bound.

		// The tasks will clean themselves up when they are completed; no need to call delete elsewhere.
		BuildTasksCounter.Increment();
		(new FAutoDeleteAsyncTask<FBuildDistanceFieldTask>(
			&BuildTasksCounter,
			&SignedDistance,
			&TaskData,
			y,
			DstSizeX,
			DstStripHeight,
			ScanRadius,
			ScaleFactor))->StartBackgroundTask();
	}

	while( BuildTasksCounter.GetValue() > 0 )
	{
		// Waiting for Distance Field to finish generating.
		GWarn->UpdateProgress(TotalDistFieldWork - BuildTasksCounter.GetValue(),TotalDistFieldWork);
		FPlatformProcess::Sleep(.1f);
	}
	
	// find min,max range of distances
	const float BadMax = CalcDistance(0,0,TaskData.SrcSizeX,TaskData.SrcSizeY);;
	const float BadMin = -BadMax;
	float MaxDistance = BadMin;
	float MinDistance = BadMax;
	for (int32 i=0; i < SignedDistance.Num(); i++)
	{
		if (SignedDistance[i] > BadMin &&
			SignedDistance[i] < BadMax)
		{
			MinDistance = FMath::Min<float>(MinDistance,SignedDistance[i]);
			MaxDistance = FMath::Max<float>(MaxDistance,SignedDistance[i]);
		}
	}
	
	// normalize distances
	float RangeDistance = MaxDistance - MinDistance;
	for (int32 i=0; i < SignedDistance.Num(); i++)
	{
		// clamp edge cases that were never found due to limited scan radius
		if (SignedDistance[i] <= MinDistance)
		{			
			SignedDistance[i] = 0.0f;
		}
		else if (SignedDistance[i] >= MaxDistance)
		{
			SignedDistance[i] = 1.0f;		
		}
		else
		{
			// normalize and remap from [-1,+1] to [0,+1]
			SignedDistance[i] = SignedDistance[i] / RangeDistance * 0.5f + 0.5f;		
		}
	}
	
	// copy results to the destination texture
	if (DstFormat == PF_G8)
	{
		for( int32 x=0; x < DstSizeX; x++ )
		{
			for( int32 y=0; y < DstSizeY; y++ )
			{
				DstTexture[x + y * DstSizeX] = static_cast<uint8>(SignedDistance[x + y * DstSizeX] * 255);
			}
		}
	}
	else if (DstFormat == PF_B8G8R8A8)
	{
		for( int32 x=0; x < DstSizeX; x++ )
		{
			for( int32 y=0; y < DstSizeY; y++ )
			{
				const FColor SrcColor(TaskData.GetSourceColor((x*ScaleFactor) + (ScaleFactor / 2), (y*ScaleFactor) + (ScaleFactor / 2)));
				DstTexture[4 * (x + y * DstSizeX) + 0] = SrcColor.B;
				DstTexture[4 * (x + y * DstSizeX) + 1] = SrcColor.G;
				DstTexture[4 * (x + y * DstSizeX) + 2] = SrcColor.R;
				DstTexture[4 * (x + y * DstSizeX) + 3] = static_cast<uint8>(SignedDistance[x + y * DstSizeX] * 255);
			}
		}	
	}
	else
	{
		checkf(0,TEXT("unsupported format specified"));
	}
}

/**
 * Calculate the signed distance at the given coordinate to the 
 * closest silhouette edge of the source texture.  
 * 
 * If the current point is solid then the closest non-solid 
 *  pixel is the edge, and if the current point is non-solid
 * then the closest solid pixel is the edge.
 *
 * @param PointX - x coordinate
 * @param PointY - y coordinate
 * @param ScanRadius - radius (in texels) to scan the source texture for a silhouette edge
 *
 * @return distance to silhouette edge
 */
float FTextureAlphaToDistanceField::FBuildDistanceFieldTask::CalcSignedDistanceToSrc(int32 PointX, int32 PointY,int32 InScanRadius) const
{
	// determine whether or not the source point is solid
	const bool BaseIsSolid = TaskData->GetSourceAlpha(PointX,PointY) > 0;

	float ClosestDistance = FTextureAlphaToDistanceField::CalcDistance(0,0,TaskData->SrcSizeX,TaskData->SrcSizeY);
	
	// If the current point is solid then the closest non-solid 
	// pixel is the edge, and if the current point is non-solid
	// then the closest solid pixel is the edge.

	// Search pattern:
	//   Use increasing ring sizes allows us to early out.
	//   In the picture below 1s indicate the first ring
	//   while 2s indicate the 2nd ring.
	//
	//        2 2 2 2 2
	//        2 1 1 1 2
	//        2 1 * 1 2
	//        2 1 1 1 2
	//        2 2 2 2 2
	//  
	// Note that the "rings" are not actually circular, so
	// we may find a sample that is up to Sqrt(2*(RingSize^2)) away.
	// Once we have found such a sample, we must search a few more
	// rings in case a nearer sample can be found.

	bool bFoundClosest = false;
	int32 RequiredRadius = InScanRadius;
	for (int RingSize=1; RingSize <= RequiredRadius; ++RingSize)
	{
		const int32 StartX = FMath::Clamp<int32>(PointX - RingSize,0,TaskData->SrcSizeX);
		const int32 EndX  = FMath::Clamp<int32>(PointX + RingSize,0,TaskData->SrcSizeX-1);
		const int32 StartY = FMath::Clamp<int32>(PointY - RingSize,0,TaskData->SrcSizeY);
		const int32 EndY = FMath::Clamp<int32>(PointY + RingSize,0,TaskData->SrcSizeY-1);
		
		//    - - -    <-- Search this top line
		//    . * .
		//    . . . 
		for (int x=StartX; x<=EndX; ++x)
		{
			const int32 y = StartY;
			const uint8 SrcAlpha(TaskData->GetSourceAlpha(x, y));
			
			if ((BaseIsSolid && SrcAlpha == 0) || (!BaseIsSolid && SrcAlpha > 0))
			{
				const float Dist = CalcDistance(PointX, PointY, x, y);
				ClosestDistance = FMath::Min<float>(Dist, ClosestDistance);
				bFoundClosest = true;
			}
		}

		//    . . .    
		//    . * .
		//    - - -    <-- Search the bottom line
		for (int x=StartX; x<=EndX; ++x)
		{
			const int32 y = EndY;
			const uint8 SrcAlpha(TaskData->GetSourceAlpha(x, y));

			if ((BaseIsSolid && SrcAlpha == 0) || (!BaseIsSolid && SrcAlpha > 0))
			{
				const float Dist = CalcDistance(PointX, PointY, x, y);
				ClosestDistance = FMath::Min<float>(Dist, ClosestDistance);
				bFoundClosest = true;
			}
		}

		//    . . .    
		//    - * -   <-- Search the left and right two vertical lines
		//    . . .    
		for (int y=StartY+1; y<=EndY-1; ++y)
		{
			{
				const int32 x = StartX;
				const uint8 SrcAlpha(TaskData->GetSourceAlpha(x, y));

				if ((BaseIsSolid && SrcAlpha == 0) || (!BaseIsSolid && SrcAlpha > 0))
				{
					const float Dist = CalcDistance(PointX, PointY, x, y);
					ClosestDistance = FMath::Min<float>(Dist, ClosestDistance);
					bFoundClosest = true;
				}
			}

			{
				const int32 x = EndX;
				const uint8 SrcAlpha(TaskData->GetSourceAlpha(x, y));

				if ((BaseIsSolid && SrcAlpha == 0) || (!BaseIsSolid && SrcAlpha > 0))
				{
					const float Dist = CalcDistance(PointX, PointY, x, y);
					ClosestDistance = FMath::Min<float>(Dist, ClosestDistance);
					bFoundClosest = true;
				}
			}
		}

		// We have found a sample on the edge, but we might have to search 
		// a few more rings to guarantee that we found the closest sample
		// on the edge.
		if ( bFoundClosest && RequiredRadius >= InScanRadius )
		{
			RequiredRadius = FMath::CeilToInt( FMath::Sqrt(static_cast<float>(RingSize*RingSize*2)) );
			RequiredRadius = FMath::Min(RequiredRadius, InScanRadius);
		}
	}

	// positive distance if inside and negative if outside
	return BaseIsSolid ? ClosestDistance : -ClosestDistance;
}



/** Called by the thread pool to do the work in this task */
void FTextureAlphaToDistanceField::FBuildDistanceFieldTask::DoWork(void)
{
	// Build the distance field for the strip specified for this task.
	for (int32 y=StartRow; y < (StartRow+NumRowsToProcess); y++)
	{
		// Build distance field for a single line
		for (int32 x=0; x < DstRowWidth; x++)
		{
			SignedDistances[x + y * DstRowWidth] = CalcSignedDistanceToSrc( 
				(x * ScaleFactor) + (ScaleFactor / 2),
				(y * ScaleFactor) + (ScaleFactor / 2),
				ScanRadius );
		}
	}
	ThreadScaleCounter->Decrement();
}



#if PLATFORM_WINDOWS

UTexture2D* UTrueTypeFontFactory::CreateTextureFromDC( UFont* Font, HDC dc, const int32 BitmapWidth, const int32 BitmapHeight, int32 TextureNum )
{
	FString TextureString = FString::Printf( TEXT("%s_Page"), *Font->GetName() );
	if( TextureNum < 26 )
	{
		TextureString = TextureString + FString::Printf(TEXT("%c"), 'A'+TextureNum);
	}
	else
	{
		TextureString = TextureString + FString::Printf(TEXT("%c%c"), 'A'+TextureNum/26, 'A'+TextureNum%26 );
	}

	if( StaticFindObject( NULL, Font, *TextureString ) )
	{
		UE_LOGF(LogTTFontImport, Warning, "A texture named %ls already exists!", *TextureString );
	}
	
	// Create texture for page.
	auto Texture = NewObject<UTexture2D>(Font, *TextureString);

	// note RF_Public because font textures can be referenced directly by material expressions
	Texture->SetFlags(RF_Public);
	Texture->Source.Init(BitmapWidth, BitmapHeight, 1, 1, TSF_BGRA8);

	// Copy the LODGroup from the font factory to the new texture
	// By default, this should be TEXTUREGROUP_UI for fonts!
	Texture->LODGroup = LODGroup;

	// Also, we never want to stream in font textures since that always looks awful
	Texture->NeverStream = true;

	// Copy bitmap data to texture page.
	FColor FontColor8Bit(ImportOptions->Data.ForegroundColor.ToFColor(true));

	// restart progress bar for font bitmap generation since this takes a while
	int32 TotalProgress = BitmapWidth-1;

	FFormatNamedArguments Args;
	Args.Add( TEXT("TextureNum"), TextureNum );
	GWarn->StatusUpdate(0,0, FText::Format( NSLOCTEXT("TrueTypeFontImport", "GeneratingFontPageStatusUpdate", "Generating font page {TextureNum}"), Args ));

	TArray<int32> SourceData;	
	// Copy the data from the Device Context to our SourceData array.
	// This approach is much faster than using GetPixel()!
	{
		// We must make a new bitmap to populate with data from the DC
		BITMAPINFO BitmapInfo;
		BitmapInfo.bmiHeader.biBitCount = 32;
		BitmapInfo.bmiHeader.biCompression = BI_RGB;
		BitmapInfo.bmiHeader.biPlanes = 1;
		BitmapInfo.bmiHeader.biSize = sizeof(BitmapInfo.bmiHeader);
		BitmapInfo.bmiHeader.biWidth = BitmapWidth;
		BitmapInfo.bmiHeader.biHeight = -BitmapHeight; // Having a positive height results in an upside-down bitmap

		// Initialize the Bitmap and the Device Context in a way that they are automatically cleaned up.
		struct CleanupResourcesScopeGuard
		{
			HDC BitmapDC;
			HBITMAP BitmapHandle;
			~CleanupResourcesScopeGuard()
			{
				DeleteDC( BitmapDC );
				DeleteObject( BitmapHandle );
			}
		} Resources;
		Resources.BitmapDC = CreateCompatibleDC(dc);

		void* BitsPtr;
		Resources.BitmapHandle = CreateDIBSection(Resources.BitmapDC, &BitmapInfo, DIB_RGB_COLORS, &BitsPtr, 0, 0);

		if (Resources.BitmapHandle)
		{
			// Bind the bitmap to the Device Context
			SelectObject(Resources.BitmapDC, Resources.BitmapHandle);

			// Copy from the Device Context to the Bitmap we created
			BitBlt(Resources.BitmapDC, 0, 0, BitmapWidth, BitmapHeight, dc, 0, 0, TTFConstants::WIN_SRCCOPY);

			// Finally copy the data from the Bitmap into an Unreal data array.
			SourceData.AddUninitialized(BitmapWidth * BitmapHeight);
			GetDIBits( Resources.BitmapDC, Resources.BitmapHandle, 0, BitmapHeight, SourceData.GetData(), &BitmapInfo, DIB_RGB_COLORS );
		}
	}

	uint8* MipData = Texture->Source.LockMip(0);
	if( !ImportOptions->Data.bEnableAntialiasing )
	{
		int32 SizeX = Texture->Source.GetSizeX();
		int32 SizeY = Texture->Source.GetSizeY();

		for( int32 i=0; i<SizeX; i++ )
		{
			// update progress bar
			GWarn->UpdateProgress(i,TotalProgress);
			
			for( int32 j=0; j<SizeY; j++ )
			{
				int32 CharAlpha = SourceData[i+j*BitmapWidth];
				int32 DropShadowAlpha;

				if( ImportOptions->Data.bEnableDropShadow && i > 0 && j > 0 )
				{
					DropShadowAlpha = ( (i-1)+(j-1)*BitmapWidth);
				}
				else
				{
					DropShadowAlpha = 0;
				}

				if( CharAlpha )
				{
					MipData[4 * (i + j * SizeX) + 0] = FontColor8Bit.B;
					MipData[4 * (i + j * SizeX) + 1] = FontColor8Bit.G;
					MipData[4 * (i + j * SizeX) + 2] = FontColor8Bit.R;
					MipData[4 * (i + j * SizeX) + 3] = 0xFF;
				}
				else if( DropShadowAlpha )
				{
					MipData[4 * (i + j * SizeX) + 0] = 0x00;
					MipData[4 * (i + j * SizeX) + 1] = 0x00;
					MipData[4 * (i + j * SizeX) + 2] = 0x00;
					MipData[4 * (i + j * SizeX) + 3] = 0xFF;
				}
				else
				{
					MipData[4 * (i + j * SizeX) + 0] = FontColor8Bit.B;
					MipData[4 * (i + j * SizeX) + 1] = FontColor8Bit.G;
					MipData[4 * (i + j * SizeX) + 2] = FontColor8Bit.R;
					MipData[4 * (i + j * SizeX) + 3] = 0x00;
				}
			}
		}
	}
	else
	{
		int32 SizeX = Texture->Source.GetSizeX();
		int32 SizeY = Texture->Source.GetSizeY();

		for( int32 i=0; i<SizeX; i++ )
		{
			for( int32 j=0; j<SizeY; j++ )
			{
				int32 CharAlpha = SourceData[i+j*BitmapWidth];
				float fCharAlpha = float( CharAlpha ) / 255.0f;

				int32 DropShadowAlpha = 0;
				if( ImportOptions->Data.bEnableDropShadow && i > 0 && j > 0 )
				{
					// Character opacity takes precedence over drop shadow opacity
					DropShadowAlpha =
						( uint8 )( ( 1.0f - fCharAlpha ) * ( float )GetRValue( SourceData[(i - 1)+(j - 1)*BitmapWidth] ) );
				}
				float fDropShadowAlpha = float( DropShadowAlpha ) / 255.0f;

				// Color channel = Font color, except for drop shadow pixels
				MipData[4 * (i + j * SizeX) + 0] = ( uint8 )( FontColor8Bit.B * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 1] = ( uint8 )( FontColor8Bit.G * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 2] = ( uint8 )( FontColor8Bit.R * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 3] = ( uint8 )( CharAlpha + DropShadowAlpha );
			}
		}
	}
	Texture->Source.UnlockMip(0);
	MipData = NULL;
	
	// convert bitmap font alpha channel to distance field
	if (ImportOptions->Data.bUseDistanceFieldAlpha)
	{
		// Initialize distance field generator with high res source bitmap texels
		FTextureAlphaToDistanceField DistanceFieldTex(
			Texture->Source.LockMip(0),
			Texture->Source.GetSizeX(),
			Texture->Source.GetSizeY(),
			PF_B8G8R8A8
			);
		// estimate scan radius based on half font height scaled by bitmap scale factor
		const int32 ScanRadius = static_cast<int32>((ImportOptions->Data.Height / 2.0f) * ImportOptions->Data.DistanceFieldScaleFactor * ImportOptions->Data.DistanceFieldScanRadiusScale);
		// generate downsampled distance field using high res source bitmap
		DistanceFieldTex.Generate(ImportOptions->Data.DistanceFieldScaleFactor,ScanRadius);
		check(DistanceFieldTex.GetResultTextureSize() > 0);
		Texture->Source.UnlockMip(0);	
		// resize/update texture using distance field values
		Texture->Source.Init(
			DistanceFieldTex.GetResultSizeX(),
			DistanceFieldTex.GetResultSizeY(),
			/*NumSlices=*/ 1,
			/*NumMips=*/ 1,
			TSF_BGRA8
			);
		FMemory::Memcpy(Texture->Source.LockMip(0),DistanceFieldTex.GetResultTexture(),DistanceFieldTex.GetResultTextureSize());		
		Texture->Source.UnlockMip(0);		
		// use PF_G8 for all distance field textures for better precision than DXT5
		Texture->CompressionSettings = TC_DistanceFieldFont;
		// disable gamma correction since storing alpha in linear color for PF_G8
		Texture->SRGB = false;
	}
	else
	{
		// if we dont care about colors then store texture as PF_G8
		if (ImportOptions->Data.bAlphaOnly &&
			!ImportOptions->Data.bEnableDropShadow)
		{
			// Not a distance field texture, but we use the same compression settings for better precision than DXT5
			Texture->CompressionSettings = TC_DistanceFieldFont;
			// disable gamma correction since storing alpha in linear color for PF_G8
			Texture->SRGB = false;
		}
	}	
	
	Texture->MipGenSettings = TMGS_NoMipmaps;
	Texture->PostEditChange();

	return Texture;
}

#if !USE_FREETYPE

bool UTrueTypeFontFactory::CreateFontTexture(
	UFont* Font,
	FFeedbackContext* Warn,
	const int32 NumResolutions,
	const int32 CharsPerPage,
	const TMap<TCHAR,TCHAR>& InverseMap,
	const TArray< float >& ResHeights )
{
	HDC tempDC = CreateCompatibleDC( NULL );
	// Always target 72 DPI. Microsoft documentation suggests that the correct way
	// to create a font is :
	//         const float LogicalPixelsY = static_cast<float>(GetDeviceCaps(tempDC, LOGPIXELSY)) / 72.0f;
	// However, we are planning to bake this font into a texture.
	// Assume that artists will set their photoshop documents to 72.0 DPI (the default).
	const float LogicalPPIYRatio = 1.0f; // 72.0 / 72.0 = 1.0
	
	DeleteDC( tempDC );

	const int32 TotalProgress = NumResolutions * CharsPerPage;

	// Zero out the Texture Index
	int32 CurrentTexture = 0;

	uint32 ImportCharSet = DEFAULT_CHARSET;
	switch ( ImportOptions->Data.CharacterSet )
	{
		case FontICS_Ansi:
			ImportCharSet = ANSI_CHARSET;
			break;

		case FontICS_Default:
			ImportCharSet = DEFAULT_CHARSET;
			break;

		case FontICS_Symbol:
			ImportCharSet = SYMBOL_CHARSET;
			break;
	}

	for( int32 Page = 0; Page < NumResolutions; ++Page )
	{
		int32 nHeight = -FMath::RoundToInt( ResHeights[Page] * LogicalPPIYRatio );
		
		// scale font height to generate high res bitmap based on scale factor
		// this high res bitmap is later used to generate the downsampled distance field
		if (ImportOptions->Data.bUseDistanceFieldAlpha)
		{
			nHeight *= ImportOptions->Data.DistanceFieldScaleFactor;
		}

		// Create the Windows font
		HFONT FontHandle =
			CreateFontW(
				nHeight,
				0,
				0,
				0,
				ImportOptions->Data.bEnableBold ? FW_BOLD : FW_NORMAL,
				ImportOptions->Data.bEnableItalic,
				ImportOptions->Data.bEnableUnderline,
				0,
				ImportCharSet,
				OUT_DEFAULT_PRECIS,
				CLIP_DEFAULT_PRECIS,
				ImportOptions->Data.bEnableAntialiasing ? ANTIALIASED_QUALITY : NONANTIALIASED_QUALITY,
				VARIABLE_PITCH,
				*ImportOptions->Data.FontName );

		if( FontHandle == NULL ) 
		{
			TCHAR ErrorBuffer[1024];
			Warn->Logf(ELogVerbosity::Error, TEXT("CreateFont failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
			return false;
		}

		// Create DC
		HDC DeviceDCHandle = GetDC( NULL );
		if( DeviceDCHandle == NULL )
		{
			TCHAR ErrorBuffer[1024];
			Warn->Logf(ELogVerbosity::Error, TEXT("GetDC failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
			return false;
		}

		HDC DCHandle = CreateCompatibleDC( DeviceDCHandle );
		if( !DCHandle )
		{
			TCHAR ErrorBuffer[1024];
			Warn->Logf(ELogVerbosity::Error, TEXT("CreateDC failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
			return false;
		}

		// Create bitmap
		BITMAPINFO WinBitmapInfo;
		FMemory::Memzero( &WinBitmapInfo, sizeof( WinBitmapInfo ) );
		HBITMAP BitmapHandle;
		void* BitmapDataPtr = NULL;
		
		const int32 ActualDistanceFieldScaleFactor = ImportOptions->Data.bUseDistanceFieldAlpha ? ImportOptions->Data.DistanceFieldScaleFactor : 1;
		const int32 BitmapWidth = ImportOptions->Data.TexturePageWidth * ActualDistanceFieldScaleFactor;
		const int32 BitmapHeight = ImportOptions->Data.TexturePageMaxHeight * ActualDistanceFieldScaleFactor;
		const int32 BitmapPaddingX = ImportOptions->Data.XPadding * ActualDistanceFieldScaleFactor;
		const int32 BitmapPaddingY = ImportOptions->Data.YPadding * ActualDistanceFieldScaleFactor;

		if( ImportOptions->Data.bEnableAntialiasing )
		{
			WinBitmapInfo.bmiHeader.biSize          = sizeof(BITMAPINFOHEADER);			
			WinBitmapInfo.bmiHeader.biWidth         = ImportOptions->Data.TexturePageWidth;
			WinBitmapInfo.bmiHeader.biHeight        = ImportOptions->Data.TexturePageMaxHeight;
			WinBitmapInfo.bmiHeader.biPlanes        = 1;      //  Must be 1
			WinBitmapInfo.bmiHeader.biBitCount      = 32;
			WinBitmapInfo.bmiHeader.biCompression   = BI_RGB; 
			WinBitmapInfo.bmiHeader.biSizeImage     = 0;      
			WinBitmapInfo.bmiHeader.biXPelsPerMeter = 0;      
			WinBitmapInfo.bmiHeader.biYPelsPerMeter = 0;      
			WinBitmapInfo.bmiHeader.biClrUsed       = 0;      
			WinBitmapInfo.bmiHeader.biClrImportant  = 0;      

			BitmapHandle = CreateDIBSection(
				(HDC)NULL, 
				&WinBitmapInfo,
				DIB_RGB_COLORS,
				&BitmapDataPtr,
				NULL,
				0);  
		}
		else
		{
			BitmapHandle = CreateBitmap( BitmapWidth, BitmapHeight, 1, 1, NULL);
		}

		if( BitmapHandle == NULL )
		{
			TCHAR ErrorBuffer[1024];
			Warn->Logf(ELogVerbosity::Error, TEXT("CreateBitmap failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
			return false;
		}

		SelectObject( DCHandle, FontHandle );

		// Grab size information for this font
		TEXTMETRIC WinTextMetrics;
		GetTextMetrics( DCHandle, &WinTextMetrics );

		float EmScale = 1024.f/-nHeight;
		Font->EmScale = EmScale;
		if (ImportOptions->Data.bUseDistanceFieldAlpha)
		{
			Font->EmScale *= ImportOptions->Data.DistanceFieldScaleFactor;
		}
		Font->Ascent = WinTextMetrics.tmAscent * EmScale;
		Font->Descent = WinTextMetrics.tmDescent * EmScale;
		Font->Leading = WinTextMetrics.tmExternalLeading * EmScale;

		HBITMAP LastBitmapHandle = ( HBITMAP )SelectObject( DCHandle, BitmapHandle );
		SetTextColor( DCHandle, 0x00ffffff );
		SetBkColor( DCHandle, 0x00000000 );

		// clear the bitmap
		HBRUSH Black = CreateSolidBrush(0x00000000);
		RECT r = {0, 0, BitmapWidth, BitmapHeight};
		FillRect( DCHandle, &r, Black );

		int32 X=BitmapPaddingX, Y=BitmapPaddingY, RowHeight=0;

		for( int32 CurCharIndex = 0; CurCharIndex < CharsPerPage; ++CurCharIndex )
		{
			GWarn->UpdateProgress( Page * CharsPerPage + CurCharIndex, TotalProgress );

			// Remap the character if we need to
			TCHAR Char = ( TCHAR )CurCharIndex;
			if( Font->IsRemapped )
			{
				const TCHAR* FoundRemappedChar = InverseMap.Find( Char );
				if( FoundRemappedChar != NULL )
				{
					Char = *FoundRemappedChar;
				}
				else
				{
					// Skip missing remapped character
					continue;
				}
			}


			// Skip ASCII character if it isn't in the list of characters to import.
			if( Char < 256 && ImportOptions->Data.Chars != TEXT("") && (!Char || !FCString::Strchr(*ImportOptions->Data.Chars, Char)) )
			{
				continue;
			}

			// Skip if the user has requested that only printable characters be
			// imported and the character isn't printable
			if( ImportOptions->Data.bCreatePrintableOnly == true && iswprint(Char) == false )
			{
				continue;
			}

			
			// Compute the size of the character
			int32 CharWidth = 0;
			int32 CharHeight = 0;
			{
				TCHAR Tmp[2];
				Tmp[0] = Char;
				Tmp[1] = 0;

				SIZE Size;
				GetTextExtentPoint32( DCHandle, Tmp, 1, &Size );

				CharWidth = Size.cx;
				CharHeight = Size.cy;
			}
			
			
			// OK, now try to grab glyph data using the GetGlyphOutline API.  This is only supported for vector-based fonts
			// like TrueType and OpenType; it won't work for raster fonts!
			bool bUsingGlyphOutlines = false;
			GLYPHMETRICS WinGlyphMetrics;
			const MAT2 WinIdentityMatrix2x2 = { { 0, 1 }, { 0, 0 }, { 0, 0 }, { 0, 1 } };
			int32 VerticalOffset = 0;
			uint32 GGODataSize = 0;
			if( !ImportOptions->Data.bEnableLegacyMode && ImportOptions->Data.bEnableAntialiasing )    // We only bother using GetGlyphOutline for AntiAliased fonts!
			{
				GGODataSize =
					GetGlyphOutlineW(
						DCHandle,                         // Device context
						Char,	                            // Character
						GGO_GRAY8_BITMAP,                 // Format
						&WinGlyphMetrics,                 // Out: Metrics
						0,																// Output buffer size
						NULL,															// Glyph data buffer or NULL
						&WinIdentityMatrix2x2 );          // Transform

				if( GGODataSize != GDI_ERROR && GGODataSize != 0)
				{
					CharWidth = WinGlyphMetrics.gmBlackBoxX;
					CharHeight = WinGlyphMetrics.gmBlackBoxY;

					VerticalOffset = WinTextMetrics.tmAscent - WinGlyphMetrics.gmptGlyphOrigin.y;

					// Extend the width of the character by 1 (empty) pixel for spacing purposes.  Note that with the legacy
					// font import, we got this "for free" from TextOut
					// @todo frick: Properly support glyph origins and cell advancement!  The only reason we even want to
					//    to continue to do this is to prevent texture bleeding across glyph cell UV rects
					++CharWidth;

					bUsingGlyphOutlines = true;
				}
				else
				{
					// GetGlyphOutline failed; it's probably a raster font.  Oh well, no big deal.
				}
			}

			// Adjust character dimensions to accommodate a drop shadow
			if( ImportOptions->Data.bEnableDropShadow )
			{
				CharWidth += 1;
				CharHeight += 1;
			}
			if (ImportOptions->Data.bUseDistanceFieldAlpha)
			{
				
				// Make X and Y positions a multiple of the scale factor.
				CharWidth = FMath::RoundToInt(CharWidth / static_cast<float>(ImportOptions->Data.DistanceFieldScaleFactor)) * ImportOptions->Data.DistanceFieldScaleFactor;
				CharHeight = FMath::RoundToInt(CharHeight / static_cast<float>(ImportOptions->Data.DistanceFieldScaleFactor)) * ImportOptions->Data.DistanceFieldScaleFactor;
			}

			// If the character is bigger than our texture size, then this isn't going to work!  The user
			// will need to specify a larger texture resolution
			const int32 RequiredWidth = CharWidth + BitmapPaddingX * 2;
			const int32 RequiredHeight = CharHeight + BitmapPaddingY * 2;
			const bool bisBitmapTooNarrow = RequiredWidth > BitmapWidth;
			const bool bisBitmapTooShort = RequiredHeight > BitmapHeight;
			if (bisBitmapTooNarrow || bisBitmapTooShort)
			{
				UE_LOGF(LogTTFontImport, Error, "At the specified font size, at least one font glyph would be larger than the maximum texture size you specified.");
				if (bisBitmapTooNarrow)
				{
					UE_LOGF(LogTTFontImport, Error, "You should increase TexturePageWidth to at least %d.", RequiredWidth / ActualDistanceFieldScaleFactor);
				}
				else
				{
					UE_LOGF(LogTTFontImport, Error, "You should increase TexturePageMaxHeight to at least %d.", RequiredHeight / ActualDistanceFieldScaleFactor);
				}

				DeleteDC( DCHandle );
				DeleteObject( BitmapHandle );
				return false;
			}

			// If it doesn't fit right here, advance to next line.
			if( (CharWidth + X + BitmapPaddingX) > BitmapWidth)
			{
				X=BitmapPaddingX;
				Y = Y + RowHeight + BitmapPaddingY;
				RowHeight = 0;
			}
			int32 OldRowHeight = RowHeight;
			if( CharHeight > RowHeight )
			{
				RowHeight = CharHeight;
			}

			// new page
			if( (Y + RowHeight + BitmapPaddingY) > BitmapHeight )
			{
				const int32 RoundedHeight = RoundUpToMultipleOf4(Y + OldRowHeight);
				check(RoundedHeight <= BitmapHeight);
				Font->Textures.Add( CreateTextureFromDC( Font, DCHandle, BitmapWidth, RoundedHeight, CurrentTexture ) );
				CurrentTexture++;

				// blank out DC
				FillRect( DCHandle, &r, Black );

				X=BitmapPaddingX;
				Y=BitmapPaddingY;
				
				RowHeight = 0;
			}

			// NOTE: This extra offset is for backwards compatibility with legacy TT/raster fonts.  With the new method
			//   of importing fonts, this offset is not needed since the glyphs already have the correct vertical size
			const int32 ExtraVertOffset = bUsingGlyphOutlines ? 0 : 1;

			// Set font character information.
			FFontCharacter& NewCharacterRef = Font->Characters[ CurCharIndex + (CharsPerPage * Page)];
			int32 FontX = X;
			int32 FontY = Y;
			int32 FontWidth = CharWidth;
			int32 FontHeight = CharHeight;
			// scale character offset UVs back down based on scale factor
			if (ImportOptions->Data.bUseDistanceFieldAlpha)
			{
				FontX = FMath::RoundToInt(FontX / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontY = FMath::RoundToInt(FontY / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontWidth = FMath::RoundToInt(FontWidth / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontHeight = FMath::RoundToInt(FontHeight / (float)ImportOptions->Data.DistanceFieldScaleFactor);
			}
			NewCharacterRef.StartU =
				FMath::Clamp<int32>( FontX - ImportOptions->Data.ExtendBoxLeft,
							0, ImportOptions->Data.TexturePageWidth - 1 );
			NewCharacterRef.StartV =
				FMath::Clamp<int32>( FontY + ExtraVertOffset-ImportOptions->Data.ExtendBoxTop,
							0, ImportOptions->Data.TexturePageMaxHeight - 1 );
			NewCharacterRef.USize =
				FMath::Clamp<int32>( FontWidth + ImportOptions->Data.ExtendBoxLeft + ImportOptions->Data.ExtendBoxRight,
							0, ImportOptions->Data.TexturePageWidth - NewCharacterRef.StartU );
			NewCharacterRef.VSize =
				FMath::Clamp<int32>( FontHeight + ImportOptions->Data.ExtendBoxTop + ImportOptions->Data.ExtendBoxBottom,
							0, ImportOptions->Data.TexturePageMaxHeight - NewCharacterRef.StartV );
			NewCharacterRef.TextureIndex = static_cast<uint8>(CurrentTexture);
			NewCharacterRef.VerticalOffset = VerticalOffset;

			// Draw character into font and advance.
			if( bUsingGlyphOutlines )
			{
				// GetGlyphOutline requires at least a uint32 aligned address
				uint8* AlignedGlyphData = ( uint8* )FMemory::Malloc( GGODataSize, DEFAULT_ALIGNMENT );
				check( AlignedGlyphData != NULL );

				// Grab the actual glyph bitmap data
				GetGlyphOutlineW(
					DCHandle,                         // Device context
					Char,	                            // Character
					GGO_GRAY8_BITMAP,                 // Format
					&WinGlyphMetrics,                 // Out: Metrics
					GGODataSize,                      // Data size
					AlignedGlyphData,                 // Out: Glyph data (aligned)
					&WinIdentityMatrix2x2 );          // Transform

				// Make sure source pitch is uint32 aligned
				int32 SourceDataPitch = WinGlyphMetrics.gmBlackBoxX;
				if( SourceDataPitch % 4 != 0 )
				{
					SourceDataPitch += 4 - SourceDataPitch % 4;
				}
				uint8* SourceDataPtr = AlignedGlyphData;

				const int32 DestDataPitch = WinBitmapInfo.bmiHeader.biWidth * WinBitmapInfo.bmiHeader.biBitCount / 8;
				uint8* DestDataPtr = ( uint8* )BitmapDataPtr;
				check( DestDataPtr != NULL );

				// We're going to write directly to the bitmap, so we'll unbind it from the GDI first
				SelectObject( DCHandle, LastBitmapHandle );

				// Copy the glyph data to our bitmap!
				for( uint32 SourceY = 0; SourceY < WinGlyphMetrics.gmBlackBoxY; ++SourceY )
				{
					for( uint32 SourceX = 0; SourceX < WinGlyphMetrics.gmBlackBoxX; ++SourceX )
					{
						// Values are between 0 and 64 inclusive (64 possible shades, including black)
						uint8 Opacity = ( uint8 )( ( ( int32 )SourceDataPtr[ SourceY * SourceDataPitch + SourceX ] * 255 ) / 64 );

						// Copy the opacity value into the RGB components of the bitmap, since that's where we'll be looking for them
						// NOTE: Alpha channel is set to zero
						const uint32 DestX = X + SourceX;
						const uint32 DestY = WinBitmapInfo.bmiHeader.biHeight - ( Y + SourceY ) - 1;  // Image is upside down!
						*( uint32* )&DestDataPtr[ DestY * DestDataPitch + DestX * sizeof( uint32 ) ] =
							( Opacity ) |            // B
							( Opacity << 8 ) |       // G
							( Opacity << 16 );       // R
					}
				}

				// OK, we can rebind it now!
				SelectObject( DCHandle, BitmapHandle );

				FMemory::Free( AlignedGlyphData );
			}
			else
			{
				TCHAR Tmp[2];
				Tmp[0] = Char;
				Tmp[1] = 0;

				TextOut( DCHandle, X, Y, Tmp, 1 );
				
				UE_LOGF(LogTTFontImport, Log, "OutPutGlyph X=%d Y=%d FontHeight=%d FontWidth=%d Char=%04x U=%d V=%d =Usize=%d VSIze=%d", 
					X,Y,FontHeight,FontWidth,Char,
					NewCharacterRef.StartU ,
					NewCharacterRef.StartV ,
					NewCharacterRef.USize,
					NewCharacterRef.VSize);
			}
			X = X + CharWidth + BitmapPaddingX;
		}
		// save final page
		const int32 RoundedHeight = RoundUpToMultipleOf4(Y + RowHeight);
		check(RoundedHeight <= BitmapHeight);
		Font->Textures.Add( CreateTextureFromDC( Font, DCHandle, BitmapWidth, RoundedHeight, CurrentTexture ) );
		CurrentTexture++;

		DeleteDC( DCHandle );
		DeleteObject( BitmapHandle );
	}

	// Store character count
	Font->CacheCharacterCountAndMaxCharHeight();

	GWarn->UpdateProgress( TotalProgress, TotalProgress );

	return true;
}

#endif // !USE_FREETYPE

#endif // PLATFORM_WINDOWS


#if PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX

#if USE_FREETYPE

#if PLATFORM_WINDOWS

FString UTrueTypeFontFactory::FindBitmapFontFile()
{
	HKEY FontsRegKey;
	LSTATUS Result = RegOpenKeyEx( HKEY_LOCAL_MACHINE, TEXT("Software\\Microsoft\\Windows NT\\CurrentVersion\\Fonts"), 0, KEY_READ, &FontsRegKey );
	if( Result != ERROR_SUCCESS )
	{
		return TEXT("");
	}

	uint32 MaxNameSize, MaxDataSize;
	Result = RegQueryInfoKey( FontsRegKey, NULL, NULL, NULL, NULL, NULL, NULL, NULL, (LPDWORD)&MaxNameSize, (LPDWORD)&MaxDataSize, NULL, NULL );
	if( Result != ERROR_SUCCESS )
	{
		return TEXT("");
	}

	TCHAR* Name = (TCHAR*)FMemory::Malloc( MaxNameSize );
	TCHAR* Data = (TCHAR*)FMemory::Malloc( MaxDataSize );
	uint32 Index = 0;
	FString FontFile;

	do 
	{
		uint32 NameSize = MaxNameSize;
		uint32 DataSize = MaxDataSize;
		uint32 Type;

		Result = RegEnumValue( FontsRegKey, Index++, Name, (LPDWORD)&NameSize, 0, (LPDWORD)&Type, (LPBYTE)Data, (LPDWORD)&DataSize );
		if( Result != ERROR_SUCCESS || Type != REG_SZ )
		{
			continue;
		}

		if( FCString::Strnicmp( *ImportOptions->Data.FontName, Name, ImportOptions->Data.FontName.Len() ) == 0 && FCString::Strifind( Name, TEXT("(TrueType)") ) == NULL )
		{
			FontFile = Data;
			break;
		}
	} while( Result != ERROR_NO_MORE_ITEMS );

	FMemory::Free( Name );
	FMemory::Free( Data );

	if( FontFile.Len() > 0 )
	{
		TCHAR WindowsFolder[MAX_PATH];
		GetWindowsDirectory( WindowsFolder, MAX_PATH );

		FontFile = FString( WindowsFolder ) + "\\Fonts\\" + FontFile;
	}

	return FontFile;
}

void* UTrueTypeFontFactory::LoadFontFace( void* FTLibrary, int32 Height, FFeedbackContext* Warn, void** OutFontData )
{
	uint32 ImportCharSet = DEFAULT_CHARSET;
	switch ( ImportOptions->Data.CharacterSet )
	{
		case FontICS_Ansi:
			ImportCharSet = ANSI_CHARSET;
			break;

		case FontICS_Default:
			ImportCharSet = DEFAULT_CHARSET;
			break;

		case FontICS_Symbol:
			ImportCharSet = SYMBOL_CHARSET;
			break;
	}

	// Create the Windows font
	HFONT FontHandle =
		CreateFontW(
			-Height,
			0,
			0,
			0,
			ImportOptions->Data.bEnableBold ? FW_BOLD : FW_NORMAL,
			ImportOptions->Data.bEnableItalic,
			ImportOptions->Data.bEnableUnderline,
			0,
			ImportCharSet,
			OUT_DEFAULT_PRECIS,
			CLIP_DEFAULT_PRECIS,
			ImportOptions->Data.bEnableAntialiasing ? ANTIALIASED_QUALITY : NONANTIALIASED_QUALITY,
			VARIABLE_PITCH,
			*ImportOptions->Data.FontName );

	if( FontHandle == NULL )
	{
		TCHAR ErrorBuffer[1024];
		Warn->Logf(ELogVerbosity::Error, TEXT("CreateFont failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
		return nullptr;
	}

	// Create DC
	HDC DeviceDCHandle = GetDC( NULL );
	if( DeviceDCHandle == NULL )
	{
		TCHAR ErrorBuffer[1024];
		Warn->Logf(ELogVerbosity::Error, TEXT("GetDC failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
		return nullptr;
	}

	HDC DCHandle = CreateCompatibleDC( DeviceDCHandle );
	if( !DCHandle )
	{
		TCHAR ErrorBuffer[1024];
		Warn->Logf(ELogVerbosity::Error, TEXT("CreateDC failed: %s"), FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, 0) );
		return nullptr;
	}

	SelectObject( DCHandle, FontHandle );

	FT_Face Face = NULL;

	uint32 FontDataSize = GetFontData( DCHandle, 0, 0, NULL, 0 );
	if( FontDataSize != GDI_ERROR )
	{
		uint8* FontData = (uint8*)FMemory::Malloc( FontDataSize );
		if( GetFontData( DCHandle, 0, 0, FontData, FontDataSize ) != GDI_ERROR )
		{
			int32 Error = FT_New_Memory_Face( (FT_Library)FTLibrary, FontData, FontDataSize, 0, &Face );
			if( Error != 0 )
			{
				Face = NULL;
			}
		}
	}
	else
	{
		// GetFontData() does not support bitmap fonts. For them we try to figure out the font file path based on Windows registry
		FString FontFile = FindBitmapFontFile();
		if( FontFile.Len() )
		{
			int32 Error = FT_New_Face( (FT_Library)FTLibrary, TCHAR_TO_ANSI(*FontFile), 0, &Face );
			if( Error != 0 )
			{
				Face = NULL;
			}
		}
	}

	DeleteDC( DCHandle );
	DeleteObject( FontHandle );

	return Face;
}

#elif PLATFORM_MAC

void* UTrueTypeFontFactory::LoadFontFace( void* FTLibrary, int32 Height, FFeedbackContext* Warn, void** OutFontData )
{
	// Prepare a dictionary with font attributes
	CFStringRef FontName = FPlatformString::TCHARToCFString( *ImportOptions->Data.FontName );
	CFNumberRef FontSize = CFNumberCreate( NULL, kCFNumberSInt32Type, &Height );

	if( !FontName || !FontSize )
	{
		return NULL;
	}

	int32 NumAttributes = 1;
	CFStringRef Keys[] = { kCTFontNameAttribute, NULL };
	CFTypeRef Values[] = { FontName, NULL };

	if( ImportOptions->Data.CharacterSet == FontICS_Symbol )
	{
		Keys[NumAttributes] = kCTFontCharacterSetAttribute;
		Values[NumAttributes] = CFCharacterSetGetPredefined( kCFCharacterSetSymbol );
		NumAttributes++;
	}

	CFDictionaryRef Attributes = CFDictionaryCreate( NULL, (const void**)&Keys, (const void**)&Values, NumAttributes, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks );

	CFRelease( FontName );
	CFRelease( FontSize );

	CFStringRef FontPath = NULL;

	// Get the path to the font file
	if( Attributes )
	{
		CTFontDescriptorRef FontDesc = CTFontDescriptorCreateWithAttributes( Attributes );
		if( FontDesc )
		{
			CFURLRef FontURL = (CFURLRef)CTFontDescriptorCopyAttribute( FontDesc, kCTFontURLAttribute );
			CFRelease( FontDesc );

			if( FontURL )
			{
				FontPath = CFURLCopyFileSystemPath( FontURL, kCFURLPOSIXPathStyle );
				CFRelease( FontURL );
			}
		}
		CFRelease( Attributes );
	}

	FT_Face Face = NULL;

	if( FontPath )
	{
		ANSICHAR AnsiPath[MAC_MAX_PATH];
		if( CFStringGetFileSystemRepresentation( FontPath, AnsiPath, MAC_MAX_PATH ) )
		{
			int32 Error = FT_New_Face( (FT_Library)FTLibrary, AnsiPath, 0, &Face );
			if( Error != 0 )
			{
				Face = NULL;
			}
		}
		CFRelease( FontPath );
	}

	return Face;
}

#elif PLATFORM_LINUX

void* UTrueTypeFontFactory::LoadFontFace(void* FTLibrary, int32 Height, FFeedbackContext* Warn, void** OutFontData)
{
	FT_Face Face = nullptr;
	int32 Error = FT_New_Face((FT_Library)FTLibrary, TCHAR_TO_ANSI(*ImportOptions->Data.FontName), 0, &Face);
	if (Error != 0)
	{
		Face = nullptr;
	}

	return Face;
}

#else 
#error "Unknown platform"
#endif

UTexture2D* UTrueTypeFontFactory::CreateTextureFromBitmap( UFont* Font, uint8* BitmapData, const int32 BitmapWidth, const int32 BitmapHeight, int32 TextureNum )
{
	FString TextureString = FString::Printf( TEXT("%s_Page"), *Font->GetName() );
	if( TextureNum < 26 )
	{
		TextureString = TextureString + FString::Printf(TEXT("%c"), 'A'+TextureNum);
	}
	else
	{
		TextureString = TextureString + FString::Printf(TEXT("%c%c"), 'A'+TextureNum/26, 'A'+TextureNum%26 );
	}

	if( StaticFindObject( NULL, Font, *TextureString ) )
	{
		UE_LOGF(LogTTFontImport, Warning, "A texture named %ls already exists!", *TextureString );
	}
	
	// Create texture for page.
	UTexture2D* Texture = NewObject<UTexture2D>(Font, *TextureString);

	// note RF_Public because font textures can be referenced directly by material expressions
	Texture->SetFlags(RF_Public);
	Texture->Source.Init(BitmapWidth, BitmapHeight, 1, 1, TSF_BGRA8);

	// Copy the LODGroup from the font factory to the new texture
	// By default, this should be TEXTUREGROUP_UI for fonts!
	Texture->LODGroup = LODGroup;

	// Also, we never want to stream in font textures since that always looks awful
	Texture->NeverStream = true;

	// Copy bitmap data to texture page.
	FColor FontColor8Bit( ImportOptions->Data.ForegroundColor.ToFColor(true) );

	// restart progress bar for font bitmap generation since this takes a while
	int32 TotalProgress = BitmapWidth-1;
	FFormatNamedArguments Args;
	Args.Add( TEXT("TextureNum"), TextureNum );
	GWarn->StatusUpdate(0,0, FText::Format( NSLOCTEXT("TrueTypeFontImport", "GeneratingFontPageStatusUpdate", "Generating font page {TextureNum}"), Args ));

	uint32* SourceData = (uint32*)BitmapData;

	uint8* MipData = Texture->Source.LockMip(0);
	if( !ImportOptions->Data.bEnableAntialiasing )
	{
		int32 SizeX = Texture->Source.GetSizeX();
		int32 SizeY = Texture->Source.GetSizeY();

		for( int32 i=0; i<SizeX; i++ )
		{
			// update progress bar
			GWarn->UpdateProgress(i,TotalProgress);
			
			for( int32 j=0; j<SizeY; j++ )
			{
				int32 CharAlpha = SourceData[i+j*BitmapWidth];
				int32 DropShadowAlpha;

				if( ImportOptions->Data.bEnableDropShadow && i > 0 && j > 0 )
				{
					DropShadowAlpha = ( (i-1)+(j-1)*BitmapWidth);
				}
				else
				{
					DropShadowAlpha = 0;
				}

				if( CharAlpha )
				{
					MipData[4 * (i + j * SizeX) + 0] = FontColor8Bit.B;
					MipData[4 * (i + j * SizeX) + 1] = FontColor8Bit.G;
					MipData[4 * (i + j * SizeX) + 2] = FontColor8Bit.R;
					MipData[4 * (i + j * SizeX) + 3] = 0xFF;
				}
				else if( DropShadowAlpha )
				{
					MipData[4 * (i + j * SizeX) + 0] = 0x00;
					MipData[4 * (i + j * SizeX) + 1] = 0x00;
					MipData[4 * (i + j * SizeX) + 2] = 0x00;
					MipData[4 * (i + j * SizeX) + 3] = 0xFF;
				}
				else
				{
					MipData[4 * (i + j * SizeX) + 0] = FontColor8Bit.B;
					MipData[4 * (i + j * SizeX) + 1] = FontColor8Bit.G;
					MipData[4 * (i + j * SizeX) + 2] = FontColor8Bit.R;
					MipData[4 * (i + j * SizeX) + 3] = 0x00;
				}
			}
		}
	}
	else
	{
		int32 SizeX = Texture->Source.GetSizeX();
		int32 SizeY = Texture->Source.GetSizeY();

		for( int32 i=0; i<SizeX; i++ )
		{
			for( int32 j=0; j<SizeY; j++ )
			{
				int32 CharAlpha = SourceData[i+j*BitmapWidth];
				float fCharAlpha = float( CharAlpha ) / 255.0f;

				int32 DropShadowAlpha = 0;
				if( ImportOptions->Data.bEnableDropShadow && i > 0 && j > 0 )
				{
					// Character opacity takes precedence over drop shadow opacity
					DropShadowAlpha =
						( uint8 )( ( 1.0f - fCharAlpha ) * ( float )( ( uint8 )( SourceData[(i - 1)+(j - 1)*BitmapWidth] ) ) );
				}
				float fDropShadowAlpha = float( DropShadowAlpha ) / 255.0f;

				// Color channel = Font color, except for drop shadow pixels
				MipData[4 * (i + j * SizeX) + 0] = ( uint8 )( FontColor8Bit.B * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 1] = ( uint8 )( FontColor8Bit.G * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 2] = ( uint8 )( FontColor8Bit.R * ( 1.0f - fDropShadowAlpha ) );
				MipData[4 * (i + j * SizeX) + 3] = CharAlpha + DropShadowAlpha;
			}
		}
	}
	Texture->Source.UnlockMip(0);
	MipData = NULL;
	
	// convert bitmap font alpha channel to distance field
	if (ImportOptions->Data.bUseDistanceFieldAlpha)
	{
		// Initialize distance field generator with high res source bitmap texels
		FTextureAlphaToDistanceField DistanceFieldTex(
			Texture->Source.LockMip(0),
			Texture->Source.GetSizeX(),
			Texture->Source.GetSizeY(),
			PF_B8G8R8A8
			);
		// estimate scan radius based on half font height scaled by bitmap scale factor
		const int32 ScanRadius = ImportOptions->Data.Height/2 * ImportOptions->Data.DistanceFieldScaleFactor * ImportOptions->Data.DistanceFieldScanRadiusScale;
		// generate downsampled distance field using high res source bitmap
		DistanceFieldTex.Generate(ImportOptions->Data.DistanceFieldScaleFactor,ScanRadius);
		check(DistanceFieldTex.GetResultTextureSize() > 0);
		Texture->Source.UnlockMip(0);	
		// resize/update texture using distance field values
		Texture->Source.Init(
			DistanceFieldTex.GetResultSizeX(),
			DistanceFieldTex.GetResultSizeY(),
			/*NumSlices=*/ 1,
			/*NumMips=*/ 1,
			TSF_BGRA8
			);
		FMemory::Memcpy(Texture->Source.LockMip(0),DistanceFieldTex.GetResultTexture(),DistanceFieldTex.GetResultTextureSize());		
		Texture->Source.UnlockMip(0);		
		// use PF_G8 for all distance field textures for better precision than DXT5
		Texture->CompressionSettings = TC_DistanceFieldFont;
		// disable gamma correction since storing alpha in linear color for PF_G8
		Texture->SRGB = false;
	}
	else
	{
		// if we dont care about colors then store texture as PF_G8
		if (ImportOptions->Data.bAlphaOnly &&
			!ImportOptions->Data.bEnableDropShadow)
		{
			// Not a distance field texture, but we use the same compression settings for better precision than DXT5
			Texture->CompressionSettings = TC_DistanceFieldFont;
			// disable gamma correction since storing alpha in linear color for PF_G8
			Texture->SRGB = false;
		}
	}	
	
	Texture->MipGenSettings = TMGS_NoMipmaps;
	Texture->PostEditChange();

	return Texture;
}

bool UTrueTypeFontFactory::CreateFontTexture(
	UFont* Font,
	FFeedbackContext* Warn,
	const int32 NumResolutions,
	const int32 CharsPerPage,
	const TMap<TCHAR,TCHAR>& InverseMap,
	const TArray< float >& ResHeights )
{
	// Init freetype
	FT_Library FTLibrary;
	int32 Error = FT_Init_FreeType( &FTLibrary );
	checkf(Error == 0, TEXT("Could not init Freetype"));

	const int32 TotalProgress = NumResolutions * CharsPerPage;

	// Zero out the Texture Index
	int32 CurrentTexture = 0;

	// Create bitmap
	const int32 ActualDistanceFieldScaleFactor = ImportOptions->Data.bUseDistanceFieldAlpha ? ImportOptions->Data.DistanceFieldScaleFactor : 1;
	const int32 BitmapWidth = ImportOptions->Data.TexturePageWidth * ActualDistanceFieldScaleFactor;
	const int32 BitmapHeight = ImportOptions->Data.TexturePageMaxHeight * ActualDistanceFieldScaleFactor;
	const int32 BitmapPaddingX = ImportOptions->Data.XPadding * ActualDistanceFieldScaleFactor;
	const int32 BitmapPaddingY = ImportOptions->Data.YPadding * ActualDistanceFieldScaleFactor;
	const int32 BitmapBytesPerPixel = 4;
	const int32 BitmapDataSize = BitmapWidth * BitmapHeight * BitmapBytesPerPixel;
	uint8* BitmapDataPtr = (uint8*)FMemory::Malloc(BitmapDataSize);
	if (BitmapDataPtr == nullptr)
	{
		UE_LOGF(LogTTFontImport, Error, "Unable to allocate bitmap buffer for font texture. Please verify the texture page size and distance field scale factor settings.");
		FT_Done_FreeType(FTLibrary);
		return false;
	}

	for( int32 Page = 0; Page < NumResolutions; ++Page )
	{
		int32 Height = FMath::RoundToInt( ResHeights[Page] );
		
		// scale font height to generate high res bitmap based on scale factor
		// this high res bitmap is later used to generate the downsampled distance field
		if (ImportOptions->Data.bUseDistanceFieldAlpha)
		{
			Height *= ImportOptions->Data.DistanceFieldScaleFactor;
		}

		void* FontData = NULL;
		FT_Face Face = (FT_Face)LoadFontFace( FTLibrary, Height, Warn, &FontData );
		if( Face == NULL )
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("Failed to load font face"));
			FT_Done_FreeType( FTLibrary );
			return false;
		}

		Error = FT_Set_Char_Size( Face, 0, Height * 64, 72, 72 );
		if (Error != 0)
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("Failed to set the font size"));
			FT_Done_FreeType( FTLibrary );
			return false;
		}

		FMemory::Memzero( BitmapDataPtr, BitmapDataSize );

		float EmScale = 1024.f/Height;
		Font->EmScale = EmScale;
		if (ImportOptions->Data.bUseDistanceFieldAlpha)
			Font->EmScale *= ImportOptions->Data.DistanceFieldScaleFactor;
		const int32 AscenderPixels = ( FT_MulFix( Face->ascender, Face->size->metrics.y_scale ) >> 6 );
		Font->Ascent = AscenderPixels * EmScale;
		Font->Descent = ( FT_MulFix( Face->descender, Face->size->metrics.y_scale ) >> 6 ) * -EmScale;
		Font->Leading = ( FT_MulFix( Face->height, Face->size->metrics.y_scale ) >> 6 ) * EmScale  - Font->Ascent - Font->Descent;

		int32 X=BitmapPaddingX, Y=BitmapPaddingY, RowHeight=0;

		for( int32 CurCharIndex = 0; CurCharIndex < CharsPerPage; ++CurCharIndex )
		{
			GWarn->UpdateProgress( Page * CharsPerPage + CurCharIndex, TotalProgress );

			// Remap the character if we need to
			TCHAR Char = ( TCHAR )CurCharIndex;
			if( Font->IsRemapped )
			{
				const TCHAR* FoundRemappedChar = InverseMap.Find( Char );
				if( FoundRemappedChar != NULL )
				{
					Char = *FoundRemappedChar;
				}
				else
				{
					// Skip missing remapped character
					continue;
				}
			}

			// Skip ASCII character if it isn't in the list of characters to import.
			if( Char < 256 && ImportOptions->Data.Chars != TEXT("") && (!Char || !FCString::Strchr(*ImportOptions->Data.Chars, Char)) )
			{
				continue;
			}

			// Skip if the user has requested that only printable characters be
			// imported and the character isn't printable
			if( ImportOptions->Data.bCreatePrintableOnly == true && iswprint(Char) == false )
			{
				continue;
			}

			FT_UInt GlyphIndex = FT_Get_Char_Index( Face, Char );
			if( GlyphIndex == 0 )
			{
				GlyphIndex = FT_Get_Char_Index( Face, ' ' );
			}

			Error = FT_Load_Glyph( Face, GlyphIndex, FT_LOAD_DEFAULT );
			check(Error==0);

			FT_GlyphSlot Glyph = Face->glyph;

			Error = FT_Render_Glyph( Glyph, FT_RENDER_MODE_NORMAL );
			check(Error==0);

			int32 CharWidth = Glyph->advance.x >> 6;
			int32 CharHeight = Glyph->bitmap.rows;

			int32 VerticalOffset = AscenderPixels - Glyph->bitmap_top;

			// Adjust character dimensions to accommodate a drop shadow
			if( ImportOptions->Data.bEnableDropShadow )
			{
				CharWidth += 1;
				CharHeight += 1;
			}
			if (ImportOptions->Data.bUseDistanceFieldAlpha)
			{
				
				// Make X and Y positions a multiple of the scale factor.
				CharWidth = FMath::RoundToInt(CharWidth / static_cast<float>(ImportOptions->Data.DistanceFieldScaleFactor)) * ImportOptions->Data.DistanceFieldScaleFactor;
				CharHeight = FMath::RoundToInt(CharHeight / static_cast<float>(ImportOptions->Data.DistanceFieldScaleFactor)) * ImportOptions->Data.DistanceFieldScaleFactor;
			}

			// If the character is bigger than our texture size, then this isn't going to work!  The user
			// will need to specify a larger texture resolution
			const int32 RequiredWidth = CharWidth + BitmapPaddingX * 2;
			const int32 RequiredHeight = CharHeight + BitmapPaddingY * 2;
			const bool bisBitmapTooNarrow = RequiredWidth > BitmapWidth;
			const bool bisBitmapTooShort = RequiredHeight > BitmapHeight;
			if (bisBitmapTooNarrow || bisBitmapTooShort )
			{
				UE_LOGF(LogTTFontImport, Error, "At the specified font size, at least one font glyph would be larger than the maximum texture size you specified." );
				if (bisBitmapTooNarrow)
				{
					UE_LOGF(LogTTFontImport, Error, "You should increase TexturePageWidth to at least %d.", RequiredWidth / ActualDistanceFieldScaleFactor);
				}
				else
				{
					UE_LOGF(LogTTFontImport, Error, "You should increase TexturePageMaxHeight to at least %d.", RequiredHeight / ActualDistanceFieldScaleFactor);
				}
				
				FMemory::Free( BitmapDataPtr );
				FT_Done_FreeType( FTLibrary );
				return false;
			}

			// If it doesn't fit right here, advance to next line.
			if( (CharWidth + X + BitmapPaddingX) > BitmapWidth)
			{
				X=BitmapPaddingX;
				Y = Y + RowHeight + BitmapPaddingY;
				RowHeight = 0;
			}
			int32 OldRowHeight = RowHeight;
			if( CharHeight > RowHeight )
			{
				RowHeight = CharHeight;
			}

			// new page
			if( (Y + RowHeight + BitmapPaddingY) > BitmapHeight )
			{
				const int32 RoundedHeight = RoundUpToMultipleOf4(Y + OldRowHeight);
				check(RoundedHeight <= BitmapHeight);
				Font->Textures.Add( CreateTextureFromBitmap( Font, BitmapDataPtr, BitmapWidth, RoundedHeight, CurrentTexture ) );
				CurrentTexture++;

				FMemory::Memzero( BitmapDataPtr, BitmapDataSize );

				X=BitmapPaddingX;
				Y=BitmapPaddingY;
				
				RowHeight = 0;
			}

			// Set font character information.
			FFontCharacter& NewCharacterRef = Font->Characters[ CurCharIndex + (CharsPerPage * Page)];
			int32 FontX = X;
			int32 FontY = Y;
			int32 FontWidth = CharWidth;
			int32 FontHeight = CharHeight;
			// scale character offset UVs back down based on scale factor
			if (ImportOptions->Data.bUseDistanceFieldAlpha)
			{
				FontX = FMath::RoundToInt(FontX / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontY = FMath::RoundToInt(FontY / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontWidth = FMath::RoundToInt(FontWidth / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				FontHeight = FMath::RoundToInt(FontHeight / (float)ImportOptions->Data.DistanceFieldScaleFactor);
				VerticalOffset = FMath::RoundToInt(VerticalOffset / (float)ImportOptions->Data.DistanceFieldScaleFactor);
			}
			NewCharacterRef.StartU =
				FMath::Clamp<int32>( FontX - ImportOptions->Data.ExtendBoxLeft,
							0, ImportOptions->Data.TexturePageWidth - 1 );
			NewCharacterRef.StartV =
				FMath::Clamp<int32>( FontY + ImportOptions->Data.ExtendBoxTop,
							0, ImportOptions->Data.TexturePageMaxHeight - 1 );
			NewCharacterRef.USize =
				FMath::Clamp<int32>( FontWidth + ImportOptions->Data.ExtendBoxLeft + ImportOptions->Data.ExtendBoxRight,
							0, ImportOptions->Data.TexturePageWidth - NewCharacterRef.StartU );
			NewCharacterRef.VSize =
				FMath::Clamp<int32>( FontHeight + ImportOptions->Data.ExtendBoxTop + ImportOptions->Data.ExtendBoxBottom,
							0, ImportOptions->Data.TexturePageMaxHeight - NewCharacterRef.StartV );
			NewCharacterRef.TextureIndex = CurrentTexture;
			NewCharacterRef.VerticalOffset = VerticalOffset;

			const int32 DestDataPitch = BitmapWidth * BitmapBytesPerPixel;

			// Draw character into font and advance.
			for( uint32 SourceY = 0; SourceY < Glyph->bitmap.rows; ++SourceY )
			{
				for( uint32 SourceX = 0; SourceX < Glyph->bitmap.width; ++SourceX )
				{
					uint8 Opacity = Glyph->bitmap.buffer[ SourceY * Glyph->bitmap.width + SourceX ];

					// Copy the opacity value into the RGB components of the bitmap, since that's where we'll be looking for them
					// NOTE: Alpha channel is set to zero
					const uint32 DestX = X + SourceX;
					const uint32 DestY = Y + SourceY;
					*( uint32* )&BitmapDataPtr[ DestY * DestDataPitch + DestX * sizeof( uint32 ) ] =
						( Opacity ) |            // B
						( Opacity << 8 ) |       // G
						( Opacity << 16 );       // R
				}
			}
			X = X + CharWidth + BitmapPaddingX;
		}
		// save final page
		const int32 RoundedHeight = RoundUpToMultipleOf4(Y + RowHeight);
		Font->Textures.Add( CreateTextureFromBitmap( Font, BitmapDataPtr, BitmapWidth, RoundedHeight, CurrentTexture ) );
		CurrentTexture++;

		if( FontData )
		{
			FMemory::Free( FontData );
		}

		FT_Done_Face( Face );
	}

	// Store character count
	Font->CacheCharacterCountAndMaxCharHeight();

	GWarn->UpdateProgress( TotalProgress, TotalProgress );

	FMemory::Free(BitmapDataPtr);
	FT_Done_FreeType( FTLibrary );

	return true;
}

#endif // USE_FREETYPE

bool UTrueTypeFontFactory::ImportTrueTypeFont(
	UFont* Font,
	FFeedbackContext* Warn,
	const int32 NumResolutions,
	const TArray< float >& ResHeights )
{
	double StartTime = FPlatformTime::Seconds();

	TMap<TCHAR,TCHAR> InverseMap;

	Font->Kerning = ImportOptions->Data.Kerning;
	Font->IsRemapped = 0;

	const bool UseFiles = ImportOptions->Data.CharsFileWildcard != TEXT("") && ImportOptions->Data.CharsFilePath != TEXT("");
	const bool UseRange = ImportOptions->Data.UnicodeRange != TEXT("");
	const bool UseSpecificText = (ImportOptions->Data.Chars.Len()>0);

	int32 CharsPerPage = 0;
	if( UseFiles || UseRange || UseSpecificText)
	{
		Font->IsRemapped = 1;

		// Only include ASCII characters if we were asked to
		uint16 MinRangeCharacter = 0;
		if( ImportOptions->Data.bIncludeASCIIRange )
		{
			// Map (ASCII)
			for( TCHAR c=0;c<256;c++ )
			{
				Font->CharRemap.Add( c, c );
				InverseMap.Add( c, c );
			}

			MinRangeCharacter = 256;
		}

		TArray<uint8> Chars;
		Chars.AddZeroed(65536);

		if( UseFiles  || UseSpecificText)
		{
			FString S;
			if (UseFiles)
			{
				// find all characters in specified path/wildcard
				TArray<FString> Files;
				IFileManager::Get().FindFiles( Files, *(ImportOptions->Data.CharsFilePath / ImportOptions->Data.CharsFileWildcard),1,0 );
				for( TArray<FString>::TIterator it(Files); it; ++it )
				{
					FString FileText;
					verify(FFileHelper::LoadFileToString(FileText,*(ImportOptions->Data.CharsFilePath / *it)));
					S+=FileText;
				}
				UE_LOGF(LogTTFontImport, Warning, "Checked %d files", Files.Num() );
			}
			else
			{
				S = ImportOptions->Data.Chars;
			}
			for( int32 i=0; i<S.Len(); i++ )
			{
				Chars[(*S)[i]] = 1;
			}
		}

		if( UseRange )
		{
			Warn->Logf(TEXT("UnicodeRange <%s>:"), *ImportOptions->Data.UnicodeRange);
			int32 From = 0;
			int32 To = 0;
			bool HadDash = 0;
			for( const TCHAR* C=*ImportOptions->Data.UnicodeRange; *C; C++ )
			{
				if( (*C>='A' && *C<='F') || (*C>='a' && *C<='f') || (*C>='0' && *C<='9') )
				{
					if( HadDash )
					{
						To = 16*To + FromHex(*C);
					}
					else
					{
						From = 16*From + FromHex(*C);
					}
				}
				else if( *C=='-' )
				{
					HadDash = 1;
				}
				else if( *C==',' )
				{
					UE_LOGF(LogTTFontImport, Warning, "Adding unicode character range %x-%x (%d-%d)",From,To,From,To);
					for( int32 i=From;i<=To&&i>=0&&i<65536;i++ )
					{
						Chars[i] = 1;
					}
					HadDash=0;
					From=0;
					To=0;
				}
			}
			UE_LOGF(LogTTFontImport, Warning, "Adding unicode character range %x-%x (%d-%d)",From,To,From,To);
			for( int32 i=From; i<=To && i>=0 && i<65536; i++ )
			{
				Chars[i] = 1;
			}

		}

		uint16 j = MinRangeCharacter;
		uint16 Min = std::numeric_limits<uint16>::max();
		uint16 Max = 0;
		for( uint16 i = MinRangeCharacter; i < std::numeric_limits<uint16>::max(); ++i )
		{
			if( Chars[i] )
			{
				if( i < Min )
				{
					Min = i;
				}
				if( i > Max )
				{
					Max = i;
				}

				Font->CharRemap.Add( i, j );
				InverseMap.Add( j++, i );
			}
		}

		UE_LOGF(LogTTFontImport, Warning, "Importing %d characters (unicode range %04x-%04x)", j, Min, Max);

		CharsPerPage = j;
	}
	else
	{
		// No range specified, so default to the ASCII range
		CharsPerPage = 256;
	}

	// Add space for characters.
	Font->Characters.AddZeroed(CharsPerPage * NumResolutions );
	
	// If all upper case chars have lower case char counterparts no mapping is required.   
	if( !Font->IsRemapped )
	{
		bool NeedToRemap = false;
		
		for( const TCHAR* p = *ImportOptions->Data.Chars; *p; p++ )
		{
			TCHAR c;
			
			if( !FChar::IsAlpha( *p ) )
			{
				continue;
			}
			
			if( FChar::IsUpper( *p ) )
			{
				c = FChar::ToLower( *p );
			}
			else
			{
				c = FChar::ToUpper( *p );
			}

			if( FCString::Strchr(*ImportOptions->Data.Chars, c) )
			{
				continue;
			}
			
			NeedToRemap = true;
			break;
		}
		
		if( NeedToRemap )
		{
			Font->IsRemapped = 1;

			for( const TCHAR* p = *ImportOptions->Data.Chars; *p; p++ )
			{
				TCHAR c;

				if( !FChar::IsAlpha( *p ) )
				{
					Font->CharRemap.Add( *p, *p );
					InverseMap.Add( *p, *p );
					continue;
				}
				
				if( FChar::IsUpper( *p ) )
				{
					c = FChar::ToLower( *p );
				}
				else
				{
					c = FChar::ToUpper( *p );
				}

				Font->CharRemap.Add( *p, *p );
				InverseMap.Add( *p, *p );

				if( !FCString::Strchr(*ImportOptions->Data.Chars, c) )
				{
					Font->CharRemap.Add( c, *p );
				}
			}
		}
	}

	bool bResult = CreateFontTexture( Font, Warn, NumResolutions, CharsPerPage, InverseMap, ResHeights );

	double EndTime = FPlatformTime::Seconds();
	UE_LOGF(LogTTFontImport, Log,"ImportTrueTypeFont: Total Time %0.2f", EndTime - StartTime);

	return bResult;
}

#endif // PLATFORM_WINDOWS || PLATFORM_MAC


#undef LOCTEXT_NAMESPACE
