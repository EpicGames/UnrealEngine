// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/Texture2DArrayFactory.h"
#include "EngineLogs.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Engine/Texture2DArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Texture2DArrayFactory)


#define LOCTEXT_NAMESPACE "Texture2DArrayFactory"

UTexture2DArrayFactory::UTexture2DArrayFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UTexture2DArray::StaticClass();
}

FText UTexture2DArrayFactory::GetDisplayName() const
{
	return LOCTEXT("Texture2DArrayFactoryDescription", "Texture 2D Array");
}

bool UTexture2DArrayFactory::ConfigureProperties()
{
	return true;
}

bool UTexture2DArrayFactory::CanCreateNew() const
{
	static const auto AllowTextureArrayAssetCreationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowTexture2DArrayCreation"));
	return (AllowTextureArrayAssetCreationVar->GetValueOnGameThread() == 1);
}

UObject* UTexture2DArrayFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	static const auto AllowTextureArrayAssetCreationVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowTexture2DArrayCreation"));

	if (AllowTextureArrayAssetCreationVar->GetValueOnGameThread() == 0)
	{
		UE_LOGF(LogTexture, Warning, "Texture2DArray creation failed. Enable by using console command r.AllowTexture2DArrayCreation\n");
		return nullptr;
	}

	// Make sure the size of all textures is the same.
	if (!CheckArrayTexturesCompatibility()) 
	{
		return nullptr;
	}

	UTexture2DArray* NewTexture2DArray = CastChecked<UTexture2DArray>(CreateOrOverwriteAsset(UTexture2DArray::StaticClass(), InParent, Name, Flags));

	if (NewTexture2DArray == 0) 
	{
		UE_LOGF(LogTexture, Warning, "Texture2DArray creation failed.\n");
	}
	else if (InitialTextures.Num() > 0)
	{
		// Upload the compatible textures to the texture array.
		for (int32 TextureIndex = 0; TextureIndex < InitialTextures.Num(); ++TextureIndex) 
		{
			NewTexture2DArray->SourceTextures.Add(InitialTextures[TextureIndex]);
		}

		// Create the texture array resource and corresponding rhi resource.
		NewTexture2DArray->UpdateSourceFromSourceTextures();
	}
	
	return NewTexture2DArray;
}

bool UTexture2DArrayFactory::CheckArrayTexturesCompatibility()
{
	bool bError = false;
	for (int32 TextureIndex = 0; TextureIndex < InitialTextures.Num(); ++TextureIndex) 
	{
		FTextureSource& TextureSource = InitialTextures[TextureIndex]->Source;
		const int32 FormatDataSize = TextureSource.GetBytesPerPixel();
		const int32 SizeX = TextureSource.GetSizeX();
		const int32 SizeY = TextureSource.GetSizeY();

		for (int32 TextureCmpIndex = TextureIndex + 1; TextureCmpIndex < InitialTextures.Num(); ++TextureCmpIndex)
		{
			FTextureSource& TextureSourceCmp = InitialTextures[TextureCmpIndex]->Source;
			if (TextureSourceCmp.GetSizeX() != SizeX || TextureSourceCmp.GetSizeY() != SizeY)
			{
				UE_LOGF(LogTexture, Warning, "Texture2DArray creation failed. Textures have different sizes.");
				bError = true;
			}

			if (TextureSourceCmp.GetBytesPerPixel() != FormatDataSize)
			{
				UE_LOGF(LogTexture, Warning, "Texture2DArray creation failed. Textures have different pixel formats.");
				bError = true;
			}
		}
	}

	return (!bError);
}

#undef LOCTEXT_NAMESPACE
