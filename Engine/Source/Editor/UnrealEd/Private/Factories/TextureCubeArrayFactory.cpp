// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TextureCubeArrayFactory.h"
#include "EngineLogs.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Engine/TextureCubeArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureCubeArrayFactory)


#define LOCTEXT_NAMESPACE "TextureCubeArrayFactory"

UTextureCubeArrayFactory::UTextureCubeArrayFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UTextureCubeArray::StaticClass();
}

FText UTextureCubeArrayFactory::GetDisplayName() const
{
	return LOCTEXT("TextureCubeArrayFactoryDescription", "Texture Cube Array");
}

bool UTextureCubeArrayFactory::ConfigureProperties()
{
	return true;
}

bool UTextureCubeArrayFactory::CanCreateNew() const
{
	return true;
}

UObject* UTextureCubeArrayFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Make sure the size of all textures is the same.
	if (!CheckArrayTexturesCompatibility())
	{
		return nullptr;
	}

	UTextureCubeArray* NewTextureCubeArray = CastChecked<UTextureCubeArray>(CreateOrOverwriteAsset(UTextureCubeArray::StaticClass(), InParent, Name, Flags));

	if (NewTextureCubeArray == 0)
	{
		UE_LOGF(LogTexture, Warning, "TextureCubeArray creation failed.\n");
	}
	else if (InitialTextures.Num() > 0)
	{
		// Upload the compatible textures to the texture array.
		for (int32 TextureIndex = 0; TextureIndex < InitialTextures.Num(); ++TextureIndex)
		{
			NewTextureCubeArray->SourceTextures.Add(InitialTextures[TextureIndex]);
		}

		// Create the texture array resource and corresponding rhi resource.
		NewTextureCubeArray->UpdateSourceFromSourceTextures();
	}

	return NewTextureCubeArray;
}

bool UTextureCubeArrayFactory::CheckArrayTexturesCompatibility()
{
	bool bError = false;
	for (int32 TextureIndex = 0; TextureIndex < InitialTextures.Num(); ++TextureIndex)
	{
		FTextureSource& TextureSource = InitialTextures[TextureIndex]->Source;
		const int32 FormatDataSize = TextureSource.GetBytesPerPixel();
		const int32 SizeX = TextureSource.GetSizeX();
		const int32 SizeY = TextureSource.GetSizeY();

		for (int32 TextureCmpIndex = TextureIndex + 1; TextureCmpIndex < InitialTextures.Num(); ++TextureCmpIndex)
		{
			FTextureSource& TextureSourceCmp = InitialTextures[TextureCmpIndex]->Source;
			if (TextureSourceCmp.GetSizeX() != SizeX || TextureSourceCmp.GetSizeY() != SizeY)
			{
				UE_LOGF(LogTexture, Warning, "TextureCubeArray creation failed. Textures have different sizes.");
				bError = true;
			}

			if (TextureSourceCmp.GetBytesPerPixel() != FormatDataSize)
			{
				UE_LOGF(LogTexture, Warning, "TextureCubeArray creation failed. Textures have different pixel formats.");
				bError = true;
			}
		}
	}

	return (!bError);
}

#undef LOCTEXT_NAMESPACE
