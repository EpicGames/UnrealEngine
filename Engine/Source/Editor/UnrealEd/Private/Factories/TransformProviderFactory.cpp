// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TransformProviderFactory.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Animation/Skeleton.h"
#include "Animation/TransformProviderData.h"
#include "Animation/AnimSequenceTransformProviderData.h"
#include "Animation/AnimRuntimeTransformProviderData.h"
#include "Editor.h"

#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Kismet2/KismetEditorUtilities.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformProviderFactory)

#define LOCTEXT_NAMESPACE "TransformProviderFactory"

UTransformProviderDataFactory::UTransformProviderDataFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UTransformProviderData::StaticClass();

	ProviderDataClass = nullptr;
}

namespace TransformProvider
{

class FAssetClassParentFilter : public IClassViewerFilter
{
public:
	FAssetClassParentFilter()
	: DisallowedClassFlags(CLASS_None)
	, bDisallowBlueprintBase(false)
	{
	}

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet< const UClass* > AllowedChildrenOfClasses;

	/** Classes that are explicitly disallowed from being selected. */
	TSet<const UClass*> DisallowedClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	/** Disallow blueprint base classes. */
	bool bDisallowBlueprintBase;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		const bool bAllowed = !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& !DisallowedClasses.Contains(InClass)
			&& InClass->CanCreateAssetOfClass()
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;

		if (bAllowed && bDisallowBlueprintBase)
		{
			if (FKismetEditorUtilities::CanCreateBlueprintOfClass(InClass))
			{
				return false;
			}
		}

		return bAllowed;
	}

	virtual bool IsUnloadedClassAllowed(
		const FClassViewerInitializationOptions& InInitOptions,
		const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData,
		TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags) && 
			InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

} // namespace

bool UTransformProviderDataFactory::ConfigureProperties()
{
	// Null the ProviderDataClass so we can get a clean class
	ProviderDataClass = nullptr;

	// Load the class viewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<TransformProvider::FAssetClassParentFilter> Filter = MakeShareable(new TransformProvider::FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UTransformProviderData::StaticClass());
	Filter->DisallowedClasses.Add(UAnimSequenceTransformProviderDataInstance::StaticClass());
	Filter->DisallowedClasses.Add(UAnimRuntimeTransformProviderData::StaticClass());

	const FText TitleText = LOCTEXT("CreateTransformProviderDataOptions", "Pick Transform Provider Data Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UTransformProviderData::StaticClass());

	if (bPressedOk)
	{
		ProviderDataClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UTransformProviderDataFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UTransformProviderData* NewProviderData = nullptr;
	
	if (ProviderDataClass != nullptr)
	{
		NewProviderData = NewObject<UTransformProviderData>(InParent, ProviderDataClass, Name, Flags);
	}

	return NewProviderData;
}

#undef LOCTEXT_NAMESPACE
