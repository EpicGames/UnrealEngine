// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/VariableFrameStrippingSettingsFactory.h"
#include "Animation/VariableFrameStrippingSettings.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VariableFrameStrippingSettingsFactory)

class FFeedbackContext;
class UClass;
class UObject;

UVariableFrameStrippingSettingsFactory::UVariableFrameStrippingSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Provide the factory with information about how to handle our asset
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UVariableFrameStrippingSettings::StaticClass();

}

UObject* UVariableFrameStrippingSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Create and return a new instance of VariableFrameStrippingSettings
	return NewObject<UVariableFrameStrippingSettings>(InParent, Class, Name, Flags);
}
