// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorFieldFactory.cpp: Factory for importing a 3D grid of vectors.
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Factories/VectorFieldStaticFactory.h"
#include "Factories/ReimportVectorFieldStaticFactory.h"
#include "EditorFramework/AssetImportData.h"
#include "Particles/ParticleSystemComponent.h"
#include "Components/VectorFieldComponent.h"
#include "Exporters/VectorFieldExporter.h"
#include "Editor.h"

#include "UObject/UObjectHash.h"
#include "ComponentReregisterContext.h"
#include "VectorField/VectorFieldStatic.h"

DEFINE_LOG_CATEGORY_STATIC(LogVectorFieldFactory, Log, All);

#define LOCTEXT_NAMESPACE "VectorFieldFactory"

/*------------------------------------------------------------------------------
	Static vector field factory.
------------------------------------------------------------------------------*/


/**
 * true if the extension is for the FluidGridAscii format.
 */
static bool IsFluidGridAscii( const TCHAR* Extension )
{
	return FCString::Stricmp( Extension, TEXT("fga") ) == 0;
}

/**
 * Walks the stream looking for the specified character. Replaces the character
 * with NULL and returns the next character in the stream.
 */
static TCHAR* ParseUntil( TCHAR* Stream, TCHAR Separator )
{
	check( Stream != NULL );

	while ( *Stream != 0 && *Stream != Separator )
	{
		Stream++;
	}
	if ( *Stream == Separator )
	{
		*Stream++ = 0;
	}
	return Stream;
}

/**
 * Returns the next value in a CSV as an integer.
 */
static TCHAR* ParseIntCsv( TCHAR* Stream, int32* OutValue )
{
	check( Stream != NULL );
	check( OutValue != NULL );

	const TCHAR* Token = Stream;
	Stream = ParseUntil( Stream, TEXT(',') );
	*OutValue = FCString::Atoi( Token );

	return Stream;
}

/**
 * Returns the next value in a CSV as a float.
 */
static TCHAR* ParseFloatCsv( TCHAR* Stream, float* OutValue )
{
	check( Stream != NULL );
	check( OutValue != NULL );

	const TCHAR* Token = Stream;
	Stream = ParseUntil( Stream, TEXT(',') );
	*OutValue = FCString::Atof( Token );

	return Stream;
}

/**
 * Contents of an FGA file.
 */
struct FFGAContents
{
	int32 GridX;
	int32 GridY;
	int32 GridZ;
	FBox Bounds;
	TArray<float> Values;
};

/**
 * Parse an FGA file.
 */
static bool ParseFGA(
	FFGAContents* OutContents,
	TCHAR* Stream,
	FFeedbackContext* Warn )
{
	int32 GridX, GridY, GridZ;
	FBox Bounds;

	check( Stream != NULL );
	check( OutContents != NULL );

	// Parse the grid size.
	Stream = ParseIntCsv( Stream, &GridX );
	Stream = ParseIntCsv( Stream, &GridY );
	Stream = ParseIntCsv( Stream, &GridZ );

	// Parse the bounding box.
	FVector3f BMin, BMax;
	Stream = ParseFloatCsv( Stream, &BMin.X );
	Stream = ParseFloatCsv( Stream, &BMin.Y );
	Stream = ParseFloatCsv( Stream, &BMin.Z );
	Stream = ParseFloatCsv( Stream, &BMax.X );
	Stream = ParseFloatCsv( Stream, &BMax.Y );
	Stream = ParseFloatCsv( Stream, &BMax.Z );
	Bounds.Min = (FVector)BMin;
	Bounds.Max = (FVector)BMax;

	// Make sure there is more to read.
	if ( *Stream == 0 )
	{
		Warn->Logf(ELogVerbosity::Warning, TEXT("Unexpected end of file.") );
		return false;
	}

	// Make sure the grid size is acceptable.
	const int32 MaxGridSize = 128;
	if ( GridX < 0 || GridX > MaxGridSize
		|| GridY < 0 || GridY > MaxGridSize
		|| GridZ < 0 || GridZ > MaxGridSize )
	{
		Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid grid size: %dx%dx%d"), GridX, GridY, GridZ );
		return false;
	}

	// Propagate grid size and bounds.
	OutContents->GridX = GridX;
	OutContents->GridY = GridY;
	OutContents->GridZ = GridZ;
	OutContents->Bounds = Bounds;
	OutContents->Bounds.IsValid = true;

	// Allocate storage for grid values.
	const int32 VectorCount = GridX * GridY * GridZ;
	const int32 ValueCount = VectorCount * 3;
	OutContents->Values.Empty( ValueCount );
	OutContents->Values.AddZeroed( ValueCount );
	float* Values = OutContents->Values.GetData();
	float* ValuesEnd = Values + OutContents->Values.Num();

	// Parse each value.
	do 
	{
		Stream = ParseFloatCsv( Stream, Values++ );
	} while ( *Stream && Values < ValuesEnd );

	// Check that the correct number of values have been read in.
	if ( Values != ValuesEnd )
	{
		Warn->Logf(ELogVerbosity::Warning, TEXT("Expected %d values but only found %d in the file."),
			ValueCount, ValuesEnd - Values );
		return false;
	}

	// Check to see that the entire file has been parsed.
	if ( *Stream != 0 )
	{
		Warn->Logf(ELogVerbosity::Warning, TEXT("File contains additional information. This is not fatal but may mean the data has been truncated.") );
	}

	return true;
}

UVectorFieldStaticFactory::UVectorFieldStaticFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	SupportedClass = UVectorFieldStatic::StaticClass();
	Formats.Add(TEXT("fga;FluidGridAscii"));
	bCreateNew = false;
	bEditorImport = true;
	bText = false;
}

UObject* UVectorFieldStaticFactory::FactoryCreateBinary(
	UClass* InClass,
	UObject* InParent,
	FName InName,
	EObjectFlags Flags,
	UObject* Context,
	const TCHAR* Type,
	const uint8*& Buffer,
	const uint8* BufferEnd,
	FFeedbackContext* Warn )
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);

	check( InClass == UVectorFieldStatic::StaticClass() );

	UVectorFieldStatic* VectorField = NULL;
	UVectorFieldStatic* ExistingVectorField = FindObject<UVectorFieldStatic>( InParent, *InName.ToString() );
	if ( ExistingVectorField )
	{
		ExistingVectorField->ReleaseResource();
	}

	// Vector field and particle system components need to be reregistered to update vector field instances.
	TComponentReregisterContext<UVectorFieldComponent> ReregisterVectorFieldComponents;
	TComponentReregisterContext<UParticleSystemComponent> ReregisterParticleComponents;

	if ( IsFluidGridAscii( Type ) )
	{
		FString String;
		const int32 BufferSize = UE_PTRDIFF_TO_INT32(BufferEnd - Buffer);
		FFileHelper::BufferToString( String, Buffer, BufferSize );
		TCHAR* Stream = (TCHAR*)*String;
		
		if ( Stream && *Stream )
		{
			FFGAContents FileContents;
			if ( ParseFGA( &FileContents, Stream, Warn ) )
			{
				VectorField = NewObject<UVectorFieldStatic>(InParent, InName,Flags);

				VectorField->SizeX = FileContents.GridX;
				VectorField->SizeY = FileContents.GridY;
				VectorField->SizeZ = FileContents.GridZ;
				VectorField->Bounds = FileContents.Bounds;
				
				VectorField->AssetImportData->Update(CurrentFilename);

				// Convert vectors to 16-bit FP and store.
				check( (FileContents.Values.Num() % 3) == 0 );
				const int32 VectorCount = FileContents.Values.Num() / 3;
				const int32 DestBufferSize = VectorCount * sizeof(FFloat16Color);
				VectorField->SourceData.Lock( LOCK_READ_WRITE );
				FFloat16Color* RESTRICT DestValues = (FFloat16Color*)VectorField->SourceData.Realloc( DestBufferSize );
				const FVector3f* RESTRICT SrcValues = (FVector3f*)FileContents.Values.GetData();
				for ( int32 VectorIndex = 0; VectorIndex < VectorCount; ++VectorIndex )
				{
					DestValues->R = SrcValues->X;
					DestValues->G = SrcValues->Y;
					DestValues->B = SrcValues->Z;
					DestValues->A = 0.0f;
					DestValues++;
					SrcValues++;
				}
				VectorField->SourceData.Unlock();
			}
		}
	}

	if ( VectorField )
	{
		VectorField->InitResource();
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, VectorField);

	return VectorField;
}

bool UVectorFieldStaticFactory::FactoryCanImport( const FString& Filename )
{
	return IsFluidGridAscii( *FPaths::GetExtension(Filename) );
}

/*-----------------------------------------------------------------------------
	Static vector field reimport factory.
-----------------------------------------------------------------------------*/
UReimportVectorFieldStaticFactory::UReimportVectorFieldStaticFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UVectorFieldStatic::StaticClass();

	bCreateNew = false;
}

bool UReimportVectorFieldStaticFactory::CanReimport( UObject* Obj, TArray<FString>& OutFilenames )
{	
	UVectorFieldStatic* VectorFieldStatic = Cast<UVectorFieldStatic>(Obj);
	if(VectorFieldStatic)
	{
		VectorFieldStatic->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UReimportVectorFieldStaticFactory::SetReimportPaths( UObject* Obj, const TArray<FString>& NewReimportPaths )
{	
	UVectorFieldStatic* VectorFieldStatic = Cast<UVectorFieldStatic>(Obj);
	if(VectorFieldStatic && ensure(NewReimportPaths.Num() == 1))
	{
		VectorFieldStatic->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UReimportVectorFieldStaticFactory::Reimport( UObject* Obj )
{
	if(!Obj || !Obj->IsA(UVectorFieldStatic::StaticClass()))
	{
		return EReimportResult::Failed;
	}

	UVectorFieldStatic* VectorFieldStatic = Cast<UVectorFieldStatic>(Obj);

	if ( VectorFieldStatic->AssetImportData->SourceData.SourceFiles.Num() != 1 )
	{
		// No source art path. Can't reimport.
		return EReimportResult::Failed;
	}

	FString ReImportFilename = VectorFieldStatic->AssetImportData->GetFirstFilename();

	UE_LOGF(LogVectorFieldFactory, Log, "Performing atomic reimport of [%ls]", *ReImportFilename);

	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*ReImportFilename) == INDEX_NONE)
	{
		UE_LOGF(LogVectorFieldFactory, Warning, "Cannot reimport: source file cannot be found." );
		return EReimportResult::Failed;
	}

	bool OutCanceled = false;

	if (ImportObject(VectorFieldStatic->GetClass(), VectorFieldStatic->GetOuter(), *VectorFieldStatic->GetName(), RF_Public|RF_Standalone, ReImportFilename, nullptr, OutCanceled) != nullptr)
	{
		UE_LOGF(LogVectorFieldFactory, Log, "Reimported successfully" );
		VectorFieldStatic->AssetImportData->Update(ReImportFilename);
		VectorFieldStatic->MarkPackageDirty();
	}
	else if (OutCanceled)
	{
		UE_LOGF(LogVectorFieldFactory, Warning, "-- import canceled");
	}
	else
	{
		UE_LOGF(LogVectorFieldFactory, Warning, "-- import failed");
	}

	return EReimportResult::Succeeded;
}

int32 UReimportVectorFieldStaticFactory::GetPriority() const
{
	return ImportPriority;
}

/*-----------------------------------------------------------------------------
	UVectorFieldExporter
-----------------------------------------------------------------------------*/
UVectorFieldExporter::UVectorFieldExporter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UVectorFieldStatic::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("FGA"));
	FormatDescription.Add(TEXT("Vector Field"));
}

bool UVectorFieldExporter::ExportText(const FExportObjectInnerContext* Context, UObject* Object, const TCHAR* Type, FOutputDevice& Ar, FFeedbackContext* Warn, uint32 PortFlags)
{
	UVectorFieldStatic* LocalVectorField = CastChecked<UVectorFieldStatic>(StaticDuplicateObject(Object, GetTransientPackage()));

	LocalVectorField->SetCPUAccessEnabled();

	Ar.Logf(TEXT("%d, %d, %d"), LocalVectorField->SizeX, LocalVectorField->SizeY, LocalVectorField->SizeZ);
	Ar.Logf(TEXT(", "));
	Ar.Logf(TEXT("%f, %f, %f, %f, %f, %f"),
		LocalVectorField->Bounds.Min.X,
		LocalVectorField->Bounds.Min.Y,
		LocalVectorField->Bounds.Min.Z,
		LocalVectorField->Bounds.Max.X,
		LocalVectorField->Bounds.Max.Y,
		LocalVectorField->Bounds.Max.Z);

	constexpr int32 ComponentCount = 3;
	const int32 SampleCount = LocalVectorField->SizeX * LocalVectorField->SizeY * LocalVectorField->SizeZ;

	if (LocalVectorField->HasCPUData())
	{
		for (int32 SampleX = 0; SampleX < LocalVectorField->SizeX; ++SampleX)
		{
			for (int32 SampleY = 0; SampleY < LocalVectorField->SizeY; ++SampleY)
			{
				for (int32 SampleZ = 0; SampleZ < LocalVectorField->SizeZ; ++SampleZ)
				{
					const FVector SampleData = LocalVectorField->Sample(FIntVector(SampleX, SampleY, SampleZ));
					Ar.Logf(TEXT(", %f, %f, %f"), SampleData.X, SampleData.Y, SampleData.Z);
				}
			}
		}
		return true;
	}

	UE_LOGF(LogVectorFieldFactory, Warning, "Failed to export [%ls] unabled to acquire bulk data", *Object->GetFullName());
	return false;
}

#undef LOCTEXT_NAMESPACE
