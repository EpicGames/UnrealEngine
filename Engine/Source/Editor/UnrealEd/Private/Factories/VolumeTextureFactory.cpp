// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/VolumeTextureFactory.h"
#include "Styling/SlateBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Engine/VolumeTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeTextureFactory)

#define LOCTEXT_NAMESPACE "VolumeTextureFactory"

UVolumeTextureFactory::UVolumeTextureFactory( const FObjectInitializer& ObjectInitializer )
 : Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UVolumeTexture::StaticClass();
}

FText UVolumeTextureFactory::GetDisplayName() const
{
	return LOCTEXT("VolumeTextureFactoryDescription", "Volume Texture");
}

bool UVolumeTextureFactory::ConfigureProperties()
{
	return true;
}

UObject* UVolumeTextureFactory::FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn)
{
	UVolumeTexture* NewVolumeTexture = NewObject<UVolumeTexture>(InParent, Name, Flags);
	NewVolumeTexture->MipGenSettings = TMGS_FromTextureGroup;
	NewVolumeTexture->NeverStream = true;
	NewVolumeTexture->CompressionNone = false;

	if (InitialTexture)
	{
		NewVolumeTexture->SRGB = InitialTexture->SRGB;

		const FTextureSource& Source = InitialTexture->Source;
		const int32 NumPixels = Source.GetSizeX() * Source.GetSizeY();

		if (NumPixels > 0)
		{
			// By default, assume that the 2d texture holds a square images.
			int32 TileSize = FMath::RoundToInt(FMath::Pow((float)NumPixels, 1.f / 3.f));

			// Remap the tiles as if there were streached accross the texture length.
			int32 NumTilesBySide = FMath::RoundToInt(FMath::Sqrt((float)(Source.GetSizeX() / TileSize) * (Source.GetSizeY() / TileSize)));

			if (NumTilesBySide > 0)
			{
				NewVolumeTexture->Source2DTexture = InitialTexture;
				NewVolumeTexture->Source2DTileSizeX = Source.GetSizeX() / NumTilesBySide;
				NewVolumeTexture->Source2DTileSizeY = Source.GetSizeY() / NumTilesBySide;

				NewVolumeTexture->UpdateSourceFromSourceTexture();
				NewVolumeTexture->UpdateResource();
			}

		}
	}
	else
	{
		// Initialize the texture with a single texel of data
		const uint8 Texel[] = { 0, 0, 0, 255 };
		NewVolumeTexture->Source.Init(1, 1, 1, 1, TSF_BGRA8, Texel);
		NewVolumeTexture->UpdateResource();
	}

	return NewVolumeTexture;
}

#undef LOCTEXT_NAMESPACE
