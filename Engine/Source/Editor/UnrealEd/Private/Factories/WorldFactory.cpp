// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/WorldFactory.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"
#include "UObject/Package.h"
#include "ThumbnailRendering/WorldThumbnailInfo.h"
#include "WorldPartition/WorldPartitionSettings.h"
#include "EditorClassUtils.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldFactory)

#define LOCTEXT_NAMESPACE "WorldFactory"

UWorldFactory::UWorldFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UWorld::StaticClass();
	WorldType = EWorldType::Inactive;
	bInformEngineOfWorld = false;
	bCreateWorldPartition = UWorldPartitionSettings::Get()->GetNewMapsEnableWorldPartition();
	bEnableWorldPartitionStreaming = UWorldPartitionSettings::Get()->GetNewMapsEnableWorldPartitionStreaming();
	FeatureLevel = ERHIFeatureLevel::Num;
}

bool UWorldFactory::ConfigureProperties()
{
	return true;
}

UObject* UWorldFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	// Create a new world.
	const bool bAddToRoot = false;

	// Those are the init values taken from the default in UWorld::CreateWorld + CreateWorldPartition.
	UWorld::InitializationValues InitValues = UWorld::InitializationValues()
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(true)
		.CreateNavigation(WorldType == EWorldType::Editor)
		.CreateAISystem(WorldType == EWorldType::Editor)
		.CreateWorldPartition(bCreateWorldPartition)
		.EnableWorldPartitionStreaming(bEnableWorldPartitionStreaming);

	UWorld* NewWorld = UWorld::CreateWorld(WorldType, bInformEngineOfWorld, Name, Cast<UPackage>(InParent), bAddToRoot, FeatureLevel, &InitValues);
	GEditor->InitBuilderBrush(NewWorld);
	NewWorld->SetFlags(Flags);
	NewWorld->ThumbnailInfo = NewObject<UWorldThumbnailInfo>(NewWorld, NAME_None, RF_Transactional);

	return NewWorld;
}

FText UWorldFactory::GetToolTip() const
{
	return ULevel::StaticClass()->GetToolTipText();
}

FString UWorldFactory::GetToolTipDocumentationPage() const
{
	return FEditorClassUtils::GetDocumentationPage(ULevel::StaticClass());
}

FString UWorldFactory::GetToolTipDocumentationExcerpt() const
{
	return FEditorClassUtils::GetDocumentationExcerpt(ULevel::StaticClass());
}

#undef LOCTEXT_NAMESPACE
