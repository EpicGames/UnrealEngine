// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxLibs.h"
#include "FbxExporter.h"

//-------------------------------------------------------------------------
// Memory management callback functions used by the FBX SDK
//-------------------------------------------------------------------------
void* MyMalloc(size_t pSize)       
{
	return FMemory::Malloc(pSize);
}

void* MyCalloc(size_t pCount,size_t pSize)
{
	void* Alloc = FMemory::Malloc(pCount*pSize);
	return FMemory::Memzero(Alloc, pCount*pSize);
}

void* MyRealloc(void* pData, size_t pSize)
{
	return FMemory::Realloc(pData, pSize);
}

void  MyFree(void* pData)
{
	FMemory::Free(pData);
}


void LoadFBxLibraries()
{
	// Specify global memory handler callbacks to be used by the FBX SDK
	FbxSetMallocHandler(	&MyMalloc);
	FbxSetCallocHandler(	&MyCalloc);
	FbxSetReallocHandler(	&MyRealloc);
	FbxSetFreeHandler(		&MyFree);
}

void UnloadFBxLibraries()
{
	UnFbx::FFbxImporter::DeleteInstance();
	UnFbx::FFbxExporter::DeleteInstance();

	// Hack: After we have freed our fbx sdk instance we need to set back to the default fbx memory handlers. 
	// This is required because there are some allocations made in the FBX dllmain before it is possible to set up our custom allocators
	// If this is not done, memory created by one allocator will be freed by another
	FbxSetMallocHandler(	FbxGetDefaultMallocHandler() );
	FbxSetCallocHandler(	FbxGetDefaultCallocHandler() );
	FbxSetReallocHandler(	FbxGetDefaultReallocHandler() );
	FbxSetFreeHandler(		FbxGetDefaultFreeHandler() );
}
