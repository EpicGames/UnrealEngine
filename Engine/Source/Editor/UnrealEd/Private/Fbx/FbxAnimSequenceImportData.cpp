// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxAnimSequenceImportData.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxAnimSequenceImportData)

UFbxAnimSequenceImportData::UFbxAnimSequenceImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bImportMeshesInBoneHierarchy(true)
	, bImportCustomAttribute(true)
	, bImportBoneTracks(true)
	, bAddCurveMetadataToSkeleton(true)
	, bRemoveRedundantKeys(true)
	, bDoNotImportCurveWithZero(true)
{
	FrameImportRange.Min = 0;
	FrameImportRange.Max = 0;

	MaterialCurveSuffixes.Add(TEXT("_mat"));
	CustomSampleRate = 0;
}

UFbxAnimSequenceImportData* UFbxAnimSequenceImportData::GetImportDataForAnimSequence(UAnimSequence* AnimSequence, UFbxAnimSequenceImportData* TemplateForCreation)
{
	check(AnimSequence);

	UFbxAnimSequenceImportData* ImportData = Cast<UFbxAnimSequenceImportData>(AnimSequence->AssetImportData);
	if ( !ImportData )
	{
		ImportData = NewObject<UFbxAnimSequenceImportData>(AnimSequence, NAME_None, RF_NoFlags, TemplateForCreation);

		// Try to preserve the source file data if possible
		if ( AnimSequence->AssetImportData != NULL )
		{
			ImportData->SourceData = AnimSequence->AssetImportData->SourceData;
		}

		AnimSequence->AssetImportData = ImportData;
	}

	return ImportData;
}

bool UFbxAnimSequenceImportData::CanEditChange(const FProperty* InProperty) const
{
	bool bMutable = Super::CanEditChange(InProperty);
	UObject* Outer = GetOuter();
	if(Outer && bMutable)
	{
		// Let the FbxImportUi object handle the editability of our properties
		bMutable = Outer->CanEditChange(InProperty);
	}
	return bMutable;
}

void UFbxAnimSequenceImportData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.UEVer() < VER_UE4_FBX_IMPORT_DATA_RANGE_ENCAPSULATION)
	{
		FrameImportRange.Min = StartFrame_DEPRECATED;
		FrameImportRange.Max = EndFrame_DEPRECATED;

		FrameImportRange.Min = FMath::Max(0, FMath::Min(FrameImportRange.Min, FrameImportRange.Max));
		FrameImportRange.Max = FMath::Max(0, FMath::Max(FrameImportRange.Min, FrameImportRange.Max));
	}
}

void UFbxAnimSequenceImportData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxAnimSequenceImportData, FrameImportRange))
	{
		FrameImportRange.Min = FMath::Max(0, FMath::Min(FrameImportRange.Min, FrameImportRange.Max));
		FrameImportRange.Max = FMath::Max(0, FMath::Max(FrameImportRange.Min, FrameImportRange.Max));
	}
}

void UFbxAnimSequenceImportData::CopyAnimationValues(const UFbxAnimSequenceImportData* Other)
{
	if (!ensure(Other))
	{
		return;
	}
	AnimationLength = Other->AnimationLength;
	bDeleteExistingCustomAttributeCurves = Other->bDeleteExistingCustomAttributeCurves;
	bDeleteExistingMorphTargetCurves = Other->bDeleteExistingMorphTargetCurves;
	bDeleteExistingNonCurveCustomAttributes = Other->bDeleteExistingNonCurveCustomAttributes;
	bDoNotImportCurveWithZero = Other->bDoNotImportCurveWithZero;
	bImportBoneTracks = Other->bImportBoneTracks;
	bImportCustomAttribute = Other->bImportCustomAttribute;
	bImportMeshesInBoneHierarchy = Other->bImportMeshesInBoneHierarchy;
	bPreserveLocalTransform = Other->bPreserveLocalTransform;
	bRemoveRedundantKeys = Other->bRemoveRedundantKeys;
	bSetMaterialDriveParameterOnCustomAttribute = Other->bSetMaterialDriveParameterOnCustomAttribute;
	bAddCurveMetadataToSkeleton = Other->bAddCurveMetadataToSkeleton;
	bUseDefaultSampleRate = Other->bUseDefaultSampleRate;
	CustomSampleRate = Other->CustomSampleRate;
	FrameImportRange = Other->FrameImportRange;
	MaterialCurveSuffixes = Other->MaterialCurveSuffixes;
	SourceAnimationName = Other->SourceAnimationName;
}
