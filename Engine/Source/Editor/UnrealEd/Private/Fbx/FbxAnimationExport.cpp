// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  Implementation of animation export related functionality from FbxExporter
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Animation/AnimTypes.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Animation/AnimSequence.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Animation/SkeletalMeshActor.h"
#include "FbxExporter.h"
#include "Exporters/FbxExportOption.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/BuiltInAttributeTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogFbxAnimationExport, Log, All);

namespace UnFbx
{

	bool FFbxExporter::SetupAnimStack(const UAnimSequence* AnimSeq)
	{
		if (AnimSeq->GetPlayLength() == 0.f)
		{
			// something is wrong
			return false;
		}

		const double FrameRate = AnimSeq->GetDataModel()->GetFrameRate().AsDecimal();
		//Configure the scene time line
		{
			FbxGlobalSettings& SceneGlobalSettings = Scene->GetGlobalSettings();
			double CurrentSceneFrameRate = FbxTime::GetFrameRate(SceneGlobalSettings.GetTimeMode());
			if (!bSceneGlobalTimeLineSet || FrameRate > CurrentSceneFrameRate)
			{
				FbxTime::EMode ComputeTimeMode = FbxTime::ConvertFrameRateToTimeMode(FrameRate);
				FbxTime::SetGlobalTimeMode(ComputeTimeMode, ComputeTimeMode == FbxTime::eCustom ? FrameRate : 0.0);
				SceneGlobalSettings.SetTimeMode(ComputeTimeMode);
				if (ComputeTimeMode == FbxTime::eCustom)
				{
					SceneGlobalSettings.SetCustomFrameRate(FrameRate);
				}
				bSceneGlobalTimeLineSet = true;
			}
		}

		// set time correctly
		FbxTime ExportedStartTime, ExportedStopTime;
		ExportedStartTime.SetSecondDouble(0.f);
		ExportedStopTime.SetSecondDouble(AnimSeq->GetDataModel()->GetFrameRate().AsSeconds(AnimSeq->GetDataModel()->GetNumberOfFrames()));

		FbxTimeSpan ExportedTimeSpan;
		ExportedTimeSpan.Set(ExportedStartTime, ExportedStopTime);
		AnimStack->SetLocalTimeSpan(ExportedTimeSpan);

		return true;
	}

void FFbxExporter::ExportAnimSequenceToFbx(const UAnimSequence* AnimSeq, const USkeletalMesh* SkelMesh, TArray<FbxNode*>& BoneNodes, FbxAnimLayer* InAnimLayer, FFrameTime StartFrameTime, FFrameTime EndFrameTime, float FrameRateScale, float StartTime)
{
	// stack allocator for extracting curve
	FMemMark Mark(FMemStack::Get());

	USkeleton* Skeleton = AnimSeq->GetSkeleton();

	if (Skeleton == nullptr || !SetupAnimStack(AnimSeq))
	{
		// something is wrong
		return;	
	}

	//Prepare root anim curves data to be exported
	TMap<FName, FbxAnimCurve*> CustomCurveMap;
	if (BoneNodes.Num() > 0)
	{
		const UFbxExportOption* ExportOptions = GetExportOptions();
		const bool bExportMorphTargetCurvesInMesh = ExportOptions && ExportOptions->bExportPreviewMesh && ExportOptions->bExportMorphTargets;

		Skeleton->ForEachCurveMetaData([&CustomCurveMap, &BoneNodes, InAnimLayer, bExportMorphTargetCurvesInMesh](const FName& InCurveName, const FCurveMetaData& InMetaData)
		{
			//Only export the custom curve if it is not used in a MorphTarget that will be exported latter on.
			if(!(bExportMorphTargetCurvesInMesh && InMetaData.Type.bMorphtarget))
			{
				FbxProperty AnimCurveFbxProp = FbxProperty::Create(BoneNodes[0], FbxDoubleDT, TCHAR_TO_ANSI(*InCurveName.ToString()));
				AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
				AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
				FbxAnimCurve* AnimFbxCurve = AnimCurveFbxProp.GetCurve(InAnimLayer, true);
				CustomCurveMap.Add(InCurveName, AnimFbxCurve);
			}
		});
	}

	ExportCustomAnimCurvesToFbx(CustomCurveMap, AnimSeq, StartFrameTime, EndFrameTime, FrameRateScale, StartTime);

	TArray<const FAnimatedBoneAttribute*> CustomAttributes;

	// Add the animation data to the bone nodes
	for(int32 BoneIndex = 0; BoneIndex < BoneNodes.Num(); ++BoneIndex)
	{
		FbxNode* CurrentBoneNode = BoneNodes[BoneIndex];
		const int32 BoneTreeIndex = Skeleton->GetSkeletonBoneIndexFromMeshBoneIndex(SkelMesh, BoneIndex);

		if (!Skeleton->GetReferenceSkeleton().IsValidIndex(BoneTreeIndex))
		{
			ensureMsgf(false, TEXT("Failed to find Bone [%hs] in Reference Skeleton, when exporting AnimSequence [%s]."), CurrentBoneNode->GetName(), *AnimSeq->GetName());

			continue;
		}

		const FName BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(BoneTreeIndex);
		const IAnimationDataModel* DataModel = AnimSeq->GetDataModel();

		CustomAttributes.Reset();
		DataModel->GetAttributesForBone(BoneName, CustomAttributes);

		TArray<TPair<int32, FbxAnimCurve*>> FloatCustomAttributeIndices;
		TArray<TPair<int32, FbxAnimCurve*>> IntCustomAttributeIndices;

		// Setup custom attribute properties and curves
		for (int32 AttributeIndex = 0; AttributeIndex < CustomAttributes.Num(); ++AttributeIndex)
		{
			if (const FAnimatedBoneAttribute* AttributePtr = CustomAttributes[AttributeIndex])
			{
				const FAnimatedBoneAttribute& Attribute = *AttributePtr;

				const FName& AttributeName = Attribute.Identifier.GetName();
				const UScriptStruct* AttributeType = Attribute.Identifier.GetType();

				if (AttributeType == FIntegerAnimationAttribute::StaticStruct())
				{
					FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxIntDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
					AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
					AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

					FbxAnimCurve* AnimFbxCurve = AnimCurveFbxProp.GetCurve(InAnimLayer, true);
					AnimFbxCurve->KeyModifyBegin();
					IntCustomAttributeIndices.Emplace(AttributeIndex, AnimFbxCurve);
				}
				else if (AttributeType == FFloatAnimationAttribute::StaticStruct())
				{
					FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxDoubleDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
					AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
					AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

					FbxAnimCurve* AnimFbxCurve = AnimCurveFbxProp.GetCurve(InAnimLayer, true);
					AnimFbxCurve->KeyModifyBegin();
					FloatCustomAttributeIndices.Emplace(AttributeIndex, AnimFbxCurve);
				}
				else if (AttributeType == FStringAnimationAttribute::StaticStruct())
				{
					FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxStringDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
					AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

					// String attributes can't be keyed, simply set a normal value.
					FStringAnimationAttribute EvaluatedAttribute = Attribute.Curve.Evaluate<FStringAnimationAttribute>(0.f);

					FbxString FbxValueString(TCHAR_TO_UTF8(*EvaluatedAttribute.Value));
					AnimCurveFbxProp.Set(FbxValueString);
				}
				else
				{
					ensureMsgf(false, TEXT("Trying to export unsupported custom attribte (float, int32 and FString are currently supported)"));
				}
			}
		}

		// Create the transform AnimCurves
		const uint32 NumberOfCurves = 9;
		FbxAnimCurve* Curves[NumberOfCurves];
		
		// Individual curves for translation, rotation and scaling
		Curves[0] = CurrentBoneNode->LclTranslation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
		Curves[1] = CurrentBoneNode->LclTranslation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
		Curves[2] = CurrentBoneNode->LclTranslation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);
		
		Curves[3] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
		Curves[4] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
		Curves[5] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);
		
		Curves[6] = CurrentBoneNode->LclScaling.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
		Curves[7] = CurrentBoneNode->LclScaling.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
		Curves[8] = CurrentBoneNode->LclScaling.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

		if(!DataModel->IsValidBoneTrackName(BoneName))
		{
			// If this sequence does not have a track for the current bone, then skip it
			continue;
		}
		
		for (FbxAnimCurve* Curve : Curves)
		{
			Curve->KeyModifyBegin();
		}

		auto ExportLambda = [this, DataModel, BoneName, AnimSeq, &FloatCustomAttributeIndices, &IntCustomAttributeIndices, &Curves, &CustomAttributes](double AnimTime, FbxTime ExportTime, bool bLastKey)
		{
			const FTransform BoneAtom = DataModel->EvaluateBoneTrackTransform(BoneName, DataModel->GetFrameRate().AsFrameTime(AnimTime), AnimSeq->Interpolation);
			const FbxAMatrix FbxMatrix = Converter.ConvertMatrix(BoneAtom.ToMatrixWithScale());
			
			const FbxVector4 Translation = FbxMatrix.GetT();
			const FbxVector4 Rotation = FbxMatrix.GetR();
			const FbxVector4 Scale = FbxMatrix.GetS();
			const FbxVector4 Vectors[3] = { Translation, Rotation, Scale };

			// Loop over each curve and channel to set correct values
			for (uint32 CurveIndex = 0; CurveIndex < 3; ++CurveIndex)
			{
				for (uint32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const uint32 OffsetCurveIndex = (CurveIndex * 3) + ChannelIndex;

					const int32 lKeyIndex = Curves[OffsetCurveIndex]->KeyAdd(ExportTime);
					Curves[OffsetCurveIndex]->KeySetValue(lKeyIndex, Vectors[CurveIndex][ChannelIndex]);
					Curves[OffsetCurveIndex]->KeySetInterpolation(lKeyIndex, bLastKey ? FbxAnimCurveDef::eInterpolationConstant : FbxAnimCurveDef::eInterpolationCubic);

					if (bLastKey)
					{
						Curves[OffsetCurveIndex]->KeySetConstantMode(lKeyIndex, FbxAnimCurveDef::eConstantStandard);
					}
				}
			}

			for (const TPair<int32, FbxAnimCurve*>& CurrentAttributeCurve : FloatCustomAttributeIndices)
			{
				if (const FAnimatedBoneAttribute* AttributePtr = CustomAttributes[CurrentAttributeCurve.Key])
				{
					ensure(AttributePtr->Identifier.GetType() == FFloatAnimationAttribute::StaticStruct());

					const FFloatAnimationAttribute EvaluatedAttribute = AttributePtr->Curve.Evaluate<FFloatAnimationAttribute>(AnimTime);
					const int32 KeyIndex = CurrentAttributeCurve.Value->KeyAdd(ExportTime);
					CurrentAttributeCurve.Value->KeySetValue(KeyIndex, EvaluatedAttribute.Value);

				}
			}

			for (const TPair<int32, FbxAnimCurve*>& CurrentAttributeCurve : IntCustomAttributeIndices)
			{
				if (const FAnimatedBoneAttribute* AttributePtr = CustomAttributes[CurrentAttributeCurve.Key])
				{
					ensure(AttributePtr->Identifier.GetType() == FIntegerAnimationAttribute::StaticStruct());

					const FIntegerAnimationAttribute EvaluatedAttribute = AttributePtr->Curve.Evaluate<FIntegerAnimationAttribute>(AnimTime);
					const int32 KeyIndex = CurrentAttributeCurve.Value->KeyAdd(ExportTime);
					CurrentAttributeCurve.Value->KeySetValue(KeyIndex, static_cast<float>(EvaluatedAttribute.Value));
				}
			}
		};

		IterateInsideAnimSequence(AnimSeq, StartFrameTime, EndFrameTime, FrameRateScale, StartTime, ExportLambda);

		for (FbxAnimCurve* Curve : Curves)
		{
			Curve->KeyModifyEnd();
		}

		auto MarkCurveEnd = [](auto& CurvesArray)
		{
			for (auto& CurvePair : CurvesArray)
			{
				CurvePair.Value->KeyModifyEnd();
			}
		};

		MarkCurveEnd(FloatCustomAttributeIndices);
		MarkCurveEnd(IntCustomAttributeIndices);
	}
}

void FFbxExporter::ExportCustomAnimCurvesToFbx(const TMap<FName, FbxAnimCurve*>& CustomCurves, const UAnimSequence* AnimSeq, 
	float AnimStartOffset, float AnimEndOffset, float AnimPlayRate, float StartTime, float ValueScale)
{
	ExportCustomAnimCurvesToFbx(CustomCurves, AnimSeq,
		AnimSeq->GetDataModel()->GetFrameRate().AsFrameTime(AnimStartOffset),
		AnimSeq->GetDataModel()->GetFrameRate().AsFrameTime(AnimSeq->GetPlayLength() - AnimEndOffset),
		AnimPlayRate, StartTime, ValueScale);
}
	
void FFbxExporter::ExportCustomAnimCurvesToFbx(const TMap<FName, FbxAnimCurve*>& CustomCurves, const UAnimSequence* AnimSeq,
	FFrameTime AnimStartOffset, FFrameTime AnimEndOffset, float FrameRateScale, float StartTime, float ValueScale /*= 1.f*/)
{
	// stack allocator for extracting curve
	FMemMark Mark(FMemStack::Get());

	if (!SetupAnimStack(AnimSeq))
	{
		//Something is wrong.
		return;
	}

	for (auto CustomCurve : CustomCurves)
	{
		CustomCurve.Value->KeyModifyBegin();
	}
	
	auto ExportLambda = [&](float AnimTime, FbxTime ExportTime, bool bLastKey) {
		FBlendedCurve BlendedCurve;
		AnimSeq->EvaluateCurveData(BlendedCurve, FAnimExtractContext(static_cast<double>(AnimTime)), true);
		
		//Loop over the custom curves and add the actual keys
		for (auto CustomCurve : CustomCurves)
		{
			float CurveValueAtTime = BlendedCurve.Get(CustomCurve.Key) * ValueScale;
			int32 KeyIndex = CustomCurve.Value->KeyAdd(ExportTime);
			CustomCurve.Value->KeySetValue(KeyIndex, CurveValueAtTime);
		}
	};

	IterateInsideAnimSequence(AnimSeq, AnimStartOffset, AnimEndOffset, FrameRateScale, StartTime, ExportLambda);

	for (auto CustomCurve : CustomCurves)
	{
		CustomCurve.Value->KeyModifyEnd();
	}
}

void FFbxExporter::IterateInsideAnimSequence(const UAnimSequence* AnimSeq, float AnimStartOffset, float AnimEndOffset, float AnimPlayRate, float StartTime, TFunctionRef<void(float, FbxTime, bool)> IterationLambda)
{
	IterateInsideAnimSequence(AnimSeq,
		AnimSeq->GetDataModel()->GetFrameRate().AsFrameTime(AnimStartOffset),
		AnimSeq->GetDataModel()->GetFrameRate().AsFrameTime(AnimSeq->GetPlayLength() - AnimEndOffset),
		AnimPlayRate,
		AnimPlayRate,
		[IterationLambda](double T, FbxTime FT, bool B) -> void
		{
			IterationLambda((float)T, FT, B);
		});
}

void FFbxExporter::IterateInsideAnimSequence(const UAnimSequence* AnimSeq, FFrameTime StartFrameTime, FFrameTime EndFrameTime, float FrameRateScale, float StartTime, TFunctionRef<void(double, FbxTime, bool)> IterationLambda)
{
	const double TimePerKey = AnimSeq->GetDataModel()->GetFrameRate().AsInterval();
	const double AnimTimeIncrement = TimePerKey * FrameRateScale;
	uint32 AnimFrameIndex = 0;

	FbxTime ExportTime;
	ExportTime.SetSecondDouble(StartTime);

	FbxTime ExportTimeIncrement;
	ExportTimeIncrement.SetSecondDouble(TimePerKey);

	// Step through each frame and add custom curve data
	bool bLastKey = false;
	FFrameTime FrameTime = StartFrameTime;
	const FFrameRate& FrameRate = AnimSeq->GetDataModel()->GetFrameRate();
	while (!bLastKey)
	{
		bLastKey = FrameTime > EndFrameTime;
		IterationLambda(FrameRate.AsSeconds(FrameTime), ExportTime, bLastKey);

		ExportTime += ExportTimeIncrement;
		AnimFrameIndex++;
		FrameTime += FFrameTime::FromDecimal(FrameRateScale);
	}
}

// The curve code doesn't differentiate between angles and other data, so an interpolation from 179 to -179
// will cause the bone to rotate all the way around through 0 degrees.  So here we make a second pass over the 
// rotation tracks to convert the angles into a more interpolation-friendly format.  
void FFbxExporter::CorrectAnimTrackInterpolation( TArray<FbxNode*>& BoneNodes, FbxAnimLayer* InAnimLayer )
{
	// Add the animation data to the bone nodes
	for(int32 BoneIndex = 0; BoneIndex < BoneNodes.Num(); ++BoneIndex)
	{
		FbxNode* CurrentBoneNode = BoneNodes[BoneIndex];

		// Fetch the AnimCurves
		FbxAnimCurve* Curves[3];
		Curves[0] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
		Curves[1] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
		Curves[2] = CurrentBoneNode->LclRotation.GetCurve(InAnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

		for(int32 CurveIndex = 0; CurveIndex < 3; ++CurveIndex)
		{
			FbxAnimCurve* CurrentCurve = Curves[CurveIndex];
			CurrentCurve->KeyModifyBegin();

			float CurrentAngleOffset = 0.f;
			for(int32 KeyIndex = 1; KeyIndex < CurrentCurve->KeyGetCount(); ++KeyIndex)
			{
				float PreviousOutVal	= CurrentCurve->KeyGetValue( KeyIndex-1 );
				float CurrentOutVal		= CurrentCurve->KeyGetValue( KeyIndex );

				float DeltaAngle = (CurrentOutVal + CurrentAngleOffset) - PreviousOutVal;

				if(DeltaAngle >= 180)
				{
					CurrentAngleOffset -= 360;
				}
				else if(DeltaAngle <= -180)
				{
					CurrentAngleOffset += 360;
				}

				CurrentOutVal += CurrentAngleOffset;

				CurrentCurve->KeySetValue(KeyIndex, CurrentOutVal);
			}

			CurrentCurve->KeyModifyEnd();
		}
	}
}


FbxNode* FFbxExporter::ExportAnimSequence( const UAnimSequence* AnimSeq, const USkeletalMesh* SkelMesh, bool bExportSkelMesh, const TCHAR* MeshName, FbxNode* ActorRootNode, const TArray<UMaterialInterface*>* OverrideMaterials /*= nullptr*/ )
{
	if( Scene == nullptr || AnimSeq == nullptr || SkelMesh == nullptr )
	{
 		return nullptr;
	}


	FbxNode* RootNode = (ActorRootNode)? ActorRootNode : Scene->GetRootNode();

	//Create a temporary node attach to the scene root.
	//This will allow us to do the binding without the scene transform (non uniform scale is not supported when binding the skeleton)
	//We then detach from the temp node and attach to the parent and remove the temp node
	FString FbxNodeName = FGuid::NewGuid().ToString(EGuidFormats::Digits);
	FbxNode* TmpNodeNoTransform = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));
	Scene->GetRootNode()->AddChild(TmpNodeNoTransform);


	// Create the Skeleton
	TArray<FbxNode*> BoneNodes;
	FbxNode* SkeletonRootNode = CreateSkeleton(SkelMesh, BoneNodes);
	TmpNodeNoTransform->AddChild(SkeletonRootNode);


	// Export the anim sequence
	{
		ExportAnimSequenceToFbx(AnimSeq,
			SkelMesh,
			BoneNodes,
			AnimLayer,
			// Start frame to export
			FFrameTime(0),
			// Final frame to export
			FFrameTime(AnimSeq->GetDataModel()->GetNumberOfFrames()),
			// Frame rate scale
			1.f,
			// FBX StartTime
			0.f);

		CorrectAnimTrackInterpolation(BoneNodes, AnimLayer);
	}


	// Optionally export the mesh
	if(bExportSkelMesh)
	{
		FString MeshNodeName;
		
		if (MeshName)
		{
			MeshNodeName = MeshName;
		}
		else
		{
			SkelMesh->GetName(MeshNodeName);
		}

		FbxNode* MeshRootNode = nullptr;
		if (GetExportOptions()->LevelOfDetail && SkelMesh->GetLODNum() > 1)
		{
			FString LodGroup_MeshName = MeshNodeName + TEXT("_LodGroup");
			MeshRootNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*LodGroup_MeshName));
			TmpNodeNoTransform->AddChild(MeshRootNode);
			LodGroup_MeshName = MeshNodeName + TEXT("_LodGroupAttribute");
			FbxLODGroup *FbxLodGroupAttribute = FbxLODGroup::Create(Scene, TCHAR_TO_UTF8(*LodGroup_MeshName));
			MeshRootNode->AddNodeAttribute(FbxLodGroupAttribute);

			FbxLodGroupAttribute->ThresholdsUsedAsPercentage = true;
			//Export an Fbx Mesh Node for every LOD and child them to the fbx node (LOD Group)
			for (int CurrentLodIndex = 0; CurrentLodIndex < SkelMesh->GetLODNum(); ++CurrentLodIndex)
			{
				FString FbxLODNodeName = MeshNodeName + TEXT("_LOD") + FString::FromInt(CurrentLodIndex);
				if (CurrentLodIndex + 1 < SkelMesh->GetLODNum())
				{
					//Convert the screen size to a threshold, it is just to be sure that we set some threshold, there is no way to convert this precisely
					double LodScreenSize = (double)(10.0f / SkelMesh->GetLODInfo(CurrentLodIndex)->ScreenSize.Default);
					FbxLodGroupAttribute->AddThreshold(LodScreenSize);
				}
				FbxNode* FbxActorLOD = CreateMesh(SkelMesh, *FbxLODNodeName, CurrentLodIndex, AnimSeq, OverrideMaterials);
				if (FbxActorLOD)
				{
					MeshRootNode->AddChild(FbxActorLOD);
					if (SkeletonRootNode)
					{
						// Bind the mesh to the skeleton
						BindMeshToSkeleton(SkelMesh, FbxActorLOD, BoneNodes, CurrentLodIndex);
						// Add the bind pose
						CreateBindPose(FbxActorLOD);
					}
				}
			}
		}
		else
		{
			const int32 LodIndex = 0;
			MeshRootNode = CreateMesh(SkelMesh, *MeshNodeName, LodIndex, AnimSeq, OverrideMaterials);
			if (MeshRootNode)
			{
				TmpNodeNoTransform->AddChild(MeshRootNode);
				if (SkeletonRootNode)
				{
					// Bind the mesh to the skeleton
					BindMeshToSkeleton(SkelMesh, MeshRootNode, BoneNodes, LodIndex);

					// Add the bind pose
					CreateBindPose(MeshRootNode);
				}
			}
		}

		if (MeshRootNode)
		{
			TmpNodeNoTransform->RemoveChild(MeshRootNode);
			RootNode->AddChild(MeshRootNode);
			//Export the preview mesh metadata
			ExportObjectMetadata(SkelMesh, MeshRootNode);
		}
	}
	
	if (SkeletonRootNode)
	{
		TmpNodeNoTransform->RemoveChild(SkeletonRootNode);
		RootNode->AddChild(SkeletonRootNode);
	}

	Scene->GetRootNode()->RemoveChild(TmpNodeNoTransform);
	Scene->RemoveNode(TmpNodeNoTransform);

	return SkeletonRootNode;
}


void FFbxExporter::ExportAnimTrack(IAnimTrackAdapter& AnimTrackAdapter, AActor* Actor, USkeletalMeshComponent* InSkeletalMeshComponent, double SamplingRate)
{
	// show a status update every 1 second worth of samples
	const double UpdateFrequency = 1.0;
	double NextUpdateTime = UpdateFrequency;

	// find root and find the bone array
	TArray<FbxNode*> BoneNodes;

	if ( FindSkeleton(InSkeletalMeshComponent, BoneNodes)==false )
	{
		UE_LOGF(LogFbx, Warning, "Error FBX Animation Export, no root skeleton found.");
		return;		
	}
	//if we have no allocated bone space transforms something wrong so try to recalc them
	if (InSkeletalMeshComponent->GetBoneSpaceTransforms().Num() <= 0 )
	{
		InSkeletalMeshComponent->RecalcRequiredBones(0);
		if (InSkeletalMeshComponent->GetBoneSpaceTransforms().Num() <= 0)
		{
			UE_LOGF(LogFbx, Warning, "Error FBX Animation Export, no bone transforms.");
			return;
		}
	}
	
	TArray<const FAnimatedBoneAttribute*> CustomAttributes;
	
	FTransform InitialInvParentTransform;

	int32 LocalStartFrame = AnimTrackAdapter.GetLocalStartFrame();
	int32 StartFrame = AnimTrackAdapter.GetStartFrame();
	int32 AnimationLength = AnimTrackAdapter.GetLength();
	double FrameRate = AnimTrackAdapter.GetFrameRate();

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	Actor->GetComponents(SkeletalMeshComponents);

	const double TickRate = 1.0/FrameRate;

	FScopedSlowTask SlowTask(AnimationLength + 1, NSLOCTEXT("UnrealEd", "ExportAnimationProgress", "Exporting Animation"));
	SlowTask.MakeDialog(true);

	for (int32 FrameCount = 0; FrameCount <= AnimationLength; ++FrameCount)
	{
		SlowTask.EnterProgressFrame();
		
		int32 LocalFrame = LocalStartFrame + FrameCount;
		double SampleTime = (StartFrame + FrameCount) / FrameRate;

		// This will call UpdateSkelPose on the skeletal mesh component to move bones based on animations in the sequence
		AnimTrackAdapter.UpdateAnimation(LocalFrame);

		if (FrameCount == 0)
		{
			InitialInvParentTransform = Actor->GetRootComponent()->GetComponentTransform().Inverse();
		}

		// This will retrieve the currently active anim sequence (topmost) for custom attributes
		const UAnimSequence* AnimSeq = AnimTrackAdapter.GetAnimSequence(LocalFrame);
		float AnimTime = AnimTrackAdapter.GetAnimTime(LocalFrame);

		// Update space bases so new animation position has an effect.
		// @todo - hack - this will be removed at some point
		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			USceneComponent* Child = SkeletalMeshComponent;
			while (Child)
			{
				if (USkeletalMeshComponent* ChildSkeletalMeshComponent = Cast<USkeletalMeshComponent>(Child))
				{
					SkeletalMeshComponent->TickAnimation(TickRate, false);

					SkeletalMeshComponent->RefreshBoneTransforms();
					SkeletalMeshComponent->RefreshFollowerComponents();
					SkeletalMeshComponent->UpdateComponentToWorld();
					SkeletalMeshComponent->FinalizeBoneTransform();
					SkeletalMeshComponent->MarkRenderTransformDirty();
					SkeletalMeshComponent->MarkRenderDynamicDataDirty();
				}

				if (Child->GetOwner())
				{
					Child->GetOwner()->Tick(TickRate);
				}

				Child = Child->GetAttachParent();
			}
		}

		FbxTime ExportTime; 
		ExportTime.SetSecondDouble(GetExportOptions()->bExportLocalTime ? LocalFrame / FrameRate : SampleTime);

		NextUpdateTime -= SamplingRate;

		if( NextUpdateTime <= 0.0 )
		{
			NextUpdateTime = UpdateFrequency;
			GWarn->StatusUpdate( FMath::RoundToInt( SampleTime ), AnimationLength, NSLOCTEXT("FbxExporter", "ExportingToFbxStatus", "Exporting to FBX") );
		}

		TArray<FTransform> LocalBoneTransforms = InSkeletalMeshComponent->GetBoneSpaceTransforms();

		if (LocalBoneTransforms.Num() == 0)
		{
			continue;
		}

		// Add the animation data to the bone nodes
		for(int32 BoneIndex = 0; BoneIndex < BoneNodes.Num(); ++BoneIndex)
		{
			if (!InSkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton().IsValidIndex(BoneIndex))
			{
				UE_LOGF(LogFbxAnimationExport, Warning, "Invalid BoneIndex %d, did the skeleton change? (animating the skeleton is not currently supported)", BoneIndex);
				continue;
			}

			FName BoneName = InSkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(BoneIndex);
			FbxNode* CurrentBoneNode = BoneNodes[BoneIndex];

			// Create the AnimCurves
			FbxAnimCurve* Curves[9];
			Curves[0] = CurrentBoneNode->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
			Curves[1] = CurrentBoneNode->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
			Curves[2] = CurrentBoneNode->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

			Curves[3] = CurrentBoneNode->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
			Curves[4] = CurrentBoneNode->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
			Curves[5] = CurrentBoneNode->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

			Curves[6] = CurrentBoneNode->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
			Curves[7] = CurrentBoneNode->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
			Curves[8] = CurrentBoneNode->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

			for(int32 i = 0; i < 9; ++i)
			{
				Curves[i]->KeyModifyBegin();
			}

			FTransform BoneTransform = LocalBoneTransforms[BoneIndex];

			if (GetExportOptions()->MapSkeletalMotionToRoot && BoneIndex == 0)
			{
				BoneTransform = InSkeletalMeshComponent->GetSocketTransform(BoneName) * InitialInvParentTransform;
			}

			FbxVector4 Translation = Converter.ConvertToFbxPos(BoneTransform.GetLocation());
			FbxVector4 Rotation = Converter.ConvertToFbxRot(BoneTransform.GetRotation().Euler());
			FbxVector4 Scale = Converter.ConvertToFbxScale(BoneTransform.GetScale3D());

			int32 lKeyIndex;

			for(int32 i = 0, j=3, k=6; i < 3; ++i, ++j, ++k)
			{
				lKeyIndex = Curves[i]->KeyAdd(ExportTime);
				Curves[i]->KeySetValue(lKeyIndex, Translation[i]);
				Curves[i]->KeySetInterpolation(lKeyIndex, FbxAnimCurveDef::eInterpolationCubic);

				lKeyIndex = Curves[j]->KeyAdd(ExportTime);
				Curves[j]->KeySetValue(lKeyIndex, Rotation[i]);
				Curves[j]->KeySetInterpolation(lKeyIndex, FbxAnimCurveDef::eInterpolationCubic);

				lKeyIndex = Curves[k]->KeyAdd(ExportTime);
				Curves[k]->KeySetValue(lKeyIndex, Scale[i]);
				Curves[k]->KeySetInterpolation(lKeyIndex, FbxAnimCurveDef::eInterpolationCubic);
			}

			for(int32 i = 0; i < 9; ++i)
			{
				Curves[i]->KeyModifyEnd();
			}

			// Custom attributes
			if (!AnimSeq)
			{
				continue;
			}

			CustomAttributes.Reset();
			AnimSeq->GetDataModel()->GetAttributesForBone(BoneName, CustomAttributes);

			TArray<TPair<int32, FbxAnimCurve*>> FloatCustomAttributeIndices;
			TArray<TPair<int32, FbxAnimCurve*>> IntCustomAttributeIndices;

			// Setup custom attribute properties and curves
			for (int32 AttributeIndex = 0; AttributeIndex < CustomAttributes.Num(); ++AttributeIndex)
			{
				const FAnimatedBoneAttribute* AttributePtr = CustomAttributes[AttributeIndex];
				if (AttributePtr)
				{
					const FAnimatedBoneAttribute& Attribute = *AttributePtr;

					const FName& AttributeName = Attribute.Identifier.GetName();
					const UScriptStruct* AttributeType = Attribute.Identifier.GetType();

					if (AttributeType == FIntegerAnimationAttribute::StaticStruct())
					{
						FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxIntDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
						AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
						AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

						FbxAnimCurve* AnimFbxCurve = AnimCurveFbxProp.GetCurve(AnimLayer, true);
						AnimFbxCurve->KeyModifyBegin();
						IntCustomAttributeIndices.Emplace(AttributeIndex, AnimFbxCurve);
					}
					else if (AttributeType == FFloatAnimationAttribute::StaticStruct())
					{
						FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxFloatDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
						AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
						AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

						FbxAnimCurve* AnimFbxCurve = AnimCurveFbxProp.GetCurve(AnimLayer, true);
						AnimFbxCurve->KeyModifyBegin();
						FloatCustomAttributeIndices.Emplace(AttributeIndex, AnimFbxCurve);
					}
					else if (AttributeType == FStringAnimationAttribute::StaticStruct())
					{
						FbxProperty AnimCurveFbxProp = FbxProperty::Create(CurrentBoneNode, FbxStringDT, TCHAR_TO_UTF8(*AttributeName.ToString()));
						AnimCurveFbxProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);

						// String attributes can't be keyed, simply set a normal value.
						FStringAnimationAttribute EvaluatedAttribute = Attribute.Curve.Evaluate<FStringAnimationAttribute>(0.f);

						FbxString FbxValueString(TCHAR_TO_UTF8(*EvaluatedAttribute.Value));
						AnimCurveFbxProp.Set(FbxValueString);
					}
					else
					{
						ensureMsgf(false, TEXT("Trying to export unsupported custom attribte (float, int32 and FString are currently supported)"));
					}

					for (TPair<int32, FbxAnimCurve*>& CurrentAttributeCurve : FloatCustomAttributeIndices)
					{
						const FAnimatedBoneAttribute* FloatAttributePtr = CustomAttributes[CurrentAttributeCurve.Key];
						ensure(FloatAttributePtr->Identifier.GetType() == FFloatAnimationAttribute::StaticStruct());

						FFloatAnimationAttribute EvaluatedAttribute = FloatAttributePtr->Curve.Evaluate<FFloatAnimationAttribute>(AnimTime);

						int32 KeyIndex = CurrentAttributeCurve.Value->KeyAdd(ExportTime);
						CurrentAttributeCurve.Value->KeySetValue(KeyIndex, EvaluatedAttribute.Value);
					}

					for (TPair<int32, FbxAnimCurve*>& CurrentAttributeCurve : IntCustomAttributeIndices)
					{
						const FAnimatedBoneAttribute* IntAttributePtr = CustomAttributes[CurrentAttributeCurve.Key];
						ensure(IntAttributePtr->Identifier.GetType() == FIntegerAnimationAttribute::StaticStruct());

						FIntegerAnimationAttribute EvaluatedAttribute = IntAttributePtr->Curve.Evaluate< FIntegerAnimationAttribute>(AnimTime);

						int32 KeyIndex = CurrentAttributeCurve.Value->KeyAdd(ExportTime);
						CurrentAttributeCurve.Value->KeySetValue(KeyIndex, static_cast<float>(EvaluatedAttribute.Value));
					}
				}
			}

			auto MarkCurveEnd = [](auto& CurvesArray)
			{
				for (auto& CurvePair : CurvesArray)
				{
					CurvePair.Value->KeyModifyEnd();
				}
			};

			MarkCurveEnd(FloatCustomAttributeIndices);
			MarkCurveEnd(IntCustomAttributeIndices);
		}
	}

	CorrectAnimTrackInterpolation(BoneNodes, AnimLayer);
}

} // namespace UnFbx
