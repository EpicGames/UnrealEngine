// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxAssetImportData.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectHash.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Misc/ConfigCacheIni.h"
#include "FbxImporter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxAssetImportData)

UFbxAssetImportData::UFbxAssetImportData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ImportTranslation(0)
	, ImportRotation(0)
	, ImportUniformScale(1.0f)
	, bConvertScene(true)
	, bForceFrontXAxis(false)
	, bConvertSceneUnit(false)
	, bImportAsScene(false)
	, FbxSceneImportDataReference(nullptr)
{
	
}

void UFbxAssetImportData::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UFbxAssetImportData, CoordinateSystemPolicy))
	{
		bConvertScene = CoordinateSystemPolicy != ECoordinateSystemPolicy::KeepXYZAxes;
		bForceFrontXAxis = CoordinateSystemPolicy == ECoordinateSystemPolicy::MatchUpForwardAxes;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
