// Copyright Epic Games, Inc. All Rights Reserved.
#include "FbxCompareWindow.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SSplitter.h"
#include "Styling/AppStyle.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Factories/FbxSceneImportFactory.h"

#include "Framework/Application/SlateApplication.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Materials/Material.h"


#define LOCTEXT_NAMESPACE "FBXOption"
bool SFbxSkeltonConflictWindow::HasConflict()
{
	if (SourceObject->IsA(USkeletalMesh::StaticClass()))
	{
		for (TSharedPtr<FSkeletonCompareData> SkeletonCompareData : DisplaySkeletonTreeItem)
		{
			if (SkeletonCompareData->bChildConflict)
			{
				//We have at least one skeleton conflict
				return true;
			}
		}
	}
	return false;
}

void SFbxSkeltonConflictWindow::Construct(const FArguments& InArgs)
{
	bRevertReimport = false;

	bShowSectionFlag[EFBXCompareSection_Skeleton] = true;
	bShowSectionFlag[EFBXCompareSection_References] = true;

	WidgetWindow = InArgs._WidgetWindow;
	if (InArgs._AssetReferencingSkeleton != nullptr)
	{
		//Copy the aray
		AssetReferencingSkeleton = *(InArgs._AssetReferencingSkeleton);
	}
	SourceData = InArgs._SourceData;
	ResultData = InArgs._ResultData;
	SourceObject = InArgs._SourceObject;
	bIsPreviewConflict = InArgs._bIsPreviewConflict;

	if (SourceObject->IsA(USkeletalMesh::StaticClass()))
	{
		FilSkeletonTreeItem();
	}

	SetMatchJointInfo();

	// Skeleton comparison
	TSharedPtr<SWidget> SkeletonCompareSection = ConstructSkeletonComparison();
	TSharedPtr<SWidget> SkeletonReferencesSection = ConstructSkeletonReference();

	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(2.0f)
					[
						SNew(SSplitter)
						.Orientation(Orient_Vertical)
						.ResizeMode(ESplitterResizeMode::Fill)
						+ SSplitter::Slot()
						.Value(0.8f)
						[
							// Skeleton Compare section
							SkeletonCompareSection.ToSharedRef()
						]
						+ SSplitter::Slot()
						.Value(0.2f)
						[
							// Skeleton Compare section
							SkeletonReferencesSection.ToSharedRef()
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("SFbxSkeltonConflictWindow_Preview_Done", "Done"))
					.OnClicked(this, &SFbxSkeltonConflictWindow::OnDone)
				]
			]
		]
	];	
}

FReply SFbxSkeltonConflictWindow::SetSectionVisible(EFBXCompareSection SectionIndex)
{
	bShowSectionFlag[SectionIndex] = !bShowSectionFlag[SectionIndex];
	return FReply::Handled();
}

EVisibility SFbxSkeltonConflictWindow::IsSectionVisible(EFBXCompareSection SectionIndex)
{
	return bShowSectionFlag[SectionIndex] ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SFbxSkeltonConflictWindow::GetCollapsableArrow(EFBXCompareSection SectionIndex) const
{
	return bShowSectionFlag[SectionIndex] ? FAppStyle::GetBrush("Symbols.DownArrow") : FAppStyle::GetBrush("Symbols.RightArrow");
}

void RecursivelyExpandTreeItem(TSharedPtr<STreeView<TSharedPtr<FSkeletonCompareData>>> CompareTree, TSharedPtr<FSkeletonCompareData> RowData)
{
	if (RowData->bInitialAutoExpand || !RowData->bMatchJoint || !RowData->bChildConflict)
	{
		return;
	}
	RowData->bInitialAutoExpand = true;
	CompareTree->SetItemExpansion(RowData, true);
	for (TSharedPtr<FSkeletonCompareData> ChildRowData : RowData->ChildJoints)
	{
		RecursivelyExpandTreeItem(CompareTree, ChildRowData);
	}
}

TSharedPtr<SWidget> SFbxSkeltonConflictWindow::ConstructSkeletonComparison()
{
	if (!SourceObject->IsA(USkeletalMesh::StaticClass()))
	{
		//Return an empty widget, we do not show the skeleton when the mesh is not a skeletal mesh
		return SNew(SBox);
	}

	FText SkeletonStatus = bIsPreviewConflict ? FText(LOCTEXT("SFbxSkeltonConflictWindow_ConstructSkeletonComparison_MatchAndMergePreview", "The skeleton has some conflicts")) :
		FText(ResultData->CompSkeleton.bSkeletonFitMesh ? LOCTEXT("SFbxSkeltonConflictWindow_ConstructSkeletonComparison_MatchAndMerge", "The skeleton can be merged") : LOCTEXT("SFbxSkeltonConflictWindow_ConstructSkeletonComparison_CannotMatchAndMerge", "The skeleton must be regenerated, it cannot be merged"));
	
	CompareTree = SNew(STreeView< TSharedPtr<FSkeletonCompareData> >)
		.SelectionMode(ESelectionMode::None)
		.TreeItemsSource(&DisplaySkeletonTreeItem)
		.OnGenerateRow(this, &SFbxSkeltonConflictWindow::OnGenerateRowCompareTreeView)
		.OnGetChildren(this, &SFbxSkeltonConflictWindow::OnGetChildrenRowCompareTreeView);
	

	for (TSharedPtr<FSkeletonCompareData> RowData : DisplaySkeletonTreeItem)
	{
		RecursivelyExpandTreeItem(CompareTree, RowData);
	}
	
	return SNew(SBox)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked(this, &SFbxSkeltonConflictWindow::SetSectionVisible, EFBXCompareSection_Skeleton)
					[
						SNew(SImage).Image(this, &SFbxSkeltonConflictWindow::GetCollapsableArrow, EFBXCompareSection_Skeleton)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Text(LOCTEXT("SFbxSkeltonConflictWindow_SkeletonCompareHeader", "Skeleton"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SNew(SBox)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFbxSkeltonConflictWindow::IsSectionVisible, EFBXCompareSection_Skeleton)))
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.Text(SkeletonStatus)
							.ColorAndOpacity(ResultData->CompSkeleton.bSkeletonFitMesh ? FSlateColor::UseForeground() : FSlateColor(FLinearColor(0.7f, 0.3f, 0.0f)))
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(SSeparator)
							.Orientation(EOrientation::Orient_Horizontal)
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(2)
						[
							CompareTree.ToSharedRef()
						]
					]
				]
			]
		]
	];
}

TSharedPtr<SWidget> SFbxSkeltonConflictWindow::ConstructSkeletonReference()
{
	if (!SourceObject->IsA(USkeletalMesh::StaticClass()))
	{
		//Return an empty widget, we do not show the skeleton when the mesh is not a skeletal mesh
		return SNew(SBox);
	}

	FString SkeletonReferenceStatistic;
	if (AssetReferencingSkeleton.Num() > 0)
	{
		SkeletonReferenceStatistic += TEXT("Skeleton is references by ") + FString::FromInt(AssetReferencingSkeleton.Num()) + TEXT(" assets.");
	}
	
	return SNew(SBox)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked(this, &SFbxSkeltonConflictWindow::SetSectionVisible, EFBXCompareSection_References)
					[
						SNew(SImage).Image(this, &SFbxSkeltonConflictWindow::GetCollapsableArrow, EFBXCompareSection_References)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Text(LOCTEXT("SFbxSkeltonConflictWindow_SkeletonReferencesHeader", "References"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SNew(SBox)
				.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFbxSkeltonConflictWindow::IsSectionVisible, EFBXCompareSection_References)))
				[
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.Text(FText::FromString(SkeletonReferenceStatistic))
						]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(SSeparator)
							.Orientation(EOrientation::Orient_Horizontal)
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(2)
						[
							//Show the asset referencing this skeleton
							SNew(SListView<TSharedPtr<FString>>)
							.ListItemsSource(&AssetReferencingSkeleton)
							.OnGenerateRow(this, &SFbxSkeltonConflictWindow::OnGenerateRowAssetReferencingSkeleton)
						]
					]
				]
			]
		]
	];
}

class SCompareSkeletonTreeViewItem : public STableRow< TSharedPtr<FSkeletonCompareData> >
{
public:

	SLATE_BEGIN_ARGS(SCompareSkeletonTreeViewItem)
		: _SkeletonCompareData(nullptr)
		, _SourceData(nullptr)
		, _ResultData(nullptr)
	{}

	/** The item content. */
	SLATE_ARGUMENT(TSharedPtr<FSkeletonCompareData>, SkeletonCompareData)
	SLATE_ARGUMENT(FCompMesh*, SourceData)
	SLATE_ARGUMENT(FCompMesh*, ResultData)
	SLATE_END_ARGS()

	/**
	* Construct the widget
	*
	* @param InArgs   A declaration from which to construct the widget
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		SkeletonCompareData = InArgs._SkeletonCompareData;
		SourceData = InArgs._SourceData;
		ResultData = InArgs._ResultData;

		//This is suppose to always be valid
		check(SkeletonCompareData.IsValid());
		check(SourceData != nullptr);
		check(ResultData != nullptr);

		const FSlateBrush* JointIcon = SkeletonCompareData->bMatchJoint ? FAppStyle::GetDefaultBrush() : SkeletonCompareData->FbxJointIndex != INDEX_NONE ? FAppStyle::GetBrush("FBXIcon.ReimportCompareAdd") : FAppStyle::GetBrush("FBXIcon.ReimportCompareRemoved");

		//Prepare the tooltip
		FString Tooltip = SkeletonCompareData->bMatchJoint ? TEXT("") : FText(SkeletonCompareData->FbxJointIndex != INDEX_NONE ? LOCTEXT("SCompareSkeletonTreeViewItem_AddJoint_tooltip", "Fbx reimport will add this joint") : LOCTEXT("SCompareSkeletonTreeViewItem_RemoveJoint_tooltip", "Fbx reimport will remove this joint")).ToString();

		FSlateColor ForegroundTextColor = (SkeletonCompareData->bMatchJoint && !SkeletonCompareData->bChildConflict) ? FSlateColor::UseForeground() : (SkeletonCompareData->bMatchJoint ? FSlateColor(FLinearColor(0.9f, 0.7f, 0.5f)) : FSlateColor(FLinearColor(0.7f, 0.3f, 0.0f)));

		this->ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 6.0f, 2.0f)
				[
					SNew(SImage)
					.Image(JointIcon)
					.Visibility(JointIcon != FAppStyle::GetDefaultBrush() ? EVisibility::Visible : EVisibility::Collapsed)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f, 3.0f, 6.0f, 3.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(SkeletonCompareData->JointName.ToString()))
					.ToolTipText(FText::FromString(Tooltip))
					.ColorAndOpacity(ForegroundTextColor)
				]
			];

		STableRow< TSharedPtr<FSkeletonCompareData> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true),
			InOwnerTableView
			);
	}

private:
	/** The node info to build the tree view row from. */
	TSharedPtr<FSkeletonCompareData> SkeletonCompareData;
	FCompMesh *SourceData;
	FCompMesh *ResultData;
};


TSharedRef<ITableRow> SFbxSkeltonConflictWindow::OnGenerateRowCompareTreeView(TSharedPtr<FSkeletonCompareData> RowData, const TSharedRef<STableViewBase>& Table)
{
	TSharedRef< SCompareSkeletonTreeViewItem > ReturnRow = SNew(SCompareSkeletonTreeViewItem, Table)
		.SkeletonCompareData(RowData)
		.SourceData(SourceData)
		.ResultData(ResultData);
	return ReturnRow;
}

void SFbxSkeltonConflictWindow::OnGetChildrenRowCompareTreeView(TSharedPtr<FSkeletonCompareData> InParent, TArray< TSharedPtr<FSkeletonCompareData> >& OutChildren)
{
	for (int32 ChildIndex = 0; ChildIndex < InParent->ChildJoints.Num(); ++ChildIndex)
	{
		TSharedPtr<FSkeletonCompareData> ChildJoint = InParent->ChildJoints[ChildIndex];
		if (ChildJoint.IsValid())
		{
			OutChildren.Add(ChildJoint);
		}
	}
}

TSharedRef<ITableRow> SFbxSkeltonConflictWindow::OnGenerateRowAssetReferencingSkeleton(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	int32 AssetListIndex = AssetReferencingSkeleton.Find(InItem);
	bool LightBackgroundColor = AssetListIndex % 2 == 0;
	return SNew(STableRow<TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SBorder)
			.BorderImage(LightBackgroundColor ? FAppStyle::GetBrush("ToolPanel.GroupBorder") : FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			[
				SNew(STextBlock)
				.Text(FText::FromString(*(InItem.Get())))
			]
		];
}

void SFbxSkeltonConflictWindow::FilSkeletonTreeItem()
{
	//Create all the entries
	for (int32 RowIndex = 0; RowIndex < SourceData->CompSkeleton.Joints.Num(); ++RowIndex)
	{
		TSharedPtr<FSkeletonCompareData> CompareRowData = MakeShareable(new FSkeletonCompareData());
		int32 AddedIndex = CurrentSkeletonTreeItem.Add(CompareRowData);
		check(AddedIndex == RowIndex);
		CompareRowData->CurrentJointIndex = RowIndex;
		CompareRowData->JointName = SourceData->CompSkeleton.Joints[RowIndex].Name;
		CompareRowData->ChildJointIndexes = SourceData->CompSkeleton.Joints[RowIndex].ChildIndexes;
	}

	//Set the childrens and parent pointer
	for (int32 RowIndex = 0; RowIndex < SourceData->CompSkeleton.Joints.Num(); ++RowIndex)
	{
		check(CurrentSkeletonTreeItem.IsValidIndex(RowIndex));
		TSharedPtr<FSkeletonCompareData> CompareRowData = CurrentSkeletonTreeItem[RowIndex];
		if (CurrentSkeletonTreeItem.IsValidIndex(SourceData->CompSkeleton.Joints[RowIndex].ParentIndex))
		{
			CompareRowData->ParentJoint = CurrentSkeletonTreeItem[SourceData->CompSkeleton.Joints[RowIndex].ParentIndex];
		}
		
		for (int32 ChildJointIndex = 0; ChildJointIndex < CompareRowData->ChildJointIndexes.Num(); ++ChildJointIndex)
		{
			if (CurrentSkeletonTreeItem.IsValidIndex(CompareRowData->ChildJointIndexes[ChildJointIndex]))
			{
				CompareRowData->ChildJoints.Add(CurrentSkeletonTreeItem[CompareRowData->ChildJointIndexes[ChildJointIndex]]);
			}
		}
	}

	for (int32 RowIndex = 0; RowIndex < ResultData->CompSkeleton.Joints.Num(); ++RowIndex)
	{
		TSharedPtr<FSkeletonCompareData> CompareRowData = MakeShareable(new FSkeletonCompareData());
		int32 AddedIndex = FbxSkeletonTreeItem.Add(CompareRowData);
		check(AddedIndex == RowIndex);
		CompareRowData->FbxJointIndex = RowIndex;
		CompareRowData->JointName = ResultData->CompSkeleton.Joints[RowIndex].Name;
		CompareRowData->ChildJointIndexes = ResultData->CompSkeleton.Joints[RowIndex].ChildIndexes;
	}

	//Set the childrens and parent pointer
	for (int32 RowIndex = 0; RowIndex < ResultData->CompSkeleton.Joints.Num(); ++RowIndex)
	{
		check(FbxSkeletonTreeItem.IsValidIndex(RowIndex));
		TSharedPtr<FSkeletonCompareData> CompareRowData = FbxSkeletonTreeItem[RowIndex];
		if (FbxSkeletonTreeItem.IsValidIndex(ResultData->CompSkeleton.Joints[RowIndex].ParentIndex))
		{
			CompareRowData->ParentJoint = FbxSkeletonTreeItem[ResultData->CompSkeleton.Joints[RowIndex].ParentIndex];
		}

		for (int32 ChildJointIndex = 0; ChildJointIndex < CompareRowData->ChildJointIndexes.Num(); ++ChildJointIndex)
		{
			if (FbxSkeletonTreeItem.IsValidIndex(CompareRowData->ChildJointIndexes[ChildJointIndex]))
			{
				CompareRowData->ChildJoints.Add(FbxSkeletonTreeItem[CompareRowData->ChildJointIndexes[ChildJointIndex]]);
			}
		}
	}
}

void SFbxSkeltonConflictWindow::RecursiveMatchJointInfo(TSharedPtr<FSkeletonCompareData> SkeletonItem)
{
	TArray<TSharedPtr<FSkeletonCompareData>> DisplayChilds;
	//Find the display child
	if (CurrentSkeletonTreeItem.IsValidIndex(SkeletonItem->CurrentJointIndex))
	{
		for (int32 ChildIndex = 0; ChildIndex < CurrentSkeletonTreeItem[SkeletonItem->CurrentJointIndex]->ChildJoints.Num(); ++ChildIndex)
		{
			DisplayChilds.Add(CurrentSkeletonTreeItem[SkeletonItem->CurrentJointIndex]->ChildJoints[ChildIndex]);
		}
	}
	if (FbxSkeletonTreeItem.IsValidIndex(SkeletonItem->FbxJointIndex))
	{
		for (int32 ChildIndex = 0; ChildIndex < FbxSkeletonTreeItem[SkeletonItem->FbxJointIndex]->ChildJoints.Num(); ++ChildIndex)
		{
			TSharedPtr<FSkeletonCompareData> FbxSkeletonItem = FbxSkeletonTreeItem[SkeletonItem->FbxJointIndex]->ChildJoints[ChildIndex];
			bool bFoundChildMatch = false;
			for (TSharedPtr<FSkeletonCompareData> DisplayChildJoint : DisplayChilds)
			{
				if (DisplayChildJoint->JointName == FbxSkeletonItem->JointName)
				{
					bFoundChildMatch = true;
					DisplayChildJoint->bMatchJoint = true;
					DisplayChildJoint->FbxJointIndex = FbxSkeletonItem->FbxJointIndex;
					break;
				}
			}
			if (!bFoundChildMatch)
			{
				DisplayChilds.Add(FbxSkeletonItem);
			}
		}
	}

	if (!SkeletonItem->bMatchJoint)
	{
		TSharedPtr<FSkeletonCompareData> ParentSkeletonItem = SkeletonItem->ParentJoint;
		while (ParentSkeletonItem.IsValid() && ParentSkeletonItem->bChildConflict == false)
		{
			ParentSkeletonItem->bChildConflict = true;
			ParentSkeletonItem = ParentSkeletonItem->ParentJoint;
		}
	}
	//Set the new child list to the display joint
	SkeletonItem->ChildJoints = DisplayChilds;
	SkeletonItem->ChildJointIndexes.Empty();
	for (TSharedPtr<FSkeletonCompareData> ChildJoint : SkeletonItem->ChildJoints)
	{
		ChildJoint->ParentJoint = SkeletonItem;
		RecursiveMatchJointInfo(ChildJoint);
	}
}

void SFbxSkeltonConflictWindow::SetMatchJointInfo()
{
	TArray<TSharedPtr<FSkeletonCompareData>> RootJoint;
	for (TSharedPtr<FSkeletonCompareData> CurrentSkeletonItem : CurrentSkeletonTreeItem)
	{
		if (!CurrentSkeletonItem->ParentJoint.IsValid())
		{
			DisplaySkeletonTreeItem.Add(CurrentSkeletonItem);
		}
	}
	for (TSharedPtr<FSkeletonCompareData> CurrentSkeletonItem : FbxSkeletonTreeItem)
	{
		if (!CurrentSkeletonItem->ParentJoint.IsValid())
		{
			bool bInsertJoint = true;
			for (TSharedPtr<FSkeletonCompareData> DisplayTreeItem : DisplaySkeletonTreeItem)
			{
				if(DisplayTreeItem->JointName == CurrentSkeletonItem->JointName)
				{
					DisplayTreeItem->FbxJointIndex = CurrentSkeletonItem->FbxJointIndex;
					DisplayTreeItem->bMatchJoint = true;
					bInsertJoint = false;
				}
			}
			if (bInsertJoint)
			{
				DisplaySkeletonTreeItem.Add(CurrentSkeletonItem);
			}
		}
	}

	for (int32 SkeletonTreeIndex = 0; SkeletonTreeIndex < DisplaySkeletonTreeItem.Num(); ++SkeletonTreeIndex)
	{
		RecursiveMatchJointInfo(DisplaySkeletonTreeItem[SkeletonTreeIndex]);
	}
}
#undef LOCTEXT_NAMESPACE
