// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "FbxImporter.h"
#include "Curves/RichCurve.h"

namespace UnFbx {

	FbxNode* GetNodeFromName(const FString& NodeName, FbxNode* NodeToQuery)
	{
		if (!FCString::Strcmp(*NodeName, UTF8_TO_TCHAR(NodeToQuery->GetName())))
		{
			return NodeToQuery;
		}

		int32 NodeCount = NodeToQuery->GetChildCount();
		for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
		{
			FbxNode* ReturnNode = GetNodeFromName(NodeName, NodeToQuery->GetChild(NodeIndex));
			if (ReturnNode)
			{
				return ReturnNode;
			}
		}

		return nullptr;
	}

	FbxNode* GetNodeFromUniqueID(uint64 UniqueID, FbxNode* NodeToQuery)
	{
		if (UniqueID == NodeToQuery->GetUniqueID())
		{
			return NodeToQuery;
		}

		int32 NodeCount = NodeToQuery->GetChildCount();
		for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
		{
			FbxNode* ReturnNode = GetNodeFromUniqueID(UniqueID, NodeToQuery->GetChild(NodeIndex));
			if (ReturnNode)
			{
				return ReturnNode;
			}
		}

		return nullptr;
	}

	void FFbxCurvesAPI::GetAllNodeNameArray(TArray<FString> &AllNodeNames) const
	{
		AllNodeNames.Empty(TransformData.Num());
		for (auto Transform : TransformData)
		{
			FbxNode* Node = GetNodeFromUniqueID(Transform.Key, Scene->GetRootNode());
			if (Node)
			{
				AllNodeNames.Add(Node->GetName());
			}
		}
	}
	void FFbxCurvesAPI::GetAnimatedNodeNameArray(TArray<FString> &AnimatedNodeNames) const
	{
		AnimatedNodeNames.Empty(CurvesData.Num());
		for (auto AnimNodeKvp : CurvesData)
		{
			AnimatedNodeNames.Add(AnimNodeKvp.Value.Name);
		}
	}

	void FFbxCurvesAPI::GetNodeAnimatedPropertyNameArray(const FString &NodeName, TArray<FString> &AnimatedPropertyNames) const
	{
		AnimatedPropertyNames.Empty();
		for (auto AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName) == 0)
			{
				for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
				{
					AnimatedPropertyNames.Add(NodePropertyKvp.Key);
				}
				for (auto AttributePropertyKvp : AnimNodeHandle.AttributeProperties)
				{
					AnimatedPropertyNames.Add(AttributePropertyKvp.Key);
				}
				return;
			}
		}
	}

	void FFbxCurvesAPI::GetCustomStringPropertyArray(const FString& NodeName, TArray<TPair<FString, FString> >& CustomPropertyPairs) const
	{
		CustomPropertyPairs.Empty();

		FbxNode* Node = GetNodeFromName(NodeName, Scene->GetRootNode());
		if (Node)
		{
			// Import all custom user-defined FBX properties from the FBX node to the object metadata
			FbxProperty CurrentProperty = Node->GetFirstProperty();
			while (CurrentProperty.IsValid())
			{
				if (CurrentProperty.GetFlag(FbxPropertyFlags::eUserDefined) && CurrentProperty.GetPropertyDataType().GetType() == eFbxString)
				{
					FString PropertyValue = UTF8_TO_TCHAR(CurrentProperty.Get<FbxString>().Buffer());

					CustomPropertyPairs.Add(TPair<FString, FString>(UTF8_TO_TCHAR(CurrentProperty.GetName()), PropertyValue));
				}
				CurrentProperty = Node->GetNextProperty(CurrentProperty);
			}
		}
	}

	void FFbxCurvesAPI::GetAllNodePropertyCurveHandles(const FString& NodeName, const FString& PropertyName, TArray<FFbxAnimCurveHandle> &PropertyCurveHandles) const
	{
		PropertyCurveHandles.Empty();
		for (auto AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName) == 0)
			{
				for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
				{
					const FFbxAnimPropertyHandle& AnimPropertyHandle = NodePropertyKvp.Value;
					if (AnimPropertyHandle.Name.Compare(PropertyName) == 0)
					{
						PropertyCurveHandles = AnimPropertyHandle.CurveHandles;
						return;
					}
				}
				for (auto AttributePropertyKvp : AnimNodeHandle.AttributeProperties)
				{
					const FFbxAnimPropertyHandle& AnimPropertyHandle = AttributePropertyKvp.Value;
					if (AnimPropertyHandle.Name.Compare(PropertyName) == 0)
					{
						PropertyCurveHandles = AnimPropertyHandle.CurveHandles;
						return;
					}
				}
				return;
			}
		}
	}

	void FFbxCurvesAPI::GetCurveHandle(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FFbxAnimCurveHandle &CurveHandle) const
	{
		for (auto AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName) == 0)
			{
				for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
				{
					const FFbxAnimPropertyHandle& AnimPropertyHandle = NodePropertyKvp.Value;
					if (AnimPropertyHandle.Name.Compare(PropertyName) == 0)
					{
						for (const FFbxAnimCurveHandle &CurrentCurveHandle : AnimPropertyHandle.CurveHandles)
						{
							if (CurrentCurveHandle.ChannelIndex == ChannelIndex && CurrentCurveHandle.CompositeIndex == CompositeIndex)
							{
								CurveHandle = CurrentCurveHandle;
								return;
							}
						}
						return;
					}
				}
				for (auto AttributePropertyKvp : AnimNodeHandle.AttributeProperties)
				{
					const FFbxAnimPropertyHandle& AnimPropertyHandle = AttributePropertyKvp.Value;
					if (AnimPropertyHandle.Name.Compare(PropertyName) == 0)
					{
						for (const FFbxAnimCurveHandle &CurrentCurveHandle : AnimPropertyHandle.CurveHandles)
						{
							if (CurrentCurveHandle.ChannelIndex == ChannelIndex && CurrentCurveHandle.CompositeIndex == CompositeIndex)
							{
								CurveHandle = CurrentCurveHandle;
								return;
							}
						}
						return;
					}
				}
				return;
			}
		}
	}

	//deprecated
	void FFbxCurvesAPI::GetCurveData(const FFbxAnimCurveHandle &CurveHandle, FInterpCurveFloat& CurveData, bool bNegative) const
	{
		if (CurveHandle.AnimCurve == nullptr)
			return;
		int32 KeyCount = CurveHandle.AnimCurve->KeyGetCount();
		CurveData.Reset();
		for (int32 KeyIndex = 0; KeyIndex < KeyCount; ++KeyIndex)
		{
			FbxAnimCurveKey CurKey = CurveHandle.AnimCurve->KeyGet(KeyIndex);
			// Create the curve keys
			FInterpCurvePoint<float> UnrealKey;
			UnrealKey.InVal = CurKey.GetTime().GetSecondDouble();

			UnrealKey.InterpMode = GetUnrealInterpMode(CurKey);

			float OutVal = bNegative ? -CurKey.GetValue() : CurKey.GetValue();
			float ArriveTangent = 0.0f;
			float LeaveTangent = 0.0f;

			// Convert the Bezier control points, if available, into Hermite tangents
			if (CurKey.GetInterpolation() == FbxAnimCurveDef::eInterpolationCubic)
			{
				float LeftTangent = CurveHandle.AnimCurve->KeyGetLeftDerivative(KeyIndex);
				float RightTangent = CurveHandle.AnimCurve->KeyGetRightDerivative(KeyIndex);

				if (KeyIndex > 0)
				{
					ArriveTangent = LeftTangent * (CurKey.GetTime().GetSecondDouble() - CurveHandle.AnimCurve->KeyGetTime(KeyIndex - 1).GetSecondDouble());
				}

				if (KeyIndex < KeyCount - 1)
				{
					LeaveTangent = RightTangent * (CurveHandle.AnimCurve->KeyGetTime(KeyIndex + 1).GetSecondDouble() - CurKey.GetTime().GetSecondDouble());
				}
			}

			UnrealKey.OutVal = OutVal;
			UnrealKey.ArriveTangent = ArriveTangent;
			UnrealKey.LeaveTangent = LeaveTangent;
			// Add this new key to the curve
			CurveData.Points.Add(UnrealKey);
		}
	}

	void FFbxCurvesAPI::GetCurveDataForSequencer(const FFbxAnimCurveHandle &CurveHandle, FRichCurve& RichCurve, bool bNegative, float Scale) const
	{
		const float DefaultCurveWeight = FbxAnimCurveDef::sDEFAULT_WEIGHT;
		FbxAnimCurve* FbxCurve = CurveHandle.AnimCurve;
		if (FbxCurve)
		{
			RichCurve.Reset();
			//Send a neutral timespan 0 to infinite, the ImportCurve, offset the keytime by doing: (KeyTime - TimeSpan.start), setting TimeSpan.start at zero will not affect the keys time value.
			FbxTimeSpan AnimTimeSpan(FBXSDK_TIME_ZERO, FBXSDK_TIME_INFINITE);
			//Sequencer use auto set tangents
			const bool bAutoSetTangents = true;
			UnFbx::FFbxImporter::ImportCurve(FbxCurve, RichCurve, AnimTimeSpan, bNegative, Scale, bAutoSetTangents);
		}
	}

	void FFbxCurvesAPI::GetCurveData(const FFbxAnimCurveHandle &CurveHandle, FRichCurve& RichCurve, bool bNegative,float Scale) const
	{
		FbxAnimCurve* FbxCurve = CurveHandle.AnimCurve;
		if (FbxCurve)
		{
			RichCurve.Reset();
			//Send a neutral timespan 0 to infinite, the ImportCurve, offset the keytime by doing: (KeyTime - TimeSpan.start), setting TimeSpan.start at zero will not affect the keys time value.
			FbxTimeSpan AnimTimeSpan(FBXSDK_TIME_ZERO, FBXSDK_TIME_INFINITE);
			const bool bAutoSetTangents = false;
			UnFbx::FFbxImporter::ImportCurve(FbxCurve, RichCurve, AnimTimeSpan, bNegative, Scale, bAutoSetTangents);
		}
	}


	void FFbxCurvesAPI::GetBakeCurveData(const FFbxAnimCurveHandle &CurveHandle, TArray<float>& CurveData, float PeriodTime, float StartTime /*= 0.0f*/, float StopTime /*= -1.0f*/, bool bNegative /*= false*/, float Scale /*=1.0f*/) const
	{
		//Make sure the parameter are ok
		if (CurveHandle.AnimCurve == nullptr || CurveHandle.AnimationTimeSecond > StartTime || PeriodTime <= 0.0001f || (StopTime > 0.0f && StopTime < StartTime) )
			return;

		CurveData.Empty();
		double CurrentTime = (double)StartTime;
		int LastEvaluateKey = 0;
		//Set the stop time
		if (StopTime <= 0.0f || StopTime > CurveHandle.AnimationTimeSecond)
		{
			StopTime = CurveHandle.AnimationTimeSecond;
		}
		while (CurrentTime < (double)StopTime)
		{
			FbxTime FbxStepTime;
			FbxStepTime.SetSecondDouble(CurrentTime);
			float CurveValue = CurveHandle.AnimCurve->Evaluate(FbxStepTime, &LastEvaluateKey) * Scale;
			if (bNegative)
				CurveValue = -CurveValue;
			CurveData.Add(CurveValue);
			CurrentTime += (double)PeriodTime;
		}
	}

	//deprecrated
	void FFbxCurvesAPI::GetCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FInterpCurveFloat& CurveData, bool bNegative) const
	{
		FFbxAnimCurveHandle CurveHandle;
		GetCurveHandle(NodeName, PropertyName, ChannelIndex, CompositeIndex, CurveHandle);
		if (CurveHandle.AnimCurve != nullptr)
		{
#pragma warning(disable : 4996) // 'function' was declared deprecated
			GetCurveData(CurveHandle, CurveData, bNegative);
#pragma warning(default : 4996) // 'function' was declared deprecated
		}
		else
		{
			CurveData.Reset();
		}
	}

	void FFbxCurvesAPI::GetCurveDataForSequencer(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FRichCurve& RichCurve, bool bNegative, float Scale) const
	{
		FFbxAnimCurveHandle CurveHandle;
		GetCurveHandle(NodeName, PropertyName, ChannelIndex, CompositeIndex, CurveHandle);
		if (CurveHandle.AnimCurve != nullptr)
		{
			GetCurveDataForSequencer(CurveHandle, RichCurve, bNegative, Scale);
		}
		else
		{
			RichCurve.Reset();
		}
	}

	void FFbxCurvesAPI::GetCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, FRichCurve& RichCurve, bool bNegative) const
	{
		FFbxAnimCurveHandle CurveHandle;
		GetCurveHandle(NodeName, PropertyName, ChannelIndex, CompositeIndex, CurveHandle);
		if (CurveHandle.AnimCurve != nullptr)
		{
			GetCurveData(CurveHandle, RichCurve, bNegative);
		}
		else
		{
			RichCurve.Reset();
		}
	}

	void FFbxCurvesAPI::GetBakeCurveData(const FString& NodeName, const FString& PropertyName, int32 ChannelIndex, int32 CompositeIndex, TArray<float>& CurveData, float PeriodTime, float StartTime /*= 0.0f*/, float StopTime /*= -1.0f*/, bool bNegative /*= false*/, float Scale /*=1.0f*/) const
	{
		FFbxAnimCurveHandle CurveHandle;
		GetCurveHandle(NodeName, PropertyName, ChannelIndex, CompositeIndex, CurveHandle);
		if (CurveHandle.AnimCurve != nullptr)
		{
			GetBakeCurveData(CurveHandle, CurveData, PeriodTime, StartTime, StopTime, bNegative, Scale);
		}
		else
		{
			CurveData.Empty();
		}
	}

	//Helper
	EInterpCurveMode FFbxCurvesAPI::GetUnrealInterpMode(FbxAnimCurveKey FbxKey) const
	{
		EInterpCurveMode Mode = CIM_CurveUser;
		// Convert the interpolation type from FBX to Unreal.
		switch (FbxKey.GetInterpolation())
		{
		case FbxAnimCurveDef::eInterpolationCubic:
		{
			FbxAnimCurveDef::ETangentMode TangentMode = FbxKey.GetTangentMode(true);
			if (TangentMode & (FbxAnimCurveDef::eTangentUser | FbxAnimCurveDef::eTangentTCB | FbxAnimCurveDef::eTangentGenericClamp | FbxAnimCurveDef::eTangentGenericClampProgressive))
			{
				Mode = CIM_CurveUser;
			}
			else if (TangentMode & FbxAnimCurveDef::eTangentGenericBreak)
			{
				Mode = CIM_CurveBreak;
			}
			else if (TangentMode & FbxAnimCurveDef::eTangentAuto)
			{
				Mode = CIM_CurveAuto;
			}
			break;
		}

		case FbxAnimCurveDef::eInterpolationConstant:
			if (FbxKey.GetTangentMode() != (FbxAnimCurveDef::ETangentMode)FbxAnimCurveDef::eConstantStandard)
			{
				// warning not support
				;
			}
			Mode = CIM_Constant;
			break;

		case FbxAnimCurveDef::eInterpolationLinear:
			Mode = CIM_Linear;
			break;
		}
		return Mode;
	}

	void ConvertRotationToUnreal(float& Roll, float& Pitch, float& Yaw, bool bIsCamera, bool bIsLight)
	{
		FRotator AnimRotator(Pitch, Yaw, Roll);

		FTransform AnimRotatorTransform;
		FTransform UnrealRootRotatorTransform;

		AnimRotatorTransform.SetRotation(AnimRotator.Quaternion());

		FRotator UnrealRootRotator;
		if (bIsCamera)
		{
			UnrealRootRotator = FFbxDataConverter::GetCameraRotation();
		}
		else if (bIsLight)
		{
			UnrealRootRotator = FFbxDataConverter::GetLightRotation();
		}
		else
		{
			UnrealRootRotator = FRotator(0.f);
		}

		UnrealRootRotatorTransform.SetRotation(UnrealRootRotator.Quaternion());

		FTransform ResultTransform = UnrealRootRotatorTransform * AnimRotatorTransform;
		FRotator ResultRotator = ResultTransform.Rotator();	

		Roll = ResultRotator.Roll;
		Pitch = ResultRotator.Pitch;
		Yaw = ResultRotator.Yaw;
	}


	void FFbxCurvesAPI::GetConvertedTransformCurveData(const FString& NodeName, FInterpCurveFloat& TranslationX, FInterpCurveFloat& TranslationY, FInterpCurveFloat& TranslationZ,
													   FInterpCurveFloat& EulerRotationX, FInterpCurveFloat& EulerRotationY, FInterpCurveFloat& EulerRotationZ, 
													   FInterpCurveFloat& ScaleX, FInterpCurveFloat& ScaleY, FInterpCurveFloat& ScaleZ, 
													   FTransform& DefaultTransform) const 
	{
		for (auto AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName) == 0)
			{
				bool bIsCamera = AnimNodeHandle.AttributeType == FbxNodeAttribute::eCamera;
				bool bIsLight = AnimNodeHandle.AttributeType == FbxNodeAttribute::eLight;
				FFbxAnimCurveHandle TransformCurves[9];
				for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
				{
					FFbxAnimPropertyHandle& AnimPropertyHandle = NodePropertyKvp.Value;
					for (FFbxAnimCurveHandle& CurveHandle : AnimPropertyHandle.CurveHandles)
					{
						if (CurveHandle.CurveType != FFbxAnimCurveHandle::NotTransform)
						{
							TransformCurves[(int32)(CurveHandle.CurveType)] = CurveHandle;
						}
					}
				}

#pragma warning(disable : 4996) // 'function' was declared deprecated

				GetCurveData(TransformCurves[0], TranslationX, false);
				GetCurveData(TransformCurves[1], TranslationY, true);
				GetCurveData(TransformCurves[2], TranslationZ, false);

				GetCurveData(TransformCurves[3], EulerRotationX, false);
				GetCurveData(TransformCurves[4], EulerRotationY, true);
				GetCurveData(TransformCurves[5], EulerRotationZ, true);

				GetCurveData(TransformCurves[6], ScaleX, false);
				GetCurveData(TransformCurves[7], ScaleY, false);
				GetCurveData(TransformCurves[8], ScaleZ, false);
#pragma warning(default : 4996) // 'function' was declared deprecated
				if (bIsCamera || bIsLight)
				{
					int32 CurvePointNum = FMath::Min3<int32>(EulerRotationX.Points.Num(), EulerRotationY.Points.Num(), EulerRotationZ.Points.Num());

					// Once the individual Euler channels are imported, then convert the rotation into Unreal coords
					for (int32 PointIndex = 0; PointIndex < CurvePointNum; ++PointIndex)
					{
						FInterpCurvePoint<float>& CurveKeyX = EulerRotationX.Points[PointIndex];
						FInterpCurvePoint<float>& CurveKeyY = EulerRotationY.Points[PointIndex];
						FInterpCurvePoint<float>& CurveKeyZ = EulerRotationZ.Points[PointIndex];

						float Pitch = CurveKeyY.OutVal;
						float Yaw = CurveKeyZ.OutVal;
						float Roll = CurveKeyX.OutVal;
						ConvertRotationToUnreal(Roll, Pitch, Yaw, bIsCamera, bIsLight);
						CurveKeyX.OutVal = Roll;
						CurveKeyY.OutVal = Pitch;
						CurveKeyZ.OutVal = Yaw;
					}

					if (bIsCamera)
					{
						// The FInterpCurve code doesn't differentiate between angles and other data, so an interpolation from 179 to -179
						// will cause the camera to rotate all the way around through 0 degrees.  So here we make a second pass over the 
						// Euler track to convert the angles into a more interpolation-friendly format.  
						float CurrentAngleOffset[3] = { 0.f, 0.f, 0.f };
						for (int32 PointIndex = 1; PointIndex < CurvePointNum; ++PointIndex)
						{
							FInterpCurvePoint<float>& PrevCurveKeyX = EulerRotationX.Points[PointIndex - 1];
							FInterpCurvePoint<float>& PrevCurveKeyY = EulerRotationY.Points[PointIndex - 1];
							FInterpCurvePoint<float>& PrevCurveKeyZ = EulerRotationZ.Points[PointIndex - 1];

							FInterpCurvePoint<float>& CurveKeyX = EulerRotationX.Points[PointIndex];
							FInterpCurvePoint<float>& CurveKeyY = EulerRotationY.Points[PointIndex];
							FInterpCurvePoint<float>& CurveKeyZ = EulerRotationZ.Points[PointIndex];


							FVector PreviousOutVal = FVector(PrevCurveKeyX.OutVal, PrevCurveKeyY.OutVal, PrevCurveKeyZ.OutVal);
							FVector CurrentOutVal = FVector(CurveKeyX.OutVal, CurveKeyY.OutVal, CurveKeyZ.OutVal);

							for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
							{
								float DeltaAngle = (CurrentOutVal[AxisIndex] + CurrentAngleOffset[AxisIndex]) - PreviousOutVal[AxisIndex];

								if (DeltaAngle >= 180)
								{
									CurrentAngleOffset[AxisIndex] -= 360;
								}
								else if (DeltaAngle <= -180)
								{
									CurrentAngleOffset[AxisIndex] += 360;
								}

								CurrentOutVal[AxisIndex] += CurrentAngleOffset[AxisIndex];
							}
							CurveKeyX.OutVal = CurrentOutVal[0];
							CurveKeyY.OutVal = CurrentOutVal[1];
							CurveKeyZ.OutVal = CurrentOutVal[2];
						}
					}
				}
			}
		}

		FbxNode* Node = GetNodeFromName(NodeName, Scene->GetRootNode());
		if (Node)
		{	
			DefaultTransform = TransformData[Node->GetUniqueID()];
		}
	}

	void FFbxCurvesAPI::GetConvertedNonTransformCurveData(const FString& NodeName, bool bUseSequencerCurve, float UniformScale, TMap<FName, FRichCurve>& OutCurves)
	{
		for (TPair<uint64, FFbxAnimNodeHandle> AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName))
			{
				continue;
			}
			for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
			{
				FFbxAnimPropertyHandle& AnimPropertyHandle = NodePropertyKvp.Value;
				for (FFbxAnimCurveHandle& CurveHandle : AnimPropertyHandle.CurveHandles)
				{
					if (CurveHandle.CurveType != FFbxAnimCurveHandle::NotTransform)
					{
						continue;
					}
					FRichCurve RichCurve;

					if (bUseSequencerCurve)
					{
						GetCurveDataForSequencer(CurveHandle, RichCurve, false, UniformScale);
					}
					else
					{
						GetCurveData(CurveHandle, RichCurve, false, UniformScale);
					}

					OutCurves.Add(*AnimPropertyHandle.Name, RichCurve);
				}
			}
		}
	}
	
	void FFbxCurvesAPI::GetConvertedTransformCurveData(const FString& NodeName, FRichCurve& TranslationX, FRichCurve& TranslationY, FRichCurve& TranslationZ,
		FRichCurve& EulerRotationX, FRichCurve& EulerRotationY, FRichCurve& EulerRotationZ,
		FRichCurve& ScaleX, FRichCurve& ScaleY, FRichCurve& ScaleZ,
		FTransform& DefaultTransform, bool bUseSequencerCurve, float UniformScale) const
	{

		for (TPair< uint64, FFbxAnimNodeHandle> AnimNodeKvp : CurvesData)
		{
			const FFbxAnimNodeHandle& AnimNodeHandle = AnimNodeKvp.Value;
			if (AnimNodeHandle.Name.Compare(NodeName) == 0)
			{
				bool bIsCamera = AnimNodeHandle.AttributeType == FbxNodeAttribute::eCamera;
				bool bIsLight = AnimNodeHandle.AttributeType == FbxNodeAttribute::eLight;
				FFbxAnimCurveHandle TransformCurves[9];
				for (auto NodePropertyKvp : AnimNodeHandle.NodeProperties)
				{
					FFbxAnimPropertyHandle& AnimPropertyHandle = NodePropertyKvp.Value;
					for (FFbxAnimCurveHandle& CurveHandle : AnimPropertyHandle.CurveHandles)
					{
						if (CurveHandle.CurveType != FFbxAnimCurveHandle::NotTransform)
						{
							TransformCurves[(int32)(CurveHandle.CurveType)] = CurveHandle;
						}
					}
				}
				const bool bNegate = true;
				if (bUseSequencerCurve)
				{
					GetCurveDataForSequencer(TransformCurves[0], TranslationX, !bNegate, UniformScale);
					GetCurveDataForSequencer(TransformCurves[1], TranslationY, bNegate, UniformScale);
					GetCurveDataForSequencer(TransformCurves[2], TranslationZ, !bNegate, UniformScale);

					GetCurveDataForSequencer(TransformCurves[3], EulerRotationX, !bNegate);
					GetCurveDataForSequencer(TransformCurves[4], EulerRotationY, bNegate);
					GetCurveDataForSequencer(TransformCurves[5], EulerRotationZ, bNegate);

					GetCurveDataForSequencer(TransformCurves[6], ScaleX, !bNegate, UniformScale);
					GetCurveDataForSequencer(TransformCurves[7], ScaleY, !bNegate, UniformScale);
					GetCurveDataForSequencer(TransformCurves[8], ScaleZ, !bNegate, UniformScale);
				}
				else
				{
					GetCurveData(TransformCurves[0], TranslationX, !bNegate, UniformScale);
					GetCurveData(TransformCurves[1], TranslationY, bNegate, UniformScale);
					GetCurveData(TransformCurves[2], TranslationZ, !bNegate, UniformScale);

					GetCurveData(TransformCurves[3], EulerRotationX, !bNegate);
					GetCurveData(TransformCurves[4], EulerRotationY, bNegate);
					GetCurveData(TransformCurves[5], EulerRotationZ, bNegate);

					GetCurveData(TransformCurves[6], ScaleX, !bNegate, UniformScale);
					GetCurveData(TransformCurves[7], ScaleY, !bNegate, UniformScale);
					GetCurveData(TransformCurves[8], ScaleZ, !bNegate, UniformScale);
				}

				if (bIsCamera || bIsLight)
				{
					{//extra scope since we can't reset Key Iterators
						//need to convert rotations to unreal space.  Uses previous FInterpCurvePoint implementation that goes through the minimal number
						//of curve keys and sets them together. Obviously if the keys are not at the same times exactly this won't work.
						auto EulerRotXIt = EulerRotationX.GetKeyHandleIterator();
						auto EulerRotYIt = EulerRotationY.GetKeyHandleIterator();
						auto EulerRotZIt = EulerRotationZ.GetKeyHandleIterator();


						while (EulerRotXIt && EulerRotYIt && EulerRotZIt)
						{
							float Pitch = EulerRotationY.GetKeyValue(*EulerRotYIt);
							float Yaw = EulerRotationZ.GetKeyValue(*EulerRotZIt);
							float Roll = EulerRotationX.GetKeyValue(*EulerRotXIt);
							ConvertRotationToUnreal(Roll, Pitch, Yaw, bIsCamera, bIsLight);
							EulerRotationX.SetKeyValue(*EulerRotXIt, Roll, false);
							EulerRotationY.SetKeyValue(*EulerRotYIt, Pitch, false);
							EulerRotationZ.SetKeyValue(*EulerRotZIt, Yaw, false);

							++EulerRotXIt;
							++EulerRotYIt;
							++EulerRotZIt;
						}
					}
				}
				if (bIsCamera)
				{
					// The RichCurve code doesn't differentiate between angles and other data, so an interpolation from 179 to -179
					// will cause the camera to rotate all the way around through 0 degrees.  So here we make a second pass over the 
					// Euler track to convert the angles into a more interpolation-friendly format.  
					float CurrentAngleOffset[3] = { 0.f, 0.f, 0.f };
				
					auto EulerRotXIt = EulerRotationX.GetKeyHandleIterator();
					auto EulerRotYIt = EulerRotationY.GetKeyHandleIterator();
					auto EulerRotZIt = EulerRotationZ.GetKeyHandleIterator();

					FVector PreviousOutVal;
					bool bFirst = true;
					while (EulerRotXIt && EulerRotYIt && EulerRotZIt)
					{
						float X = EulerRotationX.GetKeyValue(*EulerRotXIt);;
						float Y = EulerRotationY.GetKeyValue(*EulerRotYIt);
						float Z = EulerRotationZ.GetKeyValue(*EulerRotZIt);

						FVector CurrentOutVal(X, Y, Z);
						if (bFirst)
						{
							PreviousOutVal = CurrentOutVal;
							bFirst = false;
						}
						
						for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
						{
							float DeltaAngle = (CurrentOutVal[AxisIndex] + CurrentAngleOffset[AxisIndex]) - PreviousOutVal[AxisIndex];

							if (DeltaAngle >= 180)
							{
								CurrentAngleOffset[AxisIndex] -= 360;
							}
							else if (DeltaAngle <= -180)
							{
								CurrentAngleOffset[AxisIndex] += 360;
							}

							CurrentOutVal[AxisIndex] += CurrentAngleOffset[AxisIndex];
						}
						EulerRotationX.SetKeyValue(*EulerRotXIt, CurrentOutVal.X, false);
						EulerRotationY.SetKeyValue(*EulerRotYIt, CurrentOutVal.Y, false);
						EulerRotationZ.SetKeyValue(*EulerRotZIt, CurrentOutVal.Z, false);

						++EulerRotXIt;
						++EulerRotYIt;
						++EulerRotZIt;

						PreviousOutVal = CurrentOutVal;
					}
				}
			}
		}

		FbxNode* Node = GetNodeFromName(NodeName, Scene->GetRootNode());
		if (Node)
		{
			DefaultTransform = TransformData[Node->GetUniqueID()];
			DefaultTransform.SetLocation(DefaultTransform.GetLocation()* UniformScale);
			DefaultTransform.SetScale3D(DefaultTransform.GetScale3D()* UniformScale);
		}
		
	}
	
	//////////////////////////////////////////////////////////////////////////
	// FFbxImporter: Curve Extraction Implementation
	//

	void FFbxImporter::PopulateAnimatedCurveData(FFbxCurvesAPI &CurvesAPI)
	{
		if (Scene == nullptr)
			return;

		// merge animation layer at first
		FbxAnimStack* AnimStack = Scene->GetMember<FbxAnimStack>(0);
		if (!AnimStack)
			return;

		FbxAnimLayer* AnimLayer = AnimStack->GetMember<FbxAnimLayer>(0);
		if (AnimLayer == NULL)
			return;

		FbxNode *RootNode = Scene->GetRootNode();
		CurvesAPI.Scene = Scene;
		LoadNodeKeyframeAnimationRecursively(CurvesAPI, RootNode);
	}

	void FFbxImporter::LoadNodeKeyframeAnimationRecursively(FFbxCurvesAPI &CurvesAPI, FbxNode* NodeToQuery)
	{
		LoadNodeKeyframeAnimation(NodeToQuery, CurvesAPI);
		int32 NodeCount = NodeToQuery->GetChildCount();
		for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
		{
			FbxNode* ChildNode = NodeToQuery->GetChild(NodeIndex);
			LoadNodeKeyframeAnimationRecursively(CurvesAPI, ChildNode);
		}
	}

	void FFbxImporter::LoadNodeKeyframeAnimation(FbxNode* NodeToQuery, FFbxCurvesAPI &CurvesAPI)
	{
		SetupTransformForNode(NodeToQuery);
		int NumAnimations = Scene->GetSrcObjectCount<FbxAnimStack>();
		FFbxAnimNodeHandle AnimNodeHandle;
		AnimNodeHandle.Name = UTF8_TO_TCHAR(NodeToQuery->GetName());
		AnimNodeHandle.UniqueId = NodeToQuery->GetUniqueID();
		FbxNodeAttribute* NodeAttribute = NodeToQuery->GetNodeAttribute();
		if (NodeAttribute != nullptr)
		{
			AnimNodeHandle.AttributeType = NodeAttribute->GetAttributeType();
			AnimNodeHandle.AttributeUniqueId = NodeAttribute->GetUniqueID();
		}
		else
		{
			AnimNodeHandle.AttributeType = FbxNodeAttribute::eUnknown;
			AnimNodeHandle.AttributeUniqueId = 0xFFFFFFFFFFFFFFFF;
		}

		bool IsNodeAnimated = false;
		for (int AnimationIndex = 0; AnimationIndex < NumAnimations; AnimationIndex++)
		{
			FbxAnimStack *animStack = (FbxAnimStack*)Scene->GetSrcObject<FbxAnimStack>(AnimationIndex);
			FbxAnimEvaluator *animEvaluator = Scene->GetAnimationEvaluator();
			int numLayers = animStack->GetMemberCount();
			for (int layerIndex = 0; layerIndex < numLayers; layerIndex++)
			{
				FbxAnimLayer *AnimLayer = (FbxAnimLayer*)animStack->GetMember(layerIndex);
				// Display curves specific to properties
				FbxObject *ObjectToQuery = (FbxObject *)NodeToQuery;

				FbxAnimCurve *TransformCurves[9];
				TransformCurves[0] = NodeToQuery->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
				TransformCurves[1] = NodeToQuery->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
				TransformCurves[2] = NodeToQuery->LclTranslation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);
				
				TransformCurves[3] = NodeToQuery->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
				TransformCurves[4] = NodeToQuery->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
				TransformCurves[5] = NodeToQuery->LclRotation.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);

				TransformCurves[6] = NodeToQuery->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_X, false);
				TransformCurves[7] = NodeToQuery->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Y, false);
				TransformCurves[8] = NodeToQuery->LclScaling.GetCurve(AnimLayer, FBXSDK_CURVENODE_COMPONENT_Z, false);

				bool IsNodeProperty = true;
				FbxProperty CurrentProperty = ObjectToQuery->GetFirstProperty();
				while (CurrentProperty.IsValid())
				{
					FbxAnimCurve* AnimCurve = nullptr;
					TArray<TArray<int32>> KeyFrameNumber;
					TArray<TArray<float>> AnimationTimeSecond;
					TArray<TArray<FString>> CurveName;
					TArray<TArray<uint64>> CurveUniqueId;
					TArray<TArray<FbxAnimCurve*>> CurveData;
					bool PropertyHasCurve = false;

					FbxAnimCurveNode* CurveNode = CurrentProperty.GetCurveNode(AnimLayer);
					if (CurveNode != nullptr)
					{
						FbxDataType DataType = CurrentProperty.GetPropertyDataType();
						if (DataType.GetType() == eFbxBool || DataType.GetType() == eFbxDouble || DataType.GetType() == eFbxFloat || DataType.GetType() == eFbxInt || DataType.GetType() == eFbxEnum)
						{
							TArray<int32> CompositeKeyFrameNumber;
							TArray<float> CompositeAnimationTimeSecond;
							TArray<FString> CompositeCurveName;
							TArray<uint64> CompositeCurveUniqueId;
							TArray<FbxAnimCurve*> CompositeCurveData;
							for (int32 c = 0; c < CurveNode->GetCurveCount(0U); c++)
							{
								AnimCurve = CurveNode->GetCurve(0U, c);
								if (AnimCurve != nullptr)
								{
									int32 KeyNumber = AnimCurve->KeyGetCount();
									CompositeKeyFrameNumber.Add(KeyNumber);
									FbxTime frameTime = AnimCurve->KeyGetTime(KeyNumber - 1);
									CompositeAnimationTimeSecond.Add((float)frameTime.GetSecondDouble());
									PropertyHasCurve = true;
									CompositeCurveName.Add(UTF8_TO_TCHAR(AnimCurve->GetName()));
									CompositeCurveUniqueId.Add(AnimCurve->GetUniqueID());
									CompositeCurveData.Add(AnimCurve);
								}
							}
							KeyFrameNumber.Add(CompositeKeyFrameNumber);
							AnimationTimeSecond.Add(CompositeAnimationTimeSecond);
							CurveName.Add(CompositeCurveName);
							CurveUniqueId.Add(CompositeCurveUniqueId);
							CurveData.Add(CompositeCurveData);

						}
						else if (DataType.GetType() == eFbxDouble3 || DataType.GetType() == eFbxDouble4 || DataType.Is(FbxColor3DT) || DataType.Is(FbxColor4DT))
						{
							//Set the channel number to 3 or 4
							uint32 ChannelNumber = (DataType.GetType() == eFbxDouble3 || DataType.Is(FbxColor3DT)) ? 3 : 4;
							for (uint32 ChannelIndex = 0; ChannelIndex < ChannelNumber; ++ChannelIndex)
							{
								TArray<int32> CompositeKeyFrameNumber;
								TArray<float> CompositeAnimationTimeSecond;
								TArray<FString> CompositeCurveName;
								TArray<uint64> CompositeCurveUniqueId;
								TArray<FbxAnimCurve*> CompositeCurveData;
								TArray<EFbxType> CompositeCurveType;
								for (int32 c = 0; c < CurveNode->GetCurveCount(ChannelIndex); c++)
								{
									AnimCurve = CurveNode->GetCurve(ChannelIndex, c);
									if (AnimCurve)
									{
										int32 KeyNumber = AnimCurve->KeyGetCount();
										CompositeKeyFrameNumber.Add(KeyNumber);
										FbxTime frameTime = AnimCurve->KeyGetTime(KeyNumber - 1);
										CompositeAnimationTimeSecond.Add((float)frameTime.GetSecondDouble());
										PropertyHasCurve = true;
										CompositeCurveName.Add(UTF8_TO_TCHAR(AnimCurve->GetName()));
										CompositeCurveUniqueId.Add(AnimCurve->GetUniqueID());
										CompositeCurveData.Add(AnimCurve);
									}
								}
								KeyFrameNumber.Add(CompositeKeyFrameNumber);
								AnimationTimeSecond.Add(CompositeAnimationTimeSecond);
								CurveName.Add(CompositeCurveName);
								CurveUniqueId.Add(CompositeCurveUniqueId);
								CurveData.Add(CompositeCurveData);
							}
						}
						if (PropertyHasCurve == true)
						{
							IsNodeAnimated = true;
							FFbxAnimPropertyHandle PropertyHandle;
							PropertyHandle.DataType = DataType.GetType();
							PropertyHandle.Name = UTF8_TO_TCHAR(CurrentProperty.GetName());
							for (int32 ChannelIndex = 0; ChannelIndex < KeyFrameNumber.Num(); ++ChannelIndex)
							{
								for (int32 CompositeIndex = 0; CompositeIndex < KeyFrameNumber[ChannelIndex].Num(); ++CompositeIndex)
								{
									FFbxAnimCurveHandle CurveHandle;
									CurveHandle.Name = CurveName[ChannelIndex][CompositeIndex];
									CurveHandle.UniqueId = CurveUniqueId[ChannelIndex][CompositeIndex];
									CurveHandle.ChannelIndex = ChannelIndex;
									CurveHandle.CompositeIndex = CompositeIndex;
									CurveHandle.KeyNumber = KeyFrameNumber[ChannelIndex][CompositeIndex];
									CurveHandle.AnimationTimeSecond = AnimationTimeSecond[ChannelIndex][CompositeIndex];
									CurveHandle.AnimCurve = CurveData[ChannelIndex][CompositeIndex];
									for (int TransformIndex = 0; TransformIndex < 9; ++TransformIndex)
									{
										if (TransformCurves[TransformIndex] != nullptr && TransformCurves[TransformIndex]->GetUniqueID() == CurveHandle.AnimCurve->GetUniqueID())
										{
											CurveHandle.CurveType = (FFbxAnimCurveHandle::CurveTypeDescription)(TransformIndex);
											break;
										}
									}

									PropertyHandle.CurveHandles.Add(CurveHandle);
								}
							}
							if (IsNodeProperty == true)
							{
								AnimNodeHandle.NodeProperties.Add(PropertyHandle.Name, PropertyHandle);
							}
							else
							{
								AnimNodeHandle.AttributeProperties.Add(PropertyHandle.Name, PropertyHandle);
							}
						}
					}
					CurrentProperty = ObjectToQuery->GetNextProperty(CurrentProperty);
					if (!CurrentProperty.IsValid() && ObjectToQuery->GetUniqueID() == NodeToQuery->GetUniqueID())
					{
						if (NodeAttribute != nullptr)
						{
							switch (NodeAttribute->GetAttributeType())
							{
							case FbxNodeAttribute::eCamera:
							{
								FbxCamera* CameraAttribute = (FbxCamera*)NodeAttribute;
								CurrentProperty = CameraAttribute->GetFirstProperty();
							}
							break;
							case FbxNodeAttribute::eLight:
							{
								FbxLight* LightAttribute = (FbxLight*)NodeAttribute;
								CurrentProperty = LightAttribute->GetFirstProperty();
							}
							break;
							}
							ObjectToQuery = NodeAttribute;
							IsNodeProperty = false;
						}
					}
				} // while
			}
		}

		if (IsNodeAnimated)
		{
			CurvesAPI.CurvesData.Add(AnimNodeHandle.UniqueId, AnimNodeHandle);
		}

		// Store default transform values in TransformData
		bool bIsCamera = AnimNodeHandle.AttributeType == FbxNodeAttribute::eCamera;
		bool bIsLight = AnimNodeHandle.AttributeType == FbxNodeAttribute::eLight;
		FTransform Transform;
		FbxVector4 LclTranslation = NodeToQuery->LclTranslation.EvaluateValue(0.f);
		FbxVector4 LclRotation = NodeToQuery->LclRotation.EvaluateValue(0.f);
		FbxVector4 LclScaling = NodeToQuery->LclScaling.EvaluateValue(0.f);
		float EulerRotationX = LclRotation[0];
		float EulerRotationY = -LclRotation[1];
		float EulerRotationZ = -LclRotation[2];
		float Pitch = EulerRotationY;
		float Yaw = EulerRotationZ;
		float Roll = EulerRotationX;
		ConvertRotationToUnreal(Roll, Pitch, Yaw, bIsCamera, bIsLight);
		Transform.SetLocation(FVector(LclTranslation[0], -LclTranslation[1], LclTranslation[2]));
		Transform.SetRotation(FRotator(Pitch, Yaw, Roll).Quaternion());
		Transform.SetScale3D(FVector(LclScaling[0], LclScaling[1], LclScaling[2]));

		CurvesAPI.TransformData.Add(AnimNodeHandle.UniqueId, Transform);
	}

	//This function now clears out all pivots, post and pre rotations and set's the
	//RotationOrder to XYZ.
	//Updated per the latest documentation
	//https://help.autodesk.com/view/FBX/2017/ENU/?guid=__files_GUID_C35D98CB_5148_4B46_82D1_51077D8970EE_htm
	void FFbxImporter::SetupTransformForNode(FbxNode *Node)
	{

		// Activate pivot converting 
		Node->SetPivotState(FbxNode::eSourcePivot, FbxNode::ePivotActive);
		Node->SetPivotState(FbxNode::eDestinationPivot, FbxNode::ePivotActive);

		FbxVector4 Zero(0, 0, 0);

		// We want to set all these to 0 and bake them into the transforms. 
		Node->SetPostRotation(FbxNode::eDestinationPivot, Zero);
		Node->SetPreRotation(FbxNode::eDestinationPivot, Zero);
		Node->SetRotationOffset(FbxNode::eDestinationPivot, Zero);
		Node->SetScalingOffset(FbxNode::eDestinationPivot, Zero);
		Node->SetRotationPivot(FbxNode::eDestinationPivot, Zero);
		Node->SetScalingPivot(FbxNode::eDestinationPivot, Zero);

		Node->SetRotationOrder(FbxNode::eDestinationPivot, eEulerXYZ);
		//When  we support other orders do this.
		//EFbxRotationOrder ro;
		//Node->GetRotationOrder(FbxNode::eSourcePivot, ro);
		//Node->SetRotationOrder(FbxNode::eDestinationPivot, ro);

		//Most DCC's don't have this but 3ds Max does
		Node->SetGeometricTranslation(FbxNode::eDestinationPivot, Zero);
		Node->SetGeometricRotation(FbxNode::eDestinationPivot, Zero);
		Node->SetGeometricScaling(FbxNode::eDestinationPivot, Zero);
		//NOTE THAT ConvertPivotAnimationRecursive did not seem to work when getting the local transform values!!!
		Node->ResetPivotSetAndConvertAnimation(FbxTime::GetFrameRate(Scene->GetGlobalSettings().GetTimeMode()));
	}

} // namespace UnFBX

