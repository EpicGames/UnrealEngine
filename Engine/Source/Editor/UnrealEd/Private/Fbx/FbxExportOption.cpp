// Copyright Epic Games, Inc. All Rights Reserved.
#include "Exporters/FbxExportOption.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxExportOption)

UFbxExportOption::UFbxExportOption(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FbxExportCompatibility = EFbxExportCompatibility::FBX_2013;
	bASCII = false;
	bForceFrontXAxis = false;
	LevelOfDetail = true;
	Collision = true;
	bExportSourceMesh = false;
	bExportMorphTargets = true;
	VertexColor = true;
	MapSkeletalMotionToRoot = false;
	bExportLocalTime = true;
	BakeCameraAndLightAnimation = EMovieSceneBakeType::BakeTransforms;
	BakeActorAnimation = EMovieSceneBakeType::None;
}

void UFbxExportOption::ResetToDefault()
{
	ReloadConfig();
}

/** Load UI settings from ini file */
void UFbxExportOption::LoadOptions()
{
	int32 PortFlags = 0;

	for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}
		FString Section = TEXT("FBX_Export_UI_Option_") + GetClass()->GetName();
		FString Key = Property->GetName();

		const bool bIsPropertyInherited = Property->GetOwnerClass() != GetClass();
		UObject* SuperClassDefaultObject = GetClass()->GetSuperClass()->GetDefaultObject();

		const FString& PropFileName = GEditorPerProjectIni;

		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			const FConfigSection* Sec = GConfig->GetSection(*Section, 0, *GEditorPerProjectIni);
			if (Sec != nullptr)
			{
				TArray<FConfigValue> List;
				const FName KeyName(*Key, FNAME_Find);
				Sec->MultiFind(KeyName, List);

				FScriptArrayHelper_InContainer ArrayHelper(Array, this);
				// Only override default properties if there is something to override them with.
				if (List.Num() > 0)
				{
					ArrayHelper.EmptyAndAddValues(List.Num());
					for (int32 i = List.Num() - 1, c = 0; i >= 0; i--, c++)
					{
						Array->Inner->ImportText_Direct(*List[i].GetValue(), ArrayHelper.GetRawPtr(c), this, PortFlags);
					}
				}
				else
				{
					int32 Index = 0;
					const FConfigValue* ElementValue = nullptr;
					do
					{
						// Add array index number to end of key
						FString IndexedKey = FString::Printf(TEXT("%s[%i]"), *Key, Index);

						// Try to find value of key
						const FName IndexedName(*IndexedKey, FNAME_Find);
						if (IndexedName == NAME_None)
						{
							break;
						}
						ElementValue = Sec->Find(IndexedName);

						// If found, import the element
						if (ElementValue != nullptr)
						{
							// expand the array if necessary so that Index is a valid element
							ArrayHelper.ExpandForIndex(Index);
							Array->Inner->ImportText_Direct(*ElementValue->GetValue(), ArrayHelper.GetRawPtr(Index), this, PortFlags);
						}

						Index++;
					} while (ElementValue || Index < ArrayHelper.Num());
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Property->ArrayDim; i++)
			{
				if (Property->ArrayDim != 1)
				{
					Key = FString::Printf(TEXT("%s[%i]"), *Property->GetName(), i);
				}

				FString Value;
				bool bFoundValue = GConfig->GetString(*Section, *Key, Value, *GEditorPerProjectIni);

				if (bFoundValue)
				{
					if (Property->ImportText_Direct(*Value, Property->ContainerPtrToValuePtr<uint8>(this, i), this, PortFlags) == NULL)
					{
						// this should be an error as the properties from the .ini / .int file are not correctly being read in and probably are affecting things in subtle ways
					}
				}
			}
		}
	}
}

/** Save UI settings to ini file */
void UFbxExportOption::SaveOptions()
{
	int32 PortFlags = 0;

	for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}
		FString Section = TEXT("FBX_Export_UI_Option_") + GetClass()->GetName();
		FString Key = Property->GetName();

		const bool bIsPropertyInherited = Property->GetOwnerClass() != GetClass();
		UObject* SuperClassDefaultObject = GetClass()->GetSuperClass()->GetDefaultObject();

		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			GConfig->RemoveKeyFromSection(*Section, *Key, GEditorPerProjectIni);

			FScriptArrayHelper_InContainer ArrayHelper(Array, this);
			for (int32 i = 0; i < ArrayHelper.Num(); i++)
			{
				FString	Buffer;
				Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);
				GConfig->AddToSection(*Section, *Key, Buffer, GEditorPerProjectIni);
			}
		}
		else
		{
			TCHAR TempKey[MAX_SPRINTF] = TEXT("");
			for (int32 Index = 0; Index < Property->ArrayDim; Index++)
			{
				if (Property->ArrayDim != 1)
				{
					FCString::Sprintf(TempKey, TEXT("%s[%i]"), *Property->GetName(), Index);
					Key = TempKey;
				}

				FString	Value;
				Property->ExportText_InContainer(Index, Value, this, this, this, PortFlags);
				GConfig->SetString(*Section, *Key, *Value, *GEditorPerProjectIni);
			}
		}
	}
	GConfig->Flush(0);
}
