// Copyright Epic Games, Inc. All Rights Reserved.
#include "FbxExportOptionsWindow.h"

#include "DetailsViewArgs.h"
#include "Exporters/FbxExportOption.h"
#include "Fonts/SlateFontInfo.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FBXOption"

void SFbxExportOptionsWindow::Construct(const FArguments& InArgs)
{
	ExportOptions = InArgs._ExportOptions;
	WidgetWindow = InArgs._WidgetWindow;

	check (ExportOptions);
	
	FText CancelText = InArgs._BatchMode ? LOCTEXT("FbxExportOptionsWindow_CancelBatch", "Cancel All") : LOCTEXT("FbxExportOptionsWindow_Cancel", "Cancel");
	FText CancelTooltipText = InArgs._BatchMode ? LOCTEXT("FbxExportOptionsWindow_Cancel_ToolTip_Batch", "Cancel the batch export.") : LOCTEXT("FbxExportOptionsWindow_Cancel_ToolTip", "Cancel the current FBX export.");

	TSharedPtr<SBox> HeaderToolBox;
	TSharedPtr<SHorizontalBox> FbxHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SAssignNew(HeaderToolBox, SBox)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
						.Text(LOCTEXT("Export_CurrentFileTitle", "Current File: "))
					]
					+SHorizontalBox::Slot()
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
						.Text(InArgs._FullPath)
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2.0f)
			[
				SAssignNew(InspectorBox, SBox)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2.0f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxExportOptionsWindow_ExportAll", "Export All"))
					.ToolTipText(LOCTEXT("FbxExportOptionsWindow_ExportAll_ToolTip", "Export all files with these same settings"))
					.Visibility(InArgs._BatchMode ? EVisibility::All : EVisibility::Hidden)
					.OnClicked(this, &SFbxExportOptionsWindow::OnExportAll)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SAssignNew(ImportButton, SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("FbxExportOptionsWindow_Export", "Export"))
					.OnClicked(this, &SFbxExportOptionsWindow::OnExport)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(CancelText)
					.ToolTipText(CancelTooltipText)
					.OnClicked(this, &SFbxExportOptionsWindow::OnCancel)
				]
			]
		]
	];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());

	HeaderToolBox->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SAssignNew(FbxHeaderButtons, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f, 0.0f))
					[
						SNew(SButton)
						.Text(LOCTEXT("FbxExportOptionsWindow_ResetOptions", "Reset to Default"))
						.OnClicked(this, &SFbxExportOptionsWindow::OnResetToDefaultClick)
					]
				]
			]
		]
	);

	DetailsView->SetObject(ExportOptions);
}

FReply SFbxExportOptionsWindow::OnResetToDefaultClick() const
{
	ExportOptions->ResetToDefault();
	//Refresh the view to make sure the custom UI are updating correctly
	DetailsView->SetObject(ExportOptions, true);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
