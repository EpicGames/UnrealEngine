// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/FbxFactory.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetImportTask.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorReimportHandler.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/ReimportFbxAnimSequenceFactory.h"
#include "Factories/ReimportFbxSkeletalMeshFactory.h"
#include "Factories/ReimportFbxStaticMeshFactory.h"
#include "FbxImporter.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "JsonObjectConverter.h"
#include "LODUtilities.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreMisc.h"
#include "Misc/FbxErrors.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "StaticMeshResources.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxFactory)

#define LOCTEXT_NAMESPACE "FBXFactory"

UFbxFactory::UFbxFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = NULL;
	Formats.Add(TEXT("fbx;FBX meshes and animations"));
	Formats.Add(TEXT("obj;OBJ Static meshes"));
	//Formats.Add(TEXT("dae;Collada meshes and animations"));

	bCreateNew = false;
	bText = false;
	bEditorImport = true;
	bOperationCanceled = false;
	bDetectImportTypeOnImport = true;
}


void UFbxFactory::PostInitProperties()
{
	Super::PostInitProperties();
	bEditorImport = true;
	bText = false;

	ImportUI = NewObject<UFbxImportUI>(this, NAME_None, RF_NoFlags);
	UFbxImportUI::LoadOptions(ImportUI);
}


bool UFbxFactory::DoesSupportClass(UClass * Class)
{
	return (Class == UStaticMesh::StaticClass() || Class == USkeletalMesh::StaticClass() || Class == UAnimSequence::StaticClass());
}

UClass* UFbxFactory::ResolveSupportedClass()
{
	UClass* ImportClass = NULL;

	if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
	{
		ImportClass = USkeletalMesh::StaticClass();
	}
	else if (ImportUI->MeshTypeToImport == FBXIT_Animation)
	{
		ImportClass = UAnimSequence::StaticClass();
	}
	else
	{
		ImportClass = UStaticMesh::StaticClass();
	}

	return ImportClass;
}


bool UFbxFactory::DetectImportType(const FString& InFilename)
{
	UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance();
	int32 ImportType = FFbxImporter->GetImportType(InFilename);
	if ( ImportType == -1)
	{
		FFbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("NoImportTypeDetected", "Can't detect import type. No mesh is found or animation track.")), FFbxErrors::Generic_CannotDetectImportType);
		FFbxImporter->ReleaseScene();
		return false;
	}
	else if(!IsAutomatedImport() || ImportUI->bAutomatedImportShouldDetectType)
	{
		ImportUI->MeshTypeToImport = EFBXImportType(ImportType);
		ImportUI->OriginalImportType = ImportUI->MeshTypeToImport;
	}
	
	return true;
}


UObject* UFbxFactory::ImportANode(UnFbx::FFbxImporter* FbxImporter, TArray<void*> VoidNodes, UObject* InParent, FName InName, EObjectFlags Flags, FScopedSlowTask& SlowTask, UObject* InMesh, int LODIndex)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("NodeIndex"), SlowTask.CompletedWork + 1);
	Args.Add(TEXT("ArrayLength"), SlowTask.TotalAmountOfWork);
	SlowTask.EnterProgressFrame(1, GetImportTaskText(FText::Format(NSLOCTEXT("UnrealEd", "Importingf", "Importing ({NodeIndex} of {ArrayLength})"), Args)));
	TArray<FbxNode*> Nodes;
	for (void* VoidNode : VoidNodes)
	{
		Nodes.Add((FbxNode*)VoidNode);
	}
	check(Nodes.Num() > 0 && Nodes[0] != nullptr);

	UObject* CreatedObject = NULL;
	FName OutputName = FbxImporter->MakeNameForMesh(InName.ToString(), Nodes[0]);
	
	{
		// skip collision models
		FbxString NodeName(Nodes[0]->GetName());
		if ( NodeName.Find("UCX") != -1 || NodeName.Find("MCDCX") != -1 ||
			 NodeName.Find("UBX") != -1 || NodeName.Find("USP") != -1 || NodeName.Find("UCP") != -1 )
		{
			return NULL;
		}

		CreatedObject = FbxImporter->ImportStaticMeshAsSingle( InParent, Nodes, OutputName, Flags, ImportUI->StaticMeshImportData, Cast<UStaticMesh>(InMesh), LODIndex );
		bOperationCanceled |= FbxImporter->GetImportOperationCancelled();
	}

	return CreatedObject;
	}

FText UFbxFactory::GetImportTaskText(const FText& TaskText) const
{
	return bOperationCanceled ? LOCTEXT("CancellingImportingFbxMeshTask", "Cancelling FBX mesh import") : TaskText;
}

bool UFbxFactory::ConfigureProperties()
{
	bDetectImportTypeOnImport = true;
	EnableShowOption();

	return true;
}

UObject* UFbxFactory::FactoryCreateFile
(
 UClass* Class,
 UObject* InParent,
 FName Name,
 EObjectFlags Flags,
 const FString& InFilename,
 const TCHAR* Parms,
 FFeedbackContext* Warn,
 bool& bOutOperationCanceled
 )
{
	FScopedSlowTask SlowTask(5, GetImportTaskText(NSLOCTEXT("FbxFactory", "BeginReadFbxFile", "Opening FBX file.")), true);
	if (Warn->GetScopeStack().Num() == 0)
	{
		// We only display this slow task if there is no parent slowtask, because otherwise it is redundant and doesn't display any relevant information on the progress.
		// It is primarly used to regroup all the smaller import sub-tasks for a smoother progression.
		SlowTask.MakeDialog(true);
	}
	SlowTask.EnterProgressFrame(0);
	AdditionalImportedObjects.Empty();
	FString FileExtension = FPaths::GetExtension(InFilename);
	const TCHAR* Type = *FileExtension;

	if (!IFileManager::Get().FileExists(*InFilename))
	{
		UE_LOGF(LogFbx, Error, "Failed to load file '%ls'", *InFilename)
		return nullptr;
	}

	ParseParms(Parms);

	CA_ASSUME(InParent);

	if( bOperationCanceled )
	{
		bOutOperationCanceled = true;
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NULL);
		return NULL;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, Class, InParent, Name, Type);

	FString ParentPackagePath = (InParent && InParent->GetOutermost()) ? FPackageName::GetLongPackagePath(InParent->GetOutermost()->GetName()) : FString(TEXT("/Game"));

	UObject* CreatedObject = NULL;
	//Look if its a re-import, in that case we must call the re-import factory
	UObject *ExistingObject = nullptr;
	if (InParent != nullptr)
	{
		ExistingObject = StaticFindObject(UObject::StaticClass(), InParent, *(Name.ToString()));
		if (ExistingObject)
		{
			FReimportHandler* ReimportHandler = nullptr;
			UFactory* ReimportHandlerFactory = nullptr;
			UAssetImportTask* HandlerOriginalImportTask = nullptr;
			bool bIsObjectSupported = true;
			if (Cast<UStaticMesh>(ExistingObject))
			{
				bIsObjectSupported = true;
				// Using RF_NoFlags because we want to include class default object, which are used by the reimport manager.
				if (TObjectIterator<UReimportFbxStaticMeshFactory> ReimportFactoryIterator(RF_NoFlags); ReimportFactoryIterator)
				{
					HandlerOriginalImportTask = ReimportFactoryIterator->AssetImportTask;
					ReimportFactoryIterator->AssetImportTask = AssetImportTask;
					ReimportHandler = *ReimportFactoryIterator;
					ReimportHandlerFactory = *ReimportFactoryIterator;
				}
			}
			else if (Cast<USkeletalMesh>(ExistingObject))
			{
				bIsObjectSupported = true;
				// Using RF_NoFlags because we want to include class default object, which are used by the reimport manager.
				if (TObjectIterator<UReimportFbxSkeletalMeshFactory> ReimportFactoryIterator(RF_NoFlags); ReimportFactoryIterator)
				{
					HandlerOriginalImportTask = ReimportFactoryIterator->AssetImportTask;
					ReimportFactoryIterator->AssetImportTask = AssetImportTask;
					ReimportHandler = *ReimportFactoryIterator;
					ReimportHandlerFactory = *ReimportFactoryIterator;
				}
			}
			else if (Cast<UAnimSequence>(ExistingObject))
			{
				bIsObjectSupported = true;
				// Using RF_NoFlags because we want to include class default object, which are used by the reimport manager.
				if (TObjectIterator<UReimportFbxAnimSequenceFactory> ReimportFactoryIterator(RF_NoFlags); ReimportFactoryIterator)
				{
					HandlerOriginalImportTask = ReimportFactoryIterator->AssetImportTask;
					ReimportFactoryIterator->AssetImportTask = AssetImportTask;
					ReimportHandler = *ReimportFactoryIterator;
					ReimportHandlerFactory = *ReimportFactoryIterator;
				}
			}

			if (bIsObjectSupported)
			{
				TArray<UObject*> ToReimportObjects;
				ToReimportObjects.Add(ExistingObject);
				TArray<FString> Filenames;
				Filenames.Add(UFactory::CurrentFilename);
				//Set the new fbx source path before starting the re-import
				FReimportManager::Instance()->UpdateReimportPaths(ExistingObject, Filenames);
				//Do the re-import and exit
				const bool bIsAutomated = IsAutomatedImport();
				const bool bShowNotification = !bIsAutomated;
				const int32 SourceFileIndex = INDEX_NONE;
				const bool bForceNewFile = false;

				if (ReimportHandler)
				{
					const bool bAskForNewFileIfMissing = true;
					const FString PreferedReimportFile;

					FReimportManager::Instance()->Reimport(ExistingObject, bAskForNewFileIfMissing, bShowNotification, PreferedReimportFile, ReimportHandler, SourceFileIndex, bForceNewFile, bIsAutomated);
					
					if (ReimportHandlerFactory)
					{
						ReimportHandlerFactory->AssetImportTask = HandlerOriginalImportTask;
					}
				}
				else
				{
					FReimportManager::Instance()->ValidateAllSourceFileAndReimport(ToReimportObjects, bShowNotification, SourceFileIndex, bForceNewFile, bIsAutomated);
				}

				return ExistingObject;
			}
		}
	}

	// logger for all error/warnings
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FFbxLoggerSetter Logger(FbxImporter);

	if ( bDetectImportTypeOnImport)
	{
		if ( !DetectImportType(UFactory::CurrentFilename) )
		{
			// Failed to read the file info, fail the import
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NULL);
			return NULL;
		}
	}
	
	// this one prints all messages that are stored in FFbxImporter
	UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();
	if (bShowOption)
	{
		//Clean up the options
		UnFbx::FBXImportOptions::ResetOptions(ImportOptions);
	}
	
	EFBXImportType ForcedImportType = FBXIT_StaticMesh;

	bool bIsObjFormat = false;
	if( FString(Type).Equals(TEXT("obj"), ESearchCase::IgnoreCase ) )
	{
		bIsObjFormat = true;
	}

	struct FRestoreImportUI
	{
		FRestoreImportUI(UFbxFactory* InFbxFactory)
			: FbxFactory(InFbxFactory) 
		{
			ensure(FbxFactory->OriginalImportUI == nullptr);
			FbxFactory->OriginalImportUI = FbxFactory->ImportUI;
		}

		~FRestoreImportUI()
		{
			FbxFactory->ImportUI = FbxFactory->OriginalImportUI;
			FbxFactory->OriginalImportUI = nullptr;
		}

	private:
		UFbxFactory* FbxFactory;
	};
	FRestoreImportUI RestoreImportUI(this);
	UFbxImportUI* OverrideImportUI = AssetImportTask ? Cast<UFbxImportUI>(AssetImportTask->Options) : nullptr;
	if (OverrideImportUI)
	{
		if (AssetImportTask->bAutomated && OverrideImportUI->bAutomatedImportShouldDetectType)
		{
			OverrideImportUI->MeshTypeToImport = ImportUI->MeshTypeToImport;
			OverrideImportUI->OriginalImportType = ImportUI->OriginalImportType;
		}
		ImportUI = OverrideImportUI;
	}
	else if(AssetImportTask && AssetImportTask->Options)
	{
		UE_LOGF(LogFbx, Display, "The options set in the Asset Import Task are not of type UFbxImportUI and will be ignored");
	}

	//We are not re-importing
	ImportUI->bIsReimport = false;
	ImportUI->ReimportMesh = nullptr;
	ImportUI->bAllowContentTypeImport = true;

	// Show the import dialog only when not in a "yes to all" state or when automating import
	bool bIsAutomated = IsAutomatedImport();
	bool bShowImportDialog = bShowOption && !bIsAutomated;
	bool bImportAll = false;
	
	ImportOptions = GetImportOptions(FbxImporter, ImportUI, bShowImportDialog, bIsAutomated, InParent->GetPathName(), bOperationCanceled, bImportAll, bIsObjFormat, UFactory::CurrentFilename, false, ForcedImportType);
	bOutOperationCanceled = bOperationCanceled;
	SlowTask.EnterProgressFrame(4, GetImportTaskText(NSLOCTEXT("FbxFactory", "BeginImportingFbxMeshTask", "Importing FBX mesh")));
	
	if( bImportAll )
	{
		// If the user chose to import all, we don't show the dialog again and use the same settings for each object until importing another set of files
		bShowOption = false;
	}

	// Automated importing does not use the same settings and gets its settings straight from the user
	if(!bIsAutomated)
	{
		// For multiple files, use the same settings
		bDetectImportTypeOnImport = false;
	}

	if (ImportOptions)
	{
		ImportOptions->bCanShowDialog = !(GIsAutomationTesting || FApp::IsUnattended());
		ImportOptions->bIsImportCancelable = true;
		
		if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
		{
			if (ImportOptions->bImportAsSkeletalSkinning)
			{
				ImportOptions->bImportMaterials = false;
				ImportOptions->bImportTextures = false;
				ImportOptions->bImportLOD = false;
				ImportOptions->bImportSkeletalMeshLODs = false;
				ImportOptions->bImportAnimations = false;
				ImportOptions->bImportMorph = false;
			}
			else if (ImportOptions->bImportAsSkeletalGeometry)
			{
				ImportOptions->bImportAnimations = false;
				ImportOptions->bUpdateSkeletonReferencePose = false;
			}
		}
		if (!FbxImporter->CanImportClass(UPhysicsAsset::StaticClass()))
		{
			ImportOptions->bCreatePhysicsAsset = false;
		}
		
		if ( !FbxImporter->ImportFromFile( *UFactory::CurrentFilename, Type, true ) )
		{
			// Log the error message and fail the import.
			Warn->Log(ELogVerbosity::Error, FbxImporter->GetErrorMessage() );
		}
		else
		{
			// Log the import message and import the mesh.
			const TCHAR* errorMessage = FbxImporter->GetErrorMessage();
			if (errorMessage[0] != '\0')
			{
				Warn->Log( errorMessage );
			}

			FbxNode* RootNodeToImport = NULL;
			RootNodeToImport = FbxImporter->Scene->GetRootNode();

			// For animation and static mesh we assume there is at lease one interesting node by default
			int32 InterestingNodeCount = 1;
			TArray< TArray<FbxNode*>* > SkelMeshArray;

			bool bImportStaticMeshLODs = ImportUI->StaticMeshImportData->bImportMeshLODs;
			bool bCombineMeshes = ImportUI->StaticMeshImportData->bCombineMeshes;
			bool bCombineMeshesLOD = false;

			const bool bCanImportStaticMesh = FbxImporter->CanImportClass(UStaticMesh::StaticClass());
			//We can import skeletal mesh only if we can import skeleton or a skeleton was specified in the import options
			const bool bCanImportSkeletalMesh = FbxImporter->CanImportClass(USkeletalMesh::StaticClass()) && (ImportOptions->SkeletonForAnimation || FbxImporter->CanImportClass(USkeleton::StaticClass()));
			const bool bCanImportAnimSequence = FbxImporter->CanImportClass(UAnimSequence::StaticClass());

			if (bCanImportSkeletalMesh && ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh )
			{
				FbxImporter->FillFbxSkelMeshArrayInScene(RootNodeToImport, SkelMeshArray, false, (ImportOptions->bImportAsSkeletalGeometry || ImportOptions->bImportAsSkeletalSkinning));
				InterestingNodeCount = SkelMeshArray.Num();
			}
			else if(bCanImportStaticMesh && ImportUI->MeshTypeToImport == FBXIT_StaticMesh )
			{
				FbxImporter->ApplyTransformSettingsToFbxNode(RootNodeToImport, ImportUI->StaticMeshImportData);

				if( bCombineMeshes && !bImportStaticMeshLODs )
				{
					// If Combine meshes and dont import mesh LODs, the interesting node count should be 1 so all the meshes are grouped together into one static mesh
					InterestingNodeCount = 1;
				}
				else
				{
					// count meshes in lod groups if we dont care about importing LODs
					bool bCountLODGroupMeshes = !bImportStaticMeshLODs && bCombineMeshes;
					int32 NumLODGroups = 0;
					InterestingNodeCount = FbxImporter->GetFbxMeshCount(RootNodeToImport,bCountLODGroupMeshes,NumLODGroups);

					// if there were LODs in the file, do not combine meshes even if requested
					if( bImportStaticMeshLODs && bCombineMeshes && NumLODGroups > 0)
					{
						bCombineMeshes = false;
						//Combine all the LOD together and export one mesh with LODs
						bCombineMeshesLOD = true;
					}
				}
				//Find all collision models, even the one contain under a LOD Group
				FbxImporter->FillFbxCollisionMeshArray(RootNodeToImport);
			}

		
			if (InterestingNodeCount > 1)
			{
				// the option only works when there are only one asset
				ImportOptions->bUsedAsFullName = false;
			}

			const FString Filename( UFactory::CurrentFilename );
			if (RootNodeToImport && InterestingNodeCount > 0)
			{  
				int32 NodeIndex = 0;

				int32 ImportedMeshCount = 0;
				if (bCanImportStaticMesh && ImportUI->MeshTypeToImport == FBXIT_StaticMesh )  // static mesh
				{
					UStaticMesh* NewStaticMesh = NULL;
					if (bCombineMeshes)
					{
						TArray<FbxNode*> FbxMeshArray;
						FbxImporter->FillFbxMeshArray(RootNodeToImport, FbxMeshArray, FbxImporter);
						if (FbxMeshArray.Num() > 0)
						{
							NewStaticMesh = FbxImporter->ImportStaticMeshAsSingle(InParent, FbxMeshArray, Name, Flags, ImportUI->StaticMeshImportData, NULL, 0, nullptr);
							bOperationCanceled |= SlowTask.ShouldCancel() || FbxImporter->GetImportOperationCancelled();

							if (!bOperationCanceled && NewStaticMesh != nullptr)
							{
								//Build the staticmesh
								SlowTask.EnterProgressFrame(1, GetImportTaskText(NSLOCTEXT("FbxFactory", "EndingImportingFbxMeshTask", "Finalizing static mesh import.")));
								FbxImporter->PostImportStaticMesh(NewStaticMesh, FbxMeshArray);
								FbxImporter->UpdateStaticMeshImportData(NewStaticMesh, nullptr);
							}
						}

						ImportedMeshCount = NewStaticMesh ? 1 : 0;
					}
					else if (bCombineMeshesLOD)
					{
						TArray<FbxNode*> FbxMeshArray;
						TArray<FbxNode*> FbxLodGroups;
						TArray<TArray<FbxNode*>> FbxMeshesLod;
						FbxImporter->FillFbxMeshAndLODGroupArray(RootNodeToImport, FbxLodGroups, FbxMeshArray);
						FbxMeshesLod.Add(FbxMeshArray);
						for (FbxNode* LODGroup : FbxLodGroups)
						{
							if (LODGroup->GetNodeAttribute() && LODGroup->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup && LODGroup->GetChildCount() > 0)
							{
								for (int32 GroupLodIndex = 0; GroupLodIndex < LODGroup->GetChildCount(); ++GroupLodIndex)
								{
									if (GroupLodIndex >= MAX_STATIC_MESH_LODS)
									{
										FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
											LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(LODGroup->GetChildCount() - MAX_STATIC_MESH_LODS))
										), FFbxErrors::Generic_Mesh_TooManyLODs);
										break;
									}
									TArray<FbxNode*> AllNodeInLod;
									FbxImporter->FindAllLODGroupNode(AllNodeInLod, LODGroup, GroupLodIndex);
									if (AllNodeInLod.Num() > 0)
									{
										if (FbxMeshesLod.Num() <= GroupLodIndex)
										{
											FbxMeshesLod.Add(AllNodeInLod);
										}
										else
										{
											TArray<FbxNode*> &LODGroupArray = FbxMeshesLod[GroupLodIndex];
											for (FbxNode* NodeToAdd : AllNodeInLod)
											{
												LODGroupArray.Add(NodeToAdd);
											}
										}
									}
								}
							}
						}

						FFormatNamedArguments Args;
						Args.Add(TEXT("NodeIndex"), 1);
						Args.Add(TEXT("ArrayLength"), FbxMeshesLod.Num());
						FText SlowtaskText = NSLOCTEXT("UnrealEd", "Importingf", "Importing ({NodeIndex} of {ArrayLength})");
						FScopedSlowTask ImportStaticMeshSlowTask(FbxMeshesLod.Num(), GetImportTaskText(FText::Format(SlowtaskText, Args)));
						ImportStaticMeshSlowTask.MakeDialog();

						//Import the LOD root
						if (FbxMeshesLod.Num() > 0)
						{
							ImportStaticMeshSlowTask.EnterProgressFrame(1, GetImportTaskText(FText::Format(SlowtaskText, Args)));
							TArray<FbxNode*> &LODMeshesArray = FbxMeshesLod[0];
							NewStaticMesh = FbxImporter->ImportStaticMeshAsSingle(InParent, LODMeshesArray, Name, Flags, ImportUI->StaticMeshImportData, NULL, 0);
							bOperationCanceled |= FbxImporter->GetImportOperationCancelled();
						}
						//Import all LODs
						for (int32 LODIndex = 1; LODIndex < FbxMeshesLod.Num() && !bOperationCanceled; ++LODIndex)
						{
							TArray<FbxNode*> &LODMeshesArray = FbxMeshesLod[LODIndex];
							Args[TEXT("NodeIndex")] = LODIndex + 1;
							ImportStaticMeshSlowTask.EnterProgressFrame(1, GetImportTaskText(FText::Format(SlowtaskText, Args)));

							if (LODMeshesArray[0]->GetMesh() == nullptr)
							{
								FbxImporter->AddStaticMeshSourceModelGeneratedLOD(NewStaticMesh, LODIndex);
							}
							else
							{
								UStaticMesh* ImportedStaticMesh = FbxImporter->ImportStaticMeshAsSingle(InParent, LODMeshesArray, Name, Flags, ImportUI->StaticMeshImportData, NewStaticMesh, LODIndex);
								if (ImportedStaticMesh)
								{
									check(ImportedStaticMesh == NewStaticMesh);
									if (NewStaticMesh && NewStaticMesh->IsSourceModelValid(LODIndex))
									{
										NewStaticMesh->GetSourceModel(LODIndex).bImportWithBaseMesh = true;
									}
								}
								else
								{
									FbxImporter->AddStaticMeshSourceModelGeneratedLOD(NewStaticMesh, LODIndex);
								}
							}
							bOperationCanceled |= FbxImporter->GetImportOperationCancelled() || SlowTask.ShouldCancel();
						}
						
						if (!bOperationCanceled && NewStaticMesh)
						{
							FbxImporter->PostImportStaticMesh(NewStaticMesh, FbxMeshesLod[0]);
							FbxImporter->UpdateStaticMeshImportData(NewStaticMesh, nullptr);
						}
					}
					else // !bCombineMeshes && !bCombineMeshesLOD
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("NodeIndex"), 1);
						Args.Add(TEXT("ArrayLength"), InterestingNodeCount);
						FScopedSlowTask ImportNodeSlowTask(InterestingNodeCount, GetImportTaskText(FText::Format(NSLOCTEXT("UnrealEd", "Importingf", "Importing ({NodeIndex} of {ArrayLength})"), Args)));
						ImportNodeSlowTask.MakeDialog(true);
						TArray<UObject*> AllNewAssets;
						UObject* Object = RecursiveImportNode(FbxImporter, RootNodeToImport, InParent, Name, Flags, ImportNodeSlowTask, AllNewAssets);
						bOperationCanceled |= SlowTask.ShouldCancel();
						NewStaticMesh = Cast<UStaticMesh>( Object );

						// Make sure to notify the asset registry of all assets created other than the one returned, which will notify the asset registry automatically.
						for ( auto AssetIt = AllNewAssets.CreateConstIterator(); !bOperationCanceled && AssetIt; ++AssetIt )
						{
							UObject* Asset = *AssetIt;
							if ( Asset != NewStaticMesh )
							{
								FAssetRegistryModule::AssetCreated(Asset);
								Asset->MarkPackageDirty();
								//Make sure the build is up to date with the latest section info map
								Asset->PostEditChange();
								AdditionalImportedObjects.Add(Asset);
							}
						}

						ImportedMeshCount = AllNewAssets.Num();
					}

					// Importing static mesh global sockets only if one mesh is imported
					if(!bOperationCanceled && ImportedMeshCount == 1 && NewStaticMesh)
					{
						FbxImporter->ImportStaticMeshGlobalSockets( NewStaticMesh );
					}

					CreatedObject = NewStaticMesh;

				}
				else if ( bCanImportSkeletalMesh && ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh )// skeletal mesh
				{
					int32 TotalNumNodes = 0;
					for (int32 i = 0; i < SkelMeshArray.Num() && !bOperationCanceled; i++)
					{
						UPackage* Package = (SkelMeshArray.Num() == 1 ? Cast<UPackage>(InParent) : nullptr);
						USkeletalMesh* BaseSkeletalMesh = nullptr;
						TArray<FbxNode*> NodeArray = *SkelMeshArray[i];
					
						TotalNumNodes += NodeArray.Num();
						// check if there is LODGroup for this skeletal mesh
						int32 MaxNumberOfLOD = 1;
						for (int32 j = 0; j < NodeArray.Num(); j++)
						{
							FbxNode* Node = NodeArray[j];
							if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
							{
								// get max LODgroup level
								if (MaxNumberOfLOD < Node->GetChildCount())
								{
									MaxNumberOfLOD = Node->GetChildCount();
								}
							}
						}
					
						int32 SuccessfulLodIndex = 0;
						bool bImportSkeletalMeshLODs = ImportUI->SkeletalMeshImportData->bImportMeshLODs;
						
						//The skeletalmesh will be set after we import the LOD 0 since it is not created yet.
						FScopedSkeletalMeshPostEditChange ScopedPostEditChange(nullptr);

						for (int32 LODIndex = 0; LODIndex < MaxNumberOfLOD && !bOperationCanceled; LODIndex++)
						{
							//We need to know what is the imported lod index when importing the morph targets
							int32 ImportedSuccessfulLodIndex = INDEX_NONE;
							if ( !bImportSkeletalMeshLODs && LODIndex > 0) // not import LOD if UI option is OFF
							{
								break;
							}

							TArray<FbxNode*> SkelMeshNodeArray;
							for (int32 j = 0; j < NodeArray.Num(); j++)
							{
								FbxNode* Node = NodeArray[j];
								if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
								{
									TArray<FbxNode*> NodeInLod;
									if (Node->GetChildCount() > LODIndex)
									{
										FbxImporter->FindAllLODGroupNode(NodeInLod, Node, LODIndex);
									}
									else // in less some LODGroups have less level, use the last level
									{
										FbxImporter->FindAllLODGroupNode(NodeInLod, Node, Node->GetChildCount() - 1);
									}

									for (FbxNode *MeshNode : NodeInLod)
									{
										SkelMeshNodeArray.Add(MeshNode);
									}
								}
								else
								{
									SkelMeshNodeArray.Add(Node);
								}
							}
							FSkeletalMeshImportData OutData;
							bool bMapMorphTargetToTimeZero = false;
							if (LODIndex == 0 && SkelMeshNodeArray.Num() != 0)
							{
								FName OutputName = FbxImporter->MakeNameForMesh((Package == nullptr) ? TEXT("None") : Name.ToString(), SkelMeshNodeArray[0]);

								if (Package)
								{
									if (!OutputName.ToString().Equals(FPaths::GetCleanFilename(Package->GetName()), ESearchCase::IgnoreCase))
									{
										//We need to create a new package
										Package = nullptr;
									}
								}

								if (Package == nullptr)
								{
									FString NewPackageName;
									if (!ParentPackagePath.IsEmpty())
									{
										NewPackageName = ParentPackagePath + TEXT("/") + OutputName.ToString();
									}
									else
									{
										FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::Format(LOCTEXT("ImportSkeletalMesh", "Invalid Parent package when importing {0}.\nThe asset will not be imported."), FText::FromName(OutputName))), FFbxErrors::Generic_ImportingNewObjectFailed);
										NewPackageName = OutputName.ToString();
									}
									NewPackageName = UPackageTools::SanitizePackageName(NewPackageName);
									Package = CreatePackage(*NewPackageName);
									Package->FullyLoad();
								}

								UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
								ImportSkeletalMeshArgs.InParent = Package;
								ImportSkeletalMeshArgs.NodeArray = SkelMeshNodeArray;
								ImportSkeletalMeshArgs.Name = OutputName;
								ImportSkeletalMeshArgs.Flags = Flags;
								ImportSkeletalMeshArgs.TemplateImportData = ImportUI->SkeletalMeshImportData;
								ImportSkeletalMeshArgs.LodIndex = LODIndex;
								ImportSkeletalMeshArgs.OutData = &OutData;

								BaseSkeletalMesh = FbxImporter->ImportSkeletalMesh( ImportSkeletalMeshArgs );
								bOperationCanceled |= FbxImporter->GetImportOperationCancelled() || SlowTask.ShouldCancel();
								
								if ( BaseSkeletalMesh && !bOperationCanceled)
								{
									//Set the base skeletalmesh to the scoped post edit change variable
									ScopedPostEditChange.SetSkeletalMesh(BaseSkeletalMesh);

									if (ImportOptions->bImportAnimations)
									{
										// We need to remove all scaling from the root node before we set up animation data.
										// Othewise some of the global transform calculations will be incorrect.
										FbxImporter->RemoveTransformSettingsFromFbxNode(RootNodeToImport, ImportUI->SkeletalMeshImportData);
										FbxImporter->SetupAnimationDataFromMesh(BaseSkeletalMesh, Package, SkelMeshNodeArray, ImportUI->AnimSequenceImportData, OutputName.ToString());

										// Reapply the transforms for the rest of the import
										FbxImporter->ApplyTransformSettingsToFbxNode(RootNodeToImport, ImportUI->SkeletalMeshImportData);
									}
									bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
									ImportedSuccessfulLodIndex = SuccessfulLodIndex;
									//Increment the LOD index
									SuccessfulLodIndex++;
								}
							}
							else if (BaseSkeletalMesh && SkelMeshNodeArray[0]->GetMesh() == nullptr)
							{
								FSkeletalMeshUpdateContext UpdateContext;
								UpdateContext.SkeletalMesh = BaseSkeletalMesh;
								//Add a autogenerated LOD to the BaseSkeletalMesh
								FSkeletalMeshLODInfo& LODInfo = BaseSkeletalMesh->AddLODInfo();
								LODInfo.ReductionSettings.NumOfTrianglesPercentage = FMath::Pow(0.5f, (float)(SuccessfulLodIndex));
								LODInfo.ReductionSettings.BaseLOD = 0;
								LODInfo.bImportWithBaseMesh = true;
								LODInfo.SourceImportFilename = FString(TEXT(""));
								FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, SuccessfulLodIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
								ImportedSuccessfulLodIndex = SuccessfulLodIndex;
								SuccessfulLodIndex++;
							}
							else if (BaseSkeletalMesh) // the base skeletal mesh is imported successfully
							{
								FName LODObjectName = NAME_None;
								UnFbx::FFbxImporter::FImportSkeletalMeshArgs ImportSkeletalMeshArgs;
								ImportSkeletalMeshArgs.InParent = BaseSkeletalMesh->GetOutermost();
								ImportSkeletalMeshArgs.NodeArray = SkelMeshNodeArray;
								ImportSkeletalMeshArgs.Name = LODObjectName;
								ImportSkeletalMeshArgs.Flags = RF_Transient;
								ImportSkeletalMeshArgs.TemplateImportData = ImportUI->SkeletalMeshImportData;
								ImportSkeletalMeshArgs.LodIndex = SuccessfulLodIndex;
								ImportSkeletalMeshArgs.OutData = &OutData;

								USkeletalMesh *LODObject = FbxImporter->ImportSkeletalMesh( ImportSkeletalMeshArgs );
								bOperationCanceled |= FbxImporter->GetImportOperationCancelled();

								if (!bOperationCanceled && FbxImporter->ImportSkeletalMeshLOD(LODObject, BaseSkeletalMesh, SuccessfulLodIndex))
								{
									FSkeletalMeshLODInfo* LODInfo = BaseSkeletalMesh->GetLODInfo(SuccessfulLodIndex);
									LODInfo->bImportWithBaseMesh = true;
									LODInfo->SourceImportFilename = FString(TEXT(""));
									bMapMorphTargetToTimeZero = ImportSkeletalMeshArgs.bMapMorphTargetToTimeZero;
									ImportedSuccessfulLodIndex = SuccessfulLodIndex;
									SuccessfulLodIndex++;
								}
								else
								{
									FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_SkeletalMeshLOD", "Failed to import Skeletal mesh LOD.")), FFbxErrors::SkeletalMesh_LOD_FailedToImport);
								}
							}
						
							bOperationCanceled |= SlowTask.ShouldCancel();

							// import morph target
							if (!bOperationCanceled && BaseSkeletalMesh && ImportOptions->bImportMorph && ImportedSuccessfulLodIndex != INDEX_NONE)
							{
								// Disable material importing when importing morph targets
								uint32 bImportMaterials = ImportOptions->bImportMaterials;
								ImportOptions->bImportMaterials = 0;
								uint32 bImportTextures = ImportOptions->bImportTextures;
								ImportOptions->bImportTextures = 0;

								FbxImporter->ImportFbxMorphTarget(SkelMeshNodeArray, BaseSkeletalMesh, ImportedSuccessfulLodIndex, OutData, bMapMorphTargetToTimeZero);
								bOperationCanceled |= FbxImporter->GetImportOperationCancelled();
							
								ImportOptions->bImportMaterials = !!bImportMaterials;
								ImportOptions->bImportTextures = !!bImportTextures;
							}

							bOperationCanceled |= SlowTask.ShouldCancel();
						}
					
						if (BaseSkeletalMesh && !bOperationCanceled)
						{
							NodeIndex++;
							FFormatNamedArguments Args;
							Args.Add( TEXT("NodeIndex"), NodeIndex );
							Args.Add( TEXT("ArrayLength"), SkelMeshArray.Num() );
							SlowTask.EnterProgressFrame(0, GetImportTaskText(FText::Format(NSLOCTEXT("UnrealEd", "Importingf", "Importing ({NodeIndex} of {ArrayLength})"), Args)));
							
							UnFbx::FFbxImporter::UpdateSkeletalMeshImportData(BaseSkeletalMesh, ImportUI->SkeletalMeshImportData, INDEX_NONE, nullptr, nullptr);
							
							if (CreatedObject == nullptr)
							{
								CreatedObject = BaseSkeletalMesh;
							}
							else if(CreatedObject != BaseSkeletalMesh)
							{
								AdditionalImportedObjects.Add(BaseSkeletalMesh);
							}
						}

						bOperationCanceled |= SlowTask.ShouldCancel();
					}
				
					for (int32 i = 0; i < SkelMeshArray.Num(); i++)
					{
						delete SkelMeshArray[i];
					}
					
					// if total nodes we found is 0, we didn't find anything. 
					if (TotalNumNodes == 0)
					{
						FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_NoMeshFoundOnRoot", "Could not find any valid mesh on the root hierarchy. If you have mesh in the sub hierarchy, please enable option of [Import Meshes In Bone Hierarchy] when import.")), 
							FFbxErrors::SkeletalMesh_NoMeshFoundOnRoot);
					}
				}
				else if (bCanImportAnimSequence && ImportUI->MeshTypeToImport == FBXIT_Animation )// animation
				{
					if (ImportOptions->SkeletonForAnimation)
					{
						// will return the last animation sequence that were added
						CreatedObject = UEditorEngine::ImportFbxAnimation( ImportOptions->SkeletonForAnimation, InParent, ImportUI->AnimSequenceImportData, *Filename, *Name.ToString(), true );
					}
				}
			}
			else
			{
				if (RootNodeToImport == NULL)
				{
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_InvalidRoot", "Could not find root node.")), FFbxErrors::SkeletalMesh_InvalidRoot);
				}
				else if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
				{
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_InvalidBone", "Failed to find any bone hierarchy. Try disabling the \"Import As Skeletal\" option to import as a rigid mesh. ")), FFbxErrors::SkeletalMesh_InvalidBone);
				}
				else
				{
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_InvalidNode", "Could not find any node.")), FFbxErrors::SkeletalMesh_InvalidNode);
				}
			}
		}

		if (CreatedObject == NULL && !bOperationCanceled)
		{
			FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Error, LOCTEXT("FailedToImport_NoObject", "Import failed.")), FFbxErrors::Generic_ImportingNewObjectFailed);
		}
	}

	if (bOperationCanceled)
	{
		FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Info, LOCTEXT("FailedToImport_OperationCanceled", "Import was canceled.")), FFbxErrors::Generic_ImportingNewObjectFailed);
		AdditionalImportedObjects.Empty();
		CancelObjectCreation(FbxImporter);
		CreatedObject = nullptr;
	}
	else
	{
		FbxImporter->ReleaseScene();
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, CreatedObject);
	}
	bOutOperationCanceled = bOperationCanceled;

	return CreatedObject;
}

void UFbxFactory::CancelObjectCreation(UnFbx::FFbxImporter* FbxImporter) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const TArray<TWeakObjectPtr<UObject>>& CreatedObjects = FbxImporter->GetCreatedObjects();
	TArray<FAssetData> AssetsToDelete;
	TArray<TWeakObjectPtr<UPackage>> PotentialPackageToDeleteReferences;
	
	// Flag for deletion the newly created objects in memory.
	for (int32 CreatedObjectIndex = 0; CreatedObjectIndex < CreatedObjects.Num(); ++CreatedObjectIndex)
	{
		const TWeakObjectPtr<UObject>& CurrentObject = CreatedObjects[CreatedObjectIndex];
		if (CurrentObject.IsValid())
		{
			AssetsToDelete.Emplace(AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(CurrentObject.Get())));
			PotentialPackageToDeleteReferences.Add(CurrentObject->GetOutermost());
			CurrentObject->ClearFlags(RF_Standalone | RF_Public);
			CurrentObject->RemoveFromRoot();
			CurrentObject->MarkAsGarbage();
		}
	}
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	// Delete any remaining newly created assets.
	TArray<UObject*> CreatedObjectsPtr;
	for (int32 CreatedObjectIndex = 0; CreatedObjectIndex < AssetsToDelete.Num(); ++CreatedObjectIndex)
	{
		const FAssetData& CurrentObject = AssetsToDelete[CreatedObjectIndex];
		UObject* AssetToDelete = CurrentObject.GetAsset();
		if (AssetToDelete != nullptr)
		{
			CreatedObjectsPtr.AddUnique(AssetToDelete);
	}
	}
	bool bObjectsDeleted = ObjectTools::ForceDeleteObjects(CreatedObjectsPtr, false) > 0;
	if (bObjectsDeleted)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	FbxImporter->ReleaseScene();

	//We make sure sure to cleanup any remaining empty packages, otherwise we will be prompted to save them even if they don't contain any asset.
	TArray<UPackage*> PotentialPackageToDelete;
	for (int32 PackageIndex = PotentialPackageToDeleteReferences.Num() - 1; PackageIndex >= 0; --PackageIndex)
	{
		if (PotentialPackageToDeleteReferences[PackageIndex].IsValid())
		{
			PotentialPackageToDelete.Add(PotentialPackageToDeleteReferences[PackageIndex].Get());
		}
}
	ObjectTools::CleanupAfterSuccessfulDelete(PotentialPackageToDelete);

	//Make sure the content browser is in sync after we delete, to avoid stale references.
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const bool bAllowLockedBrowser = true;
	ContentBrowserModule.Get().SyncBrowserToAssets(AssetsToDelete, bAllowLockedBrowser);
}

UObject* UFbxFactory::RecursiveImportNode(UnFbx::FFbxImporter* FbxImporter, void* VoidNode, UObject* InParent, FName InName, EObjectFlags Flags, FScopedSlowTask& SlowTask, TArray<UObject*>& OutNewAssets)
{
	TArray<void*> TmpVoidArray;
	UObject* CreatedObject = NULL;
	FbxNode* Node = (FbxNode*)VoidNode;
	bOperationCanceled |= SlowTask.ShouldCancel();

	if (bOperationCanceled)
	{
		return CreatedObject;
	}

	if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup && Node->GetChildCount() > 0 )
	{
		TArray<FbxNode*> AllNodeInLod;
		// import base mesh
		FbxImporter->FindAllLODGroupNode(AllNodeInLod, Node, 0);
		if (AllNodeInLod.Num() > 0)
		{
			TmpVoidArray.Empty();
			for (FbxNode* LodNode : AllNodeInLod)
			{
				TmpVoidArray.Add(LodNode);
			}
			CreatedObject = ImportANode(FbxImporter, TmpVoidArray, InParent, InName, Flags, SlowTask);
		}

		if (CreatedObject)
		{
			OutNewAssets.AddUnique(CreatedObject);
		}

		bool bImportMeshLODs = ImportUI->StaticMeshImportData->bImportMeshLODs;

		if (CreatedObject && bImportMeshLODs)
		{
			// import LOD meshes
			SlowTask.TotalAmountOfWork += Node->GetChildCount();
			for (int32 LODIndex = 1; LODIndex < Node->GetChildCount(); LODIndex++)
			{
				if (LODIndex >= MAX_STATIC_MESH_LODS)
				{
					SlowTask.CompletedWork += Node->GetChildCount() - LODIndex;
					FbxImporter->AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(
						LOCTEXT("ImporterLimits_MaximumStaticMeshLODReach", "Reached the maximum number of LODs for a Static Mesh({0}) - discarding {1} LOD meshes."), FText::AsNumber(MAX_STATIC_MESH_LODS), FText::AsNumber(Node->GetChildCount() - MAX_STATIC_MESH_LODS))
					), FFbxErrors::Generic_Mesh_TooManyLODs);
					break;
				}
				
				bOperationCanceled |= SlowTask.ShouldCancel();
				if (bOperationCanceled)
				{
					break;
				}

				AllNodeInLod.Empty();
				FbxImporter->FindAllLODGroupNode(AllNodeInLod, Node, LODIndex);
				if (AllNodeInLod.Num() > 0)
				{
					if (AllNodeInLod[0]->GetMesh() == nullptr)
					{
						UStaticMesh* NewStaticMesh = Cast<UStaticMesh>(CreatedObject);
						//Add a Lod generated model
						while (NewStaticMesh->GetNumSourceModels() <= LODIndex)
						{
							NewStaticMesh->AddSourceModel();
						}
						
						ImportANode(FbxImporter, TmpVoidArray, InParent, InName, Flags, SlowTask, CreatedObject, LODIndex);

						FStaticMeshSourceModel& ThisSourceModel = NewStaticMesh->GetSourceModel(LODIndex);

						if (LODIndex - 1 > 0 && NewStaticMesh->IsReductionActive(LODIndex - 1))
						{
							const FStaticMeshSourceModel& PrevSourceModel = NewStaticMesh->GetSourceModel(LODIndex - 1);

							//Do not add the LODGroup bias here, since the bias will be apply during the build
							if (PrevSourceModel.ReductionSettings.PercentTriangles < 1.0f)
							{
								ThisSourceModel.ReductionSettings.PercentTriangles = PrevSourceModel.ReductionSettings.PercentTriangles * 0.5f;
							}
							else if (PrevSourceModel.ReductionSettings.MaxDeviation > 0.0f)
							{
								ThisSourceModel.ReductionSettings.MaxDeviation = PrevSourceModel.ReductionSettings.MaxDeviation + 1.0f;
							}
						}
						else
						{
							ThisSourceModel.ReductionSettings.PercentTriangles = FMath::Pow(0.5f, (float)LODIndex);
						}
					}
					else
					{
						TmpVoidArray.Empty();
						for (FbxNode* LodNode : AllNodeInLod)
						{
							TmpVoidArray.Add(LodNode);
						}
						ImportANode(FbxImporter, TmpVoidArray, InParent, InName, Flags, SlowTask, CreatedObject, LODIndex);
						UStaticMesh* NewStaticMesh = Cast<UStaticMesh>(CreatedObject);
						if(NewStaticMesh->IsSourceModelValid(LODIndex))
						{
							NewStaticMesh->GetSourceModel(LODIndex).bImportWithBaseMesh = true;
						}
					}
					
				}
			}
		}
		
		if (CreatedObject)
		{
			UStaticMesh *NewStaticMesh = Cast<UStaticMesh>(CreatedObject);
			if (NewStaticMesh != nullptr)
			{
				//Reorder the material
				TArray<FbxNode*> Nodes;
				FbxImporter->FindAllLODGroupNode(Nodes, Node, 0);
				if (Nodes.Num() > 0)
				{
					FbxImporter->PostImportStaticMesh(NewStaticMesh, Nodes);
					FbxImporter->UpdateStaticMeshImportData(NewStaticMesh, nullptr);
				}
			}
		}
	}
	else
	{
		if (Node->GetMesh())
		{
			TmpVoidArray.Empty();
			TmpVoidArray.Add(Node);
			CreatedObject = ImportANode(FbxImporter, TmpVoidArray, InParent, InName, Flags, SlowTask);

			if (CreatedObject)
			{
				UStaticMesh *NewStaticMesh = Cast<UStaticMesh>(CreatedObject);
				if (NewStaticMesh != nullptr)
				{
					//Reorder the material
					TArray<FbxNode*> Nodes;
					Nodes.Add(Node);
					FbxImporter->PostImportStaticMesh(NewStaticMesh, Nodes);
					FbxImporter->UpdateStaticMeshImportData(NewStaticMesh, nullptr);
				}
				OutNewAssets.AddUnique(CreatedObject);
			}
		}
		else
		{
			SlowTask.CompletedWork++;
		}
		
		SlowTask.TotalAmountOfWork += Node->GetChildCount();
		for (int32 ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
		{
			UObject* SubObject = RecursiveImportNode(FbxImporter, Node->GetChild(ChildIndex), InParent, InName, Flags, SlowTask, OutNewAssets);

			if ( SubObject )
			{
				OutNewAssets.AddUnique(SubObject);
			}

			if (CreatedObject ==NULL)
			{
				CreatedObject = SubObject;
			}
		}
	}

	return CreatedObject;
}


void UFbxFactory::CleanUp() 
{
	UnFbx::FFbxImporter* FFbxImporter = UnFbx::FFbxImporter::GetInstance(true/*bDoNotCreate*/);
	bDetectImportTypeOnImport = true;
	bShowOption = true;
	// load options
	if (FFbxImporter)
	{
		struct UnFbx::FBXImportOptions* ImportOptions = FFbxImporter->GetImportOptions();
		if ( ImportOptions )
		{
			ImportOptions->SkeletonForAnimation = NULL;
			ImportOptions->PhysicsAsset = NULL;
			ImportOptions->bIsImportCancelable = false;
		}
	}

	Super::CleanUp();
}

bool UFbxFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if( Extension == TEXT("fbx") || Extension == TEXT("obj") )
	{
		return true;
	}
	return false;
}

bool UFbxFactory::CanImportBeCanceled() const
{
	return true;
}

IImportSettingsParser* UFbxFactory::GetImportSettingsParser()
{
	return ImportUI;
}

TArray<FString> UFbxFactory::GetFormats() const
{
	return GetFbxFormats(this);
}

TArray<FString> UFbxFactory::GetFbxFormats(const UFactory* Factory)
{
	TArray<FString> FormatArray;
	const IConsoleVariable* CVarFbx = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.FBX"));
	const IConsoleVariable* CVarFbxLevel = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.FBX.ToLevel"));
	const bool bUseLegacyFbx = (!CVarFbx || !CVarFbx->GetBool()) || (!CVarFbxLevel || !CVarFbxLevel->GetBool());
	const IConsoleVariable* CVarObj = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.OBJ"));
	const bool bUseLegacyObj = (!CVarObj || !CVarObj->GetBool());

	for (const FString& Format : Factory->Formats)
	{
		if (Format.StartsWith(TEXT("fbx")))
		{
			//Skip if interchange fbx is enabled
			if (bUseLegacyFbx)
			{
				FormatArray.Add(Format);
			}
		}
		else if (Format.StartsWith(TEXT("obj")))
		{
			//Skip if interchange obj is enabled
			if (bUseLegacyObj)
			{
				FormatArray.Add(Format);
			}
		}
		else
		{
			FormatArray.Add(Format);
		}
	}
	return FormatArray;
}

UFbxImportUI::UFbxImportUI(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsReimport = false;
	ReimportMesh = nullptr;
	bAllowContentTypeImport = false;
	bAutomatedImportShouldDetectType = true;
	bResetToFbxOnMaterialConflict = false;
	//Make sure we are transactional to allow undo redo
	this->SetFlags(RF_Transactional);
	
	StaticMeshImportData = CreateDefaultSubobject<UFbxStaticMeshImportData>(TEXT("StaticMeshImportData"));
	StaticMeshImportData->SetFlags(RF_Transactional);
	UFbxImportUI::LoadOptions(StaticMeshImportData);
	
	SkeletalMeshImportData = CreateDefaultSubobject<UFbxSkeletalMeshImportData>(TEXT("SkeletalMeshImportData"));
	SkeletalMeshImportData->SetFlags(RF_Transactional);
	UFbxImportUI::LoadOptions(SkeletalMeshImportData);
	
	AnimSequenceImportData = CreateDefaultSubobject<UFbxAnimSequenceImportData>(TEXT("AnimSequenceImportData"));
	AnimSequenceImportData->SetFlags(RF_Transactional);
	UFbxImportUI::LoadOptions(AnimSequenceImportData);
	
	TextureImportData = CreateDefaultSubobject<UFbxTextureImportData>(TEXT("TextureImportData"));
	TextureImportData->SetFlags(RF_Transactional);
	UFbxImportUI::LoadOptions(TextureImportData);
}


bool UFbxImportUI::CanEditChange( const FProperty* InProperty ) const
{
	bool bIsMutable = Super::CanEditChange( InProperty );
	if( bIsMutable && InProperty != NULL )
	{
		FName PropName = InProperty->GetFName();

		if(PropName == TEXT("FrameImportRange"))
		{
			bIsMutable = AnimSequenceImportData->AnimationLength == FBXALIT_SetRange && bImportAnimations;
		}
		else if(PropName == TEXT("bImportCustomAttribute") || PropName == TEXT("AnimationLength") || PropName == TEXT("CustomSampleRate") || PropName == TEXT("bUseDefaultSampleRate"))
		{
			bIsMutable = bImportAnimations;
		}

		if(bIsObjImport && InProperty->GetBoolMetaData(TEXT("OBJRestrict")))
		{
			bIsMutable = false;
		}
	}

	return bIsMutable;
}

void UFbxImportUI::ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson)
{
	// Skip instanced object references. 
	int64 SkipFlags = CPF_InstancedReference;
	FJsonObjectConverter::JsonObjectToUStruct(ImportSettingsJson, GetClass(), this, 0, SkipFlags);

	bAutomatedImportShouldDetectType = true;
	if(ImportSettingsJson->TryGetField(TEXT("MeshTypeToImport")).IsValid())
	{
		// Import type was specified by the user if MeshTypeToImport exists
		bAutomatedImportShouldDetectType = false;
	}

	const TSharedPtr<FJsonObject>* StaticMeshImportJson = nullptr;
	ImportSettingsJson->TryGetObjectField(TEXT("StaticMeshImportData"), StaticMeshImportJson);
	if(StaticMeshImportJson)
	{
		FJsonObjectConverter::JsonObjectToUStruct(StaticMeshImportJson->ToSharedRef(), StaticMeshImportData->GetClass(), StaticMeshImportData, 0, 0);
	}

	const TSharedPtr<FJsonObject>* SkeletalMeshImportJson = nullptr;
	ImportSettingsJson->TryGetObjectField(TEXT("SkeletalMeshImportData"), SkeletalMeshImportJson);
	if (SkeletalMeshImportJson)
	{
		FJsonObjectConverter::JsonObjectToUStruct(SkeletalMeshImportJson->ToSharedRef(), SkeletalMeshImportData->GetClass(), SkeletalMeshImportData, 0, 0);
	}

	const TSharedPtr<FJsonObject>* AnimImportJson = nullptr;
	ImportSettingsJson->TryGetObjectField(TEXT("AnimSequenceImportData"), AnimImportJson);
	if (AnimImportJson)
	{
		FJsonObjectConverter::JsonObjectToUStruct(AnimImportJson->ToSharedRef(), AnimSequenceImportData->GetClass(), AnimSequenceImportData, 0, 0);
	}

	const TSharedPtr<FJsonObject>* TextureImportJson = nullptr;
	ImportSettingsJson->TryGetObjectField(TEXT("TextureImportData"), TextureImportJson);
	if (TextureImportJson)
	{
		FJsonObjectConverter::JsonObjectToUStruct(TextureImportJson->ToSharedRef(), TextureImportData->GetClass(), TextureImportData, 0, 0);
	}
}

void UFbxImportUI::ResetToDefault()
{
	ReloadConfig();
	AnimSequenceImportData->ReloadConfig();
	StaticMeshImportData->ReloadConfig();
	SkeletalMeshImportData->ReloadConfig();
	TextureImportData->ReloadConfig();
	
	//Make sure the UI do not display the Base Material
	TextureImportData->bUseBaseMaterial = false;
}

namespace ImportCompareHelper
{
	void SetHasConflict(FMaterialCompareData& MaterialCompareData)
	{
		MaterialCompareData.bHasConflict = false;
		for (const FMaterialData& ResultMaterial : MaterialCompareData.ResultAsset)
		{
			bool bFoundMatch = false;
			for (const FMaterialData& CurrentMaterial : MaterialCompareData.CurrentAsset)
			{
				if (ResultMaterial.ImportedMaterialSlotName == CurrentMaterial.ImportedMaterialSlotName)
				{
					bFoundMatch = true;
					break;
				}
			}
			if (!bFoundMatch)
			{
				MaterialCompareData.bHasConflict = true;
				break;
			}
		}
	}

	bool HasRemoveBoneRecursive(const FSkeletonTreeNode& ResultAssetRoot, const FSkeletonTreeNode& CurrentAssetRoot)
	{
		//Find the removed node
		for (const FSkeletonTreeNode& CurrentNode : CurrentAssetRoot.Childrens)
		{
			bool bFoundMatch = false;
			for (const FSkeletonTreeNode& ResultNode : ResultAssetRoot.Childrens)
			{
				if (ResultNode.JointName == CurrentNode.JointName)
				{
					bFoundMatch = !HasRemoveBoneRecursive(ResultNode, CurrentNode);
					break;
				}
			}
			if (!bFoundMatch)
			{
				return true;
			}
		}
		return false;
	}

	bool HasAddedBoneRecursive(const FSkeletonTreeNode& ResultAssetRoot, const FSkeletonTreeNode& CurrentAssetRoot)
	{
		//Find the added node
		for (const FSkeletonTreeNode& ResultNode : ResultAssetRoot.Childrens)
		{
			bool bFoundMatch = false;
			for (const FSkeletonTreeNode& CurrentNode : CurrentAssetRoot.Childrens)
			{
				if (ResultNode.JointName == CurrentNode.JointName)
				{
					bFoundMatch = !HasAddedBoneRecursive(ResultNode, CurrentNode);
					break;
				}
			}
			if (!bFoundMatch)
			{
				return true;
			}
		}
		return false;
	}

	void SetHasConflict(FSkeletonCompareData& SkeletonCompareData)
	{
		//Clear the skeleton Result
		SkeletonCompareData.CompareResult = ECompareResult::SCR_None;

		if (SkeletonCompareData.ResultAssetRoot.JointName != SkeletonCompareData.CurrentAssetRoot.JointName)
		{
			SkeletonCompareData.CompareResult = ECompareResult::SCR_SkeletonBadRoot;
			return;
		}

		if (HasRemoveBoneRecursive(SkeletonCompareData.ResultAssetRoot, SkeletonCompareData.CurrentAssetRoot))
		{
			SkeletonCompareData.CompareResult |= ECompareResult::SCR_SkeletonMissingBone;
		}

		if (HasAddedBoneRecursive(SkeletonCompareData.ResultAssetRoot, SkeletonCompareData.CurrentAssetRoot))
		{
			SkeletonCompareData.CompareResult |= ECompareResult::SCR_SkeletonAddedBone;
		}
	}

	void FillFbxMaterials(UnFbx::FFbxImporter* FFbxImporter, const TArray<FbxNode*>& MeshNodes, FMaterialCompareData& MaterialCompareData)
	{
		TArray<FName> NodeMaterialNames;
		for (int32 NodeIndex = 0; NodeIndex < MeshNodes.Num(); ++NodeIndex)
		{
			FbxNode* Node = MeshNodes[NodeIndex];
			if (Node->GetMesh() == nullptr)
			{
				continue;
			}

			int32 MaterialCount = Node->GetMaterialCount();
			TArray<int32> MaterialUseByMesh;
			FbxLayer* BaseLayer = Node->GetMesh()->GetLayer(0);
			FbxLayerElementMaterial* MateriallayerElement = BaseLayer->GetMaterials();
			FbxLayerElement::EMappingMode MaterialMappingMode = MateriallayerElement ? MateriallayerElement->GetMappingMode() : FbxLayerElement::eByPolygon;

			if (MaterialMappingMode == FbxLayerElement::eAllSame || MaterialCount == 0 || MateriallayerElement == nullptr)
			{
				MaterialUseByMesh.Add(0);
			}
			else
			{
				int32 PolygonCount = Node->GetMesh()->GetPolygonCount();
				for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
				{
					MaterialUseByMesh.AddUnique(MateriallayerElement->GetIndexArray().GetAt(PolygonIndex));
				}
			}

			for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
			{
				//Skip unused mesh material
				if (!MaterialUseByMesh.Contains(MaterialIndex))
				{
					continue;
				}
				FbxSurfaceMaterial* SurfaceMaterial = Node->GetMaterial(MaterialIndex);
				FName SurfaceMaterialName = FName(*(FFbxImporter->MakeName(SurfaceMaterial->GetName())));
				if (!NodeMaterialNames.Contains(SurfaceMaterialName))
				{
					FMaterialData& MaterialData = MaterialCompareData.ResultAsset.AddDefaulted_GetRef();
					MaterialData.ImportedMaterialSlotName = SurfaceMaterialName;
					MaterialData.MaterialSlotName = SurfaceMaterialName;
					MaterialData.MaterialIndex = NodeMaterialNames.Add(SurfaceMaterialName);
				}
			}
		}
	}

	void FillRecursivelySkeleton(const FReferenceSkeleton& ReferenceSkeleton, int32 CurrentIndex, FSkeletonTreeNode& SkeletonTreeNode)
	{
		SkeletonTreeNode.JointName = ReferenceSkeleton.GetBoneName(CurrentIndex);
		const int32 NumBones = ReferenceSkeleton.GetNum();
		for (int32 ChildIndex = CurrentIndex + 1; ChildIndex < NumBones; ChildIndex++)
		{
			if (CurrentIndex == ReferenceSkeleton.GetParentIndex(ChildIndex))
			{
				//Skip virtual bones
				if (ReferenceSkeleton.GetRequiredVirtualBones().Contains(ChildIndex))
				{
					return;
				}
				FSkeletonTreeNode& ChildNode = SkeletonTreeNode.Childrens.AddDefaulted_GetRef();
				ChildNode.JointName = ReferenceSkeleton.GetBoneName(ChildIndex);
				FillRecursivelySkeleton(ReferenceSkeleton, ChildIndex, ChildNode);
			}
		}
	}

	void FillRecursivelySkeletonCompareData(const FbxNode *ParentNode, FSkeletonTreeNode& SkeletonTreeNode, const UnFbx::FFbxImporter* FFbxImporter)
	{
		SkeletonTreeNode.JointName = FName(*(FFbxImporter->MakeName(ParentNode->GetName())));
		for (int32 ChildIndex = 0; ChildIndex < ParentNode->GetChildCount(); ChildIndex++)
		{
			FSkeletonTreeNode& ChildNode = SkeletonTreeNode.Childrens.AddDefaulted_GetRef();
			FillRecursivelySkeletonCompareData(ParentNode->GetChild(ChildIndex), ChildNode, FFbxImporter);
		}
	}

	void FillFbxSkeleton(UnFbx::FFbxImporter* FFbxImporter, const TArray<FbxNode*>& SkeletalMeshNodes, FSkeletonCompareData& SkeletonCompareData)
	{
		TArray<FbxNode *> JointLinks;
		FbxNode* Link = NULL;
		if (SkeletalMeshNodes.Num() > 0)
		{
			bool bHasLOD = false;
			FbxNode* SkeletalMeshRootNode = SkeletalMeshNodes[0];
			if (SkeletalMeshRootNode->GetNodeAttribute() && SkeletalMeshRootNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				// Use the first LOD group node to build the skeleton
				SkeletalMeshRootNode = FFbxImporter->FindLODGroupNode(SkeletalMeshNodes[0], 0);
				bHasLOD = true;
			}

			if (SkeletalMeshRootNode->GetMesh())
			{
				if(SkeletalMeshRootNode->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) == 0)
				{
					Link = SkeletalMeshRootNode;
					FFbxImporter->RecursiveBuildSkeleton(FFbxImporter->GetRootSkeleton(Link), JointLinks);
				}
				else
				{
					TArray<FbxCluster*> ClusterArray;
					for (int32 i = 0; i < SkeletalMeshNodes.Num(); i++)
					{
						FbxMesh* FbxMesh = (i == 0 && bHasLOD) ? SkeletalMeshRootNode->GetMesh() : SkeletalMeshNodes[i]->GetMesh();
						if (!FbxMesh)
						{
							continue;
						}
						const int32 SkinDeformerCount = FbxMesh->GetDeformerCount(FbxDeformer::eSkin);
						for (int32 DeformerIndex = 0; DeformerIndex < SkinDeformerCount; DeformerIndex++)
						{
							FbxSkin* Skin = (FbxSkin*)FbxMesh->GetDeformer(DeformerIndex, FbxDeformer::eSkin);
							for (int32 ClusterIndex = 0; ClusterIndex < Skin->GetClusterCount(); ClusterIndex++)
							{
								ClusterArray.Add(Skin->GetCluster(ClusterIndex));
							}
						}
					}
					// recurse through skeleton and build ordered table
					FFbxImporter->BuildSkeletonSystem(ClusterArray, JointLinks);
				}
			}
			
			//Fill the Result skeleton data
			FillRecursivelySkeletonCompareData(JointLinks[0], SkeletonCompareData.ResultAssetRoot, FFbxImporter);
		}
	}

	void RecursiveAddMeshNode(UnFbx::FFbxImporter* FFbxImporter, FbxNode* ParentNode, TArray<FbxNode*>& FlattenMeshNodes)
	{
		if (ParentNode->GetMesh())
		{
			FlattenMeshNodes.Add(ParentNode);
		}
		else if (ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			//In case we have some LODs, just grab the LOD 0 meshes
			ParentNode = FFbxImporter->FindLODGroupNode(ParentNode, 0);
			if (ParentNode == nullptr)
			{
				return;
			}
			FlattenMeshNodes.Add(ParentNode);
		}

		for (int32 ChildIndex = 0; ChildIndex < ParentNode->GetChildCount(); ++ChildIndex)
		{
			RecursiveAddMeshNode(FFbxImporter, ParentNode->GetChild(ChildIndex), FlattenMeshNodes);
		}
	}

	void FillStaticMeshCompareData(UnFbx::FFbxImporter* FFbxImporter, UStaticMesh* StaticMesh, UFbxImportUI* ImportUI)
	{
		//Fill the currrent asset data
		ImportUI->MaterialCompareData.CurrentAsset.Reserve(StaticMesh->GetStaticMaterials().Num());
		for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++MaterialIndex)
		{
			const FStaticMaterial& Material = StaticMesh->GetStaticMaterials()[MaterialIndex];
			FMaterialData MaterialData;
			MaterialData.MaterialIndex = MaterialIndex;
			MaterialData.ImportedMaterialSlotName = Material.ImportedMaterialSlotName;
			MaterialData.MaterialSlotName = Material.MaterialSlotName;
			ImportUI->MaterialCompareData.CurrentAsset.Add(MaterialData);
		}

		//Find the array of nodes to re-import
		TArray<FbxNode*> FbxMeshArray;
		bool bImportStaticMeshLODs = ImportUI->StaticMeshImportData->bImportMeshLODs;
		bool bCombineMeshes = ImportUI->StaticMeshImportData->bCombineMeshes;
		bool bCombineMeshesLOD = false;
		TArray<TArray<FbxNode*>> FbxMeshesLod;
		FbxNode* Node = nullptr;
		
		if (bCombineMeshes && !bImportStaticMeshLODs)
		{
			FFbxImporter->FillFbxMeshArray(FFbxImporter->Scene->GetRootNode(), FbxMeshArray, FFbxImporter);
		}
		else
		{
			// count meshes in lod groups if we dont care about importing LODs
			bool bCountLODGroupMeshes = !bImportStaticMeshLODs && bCombineMeshes;
			int32 NumLODGroups = 0;
			FFbxImporter->GetFbxMeshCount(FFbxImporter->Scene->GetRootNode(), bCountLODGroupMeshes, NumLODGroups);
			// if there were LODs in the file, do not combine meshes even if requested
			if (bImportStaticMeshLODs && bCombineMeshes && NumLODGroups > 0)
			{
				TArray<FbxNode*> FbxLodGroups;
				FFbxImporter->FillFbxMeshAndLODGroupArray(FFbxImporter->Scene->GetRootNode(), FbxLodGroups, FbxMeshArray);
				FbxMeshesLod.Add(FbxMeshArray);
				for (FbxNode* LODGroup : FbxLodGroups)
				{
					if (LODGroup->GetNodeAttribute() && LODGroup->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup && LODGroup->GetChildCount() > 0)
					{
						for (int32 GroupLodIndex = 0; GroupLodIndex < LODGroup->GetChildCount() && GroupLodIndex < MAX_STATIC_MESH_LODS; ++GroupLodIndex)
						{
							TArray<FbxNode*> AllNodeInLod;
							FFbxImporter->FindAllLODGroupNode(AllNodeInLod, LODGroup, GroupLodIndex);
							if (AllNodeInLod.Num() > 0)
							{
								if (FbxMeshesLod.Num() <= GroupLodIndex)
								{
									FbxMeshesLod.Add(AllNodeInLod);
								}
								else
								{
									TArray<FbxNode*> &LODGroupArray = FbxMeshesLod[GroupLodIndex];
									for (FbxNode* NodeToAdd : AllNodeInLod)
									{
										LODGroupArray.Add(NodeToAdd);
									}
								}
							}
						}
					}
				}
				bCombineMeshesLOD = true;
				bCombineMeshes = false;
				//Set the first LOD
				FbxMeshArray = FbxMeshesLod[0];
			}
			else
			{
				FFbxImporter->FillFbxMeshArray(FFbxImporter->Scene->GetRootNode(), FbxMeshArray, FFbxImporter);
			}
		}
		
		// if there is only one mesh, use it without name checking 
		// (because the "Used As Full Name" option enables users name the Unreal mesh by themselves
		if (!bCombineMeshesLOD && FbxMeshArray.Num() == 1)
		{
			Node = FbxMeshArray[0];
		}
		else if (!bCombineMeshes && !bCombineMeshesLOD)
		{
			Node = FFbxImporter->GetMeshNodesFromName(StaticMesh->GetName(), FbxMeshArray);
		}

		// If there is no match it may be because an LOD group was imported where
		// the mesh name does not match the file name. This is actually the common case.
		if (!bCombineMeshesLOD && !Node && FbxMeshArray.IsValidIndex(0))
		{
			FbxNode* BaseLODNode = FbxMeshArray[0];

			FbxNode* NodeParent = BaseLODNode ? FFbxImporter->RecursiveFindParentLodGroup(BaseLODNode->GetParent()) : nullptr;
			if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				// Reimport the entire LOD chain.
				Node = BaseLODNode;
			}
		}

		TArray<FbxNode*> StaticMeshNodes;
		if (bCombineMeshesLOD)
		{
			//FindLOD 0 Material
			if (FbxMeshesLod.Num() > 0)
			{
				StaticMeshNodes = FbxMeshesLod[0];
			}
			//Import all LODs
			for (int32 LODIndex = 1; LODIndex < FbxMeshesLod.Num(); ++LODIndex)
			{
				if (FbxMeshesLod[LODIndex][0]->GetMesh() != nullptr)
				{
					StaticMeshNodes.Append(FbxMeshesLod[LODIndex]);
				}
			}
		}
		else if (Node)
		{
			FbxNode* NodeParent = FFbxImporter->RecursiveFindParentLodGroup(Node->GetParent());
			// if the Fbx mesh is a part of LODGroup, update LOD
			if (NodeParent && NodeParent->GetNodeAttribute() && NodeParent->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				TArray<FbxNode*> AllNodeInLod;
				FFbxImporter->FindAllLODGroupNode(AllNodeInLod, NodeParent, 0);
				if (AllNodeInLod.Num() > 0)
				{
					StaticMeshNodes.Append(AllNodeInLod);
				}
				//If we have a valid LOD group name we don't want to re-import LODs since they will be automatically generate by the LODGroup reduce settings
				if (bImportStaticMeshLODs && StaticMesh->GetLODGroup() == NAME_None)
				{
					// import LOD meshes
					for (int32 LODIndex = 1; LODIndex < NodeParent->GetChildCount(); LODIndex++)
					{
						AllNodeInLod.Empty();
						FFbxImporter->FindAllLODGroupNode(AllNodeInLod, NodeParent, LODIndex);
						if (AllNodeInLod.Num() > 0 && AllNodeInLod[0]->GetMesh() != nullptr)
						{
							StaticMeshNodes.Append(AllNodeInLod);
						}
					}
				}
			}
			else
			{
				StaticMeshNodes.Add(Node);
			}
		}
		else
		{
			StaticMeshNodes.Append(FbxMeshArray);
		}

		FillFbxMaterials(FFbxImporter, StaticMeshNodes, ImportUI->MaterialCompareData);
		//Compare the result and set the conflict status
		SetHasConflict(ImportUI->MaterialCompareData);
	}

	void FillSkeletalMeshCompareData(UnFbx::FFbxImporter* FFbxImporter, USkeletalMesh* SkeletalMesh, UFbxImportUI* ImportUI)
	{
		bool bImportGeoOnly = ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_Geometry;
		bool bImportSkinningOnly = ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_SkinningWeights;

		//Fill the fbx data, read the scene and found the skeletalmesh nodes
		FbxNode* SceneRoot = FFbxImporter->Scene->GetRootNode();
		TArray<TArray<FbxNode*>*> SkeletalMeshArray;
		FFbxImporter->FillFbxSkelMeshArrayInScene(SceneRoot, SkeletalMeshArray, false, (bImportGeoOnly || bImportSkinningOnly), false);
		if (SkeletalMeshArray.Num() == 0)
		{
			return;
		}

		TArray<FbxNode*>* FbxNodes = nullptr;
		// if there is only one mesh, use it without name checking 
		// (because the "Used As Full Name" option enables users name the Unreal mesh by themselves
		if (SkeletalMeshArray.Num() == 1)
		{
			FbxNodes = SkeletalMeshArray[0];
		}
		else if(SkeletalMeshArray.Num() > 1)
		{
			FbxNodes = nullptr;
			//Search a set corresponding
			for (TArray<FbxNode*>*SkeletalMeshNodes : SkeletalMeshArray)
			{
				FbxNode* Node = FFbxImporter->GetMeshNodesFromName(SkeletalMesh->GetName(), *SkeletalMeshNodes);
				if (Node != nullptr)
				{
					FbxNodes = SkeletalMeshNodes;
					break;
				}
			}
			if (FbxNodes == nullptr)
			{
				//Re import the first skeletalmesh found in the fbx file
				FbxNodes = SkeletalMeshArray[0];
			}
		}
		if (FbxNodes == nullptr)
		{
			return;
		}

		const TArray<FbxNode*>& SkeletalMeshNodes = *(FbxNodes);
		if (SkeletalMeshNodes.Num() == 0)
		{
			return;
		}

		//Materials
		if (!bImportSkinningOnly)
		{
			//Fill the currrent asset data
			const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = SkeletalMesh->GetMaterials();
			ImportUI->MaterialCompareData.CurrentAsset.Reserve(SkeletalMeshMaterials.Num());
			for (int32 MaterialIndex = 0; MaterialIndex < SkeletalMeshMaterials.Num(); ++MaterialIndex)
			{
				const FSkeletalMaterial& Material = SkeletalMeshMaterials[MaterialIndex];
				FMaterialData MaterialData;
				MaterialData.MaterialIndex = MaterialIndex;
				MaterialData.ImportedMaterialSlotName = Material.ImportedMaterialSlotName;
				MaterialData.MaterialSlotName = Material.MaterialSlotName;
				ImportUI->MaterialCompareData.CurrentAsset.Add(MaterialData);
			}

			TArray<FbxNode*> FlattenSkeletalMeshNodes;
			for (FbxNode* SkeletalMeshRootNode : SkeletalMeshNodes)
			{
				RecursiveAddMeshNode(FFbxImporter, SkeletalMeshRootNode, FlattenSkeletalMeshNodes);
			}

			//Fill the result fbx data
			FillFbxMaterials(FFbxImporter, FlattenSkeletalMeshNodes, ImportUI->MaterialCompareData);

			//Compare the result and set the conflict status
			SetHasConflict(ImportUI->MaterialCompareData);
		}

		//Skeleton joints
		if (!bImportGeoOnly)
		{
			//Fill the currrent asset data
			if (ImportUI->Skeleton && SkeletalMesh->GetSkeleton() != ImportUI->Skeleton)
			{
				//In this case we can't use 
				const FReferenceSkeleton& ReferenceSkeleton = ImportUI->Skeleton->GetReferenceSkeleton();
				FillRecursivelySkeleton(ReferenceSkeleton, 0, ImportUI->SkeletonCompareData.CurrentAssetRoot);
			}
			else
			{
				FillRecursivelySkeleton(SkeletalMesh->GetRefSkeleton(), 0, ImportUI->SkeletonCompareData.CurrentAssetRoot);
			}
			
			//Fill the result fbx data
			FillFbxSkeleton(FFbxImporter, SkeletalMeshNodes, ImportUI->SkeletonCompareData);

			//Compare the result and set the conflict status
			SetHasConflict(ImportUI->SkeletonCompareData);
		}
	}

} //END namespace ImportCompareHelper

void UFbxImportUI::UpdateCompareData(UnFbx::FFbxImporter* FbxImporter)
{
	if (ReimportMesh == nullptr)
	{
		return;
	}
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(ReimportMesh);
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ReimportMesh);
	
	MaterialCompareData.Empty();
	SkeletonCompareData.Empty();

	FString Filename;
	if (StaticMesh != nullptr)
	{
		Filename = StaticMeshImportData->GetFirstFilename();
	}
	else
	{
		FString FilenameLabel;
		SkeletalMeshImportData->GetImportContentFilename(Filename, FilenameLabel);
	}
	
	if (!FbxImporter->ImportFromFile(*Filename, FPaths::GetExtension(Filename), false))
	{
		return;
	}
	
	
	if (StaticMesh)
	{
		ImportCompareHelper::FillStaticMeshCompareData(FbxImporter, StaticMesh, this);
	}
	else if (SkeletalMesh)
	{
		ImportCompareHelper::FillSkeletalMeshCompareData(FbxImporter, SkeletalMesh, this);
	}
	FbxImporter->PartialCleanUp();
}

/** Load UI settings from ini file */
void UFbxImportUI::LoadOptions(UObject* ObjectToLoadOptions)
{
	check(ObjectToLoadOptions);
	int32 PortFlags = 0;
	UClass* Class = ObjectToLoadOptions->GetClass();
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}
		FString Section = TEXT("FBX_Import_UI_Option_") + Class->GetName();
		FString Key = Property->GetName();

		const bool bIsPropertyInherited = Property->GetOwnerClass() != Class;
		UObject* SuperClassDefaultObject = Class->GetSuperClass()->GetDefaultObject();

		const FString& PropFileName = GEditorPerProjectIni;

		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			const FConfigSection* Sec = GConfig->GetSection(*Section, 0, *GEditorPerProjectIni);
			if (Sec != nullptr)
			{
				TArray<FConfigValue> List;
				const FName KeyName(*Key, FNAME_Find);
				Sec->MultiFind(KeyName, List);

				FScriptArrayHelper_InContainer ArrayHelper(Array, ObjectToLoadOptions);
				// Only override default properties if there is something to override them with.
				if (List.Num() > 0)
				{
					ArrayHelper.EmptyAndAddValues(List.Num());
					for (int32 i = List.Num() - 1, c = 0; i >= 0; i--, c++)
					{
						Array->Inner->ImportText_Direct(*List[i].GetValue(), ArrayHelper.GetRawPtr(c), ObjectToLoadOptions, PortFlags);
					}
				}
				else
				{
					int32 Index = 0;
					const FConfigValue* ElementValue = nullptr;
					do
					{
						// Add array index number to end of key
						FString IndexedKey = FString::Printf(TEXT("%s[%i]"), *Key, Index);

						// Try to find value of key
						const FName IndexedName(*IndexedKey, FNAME_Find);
						if (IndexedName == NAME_None)
						{
							break;
						}
						ElementValue = Sec->Find(IndexedName);

						// If found, import the element
						if (ElementValue != nullptr)
						{
							// expand the array if necessary so that Index is a valid element
							ArrayHelper.ExpandForIndex(Index);
							Array->Inner->ImportText_Direct(*ElementValue->GetValue(), ArrayHelper.GetRawPtr(Index), ObjectToLoadOptions, PortFlags);
						}

						Index++;
					} while (ElementValue || Index < ArrayHelper.Num());
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Property->ArrayDim; i++)
			{
				if (Property->ArrayDim != 1)
				{
					Key = FString::Printf(TEXT("%s[%i]"), *Property->GetName(), i);
				}

				FString Value;
				bool bFoundValue = GConfig->GetString(*Section, *Key, Value, *GEditorPerProjectIni);

				if (bFoundValue)
				{
					if (Property->ImportText_Direct(*Value, Property->ContainerPtrToValuePtr<uint8>(ObjectToLoadOptions, i), ObjectToLoadOptions, PortFlags) == NULL)
					{
						// this should be an error as the properties from the .ini / .int file are not correctly being read in and probably are affecting things in subtle ways
						UE_LOGF(LogFbx, Error, "FBX Options LoadOptions (%ls): import failed for %ls in: %ls", *ObjectToLoadOptions->GetPathName(), *Property->GetName(), *Value);
					}
				}
			}
		}
	}
}

/** Save UI settings to ini file */
void UFbxImportUI::SaveOptions(UObject* ObjectToSaveOptions)
{
	check(ObjectToSaveOptions);
	int32 PortFlags = 0;
	UClass* Class = ObjectToSaveOptions->GetClass();
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}
		FString Section = TEXT("FBX_Import_UI_Option_") + Class->GetName();
		FString Key = Property->GetName();

		const bool bIsPropertyInherited = Property->GetOwnerClass() != Class;
		UObject* SuperClassDefaultObject = Class->GetSuperClass()->GetDefaultObject();

		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			GConfig->RemoveKeyFromSection(*Section, *Key, GEditorPerProjectIni);

			FScriptArrayHelper_InContainer ArrayHelper(Array, ObjectToSaveOptions);
			for (int32 i = 0; i < ArrayHelper.Num(); i++)
			{
				FString	Buffer;
				Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), ObjectToSaveOptions, PortFlags);
				GConfig->AddToSection(*Section, *Key, Buffer, GEditorPerProjectIni);
			}
		}
		else
		{
			TCHAR TempKey[MAX_SPRINTF] = TEXT("");
			for (int32 Index = 0; Index < Property->ArrayDim; Index++)
			{
				if (Property->ArrayDim != 1)
				{
					FCString::Sprintf(TempKey, TEXT("%s[%i]"), *Property->GetName(), Index);
					Key = TempKey;
				}

				FString	Value;
				Property->ExportText_InContainer(Index, Value, ObjectToSaveOptions, ObjectToSaveOptions, ObjectToSaveOptions, PortFlags);
				GConfig->SetString(*Section, *Key, *Value, *GEditorPerProjectIni);
			}
		}
	}
	GConfig->Flush(0);
}

#undef LOCTEXT_NAMESPACE
