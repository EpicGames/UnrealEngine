// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Light actor creation from FBX data.
=============================================================================*/

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/Light.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/SpotLight.h"
#include "FbxImporter.h"

DEFINE_LOG_CATEGORY_STATIC(LogFbxLightImport, Log, All);

using namespace UnFbx;

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
ALight* FFbxImporter::CreateLight(FbxLight* InLight, UWorld* InWorld)
{
	ALight* UnrealLight = NULL;
	FString ActorName = MakeName(InLight->GetName());

	// Create the light actor.
	UClass* LightClass = NULL;
	switch (InLight->LightType.Get())
	{
	case FbxLight::ePoint:
		LightClass = APointLight::StaticClass();
		break;
	case FbxLight::eDirectional:
		LightClass = ADirectionalLight::StaticClass();
		break;
	case FbxLight::eSpot:
		LightClass = ASpotLight::StaticClass();
		break;
	}
	// If we have a valid class create the light.
	if( LightClass )
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.Name = *ActorName;
		UnrealLight = InWorld->SpawnActor<ALight>( LightClass, SpawnInfo );
		if (UnrealLight)
		{
			FillLightComponent(InLight,UnrealLight->GetLightComponent());	
		}
		else
		{
			UE_LOGF(LogFbxLightImport, Warning,"Failed to create light type %s ", InLight->GetName() );
		}
	}
	else
	{
		UE_LOGF(LogFbxLightImport, Warning,"Invalid light type %s ", InLight->GetName() );
	}
	
	return UnrealLight;
}

bool FFbxImporter::FillLightComponent(FbxLight* Light, ULightComponent* UnrealLightComponent)
{
	UnrealLightComponent->SetMobility(EComponentMobility::Movable);

	FbxDouble3 Color = Light->Color.Get();
	FColor UnrealColor( uint8(255.0*Color[0]), uint8(255.0*Color[1]), uint8(255.0*Color[2]) );
	UnrealLightComponent->LightColor = UnrealColor;

	FbxDouble Intensity = Light->Intensity.Get();
	UnrealLightComponent->Intensity = (float)Intensity;
	UnrealLightComponent->CastShadows = Light->CastShadows.Get();

	switch (Light->LightType.Get())
	{
	// point light properties
	case FbxLight::ePoint:
		{
			UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(UnrealLightComponent);
			if (PointLightComponent)
			{
				FbxDouble DecayStart = Light->DecayStart.Get();
				PointLightComponent->AttenuationRadius = Converter.ConvertDist(DecayStart);

				FbxLight::EDecayType Decay = Light->DecayType.Get();
				if (Decay == FbxLight::eNone)
				{
					PointLightComponent->AttenuationRadius = FBXSDK_FLOAT_MAX;
				}
			}
			else
			{
				UE_LOGF(LogFbxLightImport, Error,"FBX Light type 'Point' does not match unreal light component");
			}
		}
		break;
	// spot light properties
	case FbxLight::eSpot:
		{
			USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(UnrealLightComponent);
			if (SpotLightComponent)
			{
				FbxDouble DecayStart = Light->DecayStart.Get();
				SpotLightComponent->AttenuationRadius = Converter.ConvertDist(DecayStart);
				FbxLight::EDecayType Decay = Light->DecayType.Get();
				if (Decay == FbxLight::eNone)
				{
					SpotLightComponent->AttenuationRadius = FBXSDK_FLOAT_MAX;
				}
				SpotLightComponent->InnerConeAngle = Light->InnerAngle.Get();
				SpotLightComponent->OuterConeAngle = Light->OuterAngle.Get();
			}
			else
			{
				UE_LOGF(LogFbxLightImport, Error,"FBX Light type 'Spot' does not match unreal light component");
			}
		}
		break;
	// directional light properties 
	case FbxLight::eDirectional:
		{
			// nothing specific
		}
		break;
	}

	return true;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
ACameraActor* FFbxImporter::CreateCamera(FbxCamera* InCamera, UWorld* InWorld)
{
	ACameraActor* UnrealCamera = NULL;
	FString ActorName = MakeName(InCamera->GetName());
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.Name = *ActorName;
	UnrealCamera = InWorld->SpawnActor<ACameraActor>( SpawnInfo );
	if (UnrealCamera)
	{
		UnrealCamera->GetCameraComponent()->FieldOfView = InCamera->FieldOfView.Get();
	}
	return UnrealCamera;
}
