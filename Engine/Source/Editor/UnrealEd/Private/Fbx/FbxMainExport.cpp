// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Main implementation of FFbxExporter : export FBX data from Unreal
=============================================================================*/

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Misc/Guid.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Misc/App.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Blueprint.h"
#include "RawIndexBuffer.h"
#include "CinematicExporter.h"
#include "Components/StaticMeshComponent.h"
#include "Components/LightComponent.h"
#include "Model.h"

#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "Channels/MovieSceneByteChannel.h"

#include "Animation/AnimTypes.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/Brush.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Particles/Emitter.h"
#include "Components/PointLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Components/ChildActorComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/Polys.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Editor.h"
#include "Channels/MovieSceneChannelProxy.h"

#include "Materials/Material.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialInstance.h"

#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "Components/SplineMeshComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"

#include "FbxExporter.h"
#include "Exporters/FbxExportOption.h"
#include "FbxExportOptionsWindow.h"
#include "FbxAnimUtils.h"
#include "INodeAndChannelMappings.h"

#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include "Components/BrushComponent.h"
#include "CineCameraComponent.h"
#include "Math/UnitConversion.h"

#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieSceneColorSection.h"
#include "Sections/MovieSceneDoubleSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "DynamicMeshBuilder.h"

#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/SphereElem.h"

#include "Chaos/Core.h"
#include "Chaos/Particles.h"
#include "Chaos/Plane.h"
#include "ChaosCheck.h"
#include "Animation/AnimationSettings.h"
#include "Chaos/Convex.h"

#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "UObject/MetaData.h"

#include "FbxMaterialExportUtilities.h"

#include "Exporters/Exporter.h"
#include "MaterialPropertyEx.h"

namespace UnFbx
{

TSharedPtr<FFbxExporter> FFbxExporter::StaticInstance;

FFbxExporter::FFbxExporter()
{
	bBakeKeys = true;
	bKeepHierarchy = true;

	//We use the FGCObject pattern to keep the fbx export option alive during the editor session
	ExportOptionsUI = NewObject<UFbxExportOption>();
	//Load the option from the user save ini file
	ExportOptionsUI->LoadOptions();

	ExportOptionsOverride = nullptr;

	// Create the SdkManager
	SdkManager = FbxManager::Create();

	// create an IOSettings object
	FbxIOSettings * ios = FbxIOSettings::Create(SdkManager, IOSROOT );
	SdkManager->SetIOSettings(ios);

	DefaultCamera = NULL;
}

FFbxExporter::~FFbxExporter()
{
	if (SdkManager)
	{
		SdkManager->Destroy();
		SdkManager = NULL;
	}
}

FFbxExporter* FFbxExporter::GetInstance()
{
	if (!StaticInstance.IsValid())
	{
		StaticInstance = MakeShareable( new FFbxExporter() );
	}
	return StaticInstance.Get();
}

void FFbxExporter::DeleteInstance()
{
	StaticInstance.Reset();
}

void FFbxExporter::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (ExportOptionsUI != nullptr)
	{
		Collector.AddReferencedObject(ExportOptionsUI);
	}

	if (ExportOptionsOverride != nullptr)
	{
		Collector.AddReferencedObject(ExportOptionsOverride);
	}
}

void FFbxExporter::FillExportOptions(bool BatchMode, bool bShowOptionDialog, const FString& FullPath, bool& OutOperationCanceled, bool& bOutExportAll)
{
	OutOperationCanceled = false;
	
	//Export option should have been set in the constructor
	check(ExportOptionsUI != nullptr);
	
	//Load the option from the user save ini file
	ExportOptionsUI->LoadOptions();
	
	//Return if we do not show the export options or we are running automation test or we are unattended
	if (!bShowOptionDialog || GIsAutomationTesting || FApp::IsUnattended())
	{
		return;
	}

	bOutExportAll = false;

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(NSLOCTEXT("UnrealEd", "FBXExportOpionsTitle", "FBX Export Options"))
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.ClientSize(FVector2D(500, 445));

	TSharedPtr<SFbxExportOptionsWindow> FbxOptionWindow;
	Window->SetContent
	(
		SAssignNew(FbxOptionWindow, SFbxExportOptionsWindow)
		.ExportOptions(ExportOptionsUI)
		.WidgetWindow(Window)
		.FullPath(FText::FromString(FullPath))
		.BatchMode(BatchMode)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	//Respect the edit condition
	if(ExportOptionsUI->bExportSourceMesh)
	{
		ExportOptionsUI->Collision = false;
		ExportOptionsUI->LevelOfDetail = false;
	}

	ExportOptionsUI->SaveOptions();

	if (FbxOptionWindow->ShouldExport())
	{
		bOutExportAll = FbxOptionWindow->ShouldExportAll();
	}
	else
	{
		OutOperationCanceled = true;
	}
}

void FFbxExporter::SetExportOptionsOverride(UFbxExportOption* OverrideOptions)
{
	ExportOptionsOverride = OverrideOptions;
}

UFbxExportOption* FFbxExporter::GetExportOptions()
{
	return ExportOptionsOverride ? ExportOptionsOverride : ExportOptionsUI;
}

void FFbxExporter::CreateDocument()
{
	Scene = FbxScene::Create(SdkManager,"");
	
	// create scene info
	FbxDocumentInfo* SceneInfo = FbxDocumentInfo::Create(SdkManager,"SceneInfo");
	SceneInfo->mTitle = "Unreal FBX Exporter";
	SceneInfo->mSubject = "Export FBX meshes from Unreal";
	SceneInfo->Original_ApplicationVendor.Set( "Epic Games" );
	SceneInfo->Original_ApplicationName.Set( "Unreal Engine" );
	SceneInfo->Original_ApplicationVersion.Set( TCHAR_TO_UTF8(*FEngineVersion::Current().ToString()) );
	SceneInfo->LastSaved_ApplicationVendor.Set( "Epic Games" );
	SceneInfo->LastSaved_ApplicationName.Set( "Unreal Engine" );
	SceneInfo->LastSaved_ApplicationVersion.Set( TCHAR_TO_UTF8(*FEngineVersion::Current().ToString()) );

	Scene->SetSceneInfo(SceneInfo);
	
	//FbxScene->GetGlobalSettings().SetOriginalUpAxis(KFbxAxisSystem::Max);
	FbxAxisSystem::EFrontVector FrontVector = (FbxAxisSystem::EFrontVector)-FbxAxisSystem::eParityOdd;
	if (GetExportOptions()->bForceFrontXAxis)
		FrontVector = FbxAxisSystem::eParityEven;

	const FbxAxisSystem UnrealZUp(FbxAxisSystem::eZAxis, FrontVector, FbxAxisSystem::eRightHanded);
	Scene->GetGlobalSettings().SetAxisSystem(UnrealZUp);
	Scene->GetGlobalSettings().SetOriginalUpAxis(UnrealZUp);
	// Maya use cm by default
	Scene->GetGlobalSettings().SetSystemUnit(FbxSystemUnit::cm);
	//FbxScene->GetGlobalSettings().SetOriginalSystemUnit( KFbxSystemUnit::m );

	bSceneGlobalTimeLineSet = false;
	Scene->GetGlobalSettings().SetTimeMode(FbxTime::eDefaultMode);
	
	// setup anim stack
	AnimStack = FbxAnimStack::Create(Scene, "Unreal Take");
	//KFbxSet<KTime>(AnimStack->LocalStart, KTIME_ONE_SECOND);
	AnimStack->Description.Set("Animation Take for Unreal.");

	// this take contains one base layer. In fact having at least one layer is mandatory.
	AnimLayer = FbxAnimLayer::Create(Scene, "Base Layer");
	AnimStack->AddMember(AnimLayer);
}

#ifdef IOS_REF
#undef  IOS_REF
#define IOS_REF (*(SdkManager->GetIOSettings()))
#endif

void FFbxExporter::WriteToFile(const TCHAR* Filename)
{
	int32 Major, Minor, Revision;
	bool Status = true;

	int32 FileFormat = -1;
	bool bEmbedMedia = false;
	bool bASCII = GetExportOptions()->bASCII;

	// Create an exporter.
	FbxExporter* Exporter = FbxExporter::Create(SdkManager, "");

	// set file format
	// Write in fall back format if pEmbedMedia is true
	if (bASCII)
	{
		FileFormat = SdkManager->GetIOPluginRegistry()->FindWriterIDByDescription("FBX ascii (*.fbx)");
	}
	else
	{
		FileFormat = SdkManager->GetIOPluginRegistry()->GetNativeWriterFormat();
	}

	// Set the export states. By default, the export states are always set to 
	// true except for the option eEXPORT_TEXTURE_AS_EMBEDDED. The code below 
	// shows how to change these states.

	IOS_REF.SetBoolProp(EXP_FBX_MATERIAL,        true);
	IOS_REF.SetBoolProp(EXP_FBX_TEXTURE,         true);
	IOS_REF.SetBoolProp(EXP_FBX_EMBEDDED,        bEmbedMedia);
	IOS_REF.SetBoolProp(EXP_FBX_SHAPE,           true);
	IOS_REF.SetBoolProp(EXP_FBX_GOBO,            true);
	IOS_REF.SetBoolProp(EXP_FBX_ANIMATION,       true);
	IOS_REF.SetBoolProp(EXP_FBX_GLOBAL_SETTINGS, true);
	IOS_REF.SetBoolProp(EXP_ASCIIFBX,            bASCII);

	//Get the compatibility from the editor settings
	const char* CompatibilitySetting = FBX_2013_00_COMPATIBLE;
	const EFbxExportCompatibility FbxExportCompatibility = GetExportOptions()->FbxExportCompatibility;
	switch (FbxExportCompatibility)
	{
		case EFbxExportCompatibility::FBX_2011:
			CompatibilitySetting = FBX_2011_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2012:
			CompatibilitySetting = FBX_2012_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2013:
			CompatibilitySetting = FBX_2013_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2014:
			CompatibilitySetting = FBX_2014_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2016:
			CompatibilitySetting = FBX_2016_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2018:
			CompatibilitySetting = FBX_2018_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2019:
			CompatibilitySetting = FBX_2019_00_COMPATIBLE;
			break;
		case EFbxExportCompatibility::FBX_2020:
			CompatibilitySetting = FBX_2020_00_COMPATIBLE;
			break;
		default:
			CompatibilitySetting = FBX_2013_00_COMPATIBLE;
			break;
	}
	
	// We export using FBX 2013 format because many users are still on that version and FBX 2014 files has compatibility issues with
	// normals when importing to an earlier version of the plugin
	if (!Exporter->SetFileExportVersion(CompatibilitySetting, FbxSceneRenamer::eNone))
	{
		UE_LOGF(LogFbx, Warning, "Call to KFbxExporter::SetFileExportVersion(FBX_2013_00_COMPATIBLE) to export 2013 fbx file format failed.\n");
	}

	// Initialize the exporter by providing a filename.
	if( !Exporter->Initialize(TCHAR_TO_UTF8(Filename), FileFormat, SdkManager->GetIOSettings()) )
	{
		UE_LOGF(LogFbx, Warning, "Call to KFbxExporter::Initialize() failed.\n");
		UE_LOGF(LogFbx, Warning, "Error returned: %s\n\n", Exporter->GetStatus().GetErrorString() );
		return;
	}

	FbxManager::GetFileFormatVersion(Major, Minor, Revision);
	UE_LOGF(LogFbx, Log, "FBX version number for this version of the FBX SDK is %d.%d.%d\n\n", Major, Minor, Revision);

	// Export the scene.
	Status = Exporter->Export(Scene); 

	// Destroy the exporter.
	Exporter->Destroy();
	
	CloseDocument();
	
	return;
}

/**
 * Release the FBX scene, releasing its memory.
 */
void FFbxExporter::CloseDocument()
{
	FbxActors.Reset();
	FbxSkeletonRoots.Reset();
	FbxMaterials.Reset();
	FbxMeshes.Reset();
	FbxCollisionMeshes.Reset();
	FbxNodeNameToIndexMap.Reset();
	
	if (Scene)
	{
		Scene->Destroy();
		Scene = NULL;
	}
	bSceneGlobalTimeLineSet = false;
}

template <typename T> 
void FFbxExporter::CreateAnimatableUserProperty(FbxNode* Node, T Value, const char* Name, const char* Label, FbxDataType DataType)
{
	// Add one user property for recording the animation
	FbxProperty UserProp = FbxProperty::Create(Node, DataType, Name, Label);
	UserProp.Set(Value);
	UserProp.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
	UserProp.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
}



/**
*	Sorts actors such that parent actors will appear before children actors in the list
*	Stable sort
*/
static void SortActorsHierarchy(TArray<AActor*>& Actors)
{
	auto CalcAttachDepth = [](AActor* InActor) -> int32 {
		int32 Depth = MAX_int32;
		if (InActor)
		{
			Depth = 0;
			if (InActor->GetRootComponent())
			{
				for (const USceneComponent* Test = InActor->GetRootComponent()->GetAttachParent(); Test != nullptr; Test = Test->GetAttachParent(), Depth++);
			}
		}
		return Depth;
	};

	// Unfortunately TArray.StableSort assumes no null entries in the array
	// So it forces me to use internal unrestricted version
	StableSortInternal(Actors.GetData(), Actors.Num(), [&](AActor* L, AActor* R) {
		return CalcAttachDepth(L) < CalcAttachDepth(R);
	});
}


/**
 * Exports the basic scene information to the FBX document.
 */
void FFbxExporter::ExportLevelMesh(ULevel* InLevel, bool bExportLevelGeometry, TArray<AActor*>& ActorToExport, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq)
{
	if (InLevel == NULL)
	{
		return;
	}

	if (bExportLevelGeometry)
	{
		// Exports the level's scene geometry
		// the vertex number of Model must be more than 2 (at least a triangle panel)
		if (InLevel->Model != NULL && InLevel->Model->VertexBuffer.Vertices.Num() > 2 && InLevel->Model->MaterialIndexBuffers.Num() > 0)
		{
			// create a FbxNode
			FbxNode* Node = FbxNode::Create(Scene, "LevelMesh");

			// set the shading mode to view texture
			Node->SetShadingMode(FbxNode::eTextureShading);
			Node->LclScaling.Set(FbxVector4(1.0, 1.0, 1.0));

			Scene->GetRootNode()->AddChild(Node);

			// Export the mesh for the world
			ExportModel(InLevel->Model, Node, "Level Mesh", FFbxMaterialBakingMeshData(InLevel->Model));
		}
	}

	//Sort the hierarchy to make sure parent come first
	SortActorsHierarchy(ActorToExport);
	int32 ActorCount = ActorToExport.Num();
	for (int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex)
	{
		AActor* Actor = ActorToExport[ActorIndex];
		//We export only valid actor
		if (Actor == nullptr)
			continue;

		bool bIsBlueprintClass = false;
		if (UClass* ActorClass = Actor->GetClass())
		{
			// Check if we export the actor as a blueprint
			bIsBlueprintClass = UBlueprint::GetBlueprintFromClass(ActorClass) != nullptr;
		}

		//Blueprint can be any type of actor so it must be done first
		if (bIsBlueprintClass)
		{
			// Export blueprint actors and all their components.
			ExportActor(Actor, true, NodeNameAdapter, bSaveAnimSeq);
		}
		else if (Actor->IsA(ALight::StaticClass()))
		{
			ExportLight((ALight*)Actor, NodeNameAdapter);
		}
		else if (Actor->IsA(AStaticMeshActor::StaticClass()))
		{
			ExportStaticMesh(Actor, CastChecked<AStaticMeshActor>(Actor)->GetStaticMeshComponent(), NodeNameAdapter);
		}
		else if (Actor->IsA(ALandscapeProxy::StaticClass()))
		{
			ExportLandscape(CastChecked<ALandscapeProxy>(Actor), false, NodeNameAdapter);
		}
		else if (Actor->IsA(ABrush::StaticClass()))
		{
			// All brushes should be included within the world geometry exported above.
			ExportBrush((ABrush*)Actor, NULL, 0, NodeNameAdapter);
		}
		else if (Actor->IsA(AEmitter::StaticClass()))
		{
			ExportActor(Actor, false, NodeNameAdapter, bSaveAnimSeq); // Just export the placement of the particle emitter.
		}
		else if (Actor->IsA(ACameraActor::StaticClass()))
		{
			ExportCamera(CastChecked<ACameraActor>(Actor), false, NodeNameAdapter);
		}
		else
		{
			// Export any other type of actors and all their components.
			ExportActor(Actor, true, NodeNameAdapter, bSaveAnimSeq);
		}
	}
}

/**
 * Exports the basic scene information to the FBX document.
 */
void FFbxExporter::ExportLevelMesh( ULevel* InLevel, bool bSelectedOnly, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq)
{
	if (InLevel == NULL)
	{
		return;
	}
	
	TArray<AActor*> ActorToExport;
	int32 ActorCount = InLevel->Actors.Num();
	for (int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex)
	{
		AActor* Actor = InLevel->Actors[ActorIndex];
		if (Actor != NULL && (!bSelectedOnly || (bSelectedOnly && Actor->IsSelected())))
		{
			ActorToExport.Add(Actor);
		}
	}

	ExportLevelMesh(InLevel, !bSelectedOnly, ActorToExport, NodeNameAdapter, bSaveAnimSeq);
}

void FFbxExporter::FillFbxLightAttribute(FbxLight* Light, FbxNode* FbxParentNode, ULightComponent* BaseLight)
{
	Light->Intensity.Set(BaseLight->Intensity);
	Light->Color.Set(Converter.ConvertToFbxColor(BaseLight->LightColor));

	// Add one user property for recording the Brightness animation
	CreateAnimatableUserProperty(FbxParentNode, BaseLight->Intensity, "UE_Intensity", "UE_Matinee_Light_Intensity");
	CreateAnimatableUserProperty(FbxParentNode, BaseLight->bUseTemperature, "UE_UseTemperature", "UE_Matinee_Light_UseTemperature", FbxBoolDT);
	CreateAnimatableUserProperty(FbxParentNode, BaseLight->GetLightUnits(), "UE_IntensityUnits", "UE_Matinee_UE_IntensityUnits", FbxEnumDT);

	// Look for the higher-level light types and determine the lighting method
	if (BaseLight->IsA(UPointLightComponent::StaticClass()))
	{
		UPointLightComponent* PointLight = (UPointLightComponent*)BaseLight;
		if (BaseLight->IsA(USpotLightComponent::StaticClass()))
		{
			USpotLightComponent* SpotLight = (USpotLightComponent*)BaseLight;
			Light->LightType.Set(FbxLight::eSpot);

			// Export the spot light parameters.
			if (!FMath::IsNearlyZero(SpotLight->InnerConeAngle*2.0f))
			{
				Light->InnerAngle.Set(SpotLight->InnerConeAngle*2.0f);
			}
			else // Maya requires a non-zero inner cone angle
			{
				Light->InnerAngle.Set(0.01f);
			}
			Light->OuterAngle.Set(SpotLight->OuterConeAngle*2.0f);
		}
		else
		{
			Light->LightType.Set(FbxLight::ePoint);
		}

		// Export the point light parameters.
		Light->EnableFarAttenuation.Set(true);
		Light->FarAttenuationEnd.Set(PointLight->AttenuationRadius);
		// Add one user property for recording the FalloffExponent animation
		CreateAnimatableUserProperty(FbxParentNode, PointLight->AttenuationRadius, "UE_Radius", "UE_Matinee_Light_Radius");

		// Add one user property for recording the FalloffExponent animation
		CreateAnimatableUserProperty(FbxParentNode, PointLight->LightFalloffExponent, "UE_FalloffExponent", "UE_Matinee_Light_FalloffExponent");
	}
	else if (BaseLight->IsA(UDirectionalLightComponent::StaticClass()))
	{
		// The directional light has no interesting properties.
		Light->LightType.Set(FbxLight::eDirectional);
		Light->Intensity.Set(BaseLight->Intensity*100.0f);
	}
}

/**
 * Exports the light-specific information for a light actor.
 */
void FFbxExporter::ExportLight( ALight* Actor, INodeNameAdapter& NodeNameAdapter )
{
	if (Scene == NULL || Actor == NULL || !Actor->GetLightComponent()) return;

	// Export the basic actor information.
	FbxNode* FbxActor = ExportActor( Actor, false, NodeNameAdapter ); // this is the pivot node
	// The real fbx light node
	FbxNode* FbxLightNode = FbxActor->GetParent();

	ULightComponent* BaseLight = Actor->GetLightComponent();

	FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);

	// Export the basic light information
	FbxLight* Light = FbxLight::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));
	FillFbxLightAttribute(Light, FbxLightNode, BaseLight);	
	FbxActor->SetNodeAttribute(Light);
}

void FFbxExporter::FillFbxCameraAttribute(FbxNode* ParentNode, FbxCamera* Camera, UCameraComponent *CameraComponent)
{
	float ApertureHeightInInches = 0.612f; // 0.612f is a magic number from Maya that represents the ApertureHeight
	float ApertureWidthInInches = CameraComponent->AspectRatio * ApertureHeightInInches;
	float FocalLength = Camera->ComputeFocalLength(CameraComponent->FieldOfView);
	
	TOptional<float> FocusDistance;

	if (CameraComponent->IsA(UCineCameraComponent::StaticClass()))
	{
		UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent);
		if (CineCameraComponent)
		{
			ApertureWidthInInches = FUnitConversion::Convert(CineCameraComponent->Filmback.SensorWidth, EUnit::Millimeters, EUnit::Inches);
			ApertureHeightInInches = FUnitConversion::Convert(CineCameraComponent->Filmback.SensorHeight, EUnit::Millimeters, EUnit::Inches);
			FocalLength = CineCameraComponent->CurrentFocalLength;
			FocusDistance = CineCameraComponent->FocusSettings.ManualFocusDistance;
		}
	}

	// Export the view area information
	Camera->ProjectionType.Set(CameraComponent->ProjectionMode == ECameraProjectionMode::Type::Perspective ? FbxCamera::ePerspective : FbxCamera::eOrthogonal);
	Camera->SetAspect(FbxCamera::eFixedRatio, CameraComponent->AspectRatio, 1.0f);
	Camera->FilmAspectRatio.Set(CameraComponent->AspectRatio);
	Camera->SetApertureWidth(ApertureWidthInInches);
	Camera->SetApertureHeight(ApertureHeightInInches);
	Camera->SetApertureMode(FbxCamera::eFocalLength);
	Camera->FocalLength.Set(FocalLength);

	if (FocusDistance.IsSet())
	{
		Camera->FocusDistance.Set(FocusDistance.GetValue());
	}

	// Add one user property for recording the AspectRatio animation
	CreateAnimatableUserProperty(ParentNode, CameraComponent->AspectRatio, "UE_AspectRatio", "UE_Matinee_Camera_AspectRatio");

	// Push the near/far clip planes away, as the engine uses larger values than the default.
	Camera->SetNearPlane(10.0f);
	Camera->SetFarPlane(100000.0f);
}

void FFbxExporter::ExportCamera( ACameraActor* Actor, bool bExportComponents,INodeNameAdapter& NodeNameAdapter )
{
	if (Scene == NULL || Actor == NULL) return;

	UCameraComponent *CameraComponent = Actor->GetCameraComponent();
	// Export the basic actor information.
	FbxNode* FbxActor = ExportActor( Actor, bExportComponents, NodeNameAdapter ); // this is the pivot node
	// The real fbx camera node
	FbxNode* FbxCameraNode = FbxActor->GetParent();

	FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);

	// Create a properly-named FBX camera structure and instantiate it in the FBX scene graph
	FbxCamera* Camera = FbxCamera::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));
	FillFbxCameraAttribute(FbxCameraNode, Camera, CameraComponent);

	FbxActor->SetNodeAttribute(Camera);

	DefaultCamera = Camera;
}

/**
 * Exports the mesh and the actor information for a brush actor.
 */
void FFbxExporter::ExportBrush(ABrush* Actor, UModel* InModel, bool bConvertToStaticMesh, INodeNameAdapter& NodeNameAdapter )
{
	if (Scene == NULL || Actor == NULL || !Actor->GetBrushComponent()) return;

	if (!bConvertToStaticMesh)
	{
		// Retrieve the information structures, verifying the integrity of the data.
		UModel* Model = Actor->GetBrushComponent()->Brush;

		if (Model == NULL || Model->VertexBuffer.Vertices.Num() < 3 || Model->MaterialIndexBuffers.Num() == 0) return;
 
		// Create the FBX actor, the FBX geometry and instantiate it.
		FbxNode* FbxActor = ExportActor( Actor, false, NodeNameAdapter );
		Scene->GetRootNode()->AddChild(FbxActor);
 
		// Export the mesh information
		ExportModel(Model, FbxActor, TCHAR_TO_UTF8(*Actor->GetName()), FFbxMaterialBakingMeshData(InModel, Actor));
	}
	else
	{
		FMeshDescription Mesh;
		FStaticMeshAttributes MeshAttributes(Mesh);
		MeshAttributes.Register();
		TArray<FStaticMaterial>	Materials;
		GetBrushMesh(Actor,Actor->Brush,Mesh,Materials);

		if( Mesh.Vertices().Num() )
		{
			UStaticMesh* StaticMesh = CreateStaticMesh(Mesh,Materials,GetTransientPackage(),Actor->GetFName());
			ExportStaticMesh( StaticMesh, &Materials );
		}
	}
}

void FFbxExporter::ExportModel(UModel* Model, FbxNode* Node, const char* Name, const FFbxMaterialBakingMeshData& MaterialBakingMeshData)
{
	//int32 VertexCount = Model->VertexBuffer.Vertices.Num();
	int32 MaterialCount = Model->MaterialIndexBuffers.Num();

	const float BiasedHalfWorldExtent = HALF_WORLD_MAX * 0.95f;

	// Create the mesh and three data sources for the vertex positions, normals and texture coordinates.
	FbxMesh* Mesh = FbxMesh::Create(Scene, Name);
	
	// Create control points.
	uint32 VertCount(Model->VertexBuffer.Vertices.Num());
	Mesh->InitControlPoints(VertCount);
	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	
	// Set the normals on Layer 0.
	FbxLayer* Layer = Mesh->GetLayer(0);
	if (Layer == NULL)
	{
		Mesh->CreateLayer();
		Layer = Mesh->GetLayer(0);
	}
	
	// We want to have one normal for each vertex (or control point),
	// so we set the mapping mode to eBY_CONTROL_POINT.
	FbxLayerElementNormal* LayerElementNormal= FbxLayerElementNormal::Create(Mesh, "");

	LayerElementNormal->SetMappingMode(FbxLayerElement::eByControlPoint);

	// Set the normal values for every control point.
	LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);
	
	// Create UV for Diffuse channel.
	FbxLayerElementUV* UVDiffuseLayer = FbxLayerElementUV::Create(Mesh, "DiffuseUV");
	UVDiffuseLayer->SetMappingMode(FbxLayerElement::eByControlPoint);
	UVDiffuseLayer->SetReferenceMode(FbxLayerElement::eDirect);
	Layer->SetUVs(UVDiffuseLayer, FbxLayerElement::eTextureDiffuse);
	
	for (uint32 VertexIdx = 0; VertexIdx < VertCount; ++VertexIdx)
	{
		FModelVertex& Vertex = Model->VertexBuffer.Vertices[VertexIdx];
		FVector Normal = (FVector4)Vertex.TangentZ;

		// If the vertex is outside of the world extent, snap it to the origin.  The faces associated with
		// these vertices will be removed before exporting.  We leave the snapped vertex in the buffer so
		// we won't have to deal with reindexing everything.
		FVector FinalVertexPos = (FVector)Vertex.Position;
		if( FMath::Abs( Vertex.Position.X ) > BiasedHalfWorldExtent ||
			FMath::Abs( Vertex.Position.Y ) > BiasedHalfWorldExtent ||
			FMath::Abs( Vertex.Position.Z ) > BiasedHalfWorldExtent )
		{
			FinalVertexPos = FVector::ZeroVector;
		}

		ControlPoints[VertexIdx] = FbxVector4(FinalVertexPos.X, -FinalVertexPos.Y, FinalVertexPos.Z);
		FbxVector4 FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z);
		FbxAMatrix NodeMatrix;
		FbxVector4 Trans = Node->LclTranslation.Get();
		NodeMatrix.SetT(FbxVector4(Trans[0], Trans[1], Trans[2]));
		FbxVector4 Rot = Node->LclRotation.Get();
		NodeMatrix.SetR(FbxVector4(Rot[0], Rot[1], Rot[2]));
		NodeMatrix.SetS(Node->LclScaling.Get());
		FbxNormal = NodeMatrix.MultT(FbxNormal);
		FbxNormal.Normalize();
		LayerElementNormal->GetDirectArray().Add(FbxNormal);
		
		// update the index array of the UVs that map the texture to the face
		UVDiffuseLayer->GetDirectArray().Add(FbxVector2(Vertex.TexCoord.X, -Vertex.TexCoord.Y));
	}
	
	Layer->SetNormals(LayerElementNormal);
	Layer->SetUVs(UVDiffuseLayer);
	
	FbxLayerElementMaterial* MatLayer = FbxLayerElementMaterial::Create(Mesh, "");
	MatLayer->SetMappingMode(FbxLayerElement::eByPolygon);
	MatLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	Layer->SetMaterials(MatLayer);
	
	//Make sure the Index buffer is accessible.
	
	for (auto MaterialIterator = Model->MaterialIndexBuffers.CreateIterator(); MaterialIterator; ++MaterialIterator)
	{
		BeginReleaseResource(MaterialIterator->Value.Get());
	}
	FlushRenderingCommands();

	// Create the materials and the per-material tesselation structures.
	for (auto MaterialIterator = Model->MaterialIndexBuffers.CreateIterator(); MaterialIterator; ++MaterialIterator)
	{
		UMaterialInterface* MaterialInterface = MaterialIterator.Key();
		FRawIndexBuffer16or32& IndexBuffer = *MaterialIterator.Value();
		int32 IndexCount = IndexBuffer.Indices.Num();
		if (IndexCount < 3) continue;
		
		// Are NULL materials okay?
		int32 MaterialIndex = -1;
		FbxSurfaceMaterial* FbxMaterial;
		if (MaterialInterface != NULL && MaterialInterface->GetMaterial() != NULL)
		{
			FbxMaterial = ExportMaterial(MaterialInterface, MaterialBakingMeshData.GetUModelStaticMeshMaterialIndex(MaterialInterface), MaterialBakingMeshData);
		}
		else
		{
			// Set default material
			FbxMaterial = CreateDefaultMaterial();
		}
		MaterialIndex = Node->AddMaterial(FbxMaterial);

		// Create the Fbx polygons set.

		// Retrieve and fill in the index buffer.
		const int32 TriangleCount = IndexCount / 3;
		for( int32 TriangleIdx = 0; TriangleIdx < TriangleCount; ++TriangleIdx )
		{
			bool bSkipTriangle = false;

			for( int32 IndexIdx = 0; IndexIdx < 3; ++IndexIdx )
			{
				// Skip triangles that belong to BSP geometry close to the world extent, since its probably
				// the automatically-added-brush for new levels.  The vertices will be left in the buffer (unreferenced)
				FVector VertexPos = (FVector)Model->VertexBuffer.Vertices[ IndexBuffer.Indices[ TriangleIdx * 3 + IndexIdx ] ].Position;
				if( FMath::Abs( VertexPos.X ) > BiasedHalfWorldExtent ||
					FMath::Abs( VertexPos.Y ) > BiasedHalfWorldExtent ||
					FMath::Abs( VertexPos.Z ) > BiasedHalfWorldExtent )
				{
					bSkipTriangle = true;
					break;
				}
			}

			if( !bSkipTriangle )
			{
				// all faces of the cube have the same texture
				Mesh->BeginPolygon(MaterialIndex);
				for( int32 IndexIdx = 0; IndexIdx < 3; ++IndexIdx )
				{
					// Control point index
					Mesh->AddPolygon(IndexBuffer.Indices[ TriangleIdx * 3 + IndexIdx ]);

				}
				Mesh->EndPolygon ();
			}
		}

		BeginInitResource(&IndexBuffer);
	}
	
	FlushRenderingCommands();

	Node->SetNodeAttribute(Mesh);
}

FString FFbxExporter::GetFbxObjectName(const FString &FbxObjectNode, INodeNameAdapter& NodeNameAdapter)
{
	FString FbxTestName = FbxObjectNode;
	int32 *NodeIndex = FbxNodeNameToIndexMap.Find(FbxTestName);
	if (NodeIndex)
	{
		FbxTestName = FString::Printf(TEXT("%s%d"), *FbxTestName, *NodeIndex);
		++(*NodeIndex);
	}
	else
	{
		FbxNodeNameToIndexMap.Add(FbxTestName, 1);
	}
	return FbxTestName;
}

void FFbxExporter::ExportStaticMesh(AActor* Actor, UStaticMeshComponent* StaticMeshComponent, INodeNameAdapter& NodeNameAdapter)
{
	if (Scene == NULL || Actor == NULL || StaticMeshComponent == NULL)
	{
		return;
	}

	// Retrieve the static mesh rendering information at the correct LOD level.
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh == NULL || !StaticMesh->HasValidRenderData())
	{
		return;
	}
	FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);
	FString FbxMeshName = StaticMesh->GetName().Replace(TEXT("-"), TEXT("_"));
	FColorVertexBuffer* ColorBuffer = NULL;

	if(GetExportOptions()->LevelOfDetail && StaticMesh->GetNumLODs() > 1)
	{
		//Create a fbx LOD Group node
		FbxNode* FbxActor = ExportActor(Actor, false, NodeNameAdapter);
		FString FbxLODGroupName = NodeNameAdapter.GetActorNodeName(Actor);
		FbxLODGroupName += TEXT("_LodGroup");
		FbxLODGroupName = GetFbxObjectName(FbxLODGroupName, NodeNameAdapter);
		FbxLODGroup *FbxLodGroupAttribute = FbxLODGroup::Create(Scene, TCHAR_TO_UTF8(*FbxLODGroupName));
		FbxActor->AddNodeAttribute(FbxLodGroupAttribute);
		FbxLodGroupAttribute->ThresholdsUsedAsPercentage = true;
		//Export an Fbx Mesh Node for every LOD and child them to the fbx node (LOD Group)
		for (int CurrentLodIndex = 0; CurrentLodIndex < StaticMesh->GetNumLODs(); ++CurrentLodIndex)
		{
			if (CurrentLodIndex < StaticMeshComponent->LODData.Num())
			{
				ColorBuffer = StaticMeshComponent->LODData[CurrentLodIndex].OverrideVertexColors;
			}
			else
			{
				ColorBuffer = nullptr;
			}
			FString FbxLODNodeName = NodeNameAdapter.GetActorNodeName(Actor);
			FbxLODNodeName += TEXT("_LOD") + FString::FromInt(CurrentLodIndex);
			FbxLODNodeName = GetFbxObjectName(FbxLODNodeName, NodeNameAdapter);
			FbxNode* FbxActorLOD = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxLODNodeName));
			FbxActor->AddChild(FbxActorLOD);
			if (CurrentLodIndex + 1 < StaticMesh->GetNumLODs())
			{
				//Convert the screen size to a threshold, it is just to be sure that we set some threshold, there is no way to convert this precisely
				double LodScreenSize = (double)(10.0f / StaticMesh->GetRenderData()->ScreenSize[CurrentLodIndex].Default);
				FbxLodGroupAttribute->AddThreshold(LodScreenSize);
			}

			const int32 LightmapUVChannel = -1;
			const TArray<FStaticMaterial>* MaterialOrderOverride = nullptr;
			ExportStaticMeshToFbx(StaticMesh, CurrentLodIndex, *FbxMeshName, FbxActorLOD, FFbxMaterialBakingMeshData(StaticMesh, StaticMeshComponent, CurrentLodIndex), LightmapUVChannel, ColorBuffer, MaterialOrderOverride, &ToRawPtrTArrayUnsafe(StaticMeshComponent->OverrideMaterials));
		}
	}
	else
	{
		const int32 LODIndex = (StaticMeshComponent->ForcedLodModel > 0 ? FMath::Min(StaticMesh->GetNumLODs(), StaticMeshComponent->ForcedLodModel) - 1 : /* auto-select*/ 0);
		if (StaticMeshComponent->LODData.IsValidIndex(LODIndex))
		{
			ColorBuffer = StaticMeshComponent->LODData[LODIndex].OverrideVertexColors;
		}
		FbxNode* FbxActor = ExportActor(Actor, false, NodeNameAdapter);
		const int32 LightmapUVChannel = -1;
		const TArray<FStaticMaterial>* MaterialOrderOverride = nullptr;
		ExportStaticMeshToFbx(StaticMesh, LODIndex, *FbxMeshName, FbxActor, FFbxMaterialBakingMeshData(StaticMesh, StaticMeshComponent, LODIndex), LightmapUVChannel, ColorBuffer, MaterialOrderOverride, &ToRawPtrTArrayUnsafe(StaticMeshComponent->OverrideMaterials));
	}
}

struct FBSPExportData
{
	FMeshDescription Mesh;
	TArray<FStaticMaterial> Materials;
	TArray<uint32> SmoothGroups;
	uint32 NumVerts;
	uint32 NumFaces;
	uint32 CurrentVertAddIndex;
	uint32 CurrentFaceAddIndex;
	bool bInitialised;

	FBSPExportData()
		:NumVerts(0)
		,NumFaces(0)
		,CurrentVertAddIndex(0)
		,CurrentFaceAddIndex(0)
		,bInitialised(false)
	{

	}
};

void FFbxExporter::ExportBSP( UModel* Model, bool bSelectedOnly )
{
	TMap< ABrush*, FBSPExportData > BrushToMeshMap;
	TArray<FStaticMaterial> AllMaterials;

	for(int32 NodeIndex = 0;NodeIndex < Model->Nodes.Num();NodeIndex++)
	{
		FBspNode& Node = Model->Nodes[NodeIndex];
		if( Node.NumVertices >= 3 )
		{
			FBspSurf& Surf = Model->Surfs[Node.iSurf];
		
			ABrush* BrushActor = Surf.Actor;

			if( (Surf.PolyFlags & PF_Selected) || !bSelectedOnly || (BrushActor && BrushActor->IsSelected() ) )
			{
				FBSPExportData& Data = BrushToMeshMap.FindOrAdd( BrushActor );

				FPoly Poly;
				GEditor->polyFindBrush(Model, Node.iSurf, Poly);

				Data.NumVerts += Node.NumVertices;
				Data.NumFaces += Node.NumVertices-2;
				UMaterialInterface*	Material = Poly.Material;
				FName MaterialName = Material != nullptr ? Material->GetFName() : NAME_None;
				Data.Materials.AddUnique(FStaticMaterial(Material, MaterialName, MaterialName));
				AllMaterials.AddUnique(FStaticMaterial(Material, MaterialName, MaterialName));
			}
		}
	}
	
	for(int32 NodeIndex = 0;NodeIndex < Model->Nodes.Num();NodeIndex++)
	{
		FBspNode& Node = Model->Nodes[NodeIndex];
		FBspSurf& Surf = Model->Surfs[Node.iSurf];

		ABrush* BrushActor = Surf.Actor;

		if( (Surf.PolyFlags & PF_Selected) || !bSelectedOnly || (BrushActor && BrushActor->IsSelected() && Node.NumVertices >= 3) )
		{
			FPoly Poly;
			GEditor->polyFindBrush( Model, Node.iSurf, Poly );

			FBSPExportData* ExportData = BrushToMeshMap.Find( BrushActor );
			if( NULL == ExportData )
			{
				UE_LOGF(LogFbx, Fatal, "Error in FBX export of BSP.");
				return;
			}

			TArray<FStaticMaterial>& Materials = ExportData->Materials;
			FMeshDescription& Mesh = ExportData->Mesh;

			//Pre-allocate space for this mesh.
			if( !ExportData->bInitialised )
			{
				uint32 NumWedges = ExportData->NumFaces * 3;

				FStaticMeshAttributes InitMeshAttributes(Mesh);
				InitMeshAttributes.Register();

				ExportData->bInitialised = true;
				Mesh.Empty();
				Mesh.ReserveNewVertices(ExportData->NumVerts);
				Mesh.ReserveNewVertexInstances(NumWedges);
				Mesh.ReserveNewPolygons(ExportData->NumFaces);
				Mesh.ReserveNewEdges(ExportData->NumFaces * 2);
				Mesh.ReserveNewPolygonGroups(Materials.Num());
				//We need to get the PolygonGroupImportedMaterial
				TPolygonGroupAttributesRef<FName> InitPolygonGroupNames = InitMeshAttributes.GetPolygonGroupMaterialSlotNames();
				for (const FStaticMaterial& StaticMaterial : Materials)
				{
					const FPolygonGroupID PolygonGroupID = Mesh.CreatePolygonGroup();
					InitPolygonGroupNames[PolygonGroupID] = StaticMaterial.ImportedMaterialSlotName;
				}
				ExportData->SmoothGroups.Empty(ExportData->NumFaces);
			}
			
			//Get the Attributes
			FStaticMeshAttributes MeshAttributes(Mesh);
			TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
			TVertexInstanceAttributesRef<FVector2f> UVs = MeshAttributes.GetVertexInstanceUVs();
			TVertexInstanceAttributesRef<FVector4f> Colors = MeshAttributes.GetVertexInstanceColors();
			TVertexInstanceAttributesRef<FVector3f> Normals = MeshAttributes.GetVertexInstanceNormals();
			TEdgeAttributesRef<bool> EdgeHardnesses = MeshAttributes.GetEdgeHardnesses();

			
			UMaterialInterface*	Material = Poly.Material;
			FName MaterialName = Material != nullptr ? Material->GetFName() : NAME_None;
			
			int32 MaterialIndex = INDEX_NONE;
			if (!ExportData->Materials.Find(FStaticMaterial(Material, MaterialName, MaterialName), MaterialIndex))
			{
				MaterialIndex = 0;
			}
			const FPolygonGroupID CurrentPolygonGroupID(MaterialIndex);
			//The material ID should follow the unique materials
			check(Mesh.IsPolygonGroupValid(CurrentPolygonGroupID));

			const FVector& TextureBase = (FVector)Model->Points[Surf.pBase];
			const FVector& TextureX = (FVector)Model->Vectors[Surf.vTextureU];
			const FVector& TextureY = (FVector)Model->Vectors[Surf.vTextureV];
			const FVector& Normal = (FVector)Model->Vectors[Surf.vNormal];

			const int32 StartIndex = ExportData->CurrentVertAddIndex;

			for(int32 VertexIndex = 0; VertexIndex < Node.NumVertices ; VertexIndex++ )
			{
				const FVert& Vert = Model->Verts[Node.iVertPool + VertexIndex];
				const FVector& Vertex = (FVector)Model->Points[Vert.pVertex];
				FVertexID VertexID = Mesh.CreateVertex();
				VertexPositions[VertexID] = (FVector3f)Vertex;
			}
			ExportData->CurrentVertAddIndex += Node.NumVertices;

			for (int32 StartVertexIndex = 1; StartVertexIndex < Node.NumVertices - 1; ++StartVertexIndex)
			{
				TArray<FVertexInstanceID> VertexInstanceIDs;
				VertexInstanceIDs.SetNum(3);

				FVertexID VertexIDs[3];
				// These map the node's vertices to the 3 triangle indices to triangulate the convex polygon.
				int32 TriVertIndices[3] = {
					Node.iVertPool + StartVertexIndex + 1,
					Node.iVertPool + StartVertexIndex,
					Node.iVertPool };

				int32 WedgeIndices[3] = {
					StartIndex + StartVertexIndex + 1,
					StartIndex + StartVertexIndex,
					StartIndex };

				for (uint32 Corner = 0; Corner < 3; ++Corner)
				{
					VertexIDs[Corner] = FVertexID(WedgeIndices[Corner]);
					VertexInstanceIDs[Corner] = Mesh.CreateVertexInstance(VertexIDs[Corner]);
					const FVert& Vert = Model->Verts[TriVertIndices[Corner]];
					const FVector& Vertex = (FVector)Model->Points[Vert.pVertex];

					float U = ((Vertex - TextureBase) | TextureX) / UModel::GetGlobalBSPTexelScale();
					float V = ((Vertex - TextureBase) | TextureY) / UModel::GetGlobalBSPTexelScale();
					UVs.Set(VertexInstanceIDs[Corner], 0, FVector2f(U, V));
					//This is not exported when exporting the whole level via ExportModel so leaving out here for now. 
					//UVs.Set(VertexInstanceIDs[Corner], 1, Vert.ShadowTexCoord);
					Colors[VertexInstanceIDs[Corner]] = FVector4f(FLinearColor::White);
					Normals[VertexInstanceIDs[Corner]] = (FVector3f)Normal;
				}

				// Insert a polygon into the mesh
				TArray<FEdgeID> NewEdgeIDs;
				const FPolygonID NewPolygonID = Mesh.CreatePolygon(CurrentPolygonGroupID, VertexInstanceIDs, &NewEdgeIDs);
				for (const FEdgeID& EdgeID : NewEdgeIDs)
				{
					EdgeHardnesses[EdgeID] = false;
				}

				//Add to the smoothGroup array so we can compute hard edge later
				ExportData->SmoothGroups.Add((1 << (Node.iSurf % 32)));

				++ExportData->CurrentFaceAddIndex;
			}
		}
	}

	for( TMap< ABrush*, FBSPExportData >::TIterator It(BrushToMeshMap); It; ++It )
	{
		if( It.Value().Mesh.Vertices().Num() )
		{
			FStaticMeshOperations::ConvertSmoothGroupToHardEdges(It.Value().SmoothGroups, It.Value().Mesh);

			UStaticMesh* NewMesh = CreateStaticMesh( It.Value().Mesh, It.Value().Materials, GetTransientPackage(), It.Key()->GetFName() );

			ExportStaticMesh( NewMesh, &It.Value().Materials);
		}
	}
}

void FFbxExporter::ExportStaticMesh( UStaticMesh* StaticMesh, const TArray<FStaticMaterial>* MaterialOrder )
{
	if (Scene == NULL || StaticMesh == NULL || !StaticMesh->HasValidRenderData()) return;
	FString MeshName;
	StaticMesh->GetName(MeshName);
	FbxNode* MeshNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*MeshName));
	Scene->GetRootNode()->AddChild(MeshNode);

	if (GetExportOptions()->LevelOfDetail && StaticMesh->GetNumLODs() > 1)
	{
		FString LodGroup_MeshName = MeshName + ("_LodGroup");
		FbxLODGroup *FbxLodGroupAttribute = FbxLODGroup::Create(Scene, TCHAR_TO_UTF8(*LodGroup_MeshName));
		MeshNode->AddNodeAttribute(FbxLodGroupAttribute);
		FbxLodGroupAttribute->ThresholdsUsedAsPercentage = true;
		//Export an Fbx Mesh Node for every LOD and child them to the fbx node (LOD Group)
		for (int CurrentLodIndex = 0; CurrentLodIndex < StaticMesh->GetNumLODs(); ++CurrentLodIndex)
		{
			FString FbxLODNodeName = MeshName + TEXT("_LOD") + FString::FromInt(CurrentLodIndex);
			FbxNode* FbxActorLOD = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxLODNodeName));
			MeshNode->AddChild(FbxActorLOD);
			if (CurrentLodIndex + 1 < StaticMesh->GetNumLODs())
			{
				//Convert the screen size to a threshold, it is just to be sure that we set some threshold, there is no way to convert this precisely
				double LodScreenSize = (double)(10.0f / StaticMesh->GetRenderData()->ScreenSize[CurrentLodIndex].Default);
				FbxLodGroupAttribute->AddThreshold(LodScreenSize);
			}
			ExportStaticMeshToFbx(StaticMesh, CurrentLodIndex, *MeshName, FbxActorLOD, FFbxMaterialBakingMeshData(StaticMesh, nullptr, CurrentLodIndex), -1, nullptr, MaterialOrder);
		}
	}
	else
	{
		ExportStaticMeshToFbx(StaticMesh, 0, *MeshName, MeshNode, FFbxMaterialBakingMeshData(StaticMesh), -1, NULL, MaterialOrder);
	}
}

void FFbxExporter::ExportStaticMeshLightMap( UStaticMesh* StaticMesh, int32 LODIndex, int32 UVChannel )
{
	if (Scene == NULL || StaticMesh == NULL || !StaticMesh->HasValidRenderData()) return;

	FString MeshName;
	StaticMesh->GetName(MeshName);
	FbxNode* MeshNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*MeshName));
	Scene->GetRootNode()->AddChild(MeshNode);
	ExportStaticMeshToFbx(StaticMesh, LODIndex, *MeshName, MeshNode, FFbxMaterialBakingMeshData(StaticMesh, nullptr, LODIndex), UVChannel);
}

void FFbxExporter::ExportSkeletalMesh( USkeletalMesh* SkeletalMesh )
{
	if (Scene == NULL || SkeletalMesh == NULL) return;

	FString MeshName;
	SkeletalMesh->GetName(MeshName);

	FbxNode* MeshNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*MeshName));
	Scene->GetRootNode()->AddChild(MeshNode);

	ExportSkeletalMeshToFbx(SkeletalMesh, NULL, *MeshName, MeshNode);
}

void FFbxExporter::ExportSkeletalMesh( AActor* Actor, USkeletalMeshComponent* SkeletalMeshComponent, INodeNameAdapter& NodeNameAdapter )
{
	if (Scene == NULL || Actor == NULL || SkeletalMeshComponent == NULL) return;

	// Retrieve the skeletal mesh rendering information.
	USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

	FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);

	ExportActor( Actor, true, NodeNameAdapter );
}

FbxSurfaceMaterial* FFbxExporter::CreateDefaultMaterial()
{
	// TODO(sbc): the below cast is needed to avoid clang warning.  The upstream
	// signature in FBX should really use 'const char *'.
	FbxSurfaceMaterial* FbxMaterial = Scene->GetMaterial(const_cast<char*>("Fbx Default Material"));
	
	if (!FbxMaterial)
	{
		FbxMaterial = FbxSurfaceLambert::Create(Scene, "Fbx Default Material");
		((FbxSurfaceLambert*)FbxMaterial)->Diffuse.Set(FbxDouble3(0.72, 0.72, 0.72));
	}
	
	return FbxMaterial;
}

void FFbxExporter::ExportLandscape(ALandscapeProxy* Actor, bool bSelectedOnly, INodeNameAdapter& NodeNameAdapter)
{
	if (Scene == NULL || Actor == NULL)
	{ 
		return;
	}

	FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);

	FbxNode* FbxActor = ExportActor(Actor, true, NodeNameAdapter);
	ExportLandscapeToFbx(Actor, *FbxNodeName, FbxActor, bSelectedOnly);
}

FbxDouble3 SetMaterialComponent(FColorMaterialInput& MatInput, bool ToLinear)
{
	FColor RGBColor;
	FLinearColor LinearColor = FLinearColor::Black;
	bool LinearSet = false;
	
	if (MatInput.Expression)
	{
		if (Cast<UMaterialExpressionConstant>(MatInput.Expression))
		{
			UMaterialExpressionConstant* Expr = Cast<UMaterialExpressionConstant>(MatInput.Expression);			
			RGBColor = FColor(Expr->R);
		}
		else if (Cast<UMaterialExpressionVectorParameter>(MatInput.Expression))
		{
			UMaterialExpressionVectorParameter* Expr = Cast<UMaterialExpressionVectorParameter>(MatInput.Expression);
			LinearColor = Expr->DefaultValue;
			LinearSet = true;
			//Linear to sRGB color space conversion
			RGBColor = Expr->DefaultValue.ToFColor(true);
		}
		else if (Cast<UMaterialExpressionConstant3Vector>(MatInput.Expression))
		{
			UMaterialExpressionConstant3Vector* Expr = Cast<UMaterialExpressionConstant3Vector>(MatInput.Expression);
			RGBColor.R = Expr->Constant.R;
			RGBColor.G = Expr->Constant.G;
			RGBColor.B = Expr->Constant.B;
		}
		else if (Cast<UMaterialExpressionConstant4Vector>(MatInput.Expression))
		{
			UMaterialExpressionConstant4Vector* Expr = Cast<UMaterialExpressionConstant4Vector>(MatInput.Expression);
			RGBColor.R = Expr->Constant.R;
			RGBColor.G = Expr->Constant.G;
			RGBColor.B = Expr->Constant.B;
			//RGBColor.A = Expr->A;
		}
		else if (Cast<UMaterialExpressionConstant2Vector>(MatInput.Expression))
		{
			UMaterialExpressionConstant2Vector* Expr = Cast<UMaterialExpressionConstant2Vector>(MatInput.Expression);
			RGBColor.R = Expr->R;
			RGBColor.G = Expr->G;
			RGBColor.B = 0;
		}
		else
		{
			RGBColor.R = MatInput.Constant.R;
			RGBColor.G = MatInput.Constant.G;
			RGBColor.B = MatInput.Constant.B;
		}
	}
	else
	{
		RGBColor.R = MatInput.Constant.R;
		RGBColor.G = MatInput.Constant.G;
		RGBColor.B = MatInput.Constant.B;
	}
	
	if (ToLinear)
	{
		if (!LinearSet)
		{
			//sRGB to linear color space conversion
			LinearColor = FLinearColor(RGBColor);
		}
		return FbxDouble3(LinearColor.R, LinearColor.G, LinearColor.B);
	}
	return FbxDouble3(RGBColor.R, RGBColor.G, RGBColor.B);
}

bool FFbxExporter::FillFbxTextureProperty(const char *PropertyName, const FExpressionInput& MaterialInput, FbxSurfaceMaterial* FbxMaterial)
{
	if (Scene == NULL)
	{
		return false;
	}

	FbxProperty FbxColorProperty = FbxMaterial->FindProperty(PropertyName);
	if (FbxColorProperty.IsValid())
	{
		if (MaterialInput.IsConnected() && MaterialInput.Expression)
		{
			if (MaterialInput.Expression->IsA(UMaterialExpressionTextureSample::StaticClass()))
			{
				UMaterialExpressionTextureSample *TextureSample = Cast<UMaterialExpressionTextureSample>(MaterialInput.Expression);
				if (TextureSample && TextureSample->Texture && TextureSample->Texture->AssetImportData)
				{
					FString TextureSourceFullPath = TextureSample->Texture->AssetImportData->GetFirstFilename();
					//Create a fbx property
					FbxFileTexture* lTexture = FbxFileTexture::Create(Scene, "EnvSamplerTex");
					lTexture->SetFileName(TCHAR_TO_UTF8(*TextureSourceFullPath));
					lTexture->SetTextureUse(FbxTexture::eStandard);
					lTexture->SetMappingType(FbxTexture::eUV);
					lTexture->ConnectDstProperty(FbxColorProperty);
					return true;
				}
			}
		}
	}
	return false;
}

/**
* Exports the profile_COMMON information for a material.
*/
FbxSurfaceMaterial* FFbxExporter::ExportMaterial(UMaterialInterface* MaterialInterface, const int32& MaterialIndex, const FFbxMaterialBakingMeshData& BakingMeshData)
{
	FString ExportFolderPath = FPaths::GetPath(UExporter::CurrentFilename);

	if (Scene == nullptr || MaterialInterface == nullptr || MaterialInterface->GetMaterial() == nullptr) return nullptr;
	
	// Verify that this material has not already been exported:
	TMap<int32, FbxSurfaceMaterial*>* MaterialIndexToFbxMaterials = FbxMaterials.Find(MaterialInterface);
	if (MaterialIndexToFbxMaterials && MaterialIndexToFbxMaterials->Find(MaterialIndex))
	{
		return *MaterialIndexToFbxMaterials->Find(MaterialIndex);
	}

	// Create the Fbx material
	FbxSurfaceMaterial* FbxMaterial = nullptr;

	UMaterial *Material = MaterialInterface->GetMaterial();
	if (!Material)
	{
		//Nothing to export
		return nullptr;
	}

	if (!MaterialIndexToFbxMaterials)
	{
		MaterialIndexToFbxMaterials = &FbxMaterials.Add(MaterialInterface);
	}

	UMaterialEditorOnlyData* MaterialEditorOnly = Material->GetEditorOnlyData();
	
	// Set the shading model
	bool bUseLambert = false;
	bool bInterchangeOriginated = FFbxMaterialExportUtilities::GetInterchangeShadingModel(MaterialInterface, bUseLambert);

	FString FbxMaterialName = MaterialInterface->GetName();
	if (MaterialIndexToFbxMaterials && MaterialIndexToFbxMaterials->Num() > 0)
	{
		FbxMaterialName += FString(TEXT("_")) + FString::FromInt(MaterialIndex);
	}
	
	if ((bInterchangeOriginated && !bUseLambert) || (!bInterchangeOriginated && Material->GetShadingModels().HasOnlyShadingModel(MSM_DefaultLit)))
	{
		FbxMaterial = FbxSurfacePhong::Create(Scene, TCHAR_TO_UTF8(*FbxMaterialName));
		//((FbxSurfacePhong*)FbxMaterial)->Specular.Set(Material->Specular));
		//((FbxSurfacePhong*)FbxMaterial)->Shininess.Set(Material->SpecularPower.Constant);
	}
	else // if (Material->ShadingModel == MSM_Unlit)
	{
		FbxMaterial = FbxSurfaceLambert::Create(Scene, TCHAR_TO_UTF8(*FbxMaterialName));
	}

	if (bInterchangeOriginated)
	{
		FFbxMaterialExportUtilities::ProcessInterchangeMaterials(MaterialInterface, Scene, FbxMaterial);

		MaterialIndexToFbxMaterials->Add(MaterialIndex, FbxMaterial);

		return FbxMaterial;
	}

	//Get the base material connected expression parameter name for all the supported fbx material exported properties
	//We only support material input where the connected expression is a parameter of type (constant, scalar, vector, texture, TODO virtual texture)

	bool bBaseColorNonConstAndNonDefault = (!MaterialEditorOnly->BaseColor.UseConstant && MaterialEditorOnly->BaseColor.Expression);
	bool bEmissiveNonConstAndNonDefault = (!MaterialEditorOnly->EmissiveColor.UseConstant && MaterialEditorOnly->EmissiveColor.Expression);
	bool bNormalNonConstAndNonDefault = (MaterialEditorOnly->Normal.Expression != nullptr);
	bool bOpacityNonConstAndNonDefault = (!MaterialEditorOnly->Opacity.UseConstant && MaterialEditorOnly->Opacity.Expression);
	bool bOpacityMaskNonConstAndNonDefault = (!MaterialEditorOnly->OpacityMask.UseConstant && MaterialEditorOnly->OpacityMask.Expression);

	FName BaseColorParamName = bBaseColorNonConstAndNonDefault ? MaterialEditorOnly->BaseColor.Expression->GetParameterName() : NAME_None;
	FName EmissiveParamName = bEmissiveNonConstAndNonDefault ? MaterialEditorOnly->EmissiveColor.Expression->GetParameterName() : NAME_None;
	FName NormalParamName = bNormalNonConstAndNonDefault ? MaterialEditorOnly->Normal.Expression->GetParameterName() : NAME_None;
	FName OpacityParamName = bOpacityNonConstAndNonDefault ? MaterialEditorOnly->Opacity.Expression->GetParameterName() : NAME_None;
	FName OpacityMaskParamName = bOpacityMaskNonConstAndNonDefault ? MaterialEditorOnly->OpacityMask.Expression->GetParameterName() : NAME_None;
	
	bool BaseColorParamSet = false;
	bool EmissiveParamSet = false;
	bool NormalParamSet = false;
	bool OpacityParamSet = false;
	bool OpacityMaskParamSet = false;

	UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
	EBlendMode BlendMode = MaterialInstance != nullptr && MaterialInstance->BasePropertyOverrides.bOverride_BlendMode ? MaterialInstance->BasePropertyOverrides.BlendMode : Material->BlendMode;

	// Loop through all types of parameters for this base material and add the supported one to the fbx material.
	//The order is important we search Texture, Vector, Scalar.

	TArray<FMaterialParameterInfo> ParameterInfos;
	TArray<FGuid> Guids;
	//Query all base material texture parameters.
	Material->GetAllTextureParameterInfo(ParameterInfos, Guids);
	for (int32 ParameterIdx = 0; ParameterIdx < ParameterInfos.Num(); ParameterIdx++)
	{
		const FMaterialParameterInfo& ParameterInfo = ParameterInfos[ParameterIdx];
		FName ParameterName = ParameterInfo.Name;
		UTexture* Value;
		//Query the material instance parameter overridden value
		if (MaterialInterface->GetTextureParameterValue(ParameterInfo, Value) && Value && Value->AssetImportData)
		{
			//This lambda extract the source file path and create the fbx file texture node and connect it to the fbx material
			auto SetTextureProperty = [&FbxMaterial, &Value](const char* FbxPropertyName, FbxScene* InScene)->bool
			{
				FbxProperty FbxColorProperty = FbxMaterial->FindProperty(FbxPropertyName);
				if (FbxColorProperty.IsValid())
				{
					const FString TextureName = Value->GetName();
					const FString TextureSourceFullPath = Value->AssetImportData->GetFirstFilename();
					if(!TextureSourceFullPath.IsEmpty())
					{
						//Create a fbx property
						FbxFileTexture* lTexture = FbxFileTexture::Create(InScene, TCHAR_TO_UTF8(*TextureName));
						lTexture->SetFileName(TCHAR_TO_UTF8(*TextureSourceFullPath));
						lTexture->SetTextureUse(FbxTexture::eStandard);
						lTexture->SetMappingType(FbxTexture::eUV);
						lTexture->ConnectDstProperty(FbxColorProperty);
						return true;
					}
				}
				return false;
			};

			if (!BaseColorParamSet && BaseColorParamName != NAME_None && ParameterName == BaseColorParamName)
			{
				BaseColorParamSet = SetTextureProperty(FbxSurfaceMaterial::sDiffuse, Scene);
			}
			if (!EmissiveParamSet && EmissiveParamName != NAME_None && ParameterName == EmissiveParamName)
			{
				EmissiveParamSet = SetTextureProperty(FbxSurfaceMaterial::sEmissive, Scene);
			}
			
			if (BlendMode == BLEND_Translucent)
			{
				if (!OpacityParamSet && OpacityParamName != NAME_None && ParameterName == OpacityParamName)
				{
					OpacityParamSet = SetTextureProperty(FbxSurfaceMaterial::sTransparentColor, Scene);
				}
				if (!OpacityMaskParamSet && OpacityMaskParamName != NAME_None && ParameterName == OpacityMaskParamName)
				{
					OpacityMaskParamSet = SetTextureProperty(FbxSurfaceMaterial::sTransparencyFactor, Scene);
				}
			}
			else
			{
				//There is no normal input in Blend translucent mode
				if (!NormalParamSet && NormalParamName != NAME_None && ParameterName == NormalParamName)
				{
					NormalParamSet = SetTextureProperty(FbxSurfaceMaterial::sNormalMap, Scene);
				}
			}
		}
	}

	Material->GetAllVectorParameterInfo(ParameterInfos, Guids);
	//Query all base material vector parameters.
	for (int32 ParameterIdx = 0; ParameterIdx < ParameterInfos.Num(); ParameterIdx++)
	{
		const FMaterialParameterInfo& ParameterInfo = ParameterInfos[ParameterIdx];
		FName ParameterName = ParameterInfo.Name;
		FLinearColor LinearColor;
		//Query the material instance parameter overridden value
		if (MaterialInterface->GetVectorParameterValue(ParameterInfo, LinearColor))
		{
			FbxDouble3 LinearFbxValue(LinearColor.R, LinearColor.G, LinearColor.B);
			if (!BaseColorParamSet && BaseColorParamName != NAME_None && ParameterName == BaseColorParamName)
			{
				((FbxSurfaceLambert*)FbxMaterial)->Diffuse.Set(LinearFbxValue);
				BaseColorParamSet = true;
			}
			if (!EmissiveParamSet && EmissiveParamName != NAME_None && ParameterName == EmissiveParamName)
			{
				((FbxSurfaceLambert*)FbxMaterial)->Emissive.Set(LinearFbxValue);
				EmissiveParamSet = true;
			}
		}
	}

	//Query all base material scalar parameters.
	Material->GetAllScalarParameterInfo(ParameterInfos, Guids);
	for (int32 ParameterIdx = 0; ParameterIdx < ParameterInfos.Num(); ParameterIdx++)
	{
		const FMaterialParameterInfo& ParameterInfo = ParameterInfos[ParameterIdx];
		FName ParameterName = ParameterInfo.Name;
		float Value;
		//Query the material instance parameter overridden value
		if (MaterialInterface->GetScalarParameterValue(ParameterInfo, Value))
		{
			FbxDouble FbxValue = (FbxDouble)Value;
			FbxDouble3 FbxVector(FbxValue, FbxValue, FbxValue);
			if (!BaseColorParamSet && BaseColorParamName != NAME_None && ParameterName == BaseColorParamName)
			{
				((FbxSurfaceLambert*)FbxMaterial)->Diffuse.Set(FbxVector);
				BaseColorParamSet = true;
			}
			if (!EmissiveParamSet && EmissiveParamName != NAME_None && ParameterName == EmissiveParamName)
			{
				((FbxSurfaceLambert*)FbxMaterial)->Emissive.Set(FbxVector);
				EmissiveParamSet = true;
			}
			if (BlendMode == BLEND_Translucent)
			{
				if (!OpacityParamSet && OpacityParamName != NAME_None && ParameterName == OpacityParamName)
				{
					((FbxSurfaceLambert*)FbxMaterial)->TransparentColor.Set(FbxVector);
					OpacityParamSet = true;
				}

				if (!OpacityMaskParamSet && OpacityMaskParamName != NAME_None && ParameterName == OpacityMaskParamName)
				{
					((FbxSurfaceLambert*)FbxMaterial)->TransparencyFactor.Set(FbxValue);
					OpacityMaskParamSet = true;
				}
			}
		}
	}

	//TODO: export virtual texture to fbx
	//Query all base material runtime virtual texture parameters.
// 	Material->GetAllRuntimeVirtualTextureParameterInfo(ParameterInfos, Guids);
// 	for (int32 ParameterIdx = 0; ParameterIdx < ParameterInfos.Num(); ParameterIdx++)
// 	{
// 		const FMaterialParameterInfo& ParameterInfo = ParameterInfos[ParameterIdx];
// 		FName ParameterName = ParameterInfo.Name;
// 		URuntimeVirtualTexture* Value;
//		//Query the material instance parameter overridden value
// 		if (MaterialInterface->GetRuntimeVirtualTextureParameterValue(ParameterInfo, Value, bOveriddenOnly))
// 		{
// 		}
// 	}

	//
	//In case there is no material instance override, we extract the value from the base material
	//

	//The OpacityMaskParam set the transparency factor, so set it only if it was not set
	if(!OpacityMaskParamSet)
	{
		((FbxSurfaceLambert*)FbxMaterial)->TransparencyFactor.Set(MaterialEditorOnly->Opacity.Constant);
	}

	UFbxExportOption* FbxExportOptions = GetExportOptions();

	// Fill in the profile_COMMON effect with the material information.
	//Fill the texture or constant
	if(!BaseColorParamSet)
	{
		if (!FillFbxTextureProperty(FbxSurfaceMaterial::sDiffuse, MaterialEditorOnly->BaseColor, FbxMaterial))
		{
			if (bBaseColorNonConstAndNonDefault)
			{
				FFbxMaterialExportUtilities::BakeMaterialProperty(FbxExportOptions,
					Scene, FbxMaterial, FbxSurfaceMaterial::sDiffuse,
					MP_BaseColor, MaterialInterface, MaterialIndex,
					BakingMeshData, 
					ExportFolderPath);
			}
			else
			{
				((FbxSurfaceLambert*)FbxMaterial)->Diffuse.Set(SetMaterialComponent(MaterialEditorOnly->BaseColor, true));
			}
		}
	}

	if (!EmissiveParamSet)
	{
		if (!FillFbxTextureProperty(FbxSurfaceMaterial::sEmissive, MaterialEditorOnly->EmissiveColor, FbxMaterial))
		{
			if (bEmissiveNonConstAndNonDefault)
			{
				FFbxMaterialExportUtilities::BakeMaterialProperty(FbxExportOptions,
					Scene, FbxMaterial, FbxSurfaceMaterial::sEmissive,
					MP_EmissiveColor, MaterialInterface, MaterialIndex,
					BakingMeshData,
					ExportFolderPath);
			}
			else
			{
				((FbxSurfaceLambert*)FbxMaterial)->Emissive.Set(SetMaterialComponent(MaterialEditorOnly->EmissiveColor, true));
			}
		}
	}

	//Always set the ambient to zero since we dont have ambient in unreal we want to avoid default value in DCCs
	((FbxSurfaceLambert*)FbxMaterial)->Ambient.Set(FbxDouble3(0.0, 0.0, 0.0));

	if (BlendMode == BLEND_Translucent)
	{
		if (!OpacityParamSet)
		{
			if (!FillFbxTextureProperty(FbxSurfaceMaterial::sTransparentColor, MaterialEditorOnly->Opacity, FbxMaterial))
			{
				if (bOpacityNonConstAndNonDefault)
				{
					FFbxMaterialExportUtilities::BakeMaterialProperty(FbxExportOptions,
						Scene, FbxMaterial, FbxSurfaceMaterial::sTransparentColor,
						MP_Opacity, MaterialInterface, MaterialIndex,
						BakingMeshData,
						ExportFolderPath);
				}
				else
				{
					FbxDouble3 OpacityValue((FbxDouble)(MaterialEditorOnly->Opacity.Constant), (FbxDouble)(MaterialEditorOnly->Opacity.Constant), (FbxDouble)(MaterialEditorOnly->Opacity.Constant));
					((FbxSurfaceLambert*)FbxMaterial)->TransparentColor.Set(OpacityValue);
				}
			}
		}

		if (!OpacityMaskParamSet)
		{
			if (!FillFbxTextureProperty(FbxSurfaceMaterial::sTransparencyFactor, MaterialEditorOnly->OpacityMask, FbxMaterial))
			{
				if (bOpacityMaskNonConstAndNonDefault)
				{
					FFbxMaterialExportUtilities::BakeMaterialProperty(FbxExportOptions,
						Scene, FbxMaterial, FbxSurfaceMaterial::sTransparencyFactor,
						MP_OpacityMask, MaterialInterface, MaterialIndex,
						BakingMeshData,
						ExportFolderPath);
				}
				else
				{
					((FbxSurfaceLambert*)FbxMaterial)->TransparencyFactor.Set(MaterialEditorOnly->OpacityMask.Constant);
				}
			}
		}
	}
	else
	{
		//There is no normal input in Blend translucent mode
		if (!NormalParamSet)
		{
			//Set the Normal map only if there is a texture sampler
			if (!FillFbxTextureProperty(FbxSurfaceMaterial::sNormalMap, MaterialEditorOnly->Normal, FbxMaterial))
			{
				if (bNormalNonConstAndNonDefault)
				{
					FFbxMaterialExportUtilities::BakeMaterialProperty(FbxExportOptions,
						Scene, FbxMaterial, FbxSurfaceMaterial::sNormalMap,
						MP_Normal, MaterialInterface, MaterialIndex,
						BakingMeshData,
						ExportFolderPath);
				}
			}
		}
	}

	MaterialIndexToFbxMaterials->Add(MaterialIndex, FbxMaterial);

	return FbxMaterial;
}


FFbxExporter::FLevelSequenceNodeNameAdapter::FLevelSequenceNodeNameAdapter( UMovieScene* InMovieScene, IMovieScenePlayer* InMovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID )
{
	MovieScene = InMovieScene;
	MovieScenePlayer = InMovieScenePlayer;
	SequenceID = InSequenceID;
}

FString FFbxExporter::FLevelSequenceNodeNameAdapter::GetActorNodeName(const AActor* Actor)
{
	FString NodeName = Actor->GetName();

	for ( const FMovieSceneBinding& MovieSceneBinding : ((const UMovieScene*)MovieScene)->GetBindings() )
	{
		for ( TWeakObjectPtr<UObject> RuntimeObject : MovieScenePlayer->FindBoundObjects(MovieSceneBinding.GetObjectGuid(), SequenceID) )
		{
			if (RuntimeObject.Get() == Actor)
			{
				NodeName = MovieScene->GetObjectDisplayName(MovieSceneBinding.GetObjectGuid()).ToString();
			}
		}
	}

	// Maya does not support dashes.  Change all dashes to underscores		
	NodeName = NodeName.Replace(TEXT("-"), TEXT("_") );

	// Maya does not support spaces.  Change all spaces to underscores		
	NodeName = NodeName.Replace(TEXT(" "), TEXT("_") );

	return NodeName;
}

void FFbxExporter::FLevelSequenceNodeNameAdapter::AddFbxNode(UObject* InObject, FbxNode* FbxNode)
{
	FGuid ObjectGuid = MovieScenePlayer->FindObjectId(*InObject, SequenceID);
	if (ObjectGuid.IsValid())
	{
		if (!GuidToFbxNodeMap.Contains(ObjectGuid))
		{
			GuidToFbxNodeMap.Add(ObjectGuid);
		}
		GuidToFbxNodeMap[ObjectGuid] = FbxNode;
	}
}

FbxNode* FFbxExporter::FLevelSequenceNodeNameAdapter::GetFbxNode(UObject* InObject)
{
	FGuid ObjectGuid = MovieScenePlayer->FindObjectId(*InObject, SequenceID);
	if (ObjectGuid.IsValid())
	{
		if (GuidToFbxNodeMap.Contains(ObjectGuid))
		{
			return GuidToFbxNodeMap[ObjectGuid];
		}
	}
	return nullptr;
}

FLevelSequenceAnimTrackAdapter::FLevelSequenceAnimTrackAdapter(const FAnimTrackSettings& Settings)
{
	MovieScenePlayer = Settings.MovieScenePlayer;
	MovieSceneSequence = Settings.MovieSceneSequence;
	RootMovieSceneSequence = Settings.RootMovieSceneSequence;
	MovieScene = Settings.MovieSceneSequence->GetMovieScene();
	RootToLocalTransform = Settings.RootToLocalTransform;
	AnimTrack = Settings.AnimTrack;
	bForceUseOfMovieScenePlaybackRange = Settings.bForceUseOfMovieScenePlaybackRange;
}


//5.6 deprecrated
FLevelSequenceAnimTrackAdapter::FLevelSequenceAnimTrackAdapter(IMovieScenePlayer* InMovieScenePlayer, UMovieSceneSequence* InMovieSceneSequence, UMovieSceneSequence* InRootMovieSceneSequence, const FMovieSceneSequenceTransform& InRootToLocalTransform, UMovieSceneSkeletalAnimationTrack* InAnimTrack)
{
	MovieScenePlayer = InMovieScenePlayer;
	MovieSceneSequence = InMovieSceneSequence;
	RootMovieSceneSequence = InRootMovieSceneSequence;
	MovieScene = MovieSceneSequence->GetMovieScene();
	RootToLocalTransform = InRootToLocalTransform;
	AnimTrack = InAnimTrack;
	bForceUseOfMovieScenePlaybackRange = false;
}

//5.5 deprecrated
FLevelSequenceAnimTrackAdapter::FLevelSequenceAnimTrackAdapter(IMovieScenePlayer* InMovieScenePlayer, UMovieScene* InMovieScene, const FMovieSceneSequenceTransform& InRootToLocalTransform, UMovieSceneSkeletalAnimationTrack* InAnimTrack)
{
	MovieScenePlayer = InMovieScenePlayer;
	MovieScene = InMovieScene;
	RootToLocalTransform = InRootToLocalTransform;
	AnimTrack = InAnimTrack;
	MovieSceneSequence = nullptr;
	RootMovieSceneSequence = nullptr;
	if (MovieScene)
	{
		if (UMovieSceneSequence* OwnerSceneSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>())
		{
			MovieSceneSequence = OwnerSceneSequence;
			RootMovieSceneSequence = OwnerSceneSequence;
		}
	}
	bForceUseOfMovieScenePlaybackRange = false;
}

void FLevelSequenceAnimTrackAdapter::SetRange(const FFrameNumber& StartFrame, const FFrameNumber& EndFrame)
{
	TPair<FFrameNumber, FFrameNumber> Range(StartFrame, EndFrame);
	OptionalRange = Range;
}

TRange<FFrameNumber> FLevelSequenceAnimTrackAdapter::GetSequenceRange() const
{
	if (OptionalRange.IsSet())
	{
		TRange<FFrameNumber> SetRange(OptionalRange.GetValue().Key, (OptionalRange.GetValue().Value));
		return SetRange;
	}

	FMovieSceneEvaluationState* State = MovieScenePlayer->GetEvaluationState();
	State->AssignSequence(MovieSceneSequenceID::Root, *RootMovieSceneSequence, *MovieScenePlayer);
	FMovieSceneSequenceIDRef Template = State->FindSequenceId(MovieSceneSequence);

	FMovieSceneSequenceHierarchy Hierarchy = FMovieSceneSequenceHierarchy();
	UMovieSceneCompiledDataManager::CompileHierarchy(RootMovieSceneSequence, &Hierarchy,EMovieSceneServerClientMask::All);

	const FMovieSceneSubSequenceData* SubSequenceData = Hierarchy.FindSubData(Template);
	TRange<FFrameNumber> Range = MovieScene->GetPlaybackRange();
	FFrameTime StartTime = FFrameRate::TransformTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value, MovieScene->GetTickResolution(), GetDisplayRate());

	if (SubSequenceData && bForceUseOfMovieScenePlaybackRange == false)
	{
		Range = SubSequenceData->PlayRange.Value;
	}
	return Range;
}
int32 FLevelSequenceAnimTrackAdapter::GetLocalStartFrame() const
{
	TRange<FFrameNumber> Range = GetSequenceRange();

	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();
	return FFrameRate::TransformTime(FFrameTime(UE::MovieScene::DiscreteInclusiveLower(Range)), TickResolution, DisplayRate).RoundToFrame().Value;
}

int32 FLevelSequenceAnimTrackAdapter::GetStartFrame() const
{
	TRange<FFrameNumber> Range = GetSequenceRange();
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();
	FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(Range);

	FMovieSceneInverseSequenceTransform Inverse = RootToLocalTransform.Inverse();

	TOptional<FFrameTime> RootTime = Inverse.TryTransformTime(StartFrame);
	return FFrameRate::TransformTime(RootTime.Get(StartFrame), TickResolution, DisplayRate).RoundToFrame().Value;
}

int32 FLevelSequenceAnimTrackAdapter::GetLength() const
{
	TRange<FFrameNumber> Range = GetSequenceRange();
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();

	return FFrameRate::TransformTime(FFrameTime(UE::MovieScene::DiscreteSize(Range)), TickResolution, DisplayRate).RoundToFrame().Value;
}

void FLevelSequenceAnimTrackAdapter::UpdateAnimation( int32 LocalFrame )
{	
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();

	FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);
	TOptional<FFrameTime> GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(LocalTime);

	if (GlobalTime)
	{
		FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime.GetValue(), TickResolution), MovieScenePlayer->GetPlaybackStatus()).SetHasJumped(true);
		MovieScenePlayer->GetEvaluationTemplate().EvaluateSynchronousBlocking( Context );
	}
}

void FLevelSequenceAnimTrackAdapter::SetFrameRate(FFrameRate InFrameRate)
{
	OptionalFrameRate = InFrameRate;
}

double FLevelSequenceAnimTrackAdapter::GetFrameRate() const
{
	return OptionalFrameRate.IsSet() ? OptionalFrameRate.GetValue().AsDecimal() : MovieScene->GetDisplayRate().AsDecimal();
}

FFrameRate FLevelSequenceAnimTrackAdapter::GetDisplayRate() const
{
	return OptionalFrameRate.IsSet() ? OptionalFrameRate.GetValue() : MovieScene->GetDisplayRate();
}
		
UAnimSequence* FLevelSequenceAnimTrackAdapter::GetAnimSequence(int32 LocalFrame) const
{
	if (!AnimTrack)
	{
		return nullptr;
	}

	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();

	FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);

	for (UMovieSceneSection* Section : AnimTrack->GetAnimSectionsAtTime(LocalTime.FrameNumber))
	{
		if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
		{
			return Cast<UAnimSequence>(AnimSection->Params.Animation);
		}
	}

	return nullptr;
}

float FLevelSequenceAnimTrackAdapter::GetAnimTime(int32 LocalFrame) const
{
	if (!AnimTrack)
	{
		return 0.f;
	}

	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameRate DisplayRate = GetDisplayRate();

	FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);

	for (UMovieSceneSection* Section : AnimTrack->GetAnimSectionsAtTime(LocalTime.FrameNumber))
	{
		if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
		{
			if (AnimSection->Params.Animation)
			{
				return static_cast<float>(AnimSection->MapTimeToAnimation(LocalTime, TickResolution));
			}
		}
	}

	return 0.f;
}
bool FFbxExporter::ExportLevelSequenceTracks(UMovieScene* MovieScene, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, FbxNode* FbxActor, UObject* BoundObject, const TArray<UMovieSceneTrack*>& Tracks, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (MovieScene)
	{
		if (UMovieSceneSequence* OwnerSceneSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>())
		{
			return ExportLevelSequenceTracks(OwnerSceneSequence, OwnerSceneSequence, MovieScenePlayer, InSequenceID, FbxActor, BoundObject, Tracks,  RootToLocalTransform);
		}
	}
	return false;
}

bool FFbxExporter::ExportLevelSequenceTracks(UMovieSceneSequence* MovieSceneSequence, UMovieSceneSequence* RootMovieSceneSequence, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, FbxNode* FbxActor, UObject* BoundObject, const TArray<UMovieSceneTrack*>& Tracks, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	AActor* Actor = Cast<AActor>(BoundObject);
	if (!Actor)
	{
		UActorComponent* Component = Cast<UActorComponent>(BoundObject);
		if (Component)
		{
			Actor = Component->GetOwner();
		}
	}

	USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(BoundObject);
	if (!SkeletalMeshComp)
	{
		SkeletalMeshComp = Actor ? Cast<USkeletalMeshComponent>(Actor->GetComponentByClass(USkeletalMeshComponent::StaticClass())) : nullptr;	
	}

	const UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
	FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	bool bSkip3DTransformTrack = SkeletalMeshComp && GetExportOptions()->MapSkeletalMotionToRoot;

	// Get all the transform tracks that affect this binding
	TArray<TWeakObjectPtr<UMovieScene3DTransformTrack> > TransformTracks;
	if (BoundObject)
	{
		for ( const FMovieSceneBinding& MovieSceneBinding : MovieScene->GetBindings() )
		{
			for ( TWeakObjectPtr<UObject> RuntimeObject : MovieScenePlayer->FindBoundObjects(MovieSceneBinding.GetObjectGuid(), InSequenceID) )
			{
				if (RuntimeObject.IsValid())
				{
					AActor* RuntimeActor = Cast<AActor>(RuntimeObject);
					UActorComponent* RuntimeComponent = nullptr;
					if (!RuntimeActor)
					{
						RuntimeComponent = Cast<UActorComponent>(RuntimeObject);
						if (RuntimeComponent)
						{
							RuntimeActor = RuntimeComponent->GetOwner();
						}
					}

					if (RuntimeActor == Actor || RuntimeComponent == BoundObject)
					{
						for (UMovieSceneTrack* Track : MovieSceneBinding.GetTracks())
						{
							if (Track->IsA(UMovieScene3DTransformTrack::StaticClass()))
							{
								TransformTracks.Add(Cast<UMovieScene3DTransformTrack>(Track));
							}
						}
					}
				}
			}
		}

		// Also need to skip 3d transform track if this object is an actor and the component has a transform track because otherwise 
		if (BoundObject->IsA<AActor>())
		{
			for ( const FMovieSceneBinding& MovieSceneBinding : MovieScene->GetBindings() )
			{
				for ( TWeakObjectPtr<UObject> RuntimeObject : MovieScenePlayer->FindBoundObjects(MovieSceneBinding.GetObjectGuid(), InSequenceID) )
				{
					if (RuntimeObject.IsValid())
					{
						UActorComponent* RuntimeComponent = Cast<UActorComponent>(RuntimeObject);
						if (RuntimeComponent && RuntimeComponent->GetOwner() == BoundObject)
						{
							for (UMovieSceneTrack* Track : MovieSceneBinding.GetTracks())
							{
								if (Track->IsA(UMovieScene3DTransformTrack::StaticClass()))
								{
									bSkip3DTransformTrack = true;
								}
							}
						}
					}
				}
			}
		}
	}

	AActor* BoundActor = Cast<AActor>(BoundObject);
	USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObject);

	const bool bIsCameraActor = BoundActor ? BoundActor->IsA(ACameraActor::StaticClass()) : BoundComponent ? BoundComponent->IsA(UCameraComponent::StaticClass()) : false;
	const bool bIsLightActor = BoundActor ? BoundActor->IsA(ALight::StaticClass()) : BoundComponent ? BoundComponent->IsA(ULightComponent::StaticClass()) : false;

	bool bBakeAllTransformChannels = false;
	if (bIsCameraActor || bIsLightActor)
	{
		bBakeAllTransformChannels = (static_cast<uint8>(GetExportOptions()->BakeCameraAndLightAnimation) & static_cast<uint8>(EMovieSceneBakeType::BakeTransforms));
	}
	else
	{
		bBakeAllTransformChannels = (static_cast<uint8>(GetExportOptions()->BakeActorAnimation) & static_cast<uint8>(EMovieSceneBakeType::BakeTransforms));
	}

	// If this actor has attached actors, and the bound component isn't the root component, skip the 3d transform track because we don't want the 
	// component transform to be assigned to the actor, which would subsequently apply to the attached children 
	if (Actor && BoundComponent && Actor->GetRootComponent() != BoundComponent)
	{
		TArray<AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);

		if (AttachedActors.Num() != 0)
		{
			bSkip3DTransformTrack = true;		
		}
	}

	// If there's more than one transform track for this actor (ie. on the actor and on the root component) or if there's more than one section, evaluate baked
	if (bBakeAllTransformChannels || TransformTracks.Num() > 1 || (TransformTracks.Num() != 0 && TransformTracks[0].Get()->GetAllSections().Num() > 1))
	{
		if (!bSkip3DTransformTrack)
		{
			UnFbx::FLevelSequenceAnimTrackAdapter::FAnimTrackSettings Settings;
			Settings.MovieScenePlayer = MovieScenePlayer;
			Settings.MovieSceneSequence = MovieSceneSequence;
			Settings.RootMovieSceneSequence = RootMovieSceneSequence;
			Settings.RootToLocalTransform = RootToLocalTransform;
			Settings.bForceUseOfMovieScenePlaybackRange = false;
			FLevelSequenceAnimTrackAdapter AnimTrackAdapter(Settings);
			ExportLevelSequenceBaked3DTransformTrack(AnimTrackAdapter, FbxActor, MovieScenePlayer, InSequenceID, TransformTracks, BoundObject, MovieScene->GetPlaybackRange(), RootToLocalTransform);
		}

		bSkip3DTransformTrack = true;
	}

	// Look for the tracks that we currently support
	UMovieSceneSkeletalAnimationTrack* SkeletalAnimationTrack = nullptr;
	bool IsControlRigTrack = false;
	for (UMovieSceneTrack* Track : Tracks)
	{
		if (Track->IsA(UMovieScene3DTransformTrack::StaticClass()))
		{
			if (!bSkip3DTransformTrack)
			{
				UMovieScene3DTransformTrack* TransformTrack = (UMovieScene3DTransformTrack*)Track;
				ExportLevelSequence3DTransformTrack(FbxActor, MovieScenePlayer, InSequenceID, *TransformTrack, BoundObject, MovieScene->GetPlaybackRange(), RootToLocalTransform);
			}
		}
		else if (Track->IsA(UMovieSceneSkeletalAnimationTrack::StaticClass()))
		{
			SkeletalAnimationTrack = Cast<UMovieSceneSkeletalAnimationTrack>(Track);
		}
		else if (Track->IsA(UMovieSceneColorTrack::StaticClass()))
		{
			ExportLevelSequenceColorTrack(FbxActor, *Cast<UMovieSceneColorTrack>(Track), BoundObject, MovieScene->GetPlaybackRange(), RootToLocalTransform);
		}
		else if (Track->IsA(UMovieSceneDoubleVectorTrack::StaticClass()))
		{
			ExportLevelSequenceVectorTrack(FbxActor, *Cast<UMovieSceneDoubleVectorTrack>(Track), BoundObject, MovieScene->GetPlaybackRange(), RootToLocalTransform);
		}
		else if(Cast<INodeAndChannelMappings>(Track))
		{
			IsControlRigTrack = true;
		}
		else
		{
			bool bBakeChannels = false;
			if (bIsCameraActor || bIsLightActor)
			{
				bBakeChannels = (static_cast<uint8>(GetExportOptions()->BakeCameraAndLightAnimation) & static_cast<uint8>(EMovieSceneBakeType::BakeChannels));
			}
			else
			{
				bBakeChannels = (static_cast<uint8>(GetExportOptions()->BakeActorAnimation) & static_cast<uint8>(EMovieSceneBakeType::BakeChannels));
			}
			ExportLevelSequenceTrackChannels(FbxActor, *Track, MovieScene->GetPlaybackRange(), RootToLocalTransform, bBakeChannels);
		}
	}

	// Export all of the skeletal animation components for this actor
	if (SkeletalMeshComp && (SkeletalAnimationTrack || IsControlRigTrack))
	{
		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		SkeletalMeshComp->GetOwner()->GetComponents(SkeletalMeshComponents);
		UnFbx::FLevelSequenceAnimTrackAdapter::FAnimTrackSettings Settings;
		Settings.MovieScenePlayer = MovieScenePlayer;
		Settings.MovieSceneSequence = MovieSceneSequence;
		Settings.RootMovieSceneSequence = RootMovieSceneSequence;
		Settings.RootToLocalTransform = RootToLocalTransform;
		Settings.bForceUseOfMovieScenePlaybackRange = false;
		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			Settings.AnimTrack = SkeletalAnimationTrack;
			FLevelSequenceAnimTrackAdapter AnimTrackAdapter(Settings);
			ExportAnimTrack(AnimTrackAdapter, Actor, SkeletalMeshComponent, 1.0 / DisplayRate.AsDecimal());
		}
	}

	return true;
}
bool FFbxExporter::ExportLevelSequence(UMovieScene* MovieScene, const TArray<FGuid>& Bindings, IMovieScenePlayer* MovieScenePlayer, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (MovieScene)
	{
		if (UMovieSceneSequence* OwnerSceneSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>())
		{
			return ExportLevelSequence(OwnerSceneSequence, OwnerSceneSequence, Bindings, MovieScenePlayer, NodeNameAdapter, SequenceID, RootToLocalTransform);
		}
	}
	return false;
}

bool FFbxExporter::ExportLevelSequence(UMovieSceneSequence* MovieSceneSequence, UMovieSceneSequence* RootMovieSceneSequence, const TArray<FGuid>& Bindings, IMovieScenePlayer* MovieScenePlayer, INodeNameAdapter& NodeNameAdapter, FMovieSceneSequenceIDRef SequenceID, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (MovieSceneSequence == nullptr || RootMovieSceneSequence == nullptr || MovieScenePlayer == nullptr)
	{
		return false;
	}
	const UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
	double FrameRate = MovieScene->GetDisplayRate().AsDecimal();
	FbxTime::EMode TimeMode = FbxTime::ConvertFrameRateToTimeMode(FrameRate);

	// Unknown frame rates, (ie. 12 fps) return eDefaultMode, so override with custom so that the custom frame rate takes effect
	if (TimeMode == FbxTime::eDefaultMode)
	{
		TimeMode = FbxTime::eCustom;
	}

	Scene->GetGlobalSettings().SetTimeMode(TimeMode);
	if (TimeMode == FbxTime::eCustom)
	{
		Scene->GetGlobalSettings().SetCustomFrameRate(FrameRate);
	}

	for (const FMovieSceneBinding& MovieSceneBinding : MovieScene->GetBindings())
	{
		// If there are specific bindings to export, export those only
		if (Bindings.Num() != 0 && !Bindings.Contains(MovieSceneBinding.GetObjectGuid()))
		{
			continue;
		}

		bool bAnyBindingsExported = false;

		for (TWeakObjectPtr<UObject> RuntimeObject : MovieScenePlayer->FindBoundObjects(MovieSceneBinding.GetObjectGuid(), SequenceID))
		{
			if (RuntimeObject.IsValid())
			{
				AActor* Actor = Cast<AActor>(RuntimeObject.Get());
				UActorComponent* Component = Cast<UActorComponent>(RuntimeObject.Get());
				if (Actor == nullptr && Component != nullptr)
				{
					Actor = Component->GetOwner();
				}

				if (Actor == nullptr)
				{
					continue;
				}

				FbxNode* FbxActor = FindActor(Actor, &NodeNameAdapter);

				// now it should export everybody
				if (FbxActor)
				{
					ExportLevelSequenceTracks(MovieSceneSequence, RootMovieSceneSequence, MovieScenePlayer, SequenceID, FbxActor, RuntimeObject.Get(), MovieSceneBinding.GetTracks(), RootToLocalTransform);
					bAnyBindingsExported = true;
				}
			}
		}

		// If no bindings exported, create a dummy actor to export tracks onto
		if (!bAnyBindingsExported)
		{
			ExportLevelSequenceTracks(MovieSceneSequence, RootMovieSceneSequence, MovieScenePlayer, SequenceID, nullptr, nullptr, MovieSceneBinding.GetTracks(), RootToLocalTransform);
		}
	}

	return true;
}

void FFbxExporter::AddTimecodeAttributesAndSetKey(const UMovieSceneSection* InSection, FbxNode* InFbxNode, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);
	const FFrameRate TickResolution = InSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
	
	FName TCHourAttrName(TEXT("TCHour"));
	FName TCMinuteAttrName(TEXT("TCMinute"));
	FName TCSecondAttrName(TEXT("TCSecond"));
	FName TCFrameAttrName(TEXT("TCFrame"));

	if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
	{
		TCHourAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.HourAttributeName;
		TCMinuteAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.MinuteAttributeName;
		TCSecondAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.SecondAttributeName;
		TCFrameAttrName = AnimationSettings->BoneTimecodeCustomAttributeNameSettings.FrameAttributeName;
	}

	const TArray<FString> TimecodePropertyNames = {
		TCHourAttrName.ToString(),
		TCMinuteAttrName.ToString(),
		TCSecondAttrName.ToString(),
		TCFrameAttrName.ToString()
	};

	const TArray<int32> TimecodeValues = {
		InSection->TimecodeSource.Timecode.Hours,
		InSection->TimecodeSource.Timecode.Minutes,
		InSection->TimecodeSource.Timecode.Seconds,
		InSection->TimecodeSource.Timecode.Frames
	};

	const FFrameNumber KeyTime = InSection->GetTypedOuter<UMovieScene>()->GetPlaybackRange().GetLowerBoundValue();
		
	for (int i = 0; i < TimecodePropertyNames.Num(); i++)
	{
		const FString PropertyName = TimecodePropertyNames[i];
		CreateAnimatableUserProperty(InFbxNode, 0.0, StringCast<char>(*PropertyName).Get(), StringCast<char>(*PropertyName).Get(), FbxFloatDT);
		FbxProperty Property = InFbxNode->FindProperty(StringCast<char>(*PropertyName).Get());

		if (Property.IsValid())
		{
			if (FbxAnimCurve* AnimCurve = Property.GetCurve(BaseLayer, nullptr, true))
			{
				const float KeyValue = (float)TimecodeValues[i];

				AnimCurve->KeyModifyBegin();

				FbxTime FbxTime;

				TOptional<FFrameTime> GlobalKeyTime = RootToLocalTransform.Inverse().TryTransformTime(KeyTime);

				const double KeyTimeSeconds = GetExportOptions()->bExportLocalTime ? KeyTime / TickResolution : GlobalKeyTime.Get(KeyTime) / TickResolution;

				FbxTime.SetSecondDouble(KeyTimeSeconds);

				const int FbxKeyIndex = AnimCurve->KeyAdd(FbxTime);
				AnimCurve->KeySet(FbxKeyIndex, FbxTime, KeyValue, FbxAnimCurveDef::eInterpolationConstant);
				AnimCurve->KeySetConstantMode(FbxKeyIndex, FbxAnimCurveDef::EConstantMode::eConstantStandard);

				AnimCurve->KeyModifyEnd();
			}
		}
	}
}

bool FFbxExporter::ExportControlRigSection(const UMovieSceneSection* Section, const TArray<FControlRigFbxNodeMapping>& ChannelsMapping,
	const TArray<FName>& FilterControls, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	if (!Section)
	{
		return false;
	}
	
	UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>();

	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);

	const FFrameRate TickResolution = Track->GetTypedOuter<UMovieScene>()->GetTickResolution();

	// Helpers to convert channel/property names to indices and vice versa
	const TArray<FString> TransformComponentsArray = {"X", "Y", "Z"};
	const TArray<FString> TransformPropertyArray = {"Location", "Rotation", "Scale"};

	// Cached transform channels for a single property to bake and export the 3 channels in one go
	TPair<FMovieSceneFloatChannel*, bool> CachedTransformChannels[3] = {{nullptr, false}, {nullptr, false}, {nullptr, false}};
	// Cached transform property index keep track of which property we are currently caching to bake & export Transforms & TransformNoScale
	int CurrentTmPropertyIndex = 0;
	
	// The control rig parent group node
	FbxNode* RootFbxNode = CreateNode(Track->GetDisplayName().ToString());

	// Export timecode
	AddTimecodeAttributesAndSetKey(Section, RootFbxNode, RootToLocalTransform);
	
	const FName BoolChannelTypeName = FMovieSceneBoolChannel::StaticStruct()->GetFName();
	const FName ByteChannelTypeName = FMovieSceneByteChannel::StaticStruct()->GetFName();
	const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();
	const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
	const FName IntegerChannelTypeName = FMovieSceneIntegerChannel::StaticStruct()->GetFName();
	
	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
	{
		TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> AllMetaData = Entry.GetMetaData();

		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			const FName ChannelTypeName = Entry.GetChannelTypeName();
			FMovieSceneChannelHandle ChannelHandle = ChannelProxy.MakeHandle(ChannelTypeName, Index);

			FMovieSceneBoolChannel* BoolChannel = ChannelTypeName == BoolChannelTypeName ? ChannelHandle.Cast<FMovieSceneBoolChannel>().Get() : nullptr;
			FMovieSceneByteChannel* ByteChannel = ChannelTypeName == ByteChannelTypeName ? ChannelHandle.Cast<FMovieSceneByteChannel>().Get() : nullptr;
			FMovieSceneDoubleChannel* DoubleChannel = ChannelTypeName == DoubleChannelTypeName ? ChannelHandle.Cast<FMovieSceneDoubleChannel>().Get() : nullptr;
			FMovieSceneFloatChannel* FloatChannel = ChannelTypeName == FloatChannelTypeName ? ChannelHandle.Cast<FMovieSceneFloatChannel>().Get() : nullptr;
			FMovieSceneIntegerChannel* IntegerChannel = ChannelTypeName == IntegerChannelTypeName ? ChannelHandle.Cast<FMovieSceneIntegerChannel>().Get() : nullptr;

			if (!BoolChannel && !ByteChannel && !DoubleChannel && !FloatChannel && !IntegerChannel)
			{
				continue;
			}

			// Find the node and attribute names
			const FMovieSceneChannelMetaData& MetaData = AllMetaData[Index];
			const FString ChannelName = MetaData.Name.ToString();

			FControlRigFbxCurveData FbxCurveData;
			if (INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(Track))
			{
				if (!ChannelMapping->GetFbxCurveDataFromChannelMetadata(MetaData, FbxCurveData))
				{
					continue;
				}
			}

			// Skip any control not in the filter if the filter isn't empty
			if (!FilterControls.IsEmpty() && !FilterControls.Contains(FbxCurveData.ControlName))
			{
				continue;
			}

			// Find or create the node
			FbxNode* ControlFbxNode = RootFbxNode->FindChild(StringCast<char>(*FbxCurveData.NodeName).Get());
			if (ControlFbxNode == nullptr)
			{
				ControlFbxNode = FbxNode::Create(Scene, StringCast<char>(*FbxCurveData.NodeName).Get());
				RootFbxNode->AddChild(ControlFbxNode);
			}

			// Remap channel, optionally baking & exporting it for transforms & rotations 
			bool bNegate = false;
			int TmComponentIndex = TransformComponentsArray.Find(FbxCurveData.AttributeName);

			// Find the attribute to remap onto and whether to negate the channel if required
			const FControlRigFbxNodeMapping* NodeMapping = ChannelsMapping.FindByPredicate(
				[ChannelTypeName, FbxCurveData, TmComponentIndex](const FControlRigFbxNodeMapping& AMapping)
			{
				const bool bAttrMatch = AMapping.ChannelAttrIndex == TmComponentIndex;
				const bool bControlTypeOk = AMapping.ControlType == FbxCurveData.ControlType;
				// todo: am: need to allow mapping of position/rotation/scale for transforms & euler transforms to allow export of negated transform curves
				const bool bChannelTypeOk = AMapping.ChannelType == ChannelTypeName;

				// If the control is treated as an attribute of another control, we don't need to check the attribute to remap
				const bool bAttrOk = !FbxCurveData.IsControlNode() || bAttrMatch;
					
				const bool bRemapControl = bControlTypeOk && bChannelTypeOk && bAttrOk;
				return bRemapControl;
			});
			
			if (TmComponentIndex != INDEX_NONE)
			{
				// Retrieve property name for Vector2D, Position, Scale
				if (FbxCurveData.ControlType == FFBXControlRigTypeProxyEnum::Vector2D || FbxCurveData.ControlType == FFBXControlRigTypeProxyEnum::Position)
				{
					FbxCurveData.AttributePropertyName = "Location";
				}
				if (FbxCurveData.ControlType == FFBXControlRigTypeProxyEnum::Scale)
				{
					FbxCurveData.AttributePropertyName = "Scale";
				}
				
				// Export converted & baked for Rotations & Transforms
				if ((uint8)FbxCurveData.ControlType >= (uint8)FFBXControlRigTypeProxyEnum::Rotator)
				{
					const bool bExportCachedTransforms = TmComponentIndex == 2;
					const bool bRotator = FbxCurveData.ControlType == FFBXControlRigTypeProxyEnum::Rotator;
					
					int TmPropertyIndex = bRotator ? 1 : CurrentTmPropertyIndex;

					// todo: am: does not handle remapping to another property (i.e. rotation to location)
					// Remap transform channel
					if (NodeMapping)
					{
						TmPropertyIndex = NodeMapping->FbxAttrIndex / 3;
						TmComponentIndex = NodeMapping->FbxAttrIndex % 3;
						bNegate = NodeMapping->bNegate;
					}

					// Cache a single transform property channel
					CachedTransformChannels[TmComponentIndex].Key = FloatChannel;
					CachedTransformChannels[TmComponentIndex].Value = bNegate;

					// Once all 3 channels in the current property have been cached, export baked
					if (bExportCachedTransforms)
					{
						for (int i = 0; i < 3; i++)
						{
							if (CachedTransformChannels[i].Key && CachedTransformChannels[i].Key->GetNumKeys() > 0)
							{
								// Only export baked if at least one curve has keys
								ExportTransformChannelsToFbxCurve(ControlFbxNode, CachedTransformChannels[0],
									CachedTransformChannels[1], CachedTransformChannels[2],
									TmPropertyIndex, Track, RootToLocalTransform);
								
								break;
							}
						}

						// Increase or reset CurrentTmPropertyIndex to keep track of which property in a Transform we are currently caching
						const bool bNoScale = FbxCurveData.ControlType == FFBXControlRigTypeProxyEnum::TransformNoScale;
						const bool bTransformProcessed = bRotator || CurrentTmPropertyIndex == 2 || (bNoScale && CurrentTmPropertyIndex == 1);
						CurrentTmPropertyIndex = bTransformProcessed ? 0 : CurrentTmPropertyIndex + 1;
					}

					continue;
				}
			}
			// Remap non transform channels if required (rotations & transforms remapped separately)
			if (NodeMapping)
			{
				FbxCurveData.AttributePropertyName = TransformPropertyArray[NodeMapping->FbxAttrIndex / 3];
				FbxCurveData.AttributeName = TransformComponentsArray[NodeMapping->FbxAttrIndex % 3];
				bNegate = NodeMapping->bNegate;
			}
			
			FbxProperty Property;
			
			// Use AttributeName (i.e. Weight) as the property if no AttributePropertyName (i.e. Scale). Use NodeName if AttributeName was also empty...
			const FString FbxPropertyName = FbxCurveData.AttributePropertyName.IsEmpty() ? (!FbxCurveData.AttributeName.IsEmpty() ? FbxCurveData.AttributeName : FbxCurveData.NodeName) : FbxCurveData.AttributePropertyName;

			// Try and find the existing property
			if (FbxCurveData.AttributePropertyName == "Location")
			{
				Property = ControlFbxNode->LclTranslation;
			}
			else if (FbxCurveData.AttributePropertyName == "Rotation")
			{
				Property = ControlFbxNode->LclRotation;
			}
			else if (FbxCurveData.AttributePropertyName == "Scale")
			{
				Property = ControlFbxNode->LclScaling;
			}
			else
			{
				Property = ControlFbxNode->FindProperty(StringCast<char>(*FbxPropertyName).Get());
			}

			// Or create it otherwise
			if (!Property.IsValid())
			{
				if (DoubleChannel)
				{
					double Default = DoubleChannel->GetDefault().Get(0);
					CreateAnimatableUserProperty(ControlFbxNode, Default, StringCast<char>(*FbxPropertyName).Get(), StringCast<char>(*FbxPropertyName).Get(), FbxDoubleDT);
				}
				else if (FloatChannel)
				{
					float Default = FloatChannel->GetDefault().Get(0);
					CreateAnimatableUserProperty(ControlFbxNode, Default, StringCast<char>(*FbxPropertyName).Get(), StringCast<char>(*FbxPropertyName).Get(), FbxFloatDT);
				}
				else if (IntegerChannel)
				{
					int32 Default = IntegerChannel->GetDefault().Get(0);
					CreateAnimatableUserProperty(ControlFbxNode, Default, StringCast<char>(*FbxPropertyName).Get(), StringCast<char>(*FbxPropertyName).Get(), FbxIntDT);
				}
				else if (BoolChannel)
				{
					bool Default = BoolChannel->GetDefault().Get(false);
					CreateAnimatableUserProperty(ControlFbxNode, Default, StringCast<char>(*FbxPropertyName).Get(), StringCast<char>(*FbxPropertyName).Get(), FbxBoolDT);
				}
				else if (ByteChannel)
				{
					uint8 Default = ByteChannel->GetDefault().Get(0);
					CreateAnimatableUserProperty(ControlFbxNode, Default, StringCast<char>(*FbxPropertyName).Get(), StringCast<char>(*FbxPropertyName).Get(), FbxEnumDT);
				}

				Property = ControlFbxNode->FindProperty(StringCast<char>(*FbxPropertyName).Get());
			}

			if (!Property.IsValid())
			{
				continue;
			}
			
			// Create the anim curve, optionally of the given name if the attribute had a component
			FbxAnimCurveNode* AnimCurveNode = Property.GetCurveNode(BaseLayer, true);
			FbxAnimCurve* AnimCurve = Property.GetCurve(BaseLayer, FbxCurveData.AttributePropertyName.IsEmpty() ? nullptr : StringCast<char>(*FbxCurveData.AttributeName).Get(), true);
			if (!AnimCurve)
			{
				continue;
			}

			// Export the curve
			if (BoolChannel)
			{
				ExportChannelToFbxCurve(*AnimCurve, *BoolChannel, TickResolution, RootToLocalTransform);
			}
			else if (ByteChannel)
			{
				ExportChannelToFbxCurve(*AnimCurve, *ByteChannel, TickResolution, RootToLocalTransform);
			}
			else if (DoubleChannel)
			{
				ExportChannelToFbxCurve(*AnimCurve, *DoubleChannel, TickResolution, ERichCurveValueMode::Default, bNegate, RootToLocalTransform);
			}
			else if (FloatChannel)
			{
				ExportChannelToFbxCurve(*AnimCurve, *FloatChannel, TickResolution, ERichCurveValueMode::Default, bNegate, RootToLocalTransform);
			}
			else if (IntegerChannel)
			{
				ExportChannelToFbxCurve(*AnimCurve, *IntegerChannel, TickResolution, RootToLocalTransform);
			}
		}
	}
	return true;
}

void FFbxExporter::ExportTransformChannelsToFbxCurve(FbxNode* InFbxNode, TPair<FMovieSceneFloatChannel*, bool> ChannelX, TPair<FMovieSceneFloatChannel*, bool> ChannelY, TPair<FMovieSceneFloatChannel*, bool> ChannelZ, int TmPropertyIndex, const UMovieSceneTrack* Track, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);

	FbxProperty Property = TmPropertyIndex == 0 ? InFbxNode->LclTranslation : TmPropertyIndex == 1 ? InFbxNode->LclRotation : InFbxNode->LclScaling;
	
	FbxAnimCurve* FbxCurveX = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveY = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveZ = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FTransform RotationDirectionConvert;

	FbxCurveX->KeyModifyBegin();
	FbxCurveY->KeyModifyBegin();
	FbxCurveZ->KeyModifyBegin();

	FFrameRate TickResolution = Track->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameRate DisplayRate = Track->GetTypedOuter<UMovieScene>()->GetDisplayRate();
	TRange<FFrameNumber> PlaybackRange = Track->GetTypedOuter<UMovieScene>()->GetPlaybackRange();

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToLocalTransform.Inverse();

	int32 LocalStartFrame = FFrameRate::TransformTime(FFrameTime(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange)), TickResolution, DisplayRate).RoundToFrame().Value;
	int32 AnimationLength = FFrameRate::TransformTime(FFrameTime(FFrameNumber(UE::MovieScene::DiscreteSize(PlaybackRange))), TickResolution, DisplayRate).RoundToFrame().Value;

	for (int32 FrameCount = 0; FrameCount <= AnimationLength; ++FrameCount)
	{
		int32 LocalFrame = LocalStartFrame + FrameCount;

		FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);

		FVector3f Vec = FVector3f::ZeroVector;
		if (ChannelX.Key)
		{
			ChannelX.Key->Evaluate(LocalTime, Vec.X);
			if (ChannelX.Value)
			{
				Vec.X = -Vec.X;
			}
		}
		if (ChannelY.Key)
		{
			ChannelY.Key->Evaluate(LocalTime, Vec.Y);
			if (ChannelY.Value)
			{
				Vec.Y = -Vec.Y;
			}
		}
		if (ChannelZ.Key)
		{
			ChannelZ.Key->Evaluate(LocalTime, Vec.Z);
			if (ChannelZ.Value)
			{
				Vec.Z = -Vec.Z;
			}
		}

		FbxVector4 KeyVec;

		if (TmPropertyIndex == 0)
		{
			KeyVec = Converter.ConvertToFbxPos((FVector)Vec);
		}
		else if (TmPropertyIndex == 1)
		{
			KeyVec = Converter.ConvertToFbxRot((FVector)Vec);
		}
		else
		{
			KeyVec = Converter.ConvertToFbxScale((FVector)Vec);
		}

		FbxTime FbxTime;

		if (GetExportOptions()->bExportLocalTime)
		{
			FbxTime.SetSecondDouble(DisplayRate.AsSeconds(LocalFrame));
		}
		// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
		//        not have that information here so we have to just assume they match. This should be improved so that we have
		//        all the information we need to do the right thing
		else if (TOptional<FFrameTime> GlobalKeyTime = SequenceToRootTransform.TryTransformTime(LocalTime))
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
		}
		else
		{
			// Doesn't map to a root time
			continue;
		}

		FbxCurveX->KeySet(FbxCurveX->KeyAdd(FbxTime), FbxTime, KeyVec[0]);
		FbxCurveY->KeySet(FbxCurveY->KeyAdd(FbxTime), FbxTime, KeyVec[1]);
		FbxCurveZ->KeySet(FbxCurveZ->KeyAdd(FbxTime), FbxTime, KeyVec[2]);
	}

	FbxCurveX->KeyModifyEnd();
	FbxCurveY->KeyModifyEnd();
	FbxCurveZ->KeyModifyEnd();
}

/**
 * Exports a scene node with the placement indicated by a given actor.
 * This scene node will always have two transformations: one translation vector and one Euler rotation.
 */
FbxNode* FFbxExporter::ExportActor(AActor* Actor, bool bExportComponents, INodeNameAdapter& NodeNameAdapter, bool bSaveAnimSeq )
{
	// Verify that this actor isn't already exported, create a structure for it
	// and buffer it.
	FbxNode* ActorNode = FindActor(Actor, &NodeNameAdapter);
	if (ActorNode == NULL)
	{
		FString FbxNodeName = NodeNameAdapter.GetActorNodeName(Actor);
		FbxNodeName = GetFbxObjectName(FbxNodeName, NodeNameAdapter);
		ActorNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));

		AActor* ParentActor = Actor->GetAttachParentActor();
		// this doesn't work with skeletalmeshcomponent
		FbxNode* ParentNode = FindActor(ParentActor, &NodeNameAdapter);
		FVector ActorLocation, ActorRotation, ActorScale;

		TArray<AActor*> AttachedActors;
		Actor->GetAttachedActors(AttachedActors);
		const bool bHasAttachedActors = AttachedActors.Num() != 0;

		// For cameras and lights: always add a rotation to get the correct coordinate system.
        FTransform RotationDirectionConvert = FTransform::Identity;
		const bool bIsCameraActor = Actor->IsA(ACameraActor::StaticClass());
		const bool bIsLightActor = Actor->IsA(ALight::StaticClass());
		if (bIsCameraActor || bIsLightActor)
		{
			if (bIsCameraActor)
			{
                FRotator Rotator = FFbxDataConverter::GetCameraRotation().GetInverse();
				RotationDirectionConvert = FTransform(Rotator);
			}
			else if (bIsLightActor)
			{
				FRotator Rotator = FFbxDataConverter::GetLightRotation().GetInverse();
				RotationDirectionConvert = FTransform(Rotator);
			}
		}

		//If the parent is the root or is not export use the root node as the parent
		if (bKeepHierarchy && ParentNode)
		{
			// Set the default position of the actor on the transforms
			// The transformation is different from FBX's Z-up: invert the Y-axis for translations and the Y/Z angle values in rotations.
			const FTransform RelativeTransform = RotationDirectionConvert * Actor->GetTransform().GetRelativeTransform(ParentActor->GetTransform());
			ActorLocation = RelativeTransform.GetTranslation();
			ActorRotation = RelativeTransform.GetRotation().Euler();
			ActorScale = RelativeTransform.GetScale3D();
		}
		else
		{
			ParentNode = Scene->GetRootNode();
			// Set the default position of the actor on the transforms
			// The transformation is different from FBX's Z-up: invert the Y-axis for translations and the Y/Z angle values in rotations.
			if (ParentActor != NULL)
			{
				//In case the parent was not export, get the absolute transform
				const FTransform AbsoluteTransform = RotationDirectionConvert * Actor->GetTransform();
				ActorLocation = AbsoluteTransform.GetTranslation();
				ActorRotation = AbsoluteTransform.GetRotation().Euler();
				ActorScale = AbsoluteTransform.GetScale3D();
			}
			else
			{
				const FTransform ConvertedTransform = RotationDirectionConvert * Actor->GetTransform();
				ActorLocation = ConvertedTransform.GetTranslation();
				ActorRotation = ConvertedTransform.GetRotation().Euler();
				ActorScale = ConvertedTransform.GetScale3D();
			}
		}

		ParentNode->AddChild(ActorNode);
		FbxActors.Add(Actor, ActorNode);
		NodeNameAdapter.AddFbxNode(Actor, ActorNode);

		// Set the default position of the actor on the transforms
		// The transformation is different from FBX's Z-up: invert the Y-axis for translations and the Y/Z angle values in rotations.
		ActorNode->LclTranslation.Set(Converter.ConvertToFbxPos(ActorLocation));
		ActorNode->LclRotation.Set(Converter.ConvertToFbxRot(ActorRotation));
		ActorNode->LclScaling.Set(Converter.ConvertToFbxScale(ActorScale));
	
		if( bExportComponents )
		{
			TInlineComponentArray<USceneComponent*> ComponentsToExport;
			for (UActorComponent* ActorComp : Actor->GetComponents())
			{
				USceneComponent* Component = Cast<USceneComponent>(ActorComp);
	
				if (Component == nullptr || Component->bHiddenInGame)
				{
					//Skip hidden component like camera mesh or other editor helper
					continue;
				}

				UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>( Component );
				USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>( Component );
				UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>( Component );

				if( StaticMeshComp && StaticMeshComp->GetStaticMesh())
				{
					ComponentsToExport.Add( StaticMeshComp );
				}
				else if( SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
				{
					ComponentsToExport.Add( SkelMeshComp );
				}
				else if (Component->IsA(UCameraComponent::StaticClass()))
				{
					ComponentsToExport.Add(Component);
				}
				else if (Component->IsA(ULightComponent::StaticClass()))
				{
					ComponentsToExport.Add(Component);
				}
				else if (ChildActorComp && ChildActorComp->GetChildActor())
				{
					ComponentsToExport.Add(ChildActorComp);
				}
			}

			for( int32 CompIndex = 0; CompIndex < ComponentsToExport.Num(); ++CompIndex )
			{
				USceneComponent* Component = ComponentsToExport[CompIndex];

                RotationDirectionConvert = FTransform::Identity;
                // For cameras and lights: always add a rotation to get the correct coordinate system
				// Unless we are parented to an Actor of the same type, since the rotation direction was already added
				if (Component->IsA(UCameraComponent::StaticClass()) || Component->IsA(ULightComponent::StaticClass()))
				{
					if (!bIsCameraActor && Component->IsA(UCameraComponent::StaticClass()))
					{
                    	FRotator Rotator = FFbxDataConverter::GetCameraRotation().GetInverse();
						RotationDirectionConvert = FTransform(Rotator);
					}
					else if (!bIsLightActor && Component->IsA(ULightComponent::StaticClass()))
					{
						FRotator Rotator = FFbxDataConverter::GetLightRotation().GetInverse();
						RotationDirectionConvert = FTransform(Rotator);
					}
				}

				FbxNode* ExportNode = ActorNode;
				if( ComponentsToExport.Num() > 1 )
				{
					// This actor has multiple components
					// create a child node under the actor for each component
					FbxNode* CompNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*Component->GetName()));
					
					if( Component != Actor->GetRootComponent() )
					{
						// Transform is relative to the root component
						const FTransform RelativeTransform = RotationDirectionConvert * Component->GetComponentToWorld().GetRelativeTransform(Actor->GetTransform());
						CompNode->LclTranslation.Set(Converter.ConvertToFbxPos(RelativeTransform.GetTranslation()));
						CompNode->LclRotation.Set(Converter.ConvertToFbxRot(RelativeTransform.GetRotation().Euler()));
						CompNode->LclScaling.Set(Converter.ConvertToFbxScale(RelativeTransform.GetScale3D()));
					}

					ExportNode = CompNode;
					ActorNode->AddChild(CompNode);
				}
				// If this actor has attached actors, don't allow its non root component components to contribute to the transform
				else if(Component != Actor->GetRootComponent() && !bHasAttachedActors)
				{
					// Merge the component relative transform in the ActorNode transform since this is the only component to export and its not the root
					const FTransform RelativeTransform = RotationDirectionConvert * Component->GetComponentToWorld().GetRelativeTransform(Actor->GetTransform());

					FTransform ActorTransform(FRotator::MakeFromEuler(ActorRotation).Quaternion(), ActorLocation, ActorScale);
					FTransform TotalTransform = RelativeTransform * ActorTransform;

					ActorNode->LclTranslation.Set(Converter.ConvertToFbxPos(TotalTransform.GetLocation()));
					ActorNode->LclRotation.Set(Converter.ConvertToFbxRot(TotalTransform.GetRotation().Euler()));
					ActorNode->LclScaling.Set(Converter.ConvertToFbxScale(TotalTransform.GetScale3D()));
				}

				UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>( Component );
				USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>( Component );
				UChildActorComponent* ChildActorComp = Cast<UChildActorComponent>( Component );

				if (StaticMeshComp && StaticMeshComp->GetStaticMesh())
				{
					const int32 LODIndex = (StaticMeshComp->ForcedLodModel > 0 ? FMath::Min(StaticMeshComp->GetStaticMesh()->GetNumLODs(), StaticMeshComp->ForcedLodModel) - 1 : /* auto-select*/ 0);
					if (USplineMeshComponent* SplineMeshComp = Cast<USplineMeshComponent>(StaticMeshComp))
					{
						//TODO: Validate Spline material baking
						//(Spline mesh staticmesh does not take into account the Spline transforms, might not be accurate/enough to use Spline->StaticMesh for material baking.)
						ExportSplineMeshToFbx(SplineMeshComp, *SplineMeshComp->GetName(), ExportNode, FFbxMaterialBakingMeshData(SplineMeshComp->GetStaticMesh(), SplineMeshComp, LODIndex));
					}
					else if (UInstancedStaticMeshComponent* InstancedMeshComp = Cast<UInstancedStaticMeshComponent>(StaticMeshComp))
					{
						ExportInstancedMeshToFbx(InstancedMeshComp, *InstancedMeshComp->GetName(), ExportNode, FFbxMaterialBakingMeshData(InstancedMeshComp->GetStaticMesh(), InstancedMeshComp, LODIndex));
					}
					else
					{
						const int32 LightmapUVChannel = -1;
						const TArray<FStaticMaterial>* MaterialOrderOverride = nullptr;
						const FColorVertexBuffer* ColorBuffer = nullptr;
						ExportStaticMeshToFbx(StaticMeshComp->GetStaticMesh(), LODIndex, *StaticMeshComp->GetName(), ExportNode, FFbxMaterialBakingMeshData(StaticMeshComp->GetStaticMesh(), StaticMeshComp, LODIndex), LightmapUVChannel, ColorBuffer, MaterialOrderOverride, &ToRawPtrTArrayUnsafe(StaticMeshComp->OverrideMaterials));
					}
				}
				else if (SkelMeshComp && SkelMeshComp->GetSkeletalMeshAsset())
				{
					ExportSkeletalMeshComponent(SkelMeshComp, *SkelMeshComp->GetName(), ExportNode, NodeNameAdapter, bSaveAnimSeq);
				}
				// If this actor has attached actors, don't allow a camera component to determine the node attributes because that would alter the transform
				else if (Component->IsA(UCameraComponent::StaticClass()) && !bHasAttachedActors)
				{
					FbxCamera* Camera = FbxCamera::Create(Scene, TCHAR_TO_UTF8(*Component->GetName()));
					FillFbxCameraAttribute(ActorNode, Camera, Cast<UCameraComponent>(Component));
					ExportNode->SetNodeAttribute(Camera);
				}
				else if (Component->IsA(ULightComponent::StaticClass()) && !bHasAttachedActors)
				{
					FbxLight* Light = FbxLight::Create(Scene, TCHAR_TO_UTF8(*Component->GetName()));
					FillFbxLightAttribute(Light, ActorNode, Cast<ULightComponent>(Component));
					ExportNode->SetNodeAttribute(Light);
				}
				else if (ChildActorComp && ChildActorComp->GetChildActor())
				{
					FbxNode* ChildActorNode = ExportActor(ChildActorComp->GetChildActor(), true, NodeNameAdapter, bSaveAnimSeq);
					FbxActors.Add(ChildActorComp->GetChildActor(), ChildActorNode);
					NodeNameAdapter.AddFbxNode(ChildActorComp->GetChildActor(), ChildActorNode);
				}
			}
		}
		
	}

	return ActorNode;
}

void ConvertInterpToFBX(uint8 UnrealInterpMode, FbxAnimCurveDef::EInterpolationType& Interpolation, FbxAnimCurveDef::ETangentMode& Tangent)
{
	switch(UnrealInterpMode)
	{
	case CIM_Linear:
		Interpolation = FbxAnimCurveDef::eInterpolationLinear;
		Tangent = FbxAnimCurveDef::eTangentUser;
		break;
	case CIM_CurveAuto:
		Interpolation = FbxAnimCurveDef::eInterpolationCubic;
		Tangent = FbxAnimCurveDef::eTangentAuto;
		break;
	case CIM_Constant:
		Interpolation = FbxAnimCurveDef::eInterpolationConstant;
		Tangent = (FbxAnimCurveDef::ETangentMode)FbxAnimCurveDef::eConstantStandard;
		break;
	case CIM_CurveUser:
		Interpolation = FbxAnimCurveDef::eInterpolationCubic;
		Tangent = FbxAnimCurveDef::eTangentUser;
		break;
	case CIM_CurveBreak:
		Interpolation = FbxAnimCurveDef::eInterpolationCubic;
		Tangent = (FbxAnimCurveDef::ETangentMode) FbxAnimCurveDef::eTangentBreak;
		break;
	case CIM_CurveAutoClamped:
		Interpolation = FbxAnimCurveDef::eInterpolationCubic;
		Tangent = (FbxAnimCurveDef::ETangentMode) (FbxAnimCurveDef::eTangentAuto | FbxAnimCurveDef::eTangentGenericClamp);
		break;
	case CIM_Unknown:  // ???
		Interpolation = FbxAnimCurveDef::eInterpolationConstant;
		Tangent = FbxAnimCurveDef::eTangentAuto;
		break;
	}
}


// float-float comparison that allows for a certain error in the floating point values
// due to floating-point operations never being exact.
static bool IsEquivalent(float a, float b, float Tolerance = KINDA_SMALL_NUMBER)
{
	return (a - b) > -Tolerance && (a - b) < Tolerance;
}


void RichCurveInterpolationToFbxInterpolation(ERichCurveInterpMode InInterpolation, ERichCurveTangentMode InTangentMode, ERichCurveTangentWeightMode InTangentWeightMode,
	FbxAnimCurveDef::EInterpolationType &OutInterpolation, FbxAnimCurveDef::ETangentMode &OutTangentMode, FbxAnimCurveDef::EWeightedMode &OutTangentWeightMode)
{
	if (InInterpolation == ERichCurveInterpMode::RCIM_Cubic)
	{
		OutInterpolation = FbxAnimCurveDef::eInterpolationCubic;
		OutTangentMode = FbxAnimCurveDef::eTangentUser;

		// Always set tangent on the fbx key, so OutTangentMode should explicitly be eTangentUser unless Break.
		if (InTangentMode == RCTM_Break)
		{
			OutTangentMode = FbxAnimCurveDef::eTangentBreak;
		}
		
		switch (InTangentWeightMode)
		{
		case ERichCurveTangentWeightMode::RCTWM_WeightedBoth:
			OutTangentWeightMode = FbxAnimCurveDef::eWeightedAll;
			break;
		case ERichCurveTangentWeightMode::RCTWM_WeightedArrive:
			OutTangentWeightMode = FbxAnimCurveDef::eWeightedNextLeft;
			break;
		case ERichCurveTangentWeightMode::RCTWM_WeightedLeave:
			OutTangentWeightMode = FbxAnimCurveDef::eWeightedRight;
			break;
		case ERichCurveTangentWeightMode::RCTWM_WeightedNone:
		default:
			OutTangentWeightMode = FbxAnimCurveDef::eWeightedNone;
			break;

		};
	}
	else if (InInterpolation == ERichCurveInterpMode::RCIM_Linear)
	{
		OutInterpolation = FbxAnimCurveDef::eInterpolationLinear;
		OutTangentMode = FbxAnimCurveDef::eTangentUser;
		OutTangentWeightMode = FbxAnimCurveDef::eWeightedNone;
	}
	else if (InInterpolation == ERichCurveInterpMode::RCIM_Constant)
	{
		OutInterpolation = FbxAnimCurveDef::eInterpolationConstant;
		OutTangentMode = (FbxAnimCurveDef::ETangentMode)FbxAnimCurveDef::eConstantStandard;
		OutTangentWeightMode = FbxAnimCurveDef::eWeightedNone;
	}
	else
	{
		OutInterpolation = FbxAnimCurveDef::eInterpolationCubic;
		OutTangentMode = FbxAnimCurveDef::eTangentUser;
		OutTangentWeightMode = FbxAnimCurveDef::eWeightedNone;
	}
}

template<typename ChannelType>
void FFbxExporter::ExportBezierChannelToFbxCurveBaked(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, const UMovieSceneTrack* Track, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	using CurveValueType = typename ChannelType::CurveValueType;

	const float NegateFactor = bNegative ? -1.f : 1.f;
	const float kOneThird = 1.f / 3.f;

	InFbxCurve.KeyModifyBegin();

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToLocalTransform.Inverse();

	FFrameRate DisplayRate = Track->GetTypedOuter<UMovieScene>()->GetDisplayRate();
	TRange<FFrameNumber> PlaybackRange = Track->GetTypedOuter<UMovieScene>()->GetPlaybackRange();

	int32 LocalStartFrame = FFrameRate::TransformTime(FFrameTime(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange)), TickResolution, DisplayRate).RoundToFrame().Value;
	int32 AnimationLength = FFrameRate::TransformTime(FFrameTime(FFrameNumber(UE::MovieScene::DiscreteSize(PlaybackRange))), TickResolution, DisplayRate).RoundToFrame().Value;

	for (int32 FrameCount = 0; FrameCount <= AnimationLength; ++FrameCount)
	{
		int32 LocalFrame = LocalStartFrame + FrameCount;

		FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);

		float Value;
		InChannel.Evaluate(LocalTime, Value);

		FbxTime FbxTime;

		if (GetExportOptions()->bExportLocalTime)
		{
			FbxTime.SetSecondDouble(DisplayRate.AsSeconds(LocalFrame));
		}
		// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
		//        not have that information here so we have to just assume they match. This should be improved so that we have
		//        all the information we need to do the right thing
		else if (TOptional<FFrameTime> GlobalKeyTime = SequenceToRootTransform.TryTransformTime(LocalTime))
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
		}
		else
		{
			// Doesn't map to a root time
			continue;
		}

		InFbxCurve.KeySet(InFbxCurve.KeyAdd(FbxTime), FbxTime, Value, FbxAnimCurveDef::eInterpolationLinear);
	}
	InFbxCurve.KeyModifyEnd();
}

template<typename ChannelType>
void FFbxExporter::ExportBezierChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	using CurveValueType = typename ChannelType::CurveValueType;

	const float NegateFactor = bNegative ? -1.f : 1.f;
	const float kOneThird = 1.f / 3.f;
	InFbxCurve.KeyModifyBegin();

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToLocalTransform.Inverse();

	TArrayView<const FFrameNumber>          Times  = InChannel.GetTimes();
	TArrayView<const ChannelValueType> Values = InChannel.GetValues();

	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		const FFrameNumber       KeyTime  = Times[Index];
		const ChannelValueType   KeyValue = Values[Index];

		const CurveValueType Value = (ValueMode == ERichCurveValueMode::Fov ? DefaultCamera->ComputeFocalLength( KeyValue.Value ) : KeyValue.Value) * NegateFactor;

		FbxTime FbxTime;
		FbxAnimCurveKey FbxKey;

		if (GetExportOptions()->bExportLocalTime)
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(KeyTime));
		}
		// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
		//        not have that information here so we have to just assume they match. This should be improved so that we have
		//        all the information we need to do the right thing
		else if (TOptional<FFrameTime> GlobalKeyTime = SequenceToRootTransform.TryTransformTime(KeyTime))
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
		}
		else
		{
			// Doesn't map to a root time
			continue;
		}

		const int FbxKeyIndex = InFbxCurve.KeyAdd(FbxTime);

		FbxAnimCurveDef::EInterpolationType Interpolation = FbxAnimCurveDef::eInterpolationCubic;
		FbxAnimCurveDef::ETangentMode Tangent = FbxAnimCurveDef::eTangentAuto;
		FbxAnimCurveDef::EWeightedMode WeightedMode = FbxAnimCurveDef::eWeightedNone;
		RichCurveInterpolationToFbxInterpolation(KeyValue.InterpMode, KeyValue.TangentMode, KeyValue.Tangent.TangentWeightMode, Interpolation, Tangent, WeightedMode);

		if (Interpolation == FbxAnimCurveDef::eInterpolationCubic)
		{
			if (Index < Times.Num() - 1)
			{
				float LeaveTangentWeight = kOneThird;
				float NextArriveTangentWeight = kOneThird;
				const double  NextTime = Times[Index + 1] / TickResolution;
				const float TimeDiff = static_cast<float>(NextTime - FbxTime.GetSecondDouble());
				
				float LeaveTangent = KeyValue.Tangent.LeaveTangent * TickResolution.AsDecimal();
				float NextArriveTangent = Values[Index + 1].Tangent.ArriveTangent * TickResolution.AsDecimal();

				//Need to convert UE tangent weight which is the length of the hypotenuse to FBX normalized X(time) weight
				if (WeightedMode == FbxAnimCurveDef::eWeightedAll || WeightedMode == FbxAnimCurveDef::eWeightedRight)
				{
					const CurveValueType XVal = FMath::Sqrt((KeyValue.Tangent.LeaveTangentWeight * KeyValue.Tangent.LeaveTangentWeight) / (1.0f + LeaveTangent * LeaveTangent));
					LeaveTangentWeight = XVal / TimeDiff;
				}

				//make sure next tangent is weighted else use default weight
				if ((Values[Index + 1].Tangent.TangentWeightMode == ERichCurveTangentWeightMode::RCTWM_WeightedBoth || Values[Index + 1].Tangent.TangentWeightMode == ERichCurveTangentWeightMode::RCTWM_WeightedArrive) )
				{
					const CurveValueType XVal = FMath::Sqrt((Values[Index + 1].Tangent.ArriveTangentWeight * Values[Index + 1].Tangent.ArriveTangentWeight) / (1.0f + NextArriveTangent * NextArriveTangent));
					NextArriveTangentWeight = XVal / TimeDiff;
				}
				if (bNegative)
				{
					LeaveTangent = -LeaveTangent;
					NextArriveTangent = -NextArriveTangent;
				}

				InFbxCurve.KeySet(FbxKeyIndex, FbxTime, Value, Interpolation, Tangent, LeaveTangent, NextArriveTangent, WeightedMode, LeaveTangentWeight, NextArriveTangentWeight );
			}
			else
			{
				InFbxCurve.KeySet(FbxKeyIndex, FbxTime, Value, Interpolation, Tangent, 0.f, 0.f, WeightedMode);
			}
		}
		else
		{
			InFbxCurve.KeySet(FbxKeyIndex, FbxTime, Value, Interpolation, Tangent);
		}
	}
	InFbxCurve.KeyModifyEnd();
}

void FFbxExporter::ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneFloatChannel& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	ExportBezierChannelToFbxCurve(InFbxCurve, InChannel, TickResolution, ValueMode, bNegative, RootToLocalTransform);
}

void FFbxExporter::ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneDoubleChannel& InChannel, FFrameRate TickResolution, ERichCurveValueMode ValueMode, bool bNegative, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	ExportBezierChannelToFbxCurve(InFbxCurve, InChannel, TickResolution, ValueMode, bNegative, RootToLocalTransform);
}

void FFbxExporter::ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneIntegerChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	ExportConstantChannelToFbxCurve<FMovieSceneIntegerChannel, int32>(InFbxCurve, InChannel, TickResolution, RootToLocalTransform);
}

void FFbxExporter::ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneBoolChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	ExportConstantChannelToFbxCurve<FMovieSceneBoolChannel, bool>(InFbxCurve, InChannel, TickResolution, RootToLocalTransform);
}

void FFbxExporter::ExportChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const FMovieSceneByteChannel& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	ExportConstantChannelToFbxCurve<FMovieSceneByteChannel, uint8>(InFbxCurve, InChannel, TickResolution, RootToLocalTransform);
}

template<typename ChannelType, typename T>
void FFbxExporter::ExportConstantChannelToFbxCurve(FbxAnimCurve& InFbxCurve, const ChannelType& InChannel, FFrameRate TickResolution, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	InFbxCurve.KeyModifyBegin();

	const TArrayView<const FFrameNumber> Times  = InChannel.GetTimes();
	const TArrayView<const T> Values = InChannel.GetValues();

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToLocalTransform.Inverse();

	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		const FFrameNumber KeyTime = Times[Index];
		const T KeyValue = Values[Index];

		FbxTime FbxTime;

		if (GetExportOptions()->bExportLocalTime)
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(KeyTime));
		}
		// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
		//        not have that information here so we have to just assume they match. This should be improved so that we have
		//        all the information we need to do the right thing
		else if (TOptional<FFrameTime> GlobalKeyTime = SequenceToRootTransform.TryTransformTime(KeyTime))
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
		}
		else
		{
			// Doesn't map to a root time
			continue;
		}

		const int FbxKeyIndex = InFbxCurve.KeyAdd(FbxTime);

		InFbxCurve.KeySet(FbxKeyIndex, FbxTime, KeyValue, FbxAnimCurveDef::eInterpolationConstant);
		InFbxCurve.KeySetConstantMode(FbxKeyIndex, FbxAnimCurveDef::EConstantMode::eConstantStandard);
	}
	InFbxCurve.KeyModifyEnd();
}

void FFbxExporter::ExportLevelSequenceColorTrack(FbxNode* FbxNode, UMovieSceneColorTrack& ColorTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	UMovieSceneColorSection* ColorSection = ColorTrack.GetAllSections().Num() > 0
		? Cast<UMovieSceneColorSection>(ColorTrack.GetAllSections()[0])
		: nullptr;

	if(!ColorSection)
	{
		return;
	}

	if(!FbxNode)
	{
		FbxNode = CreateNode(ColorTrack.GetDisplayName().ToString());
	}

	FbxLight* FbxLight = FbxNode->GetLight();	

	const FString PropertyName = ColorTrack.GetTrackName().ToString();

	FbxProperty Property;

	if(PropertyName == TEXT("LightColor") && FbxLight)
	{
		Property = FbxLight->Color;
	}

	if(Property == 0)
	{
		CreateAnimatableUserProperty(FbxNode, FbxDouble3{ 0 }, TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName), FbxDouble3DT);
		Property = FbxNode->FindProperty(TCHAR_TO_UTF8(*PropertyName));
	}

	if(Property == 0)
	{
		return;
	}

	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);
	FbxAnimCurve* CurveRed = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COLOR_RED, true);
	FbxAnimCurve* CurveGreen = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COLOR_GREEN, true);
	FbxAnimCurve* CurveBlue = Property.GetCurve(BaseLayer, FBXSDK_CURVENODE_COLOR_BLUE, true);

	FFrameRate TickResolution = ColorTrack.GetTypedOuter<UMovieScene>()->GetTickResolution();
	ExportChannelToFbxCurve(*CurveRed, ColorSection->GetRedChannel(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
	ExportChannelToFbxCurve(*CurveGreen, ColorSection->GetGreenChannel(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
	ExportChannelToFbxCurve(*CurveBlue, ColorSection->GetBlueChannel(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
}

void FFbxExporter::ExportLevelSequenceVectorTrack(FbxNode* FbxNode, UMovieSceneDoubleVectorTrack& VectorTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	UMovieSceneDoubleVectorSection* VectorSection = VectorTrack.GetAllSections().Num() > 0
		? Cast<UMovieSceneDoubleVectorSection>(VectorTrack.GetAllSections()[0])
		: nullptr;

	if(!VectorSection)
	{
		return;
	}

	if(!FbxNode)
	{
		FbxNode = CreateNode(VectorTrack.GetDisplayName().ToString());
	}

	const FString PropertyName = VectorTrack.GetTrackName().ToString();


	//VectorSections supports up to 2-4 channels
	const int32 NumChannelsUsed = VectorSection->GetChannelsUsed();

	CreateAnimatableUserProperty(FbxNode,
								 NumChannelsUsed == 2 ? FbxDouble2{ 0 } :
								 NumChannelsUsed == 3 ? FbxDouble3{ 0 } :
														FbxDouble4{ 0 },
								 TCHAR_TO_UTF8(*PropertyName),
								 TCHAR_TO_UTF8(*PropertyName),
								 NumChannelsUsed == 2 ? FbxDouble2DT :
								 NumChannelsUsed == 3 ? FbxDouble3DT :
														FbxDouble4DT);
	FbxProperty Property = FbxNode->FindProperty(TCHAR_TO_UTF8(*PropertyName));

	if(Property == 0)
	{
		return;
	}

	const char* CurvesNames[] = { "X", "Y", "Z", "W" };
	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);
	FFrameRate TickResolution = VectorTrack.GetTypedOuter<UMovieScene>()->GetTickResolution();

	for(int32 Index = 0; Index < NumChannelsUsed; ++Index)
	{
		FbxAnimCurve* Curve = Property.GetCurve(BaseLayer, CurvesNames[Index], true);
		ExportChannelToFbxCurve(*Curve, VectorSection->GetChannel(Index), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
	}
}

void FFbxExporter::ExportLevelSequence3DTransformTrack(FbxNode* FbxNode, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, UMovieScene3DTransformTrack& TransformTrack, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	// TODO: Support more than one section?
	UMovieScene3DTransformSection* TransformSection = TransformTrack.GetAllSections().Num() > 0
		? Cast<UMovieScene3DTransformSection>(TransformTrack.GetAllSections()[0])
		: nullptr;

	if (TransformSection == nullptr)
	{
		return;
	}

	FMovieSceneInverseSequenceTransform SequenceToRootTransform = RootToLocalTransform.Inverse();

	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);

	AActor* BoundActor = Cast<AActor>(BoundObject);
	USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObject);

	const bool bIsCameraActor = BoundActor ? BoundActor->IsA(ACameraActor::StaticClass()) : BoundComponent ? BoundComponent->IsA(UCameraComponent::StaticClass()) : false;
	const bool bIsLightActor = BoundActor ? BoundActor->IsA(ALight::StaticClass()) : BoundComponent ? BoundComponent->IsA(ULightComponent::StaticClass()) : false;

	// If bake rotations is set, and actor is a camera or light actor, only bake rotation channel
	const bool bBakeRotations = (bIsCameraActor || bIsLightActor) && (static_cast<uint8>(GetExportOptions()->BakeCameraAndLightAnimation) & static_cast<uint8>(EMovieSceneBakeType::BakeTransforms));

	if (!FbxNode)
	{
		FbxNode = CreateNode(TransformTrack.GetDisplayName().ToString());
	}

	FFrameRate TickResolution = TransformTrack.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameRate DisplayRate = TransformTrack.GetTypedOuter<UMovieScene>()->GetDisplayRate();

	FbxAnimCurveNode* TranslationNode = FbxNode->LclTranslation.GetCurveNode(BaseLayer, true);
	FbxAnimCurveNode* RotationNode = FbxNode->LclRotation.GetCurveNode(BaseLayer, true);
	FbxAnimCurveNode* ScaleNode = FbxNode->LclScaling.GetCurveNode(BaseLayer, true);

	FbxAnimCurve* FbxCurveTransX = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveTransY = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveTransZ = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FbxAnimCurve* FbxCurveRotX = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveRotY = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveRotZ = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FbxAnimCurve* FbxCurveScaleX = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveScaleY = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveScaleZ = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannels[] = {
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
		SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
	};

	// Translation
	if (DoubleChannels[0].Get())
	{
		ExportChannelToFbxCurve(*FbxCurveTransX, *DoubleChannels[0].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
	}
	if (DoubleChannels[1].Get())
	{
		ExportChannelToFbxCurve(*FbxCurveTransY, *DoubleChannels[1].Get(), TickResolution, ERichCurveValueMode::Default, true, RootToLocalTransform);
	}
	if (DoubleChannels[2].Get())
	{
		ExportChannelToFbxCurve(*FbxCurveTransZ, *DoubleChannels[2].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
	}

	// Scale - don't generate scale keys for cameras
	if (!bIsCameraActor)
	{
		if (DoubleChannels[6].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveScaleX, *DoubleChannels[6].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
		}
		if (DoubleChannels[7].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveScaleY, *DoubleChannels[7].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
		}
		if (DoubleChannels[8].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveScaleZ, *DoubleChannels[8].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
		}
	}

	// Rotation - bake rotation for cameras and lights
	if (!bBakeRotations)
	{
		if (DoubleChannels[3].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveRotX, *DoubleChannels[3].Get(), TickResolution, ERichCurveValueMode::Default, false, RootToLocalTransform);
		}
		if (DoubleChannels[4].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveRotY, *DoubleChannels[4].Get(), TickResolution, ERichCurveValueMode::Default, true, RootToLocalTransform);
		}
		if (DoubleChannels[5].Get())
		{
			ExportChannelToFbxCurve(*FbxCurveRotZ, *DoubleChannels[5].Get(), TickResolution, ERichCurveValueMode::Default, true, RootToLocalTransform);
		}
	}
	else
	{
		FTransform RotationDirectionConvert;
		if (bIsCameraActor)
		{
			FRotator Rotator = FFbxDataConverter::GetCameraRotation().GetInverse();
			RotationDirectionConvert = FTransform(Rotator);
		}
		else if (bIsLightActor)
		{
			FRotator Rotator = FFbxDataConverter::GetLightRotation().GetInverse();
			RotationDirectionConvert = FTransform(Rotator);
		}

		FbxCurveRotX->KeyModifyBegin();
		FbxCurveRotY->KeyModifyBegin();
		FbxCurveRotZ->KeyModifyBegin();

		int32 LocalStartFrame = FFrameRate::TransformTime(FFrameTime(UE::MovieScene::DiscreteInclusiveLower(InPlaybackRange)), TickResolution, DisplayRate).RoundToFrame().Value;
		int32 AnimationLength = FFrameRate::TransformTime(FFrameTime(FFrameNumber(UE::MovieScene::DiscreteSize(InPlaybackRange))), TickResolution, DisplayRate).RoundToFrame().Value;

		for (int32 FrameCount = 0; FrameCount <= AnimationLength; ++FrameCount)
		{
			int32 LocalFrame = LocalStartFrame + FrameCount;

			FFrameTime LocalTime = FFrameRate::TransformTime(FFrameTime(LocalFrame), DisplayRate, TickResolution);

			FVector3f Trans = FVector3f::ZeroVector;
			if (DoubleChannels[0].Get())
			{
				DoubleChannels[0].Get()->Evaluate(LocalTime, Trans.X);
			}
			if (DoubleChannels[1].Get())
			{
				DoubleChannels[1].Get()->Evaluate(LocalTime, Trans.Y);
			}
			if (DoubleChannels[2].Get())
			{
				DoubleChannels[2].Get()->Evaluate(LocalTime, Trans.Z);
			}

			FRotator Rotator;
			if (DoubleChannels[3].Get())
			{
				DoubleChannels[3].Get()->Evaluate(LocalTime, Rotator.Roll);
			}
			if (DoubleChannels[4].Get())
			{
				DoubleChannels[4].Get()->Evaluate(LocalTime, Rotator.Pitch);
			}
			if (DoubleChannels[5].Get())
			{
				DoubleChannels[5].Get()->Evaluate(LocalTime, Rotator.Yaw);
			}

			FVector3f Scale;
			if (DoubleChannels[6].Get())
			{
				DoubleChannels[6].Get()->Evaluate(LocalTime, Scale.X);
			}
			if (DoubleChannels[7].Get())
			{
				DoubleChannels[7].Get()->Evaluate(LocalTime, Scale.Y);
			}
			if (DoubleChannels[8].Get())
			{
				DoubleChannels[8].Get()->Evaluate(LocalTime, Scale.Z);
			}

			FTransform RelativeTransform;
			RelativeTransform.SetTranslation((FVector)Trans);
			RelativeTransform.SetRotation(Rotator.Quaternion());
			RelativeTransform.SetScale3D((FVector)Scale);

			RelativeTransform = RotationDirectionConvert * RelativeTransform;

			FbxVector4 KeyTrans = Converter.ConvertToFbxPos(RelativeTransform.GetTranslation());
			FbxVector4 KeyRot = Converter.ConvertToFbxRot(RelativeTransform.GetRotation().Euler());
			FbxVector4 KeyScale = Converter.ConvertToFbxScale(RelativeTransform.GetScale3D());

			FbxTime FbxTime;
			if (GetExportOptions()->bExportLocalTime)
			{
				FbxTime.SetSecondDouble(DisplayRate.AsSeconds(LocalFrame));
			}
			// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
			//        not have that information here so we have to just assume they match. This should be improved so that we have
			//        all the information we need to do the right thing
			else if (TOptional<FFrameTime> GlobalKeyTime = SequenceToRootTransform.TryTransformTime(LocalTime))
			{
				FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
			}
			else
			{
				// Doesn't map to a root time
				continue;
			}

			FbxCurveRotX->KeySet(FbxCurveRotX->KeyAdd(FbxTime), FbxTime, KeyRot[0]);
			FbxCurveRotY->KeySet(FbxCurveRotY->KeyAdd(FbxTime), FbxTime, KeyRot[1]);
			FbxCurveRotZ->KeySet(FbxCurveRotZ->KeyAdd(FbxTime), FbxTime, KeyRot[2]);
		}

		FbxCurveRotX->KeyModifyEnd();
		FbxCurveRotY->KeyModifyEnd();
		FbxCurveRotZ->KeyModifyEnd();
	}
}

void FFbxExporter::ExportLevelSequenceBaked3DTransformTrack(IAnimTrackAdapter& AnimTrackAdapter, FbxNode* FbxNode, IMovieScenePlayer* MovieScenePlayer, FMovieSceneSequenceIDRef InSequenceID, TArray<TWeakObjectPtr<UMovieScene3DTransformTrack> > TransformTracks, UObject* BoundObject, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform)
{
	using namespace UE::MovieScene;

	UMovieScene3DTransformTrack* TransformTrack = nullptr;
	int32 NumSections = 0;
	for (TWeakObjectPtr<UMovieScene3DTransformTrack> WeakTransformTrack : TransformTracks)
	{
		if (WeakTransformTrack.IsValid())
		{
			TransformTrack = WeakTransformTrack.Get();
			NumSections += TransformTrack->GetAllSections().Num();
		}
	}

	if (NumSections <= 0 || !TransformTrack)
	{
		return;
	}

	FbxAnimLayer* BaseLayer = AnimStack->GetMember<FbxAnimLayer>(0);

	AActor* BoundActor = Cast<AActor>(BoundObject);
	USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObject);

	const bool bIsCameraActor = BoundActor ? BoundActor->IsA(ACameraActor::StaticClass()) : BoundComponent ? BoundComponent->IsA(UCameraComponent::StaticClass()) : false;
	const bool bIsLightActor = BoundActor ? BoundActor->IsA(ALight::StaticClass()) : BoundComponent ? BoundComponent->IsA(ULightComponent::StaticClass()) : false;
	const bool bBakeRotations = bIsCameraActor || bIsLightActor;

	USceneComponent* InterrogatedComponent = nullptr;
	if (BoundComponent)
	{
		InterrogatedComponent = BoundComponent;
	}
	else if (BoundActor)
	{
		InterrogatedComponent = BoundActor->GetRootComponent();
	}

	if (bIsCameraActor)
	{
		if (InterrogatedComponent && InterrogatedComponent->IsA<UCameraComponent>())
		{
			// all set
		}
		else if (BoundActor && BoundActor->IsA(ACameraActor::StaticClass()))
		{
			ACameraActor* CameraActor = Cast<ACameraActor>(BoundActor);
			InterrogatedComponent = CameraActor->GetCameraComponent();
		}
	}

	if (!InterrogatedComponent)
	{
		UE_LOGF(LogFbx, Warning, "Export transform track for %ls failed because could not find suitable scene component", *BoundObject->GetName());
		return;
	}

	if (!FbxNode)
	{
		FbxNode = CreateNode(TransformTrack->GetDisplayName().ToString());
	}

	FFrameRate TickResolution = TransformTrack->GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameRate DisplayRate = TransformTrack->GetTypedOuter<UMovieScene>()->GetDisplayRate();

	FbxAnimCurveNode* TranslationNode = FbxNode->LclTranslation.GetCurveNode(BaseLayer, true);
	FbxAnimCurveNode* RotationNode = FbxNode->LclRotation.GetCurveNode(BaseLayer, true);
	FbxAnimCurveNode* ScaleNode = FbxNode->LclScaling.GetCurveNode(BaseLayer, true);

	FbxAnimCurve* FbxCurveTransX = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveTransY = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveTransZ = FbxNode->LclTranslation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FbxAnimCurve* FbxCurveRotX = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveRotY = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveRotZ = FbxNode->LclRotation.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FbxAnimCurve* FbxCurveScaleX = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_X, true);
	FbxAnimCurve* FbxCurveScaleY = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Y, true);
	FbxAnimCurve* FbxCurveScaleZ = FbxNode->LclScaling.GetCurve(BaseLayer, FBXSDK_CURVENODE_COMPONENT_Z, true);

	FTransform RotationDirectionConvert;
	if (bIsCameraActor)
	{
		FRotator Rotator = FFbxDataConverter::GetCameraRotation().GetInverse();
		RotationDirectionConvert = FTransform(Rotator);
	}
	else if (bIsLightActor)
	{
		FRotator Rotator = FFbxDataConverter::GetLightRotation().GetInverse();
		RotationDirectionConvert = FTransform(Rotator);
	}

	FbxCurveTransX->KeyModifyBegin();
	FbxCurveTransY->KeyModifyBegin();
	FbxCurveTransZ->KeyModifyBegin();

	FbxCurveRotX->KeyModifyBegin();
	FbxCurveRotY->KeyModifyBegin();
	FbxCurveRotZ->KeyModifyBegin();

	if (!bIsCameraActor)
	{
		FbxCurveScaleX->KeyModifyBegin();
		FbxCurveScaleY->KeyModifyBegin();
		FbxCurveScaleZ->KeyModifyBegin();
	}

	FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

	TArray<FTransform> RelativeTransforms;
	int32 LocalStartFrame = FFrameRate::TransformTime(FFrameTime(DiscreteInclusiveLower(InPlaybackRange)), TickResolution, DisplayRate).RoundToFrame().Value;
	int32 AnimationLength = FFrameRate::TransformTime(FFrameTime(FFrameNumber(DiscreteSize(InPlaybackRange))), TickResolution, DisplayRate).RoundToFrame().Value + 1; // Add one so that we export a key for the end frame

	const double SampleRate = 1.0/DisplayRate.AsDecimal();

	for (int32 FrameNumber = LocalStartFrame; FrameNumber < LocalStartFrame + AnimationLength; ++FrameNumber)
	{
		const FFrameTime FrameTime = FFrameRate::TransformTime(FrameNumber, DisplayRate, TickResolution);

		// This will call UpdateSkelPose on the skeletal mesh component to move bones based on animations in the matinee group
		AnimTrackAdapter.UpdateAnimation(FrameNumber);

		USceneComponent* Child = InterrogatedComponent;
		while (Child)
		{
			if (USkeletalMeshComponent* ChildSkeletalMeshComponent = Cast<USkeletalMeshComponent>(Child))
			{
				ChildSkeletalMeshComponent->TickAnimation(SampleRate, false);

				ChildSkeletalMeshComponent->RefreshBoneTransforms();
				ChildSkeletalMeshComponent->RefreshFollowerComponents();
				ChildSkeletalMeshComponent->UpdateComponentToWorld();
				ChildSkeletalMeshComponent->FinalizeBoneTransform();
				ChildSkeletalMeshComponent->MarkRenderTransformDirty();
				ChildSkeletalMeshComponent->MarkRenderDynamicDataDirty();
			}

			if (Child->GetOwner())
			{
				Child->GetOwner()->Tick(SampleRate);
			}

			Child = Child->GetAttachParent();
		}
		
		// Get the relative transform for this component. This can be complicated because we don't export scene components in the hierarchy. For example,
		// ParentActor
		// ChildActor
		//    SceneComponent
		//        CameraComponent
		// When exporting CameraComponent, we don't export SceneComponent, so we need to get CameraComponent's world transform relative to ParentActor
		// 
		AActor* InterrogatedOwner = InterrogatedComponent->GetOwner();
		AActor* AttachParentActor = InterrogatedOwner ? InterrogatedOwner->GetAttachParentActor() : nullptr;
		FTransform ParentTransform = AttachParentActor ? AttachParentActor->GetTransform() : FTransform::Identity;
		FTransform RelativeTransform = InterrogatedComponent->GetComponentToWorld().GetRelativeTransform(ParentTransform);

		RelativeTransforms.Add(RelativeTransform);
	}
		
	// Reset 
	AnimTrackAdapter.UpdateAnimation(LocalStartFrame);

	for (int32 TransformIndex = 0; TransformIndex < RelativeTransforms.Num(); ++TransformIndex)
	{
		FTransform RelativeTransform = RotationDirectionConvert * RelativeTransforms[TransformIndex];

		FbxVector4 KeyTrans = Converter.ConvertToFbxPos(RelativeTransform.GetTranslation());
		FbxVector4 KeyRot = Converter.ConvertToFbxRot(RelativeTransform.GetRotation().Euler());
		FbxVector4 KeyScale = Converter.ConvertToFbxScale(RelativeTransform.GetScale3D());

		const int32 CurrentFrame = LocalStartFrame + TransformIndex;

		const FFrameTime LocalTime = FFrameRate::TransformTime(CurrentFrame, DisplayRate, TickResolution);

		FbxTime FbxTime;
		if (GetExportOptions()->bExportLocalTime)
		{
			FbxTime.SetSecondDouble(DisplayRate.AsSeconds(CurrentFrame));
		}
		// @todo: This code does not handle the root sequence having a different tick resolution than the local space, but we do
		//        not have that information here so we have to just assume they match. This should be improved so that we have
		//        all the information we need to do the right thing
		else if (TOptional<FFrameTime> GlobalKeyTime = LocalToRootTransform.TryTransformTime(LocalTime))
		{
			FbxTime.SetSecondDouble(TickResolution.AsSeconds(GlobalKeyTime.GetValue()));
		}
		else
		{
			// Doesn't map to a root time
			continue;
		}

		FbxCurveTransX->KeySet(FbxCurveTransX->KeyAdd(FbxTime), FbxTime, KeyTrans[0]);
		FbxCurveTransY->KeySet(FbxCurveTransY->KeyAdd(FbxTime), FbxTime, KeyTrans[1]);
		FbxCurveTransZ->KeySet(FbxCurveTransZ->KeyAdd(FbxTime), FbxTime, KeyTrans[2]);

		FbxCurveRotX->KeySet(FbxCurveRotX->KeyAdd(FbxTime), FbxTime, KeyRot[0]);
		FbxCurveRotY->KeySet(FbxCurveRotY->KeyAdd(FbxTime), FbxTime, KeyRot[1]);
		FbxCurveRotZ->KeySet(FbxCurveRotZ->KeyAdd(FbxTime), FbxTime, KeyRot[2]);

		if (!bIsCameraActor)
		{
			FbxCurveScaleX->KeySet(FbxCurveScaleX->KeyAdd(FbxTime), FbxTime, KeyScale[0]);
			FbxCurveScaleY->KeySet(FbxCurveScaleY->KeyAdd(FbxTime), FbxTime, KeyScale[1]);
			FbxCurveScaleZ->KeySet(FbxCurveScaleZ->KeyAdd(FbxTime), FbxTime, KeyScale[2]);
		}
	}

	FbxCurveTransX->KeyModifyEnd();
	FbxCurveTransY->KeyModifyEnd();
	FbxCurveTransZ->KeyModifyEnd();

	FbxCurveRotX->KeyModifyEnd();
	FbxCurveRotY->KeyModifyEnd();
	FbxCurveRotZ->KeyModifyEnd();

	if (!bIsCameraActor)
	{
		FbxCurveScaleX->KeyModifyEnd();
		FbxCurveScaleY->KeyModifyEnd();
		FbxCurveScaleZ->KeyModifyEnd();
	}
}
void FFbxExporter::ExportLevelSequenceTrackChannels( FbxNode* FbxNode, UMovieSceneTrack& Track, const TRange<FFrameNumber>& InPlaybackRange, const FMovieSceneSequenceTransform& RootToLocalTransform, bool bBakeBezierCurves)
{
	// TODO: Support more than one section?
	UMovieSceneSection* Section = Track.GetAllSections().Num() > 0 ? Track.GetAllSections()[0] : nullptr;

	if (!Section)
	{
		return;
	}

	if (!FbxNode)
	{
		FbxNode = CreateNode(Track.GetDisplayName().ToString());
	}

	FbxCamera* FbxCamera = FbxNode->GetCamera();
	FbxLight* FbxLight = FbxNode->GetLight();
	FFrameRate TickResolution = Track.GetTypedOuter<UMovieScene>()->GetTickResolution();

	const FName BoolChannelTypeName = FMovieSceneBoolChannel::StaticStruct()->GetFName();
	const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();
	const FName EnumChannelTypeName = FMovieSceneByteChannel::StaticStruct()->GetFName();
	const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
	const FName IntegerChannelTypeName = FMovieSceneIntegerChannel::StaticStruct()->GetFName();
	const FName StringChannelTypeName = FMovieSceneStringChannel::StaticStruct()->GetFName();
	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
	{
		const FName ChannelTypeName = Entry.GetChannelTypeName();
		if (ChannelTypeName != BoolChannelTypeName && 
			ChannelTypeName != DoubleChannelTypeName && 
			ChannelTypeName != EnumChannelTypeName && 
			ChannelTypeName != FloatChannelTypeName && 
			ChannelTypeName != IntegerChannelTypeName && 
			ChannelTypeName != StringChannelTypeName)
		{
			continue;
		}
		
		TArrayView<FMovieSceneChannel* const>        Channels = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> AllMetaData = Entry.GetMetaData();

		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			FMovieSceneChannelHandle Channel = ChannelProxy.MakeHandle(ChannelTypeName, Index);

			FMovieSceneBoolChannel* BoolChannel = Entry.GetChannelTypeName() == BoolChannelTypeName ? Channel.Cast<FMovieSceneBoolChannel>().Get() : nullptr;
			FMovieSceneDoubleChannel* DoubleChannel = Entry.GetChannelTypeName() == DoubleChannelTypeName ? Channel.Cast<FMovieSceneDoubleChannel>().Get() : nullptr;
			FMovieSceneByteChannel* EnumChannel = Entry.GetChannelTypeName() == EnumChannelTypeName ? Channel.Cast<FMovieSceneByteChannel>().Get() : nullptr;
			FMovieSceneFloatChannel* FloatChannel = Entry.GetChannelTypeName() == FloatChannelTypeName ? Channel.Cast<FMovieSceneFloatChannel>().Get() : nullptr;
			FMovieSceneIntegerChannel* IntegerChannel = Entry.GetChannelTypeName() == IntegerChannelTypeName ? Channel.Cast<FMovieSceneIntegerChannel>().Get() : nullptr;
			FMovieSceneStringChannel* StringChannel = Entry.GetChannelTypeName() == StringChannelTypeName ? Channel.Cast<FMovieSceneStringChannel>().Get() : nullptr;

			if (!BoolChannel && !DoubleChannel && !EnumChannel && !FloatChannel && !IntegerChannel && !StringChannel)
			{
				continue;
			}
			
			const FMovieSceneChannelMetaData& MetaData = AllMetaData[Index];
			FText Name = FText::FromName(MetaData.Name);

			FbxProperty Property;
			FString PropertyName = MetaData.Name.IsNone() ? Track.GetTrackName().ToString() : MetaData.Name.ToString();
			bool IsFoV = false;
			// most properties are created as user property, only FOV of camera in FBX supports animation
			if (PropertyName == "Intensity" && FbxLight)
			{
				Property = FbxLight->Intensity;
			}
			else if (PropertyName == "FalloffExponent")
			{
				Property = FbxNode->FindProperty("UE_FalloffExponent", false);
			}
			else if (PropertyName == "AttenuationRadius")
			{
				Property = FbxNode->FindProperty("UE_Radius", false);
			}
			else if (PropertyName == "FieldOfView" && FbxCamera)
			{
				Property = FbxCamera->FieldOfView;
				IsFoV = true;
			}
			else if (PropertyName == "FOVAngle" && FbxCamera)
			{
				Property = FbxCamera->FocalLength;
				IsFoV = true;
			}
			else if (PropertyName == "CurrentFocalLength" && FbxCamera)
			{
				Property = FbxCamera->FocalLength;
			}
			else if (PropertyName == "AspectRatio")
			{
				Property = FbxNode->FindProperty("UE_AspectRatio", false);
			}
			else if (PropertyName == "MotionBlur_Amount")
			{
				Property = FbxNode->FindProperty("UE_MotionBlur_Amount", false);
			}
			else if ( PropertyName == "FocusSettings.ManualFocusDistance" && FbxCamera )
			{
				Property = FbxCamera->FocusDistance;
			}
			else if(PropertyName == "bUseTemperature")
			{
				Property = FbxNode->FindProperty("UE_UseTemperature", false);
			}
			else if(PropertyName == "IntensityUnits")
			{
				Property = FbxNode->FindProperty("UE_IntensityUnits", false);
			}
			else if(PropertyName == "Filmback.SensorAspectRatio" && FbxCamera)
			{
				Property = FbxCamera->FilmAspectRatio;
			}
			else if(PropertyName == "Filmback.SensorHeight" && FbxCamera)
			{
				Property = FbxCamera->FilmHeight;
			}
			else if(PropertyName == "Filmback.SensorWidth" && FbxCamera)
			{
				Property = FbxCamera->FilmWidth;
			}

			if (Property == 0)
			{
				if(BoolChannel)
				{
					CreateAnimatableUserProperty(FbxNode, BoolChannel->GetDefault().Get(false), TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName), FbxBoolDT);
				}
				else if (DoubleChannel)
				{
					CreateAnimatableUserProperty(FbxNode, DoubleChannel->GetDefault().Get(MAX_flt), TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName));
				}
				else if (EnumChannel)
				{
					CreateAnimatableUserProperty(FbxNode, EnumChannel->GetDefault().Get(0), TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName), FbxEnumDT);
				}
				else if (FloatChannel)
				{
					CreateAnimatableUserProperty(FbxNode, FloatChannel->GetDefault().Get(MAX_flt), TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName));
				}
				else if (IntegerChannel)
				{
					CreateAnimatableUserProperty(FbxNode, IntegerChannel->GetDefault().Get(0), TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName), FbxIntDT);
				}
				else if (StringChannel)
				{
					FbxString FbxValueString(TCHAR_TO_UTF8(*StringChannel->GetDefault().Get(TEXT(""))));

					CreateAnimatableUserProperty(FbxNode, FbxValueString, TCHAR_TO_UTF8(*PropertyName), TCHAR_TO_UTF8(*PropertyName), FbxStringDT);
				}

				Property = FbxNode->FindProperty(TCHAR_TO_UTF8(*PropertyName), false);
			}

			if (Property == 0)
			{
				continue;
			}

			// Ensure that the property is animatable so that GetCurveNode succeeds
			Property.ModifyFlag(FbxPropertyFlags::eAnimatable, true);
			
			FbxAnimCurve* AnimCurve = FbxAnimCurve::Create(Scene, "");
			FbxAnimCurveNode* CurveNode = Property.GetCurveNode(true);
			if (!CurveNode)
			{
				continue;
			}

			if (DoubleChannel)
			{
				CurveNode->SetChannelValue<double>(0U, DoubleChannel->GetDefault().Get(MAX_dbl));
				CurveNode->ConnectToChannel(AnimCurve, 0U);

				if (bBakeBezierCurves)
				{
					ExportBezierChannelToFbxCurveBaked(*AnimCurve, *DoubleChannel, TickResolution, &Track, IsFoV ? ERichCurveValueMode::Fov : ERichCurveValueMode::Default, false, RootToLocalTransform);
				}
 				else
 				{
					ExportChannelToFbxCurve(*AnimCurve, *DoubleChannel, TickResolution, IsFoV ? ERichCurveValueMode::Fov : ERichCurveValueMode::Default, false, RootToLocalTransform);
				}
			}
			else if (FloatChannel)
			{
				CurveNode->SetChannelValue<double>(0U, FloatChannel->GetDefault().Get(MAX_flt));
				CurveNode->ConnectToChannel(AnimCurve, 0U);

				if (bBakeBezierCurves)
				{
					ExportBezierChannelToFbxCurveBaked(*AnimCurve, *FloatChannel, TickResolution, &Track, IsFoV ? ERichCurveValueMode::Fov : ERichCurveValueMode::Default, false, RootToLocalTransform);
				}
				else
				{
					ExportChannelToFbxCurve(*AnimCurve, *FloatChannel, TickResolution, IsFoV ? ERichCurveValueMode::Fov : ERichCurveValueMode::Default, false, RootToLocalTransform);
				}
			}
			else if (IntegerChannel)
			{
				CurveNode->SetChannelValue<int32>(0U, IntegerChannel->GetDefault().Get(0));
				CurveNode->ConnectToChannel(AnimCurve, 0U);

				ExportChannelToFbxCurve(*AnimCurve, *IntegerChannel, TickResolution, RootToLocalTransform);
			}
			else if (BoolChannel)
			{
				CurveNode->SetChannelValue<bool>(0U, BoolChannel->GetDefault().Get(false));
				CurveNode->ConnectToChannel(AnimCurve, 0U);

				ExportChannelToFbxCurve(*AnimCurve, *BoolChannel, TickResolution, RootToLocalTransform);
			}
			else if (EnumChannel)
			{
				CurveNode->SetChannelValue<uint8>(0U, EnumChannel->GetDefault().Get(0U));
				CurveNode->ConnectToChannel(AnimCurve, 0U);

				ExportChannelToFbxCurve(*AnimCurve, *EnumChannel, TickResolution, RootToLocalTransform);
			}
		}
	}
}
		
/**
 * Finds the given actor in the already-exported list of structures
 */
FbxNode* FFbxExporter::FindActor(AActor* Actor, INodeNameAdapter* NodeNameAdapter)
{
	if (NodeNameAdapter)
	{
		FbxNode* ActorNode = NodeNameAdapter->GetFbxNode(Actor);

		if (ActorNode)
		{
			return ActorNode;
		}
	}

	if (FbxActors.Find(Actor))
	{
		return *FbxActors.Find(Actor);
	}
	else
	{
		return NULL;
	}
}

FbxNode* FFbxExporter::CreateNode(const FString& NodeName)
{
	FbxNode* FbxNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*NodeName));
	Scene->GetRootNode()->AddChild(FbxNode);
	return FbxNode;
}

bool FFbxExporter::FindSkeleton(USkeletalMeshComponent* SkelComp, TArray<FbxNode*>& BoneNodes, INodeNameAdapter* NodeNameAdapter)
{
	if (NodeNameAdapter)
	{
		FbxNode* SkelRoot = NodeNameAdapter->GetFbxNode(SkelComp);
		if (SkelRoot)
		{
			BoneNodes.Empty();
			GetSkeleton(SkelRoot, BoneNodes);

			return true;
		}
	}

	FbxNode** SkelRoot = FbxSkeletonRoots.Find(SkelComp);

	if (SkelRoot)
	{
		BoneNodes.Empty();
		GetSkeleton(*SkelRoot, BoneNodes);

		return true;
	}

	return false;
}
/**
 * Determines the UVS to weld when exporting a Static Mesh
 * 
 * @param VertRemap		Index of each UV (out)
 * @param UniqueVerts	
 */
void DetermineUVsToWeld(TArray<int32>& VertRemap, TArray<int32>& UniqueVerts, const FStaticMeshVertexBuffer& VertexBuffer, int32 TexCoordSourceIndex)
{
	const int32 VertexCount = VertexBuffer.GetNumVertices();

	// Maps unreal verts to reduced list of verts
	VertRemap.Empty(VertexCount);
	VertRemap.AddUninitialized(VertexCount);

	// List of Unreal Verts to keep
	UniqueVerts.Empty(VertexCount);

	// Combine matching verts using hashed search to maintain good performance
	TMap<FVector2D,int32> HashedVerts;
	for(int32 Vertex=0; Vertex < VertexCount; Vertex++)
	{
		const FVector2D& PositionA = FVector2D(VertexBuffer.GetVertexUV(Vertex,TexCoordSourceIndex));
		const int32* FoundIndex = HashedVerts.Find(PositionA);
		if ( !FoundIndex )
		{
			int32 NewIndex = UniqueVerts.Add(Vertex);
			VertRemap[Vertex] = NewIndex;
			HashedVerts.Add(PositionA, NewIndex);
		}
		else
		{
			VertRemap[Vertex] = *FoundIndex;
		}
	}
}

void DetermineVertsToWeld(TArray<int32>& VertRemap, TArray<int32>& UniqueVerts, const FStaticMeshLODResources& RenderMesh)
{
	const int32 VertexCount = RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();

	// Maps unreal verts to reduced list of verts 
	VertRemap.Empty(VertexCount);
	VertRemap.AddUninitialized(VertexCount);

	// List of Unreal Verts to keep
	UniqueVerts.Empty(VertexCount);

	// Combine matching verts using hashed search to maintain good performance
	TMap<FVector,int32> HashedVerts;
	for(int32 a=0; a < VertexCount; a++)
	{
		const FVector& PositionA = (FVector)RenderMesh.VertexBuffers.PositionVertexBuffer.VertexPosition(a);
		const int32* FoundIndex = HashedVerts.Find(PositionA);
		if ( !FoundIndex )
		{
			int32 NewIndex = UniqueVerts.Add(a);
			VertRemap[a] = NewIndex;
			HashedVerts.Add(PositionA, NewIndex);
		}
		else
		{
			VertRemap[a] = *FoundIndex;
		}
	}
}

class FCollisionFbxExporter
{
public:
	FCollisionFbxExporter(const UStaticMesh *StaticMeshToExport, FbxMesh* ExportMesh, int32 ActualMatIndexToExport)
	{
		BoxPositions[0] = FVector(-1, -1, +1);
		BoxPositions[1] = FVector(-1, +1, +1);
		BoxPositions[2] = FVector(+1, +1, +1);
		BoxPositions[3] = FVector(+1, -1, +1);

		BoxFaceRotations[0] = FRotator(0, 0, 0);
		BoxFaceRotations[1] = FRotator(90.f, 0, 0);
		BoxFaceRotations[2] = FRotator(-90.f, 0, 0);
		BoxFaceRotations[3] = FRotator(0, 0, 90.f);
		BoxFaceRotations[4] = FRotator(0, 0, -90.f);
		BoxFaceRotations[5] = FRotator(180.f, 0, 0);

		DrawCollisionSides = 16;

		SpherNumSides = DrawCollisionSides;
		SphereNumRings = DrawCollisionSides / 2;
		SphereNumVerts = (SpherNumSides + 1) * (SphereNumRings + 1);

		CapsuleNumSides = DrawCollisionSides;
		CapsuleNumRings = (DrawCollisionSides / 2) + 1;
		CapsuleNumVerts = (CapsuleNumSides + 1) * (CapsuleNumRings + 1);

		CurrentVertexOffset = 0;

		StaticMesh = StaticMeshToExport;
		Mesh = ExportMesh;
		ActualMatIndex = ActualMatIndexToExport;
	}
	
	void ExportCollisions()
	{
		const FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;

		int32 VerticeNumber = 0;
		for (const FKConvexElem &ConvexElem : AggGeo.ConvexElems)
		{
			VerticeNumber += GetConvexVerticeNumber(ConvexElem);
		}
		for (const FKBoxElem &BoxElem : AggGeo.BoxElems)
		{
			VerticeNumber += GetBoxVerticeNumber();
		}
		for (const FKSphereElem &SphereElem : AggGeo.SphereElems)
		{
			VerticeNumber += GetSphereVerticeNumber();
		}
		for (const FKSphylElem &CapsuleElem : AggGeo.SphylElems)
		{
			VerticeNumber += GetCapsuleVerticeNumber();
		}

		Mesh->InitControlPoints(VerticeNumber);
		ControlPoints = Mesh->GetControlPoints();
		CurrentVertexOffset = 0;
		//////////////////////////////////////////////////////////////////////////
		// Set all vertex
		for (const FKConvexElem &ConvexElem : AggGeo.ConvexElems)
		{
			AddConvexVertex(ConvexElem);
		}

		for (const FKBoxElem &BoxElem : AggGeo.BoxElems)
		{
			AddBoxVertex(BoxElem);
		}

		for (const FKSphereElem &SphereElem : AggGeo.SphereElems)
		{
			AddSphereVertex(SphereElem);
		}

		for (const FKSphylElem &CapsuleElem : AggGeo.SphylElems)
		{
			AddCapsuleVertex(CapsuleElem);
		}

		// Set the normals on Layer 0.
		FbxLayer* Layer = Mesh->GetLayer(0);
		if (Layer == nullptr)
		{
			Mesh->CreateLayer();
			Layer = Mesh->GetLayer(0);
		}
		// Create and fill in the per-face-vertex normal data source.
		LayerElementNormal = FbxLayerElementNormal::Create(Mesh, "");
		// Set the normals per polygon instead of storing normals on positional control points
		LayerElementNormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		// Set the normal values for every polygon vertex.
		LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);

		//////////////////////////////////////////////////////////////////////////
		//Set the Normals
		for (const FKConvexElem &ConvexElem : AggGeo.ConvexElems)
		{
			AddConvexNormals(ConvexElem);
		}
		for (const FKBoxElem &BoxElem : AggGeo.BoxElems)
		{
			AddBoxNormal(BoxElem);
		}

		int32 SphereIndex = 0;
		for (const FKSphereElem &SphereElem : AggGeo.SphereElems)
		{
			AddSphereNormals(SphereElem, SphereIndex);
			SphereIndex++;
		}

		int32 CapsuleIndex = 0;
		for (const FKSphylElem &CapsuleElem : AggGeo.SphylElems)
		{
			AddCapsuleNormals(CapsuleElem, CapsuleIndex);
			CapsuleIndex++;
		}

		Layer->SetNormals(LayerElementNormal);

		//////////////////////////////////////////////////////////////////////////
		// Set polygons
		// Build list of polygon re-used multiple times to lookup Normals, UVs, other per face vertex information
		CurrentVertexOffset = 0; //Reset the current VertexCount
		for (const FKConvexElem &ConvexElem : AggGeo.ConvexElems)
		{
			AddConvexPolygon(ConvexElem);
		}

		for (const FKBoxElem &BoxElem : AggGeo.BoxElems)
		{
			AddBoxPolygons();
		}

		for (const FKSphereElem &SphereElem : AggGeo.SphereElems)
		{
			AddSpherePolygons();
		}

		for (const FKSphylElem &CapsuleElem : AggGeo.SphylElems)
		{
			AddCapsulePolygons();
		}

		//////////////////////////////////////////////////////////////////////////
		//Free the sphere resources
		for (FDynamicMeshVertex* DynamicMeshVertex : SpheresVerts)
		{
			FMemory::Free(DynamicMeshVertex);
		}
		SpheresVerts.Empty();

		//////////////////////////////////////////////////////////////////////////
		//Free the capsule resources
		for (FDynamicMeshVertex* DynamicMeshVertex : CapsuleVerts)
		{
			FMemory::Free(DynamicMeshVertex);
		}
		CapsuleVerts.Empty();
	}

private:
	uint32 GetConvexVerticeNumber(const FKConvexElem &ConvexElem)
	{
		return ConvexElem.VertexData.Num();
	}

	uint32 GetBoxVerticeNumber() { return 24; }

	uint32 GetSphereVerticeNumber() { return SphereNumVerts; }

	uint32 GetCapsuleVerticeNumber() { return CapsuleNumVerts; }

	void AddConvexVertex(const FKConvexElem &ConvexElem)
	{
		const TArray<FVector>& VertexArray = ConvexElem.VertexData;
		for (int32 PosIndex = 0; PosIndex < VertexArray.Num(); ++PosIndex)
		{
			FVector Position = VertexArray[PosIndex];
			ControlPoints[CurrentVertexOffset + PosIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
		}
		CurrentVertexOffset += VertexArray.Num();
	}

	void AddConvexNormals(const FKConvexElem &ConvexElem)
	{
		const auto& ConvexMesh = ConvexElem.GetChaosConvexMesh();
		if (!ConvexMesh.IsValid())
		{
			return;
		}
		const TArray<Chaos::FConvex::FPlaneType>& Faces = ConvexMesh->GetFaces();
		for (int32 PolyIndex = 0; PolyIndex < Faces.Num(); ++PolyIndex)
		{
			FVector Normal = (FVector)Faces[PolyIndex].Normal().GetSafeNormal();
			FbxVector4 FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z);
			// add vertices 
			for (int32 j = 0; j < 3; ++j)
			{
				LayerElementNormal->GetDirectArray().Add(FbxNormal);
			}
		}
	}

	void AddConvexPolygon(const FKConvexElem &ConvexElem)
	{
		const auto& ConvexMesh = ConvexElem.GetChaosConvexMesh();
		if (!ConvexMesh.IsValid())
		{
			return;
		}
		TArray<int32> IndexData(ConvexElem.IndexData);
		if (IndexData.Num() == 0)
		{
			IndexData = ConvexElem.GetChaosConvexIndices();
		}
		check(IndexData.Num() % 3 == 0);
		for (int32 VertexIndex = 0; VertexIndex < IndexData.Num(); VertexIndex += 3)
		{
			Mesh->BeginPolygon(ActualMatIndex);
			Mesh->AddPolygon(CurrentVertexOffset + IndexData[VertexIndex]);
			Mesh->AddPolygon(CurrentVertexOffset + IndexData[VertexIndex + 1]);
			Mesh->AddPolygon(CurrentVertexOffset + IndexData[VertexIndex + 2]);
			Mesh->EndPolygon();
		}

		CurrentVertexOffset += ConvexElem.VertexData.Num();
	}

	void AddBoxVertex(const FKBoxElem &BoxElem)
	{
		FScaleMatrix ExtendScale(0.5f * FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
		// Calculate verts for a face pointing down Z
		FMatrix BoxTransform = BoxElem.GetTransform().ToMatrixWithScale();
		for (int32 f = 0; f < 6; f++)
		{
			FMatrix FaceTransform = FRotationMatrix(BoxFaceRotations[f])*ExtendScale*BoxTransform;

			for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
			{
				FVector4 VertexPosition = FaceTransform.TransformPosition(BoxPositions[VertexIndex]);
				ControlPoints[CurrentVertexOffset + VertexIndex] = FbxVector4(VertexPosition.X, -VertexPosition.Y, VertexPosition.Z);
			}
			CurrentVertexOffset += 4;
		}
	}

	void AddBoxNormal(const FKBoxElem &BoxElem)
	{
		FScaleMatrix ExtendScale(0.5f * FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
		FMatrix BoxTransform = BoxElem.GetTransform().ToMatrixWithScale();
		for (int32 f = 0; f < 6; f++)
		{
			FMatrix FaceTransform = FRotationMatrix(BoxFaceRotations[f])*ExtendScale*BoxTransform;
			FVector4 TangentZ = FaceTransform.TransformVector(FVector(0, 0, 1));
			FbxVector4 FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
			FbxNormal.Normalize();
			for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
			{
				LayerElementNormal->GetDirectArray().Add(FbxNormal);
			}
		}
	}

	void AddBoxPolygons()
	{
		for (int32 f = 0; f < 6; f++)
		{
			Mesh->BeginPolygon(ActualMatIndex);
			for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
			{
				const uint32 VertIndex = CurrentVertexOffset + VertexIndex;
				Mesh->AddPolygon(VertIndex);
			}
			Mesh->EndPolygon();
			CurrentVertexOffset += 4;
		}
	}

	void AddSphereVertex(const FKSphereElem &SphereElem)
	{
		FMatrix SphereTransform = FScaleMatrix(SphereElem.Radius * FVector(1.0f)) * SphereElem.GetTransform().ToMatrixWithScale();
		FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(SphereNumVerts * sizeof(FDynamicMeshVertex));
		// Calculate verts for one arc
		FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((SphereNumRings + 1) * sizeof(FDynamicMeshVertex));

		for (int32 i = 0; i < SphereNumRings + 1; i++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[i];

			float angle = ((float)i / SphereNumRings) * PI;

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!			
			ArcVert->Position.X = 0.0f;
			ArcVert->Position.Y = FMath::Sin(angle);
			ArcVert->Position.Z = FMath::Cos(angle);

			ArcVert->SetTangents(
				FVector3f(1, 0, 0),
				FVector3f(0.0f, -ArcVert->Position.Z, ArcVert->Position.Y),
				ArcVert->Position
				);
		}

		// Then rotate this arc SpherNumSides+1 times.
		for (int32 s = 0; s < SpherNumSides + 1; s++)
		{
			FRotator3f ArcRotator(0, 360.f * (float)s / SpherNumSides, 0);
			FRotationMatrix44f ArcRot(ArcRotator);

			for (int32 v = 0; v < SphereNumRings + 1; v++)
			{
				int32 VIx = (SphereNumRings + 1)*s + v;

				Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[v].Position);

				Verts[VIx].SetTangents(
					ArcRot.TransformVector(ArcVerts[v].TangentX.ToFVector3f()),
					ArcRot.TransformVector(ArcVerts[v].GetTangentY()),
					ArcRot.TransformVector(ArcVerts[v].TangentZ.ToFVector3f())
					);
			}
		}

		// Add all of the vertices we generated to the mesh builder.
		for (int32 VertexIndex = 0; VertexIndex < SphereNumVerts; VertexIndex++)
		{
			FVector Position = (FVector)SphereTransform.TransformPosition((FVector)Verts[VertexIndex].Position);
			ControlPoints[CurrentVertexOffset + VertexIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
		}
		CurrentVertexOffset += SphereNumVerts;
		// Free our local copy of arc verts
		FMemory::Free(ArcVerts);
		SpheresVerts.Add(Verts);
	}

	void AddSphereNormals(const FKSphereElem &SphereElem, int32 SphereIndex)
	{
		FMatrix SphereTransform = FScaleMatrix(SphereElem.Radius * FVector(1.0f)) * SphereElem.GetTransform().ToMatrixWithScale();
		for (int32 s = 0; s < SpherNumSides; s++)
		{
			int32 a0start = (s + 0) * (SphereNumRings + 1);
			int32 a1start = (s + 1) * (SphereNumRings + 1);

			for (int32 r = 0; r < SphereNumRings; r++)
			{
				if (r != 0)
				{
					int32 indexV = a0start + r + 0;
					FVector TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxVector4 FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a1start + r + 0;
					TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a0start + r + 1;
					TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);
				}
				if (r != SphereNumRings - 1)
				{
					int32 indexV = a1start + r + 0;
					FVector TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxVector4 FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a1start + r + 1;
					TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a0start + r + 1;
					TangentZ = SphereTransform.TransformVector(SpheresVerts[SphereIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);
				}
			}
		}
	}

	void AddSpherePolygons()
	{
		for (int32 s = 0; s < SpherNumSides; s++)
		{
			int32 a0start = (s + 0) * (SphereNumRings + 1);
			int32 a1start = (s + 1) * (SphereNumRings + 1);

			for (int32 r = 0; r < SphereNumRings; r++)
			{
				if (r != 0)
				{
					Mesh->BeginPolygon(ActualMatIndex);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + r + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + r + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + r + 1);
					Mesh->EndPolygon();
				}
				if (r != SphereNumRings - 1)
				{
					Mesh->BeginPolygon(ActualMatIndex);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + r + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + r + 1);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + r + 1);
					Mesh->EndPolygon();
				}
			}
		}
		CurrentVertexOffset += SphereNumVerts;
	}

	void AddCapsuleVertex(const FKSphylElem &CapsuleElem)
	{
		FMatrix CapsuleTransform = CapsuleElem.GetTransform().ToMatrixWithScale();
		float Length = CapsuleElem.Length;
		float Radius = CapsuleElem.Radius;
		FDynamicMeshVertex* Verts = (FDynamicMeshVertex*)FMemory::Malloc(CapsuleNumVerts * sizeof(FDynamicMeshVertex));

		// Calculate verts for one arc
		FDynamicMeshVertex* ArcVerts = (FDynamicMeshVertex*)FMemory::Malloc((CapsuleNumRings + 1) * sizeof(FDynamicMeshVertex));

		for (int32 RingIdx = 0; RingIdx < CapsuleNumRings + 1; RingIdx++)
		{
			FDynamicMeshVertex* ArcVert = &ArcVerts[RingIdx];

			float Angle;
			float ZOffset;
			if (RingIdx <= DrawCollisionSides / 4)
			{
				Angle = ((float)RingIdx / (CapsuleNumRings - 1)) * PI;
				ZOffset = 0.5 * Length;
			}
			else
			{
				Angle = ((float)(RingIdx - 1) / (CapsuleNumRings - 1)) * PI;
				ZOffset = -0.5 * Length;
			}

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
			FVector SpherePos;
			SpherePos.X = 0.0f;
			SpherePos.Y = Radius * FMath::Sin(Angle);
			SpherePos.Z = Radius * FMath::Cos(Angle);

			ArcVert->Position = (FVector3f)SpherePos + FVector3f(0, 0, ZOffset);

			ArcVert->SetTangents(
				FVector3f(1, 0, 0),
				FVector3f(0.0f, -SpherePos.Z, SpherePos.Y),
				(FVector3f)SpherePos
				);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 SideIdx = 0; SideIdx < CapsuleNumSides + 1; SideIdx++)
		{
			const FRotator3f ArcRotator(0, 360.f * ((float)SideIdx / CapsuleNumSides), 0);
			const FRotationMatrix44f ArcRot(ArcRotator);

			for (int32 VertIdx = 0; VertIdx < CapsuleNumRings + 1; VertIdx++)
			{
				int32 VIx = (CapsuleNumRings + 1)*SideIdx + VertIdx;

				Verts[VIx].Position = ArcRot.TransformPosition(ArcVerts[VertIdx].Position);

				Verts[VIx].SetTangents(
					ArcRot.TransformVector(ArcVerts[VertIdx].TangentX.ToFVector3f()),
					ArcRot.TransformVector(ArcVerts[VertIdx].GetTangentY()),
					ArcRot.TransformVector(ArcVerts[VertIdx].TangentZ.ToFVector3f())
					);
			}
		}

		// Add all of the vertices we generated to the mesh builder.
		for (int32 VertexIndex = 0; VertexIndex < CapsuleNumVerts; VertexIndex++)
		{
			FVector Position = (FVector)CapsuleTransform.TransformPosition((FVector)Verts[VertexIndex].Position);
			ControlPoints[CurrentVertexOffset + VertexIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
		}
		CurrentVertexOffset += CapsuleNumVerts;
		// Free our local copy of arc verts
		FMemory::Free(ArcVerts);
		CapsuleVerts.Add(Verts);
	}
	
	void AddCapsuleNormals(const FKSphylElem &CapsuleElem, int32 CapsuleIndex)
	{
		FMatrix CapsuleTransform = CapsuleElem.GetTransform().ToMatrixWithScale();
		// Add all of the triangles to the mesh.
		for (int32 SideIdx = 0; SideIdx < CapsuleNumSides; SideIdx++)
		{
			const int32 a0start = (SideIdx + 0) * (CapsuleNumRings + 1);
			const int32 a1start = (SideIdx + 1) * (CapsuleNumRings + 1);

			for (int32 RingIdx = 0; RingIdx < CapsuleNumRings; RingIdx++)
			{
				if (RingIdx != 0)
				{
					int32 indexV = a0start + RingIdx + 0;
					FVector TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxVector4 FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a1start + RingIdx + 0;
					TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a0start + RingIdx + 1;
					TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);
				}
				if (RingIdx != CapsuleNumRings - 1)
				{
					int32 indexV = a1start + RingIdx + 0;
					FVector TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxVector4 FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a1start + RingIdx + 1;
					TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					indexV = a0start + RingIdx + 1;
					TangentZ = CapsuleTransform.TransformVector(CapsuleVerts[CapsuleIndex][indexV].TangentZ.ToFVector());
					FbxNormal = FbxVector4(TangentZ.X, -TangentZ.Y, TangentZ.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);
				}
			}
		}
	}

	void AddCapsulePolygons()
	{
		// Add all of the triangles to the mesh.
		for (int32 SideIdx = 0; SideIdx < CapsuleNumSides; SideIdx++)
		{
			const int32 a0start = (SideIdx + 0) * (CapsuleNumRings + 1);
			const int32 a1start = (SideIdx + 1) * (CapsuleNumRings + 1);

			for (int32 RingIdx = 0; RingIdx < CapsuleNumRings; RingIdx++)
			{
				if (RingIdx != 0)
				{
					Mesh->BeginPolygon(ActualMatIndex);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + RingIdx + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + RingIdx + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + RingIdx + 1);
					Mesh->EndPolygon();
				}
				if (RingIdx != CapsuleNumRings - 1)
				{
					Mesh->BeginPolygon(ActualMatIndex);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + RingIdx + 0);
					Mesh->AddPolygon(CurrentVertexOffset + a1start + RingIdx + 1);
					Mesh->AddPolygon(CurrentVertexOffset + a0start + RingIdx + 1);
					Mesh->EndPolygon();
				}
			}
		}
		CurrentVertexOffset += CapsuleNumVerts;
	}

	//////////////////////////////////////////////////////////////////////////
	//Box data
	FVector BoxPositions[4];
	FRotator BoxFaceRotations[6];
	

	int32 DrawCollisionSides;
	//////////////////////////////////////////////////////////////////////////
	//Sphere data
	int32 SpherNumSides;
	int32 SphereNumRings;
	int32 SphereNumVerts;
	TArray<FDynamicMeshVertex*> SpheresVerts;

	//////////////////////////////////////////////////////////////////////////
	//Capsule data
	int32 CapsuleNumSides;
	int32 CapsuleNumRings;
	int32 CapsuleNumVerts;
	TArray<FDynamicMeshVertex*> CapsuleVerts;

	//////////////////////////////////////////////////////////////////////////
	//Mesh Data
	uint32 CurrentVertexOffset;

	const UStaticMesh *StaticMesh;
	FbxMesh* Mesh;
	int32 ActualMatIndex;
	FbxVector4* ControlPoints;
	FbxLayerElementNormal* LayerElementNormal;
};

FbxNode* FFbxExporter::ExportCollisionMesh(const UStaticMesh* StaticMesh, const TCHAR* MeshName, FbxNode* ParentActor)
{
	const FKAggregateGeom& AggGeo = StaticMesh->GetBodySetup()->AggGeom;
	if (AggGeo.GetElementCount() <= 0)
	{
		return nullptr;
	}
	FbxMesh* Mesh = FbxMeshes.FindRef(StaticMesh);
	if (!Mesh)
	{
		//We export collision only if the mesh is already exported
		return nullptr;
	}

	//Name the node with the actor name
	FString MeshCollisionName = TEXT("UCX_");
	MeshCollisionName += UTF8_TO_TCHAR(ParentActor->GetName()); //-V595
	FbxNode* FbxActor = FbxNode::Create(Scene, TCHAR_TO_UTF8(*MeshCollisionName));

	if (ParentActor != nullptr)
	{
		// Collision meshes are added directly to the scene root, so we need to use the global transform instead of the relative one.
		FbxAMatrix& GlobalTransform = ParentActor->EvaluateGlobalTransform();
		FbxActor->LclTranslation.Set(GlobalTransform.GetT());
		FbxActor->LclRotation.Set(GlobalTransform.GetR());
		FbxActor->LclScaling.Set(GlobalTransform.GetS());
	}

	Scene->GetRootNode()->AddChild(FbxActor);

	FbxMesh* CollisionMesh = FbxCollisionMeshes.FindRef(StaticMesh);
	if (!CollisionMesh)
	{
		//Name the mesh attribute with the mesh name
		MeshCollisionName = TEXT("UCX_");
		MeshCollisionName += MeshName;
		CollisionMesh = FbxMesh::Create(Scene, TCHAR_TO_UTF8(*MeshCollisionName));
		//Export all collision elements in one mesh
		FbxSurfaceMaterial* FbxMaterial = nullptr;
		int32 ActualMatIndex = FbxActor->AddMaterial(FbxMaterial);
		FCollisionFbxExporter CollisionFbxExporter(StaticMesh, CollisionMesh, ActualMatIndex);
		CollisionFbxExporter.ExportCollisions();
		FbxCollisionMeshes.Add(StaticMesh, CollisionMesh);
	}

	//Set the original meshes in case it was already existing
	FbxActor->SetNodeAttribute(CollisionMesh);
	return FbxActor;
}


void FFbxExporter::ExportObjectMetadata(const UObject* ObjectToExport, FbxNode* Node)
{
	if (ObjectToExport && Node)
	{
		// Retrieve the metadata map without creating it
		const TMap<FName, FString>* MetadataMap = FMetaData::GetMapForObject(ObjectToExport);
		if (MetadataMap)
		{
			static const FString MetadataPrefix(FBX_METADATA_PREFIX);
			for (const auto& MetadataIt : *MetadataMap)
			{
				// Export object metadata tags that are prefixed as FBX custom user-defined properties
				// Remove the prefix since it's for Unreal use only (and '.' is considered an invalid character for user property names in DCC like Maya)
				FString TagAsString = MetadataIt.Key.ToString();
				if (TagAsString.RemoveFromStart(MetadataPrefix))
				{
					// Remaining tag follows the format NodeName.PropertyName, so replace '.' with '_'
					TagAsString.ReplaceInline(TEXT("."), TEXT("_"));

					if (MetadataIt.Value == TEXT("true") || MetadataIt.Value == TEXT("false"))
					{
						FbxProperty Property = FbxProperty::Create(Node, FbxBoolDT, TCHAR_TO_UTF8(*TagAsString));
						FbxBool ValueBool = MetadataIt.Value == TEXT("true") ? true : false;

						Property.Set(ValueBool);
						Property.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
					}
					else
					{
						FbxProperty Property = FbxProperty::Create(Node, FbxStringDT, TCHAR_TO_UTF8(*TagAsString));
						FbxString ValueString(TCHAR_TO_UTF8(*MetadataIt.Value));

						Property.Set(ValueString);
						Property.ModifyFlag(FbxPropertyFlags::eUserDefined, true);
					}
				}
			}
		}
	}
}

bool FFbxExporter::ExportStaticMeshFromMeshDescription(FbxMesh* Mesh
	, const UStaticMesh* StaticMesh
	, const FMeshDescription* MeshDescription
	, FbxNode* FbxActor
	, int32 LightmapUVChannel
	, const TArray<FStaticMaterial>* MaterialOrderOverride
	, const TArray<UMaterialInterface*>* OverrideMaterials
	, const FFbxMaterialBakingMeshData& MaterialBakingMeshData)
{

	if (MeshDescription->IsEmpty() || MeshDescription->Vertices().Num() == 0)
	{
		return false;
	}

	FStaticMeshConstAttributes Attributes(*MeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TEdgeAttributesConstRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();

	const int32 VertexCount = MeshDescription->Vertices().Num();
	const int32 VertexInstanceCount = MeshDescription->VertexInstances().Num();

	Mesh->InitControlPoints(VertexCount);

	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	for (int32 PosIndex = 0; PosIndex < VertexCount; ++PosIndex)
	{
		FVector Position = (FVector)VertexPositions[FVertexID(PosIndex)];
		ControlPoints[PosIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
	}

	// Set the normals on Layer 0.
	FbxLayer* Layer = Mesh->GetLayer(0);
	if (Layer == nullptr)
	{
		Mesh->CreateLayer();
		Layer = Mesh->GetLayer(0);
	}

	TArray<uint32> Indices;
	Indices.Reserve(VertexInstanceCount);
	for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
	{
		for(const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
			for(const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
			{
				Indices.Add(MeshDescription->GetVertexInstanceVertex(VertexInstanceID).GetValue());
			}
		}
	}

	// Create and fill in the per-face-vertex normal data source.
	// We extract the Z-tangent and the X/Y-tangents which are also stored in the render mesh.
	FbxLayerElementNormal* LayerElementNormal = FbxLayerElementNormal::Create(Mesh, "");
	FbxLayerElementTangent* LayerElementTangent = FbxLayerElementTangent::Create(Mesh, "");
	FbxLayerElementBinormal* LayerElementBinormal = FbxLayerElementBinormal::Create(Mesh, "");

	// Set 3 NTBs per triangle instead of storing on positional control points
	LayerElementNormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);
	LayerElementTangent->SetMappingMode(FbxLayerElement::eByPolygonVertex);
	LayerElementBinormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);

	// Set the NTBs values for every polygon vertex.
	LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);
	LayerElementTangent->SetReferenceMode(FbxLayerElement::eDirect);
	LayerElementBinormal->SetReferenceMode(FbxLayerElement::eDirect);

	//Extract the tangent space
	{
		for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			for (const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					//Query the tangent space
					FVector3f Normal, Tangent, Binormal;
					Tangent = VertexInstanceTangents[VertexInstanceID];
					Normal = VertexInstanceNormals[VertexInstanceID];
					float BinormalSign = VertexInstanceBinormalSigns[VertexInstanceID];
					Binormal = (FVector3f::CrossProduct(Normal, Tangent).GetSafeNormal() * BinormalSign);

					FbxVector4 FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z);
					FbxNormal.Normalize();
					LayerElementNormal->GetDirectArray().Add(FbxNormal);

					FbxVector4 FbxTangent = FbxVector4(Tangent.X, -Tangent.Y, Tangent.Z);
					FbxTangent.Normalize();
					LayerElementTangent->GetDirectArray().Add(FbxTangent);

					FbxVector4 FbxBinormal = FbxVector4(-Binormal.X, Binormal.Y, -Binormal.Z);
					FbxBinormal.Normalize();
					LayerElementBinormal->GetDirectArray().Add(FbxBinormal);
				}
			}
		}
	}

	Layer->SetNormals(LayerElementNormal);
	Layer->SetTangents(LayerElementTangent);
	Layer->SetBinormals(LayerElementBinormal);

	// Create and fill in the per-face-vertex texture coordinate data source(s).
	// Create UV for Diffuse channel.
	int32 TexCoordSourceCount = (LightmapUVChannel == -1) ? MeshDescription->GetNumUVElementChannels() : LightmapUVChannel + 1;
	int32 TexCoordSourceIndex = (LightmapUVChannel == -1) ? 0 : LightmapUVChannel;
	for (; TexCoordSourceIndex < TexCoordSourceCount; ++TexCoordSourceIndex)
	{
		FbxLayer* UVsLayer = (LightmapUVChannel == -1) ? Mesh->GetLayer(TexCoordSourceIndex) : Mesh->GetLayer(0);
		if (UVsLayer == NULL)
		{
			Mesh->CreateLayer();
			UVsLayer = (LightmapUVChannel == -1) ? Mesh->GetLayer(TexCoordSourceIndex) : Mesh->GetLayer(0);
		}
		check(UVsLayer);

		FString UVChannelNameBuilder = TEXT("UVmap_") + FString::FromInt(TexCoordSourceIndex);
		const auto UVChannelNameUTF8 = TStringConversion<FTCHARToUTF8_Convert>(*UVChannelNameBuilder); // Do not inline it! The lifetime of this object needs to extend over the usage of the converted buffer.
		const char* UVChannelName = UVChannelNameUTF8.Get(); // actually UTF8 as required by Fbx, but can't use UE's UTF8CHAR type because that's a uint8 aka *unsigned* char
		if ((LightmapUVChannel >= 0) || ((LightmapUVChannel == -1) && (TexCoordSourceIndex == StaticMesh->GetLightMapCoordinateIndex())))
		{
			UVChannelName = "LightMapUV";
		}
		
		FbxLayerElementUV* UVDiffuseLayer = FbxLayerElementUV::Create(Mesh, UVChannelName);

		UVDiffuseLayer->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		UVDiffuseLayer->SetReferenceMode(FbxLayerElement::eDirect);

		for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			for (const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					const FVector2f& VertexUV = VertexInstanceUVs.Get(VertexInstanceID, TexCoordSourceIndex);
					UVDiffuseLayer->GetDirectArray().Add(FbxVector2(VertexUV.X, -VertexUV.Y + 1.0));
				}
			}
		}
		UVsLayer->SetUVs(UVDiffuseLayer, FbxLayerElement::eTextureDiffuse);
	}

	FbxLayerElementMaterial* MatLayer = FbxLayerElementMaterial::Create(Mesh, "");
	MatLayer->SetMappingMode(FbxLayerElement::eByPolygon);
	MatLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	Layer->SetMaterials(MatLayer);

	FbxLayerElementSmoothing* SmoothingInfoLayer = FbxLayerElementSmoothing::Create(Mesh, ""); 
	SmoothingInfoLayer->SetMappingMode(FbxLayerElement::eByEdge);
	SmoothingInfoLayer->SetReferenceMode(FbxLayerElement::eDirect);
	Layer->SetSmoothing(SmoothingInfoLayer);

	//Create the polygon with the correct material
	{
		int32 IndiceIndex = 0;
		TSet<FEdgeID> ProcessEdges;
		ProcessEdges.Reserve(Indices.Num());
		for(const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			UMaterialInterface* Material = nullptr;

			FName CurrentMaterialSlotName = PolygonGroupMaterialSlotNames[PolygonGroupID];
			int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(CurrentMaterialSlotName);
			if (MaterialIndex == INDEX_NONE)
			{
				MaterialIndex = PolygonGroupID.GetValue();
			}
			if (!StaticMesh->GetStaticMaterials().IsValidIndex(MaterialIndex))
			{
				MaterialIndex = 0;
			}

			if (OverrideMaterials && OverrideMaterials->IsValidIndex(PolygonGroupID.GetValue()))
			{
				Material = (*OverrideMaterials)[PolygonGroupID.GetValue()];
			}
			else
			{
				Material = StaticMesh->GetMaterial(MaterialIndex);
			}

			FbxSurfaceMaterial* FbxMaterial = Material ? ExportMaterial(Material, MaterialIndex, MaterialBakingMeshData) : nullptr;
			if (!FbxMaterial)
			{
				FbxMaterial = CreateDefaultMaterial();
			}
			int32 MatIndex = FbxActor->AddMaterial(FbxMaterial);

			// Determine the actual material index
			int32 ActualMatIndex = MatIndex;

			if (MaterialOrderOverride)
			{
				ActualMatIndex = MaterialOrderOverride->Find(Material);
			}

			//Create the triangles
			{
				for (const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
				{
					FVertexID Corner[3];
					int32 CornerIndex = 0;
					Mesh->BeginPolygon(ActualMatIndex);
					TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
					for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
					{
						Corner[CornerIndex] = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
						Mesh->AddPolygon(Indices[IndiceIndex]);
						IndiceIndex++;
						CornerIndex++;
					}
					Mesh->EndPolygon();
				}
			}
		}
		//Build the edge so we can set the edge hardness
		Mesh->BuildMeshEdgeArray();

		Mesh->BeginGetMeshEdgeIndexForPolygon();
		int32 PolygonIndex = 0;
		for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			for (const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
			{
				FVertexID Corner[3];
				int32 CornerIndex = 0;
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					Corner[CornerIndex] = MeshDescription->GetVertexInstanceVertex(VertexInstanceID);
					CornerIndex++;
				}

				//Add the smoothing group for the triangle
				for (CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
				{
					FVertexID EdgeStart = Corner[CornerIndex];
					FVertexID EdgeEnd = Corner[(CornerIndex + 1) % 3];
					FEdgeID MatchEdgeId = MeshDescription->GetVertexPairEdge(EdgeStart, EdgeEnd);
					if (ProcessEdges.Contains(MatchEdgeId))
					{
						continue;
					}
					ProcessEdges.Add(MatchEdgeId);

					int32 FbxEdgeIndex = Mesh->GetMeshEdgeIndexForPolygon(PolygonIndex, CornerIndex);
					if (FbxEdgeIndex == -1)
					{
						continue;
					}
					int32 EdgeHardnessValue = 1;
					if (MatchEdgeId != INDEX_NONE)
					{
						EdgeHardnessValue = EdgeHardnesses[MatchEdgeId] ? 0 : 1;
					}
					int32 LayerAddIndex = SmoothingInfoLayer->GetDirectArray().Add(EdgeHardnessValue);
					ensure(LayerAddIndex == FbxEdgeIndex);
				}
				PolygonIndex++;
			}
		}
		Mesh->EndGetMeshEdgeIndexForPolygon();
	}

	// Create and fill in the vertex color data source.
	uint32 ColorVertexCount = MeshDescription->VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Color) ? MeshDescription->VertexInstances().Num() : 0;

	// Only export vertex colors if they exist
	if (GetExportOptions()->VertexColor && ColorVertexCount > 0)
	{
		FbxLayerElementVertexColor* VertexColor = FbxLayerElementVertexColor::Create(Mesh, "");
		VertexColor->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		VertexColor->SetReferenceMode(FbxLayerElement::eDirect);
		FbxLayerElementArrayTemplate<FbxColor>& VertexColorArray = VertexColor->GetDirectArray();
		Layer->SetVertexColors(VertexColor);

		for (const FPolygonGroupID& PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			for (const FTriangleID& TriangleID : MeshDescription->GetPolygonGroupTriangles(PolygonGroupID))
			{
				TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = MeshDescription->GetTriangleVertexInstances(TriangleID);
				for (const FVertexInstanceID& VertexInstanceID : TriangleVertexInstanceIDs)
				{
					const FVector4f& SourceVertexColor = VertexInstanceColors[VertexInstanceID];
					FLinearColor LinearColor(SourceVertexColor);
					//Convert to sRGB
					FColor Color = LinearColor.ToFColor(true);
					VertexColorArray.Add(FbxColor(static_cast<float>(Color.R)/255.0f, static_cast<float>(Color.G) / 255.0f, static_cast<float>(Color.B) / 255.0f, static_cast<float>(Color.A) / 255.0f));
				}
			}
		}
	}
	return true;
}

bool FFbxExporter::ExportStaticMeshFromRenderData(FbxMesh* Mesh
	, const UStaticMesh* StaticMesh
	, const FStaticMeshLODResources& RenderMesh
	, FbxNode* FbxActor
	, int32 LightmapUVChannel
	, const FColorVertexBuffer* ColorBuffer
	, const TArray<FStaticMaterial>* MaterialOrderOverride
	, const TArray<UMaterialInterface*>* OverrideMaterials
	, const FFbxMaterialBakingMeshData& MaterialBakingMeshData)
{
	// Verify the integrity of the static mesh.
	if (RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		return false;
	}

	if (RenderMesh.Sections.Num() == 0)
	{
		return false;
	}

	// Remaps an Unreal vert to final reduced vertex list
	TArray<int32> VertRemap;
	TArray<int32> UniqueVerts;

	// Weld verts
	DetermineVertsToWeld(VertRemap, UniqueVerts, RenderMesh);

	// Create and fill in the vertex position data source.
	// The position vertices are duplicated, for some reason, retrieve only the first half vertices.
	const int32 VertexCount = VertRemap.Num();
	const int32 PolygonsCount = RenderMesh.Sections.Num();

	Mesh->InitControlPoints(UniqueVerts.Num());

	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	for (int32 PosIndex = 0; PosIndex < UniqueVerts.Num(); ++PosIndex)
	{
		int32 UnrealPosIndex = UniqueVerts[PosIndex];
		FVector Position = (FVector)RenderMesh.VertexBuffers.PositionVertexBuffer.VertexPosition(UnrealPosIndex);
		ControlPoints[PosIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
	}

	// Set the normals on Layer 0.
	FbxLayer* Layer = Mesh->GetLayer(0);
	if (Layer == nullptr)
	{
		Mesh->CreateLayer();
		Layer = Mesh->GetLayer(0);
	}

	// Build list of Indices re-used multiple times to lookup Normals, UVs, other per face vertex information
	TArray<uint32> Indices;
	for (int32 PolygonsIndex = 0; PolygonsIndex < PolygonsCount; ++PolygonsIndex)
	{
		FIndexArrayView RawIndices = RenderMesh.IndexBuffer.GetArrayView();
		const FStaticMeshSection& Polygons = RenderMesh.Sections[PolygonsIndex];
		const uint32 TriangleCount = Polygons.NumTriangles;
		for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				uint32 UnrealVertIndex = RawIndices[Polygons.FirstIndex + ((TriangleIndex * 3) + PointIndex)];
				Indices.Add(UnrealVertIndex);
			}
		}
	}

	// Create and fill in the per-face-vertex normal data source.
	// We extract the Z-tangent and the X/Y-tangents which are also stored in the render mesh.
	FbxLayerElementNormal* LayerElementNormal = FbxLayerElementNormal::Create(Mesh, "");
	FbxLayerElementTangent* LayerElementTangent = FbxLayerElementTangent::Create(Mesh, "");
	FbxLayerElementBinormal* LayerElementBinormal = FbxLayerElementBinormal::Create(Mesh, "");

	// Set 3 NTBs per triangle instead of storing on positional control points
	LayerElementNormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);
	LayerElementTangent->SetMappingMode(FbxLayerElement::eByPolygonVertex);
	LayerElementBinormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);

	// Set the NTBs values for every polygon vertex.
	LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);
	LayerElementTangent->SetReferenceMode(FbxLayerElement::eDirect);
	LayerElementBinormal->SetReferenceMode(FbxLayerElement::eDirect);

	TArray<FbxVector4> FbxNormals;
	TArray<FbxVector4> FbxTangents;
	TArray<FbxVector4> FbxBinormals;

	FbxNormals.AddUninitialized(VertexCount);
	FbxTangents.AddUninitialized(VertexCount);
	FbxBinormals.AddUninitialized(VertexCount);

	for (int32 NTBIndex = 0; NTBIndex < VertexCount; ++NTBIndex)
	{
		FVector3f Normal = (FVector3f)(RenderMesh.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(NTBIndex));
		FbxVector4& FbxNormal = FbxNormals[NTBIndex];
		FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z);
		FbxNormal.Normalize();

		FVector3f Tangent = (FVector3f)(RenderMesh.VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(NTBIndex));
		FbxVector4& FbxTangent = FbxTangents[NTBIndex];
		FbxTangent = FbxVector4(Tangent.X, -Tangent.Y, Tangent.Z);
		FbxTangent.Normalize();

		FVector3f Binormal = -(FVector3f)(RenderMesh.VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(NTBIndex));
		FbxVector4& FbxBinormal = FbxBinormals[NTBIndex];
		FbxBinormal = FbxVector4(Binormal.X, -Binormal.Y, Binormal.Z);
		FbxBinormal.Normalize();
	}

	// Add one normal per each face index (3 per triangle)
	for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
	{
		uint32 UnrealVertIndex = Indices[FbxVertIndex];
		LayerElementNormal->GetDirectArray().Add(FbxNormals[UnrealVertIndex]);
		LayerElementTangent->GetDirectArray().Add(FbxTangents[UnrealVertIndex]);
		LayerElementBinormal->GetDirectArray().Add(FbxBinormals[UnrealVertIndex]);
	}

	Layer->SetNormals(LayerElementNormal);
	Layer->SetTangents(LayerElementTangent);
	Layer->SetBinormals(LayerElementBinormal);

	FbxNormals.Empty();
	FbxTangents.Empty();
	FbxBinormals.Empty();

	// Create and fill in the per-face-vertex texture coordinate data source(s).
	// Create UV for Diffuse channel.
	int32 TexCoordSourceCount = (LightmapUVChannel == -1) ? RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() : LightmapUVChannel + 1;
	int32 TexCoordSourceIndex = (LightmapUVChannel == -1) ? 0 : LightmapUVChannel;
	for (; TexCoordSourceIndex < TexCoordSourceCount; ++TexCoordSourceIndex)
	{
		FbxLayer* UVsLayer = (LightmapUVChannel == -1) ? Mesh->GetLayer(TexCoordSourceIndex) : Mesh->GetLayer(0);
		if (UVsLayer == NULL)
		{
			Mesh->CreateLayer();
			UVsLayer = (LightmapUVChannel == -1) ? Mesh->GetLayer(TexCoordSourceIndex) : Mesh->GetLayer(0);
		}
		check(UVsLayer);

		FString UVChannelNameBuilder = TEXT("UVmap_") + FString::FromInt(TexCoordSourceIndex);
		const auto UVChannelNameUTF8 = TStringConversion<FTCHARToUTF8_Convert>(*UVChannelNameBuilder); // Do not inline it! The lifetime of this object needs to extend over the usage of the converted buffer.
		const char* UVChannelName = UVChannelNameUTF8.Get(); // actually UTF8 as required by Fbx, but can't use UE's UTF8CHAR type because that's a uint8 aka *unsigned* char
		if ((LightmapUVChannel >= 0) || ((LightmapUVChannel == -1) && (TexCoordSourceIndex == StaticMesh->GetLightMapCoordinateIndex())))
		{
			UVChannelName = "LightMapUV";
		}

		FbxLayerElementUV* UVDiffuseLayer = FbxLayerElementUV::Create(Mesh, UVChannelName);

		// Note: when eINDEX_TO_DIRECT is used, IndexArray must be 3xTriangle count, DirectArray can be smaller
		UVDiffuseLayer->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		UVDiffuseLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);

		TArray<int32> UvsRemap;
		TArray<int32> UniqueUVs;
		// Weld UVs
		DetermineUVsToWeld(UvsRemap, UniqueUVs, RenderMesh.VertexBuffers.StaticMeshVertexBuffer, TexCoordSourceIndex);

		// Create the texture coordinate data source.
		for (int32 FbxVertIndex = 0; FbxVertIndex < UniqueUVs.Num(); FbxVertIndex++)
		{
			int32 UnrealVertIndex = UniqueUVs[FbxVertIndex];
			const FVector2f& TexCoord = RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(UnrealVertIndex, TexCoordSourceIndex);
			UVDiffuseLayer->GetDirectArray().Add(FbxVector2(TexCoord.X, -TexCoord.Y + 1.0));
		}

		// For each face index, point to a texture uv
		UVDiffuseLayer->GetIndexArray().SetCount(Indices.Num());
		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			uint32 UnrealVertIndex = Indices[FbxVertIndex];
			int32 NewVertIndex = UvsRemap[UnrealVertIndex];
			UVDiffuseLayer->GetIndexArray().SetAt(FbxVertIndex, NewVertIndex);
		}

		UVsLayer->SetUVs(UVDiffuseLayer, FbxLayerElement::eTextureDiffuse);
	}

	FbxLayerElementMaterial* MatLayer = FbxLayerElementMaterial::Create(Mesh, "");
	MatLayer->SetMappingMode(FbxLayerElement::eByPolygon);
	MatLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	Layer->SetMaterials(MatLayer);

	// Keep track of the number of tri's we export
	uint32 AccountedTriangles = 0;
	for (int32 PolygonsIndex = 0; PolygonsIndex < PolygonsCount; ++PolygonsIndex)
	{
		const FStaticMeshSection& Polygons = RenderMesh.Sections[PolygonsIndex];
		FIndexArrayView RawIndices = RenderMesh.IndexBuffer.GetArrayView();
		UMaterialInterface* Material = nullptr;

		if (OverrideMaterials && OverrideMaterials->IsValidIndex(Polygons.MaterialIndex))
		{
			Material = (*OverrideMaterials)[Polygons.MaterialIndex];
		}
		else
		{
			Material = StaticMesh->GetMaterial(Polygons.MaterialIndex);
		}

		FbxSurfaceMaterial* FbxMaterial = Material ? ExportMaterial(Material, Polygons.MaterialIndex, MaterialBakingMeshData) : NULL;
		if (!FbxMaterial)
		{
			FbxMaterial = CreateDefaultMaterial();
		}
		int32 MatIndex = FbxActor->AddMaterial(FbxMaterial);

		// Determine the actual material index
		int32 ActualMatIndex = MatIndex;

		if (MaterialOrderOverride)
		{
			ActualMatIndex = MaterialOrderOverride->Find(Material);
		}
		// Static meshes contain one triangle list per element.
		// [GLAFORTE] Could it occasionally contain triangle strips? How do I know?
		uint32 TriangleCount = Polygons.NumTriangles;

		// Copy over the index buffer into the FBX polygons set.
		for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			Mesh->BeginPolygon(ActualMatIndex);
			for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				uint32 OriginalUnrealVertIndex = RawIndices[Polygons.FirstIndex + ((TriangleIndex * 3) + PointIndex)];
				int32 RemappedVertIndex = VertRemap[OriginalUnrealVertIndex];
				Mesh->AddPolygon(RemappedVertIndex);
			}
			Mesh->EndPolygon();
		}

		AccountedTriangles += TriangleCount;
	}

#ifdef TODO_FBX
	// Throw a warning if this is a lightmap export and the exported poly count does not match the raw triangle data count
	if (LightmapUVChannel != -1 && AccountedTriangles != RenderMesh.RawTriangles.GetElementCount())
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "StaticMeshEditor_LightmapExportFewerTriangles", "Fewer polygons have been exported than the raw triangle count.  This Lightmapped UV mesh may contain fewer triangles than the destination mesh on import."));
	}

	// Create and fill in the smoothing data source.
	FbxLayerElementSmoothing* SmoothingInfo = FbxLayerElementSmoothing::Create(Mesh, "");
	SmoothingInfo->SetMappingMode(FbxLayerElement::eByPolygon);
	SmoothingInfo->SetReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElementArrayTemplate<int>& SmoothingArray = SmoothingInfo->GetDirectArray();
	Layer->SetSmoothing(SmoothingInfo);

	// This is broken. We are exporting the render mesh but providing smoothing
	// information from the source mesh. The render triangles are not in the
	// same order. Therefore we should export the raw mesh or not export
	// smoothing group information!
	int32 TriangleCount = RenderMesh.RawTriangles.GetElementCount();
	FStaticMeshTriangle* RawTriangleData = (FStaticMeshTriangle*)RenderMesh.RawTriangles.Lock(LOCK_READ_ONLY);
	for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; TriangleIndex++)
	{
		FStaticMeshTriangle* Triangle = (RawTriangleData++);

		SmoothingArray.Add(Triangle->SmoothingMask);
	}
	RenderMesh.RawTriangles.Unlock();
#endif // #if TODO_FBX

	// Create and fill in the vertex color data source.
	const FColorVertexBuffer* ColorBufferToUse = ColorBuffer ? ColorBuffer : &RenderMesh.VertexBuffers.ColorVertexBuffer;
	uint32 ColorVertexCount = ColorBufferToUse->GetNumVertices();

	// Only export vertex colors if they exist
	if (GetExportOptions()->VertexColor && ColorVertexCount > 0)
	{
		FbxLayerElementVertexColor* VertexColor = FbxLayerElementVertexColor::Create(Mesh, "");
		VertexColor->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		VertexColor->SetReferenceMode(FbxLayerElement::eIndexToDirect);
		FbxLayerElementArrayTemplate<FbxColor>& VertexColorArray = VertexColor->GetDirectArray();
		Layer->SetVertexColors(VertexColor);

		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			FLinearColor VertColor(1.0f, 1.0f, 1.0f);
			uint32 UnrealVertIndex = Indices[FbxVertIndex];
			if (UnrealVertIndex < ColorVertexCount)
			{
				VertColor = ColorBufferToUse->VertexColor(UnrealVertIndex).ReinterpretAsLinear();
			}

			VertexColorArray.Add(FbxColor(VertColor.R, VertColor.G, VertColor.B, VertColor.A));
		}

		VertexColor->GetIndexArray().SetCount(Indices.Num());
		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			VertexColor->GetIndexArray().SetAt(FbxVertIndex, FbxVertIndex);
		}
	}
	return true;
}

/**
 * Exports a static mesh
 * @param StaticMesh	The static mesh to export
 * @param MeshName		The name of the mesh for the FBX file
 * @param FbxActor		The fbx node representing the mesh
 * @param ExportLOD		The LOD of the mesh to export
 * @param LightmapUVChannel If set, performs a "lightmap export" and exports only the single given UV channel
 * @param ColorBuffer	Vertex color overrides to export
 * @param MaterialOrderOverride	Optional ordering of materials to set up correct material ID's across multiple meshes being export such as BSP surfaces which share common materials. Should be used sparingly
 */
FbxNode* FFbxExporter::ExportStaticMeshToFbx(const UStaticMesh* StaticMesh, int32 ExportLOD, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData, int32 LightmapUVChannel /*= -1*/, const FColorVertexBuffer* ColorBuffer /*= NULL*/, const TArray<FStaticMaterial>* MaterialOrderOverride /*= NULL*/, const TArray<UMaterialInterface*>* OverrideMaterials /*= NULL*/)
{
	FbxMesh* Mesh = nullptr;
	if ((ExportLOD == 0 || ExportLOD == -1) && LightmapUVChannel == -1 && ColorBuffer == nullptr && MaterialOrderOverride == nullptr)
	{
		Mesh = FbxMeshes.FindRef(StaticMesh);
	}

	if (!Mesh)
	{
		Mesh = FbxMesh::Create(Scene, TCHAR_TO_UTF8(MeshName));

		int32 LodIndex = ExportLOD == INDEX_NONE ? 0 : ExportLOD;
		bool bExportSourceMesh = GetExportOptions()->bExportSourceMesh && !GetExportOptions()->LevelOfDetail && !GetExportOptions()->Collision;
		const bool bUseNaniteData = LodIndex == 0 && StaticMesh->IsNaniteEnabled() && StaticMesh->IsHiResMeshDescriptionValid();
		const bool bUseLodData = !bUseNaniteData && StaticMesh->IsMeshDescriptionValid(LodIndex);
		bExportSourceMesh &= (bUseNaniteData || bUseLodData);

		if (bExportSourceMesh)
		{
			if (bUseNaniteData)
			{
				//Export the nanite mesh description
				if (!ExportStaticMeshFromMeshDescription(Mesh, StaticMesh, StaticMesh->GetHiResMeshDescription(), FbxActor, LightmapUVChannel, MaterialOrderOverride, OverrideMaterials, MaterialBakingMeshData))
				{
					return nullptr;
				}
			}
			else
			{
				ensure(bUseLodData);
				//Export the lod mesh description
				if(!ExportStaticMeshFromMeshDescription(Mesh, StaticMesh, StaticMesh->GetMeshDescription(LodIndex), FbxActor, LightmapUVChannel, MaterialOrderOverride, OverrideMaterials, MaterialBakingMeshData))
				{
					return nullptr;
				}
			}
		}
		else
		{
			//Export the render data
			const FStaticMeshLODResources& RenderMesh = StaticMesh->GetLODForExport(LodIndex);
			if (!ExportStaticMeshFromRenderData(Mesh, StaticMesh, RenderMesh, FbxActor, LightmapUVChannel, ColorBuffer, MaterialOrderOverride, OverrideMaterials, MaterialBakingMeshData))
			{
				return nullptr;
			}
		}
		
		if (LodIndex == 0 && LightmapUVChannel == -1 && ColorBuffer == nullptr && MaterialOrderOverride == nullptr)
		{
			FbxMeshes.Add(StaticMesh, Mesh);
		}
	}
	else
	{
		//Materials in fbx are store in the node and not in the mesh, so even if the mesh was already export
		//we have to find and assign the mesh material.
		const FStaticMeshLODResources& RenderMesh = StaticMesh->GetLODForExport(ExportLOD);
		const int32 PolygonsCount = RenderMesh.Sections.Num();
		uint32 AccountedTriangles = 0;
		for (int32 PolygonsIndex = 0; PolygonsIndex < PolygonsCount; ++PolygonsIndex)
		{
			const FStaticMeshSection& Polygons = RenderMesh.Sections[PolygonsIndex];
			FIndexArrayView RawIndices = RenderMesh.IndexBuffer.GetArrayView();
			UMaterialInterface* Material = nullptr;
			
			if (OverrideMaterials && OverrideMaterials->IsValidIndex(Polygons.MaterialIndex))
			{
				Material = (*OverrideMaterials)[Polygons.MaterialIndex];
			}
			else
			{
				Material = StaticMesh->GetMaterial(Polygons.MaterialIndex);
			}

			FbxSurfaceMaterial* FbxMaterial = Material ? ExportMaterial(Material, Polygons.MaterialIndex, MaterialBakingMeshData) : NULL;
			if (!FbxMaterial)
			{
				FbxMaterial = CreateDefaultMaterial();
			}
			FbxActor->AddMaterial(FbxMaterial);
		}
	}

	if ((ExportLOD == 0 || ExportLOD == -1) && GetExportOptions()->Collision)
	{
		ExportCollisionMesh(StaticMesh, MeshName, FbxActor);
	}

	//Set the original meshes in case it was already existing
	FbxActor->SetNodeAttribute(Mesh);

	ExportObjectMetadata(StaticMesh, FbxActor);

	return FbxActor;
}

void FFbxExporter::ExportSplineMeshToFbx(const USplineMeshComponent* SplineMeshComp, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData)
{
	const UStaticMesh* StaticMesh = SplineMeshComp->GetStaticMesh();
	check(StaticMesh);

	const int32 LODIndex = (SplineMeshComp->ForcedLodModel > 0 ? FMath::Min(StaticMesh->GetNumLODs(), SplineMeshComp->ForcedLodModel) - 1 : /* auto-select*/ 0);
	if (!SplineMeshComp->LODData.IsValidIndex(LODIndex))
	{
		return;
	}

	const FStaticMeshLODResources& RenderMesh = StaticMesh->GetLODForExport(LODIndex);

	// Verify the integrity of the static mesh.
	if (RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		return;
	}

	if (RenderMesh.Sections.Num() == 0)
	{
		return;
	}

	// Remaps an Unreal vert to final reduced vertex list
	TArray<int32> VertRemap;
	TArray<int32> UniqueVerts;

	// Weld verts
	DetermineVertsToWeld(VertRemap, UniqueVerts, RenderMesh);

	FbxMesh* Mesh = FbxMesh::Create(Scene, TCHAR_TO_UTF8(MeshName));

	// Create and fill in the vertex position data source.
	// The position vertices are duplicated, for some reason, retrieve only the first half vertices.
	const int32 VertexCount = VertRemap.Num();
	const int32 PolygonsCount = RenderMesh.Sections.Num();

	Mesh->InitControlPoints(UniqueVerts.Num());

	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	for (int32 PosIndex = 0; PosIndex < UniqueVerts.Num(); ++PosIndex)
	{
		int32 UnrealPosIndex = UniqueVerts[PosIndex];
		FVector Position = (FVector)RenderMesh.VertexBuffers.PositionVertexBuffer.VertexPosition(UnrealPosIndex);

		const FTransform SliceTransform = SplineMeshComp->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComp->ForwardAxis));
		USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComp->ForwardAxis) = 0;
		Position = SliceTransform.TransformPosition(Position);

		ControlPoints[PosIndex] = FbxVector4(Position.X, -Position.Y, Position.Z);
	}

	// Set the normals on Layer 0.
	FbxLayer* Layer = Mesh->GetLayer(0);
	if (Layer == NULL)
	{
		Mesh->CreateLayer();
		Layer = Mesh->GetLayer(0);
	}

	// Build list of Indices re-used multiple times to lookup Normals, UVs, other per face vertex information
	TArray<uint32> Indices;
	for (int32 PolygonsIndex = 0; PolygonsIndex < PolygonsCount; ++PolygonsIndex)
	{
		FIndexArrayView RawIndices = RenderMesh.IndexBuffer.GetArrayView();
		const FStaticMeshSection& Polygons = RenderMesh.Sections[PolygonsIndex];
		const uint32 TriangleCount = Polygons.NumTriangles;
		for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				uint32 UnrealVertIndex = RawIndices[Polygons.FirstIndex + ((TriangleIndex * 3) + PointIndex)];
				Indices.Add(UnrealVertIndex);
			}
		}
	}

	// Create and fill in the per-face-vertex normal data source.
	// We extract the Z-tangent and drop the X/Y-tangents which are also stored in the render mesh.
	FbxLayerElementNormal* LayerElementNormal = FbxLayerElementNormal::Create(Mesh, "");

	// Set 3 normals per triangle instead of storing normals on positional control points
	LayerElementNormal->SetMappingMode(FbxLayerElement::eByPolygonVertex);

	// Set the normal values for every polygon vertex.
	LayerElementNormal->SetReferenceMode(FbxLayerElement::eDirect);

	TArray<FbxVector4> FbxNormals;
	FbxNormals.AddUninitialized(VertexCount);
	for (int32 VertIndex = 0; VertIndex < VertexCount; ++VertIndex)
	{
		FVector Position = (FVector)RenderMesh.VertexBuffers.PositionVertexBuffer.VertexPosition(VertIndex);
		const FTransform SliceTransform = SplineMeshComp->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComp->ForwardAxis));
		FVector Normal = FVector(RenderMesh.VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertIndex));
		Normal = SliceTransform.TransformVector(Normal);
		FbxVector4& FbxNormal = FbxNormals[VertIndex];
		FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z);
		FbxNormal.Normalize();
	}

	// Add one normal per each face index (3 per triangle)
	for (uint32 UnrealVertIndex : Indices)
	{
		LayerElementNormal->GetDirectArray().Add(FbxNormals[UnrealVertIndex]);
	}
	Layer->SetNormals(LayerElementNormal);
	FbxNormals.Empty();

	// Create and fill in the per-face-vertex texture coordinate data source(s).
	// Create UV for Diffuse channel.
	int32 TexCoordSourceCount = RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	for (int32 TexCoordSourceIndex = 0; TexCoordSourceIndex < TexCoordSourceCount; ++TexCoordSourceIndex)
	{
		FbxLayer* UVsLayer = Mesh->GetLayer(TexCoordSourceIndex);
		if (UVsLayer == NULL)
		{
			Mesh->CreateLayer();
			UVsLayer = Mesh->GetLayer(TexCoordSourceIndex);
		}
		FString UVChannelNameBuilder = TEXT("UVmap_") + FString::FromInt(TexCoordSourceIndex);
		const auto UVChannelNameUTF8 = TStringConversion<FTCHARToUTF8_Convert>(*UVChannelNameBuilder); // Do not inline it! The lifetime of this object needs to extend over the usage of the converted buffer.
		const char* UVChannelName = UVChannelNameUTF8.Get(); // actually UTF8 as required by Fbx, but can't use UE's UTF8CHAR type because that's a uint8 aka *unsigned* char
		if (TexCoordSourceIndex == StaticMesh->GetLightMapCoordinateIndex())
		{
			UVChannelName = "LightMapUV";
		}

		FbxLayerElementUV* UVDiffuseLayer = FbxLayerElementUV::Create(Mesh, UVChannelName);

		// Note: when eINDEX_TO_DIRECT is used, IndexArray must be 3xTriangle count, DirectArray can be smaller
		UVDiffuseLayer->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		UVDiffuseLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);

		TArray<int32> UvsRemap;
		TArray<int32> UniqueUVs;
		// Weld UVs
		DetermineUVsToWeld(UvsRemap, UniqueUVs, RenderMesh.VertexBuffers.StaticMeshVertexBuffer, TexCoordSourceIndex);

		// Create the texture coordinate data source.
		for (int32 UnrealVertIndex : UniqueUVs)
		{
			const FVector2f& TexCoord = RenderMesh.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(UnrealVertIndex, TexCoordSourceIndex);
			UVDiffuseLayer->GetDirectArray().Add(FbxVector2(TexCoord.X, -TexCoord.Y + 1.0));
		}

		// For each face index, point to a texture uv
		UVDiffuseLayer->GetIndexArray().SetCount(Indices.Num());
		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			uint32 UnrealVertIndex = Indices[FbxVertIndex];
			int32 NewVertIndex = UvsRemap[UnrealVertIndex];
			UVDiffuseLayer->GetIndexArray().SetAt(FbxVertIndex, NewVertIndex);
		}

		UVsLayer->SetUVs(UVDiffuseLayer, FbxLayerElement::eTextureDiffuse);
	}

	FbxLayerElementMaterial* MatLayer = FbxLayerElementMaterial::Create(Mesh, "");
	MatLayer->SetMappingMode(FbxLayerElement::eByPolygon);
	MatLayer->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	Layer->SetMaterials(MatLayer);

	for (int32 PolygonsIndex = 0; PolygonsIndex < PolygonsCount; ++PolygonsIndex)
	{
		const FStaticMeshSection& Polygons = RenderMesh.Sections[PolygonsIndex];
		FIndexArrayView RawIndices = RenderMesh.IndexBuffer.GetArrayView();
		UMaterialInterface* Material = StaticMesh->GetMaterial(Polygons.MaterialIndex);

		FbxSurfaceMaterial* FbxMaterial = Material ? ExportMaterial(Material, Polygons.MaterialIndex, MaterialBakingMeshData) : NULL;
		if (!FbxMaterial)
		{
			FbxMaterial = CreateDefaultMaterial();
		}
		int32 MatIndex = FbxActor->AddMaterial(FbxMaterial);

		// Static meshes contain one triangle list per element.
		uint32 TriangleCount = Polygons.NumTriangles;

		// Copy over the index buffer into the FBX polygons set.
		for (uint32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			Mesh->BeginPolygon(MatIndex);
			for (uint32 PointIndex = 0; PointIndex < 3; PointIndex++)
			{
				uint32 OriginalUnrealVertIndex = RawIndices[Polygons.FirstIndex + ((TriangleIndex * 3) + PointIndex)];
				int32 RemappedVertIndex = VertRemap[OriginalUnrealVertIndex];
				Mesh->AddPolygon(RemappedVertIndex);
			}
			Mesh->EndPolygon();
		}
	}

#ifdef TODO_FBX
	// This is broken. We are exporting the render mesh but providing smoothing
	// information from the source mesh. The render triangles are not in the
	// same order. Therefore we should export the raw mesh or not export
	// smoothing group information!
	int32 TriangleCount = RenderMesh.RawTriangles.GetElementCount();
	FStaticMeshTriangle* RawTriangleData = (FStaticMeshTriangle*)RenderMesh.RawTriangles.Lock(LOCK_READ_ONLY);
	for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; TriangleIndex++)
	{
		FStaticMeshTriangle* Triangle = (RawTriangleData++);

		SmoothingArray.Add(Triangle->SmoothingMask);
	}
	RenderMesh.RawTriangles.Unlock();
#endif // #if TODO_FBX

	// Create and fill in the vertex color data source.
	const FColorVertexBuffer* ColorBufferToUse = &RenderMesh.VertexBuffers.ColorVertexBuffer;
	uint32 ColorVertexCount = ColorBufferToUse->GetNumVertices();

	// Only export vertex colors if they exist
	if (GetExportOptions()->VertexColor && ColorVertexCount > 0)
	{
		FbxLayerElementVertexColor* VertexColor = FbxLayerElementVertexColor::Create(Mesh, "");
		VertexColor->SetMappingMode(FbxLayerElement::eByPolygonVertex);
		VertexColor->SetReferenceMode(FbxLayerElement::eIndexToDirect);
		FbxLayerElementArrayTemplate<FbxColor>& VertexColorArray = VertexColor->GetDirectArray();
		Layer->SetVertexColors(VertexColor);

		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			FLinearColor VertColor(1.0f, 1.0f, 1.0f);
			uint32 UnrealVertIndex = Indices[FbxVertIndex];
			if (UnrealVertIndex < ColorVertexCount)
			{
				VertColor = ColorBufferToUse->VertexColor(UnrealVertIndex).ReinterpretAsLinear();
			}

			VertexColorArray.Add(FbxColor(VertColor.R, VertColor.G, VertColor.B, VertColor.A));
		}

		VertexColor->GetIndexArray().SetCount(Indices.Num());
		for (int32 FbxVertIndex = 0; FbxVertIndex < Indices.Num(); FbxVertIndex++)
		{
			VertexColor->GetIndexArray().SetAt(FbxVertIndex, FbxVertIndex);
		}
	}

	FbxActor->SetNodeAttribute(Mesh);
}

void FFbxExporter::ExportInstancedMeshToFbx(const UInstancedStaticMeshComponent* InstancedMeshComp, const TCHAR* MeshName, FbxNode* FbxActor, const FFbxMaterialBakingMeshData& MaterialBakingMeshData)
{
	const UStaticMesh* StaticMesh = InstancedMeshComp->GetStaticMesh();
	check(StaticMesh);

	const int32 LODIndex = (InstancedMeshComp->ForcedLodModel > 0 ? FMath::Min(StaticMesh->GetNumLODs(), InstancedMeshComp->ForcedLodModel) - 1 : /* auto-select*/ 0);
	const int32 NumInstances = InstancedMeshComp->GetInstanceCount();
	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
	{
		FTransform RelativeTransform;
		if (ensure(InstancedMeshComp->GetInstanceTransform(InstanceIndex, RelativeTransform, /*bWorldSpace=*/false)))
		{
			FbxNode* InstNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FString::Printf(TEXT("%d"), InstanceIndex)));

			InstNode->LclTranslation.Set(Converter.ConvertToFbxPos(RelativeTransform.GetTranslation()));
			InstNode->LclRotation.Set(Converter.ConvertToFbxRot(RelativeTransform.GetRotation().Euler()));
			InstNode->LclScaling.Set(Converter.ConvertToFbxScale(RelativeTransform.GetScale3D()));

			// Todo - export once and then clone the node
			const int32 LightmapUVChannel = -1;
			const TArray<FStaticMaterial>* MaterialOrderOverride = nullptr;
			const FColorVertexBuffer* ColorBuffer = nullptr;
			ExportStaticMeshToFbx(StaticMesh, LODIndex, *FString::Printf(TEXT("%d"), InstanceIndex), InstNode, MaterialBakingMeshData, LightmapUVChannel, ColorBuffer, MaterialOrderOverride, &ToRawPtrTArrayUnsafe(InstancedMeshComp->OverrideMaterials));
			FbxActor->AddChild(InstNode);
		}
	}
}

/**
 * Exports a Landscape
 */
void FFbxExporter::ExportLandscapeToFbx(ALandscapeProxy* Landscape, const TCHAR* MeshName, FbxNode* FbxActor, bool bSelectedOnly)
{
	const ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
	
	TSet<ULandscapeComponent*> SelectedComponents;
	if (bSelectedOnly && LandscapeInfo)
	{
		SelectedComponents = LandscapeInfo->GetSelectedComponents();
	}

	bSelectedOnly = bSelectedOnly && SelectedComponents.Num() > 0;

	int32 MinX = MAX_int32, MinY = MAX_int32;
	int32 MaxX = MIN_int32, MaxY = MIN_int32;

	// Find range of entire landscape
	for (int32 ComponentIndex = 0; ComponentIndex < Landscape->LandscapeComponents.Num(); ComponentIndex++)
	{
		ULandscapeComponent* Component = Landscape->LandscapeComponents[ComponentIndex];

		if (bSelectedOnly && !SelectedComponents.Contains(Component))
		{
			continue;
		}

		Component->GetComponentExtent(MinX, MinY, MaxX, MaxY);
	}

	FbxMesh* Mesh = FbxMesh::Create(Scene, TCHAR_TO_UTF8(MeshName));

	// Create and fill in the vertex position data source.
	const int32 ComponentSizeQuads = ((Landscape->ComponentSizeQuads + 1) >> Landscape->ExportLOD) - 1;
	const float ScaleFactor = (float)Landscape->ComponentSizeQuads / (float)ComponentSizeQuads;
	const int32 NumComponents = bSelectedOnly ? SelectedComponents.Num() : Landscape->LandscapeComponents.Num();
	const int32 VertexCountPerComponent = FMath::Square(ComponentSizeQuads + 1);
	const int32 VertexCount = NumComponents * VertexCountPerComponent;
	const int32 TriangleCount = NumComponents * FMath::Square(ComponentSizeQuads) * 2;
	
	Mesh->InitControlPoints(VertexCount);

	// Normals and Tangents
	FbxLayerElementNormal* LayerElementNormals= FbxLayerElementNormal::Create(Mesh, "");
	LayerElementNormals->SetMappingMode(FbxLayerElement::eByControlPoint);
	LayerElementNormals->SetReferenceMode(FbxLayerElement::eDirect);

	FbxLayerElementTangent* LayerElementTangents= FbxLayerElementTangent::Create(Mesh, "");
	LayerElementTangents->SetMappingMode(FbxLayerElement::eByControlPoint);
	LayerElementTangents->SetReferenceMode(FbxLayerElement::eDirect);

	FbxLayerElementBinormal* LayerElementBinormals= FbxLayerElementBinormal::Create(Mesh, "");
	LayerElementBinormals->SetMappingMode(FbxLayerElement::eByControlPoint);
	LayerElementBinormals->SetReferenceMode(FbxLayerElement::eDirect);

	// Add Texture UVs (which are simply incremented 1.0 per vertex)
	FbxLayerElementUV* LayerElementTextureUVs = FbxLayerElementUV::Create(Mesh, "TextureUVs");
	LayerElementTextureUVs->SetMappingMode(FbxLayerElement::eByControlPoint);
	LayerElementTextureUVs->SetReferenceMode(FbxLayerElement::eDirect);

	// Add Weightmap UVs (to match up with an exported weightmap, not the original weightmap UVs, which are per-component)
	const FVector2D UVScale = FVector2D(1.0f, 1.0f) / FVector2D((MaxX - MinX) + 1, (MaxY - MinY) + 1);
	FbxLayerElementUV* LayerElementWeightmapUVs = FbxLayerElementUV::Create(Mesh, "WeightmapUVs");
	LayerElementWeightmapUVs->SetMappingMode(FbxLayerElement::eByControlPoint);
	LayerElementWeightmapUVs->SetReferenceMode(FbxLayerElement::eDirect);

	FbxVector4* ControlPoints = Mesh->GetControlPoints();
	FbxLayerElementArrayTemplate<FbxVector4>& Normals = LayerElementNormals->GetDirectArray();
	Normals.Resize(VertexCount);
	FbxLayerElementArrayTemplate<FbxVector4>& Tangents = LayerElementTangents->GetDirectArray();
	Tangents.Resize(VertexCount);
	FbxLayerElementArrayTemplate<FbxVector4>& Binormals = LayerElementBinormals->GetDirectArray();
	Binormals.Resize(VertexCount);
	FbxLayerElementArrayTemplate<FbxVector2>& TextureUVs = LayerElementTextureUVs->GetDirectArray();
	TextureUVs.Resize(VertexCount);
	FbxLayerElementArrayTemplate<FbxVector2>& WeightmapUVs = LayerElementWeightmapUVs->GetDirectArray();
	WeightmapUVs.Resize(VertexCount);

	TArray<uint8> VisibilityData;
	VisibilityData.Empty(VertexCount);
	VisibilityData.AddZeroed(VertexCount);

	for (int32 ComponentIndex = 0, SelectedComponentIndex = 0; ComponentIndex < Landscape->LandscapeComponents.Num(); ComponentIndex++)
	{
		ULandscapeComponent* Component = Landscape->LandscapeComponents[ComponentIndex];

		if (bSelectedOnly && !SelectedComponents.Contains(Component))
		{
			continue;
		}

		FLandscapeComponentDataInterface CDI(Component, Landscape->ExportLOD);
		const int32 BaseVertIndex = SelectedComponentIndex++ * VertexCountPerComponent;

		const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations();
		TArray<uint8> CompVisData;
		for (int32 AllocIdx = 0; AllocIdx < ComponentWeightmapLayerAllocations.Num(); AllocIdx++)
		{
			const FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[AllocIdx];
			if (AllocInfo.LayerInfo == ALandscapeProxy::VisibilityLayer)
			{
				CDI.GetWeightmapTextureData(AllocInfo.LayerInfo, CompVisData);
			}
		}

		if (CompVisData.Num() > 0)
		{
			for (int32 i = 0; i < VertexCountPerComponent; ++i)
			{
				VisibilityData[BaseVertIndex + i] = CompVisData[CDI.VertexIndexToTexel(i)];
			}
		}
		
		for (int32 VertIndex = 0; VertIndex < VertexCountPerComponent; VertIndex++)
		{
			int32 VertX, VertY;
			CDI.VertexIndexToXY(VertIndex, VertX, VertY);

			FVector Position = CDI.GetLocalVertex(VertX, VertY) + Component->GetRelativeLocation();
			FbxVector4 FbxPosition = FbxVector4(Position.X, -Position.Y, Position.Z);
			ControlPoints[BaseVertIndex + VertIndex] = FbxPosition;

			FVector Normal, TangentX, TangentY;
			CDI.GetLocalTangentVectors(VertX, VertY, TangentX, TangentY, Normal);
			Normal /= Component->GetComponentTransform().GetScale3D(); Normal.Normalize();
			TangentX /= Component->GetComponentTransform().GetScale3D(); TangentX.Normalize();
			TangentY /= Component->GetComponentTransform().GetScale3D(); TangentY.Normalize();
			FbxVector4 FbxNormal = FbxVector4(Normal.X, -Normal.Y, Normal.Z); FbxNormal.Normalize();
			Normals.SetAt(BaseVertIndex + VertIndex, FbxNormal);
			FbxVector4 FbxTangent = FbxVector4(TangentX.X, -TangentX.Y, TangentX.Z); FbxTangent.Normalize();
			Tangents.SetAt(BaseVertIndex + VertIndex, FbxTangent);
			FbxVector4 FbxBinormal = FbxVector4(TangentY.X, -TangentY.Y, TangentY.Z); FbxBinormal.Normalize();
			Binormals.SetAt(BaseVertIndex + VertIndex, FbxBinormal);

			FVector2D TextureUV = FVector2D(VertX * ScaleFactor + Component->GetSectionBase().X, VertY * ScaleFactor + Component->GetSectionBase().Y);
			FbxVector2 FbxTextureUV = FbxVector2(TextureUV.X, TextureUV.Y);
			TextureUVs.SetAt(BaseVertIndex + VertIndex, FbxTextureUV);

			FVector2D WeightmapUV = (TextureUV - FVector2D(MinX, MinY)) * UVScale;
			FbxVector2 FbxWeightmapUV = FbxVector2(WeightmapUV.X, WeightmapUV.Y);
			WeightmapUVs.SetAt(BaseVertIndex + VertIndex, FbxWeightmapUV);
		}
	}

	FbxLayer* Layer0 = Mesh->GetLayer(0);
	if (Layer0 == NULL)
	{
		Mesh->CreateLayer();
		Layer0 = Mesh->GetLayer(0);
	}

	Layer0->SetNormals(LayerElementNormals);
	Layer0->SetTangents(LayerElementTangents);
	Layer0->SetBinormals(LayerElementBinormals);
	Layer0->SetUVs(LayerElementTextureUVs);
	Layer0->SetUVs(LayerElementWeightmapUVs, FbxLayerElement::eTextureBump);

	// this doesn't seem to work, on import the mesh has no smoothing layer at all
	//FbxLayerElementSmoothing* SmoothingInfo = FbxLayerElementSmoothing::Create(Mesh, "");
	//SmoothingInfo->SetMappingMode(FbxLayerElement::eAllSame);
	//SmoothingInfo->SetReferenceMode(FbxLayerElement::eDirect);
	//FbxLayerElementArrayTemplate<int>& Smoothing = SmoothingInfo->GetDirectArray();
	//Smoothing.Add(0);
	//Layer0->SetSmoothing(SmoothingInfo);

	FbxLayerElementMaterial* LayerElementMaterials = FbxLayerElementMaterial::Create(Mesh, "");
	LayerElementMaterials->SetMappingMode(FbxLayerElement::eAllSame);
	LayerElementMaterials->SetReferenceMode(FbxLayerElement::eIndexToDirect);
	Layer0->SetMaterials(LayerElementMaterials);

	UMaterialInterface* Material = Landscape->GetLandscapeMaterial();
	FbxSurfaceMaterial* FbxMaterial = Material ? ExportMaterial(Material, 0, FFbxMaterialBakingMeshData()) : NULL;
	if (!FbxMaterial)
	{
		FbxMaterial = CreateDefaultMaterial();
	}
	const int32 FbxMaterialIndex = FbxActor->AddMaterial(FbxMaterial);
	LayerElementMaterials->GetIndexArray().Add(FbxMaterialIndex);

	const int32 VisThreshold = 170;
	// Copy over the index buffer into the FBX polygons set.
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
	{
		int32 BaseVertIndex = ComponentIndex * VertexCountPerComponent;

		for (int32 Y = 0; Y < ComponentSizeQuads; Y++)
		{
			for (int32 X = 0; X < ComponentSizeQuads; X++)
			{
				if (VisibilityData[BaseVertIndex + Y * (ComponentSizeQuads + 1) + X] < VisThreshold)
				{
					Mesh->BeginPolygon();
					Mesh->AddPolygon(BaseVertIndex + (X + 0) + (Y + 0)*(ComponentSizeQuads + 1));
					Mesh->AddPolygon(BaseVertIndex + (X + 1) + (Y + 1)*(ComponentSizeQuads + 1));
					Mesh->AddPolygon(BaseVertIndex + (X + 1) + (Y + 0)*(ComponentSizeQuads + 1));
					Mesh->EndPolygon();

					Mesh->BeginPolygon();
					Mesh->AddPolygon(BaseVertIndex + (X + 0) + (Y + 0)*(ComponentSizeQuads + 1));
					Mesh->AddPolygon(BaseVertIndex + (X + 0) + (Y + 1)*(ComponentSizeQuads + 1));
					Mesh->AddPolygon(BaseVertIndex + (X + 1) + (Y + 1)*(ComponentSizeQuads + 1));
					Mesh->EndPolygon();
				}
			}
		}
	}

	FbxActor->SetNodeAttribute(Mesh);
}


} // namespace UnFbx
