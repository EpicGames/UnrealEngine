// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Main implementation of FFbxImporter : import FBX data to Unreal
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/SecureHash.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Materials/MaterialInterface.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/FbxErrors.h"
#include "FbxImporter.h"
#include "FbxOptionWindow.h"
#include "Interfaces/IMainFrameModule.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "UObject/MetaData.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/ARFilter.h"
#include "Animation/Skeleton.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "IMeshReductionInterfaces.h"
#include "ObjectTools.h"
#include "Misc/AutomationTest.h"
#include "Misc/AxisDisplayInfo.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Misc/NamePermissionList.h"
#include "PhysicsEngine/PhysicsAsset.h"

DEFINE_LOG_CATEGORY(LogFbx);

#define LOCTEXT_NAMESPACE "FbxMainImport"

#define GeneratedLODNameSuffix "_GeneratedLOD_"
namespace UnFbx
{

TSharedPtr<FFbxImporter> FFbxImporter::StaticInstance;

TSharedPtr<FFbxImporter> FFbxImporter::StaticPreviewInstance;

template<typename TMaterialType>
void PrepareAndShowMaterialConflictPreviewDialog(UFbxImportUI* ImportUI)
{
	TArray<TMaterialType> CurrentMaterial;
	TArray<TMaterialType> ResultMaterial;
	TArray<int32> RemapMaterial;
	TArray<FName> RemapMaterialName;
	RemapMaterial.AddZeroed(ImportUI->MaterialCompareData.ResultAsset.Num());
	RemapMaterialName.AddZeroed(ImportUI->MaterialCompareData.ResultAsset.Num());
	CurrentMaterial.AddDefaulted(ImportUI->MaterialCompareData.CurrentAsset.Num());
	for (int32 Materialindex = 0; Materialindex < ImportUI->MaterialCompareData.CurrentAsset.Num(); ++Materialindex)
	{
		CurrentMaterial[Materialindex].MaterialSlotName = ImportUI->MaterialCompareData.CurrentAsset[Materialindex].MaterialSlotName;
		CurrentMaterial[Materialindex].ImportedMaterialSlotName = ImportUI->MaterialCompareData.CurrentAsset[Materialindex].ImportedMaterialSlotName;
	}
	ResultMaterial.AddDefaulted(ImportUI->MaterialCompareData.ResultAsset.Num());
	for (int32 Materialindex = 0; Materialindex < ImportUI->MaterialCompareData.ResultAsset.Num(); ++Materialindex)
	{
		ResultMaterial[Materialindex].MaterialSlotName = ImportUI->MaterialCompareData.ResultAsset[Materialindex].MaterialSlotName;
		ResultMaterial[Materialindex].ImportedMaterialSlotName = ImportUI->MaterialCompareData.ResultAsset[Materialindex].ImportedMaterialSlotName;
	}
	UnFbx::EFBXReimportDialogReturnOption OutReturnOption;
	UnFbx::FFbxImporter::PrepareAndShowMaterialConflictDialog<TMaterialType>(CurrentMaterial, ResultMaterial, RemapMaterial, RemapMaterialName, true, true, false, OutReturnOption);
}

void PrepareAndShowSkeletonConflictPreviewDialog(UFbxImportUI* ImportUI)
{
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ImportUI->ReimportMesh);
	UnFbx::FFbxImporter::ShowFbxSkeletonConflictWindow(SkeletalMesh, ImportUI->Skeleton, ImportUI->SkeletonCompareData);
}

FBXImportOptions* GetImportOptions( UnFbx::FFbxImporter* FbxImporter, UFbxImportUI* ImportUI, bool bShowOptionDialog, bool bIsAutomated, const FString& FullPath, bool& OutOperationCanceled, bool& bOutImportAll, bool bIsObjFormat, const FString& InFilename, bool bForceImportType, EFBXImportType ImportType)
{
	OutOperationCanceled = false;

	if ( bShowOptionDialog )
	{
		bOutImportAll = false;
		UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();

		// if Skeleton was set by outside, please make sure copy back to UI
		if ( ImportOptions->SkeletonForAnimation )
		{
			ImportUI->Skeleton = ImportOptions->SkeletonForAnimation;
		}
		else
		{
			// Look in the current target directory to see if we have a skeleton
			FARFilter Filter;
			Filter.PackagePaths.Add(*FPaths::GetPath(FullPath));
			Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());

			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			TArray<FAssetData> SkeletonAssets;
			AssetRegistry.GetAssets(Filter, SkeletonAssets);
			if(SkeletonAssets.Num() > 0)
			{
				ImportUI->Skeleton = CastChecked<USkeleton>(SkeletonAssets[0].GetAsset());
			}
			else
			{
				ImportUI->Skeleton = NULL;
			}
		}

		if ( ImportOptions->PhysicsAsset )
		{
			ImportUI->PhysicsAsset = ImportOptions->PhysicsAsset;
		}
		else
		{
			ImportUI->PhysicsAsset = NULL;
		}

		if(bForceImportType)
		{
			ImportUI->MeshTypeToImport = ImportType;
			ImportUI->OriginalImportType = ImportType;
		}

		ImportUI->bImportAsSkeletal = ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh;
		ImportUI->bImportMesh = ImportUI->MeshTypeToImport != FBXIT_Animation;
		ImportUI->bIsObjImport = bIsObjFormat;

		//This option must always be the same value has the skeletalmesh one.
		ImportUI->AnimSequenceImportData->bImportMeshesInBoneHierarchy = ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy;

		//////////////////////////////////////////////////////////////////////////
		// Set the information section data
		
		//Make sure the file is open to be able to read the header before showing the options
		//If the file is already open it will simply return false.
		if (FbxImporter->ReadHeaderFromFile(InFilename, true))
		{

			ImportUI->FileVersion = FbxImporter->GetFbxFileVersion();
			ImportUI->FileCreator = FbxImporter->GetFileCreator();
			// do analytics on getting Fbx data
			FbxDocumentInfo* DocInfo = FbxImporter->Scene->GetSceneInfo();
			if (DocInfo)
			{
				FString LastSavedVendor(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVendor.Get().Buffer()));
				FString LastSavedAppName(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationName.Get().Buffer()));
				FString LastSavedAppVersion(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVersion.Get().Buffer()));

				ImportUI->FileCreatorApplication = LastSavedVendor + TEXT(" ") + LastSavedAppName + TEXT(" ") + LastSavedAppVersion;
			}
			else
			{
				ImportUI->FileCreatorApplication = TEXT("");
			}

			ImportUI->FileUnits = FbxImporter->GetFileUnitSystem();

			ImportUI->FileAxisDirection = FbxImporter->GetFileAxisDirection();
		}

		if (ImportUI->MeshTypeToImport != FBXIT_Animation && ImportUI->ReimportMesh != nullptr)
		{
			ImportUI->OnUpdateCompareFbx = FOnUpdateCompareFbx::CreateLambda([&ImportUI, &FbxImporter]
			{
				//Fill the importUI compare
				ImportUI->UpdateCompareData(FbxImporter);
			});

			ImportUI->OnShowMaterialConflictDialog = FOnShowConflictDialog::CreateLambda([&ImportUI, &FbxImporter]
			{
				if (!ImportUI->MaterialCompareData.bHasConflict)
				{
					return;
				}
				if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
				{

					PrepareAndShowMaterialConflictPreviewDialog<FSkeletalMaterial>(ImportUI);
				}
				else if (ImportUI->MeshTypeToImport == FBXIT_StaticMesh)
				{
					PrepareAndShowMaterialConflictPreviewDialog<FStaticMaterial>(ImportUI);
				}
			});

			ImportUI->OnShowSkeletonConflictDialog = FOnShowConflictDialog::CreateLambda([&ImportUI, &FbxImporter]()
			{
				if (ImportUI->SkeletonCompareData.CompareResult == ImportCompareHelper::ECompareResult::SCR_None)
				{
					return;
				}
				if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
				{
					PrepareAndShowSkeletonConflictPreviewDialog(ImportUI);
				}
			});
			
		}

		auto UpdateCoordinateSystemPolicy = [](UFbxAssetImportData* FbxAssetImportData)
			{
				if (FbxAssetImportData)
				{
					FbxAssetImportData->bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
					FbxAssetImportData->CoordinateSystemPolicy = FbxAssetImportData->bConvertScene ? (FbxAssetImportData->bForceFrontXAxis ? ECoordinateSystemPolicy::MatchUpForwardAxes : ECoordinateSystemPolicy::MatchUpAxis) : ECoordinateSystemPolicy::KeepXYZAxes;
				}
			};

		UpdateCoordinateSystemPolicy(ImportUI->StaticMeshImportData);
		UpdateCoordinateSystemPolicy(ImportUI->SkeletalMeshImportData);
		UpdateCoordinateSystemPolicy(ImportUI->AnimSequenceImportData);

		TSharedPtr<SWindow> ParentWindow;

		if( FModuleManager::Get().IsModuleLoaded( "MainFrame" ) )
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>( "MainFrame" );
			ParentWindow = MainFrame.GetParentWindow();
		}

		// Compute centered window position based on max window size, which include when all categories are expanded
		const float FbxImportWindowWidth = 450.0f;
		const float FbxImportWindowHeight = 750.0f;
		FVector2D FbxImportWindowSize = FVector2D(FbxImportWindowWidth, FbxImportWindowHeight); // Max window size it can get based on current slate


		FSlateRect WorkAreaRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
		FVector2D DisplayTopLeft(WorkAreaRect.Left, WorkAreaRect.Top);
		FVector2D DisplaySize(WorkAreaRect.Right - WorkAreaRect.Left, WorkAreaRect.Bottom - WorkAreaRect.Top);

		float ScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayTopLeft.X, DisplayTopLeft.Y);
		FbxImportWindowSize *= ScaleFactor;

		FVector2D WindowPosition = (DisplayTopLeft + (DisplaySize - FbxImportWindowSize) / 2.0f) / ScaleFactor;
	

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(NSLOCTEXT("UnrealEd", "FBXImportOpionsTitle", "FBX Import Options"))
			.SizingRule(ESizingRule::Autosized)
			.AutoCenter(EAutoCenter::None)
			.ClientSize(FbxImportWindowSize)
			.ScreenPosition(WindowPosition);
		
		TSharedPtr<SFbxOptionWindow> FbxOptionWindow;
		Window->SetContent
		(
			SAssignNew(FbxOptionWindow, SFbxOptionWindow)
			.ImportUI(ImportUI)
			.WidgetWindow(Window)
			.FullPath(FText::FromString(FullPath))
			.ForcedImportType( bForceImportType ? TOptional<EFBXImportType>( ImportType ) : TOptional<EFBXImportType>() )
			.IsObjFormat( bIsObjFormat )
			.MaxWindowHeight(FbxImportWindowHeight)
			.MaxWindowWidth(FbxImportWindowWidth)
		);

		// @todo: we can make this slow as showing progress bar later
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

		if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh || ImportUI->MeshTypeToImport == FBXIT_Animation)
		{
			//Set some hardcoded options for skeletal mesh
			ImportUI->SkeletalMeshImportData->bBakePivotInVertex = false;
			ImportOptions->bBakePivotInVertex = false;
			ImportUI->SkeletalMeshImportData->bTransformVertexToAbsolute = true;
			ImportOptions->bTransformVertexToAbsolute = true;
			//when user import animation only we must get duplicate "bImportMeshesInBoneHierarchy" option from ImportUI anim sequence data
			if (!ImportUI->bImportMesh && ImportUI->bImportAnimations)
			{
				ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy = ImportUI->AnimSequenceImportData->bImportMeshesInBoneHierarchy;
			}
			else
			{
				ImportUI->AnimSequenceImportData->bImportMeshesInBoneHierarchy = ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy;
			}
		}

		if (!FbxImporter->CanImportClass(UPhysicsAsset::StaticClass()))
		{
			ImportUI->bCreatePhysicsAsset = false;
		}

		UFbxImportUI::SaveOptions(ImportUI);

		auto SaveImportData = [](UFbxAssetImportData* FbxAssetImportData)
			{
				if (FbxAssetImportData)
				{
					if (FbxAssetImportData->bUsingLUFCoordinateSysem)
					{
						FbxAssetImportData->bConvertScene = FbxAssetImportData->CoordinateSystemPolicy != ECoordinateSystemPolicy::KeepXYZAxes;
						FbxAssetImportData->bForceFrontXAxis = FbxAssetImportData->CoordinateSystemPolicy == ECoordinateSystemPolicy::MatchUpForwardAxes;
					}
					UFbxImportUI::SaveOptions(FbxAssetImportData);
				}
			};

		SaveImportData(ImportUI->StaticMeshImportData);
		SaveImportData(ImportUI->SkeletalMeshImportData);
		SaveImportData(ImportUI->AnimSequenceImportData);

		if( ImportUI->TextureImportData )
		{
			UFbxImportUI::SaveOptions(ImportUI->TextureImportData);
		}

		if (FbxOptionWindow->ShouldImport())
		{
			bOutImportAll = FbxOptionWindow->ShouldImportAll(); 

			// open dialog
			// see if it's canceled
			ApplyImportUIToImportOptions(ImportUI, *ImportOptions);

			return ImportOptions;
		}
		else
		{
			OutOperationCanceled = true;
		}
	}
	else if (bIsAutomated)
	{
		//Automation tests set ImportUI settings directly.  Just copy them over
		UnFbx::FBXImportOptions* ImportOptions = FbxImporter->GetImportOptions();
		//Clean up the options
		UnFbx::FBXImportOptions::ResetOptions(ImportOptions);
		ApplyImportUIToImportOptions(ImportUI, *ImportOptions);
		
		if (!FbxImporter->CanImportClass(UPhysicsAsset::StaticClass()))
		{
			ImportOptions->bCreatePhysicsAsset = false;
		}
		
		return ImportOptions;
	}
	else
	{
		
		if (!FbxImporter->CanImportClass(UPhysicsAsset::StaticClass()))
		{
			FbxImporter->GetImportOptions()->bCreatePhysicsAsset = false;
		}

		return FbxImporter->GetImportOptions();
	}

	return NULL;

}

void ApplyImportUIToImportOptions(UFbxImportUI* ImportUI, FBXImportOptions& InOutImportOptions)
{
	check(ImportUI);

	//General options
	{
		InOutImportOptions.bUsedAsFullName			= ImportUI->bOverrideFullName;
		InOutImportOptions.ImportType				= ImportUI->MeshTypeToImport;

		InOutImportOptions.bResetToFbxOnMaterialConflict = ImportUI->bResetToFbxOnMaterialConflict;
		InOutImportOptions.bAutoComputeLodDistances	= ImportUI->bAutoComputeLodDistances;
		InOutImportOptions.LodDistances.Empty(8);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance0);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance1);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance2);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance3);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance4);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance5);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance6);
		InOutImportOptions.LodDistances.Add(ImportUI->LodDistance7);
		InOutImportOptions.LodNumber				= ImportUI->LodNumber;
		InOutImportOptions.MinimumLodNumber			= ImportUI->MinimumLodNumber;
	}

	InOutImportOptions.bBuildNanite = ImportUI->StaticMeshImportData->bBuildNanite;

	//Animation and skeletal mesh options
	{
		InOutImportOptions.bImportAnimations		= ImportUI->bImportAnimations;
		InOutImportOptions.SkeletonForAnimation		= ImportUI->Skeleton;
		InOutImportOptions.bCreatePhysicsAsset		= ImportUI->bCreatePhysicsAsset;
		InOutImportOptions.PhysicsAsset				= ImportUI->PhysicsAsset;
		InOutImportOptions.AnimationName			= ImportUI->OverrideAnimationName;
	}

	//Material options
	{
		InOutImportOptions.bImportMaterials			= ImportUI->bImportMaterials;
		InOutImportOptions.bInvertNormalMap			= ImportUI->TextureImportData->bInvertNormalMaps;
		InOutImportOptions.MaterialSearchLocation	= ImportUI->TextureImportData->MaterialSearchLocation;
		UMaterialInterface* BaseMaterialInterface	= Cast<UMaterialInterface>(ImportUI->TextureImportData->BaseMaterialName.TryLoad());
		if (BaseMaterialInterface) {
			InOutImportOptions.BaseMaterial			= BaseMaterialInterface;
			InOutImportOptions.BaseColorName		= ImportUI->TextureImportData->BaseColorName;
			InOutImportOptions.BaseDiffuseTextureName = ImportUI->TextureImportData->BaseDiffuseTextureName;
			InOutImportOptions.BaseNormalTextureName = ImportUI->TextureImportData->BaseNormalTextureName;
			InOutImportOptions.BaseEmmisiveTextureName = ImportUI->TextureImportData->BaseEmmisiveTextureName;
			InOutImportOptions.BaseSpecularTextureName = ImportUI->TextureImportData->BaseSpecularTextureName;
			InOutImportOptions.BaseOpacityTextureName = ImportUI->TextureImportData->BaseOpacityTextureName;
			InOutImportOptions.BaseEmissiveColorName = ImportUI->TextureImportData->BaseEmissiveColorName;
		}
		InOutImportOptions.bImportTextures			= ImportUI->bImportTextures;
	}

	//Some options are overlap between static mesh, skeletal mesh and anim sequence. We must set them according to the import type to set them only once
	if ( ImportUI->MeshTypeToImport == FBXIT_StaticMesh )
	{
		UFbxStaticMeshImportData* StaticMeshData	= ImportUI->StaticMeshImportData;
		InOutImportOptions.NormalImportMethod		= StaticMeshData->NormalImportMethod;
		InOutImportOptions.NormalGenerationMethod	= StaticMeshData->NormalGenerationMethod;
		InOutImportOptions.bComputeWeightedNormals	= StaticMeshData->bComputeWeightedNormals;
		InOutImportOptions.ImportTranslation		= StaticMeshData->ImportTranslation;
		InOutImportOptions.ImportRotation			= StaticMeshData->ImportRotation;
		InOutImportOptions.ImportUniformScale		= StaticMeshData->ImportUniformScale;
		InOutImportOptions.bTransformVertexToAbsolute = StaticMeshData->bTransformVertexToAbsolute;
		InOutImportOptions.bBakePivotInVertex		= StaticMeshData->bBakePivotInVertex;
		InOutImportOptions.bImportStaticMeshLODs	= StaticMeshData->bImportMeshLODs;
		InOutImportOptions.bConvertScene			= StaticMeshData->bConvertScene;
		InOutImportOptions.bForceFrontXAxis			= StaticMeshData->bForceFrontXAxis;
		InOutImportOptions.bConvertSceneUnit		= StaticMeshData->bConvertSceneUnit;
		InOutImportOptions.VertexColorImportOption	= StaticMeshData->VertexColorImportOption;
		InOutImportOptions.VertexOverrideColor		= StaticMeshData->VertexOverrideColor;
		InOutImportOptions.bReorderMaterialToFbxOrder = StaticMeshData->bReorderMaterialToFbxOrder;
		InOutImportOptions.DistanceFieldResolutionScale = StaticMeshData->DistanceFieldResolutionScale;
	}
	else if ( ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh )
	{
		UFbxSkeletalMeshImportData* SkeletalMeshData	= ImportUI->SkeletalMeshImportData;
		InOutImportOptions.NormalImportMethod			= SkeletalMeshData->NormalImportMethod;
		InOutImportOptions.NormalGenerationMethod		= SkeletalMeshData->NormalGenerationMethod;
		InOutImportOptions.bComputeWeightedNormals		= SkeletalMeshData->bComputeWeightedNormals;
		InOutImportOptions.ImportTranslation			= SkeletalMeshData->ImportTranslation;
		InOutImportOptions.ImportRotation				= SkeletalMeshData->ImportRotation;
		InOutImportOptions.ImportUniformScale			= SkeletalMeshData->ImportUniformScale;
		InOutImportOptions.bTransformVertexToAbsolute	= SkeletalMeshData->bTransformVertexToAbsolute;
		InOutImportOptions.bBakePivotInVertex			= SkeletalMeshData->bBakePivotInVertex;
		InOutImportOptions.bImportSkeletalMeshLODs		= SkeletalMeshData->bImportMeshLODs;
		InOutImportOptions.bConvertScene				= SkeletalMeshData->bConvertScene;
		InOutImportOptions.bForceFrontXAxis				= SkeletalMeshData->bForceFrontXAxis;
		InOutImportOptions.bConvertSceneUnit			= SkeletalMeshData->bConvertSceneUnit;
		InOutImportOptions.VertexColorImportOption		= SkeletalMeshData->VertexColorImportOption;
		InOutImportOptions.VertexOverrideColor			= SkeletalMeshData->VertexOverrideColor;
		InOutImportOptions.bReorderMaterialToFbxOrder	= SkeletalMeshData->bReorderMaterialToFbxOrder;
		InOutImportOptions.bImportVertexAttributes		= SkeletalMeshData->bImportVertexAttributes;

		if(ImportUI->bImportAnimations)
		{
			// Copy the transform information into the animation data to match the mesh.
			UFbxAnimSequenceImportData* AnimData	= ImportUI->AnimSequenceImportData;
			AnimData->ImportTranslation				= SkeletalMeshData->ImportTranslation;
			AnimData->ImportRotation				= SkeletalMeshData->ImportRotation;
			AnimData->ImportUniformScale			= SkeletalMeshData->ImportUniformScale;
			AnimData->bConvertScene					= SkeletalMeshData->bConvertScene;
			AnimData->bForceFrontXAxis				= SkeletalMeshData->bForceFrontXAxis;
			AnimData->bConvertSceneUnit				= SkeletalMeshData->bConvertSceneUnit;
		}
	}
	else
	{
		UFbxAnimSequenceImportData* AnimData	= ImportUI->AnimSequenceImportData;
		InOutImportOptions.NormalImportMethod	= FBXNIM_ComputeNormals;
		InOutImportOptions.bComputeWeightedNormals = true;
		InOutImportOptions.ImportTranslation	= AnimData->ImportTranslation;
		InOutImportOptions.ImportRotation		= AnimData->ImportRotation;
		InOutImportOptions.ImportUniformScale	= AnimData->ImportUniformScale;
		InOutImportOptions.bConvertScene		= AnimData->bConvertScene;
		InOutImportOptions.bForceFrontXAxis		= AnimData->bForceFrontXAxis;
		InOutImportOptions.bConvertSceneUnit	= AnimData->bConvertSceneUnit;
	}

	//Skeletal mesh unshared options
	{
		InOutImportOptions.bImportAsSkeletalGeometry	= ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_Geometry;
		InOutImportOptions.bImportAsSkeletalSkinning	= ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_SkinningWeights;
		InOutImportOptions.bImportMorph					= ImportUI->SkeletalMeshImportData->bImportMorphTargets;
		InOutImportOptions.bUpdateSkeletonReferencePose = ImportUI->SkeletalMeshImportData->bUpdateSkeletonReferencePose;
		InOutImportOptions.bImportRigidMesh				= ImportUI->OriginalImportType == FBXIT_StaticMesh && ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh;
		InOutImportOptions.bUseT0AsRefPose				= ImportUI->SkeletalMeshImportData->bUseT0AsRefPose;
		InOutImportOptions.bPreserveSmoothingGroups		= ImportUI->SkeletalMeshImportData->bPreserveSmoothingGroups;
		InOutImportOptions.bKeepSectionsSeparate		= ImportUI->SkeletalMeshImportData->bKeepSectionsSeparate;
		InOutImportOptions.OverlappingThresholds.ThresholdPosition = ImportUI->SkeletalMeshImportData->ThresholdPosition;
		InOutImportOptions.OverlappingThresholds.ThresholdTangentNormal = ImportUI->SkeletalMeshImportData->ThresholdTangentNormal;
		InOutImportOptions.OverlappingThresholds.ThresholdUV = ImportUI->SkeletalMeshImportData->ThresholdUV;
		InOutImportOptions.OverlappingThresholds.MorphThresholdPosition = ImportUI->SkeletalMeshImportData->MorphThresholdPosition;
		InOutImportOptions.bImportMeshesInBoneHierarchy = ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy;
	}

	//Static mesh unshared options
	{
		InOutImportOptions.bCombineToSingle				= ImportUI->StaticMeshImportData->bCombineMeshes;
		InOutImportOptions.bRemoveDegenerates			= ImportUI->StaticMeshImportData->bRemoveDegenerates;
		InOutImportOptions.bBuildReversedIndexBuffer	= ImportUI->StaticMeshImportData->bBuildReversedIndexBuffer;
		InOutImportOptions.bGenerateLightmapUVs			= ImportUI->StaticMeshImportData->bGenerateLightmapUVs;
		InOutImportOptions.bOneConvexHullPerUCX			= ImportUI->StaticMeshImportData->bOneConvexHullPerUCX;
		InOutImportOptions.bAutoGenerateCollision		= ImportUI->StaticMeshImportData->bAutoGenerateCollision;
		InOutImportOptions.StaticMeshLODGroup			= ImportUI->StaticMeshImportData->StaticMeshLODGroup;
	}
	
	// animation unshared options
	{
		
		InOutImportOptions.AnimationLengthImportType	= ImportUI->AnimSequenceImportData->AnimationLength;
		InOutImportOptions.AnimationRange.X				= ImportUI->AnimSequenceImportData->FrameImportRange.Min;
		InOutImportOptions.AnimationRange.Y				= ImportUI->AnimSequenceImportData->FrameImportRange.Max;
		// only re-sample if they don't want to use default sample rate
		InOutImportOptions.bResample					= !ImportUI->AnimSequenceImportData->bUseDefaultSampleRate;
		InOutImportOptions.ResampleRate					= ImportUI->AnimSequenceImportData->CustomSampleRate;
		InOutImportOptions.bSnapToClosestFrameBoundary	= ImportUI->AnimSequenceImportData->bSnapToClosestFrameBoundary;
		InOutImportOptions.bPreserveLocalTransform		= ImportUI->AnimSequenceImportData->bPreserveLocalTransform;
		InOutImportOptions.bDeleteExistingMorphTargetCurves = ImportUI->AnimSequenceImportData->bDeleteExistingMorphTargetCurves;
		InOutImportOptions.bRemoveRedundantKeys			= ImportUI->AnimSequenceImportData->bRemoveRedundantKeys;
		InOutImportOptions.bDoNotImportCurveWithZero	= ImportUI->AnimSequenceImportData->bDoNotImportCurveWithZero;
		InOutImportOptions.bImportCustomAttribute		= ImportUI->AnimSequenceImportData->bImportCustomAttribute;
		InOutImportOptions.bDeleteExistingCustomAttributeCurves = ImportUI->AnimSequenceImportData->bDeleteExistingCustomAttributeCurves;
		InOutImportOptions.bDeleteExistingNonCurveCustomAttributes = ImportUI->AnimSequenceImportData->bDeleteExistingNonCurveCustomAttributes;
		InOutImportOptions.bImportBoneTracks			= ImportUI->AnimSequenceImportData->bImportBoneTracks;
		InOutImportOptions.bSetMaterialDriveParameterOnCustomAttribute = ImportUI->AnimSequenceImportData->bSetMaterialDriveParameterOnCustomAttribute;
		InOutImportOptions.bAddCurveMetadataToSkeleton	= ImportUI->AnimSequenceImportData->bAddCurveMetadataToSkeleton;
		InOutImportOptions.MaterialCurveSuffixes		= ImportUI->AnimSequenceImportData->MaterialCurveSuffixes;
	}
}

static bool AssetClassPassesFilter(UClass* Class, EAssetClassAction AssetClassAction)
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(AssetClassAction);
	if (Class && AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		return AssetClassPermissionList->PassesFilter(Class->GetPathName());
	}
	return true;
}

void FImportedMaterialData::AddImportedMaterial( const FbxSurfaceMaterial& FbxMaterial, UMaterialInterface& UnrealMaterial )
{
	FbxToUnrealMaterialMap.Add( &FbxMaterial, &UnrealMaterial );
	ImportedMaterialNames.Add( *UnrealMaterial.GetPathName() );
}

bool FImportedMaterialData::IsAlreadyImported( const FbxSurfaceMaterial& FbxMaterial, FName ImportedMaterialName ) const
{
	const UMaterialInterface* FoundMaterial = GetUnrealMaterial( FbxMaterial );

	return FoundMaterial != NULL || ImportedMaterialNames.Contains( ImportedMaterialName );
}

UMaterialInterface* FImportedMaterialData::GetUnrealMaterial( const FbxSurfaceMaterial& FbxMaterial ) const
{
	return FbxToUnrealMaterialMap.FindRef( &FbxMaterial ).Get();
}

void FImportedMaterialData::Clear()
{
	FbxToUnrealMaterialMap.Empty();
	ImportedMaterialNames.Empty();
}

FFbxImporter::FFbxImporter()
	: Scene(NULL)
	, ImportOptions(NULL)
	, GeometryConverter(NULL)
	, SdkManager(NULL)
	, Importer(NULL)
	, bFirstMesh(true)
	, FbxCreator(UnFbx::EFbxCreator::Unknow)
	, Logger(NULL)
{
	// Create the SdkManager
	SdkManager = FbxManager::Create();
	
	// create an IOSettings object
	FbxIOSettings * ios = FbxIOSettings::Create(SdkManager, IOSROOT );
	SdkManager->SetIOSettings(ios);

	// Create the geometry converter
	GeometryConverter = new FbxGeometryConverter(SdkManager);
	Scene = NULL;
	
	ImportOptions = new FBXImportOptions();
	FMemory::Memzero(*ImportOptions);
	ImportOptions->MaterialBasePath = NAME_None;
	
	CurPhase = NOTSTARTED;

	//The FFbxImporter is a singleton is constructor is protected
	//We must release the resource in the pre-exit delegate because in some cases the
	//Instance is not valid anymore when the destructor get called (i.e. when we build the editor in monolithic)
	FCoreDelegates::OnPreExit.AddLambda([]()
	{
		FFbxImporter::GetInstance()->CleanUp();
	});
}
	
//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
FFbxImporter::~FFbxImporter()
{
	//The clean up should have been done in the pre-exit core delegate implement in the FFbxImporter constructor
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
FFbxImporter* FFbxImporter::GetInstance(bool bDoNotCreate /*= false*/)
{
	if (!StaticInstance.IsValid())
	{
		//Return nullptr if we cannot create the instance
		if (bDoNotCreate)
		{
			return nullptr;
		}

		StaticInstance = MakeShareable( new FFbxImporter() );
	}
	return StaticInstance.Get();
}

void FFbxImporter::DeleteInstance()
{
	StaticInstance.Reset();
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
void FFbxImporter::CleanUp()
{
	ClearTokenizedErrorMessages();
	ReleaseScene();
	
	delete GeometryConverter;
	GeometryConverter = NULL;
	delete ImportOptions;
	ImportOptions = NULL;

	if (SdkManager)
	{
		SdkManager->Destroy();
	}
	SdkManager = NULL;
	Logger = NULL;
}

void FFbxImporter::PartialCleanUp()
{
	ClearTokenizedErrorMessages();
	ReleaseScene();
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
void FFbxImporter::ReleaseScene()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ReleaseScene);

	if (Importer)
	{
		Importer->Destroy();
		Importer = NULL;
	}
	
	if (Scene)
	{
		Scene->Destroy();
		Scene = NULL;
	}
	
	ImportedMaterialData.Clear();

	// reset
	FbxTextureToUniqueNameMap.Empty();
	NodeUniqueNameToOriginalNameMap.Clear();
	CollisionModels.Clear();
	CreatedObjects.Empty();
	CurPhase = NOTSTARTED;
	bFirstMesh = true;
	LastMergeBonesChoice = EAppReturnType::Ok;
}

bool FFbxImporter::CanImportClass(UClass* Class) const
{
	return AssetClassPassesFilter(Class, EAssetClassAction::ImportAsset);
}

bool FFbxImporter::CanCreateClass(UClass* Class) const
{
	return AssetClassPassesFilter(Class, EAssetClassAction::CreateAsset);
}

FBXImportOptions* UnFbx::FFbxImporter::GetImportOptions() const
{
	return ImportOptions;
}

int32 FFbxImporter::GetImportType(const FString& InFilename)
{
	int32 Result = -1; // Default to invalid
	FString Filename = InFilename;

	// Prioritized in the order of SkeletalMesh > StaticMesh > Animation (only if animation data is found)
	if (OpenFile(Filename))
	{
		bool bHasAnimation = false;
		bool bHasAnimationOnSkeletalMesh = false;
		FbxSceneInfo SceneInfo;
		if (GetSceneInfo(Filename, SceneInfo, true))
		{
			if (SceneInfo.SkinnedMeshNum > 0)
			{
				Result = 1;
			}
			else if (SceneInfo.TotalGeometryNum > 0)
			{
				Result = 0;
			}

			bHasAnimation = SceneInfo.bHasAnimation;
			bHasAnimationOnSkeletalMesh = SceneInfo.bHasAnimationOnSkeletalMesh;
		}

		// In case no Geometry was found, check for animation (FBX can still contain mesh data though)
		if (bHasAnimation)
		{
			if ( Result == -1)
			{
				Result = 2;
			}
			else if (Result == 0)
			{
				// by default detects as skeletalmesh since it has animation curves
				if (bHasAnimationOnSkeletalMesh)
				{
					Result = 1;
				}
				else
				{
					Result = 0;
				}
			}
		}
	}
	
	return Result; 
}

bool FFbxImporter::GetSceneInfo(FString Filename, FbxSceneInfo& SceneInfo, bool bPreventMaterialNameClash /*= false*/)
{
	bool Result = true;
	GWarn->BeginSlowTask( NSLOCTEXT("FbxImporter", "BeginGetSceneInfoTask", "Parse FBX file to get scene info"), true );
	
	bool bSceneInfo = true;
	switch (CurPhase)
	{
	case NOTSTARTED:
		if (!OpenFile(Filename))
		{
			Result = false;
			break;
		}
		GWarn->UpdateProgress( 40, 100 );
	case FILEOPENED:
		if (!ImportFile(Filename, bPreventMaterialNameClash))
		{
			Result = false;
			break;
		}
		GWarn->UpdateProgress( 90, 100 );
	case IMPORTED:
	case FIXEDANDCONVERTED:
	default:
		break;
	}
	
	if (Result)
	{
		FbxTimeSpan GlobalTimeSpan(FBXSDK_TIME_INFINITE,FBXSDK_TIME_MINUS_INFINITE);
		
		SceneInfo.TotalMaterialNum = Scene->GetMaterialCount();
		SceneInfo.TotalTextureNum = Scene->GetTextureCount();
		SceneInfo.TotalGeometryNum = 0;
		SceneInfo.NonSkinnedMeshNum = 0;
		SceneInfo.SkinnedMeshNum = 0;
		for ( int32 GeometryIndex = 0; GeometryIndex < Scene->GetGeometryCount(); GeometryIndex++ )
		{
			FbxGeometry * Geometry = Scene->GetGeometry(GeometryIndex);
			if (Geometry->GetAttributeType() == FbxNodeAttribute::eMesh)
			{
				FbxNode* GeoNode = Geometry->GetNode();
				FbxMesh* Mesh = (FbxMesh*)Geometry;
				//Skip staticmesh sub LOD group that will be merge with the other same lod index mesh
				if (GeoNode && Mesh->GetDeformerCount(FbxDeformer::eSkin) <= 0)
				{
					FbxNode* ParentNode = RecursiveFindParentLodGroup(GeoNode->GetParent());
					if (ParentNode != nullptr && ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
					{
						bool IsLodRoot = false;
						for (int32 ChildIndex = 0; ChildIndex < ParentNode->GetChildCount(); ++ChildIndex)
						{
							FbxNode *MeshNode = FindLODGroupNode(ParentNode, ChildIndex);
							if (GeoNode == MeshNode)
							{
								IsLodRoot = true;
								break;
							}
						}
						if (!IsLodRoot)
						{
							//Skip static mesh sub LOD
							continue;
						}
					}
				}
				SceneInfo.TotalGeometryNum++;
				
				SceneInfo.MeshInfo.AddZeroed(1);
				FbxMeshInfo& MeshInfo = SceneInfo.MeshInfo.Last();
				if(Geometry->GetName()[0] != '\0')
					MeshInfo.Name = MakeName(Geometry->GetName());
				else
					MeshInfo.Name = MakeString(GeoNode ? GeoNode->GetName() : "None");
				MeshInfo.bTriangulated = Mesh->IsTriangleMesh();
				MeshInfo.MaterialNum = GeoNode? GeoNode->GetMaterialCount() : 0;
				MeshInfo.FaceNum = Mesh->GetPolygonCount();
				MeshInfo.VertexNum = Mesh->GetControlPointsCount();
				
				// LOD info
				MeshInfo.LODGroup = NULL;
				if (GeoNode)
				{
					FbxNode* ParentNode = RecursiveFindParentLodGroup(GeoNode->GetParent());
					if (ParentNode != nullptr && ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
					{
						MeshInfo.LODGroup = MakeString(ParentNode->GetName());
						for (int32 LODIndex = 0; LODIndex < ParentNode->GetChildCount(); LODIndex++)
						{
							FbxNode *MeshNode = FindLODGroupNode(ParentNode, LODIndex, GeoNode);
							if (GeoNode == MeshNode)
							{
								MeshInfo.LODLevel = LODIndex;
								break;
							}
						}
					}
				}
				
				// skeletal mesh
				if (Mesh->GetDeformerCount(FbxDeformer::eSkin) > 0)
				{
					SceneInfo.SkinnedMeshNum++;
					MeshInfo.bIsSkelMesh = true;
					MeshInfo.MorphNum = Mesh->GetShapeCount();
					// skeleton root
					FbxSkin* Skin = (FbxSkin*)Mesh->GetDeformer(0, FbxDeformer::eSkin);
					int32 ClusterCount = Skin->GetClusterCount();
					FbxNode* Link = NULL;
					for (int32 ClusterId = 0; ClusterId < ClusterCount; ++ClusterId)
					{
						FbxCluster* Cluster = Skin->GetCluster(ClusterId);
						Link = Cluster->GetLink();
						while (Link && Link->GetParent() && Link->GetParent()->GetSkeleton())
						{
							Link = Link->GetParent();
						}

						if (Link != NULL)
						{
							break;
						}
					}

					MeshInfo.SkeletonRoot = MakeString(Link ? Link->GetName() : ("None"));
					MeshInfo.SkeletonElemNum = Link ? Link->GetChildCount(true) : 0;

					if (Link)
					{
						FbxTimeSpan AnimTimeSpan(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
						Link->GetAnimationInterval(AnimTimeSpan);
						GlobalTimeSpan.UnionAssignment(AnimTimeSpan);
					}
				}
				else
				{
					SceneInfo.NonSkinnedMeshNum++;
					MeshInfo.bIsSkelMesh = false;
					MeshInfo.SkeletonRoot = NULL;
				}
				MeshInfo.UniqueId = Mesh->GetUniqueID();
			}
		}
		
		SceneInfo.bHasAnimation = false;
		SceneInfo.bHasAnimationOnSkeletalMesh = false;
		int32 AnimCurveNodeCount = Scene->GetSrcObjectCount<FbxAnimCurveNode>();
		// sadly Max export with animation curve node by default without any change, so 
		// we'll have to skip the first two curves, which is translation/rotation
		// if there is a valid animation, we'd expect there are more curve nodes than 2. 
		for (int32 AnimCurveNodeIndex = 2; AnimCurveNodeIndex < AnimCurveNodeCount; AnimCurveNodeIndex++)
		{
			FbxAnimCurveNode* CurAnimCruveNode = Scene->GetSrcObject<FbxAnimCurveNode>(AnimCurveNodeIndex);
			if (CurAnimCruveNode->IsAnimated(true))
			{
				SceneInfo.bHasAnimation = true;

				const FbxProperty DstProperty = CurAnimCruveNode->GetDstProperty();
				const FbxObject* AnimatedObject = DstProperty.GetFbxObject();
				if (AnimatedObject && (AnimatedObject->Is<FbxGeometry>() || (AnimatedObject->Is<FbxNode>() && IsUnrealBone((FbxNode*)AnimatedObject))))
				{
					SceneInfo.bHasAnimationOnSkeletalMesh = true;
				}
			}

			if (SceneInfo.bHasAnimation && SceneInfo.bHasAnimationOnSkeletalMesh)
			{
				break;
			}
		}

		SceneInfo.FrameRate = FbxTime::GetFrameRate(Scene->GetGlobalSettings().GetTimeMode());
		
		if ( GlobalTimeSpan.GetDirection() == FBXSDK_TIME_FORWARD)
		{
			SceneInfo.TotalTime = (GlobalTimeSpan.GetDuration().GetMilliSeconds())/1000.f * SceneInfo.FrameRate;
		}
		else
		{
			SceneInfo.TotalTime = 0;
		}
		
		FbxNode* RootNode = Scene->GetRootNode();
		FbxNodeInfo RootInfo;
		RootInfo.ObjectName = MakeName(RootNode->GetName());
		RootInfo.UniqueId = RootNode->GetUniqueID();
		RootInfo.Transform = RootNode->EvaluateGlobalTransform();

		RootInfo.AttributeName = NULL;
		RootInfo.AttributeUniqueId = 0;
		RootInfo.AttributeType = NULL;

		RootInfo.ParentName = NULL;
		RootInfo.ParentUniqueId = 0;
		
		//Add the rootnode to the SceneInfo
		SceneInfo.HierarchyInfo.Add(RootInfo);
		//Fill the hierarchy info
		TraverseHierarchyNodeRecursively(SceneInfo, RootNode, RootInfo);
	}
	
	GWarn->EndSlowTask();
	return Result;
}

void FFbxImporter::TraverseHierarchyNodeRecursively(FbxSceneInfo& SceneInfo, FbxNode *ParentNode, FbxNodeInfo &ParentInfo)
{
	int32 NodeCount = ParentNode->GetChildCount();
	for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
	{
		FbxNode* ChildNode = ParentNode->GetChild(NodeIndex);
		FbxNodeInfo ChildInfo;
		ChildInfo.ObjectName = MakeName(ChildNode->GetName());
		ChildInfo.UniqueId = ChildNode->GetUniqueID();
		ChildInfo.ParentName = ParentInfo.ObjectName;
		ChildInfo.ParentUniqueId = ParentInfo.UniqueId;
		ChildInfo.RotationPivot = ChildNode->RotationPivot.Get();
		ChildInfo.ScalePivot = ChildNode->ScalingPivot.Get();
		ChildInfo.Transform = ChildNode->EvaluateLocalTransform();
		if (ChildNode->GetNodeAttribute())
		{
			FbxNodeAttribute *ChildAttribute = ChildNode->GetNodeAttribute();
			ChildInfo.AttributeUniqueId = ChildAttribute->GetUniqueID();
			if (ChildAttribute->GetName()[0] != '\0')
			{
				ChildInfo.AttributeName = MakeName(ChildAttribute->GetName());
			}
			else
			{
				//Get the name of the first node that link this attribute
				ChildInfo.AttributeName = MakeName(ChildAttribute->GetNode()->GetName());
			}

			switch (ChildAttribute->GetAttributeType())
			{
			case FbxNodeAttribute::eUnknown:
				ChildInfo.AttributeType = "eUnknown";
				break;
			case FbxNodeAttribute::eNull:
				ChildInfo.AttributeType = "eNull";
				break;
			case FbxNodeAttribute::eMarker:
				ChildInfo.AttributeType = "eMarker";
				break;
			case FbxNodeAttribute::eSkeleton:
				ChildInfo.AttributeType = "eSkeleton";
				break;
			case FbxNodeAttribute::eMesh:
				ChildInfo.AttributeType = "eMesh";
				break;
			case FbxNodeAttribute::eNurbs:
				ChildInfo.AttributeType = "eNurbs";
				break;
			case FbxNodeAttribute::ePatch:
				ChildInfo.AttributeType = "ePatch";
				break;
			case FbxNodeAttribute::eCamera:
				ChildInfo.AttributeType = "eCamera";
				break;
			case FbxNodeAttribute::eCameraStereo:
				ChildInfo.AttributeType = "eCameraStereo";
				break;
			case FbxNodeAttribute::eCameraSwitcher:
				ChildInfo.AttributeType = "eCameraSwitcher";
				break;
			case FbxNodeAttribute::eLight:
				ChildInfo.AttributeType = "eLight";
				break;
			case FbxNodeAttribute::eOpticalReference:
				ChildInfo.AttributeType = "eOpticalReference";
				break;
			case FbxNodeAttribute::eOpticalMarker:
				ChildInfo.AttributeType = "eOpticalMarker";
				break;
			case FbxNodeAttribute::eNurbsCurve:
				ChildInfo.AttributeType = "eNurbsCurve";
				break;
			case FbxNodeAttribute::eTrimNurbsSurface:
				ChildInfo.AttributeType = "eTrimNurbsSurface";
				break;
			case FbxNodeAttribute::eBoundary:
				ChildInfo.AttributeType = "eBoundary";
				break;
			case FbxNodeAttribute::eNurbsSurface:
				ChildInfo.AttributeType = "eNurbsSurface";
				break;
			case FbxNodeAttribute::eShape:
				ChildInfo.AttributeType = "eShape";
				break;
			case FbxNodeAttribute::eLODGroup:
				ChildInfo.AttributeType = "eLODGroup";
				break;
			case FbxNodeAttribute::eSubDiv:
				ChildInfo.AttributeType = "eSubDiv";
				break;
			case FbxNodeAttribute::eCachedEffect:
				ChildInfo.AttributeType = "eCachedEffect";
				break;
			case FbxNodeAttribute::eLine:
				ChildInfo.AttributeType = "eLine";
				break;
			}
		}
		else
		{
			ChildInfo.AttributeUniqueId = INVALID_UNIQUE_ID;
			ChildInfo.AttributeType = "eNull";
			ChildInfo.AttributeName = NULL;
		}
		
		SceneInfo.HierarchyInfo.Add(ChildInfo);
		TraverseHierarchyNodeRecursively(SceneInfo, ChildNode, ChildInfo);
	}
}

bool FFbxImporter::OpenFile(FString Filename)
{
	bool Result = true;
	
	if (CurPhase != NOTSTARTED)
	{
		// something went wrong
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::OpenFile);

	GWarn->BeginSlowTask( LOCTEXT("OpeningFile", "Reading File"), true);
	GWarn->StatusForceUpdate(20, 100, LOCTEXT("OpeningFile", "Reading File"));

	ClearAllCaches();

	int32 SDKMajor,  SDKMinor,  SDKRevision;

	// Create an importer.
	Importer = FbxImporter::Create(SdkManager,"");

	// Get the version number of the FBX files generated by the
	// version of FBX SDK that you are using.
	FbxManager::GetFileFormatVersion(SDKMajor, SDKMinor, SDKRevision);

	if (SdkManager->GetIOSettings())
	{
		SdkManager->GetIOSettings()->SetBoolProp(IMP_RELAXED_FBX_CHECK, true);
	}

	// Initialize the importer by providing a filename.
	const bool bImportStatus = Importer->Initialize(TCHAR_TO_UTF8(*Filename), -1, SdkManager->GetIOSettings());
	
	FbxCreator = EFbxCreator::Unknow;
	FbxIOFileHeaderInfo *FileHeaderInfo = Importer->GetFileHeaderInfo();
	if (FileHeaderInfo)
	{
		//Example of creator file info string
		//Blender (stable FBX IO) - 2.78 (sub 0) - 3.7.7
		//Maya and Max use the same string where they specify the fbx sdk version, so we cannot know it is coming from which software
		//We need blender creator when importing skeletal mesh containing the "armature" dummy node as the parent of the root joint. We want to remove this dummy "armature" node
		FString CreatorStr(FileHeaderInfo->mCreator.Buffer());
		if (CreatorStr.StartsWith(TEXT("Blender")))
		{
			FbxCreator = EFbxCreator::Blender;
		}
	}
	GWarn->StatusForceUpdate(100, 100, LOCTEXT("OpeningFile", "Reading File"));
	GWarn->EndSlowTask();
	if( !bImportStatus )  // Problem with the file to be imported
	{
		UE_LOGF(LogFbx, Error,"Call to FbxImporter::Initialize() failed.");
		UE_LOGF(LogFbx, Warning, "Error returned: %ls", UTF8_TO_TCHAR(Importer->GetStatus().GetErrorString()));

		if (Importer->GetStatus().GetCode() == FbxStatus::eInvalidFileVersion )
		{
			UE_LOGF(LogFbx, Warning, "FBX version number for this FBX SDK is %d.%d.%d",
				SDKMajor, SDKMinor, SDKRevision);
		}

		return false;
	}

	// Version out of date warning
	int32 FileMajor = 0, FileMinor = 0, FileRevision = 0;
	Importer->GetFileVersion(FileMajor, FileMinor, FileRevision);
	int32 FileVersion = (FileMajor << 16 | FileMinor << 8 | FileRevision);
	int32 SDKVersion = (SDKMajor << 16 | SDKMinor << 8 | SDKRevision);
	if( FileVersion != SDKVersion )
	{
		// Appending the SDK version to the config key causes the warning to automatically reappear even if previously suppressed when the SDK version we use changes. 
		FString ConfigStr = FString::Printf( TEXT("Warning_OutOfDateFBX_%d"), SDKVersion );

		FString FileVerStr = FString::Printf( TEXT("%d.%d.%d"), FileMajor, FileMinor, FileRevision );
		FString SDKVerStr  = FString::Printf( TEXT("%d.%d.%d"), SDKMajor, SDKMinor, SDKRevision );

		const FText WarningText = FText::Format(
			NSLOCTEXT("UnrealEd", "Warning_OutOfDateFBX", "An out of date FBX has been detected.\nImporting different versions of FBX files than the SDK version can cause undesirable results.\n\nFile Version: {0}\nSDK Version: {1}" ),
			FText::FromString(FileVerStr), FText::FromString(SDKVerStr) );
	}

	//Cache the current file hash
	Md5Hash = FMD5Hash::HashFile(*Filename);

	CurPhase = FILEOPENED;
	// Destroy the importer
	//Importer->Destroy();

	return Result;
}

TSet<FbxFileTexture*> GetFbxMaterialTextures(const FbxSurfaceMaterial& Material)
{
	TSet<FbxFileTexture*> TextureSet;

	int32 TextureIndex;
	FBXSDK_FOR_EACH_TEXTURE(TextureIndex)
	{
		FbxProperty Property = Material.FindProperty(FbxLayerElement::sTextureChannelNames[TextureIndex]);

		if (Property.IsValid())
		{
			//We use auto as the parameter type to allow for a generic lambda accepting both FbxProperty and FbxLayeredTexture
			auto AddSrcTextureToSet =  [&TextureSet](const auto& InObject) {
				int32 NbTextures = InObject.template GetSrcObjectCount<FbxTexture>();
				for (int32 TexIndex = 0; TexIndex < NbTextures; ++TexIndex)
				{
					FbxFileTexture* Texture = InObject.template GetSrcObject<FbxFileTexture>(TexIndex);
					if (Texture)
					{
						TextureSet.Add(Texture);
					}
				}
			};

			//Here we have to check if it's layered textures, or just textures:
			const int32 LayeredTextureCount = Property.GetSrcObjectCount<FbxLayeredTexture>();
			if (LayeredTextureCount > 0)
			{
				for (int32 LayerIndex = 0; LayerIndex < LayeredTextureCount; ++LayerIndex)
				{
					if (const FbxLayeredTexture* lLayeredTexture = Property.GetSrcObject<FbxLayeredTexture>(LayerIndex))
					{
						AddSrcTextureToSet(*lLayeredTexture);
					}
				}
			}
			else
			{
				//no layered texture simply get on the property
				AddSrcTextureToSet(Property);
			}
		}
	}

	return TextureSet;
}

void FFbxImporter::FixMaterialClashName()
{
	const bool bKeepNamespace = GetDefault<UEditorPerProjectUserSettings>()->bKeepFbxNamespace;

	FbxArray<FbxSurfaceMaterial*> MaterialArray;
	Scene->FillMaterialArray(MaterialArray);

	TSet<FString> AllMaterialAndTextureNames;
	TSet<FbxFileTexture*> MaterialTextures;

	auto FixNameIfNeeded = [this, &AllMaterialAndTextureNames](const FString& AssetName, 
		TFunctionRef<void(const FString& /*UniqueName*/)> ApplyUniqueNameFunction,
		TFunctionRef<FText(const FString& /*UniqueName*/)> GetErrorTextFunction){

		FString UniqueName(AssetName);
		if (AllMaterialAndTextureNames.Contains(UniqueName))
		{
			//Use the fbx nameclash 1 convention: NAMECLASH1_KEY
			//This will add _ncl1_
			FString AssetBaseName = UniqueName + TEXT(NAMECLASH1_KEY);
			int32 NameIndex = 1;
			do 
			{
				UniqueName = AssetBaseName + FString::FromInt(NameIndex++);
			} while (AllMaterialAndTextureNames.Contains(UniqueName));

			//Apply the unique name.
			ApplyUniqueNameFunction(UniqueName);
			if (!GIsAutomationTesting)
			{
				AddTokenizedErrorMessage(
					FTokenizedMessage::Create(EMessageSeverity::Info, GetErrorTextFunction(UniqueName)),
					FFbxErrors::Generic_LoadingSceneFailed);
			}
		}
		AllMaterialAndTextureNames.Add(UniqueName);
	};

	// First rename materials to unique names and gather their texture.
	for (int32 MaterialIndex = 0; MaterialIndex < MaterialArray.Size(); ++MaterialIndex)
	{
		FbxSurfaceMaterial *Material = MaterialArray[MaterialIndex];
		FString MaterialName = MakeName(Material->GetName());
		MaterialTextures.Append(GetFbxMaterialTextures(*Material));

		if (!bKeepNamespace)
		{
			Material->SetName(TCHAR_TO_UTF8(*MaterialName));
		}

		FixNameIfNeeded(MaterialName,
			[&](const FString& UniqueName) { Material->SetName(TCHAR_TO_UTF8(*UniqueName)); },
			[&](const FString& UniqueName) {
				return FText::Format(LOCTEXT("FbxImport_MaterialNameClash", "FBX Scene Loading: Found material name clash, name clash can be wrongly reassign at reimport , material '{0}' was renamed '{1}'"), FText::FromString(MaterialName), FText::FromString(UniqueName));
			}
		);
	}

	// Then rename make sure the texture have unique names as well.
	for (FbxFileTexture* CurrentTexture : MaterialTextures)
	{
		FString AbsoluteFilename = UTF8_TO_TCHAR(CurrentTexture->GetFileName());
		FString TextureName = FPaths::GetBaseFilename(AbsoluteFilename);
		TextureName = ObjectTools::SanitizeObjectName(TextureName);

		FixNameIfNeeded(TextureName,
			[&](const FString& UniqueName) { FbxTextureToUniqueNameMap.Add(CurrentTexture, UniqueName); },
			[&](const FString& UniqueName) {
				return FText::Format(LOCTEXT("FbxImport_TextureNameClash", "FBX Scene Loading: Found texture name clash, name clash can be wrongly reassign at reimport , texture '{0}' was renamed '{1}'"), FText::FromString(TextureName), FText::FromString(UniqueName));
			}
		);
	}
}

void FFbxImporter::EnsureNodeNameAreValid(const FString& BaseFilename)
{
	const bool bKeepNamespace = GetDefault<UEditorPerProjectUserSettings>()->bKeepFbxNamespace;

	TSet<FString> AllNodeName;
	int32 CurrentNameIndex = 1;
	for (int32 NodeIndex = 0; NodeIndex < Scene->GetNodeCount(); ++NodeIndex)
	{
		FbxNode* Node = Scene->GetNode(NodeIndex);
		FString NodeName = UTF8_TO_TCHAR(Node->GetName());
		if (NodeName.IsEmpty())
		{
			do
			{
				NodeName = TEXT("ncl1_") + FString::FromInt(CurrentNameIndex++);
			} while (AllNodeName.Contains(NodeName));

			Node->SetName(TCHAR_TO_UTF8(*NodeName));
			if (!GIsAutomationTesting)
			{
				AddTokenizedErrorMessage(
					FTokenizedMessage::Create(EMessageSeverity::Warning,
					FText::Format(LOCTEXT("FbxImport_NoNodeName", "FBX File Loading: Found node with no name, new node name is '{0}'"), FText::FromString(NodeName))),
					FFbxErrors::Generic_LoadingSceneFailed);
			}
		}
		if (bKeepNamespace)
		{
			if (NodeName.Contains(TEXT(":")))
			{
				NodeName = NodeName.Replace(TEXT(":"), TEXT("_"));
				Node->SetName(TCHAR_TO_UTF8(*NodeName));
			}
		}
		// Do not allow node to be named same as filename as this creates problems later on (reimport)
		if (AllNodeName.Contains(NodeName) || (ImportOptions->bImportScene && 0 == NodeName.Compare(BaseFilename, ESearchCase::IgnoreCase)))
		{
			FString UniqueNodeName;
			do
			{
				UniqueNodeName = NodeName + FString::FromInt(CurrentNameIndex++);
			} while (AllNodeName.Contains(UniqueNodeName));

			FbxString UniqueName(TCHAR_TO_UTF8(*UniqueNodeName));
			NodeUniqueNameToOriginalNameMap[UniqueName] = Node->GetName();
			Node->SetName(UniqueName);
			
			if (!GIsAutomationTesting)
			{
				AddTokenizedErrorMessage(
					FTokenizedMessage::Create(EMessageSeverity::Info,
						FText::Format(LOCTEXT("FbxImport_NodeNameClash", "FBX File Loading: Found name clash, node '{0}' was renamed to '{1}'"), FText::FromString(NodeName), FText::FromString(UniqueNodeName))),
					FFbxErrors::Generic_LoadingSceneFailed);
			}
		}
		AllNodeName.Add(NodeName);
	}
}

void FFbxImporter::RemoveFBXMetaData(const UObject* Object)
{
	TArray<FName> KeysToRemove;
	if (const TMap<FName, FString>* ExistingUMetaDataTagValues = FMetaData::GetMapForObject(Object))
	{
		for (const TPair<FName, FString>& KeyValue : *ExistingUMetaDataTagValues)
		{
			if (KeyValue.Key.ToString().StartsWith(FBX_METADATA_PREFIX, ESearchCase::IgnoreCase))
			{
				KeysToRemove.Add(KeyValue.Key);
			}
		}
	}

	if (KeysToRemove.Num() > 0)
	{
		FMetaData& PackageMetaData = Object->GetPackage()->GetMetaData();
		for (const FName& KeyToRemove : KeysToRemove)
		{
			PackageMetaData.RemoveValue(Object, KeyToRemove);
		}
	}
}


FString FFbxImporter::GetFileAxisDirection()
{
	FString AxisDirection;
	int32 Sign = 1;
	switch (FileAxisSystem.GetUpVector(Sign))
	{
	case FbxAxisSystem::eXAxis:
		{
			AxisDirection += TEXT("X");
		}
		break;
	case FbxAxisSystem::eYAxis:
		{
			AxisDirection += TEXT("Y");
		}
		break;
	case FbxAxisSystem::eZAxis:
		{
			AxisDirection += TEXT("Z");
		}
		break;
	}
	//Negative sign mean down instead of up
	AxisDirection += Sign == 1 ? TEXT("-UP") : TEXT("-DOWN");
		
	switch (FileAxisSystem.GetCoorSystem())
	{
	case FbxAxisSystem::eLeftHanded:
		{
			AxisDirection += TEXT(" (LH)");
		}
		break;
	case FbxAxisSystem::eRightHanded:
		{
			AxisDirection += TEXT(" (RH)");
		}
		break;
	}
	return AxisDirection;
}

#ifdef IOS_REF
#undef  IOS_REF
#define IOS_REF (*(SdkManager->GetIOSettings()))
#endif

bool FFbxImporter::ImportFile(FString Filename, bool bPreventMaterialNameClash /*=false*/)
{
	if (Scene)
	{
		UE_LOGF(LogFbx, Error, "FBX Scene already loaded from %ls", *Filename);
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ImportFile);

	bool Result = true;
	
	bool bStatus;
	
	FileBasePath = FPaths::GetPath(Filename);

	// Create the Scene
	Scene = FbxScene::Create(SdkManager,"");
	UE_LOGF(LogFbx, Log, "Loading FBX Scene from %ls", *Filename);

	int32 FileMajor, FileMinor, FileRevision;

	IOS_REF.SetBoolProp(IMP_FBX_MATERIAL,		true);
	IOS_REF.SetBoolProp(IMP_FBX_TEXTURE,		 true);
	IOS_REF.SetBoolProp(IMP_FBX_LINK,			true);
	IOS_REF.SetBoolProp(IMP_FBX_SHAPE,		   true);
	IOS_REF.SetBoolProp(IMP_FBX_GOBO,			true);
	IOS_REF.SetBoolProp(IMP_FBX_ANIMATION,	   true);
	IOS_REF.SetBoolProp(IMP_SKINS,			   true);
	IOS_REF.SetBoolProp(IMP_DEFORMATION,		 true);
	IOS_REF.SetBoolProp(IMP_FBX_GLOBAL_SETTINGS, true);
	IOS_REF.SetBoolProp(IMP_TAKE,				true);

	// Import the scene.
	bStatus = Importer->Import(Scene);

	const bool bRemovePath = true;
	EnsureNodeNameAreValid(FPaths::GetBaseFilename(Filename, bRemovePath));

	//Make sure we don't have name clash for materials
	if (bPreventMaterialNameClash)
	{
		FixMaterialClashName();
	}

	// Get the version number of the FBX file format.
	Importer->GetFileVersion(FileMajor, FileMinor, FileRevision);
	FbxFileVersion = FString::Printf(TEXT("%d.%d.%d"), FileMajor, FileMinor, FileRevision);
	
	FbxFileCreator = UTF8_TO_TCHAR(Importer->GetFileHeaderInfo()->mCreator.Buffer());
	// output result
	if(bStatus)
	{
		UE_LOGF(LogFbx, Log, "FBX Scene Loaded Succesfully");
		CurPhase = IMPORTED;
	}
	else
	{
		ErrorMessage = UTF8_TO_TCHAR(Importer->GetStatus().GetErrorString());
		AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::Format(LOCTEXT("FbxSkeletaLMeshimport_FileLoadingFailed", "FBX Scene Loading Failed : '{0}'"), FText::FromString(ErrorMessage))), FFbxErrors::Generic_LoadingSceneFailed);
		// ReleaseScene will also release the importer if it was initialized
		ReleaseScene();
		Result = false;
		CurPhase = NOTSTARTED;
		return Result;
	}

	const FbxGlobalSettings& GlobalSettings = Scene->GetGlobalSettings();
	FbxTime::EMode TimeMode = GlobalSettings.GetTimeMode();
	//Set the original framerate from the current fbx file
	OriginalFbxFramerate = FbxTime::GetFrameRate(TimeMode);

	return Result;
}

void FFbxImporter::ConvertScene()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FFbxImporter::ConvertScene);

	//Merge the anim stack before the conversion since the above 0 layer will not be converted
	int32 AnimStackCount = Scene->GetSrcObjectCount<FbxAnimStack>();
	//Merge the animation stack layer before converting the scene
	for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
	{
		FbxAnimStack* CurAnimStack = Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);
		if (CurAnimStack->GetMemberCount() > 1)
		{
			int32 ResampleRate = GetGlobalAnimStackSampleRate(CurAnimStack);
			MergeAllLayerAnimation(CurAnimStack, ResampleRate);
		}
	}

	//Set the original file information
	FileAxisSystem = Scene->GetGlobalSettings().GetAxisSystem();
	FileUnitSystem = Scene->GetGlobalSettings().GetSystemUnit();

	FbxAMatrix AxisConversionMatrix;
	AxisConversionMatrix.SetIdentity();

	FbxAMatrix JointOrientationMatrix;
	JointOrientationMatrix.SetIdentity();

	if (GetImportOptions()->bConvertScene)
	{
		// we use -Y as forward axis here when we import. This is odd considering our forward axis is technically +X
		// but this is to mimic Maya/Max behavior where if you make a model facing +X facing, 
		// when you import that mesh, you want +X facing in engine. 
		// only thing that doesn't work is hand flipping because Max/Maya is RHS but UE is LHS
		// On the positive note, we now have import transform set up you can do to rotate mesh if you don't like default setting
		FbxAxisSystem::ECoordSystem CoordSystem = FbxAxisSystem::eRightHanded;
		FbxAxisSystem::EUpVector UpVector = FbxAxisSystem::eZAxis;
		FbxAxisSystem::EFrontVector FrontVector = (FbxAxisSystem::EFrontVector) - FbxAxisSystem::eParityOdd;
		if (GetImportOptions()->bForceFrontXAxis)
		{
			FrontVector = FbxAxisSystem::eParityEven;
		}


		FbxAxisSystem UnrealImportAxis(UpVector, FrontVector, CoordSystem);

		FbxAxisSystem SourceSetup = Scene->GetGlobalSettings().GetAxisSystem();


		if (SourceSetup != UnrealImportAxis)
		{
			FbxRootNodeUtility::RemoveAllFbxRoots(Scene);
			UnrealImportAxis.ConvertScene(Scene);
			
			FbxAMatrix SourceMatrix;
			SourceSetup.GetMatrix(SourceMatrix);
			FbxAMatrix UE4Matrix;
			UnrealImportAxis.GetMatrix(UE4Matrix);
			AxisConversionMatrix = SourceMatrix.Inverse() * UE4Matrix;
			
			
			if (GetImportOptions()->bForceFrontXAxis)
			{
				JointOrientationMatrix.SetR(FbxVector4(-90.0, -90.0, 0.0));
			}
		}
	}

	FFbxDataConverter::SetJointPostConversionMatrix(JointOrientationMatrix);

	FFbxDataConverter::SetAxisConversionMatrix(AxisConversionMatrix);

	// Convert the scene's units to what is used in this program, if needed.
	// The base unit used in both FBX and Unreal is centimeters.  So unless the units 
	// are already in centimeters (ie: scalefactor 1.0) then it needs to be converted
	if (GetImportOptions()->bConvertSceneUnit && Scene->GetGlobalSettings().GetSystemUnit() != FbxSystemUnit::cm)
	{
		FbxSystemUnit::cm.ConvertScene(Scene);
	}

	//Reset all the transform evaluation cache since we change some node transform
	Scene->GetAnimationEvaluator()->Reset();
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
bool FFbxImporter::ReadHeaderFromFile(const FString& Filename, bool bPreventMaterialNameClash /*= false*/)
{
	bool Result = true;


	switch (CurPhase)
	{
	case NOTSTARTED:
		if (!OpenFile(FString(Filename)))
		{
			Result = false;
			break;
		}
	case FILEOPENED:
		if (!ImportFile(FString(Filename), bPreventMaterialNameClash))
		{
			Result = false;
			CurPhase = NOTSTARTED;
			break;
		}
	}
	return Result;
}

//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
bool FFbxImporter::ImportFromFile(const FString& Filename, const FString& Type, bool bPreventMaterialNameClash /*= false*/)
{
	FFbxScopedOperation ScopedImportOperation(this);
	bool Result = true;


	switch (CurPhase)
	{
	case NOTSTARTED:
		if (!OpenFile(FString(Filename)))
		{
			Result = false;
			break;
		}
	case FILEOPENED:
		if (!ImportFile(FString(Filename), bPreventMaterialNameClash))
		{
			Result = false;
			CurPhase = NOTSTARTED;
			break;
		}
	case IMPORTED:
		{
			static const FString Obj(TEXT("obj"));

			// The imported axis system is unknown for obj files
			if( !Type.Equals( Obj, ESearchCase::IgnoreCase ) )
			{
				//Convert the scene
				ConvertScene();

				// Run Analytics for FBX Import data
				/**
				  * @EventName Editor.Usage.FBX
				  * @Trigger Fires when the user clicks OK in the FBX Import Dialog
				  * @Type Editor
				  * @EventParam LastSavedVendor string Returns the name of the vendor that manufactures the last application used to modify the imported FBX
				  * @EventParam LastSavedAppName string Returns the name of the last application used to modify the imported FBX
				  * @EventParam LastSavedAppVersion string Returns the revision of the last application used to modify the imported FBX
				  * @EventParam FBXFileVersion string Returns the FBX SDK used to generate the imported FBX
				  * @EventParam ImportType string Returns the mesh data type (Static, Skeletal, Animation) being imported from the FBX
				  * @EventParam ConvertScene boolean Returns whether the import fbx should be converted to unreal axis system
				  * @EventParam ConvertSceneUnit boolean Returns whether the import fbx should converted the unit to unreal unit (cm)
				  * @EventParam ForceFrontXAxis boolean Returns whether the import fbx should be converted to unreal axis system with front axis being X
				  * @EventParam ImportMaterials boolean Returns whether the importer should create the missing materials
				  * @EventParam ImportTextures boolean Returns whether the importer should create the missing textures
				  * @EventParam InvertNormalMap boolean Returns whether the importer should inverse the incoming normal map textures
				  * @EventParam RemoveNameSpace boolean Returns whether the importer should remove namespace on all nodes name
				  * @EventParam UsedAsFullName boolean Returns whether the importer should use the filename to name the imported mesh
				  * @EventParam ImportTranslation string Returns the translation vector apply on the import data
				  * @EventParam ImportRotation string Returns the FRotator vector apply on the import data
				  * @EventParam ImportUniformScale float Returns the uniform scale apply on the import data
				  * @EventParam MaterialBasePath string Returns the path pointing on the base material use to import material instance
				  * @EventParam MaterialSearchLocation string Returns the scope of the search for existing materials (if material not found it can create one depending on bImportMaterials value)
				  * @EventParam ReorderMaterialToFbxOrder boolean returns weather the importer should reorder the materials in the same order has the fbx file
				  * @EventParam AutoGenerateCollision boolean Returns whether the importer should create collision primitive
				  * @EventParam CombineToSingle boolean Returns whether the importer should combine all mesh part together or import many meshes
				  * @EventParam BakePivotInVertex boolean Returns whether the importer should bake the fbx mesh pivot into the vertex position
				  * @EventParam TransformVertexToAbsolute boolean Returns whether the importer should bake the global fbx node transform into the vertex position
				  * @EventParam ImportRigidMesh boolean Returns whether the importer should try to create a rigid mesh (static mesh import as skeletal mesh)
				  * @EventParam NormalImportMethod string Return if the tangents or normal should be imported or compute
				  * @EventParam NormalGenerationMethod string Return tangents generation method
				  * @EventParam CreatePhysicsAsset boolean Returns whether the importer should create the physic asset
				  * @EventParam ImportAnimations boolean Returns whether the importer should import also the animation
				  * @EventParam ImportAsSkeletalGeometry boolean Returns whether the importer should import only the geometry
				  * @EventParam ImportAsSkeletalSkinning boolean Returns whether the importer should import only the skinning
				  * @EventParam ImportMeshesInBoneHierarchy boolean Returns whether the importer should import also the mesh found in the bone hierarchy
				  * @EventParam ImportMorph boolean Returns whether the importer should import the morph targets
				  * @EventParam ImportSkeletalMeshLODs boolean Returns whether the importer should import the LODs
				  * @EventParam PreserveSmoothingGroups boolean Returns whether the importer should import the smoothing groups
				  * @EventParam UpdateSkeletonReferencePose boolean Returns whether the importer should update the skeleton reference pose
				  * @EventParam UseT0AsRefPose boolean Returns whether the importer should use the the animation 0 time has the reference pose
				  * @EventParam ThresholdPosition float Returns the threshold delta to weld vertices
				  * @EventParam ThresholdTangentNormal float Returns the threshold delta to weld tangents and normals
				  * @EventParam ThresholdUV float Returns the threshold delta to weld UVs
				  * @EventParam MorphThresholdPosition float Returns the morph target threshold delta to compute deltas
				  * @EventParam AutoComputeLodDistances boolean Returns whether the importer should set the auto compute LOD distance
				  * @EventParam LodNumber integer Returns the LOD number we should have after the import
				  * @EventParam BuildReversedIndexBuffer boolean Returns whether the importer should fill the reverse index buffer when building the static mesh
				  * @EventParam GenerateLightmapUVs boolean Returns whether the importer should generate light map UVs
				  * @EventParam ImportStaticMeshLODs boolean Returns whether the importer should import the LODs
				  * @EventParam RemoveDegenerates boolean Returns whether the importer should remove the degenerated triangles when building the static mesh
				  * @EventParam MinimumLodNumber integer Returns the minimum LOD use by the rendering
				  * @EventParam StaticMeshLODGroup string Returns the LOD Group settings we use to build this imported static mesh
				  * @EventParam VertexColorImportOption string Returns how the importer should import the vertex color
				  * @EventParam VertexOverrideColor string Returns the color use if we need to override the vertex color
				  * @EventParam AnimationLengthImportType string Returns how we choose the animation time span
				  * @EventParam DeleteExistingMorphTargetCurves boolean Returns whether the importer should delete the existing morph target curves
				  * @EventParam AnimationRange string Returns the range of animation the importer should sample if the time span is custom
				  * @EventParam DoNotImportCurveWithZero boolean Returns whether the importer should import curves containing only zero value
				  * @EventParam ImportBoneTracks boolean Returns whether the importer should import the bone tracks
				  * @EventParam ImportCustomAttribute boolean Returns whether the importer should import the custom attribute curves
				  * @EventParam PreserveLocalTransform boolean Returns whether the importer should preserve the local transform when importing the animation
				  * @EventParam RemoveRedundantKeys boolean Returns whether the importer should remove all redundant key in an animation
				  * @EventParam Resample boolean Returns whether the importer should re-sample the animation
				  * @EventParam SetMaterialDriveParameterOnCustomAttribute boolean Returns whether the importer should hook all custom attribute curve to unreal material attribute
				  * @EventParam SetMaterialDriveParameterOnCustomAttribute boolean Returns whether the importer should hook some custom attribute (having the suffix) curve to unreal material attribute
				  * @EventParam ResampleRate float Returns the rate the exporter is suppose to re-sample any imported animations
				  * 
				  * @Owner Alexis.Matte
				*/
				FbxDocumentInfo* DocInfo = Scene->GetSceneInfo();
				if (DocInfo)
				{
					if( FEngineAnalytics::IsAvailable() )
					{
						const static UEnum* FBXImportTypeEnum = StaticEnum<EFBXImportType>();
						const static UEnum* FBXAnimationLengthImportTypeEnum = StaticEnum<EFBXAnimationLengthImportType>();
						const static UEnum* MaterialSearchLocationEnum = StaticEnum<EMaterialSearchLocation>();
						const static UEnum* FBXNormalGenerationMethodEnum = StaticEnum<EFBXNormalGenerationMethod::Type>();
						const static UEnum* FBXNormalImportMethodEnum = StaticEnum<EFBXNormalImportMethod>();
						const static UEnum* VertexColorImportOptionEnum = StaticEnum<EVertexColorImportOption::Type>();
						
						TArray<FAnalyticsEventAttribute> Attribs;

						FString LastSavedVendor(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVendor.Get().Buffer()));
						FString LastSavedAppName(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationName.Get().Buffer()));
						FString LastSavedAppVersion(UTF8_TO_TCHAR(DocInfo->LastSaved_ApplicationVersion.Get().Buffer()));

						Attribs.Add(FAnalyticsEventAttribute(TEXT("LastSaved Application Vendor"), LastSavedVendor));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("LastSaved Application Name"), LastSavedAppName));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("LastSaved Application Version"), LastSavedAppVersion));

						Attribs.Add(FAnalyticsEventAttribute(TEXT("FBX Version"), FbxFileVersion));

						//////////////////////////////////////////////////////////////////////////
						//FBX import options
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportType"), FBXImportTypeEnum->GetNameStringByValue(ImportOptions->ImportType)));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ConvertScene"), ImportOptions->bConvertScene));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ConvertSceneUnit"), ImportOptions->bConvertSceneUnit));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ForceFrontXAxis"), ImportOptions->bForceFrontXAxis));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportMaterials"), ImportOptions->bImportMaterials));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportTextures"), ImportOptions->bImportTextures));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt InvertNormalMap"), ImportOptions->bInvertNormalMap));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt RemoveNameSpace"), ImportOptions->bRemoveNameSpace));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt UsedAsFullName"), ImportOptions->bUsedAsFullName));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportTranslation"), ImportOptions->ImportTranslation.ToString()));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportRotation"), ImportOptions->ImportRotation.ToString()));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ImportUniformScale"), ImportOptions->ImportUniformScale));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt MaterialBasePath"), ImportOptions->MaterialBasePath));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt MaterialSearchLocation"), MaterialSearchLocationEnum->GetNameStringByValue((uint64)(ImportOptions->MaterialSearchLocation))));
						Attribs.Add(FAnalyticsEventAttribute(TEXT("GenOpt ReorderMaterialToFbxOrder"), ImportOptions->bReorderMaterialToFbxOrder));

						//We cant capture a this member, so just assign the pointer here
						FBXImportOptions* CaptureImportOptions = ImportOptions;
						auto AddMeshAnalytic = [&Attribs, &CaptureImportOptions]()
						{
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt AutoGenerateCollision"), CaptureImportOptions->bAutoGenerateCollision));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt CombineToSingle"), CaptureImportOptions->bCombineToSingle));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt BakePivotInVertex"), CaptureImportOptions->bBakePivotInVertex));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt TransformVertexToAbsolute"), CaptureImportOptions->bTransformVertexToAbsolute));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt ImportRigidMesh"), CaptureImportOptions->bImportRigidMesh));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt NormalGenerationMethod"), FBXNormalGenerationMethodEnum->GetNameStringByValue(CaptureImportOptions->NormalGenerationMethod)));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt NormalImportMethod"), FBXNormalImportMethodEnum->GetNameStringByValue(CaptureImportOptions->NormalImportMethod)));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("MeshOpt ComputeWeightedNormals"), CaptureImportOptions->bComputeWeightedNormals));
						};

						auto AddSKAnalytic = [&Attribs, &CaptureImportOptions]()
						{
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt CreatePhysicsAsset"), CaptureImportOptions->bCreatePhysicsAsset));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportAnimations"), CaptureImportOptions->bImportAnimations));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportAsSkeletalGeometry"), CaptureImportOptions->bImportAsSkeletalGeometry));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportAsSkeletalSkinning"), CaptureImportOptions->bImportAsSkeletalSkinning));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportMeshesInBoneHierarchy"), CaptureImportOptions->bImportMeshesInBoneHierarchy));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportMorph"), CaptureImportOptions->bImportMorph));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt ImportSkeletalMeshLODs"), CaptureImportOptions->bImportSkeletalMeshLODs));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt PreserveSmoothingGroups"), CaptureImportOptions->bPreserveSmoothingGroups));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt KeepSectionsSeparate"), CaptureImportOptions->bKeepSectionsSeparate));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt UpdateSkeletonReferencePose"), CaptureImportOptions->bUpdateSkeletonReferencePose));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt UseT0AsRefPose"), CaptureImportOptions->bUseT0AsRefPose));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt OverlappingThresholds.ThresholdPosition"), CaptureImportOptions->OverlappingThresholds.ThresholdPosition));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt OverlappingThresholds.ThresholdTangentNormal"), CaptureImportOptions->OverlappingThresholds.ThresholdTangentNormal));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt OverlappingThresholds.ThresholdUV"), CaptureImportOptions->OverlappingThresholds.ThresholdUV));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("SkeletalMeshOpt OverlappingThresholds.MorphThresholdPosition"), CaptureImportOptions->OverlappingThresholds.MorphThresholdPosition));
						};

						auto AddSMAnalytic = [&Attribs, &CaptureImportOptions]()
						{
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt AutoComputeLodDistances"), CaptureImportOptions->bAutoComputeLodDistances));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt LodNumber"), CaptureImportOptions->LodNumber));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt BuildReversedIndexBuffer"), CaptureImportOptions->bBuildReversedIndexBuffer));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt GenerateLightmapUVs"), CaptureImportOptions->bGenerateLightmapUVs));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt ImportStaticMeshLODs"), CaptureImportOptions->bImportStaticMeshLODs));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt RemoveDegenerates"), CaptureImportOptions->bRemoveDegenerates));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt MinimumLodNumber"), CaptureImportOptions->MinimumLodNumber));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt StaticMeshLODGroup"), CaptureImportOptions->StaticMeshLODGroup));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt VertexColorImportOption"), VertexColorImportOptionEnum->GetNameStringByValue(CaptureImportOptions->VertexColorImportOption)));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("StaticMeshOpt VertexOverrideColor"), CaptureImportOptions->VertexOverrideColor.ToString()));
						};

						auto AddAnimAnalytic = [&Attribs, &CaptureImportOptions]()
						{
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt AnimationLengthImportType"), FBXAnimationLengthImportTypeEnum->GetNameStringByValue(CaptureImportOptions->AnimationLengthImportType)));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt DeleteExistingMorphTargetCurves"), CaptureImportOptions->bDeleteExistingMorphTargetCurves));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt AnimationRange"), CaptureImportOptions->AnimationRange.ToString()));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt DoNotImportCurveWithZero"), CaptureImportOptions->bDoNotImportCurveWithZero));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt ImportBoneTracks"), CaptureImportOptions->bImportBoneTracks));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt ImportCustomAttribute"), CaptureImportOptions->bImportCustomAttribute));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt DeleteExistingCustomAttributeCurves"), CaptureImportOptions->bDeleteExistingCustomAttributeCurves));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt DeleteExistingNonCurveCustomAttributes"), CaptureImportOptions->bDeleteExistingNonCurveCustomAttributes));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt PreserveLocalTransform"), CaptureImportOptions->bPreserveLocalTransform));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt RemoveRedundantKeys"), CaptureImportOptions->bRemoveRedundantKeys));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt Resample"), CaptureImportOptions->bResample));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt SetMaterialDriveParameterOnCustomAttribute"), CaptureImportOptions->bSetMaterialDriveParameterOnCustomAttribute));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt MaterialCurveSuffixes"), CaptureImportOptions->MaterialCurveSuffixes));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt ResampleRate"), CaptureImportOptions->ResampleRate));
							Attribs.Add(FAnalyticsEventAttribute(TEXT("AnimOpt SnapToClosestFrameBoundary"), CaptureImportOptions->bSnapToClosestFrameBoundary));
						};
						
						if (ImportOptions->ImportType == FBXIT_SkeletalMesh)
						{
							AddMeshAnalytic();
							AddSKAnalytic();
							if (ImportOptions->bImportAnimations)
							{
								AddAnimAnalytic();
							}
						}
						else if (ImportOptions->ImportType == FBXIT_StaticMesh)
						{
							AddMeshAnalytic();
							AddSMAnalytic();
						}
						else if (ImportOptions->ImportType == FBXIT_Animation)
						{
							AddAnimAnalytic();
						}

						FString EventString = FString::Printf(TEXT("Editor.Usage.FBX.Import"));
						FEngineAnalytics::GetProvider().RecordEvent(EventString, Attribs);
					}
				}
			}

			//Warn the user if there is some geometry that cannot be imported because they are not reference by any scene node attribute
			ValidateAllMeshesAreReferenceByNodeAttribute();

			ConvertLodPrefixToLodGroup();

			MeshNamesCache.Empty();
			CurPhase = FIXEDANDCONVERTED;
			break;
		}
	case FIXEDANDCONVERTED:
	default:
		break;
	}
	
	return Result;
}

void FFbxImporter::SetScene(FbxScene* InScene)
{
	ClearAllCaches();
	Scene = InScene;
	FbxCreator = EFbxCreator::Unknow;
}

FString FFbxImporter::MakeName(const ANSICHAR* Name)
{
	const TCHAR SpecialChars[] = {TEXT('.'), TEXT(','), TEXT('/'), TEXT('`'), TEXT('%')};

	FString TmpName = MakeString(Name);
	
	// Remove namespaces
	int32 LastNamespaceTokenIndex = INDEX_NONE;
	if (TmpName.FindLastChar(TEXT(':'), LastNamespaceTokenIndex))
	{
		//+1 to remove the ':' character we found
		TmpName.RightChopInline(LastNamespaceTokenIndex + 1, EAllowShrinking::Yes);
	}

	//Remove the special chars
	for ( int32 i = 0; i < UE_ARRAY_COUNT(SpecialChars); i++ )
	{
		TmpName.ReplaceCharInline(SpecialChars[i], TEXT('_'), ESearchCase::CaseSensitive);
	}

	
	return TmpName;
}

FString FFbxImporter::MakeString(const ANSICHAR* Name)
{
	return FString(UTF8_TO_TCHAR(Name));
}

FName FFbxImporter::MakeNameForMesh(FString InName, FbxObject* FbxObject)
{
	FName OutputName;

	//Cant name the mesh if the object is null and there InName arguments is None.
	check(FbxObject != nullptr || InName != TEXT("None"))

	if ((ImportOptions->bUsedAsFullName || FbxObject == nullptr) && InName != TEXT("None"))
	{
		OutputName = *InName;
	}
	else
	{
		check(FbxObject);

		char Name[MAX_SPRINTF];
		int SpecialChars[] = {'.', ',', '/', '`', '%'};

		FCStringAnsi::Sprintf(Name, "%s", FbxObject->GetName());

		for ( int32 i = 0; i < 5; i++ )
		{
			char* CharPtr = Name;
			while ( (CharPtr = FCStringAnsi::Strchr(CharPtr,SpecialChars[i])) != nullptr )
			{
				CharPtr[0] = '_';
			}
		}

		// for mesh, replace ':' with '_' because Unreal doesn't support ':' in mesh name
		char* NewName = FCStringAnsi::Strchr(Name, ':');

		if (NewName)
		{
			char* Tmp;
			Tmp = NewName;
			while (Tmp)
			{

				// Always remove namespaces
				NewName = Tmp + 1;
				
				// there may be multiple namespace, so find the last ':'
				Tmp = FCStringAnsi::Strchr(NewName + 1, ':');
			}
		}
		else
		{
			NewName = Name;
		}

		int32 NameCount = 0;
		FString ComposeName;
		do 
		{
			if ( InName == FString("None"))
			{
				ComposeName = FString::Printf(TEXT("%s"), UTF8_TO_TCHAR(NewName ));
			}
			else
			{
				ComposeName = FString::Printf(TEXT("%s_%s"), *InName,UTF8_TO_TCHAR(NewName));
			}
			if (NameCount > 0)
			{
				ComposeName += TEXT("_") + FString::FromInt(NameCount);
			}
			NameCount++;
		} while (MeshNamesCache.Contains(ComposeName));
		OutputName = FName(*ComposeName);
	}
	
	MeshNamesCache.Add(OutputName.ToString());
	return OutputName;
}

/* The score is the difference between name lower score is better*/
int32 GetNodeNameDiffScore(const char* MeshName, const FbxNode* FbxMeshNode)
{
	if (FbxMeshNode == nullptr || MeshName == nullptr)
	{
		return INDEX_NONE;
	}
	int32 MeshNameLen = FCStringAnsi::Strlen(MeshName);
	if (MeshNameLen == 0)
	{
		return INDEX_NONE;
	}
	const char* FbxNodeName = FbxMeshNode->GetName();
	int32 FbxNodeNameLen = FCStringAnsi::Strlen(FbxNodeName);
	if (FbxNodeNameLen == 0 || FbxNodeNameLen > MeshNameLen)
	{
		return INDEX_NONE;
	}
	// The name of Unreal mesh may have a prefix, so we match from end
	int32 i = 0;
	const char* MeshNamePtr = MeshName + MeshNameLen - 1;
	const char* FbxMeshPtr = FbxNodeName + FbxNodeNameLen - 1;
	while (i < FbxNodeNameLen)
	{
		bool bIsPointAndUnderscore = *FbxMeshPtr == '.' && *MeshNamePtr == '_';

		if (*MeshNamePtr != *FbxMeshPtr && !bIsPointAndUnderscore)
		{
			break;
		}
		else
		{
			i++;
			MeshNamePtr--;
			FbxMeshPtr--;
		}
	}

	if (i == FbxNodeNameLen) // matched
	{
		if (FbxNodeNameLen == MeshNameLen) // the name of Unreal mesh is full match
		{
			return 0;
		}
		// The name of Unreal mesh has a prefix
		if (*MeshNamePtr == '_')
		{
			return (MeshNameLen - i);
		}
	}
	return INDEX_NONE;
}

FbxNode* FFbxImporter::GetMeshNodesFromName(const FString& ReimportMeshName, TArray<FbxNode*>& FbxMeshArray)
{
	char MeshName[2048];
	FCStringAnsi::Strncpy(MeshName, TCHAR_TO_UTF8(*ReimportMeshName), UE_ARRAY_COUNT(MeshName));

	// find the Fbx mesh node that the Unreal Mesh matches according to name
	int32 BestMatchIndex = INDEX_NONE;
	int32 BestDiffScore = MAX_int32;
	for (int32 MeshIndex = 0; MeshIndex < FbxMeshArray.Num(); MeshIndex++)
	{
		int32 DiffScore = GetNodeNameDiffScore(MeshName, FbxMeshArray[MeshIndex]);
		if (DiffScore != INDEX_NONE && DiffScore < BestDiffScore)
		{
			BestMatchIndex = MeshIndex;
			BestDiffScore = DiffScore;
		}
	}
	return FbxMeshArray.IsValidIndex(BestMatchIndex) ? FbxMeshArray[BestMatchIndex] : nullptr;
}

FbxAMatrix FFbxImporter::ComputeSkeletalMeshTotalMatrix(FbxNode* Node, FbxNode *RootSkeletalNode)
{
	if (ImportOptions->bImportScene && !ImportOptions->bTransformVertexToAbsolute && RootSkeletalNode != nullptr && RootSkeletalNode != Node)
	{
		FbxAMatrix GlobalTransform = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node);
		FbxAMatrix GlobalSkeletalMeshRootTransform = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(RootSkeletalNode);
		FbxAMatrix TotalMatrix = GlobalSkeletalMeshRootTransform.Inverse() * GlobalTransform;
		return TotalMatrix;
	}
	return ComputeTotalMatrix(Node);
}

FbxAMatrix FFbxImporter::ComputeTotalMatrix(FbxNode* Node)
{
	FbxAMatrix Geometry;
	FbxVector4 Translation, Rotation, Scaling;
	Translation = Node->GetGeometricTranslation(FbxNode::eSourcePivot);
	Rotation = Node->GetGeometricRotation(FbxNode::eSourcePivot);
	Scaling = Node->GetGeometricScaling(FbxNode::eSourcePivot);
	Geometry.SetT(Translation);
	Geometry.SetR(Rotation);
	Geometry.SetS(Scaling);

	//For Single Matrix situation, obtain transfrom matrix from eDESTINATION_SET, which include pivot offsets and pre/post rotations.
	FbxAMatrix& GlobalTransform = Scene->GetAnimationEvaluator()->GetNodeGlobalTransform(Node);
	
	//We can bake the pivot only if we don't transform the vertex to the absolute position
	if (!ImportOptions->bTransformVertexToAbsolute)
	{
		if (ImportOptions->bBakePivotInVertex)
		{
			FbxAMatrix PivotGeometry;
			FbxVector4 RotationPivot = Node->GetRotationPivot(FbxNode::eSourcePivot);
			FbxVector4 FullPivot;
			FullPivot[0] = -RotationPivot[0];
			FullPivot[1] = -RotationPivot[1];
			FullPivot[2] = -RotationPivot[2];
			PivotGeometry.SetT(FullPivot);
			Geometry = Geometry * PivotGeometry;
		}
		else
		{
			//No Vertex transform and no bake pivot, it will be the mesh as-is.
			Geometry.SetIdentity();
		}
	}
	//We must always add the geometric transform. Only Max use the geometric transform which is an offset to the local transform of the node
	FbxAMatrix TotalMatrix = ImportOptions->bTransformVertexToAbsolute ? GlobalTransform * Geometry : Geometry;

	return TotalMatrix;
}

bool FFbxImporter::IsOddNegativeScale(FbxAMatrix& TotalMatrix)
{
	FbxVector4 Scale = TotalMatrix.GetS();
	int32 NegativeNum = 0;

	if (Scale[0] < 0) NegativeNum++;
	if (Scale[1] < 0) NegativeNum++;
	if (Scale[2] < 0) NegativeNum++;

	return NegativeNum == 1 || NegativeNum == 3;
}

/**
* Recursively get skeletal mesh count
*
* @param Node Root node to find skeletal meshes
* @return int32 skeletal mesh count
*/
int32 GetFbxSkeletalMeshCount(FbxNode* Node)
{
	int32 SkeletalMeshCount = 0;
	if (Node->GetMesh() && (Node->GetMesh()->GetDeformerCount(FbxDeformer::eSkin)>0))
	{
		SkeletalMeshCount = 1;
	}

	int32 ChildIndex;
	for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
	{
		SkeletalMeshCount += GetFbxSkeletalMeshCount(Node->GetChild(ChildIndex));
	}

	return SkeletalMeshCount;
}

/**
* Get mesh count (including static mesh and skeletal mesh, except collision models) and find collision models
*
* @param Node Root node to find meshes
* @return int32 mesh count
*/
int32 FFbxImporter::GetFbxMeshCount( FbxNode* Node, bool bCountLODs, int32& OutNumLODGroups )
{
	// Is this node an LOD group
	bool bLODGroup = Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup;
	
	if( bLODGroup )
	{
		++OutNumLODGroups;
	}
	int32 MeshCount = 0;
	// Don't count LOD group nodes unless we are ignoring them
	if( !bLODGroup || bCountLODs )
	{
		if (Node->GetMesh())
		{
			if (!FillCollisionModelList(Node))
			{
				MeshCount = 1;
			}
		}

		int32 ChildIndex;
		for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
		{
			MeshCount += GetFbxMeshCount(Node->GetChild(ChildIndex),bCountLODs,OutNumLODGroups);
		}
	}
	else
	{
		// An LOD group should count as one mesh
		MeshCount = 1;
	}

	return MeshCount;
}

/**
* Fill the collision models array by going through all mesh node recursively
*
* @param Node Root node to find collision meshes
*/
void FFbxImporter::FillFbxCollisionMeshArray(FbxNode* Node)
{
	if (Node->GetMesh())
	{
		FillCollisionModelList(Node);
	}

	int32 ChildIndex;
	for (ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
	{
		FillFbxCollisionMeshArray(Node->GetChild(ChildIndex));
	}
}

/**
* Get all Fbx mesh objects
*
* @param Node Root node to find meshes
* @param outMeshArray return Fbx meshes
*/
void FFbxImporter::FillFbxMeshArray(FbxNode* Node, TArray<FbxNode*>& outMeshArray, UnFbx::FFbxImporter* FFbxImporter)
{
	if (Node->GetMesh())
	{
		if (!FFbxImporter->FillCollisionModelList(Node) && Node->GetMesh()->GetPolygonVertexCount() > 0)
		{ 
			outMeshArray.Add(Node);
		}
	}

	int32 ChildIndex;
	for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
	{
		FillFbxMeshArray(Node->GetChild(ChildIndex), outMeshArray, FFbxImporter);
	}
}

void FFbxImporter::FillFbxMeshAndLODGroupArray(FbxNode* Node, TArray<FbxNode*>& outLODGroupArray, TArray<FbxNode*>& outMeshArray)
{
	// Is this node an LOD group
	bool bLODGroup = Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup;

	if (bLODGroup)
	{
		outLODGroupArray.Add(Node);
		//Do not do LOD group childrens
		return;
	}
	
	if (Node->GetMesh())
	{
		if (!FillCollisionModelList(Node) && Node->GetMesh()->GetPolygonVertexCount() > 0)
		{
			outMeshArray.Add(Node);
		}
	}
	
	// Cycle the childrens
	int32 ChildIndex;
	for (ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
	{
		FillFbxMeshAndLODGroupArray(Node->GetChild(ChildIndex), outLODGroupArray, outMeshArray);
	}
}

/**
* Get all Fbx skeletal mesh objects
*
* @param Node Root node to find skeletal meshes
* @param outSkelMeshArray return Fbx meshes
*/
void FillFbxSkelMeshArray(FbxNode* Node, TArray<FbxNode*>& outSkelMeshArray)
{
	if (Node->GetMesh() && Node->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) > 0 )
	{
		outSkelMeshArray.Add(Node);
	}

	int32 ChildIndex;
	for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
	{
		FillFbxSkelMeshArray(Node->GetChild(ChildIndex), outSkelMeshArray);
	}
}

void FFbxImporter::ValidateAllMeshesAreReferenceByNodeAttribute()
{
	TSet< FbxUInt64 > NodeGeometryIds;
	NodeGeometryIds.Reserve( Scene->GetNodeCount() );

	for (int NodeIndex = 0; NodeIndex < Scene->GetNodeCount(); ++NodeIndex)
	{
		FbxNode* SceneNode = Scene->GetNode(NodeIndex);
		FbxGeometry* NodeGeometry = static_cast<FbxGeometry*>(SceneNode->GetMesh());

		if ( NodeGeometry )
		{
			NodeGeometryIds.Add( NodeGeometry->GetUniqueID() );
		}
	}

	for (int GeoIndex = 0; GeoIndex < Scene->GetGeometryCount(); ++GeoIndex)
	{
		FbxGeometry* Geometry = Scene->GetGeometry(GeoIndex);

		if ( !NodeGeometryIds.Contains( Geometry->GetUniqueID() ) )
		{
			FString GeometryName = (Geometry->GetName() && Geometry->GetName()[0] != '\0') ? UTF8_TO_TCHAR(Geometry->GetName()) : TEXT("[Geometry have no name]");
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning,
				FText::Format(LOCTEXT("FailedToImport_NoObjectLinkToNode", "Mesh {0} in the fbx file is not reference by any hierarchy node."), FText::FromString(GeometryName))),
				FFbxErrors::Generic_ImportingNewObjectFailed);
		}
	}
}

void FFbxImporter::ConvertLodPrefixToLodGroup()
{
	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
	IMeshReduction* SkeletalMeshReduction = ReductionModule.GetSkeletalMeshReductionInterface();
	IMeshReduction* StaticMeshReduction = ReductionModule.GetStaticMeshReductionInterface();
	bool bCanReduce = true;
	bool bWarnUserNoReduction = false;
	if (ImportOptions->ImportType == FBXIT_SkeletalMesh && !SkeletalMeshReduction)
	{
		bCanReduce = false;
	}

	if (ImportOptions->ImportType == FBXIT_StaticMesh && !StaticMeshReduction)
	{
		bCanReduce = false;
	}

	const FString LodPrefix = TEXT("LOD");
	TMap<FString, TArray<uint64>> LodPrefixNodeMap;
	TMap<uint64, FbxNode*> NodeMap;
	for (int NodeIndex = 0; NodeIndex < Scene->GetNodeCount(); ++NodeIndex)
	{
		FbxNode *SceneNode = Scene->GetNode(NodeIndex);
		if (SceneNode == nullptr)
		{
			continue;
		}
		FbxGeometry *NodeGeometry = static_cast<FbxGeometry*>(SceneNode->GetMesh());
		if (NodeGeometry && NodeGeometry->GetUniqueID() != SceneNode->GetUniqueID())
		{
			FString SceneNodeName = UTF8_TO_TCHAR(SceneNode->GetName());
			if (SceneNodeName.Len() > 5 && SceneNodeName.StartsWith(LodPrefix, ESearchCase::CaseSensitive) && SceneNodeName[4] == '_')
			{
				FString LODXNumber = SceneNodeName.RightChop(3).Left(1);
				if (LODXNumber.IsNumeric())
				{
					NodeMap.FindOrAdd(SceneNode->GetUniqueID()) = SceneNode;
					int32 LodNumber = FPlatformString::Atoi(*FString(&SceneNodeName[3]));

					FString MatchName = SceneNodeName.RightChop(5);
					if (SceneNode->GetParent())
					{
						uint64 ParentUniqueID = SceneNode->GetParent()->GetUniqueID();
						FString ParentID = FString::FromInt((int32)ParentUniqueID);
						if (ParentUniqueID > MAX_int32)
						{
							ParentID = FString::FromInt((int32)(ParentUniqueID >> 32)) + FString::FromInt((int32)ParentUniqueID);
						}
						MatchName += TEXT("_") + ParentID;
					}
					TArray<uint64>& LodPrefixNodeValues = LodPrefixNodeMap.FindOrAdd(MatchName);
					//Add LOD in the correct order
					if (LodNumber >= LodPrefixNodeValues.Num())
					{
						int32 AddCount = LodNumber + 1 - LodPrefixNodeValues.Num();
						for (int32 AddIndex = 0; AddIndex < AddCount; ++AddIndex)
						{
							LodPrefixNodeValues.Add(MAX_uint64);
						}
					}
					LodPrefixNodeValues[LodNumber] = SceneNode->GetUniqueID();
				}
			}
		}
	}
	
	for (const auto& Kvp : LodPrefixNodeMap)
	{
		if (Kvp.Value.Num() <= 1)
		{
			continue;
		}
		//Find the first valid node to be able to discover the parent of this LOD Group
		const TArray<uint64>& LodGroupNodes = Kvp.Value;
		FbxNode* FirstNode = nullptr;
		int32 ValidNodeCount = 0;
		for (int CurrentLodIndex = 0; CurrentLodIndex < LodGroupNodes.Num(); ++CurrentLodIndex)
		{
			if (LodGroupNodes[CurrentLodIndex] != MAX_uint64)
			{
				if (FirstNode == nullptr)
				{
					FirstNode = NodeMap[LodGroupNodes[CurrentLodIndex]];
				}
				ValidNodeCount++;
			}
		}
		//Do not create LODGroup with less then two child
		if (ValidNodeCount <= 1)
		{
			continue;
		}
		check(FirstNode != nullptr);
		//Set the parent node, we assume all node in LodGroupNodes have the same parent
		FbxNode* ParentNode = FirstNode->GetParent() == nullptr ? Scene->GetRootNode() : FirstNode->GetParent();
		if (ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		{
			//LODGroup already exist no need to create one
			continue;
		}

		//Get a valid name for the LODGroup actor
		FString FbxNodeName = UTF8_TO_TCHAR(FirstNode->GetName());
		FbxNodeName.RightChopInline(5, EAllowShrinking::No);
		FbxNodeName += TEXT("_LodGroup");
		//Create a LodGroup and child all fbx node to the Group
		FbxNode* ActorNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxNodeName));
		FString FbxLODGroupName = FbxNodeName + TEXT("Attribute");
		FbxLODGroup *FbxLodGroupAttribute = FbxLODGroup::Create(Scene, TCHAR_TO_UTF8(*FbxLODGroupName));
		ActorNode->AddNodeAttribute(FbxLodGroupAttribute);

		for (int CurrentLodIndex = 0; CurrentLodIndex < LodGroupNodes.Num(); ++CurrentLodIndex)
		{
			if (LodGroupNodes[CurrentLodIndex] == MAX_uint64)
			{
				if (bCanReduce)
				{
					FString FbxGeneratedNodeName = UTF8_TO_TCHAR(FirstNode->GetName());
					FbxGeneratedNodeName.RightChopInline(5, EAllowShrinking::No);
					FbxGeneratedNodeName += TEXT(GeneratedLODNameSuffix) + FString::FromInt(CurrentLodIndex);
					//Generated LOD add dummy FbxNode to tell the import to add such a LOD
					FbxNode* DummyGeneratedLODActorNode = FbxNode::Create(Scene, TCHAR_TO_UTF8(*FbxGeneratedNodeName));
					ActorNode->AddChild(DummyGeneratedLODActorNode);
				}
				else
				{
					bWarnUserNoReduction = true;
				}
				continue;
			}
			FbxNode* CurrentNode = NodeMap[LodGroupNodes[CurrentLodIndex]];
			if (CurrentNode->GetParent() != nullptr)
			{
				//All parent should be the same for a LOD group
				check(ParentNode == CurrentNode->GetParent());
				ParentNode->RemoveChild(CurrentNode);
			}
			ActorNode->AddChild(CurrentNode);
		}
		//We must have a parent node
		check(ParentNode != nullptr);
		ParentNode->AddChild(ActorNode);
	}

	if (bWarnUserNoReduction)
	{
		FText WarningMessage;
		if (ImportOptions->ImportType == FBXIT_SkeletalMesh && !SkeletalMeshReduction)
		{
			WarningMessage = FText(LOCTEXT("FBX_ImportSkeletalMeshNoReductionModule", "No skeletal mesh reduction module available. Cannot add generated LOD between fbx node LOD prefix."));
		}

		if (ImportOptions->ImportType == FBXIT_StaticMesh && !StaticMeshReduction)
		{
			WarningMessage = FText(LOCTEXT("FBX_ImportStaticMeshNoReductionModule", "No static mesh reduction module available. Cannot add generated LOD between fbx node LOD prefix."));
		}

		AddTokenizedErrorMessage( FTokenizedMessage::Create( EMessageSeverity::Warning, WarningMessage ), FFbxErrors::Generic_Mesh_NoReductionModuleAvailable );
	}
}

FbxNode *FFbxImporter::RecursiveGetFirstMeshNode(FbxNode* Node, FbxNode* NodeToFind)
{
	if (Node == nullptr)
	{
		return nullptr;
	}
	if(Node->GetMesh() != nullptr)
		return Node;
	for (int32 ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
	{
		FbxNode *MeshNode = RecursiveGetFirstMeshNode(Node->GetChild(ChildIndex), NodeToFind);
		if (NodeToFind == nullptr)
		{
			if (MeshNode != nullptr)
			{
				return MeshNode;
			}
		}
		else if (MeshNode == NodeToFind)
		{
			return MeshNode;
		}
	}
	return nullptr;
}

void FFbxImporter::RecursiveGetAllMeshNode(TArray<FbxNode *> &OutAllNode, FbxNode* Node)
{
	if (Node == nullptr)
	{
		return;
	}

	if (Node->GetMesh() != nullptr)
	{
		OutAllNode.Add(Node);
		return;
	}

	//Look if its a generated LOD
	FString FbxGeneratedNodeName = UTF8_TO_TCHAR(Node->GetName());
	if (FbxGeneratedNodeName.Contains(TEXT(GeneratedLODNameSuffix)))
	{
		FString SuffixSearch = TEXT(GeneratedLODNameSuffix);
		int32 SuffixIndex = FbxGeneratedNodeName.Find(SuffixSearch, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		SuffixIndex += SuffixSearch.Len();
		FString LODXNumber = FbxGeneratedNodeName.RightChop(SuffixIndex).Left(1);
		if (LODXNumber.IsNumeric())
		{
			OutAllNode.Add(Node);
			return;
		}
	}

	for (int32 ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
	{
		RecursiveGetAllMeshNode(OutAllNode, Node->GetChild(ChildIndex));
	}
}

FbxNode* FFbxImporter::FindLODGroupNode(FbxNode* NodeLodGroup, int32 LodIndex, FbxNode *NodeToFind)
{
	check(NodeLodGroup->GetChildCount() >= LodIndex);
	FbxNode *ChildNode = NodeLodGroup->GetChild(LodIndex);
	if (ChildNode == nullptr)
	{
		return nullptr;
	}
	return RecursiveGetFirstMeshNode(ChildNode, NodeToFind);
}

void FFbxImporter::FindAllLODGroupNode(TArray<FbxNode*> &OutNodeInLod, FbxNode* NodeLodGroup, int32 LodIndex)
{
	check(NodeLodGroup->GetChildCount() >= LodIndex);
	FbxNode *ChildNode = NodeLodGroup->GetChild(LodIndex);
	if (ChildNode == nullptr)
	{
		return;
	}
	RecursiveGetAllMeshNode(OutNodeInLod, ChildNode);
}

FbxNode *FFbxImporter::RecursiveFindParentLodGroup(FbxNode *ParentNode)
{
	if (ParentNode == nullptr)
		return nullptr;
	if (ParentNode->GetNodeAttribute() && ParentNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
		return ParentNode;
	return RecursiveFindParentLodGroup(ParentNode->GetParent());
}

void FFbxImporter::RecursiveFixSkeleton(FbxNode* Node, TArray<FbxNode*> &SkelMeshes, bool bImportNestedMeshes )
{
	FbxNodeAttribute* Attr = Node->GetNodeAttribute();
	bool NodeIsLodGroup = (Attr && (Attr->GetAttributeType() == FbxNodeAttribute::eLODGroup));
	if (!NodeIsLodGroup)
	{
		for (int32 i = 0; i < Node->GetChildCount(); i++)
		{
			RecursiveFixSkeleton(Node->GetChild(i), SkelMeshes, bImportNestedMeshes);
		}
	}

	if ( Attr && (Attr->GetAttributeType() == FbxNodeAttribute::eMesh || Attr->GetAttributeType() == FbxNodeAttribute::eNull ) )
	{
		if( bImportNestedMeshes  && Attr->GetAttributeType() == FbxNodeAttribute::eMesh )
		{
			// for leaf mesh, keep them as mesh
			int32 ChildCount = Node->GetChildCount();
			int32 ChildIndex;
			for (ChildIndex = 0; ChildIndex < ChildCount; ChildIndex++)
			{
				FbxNode* Child = Node->GetChild(ChildIndex);
				if (Child->GetMesh() == NULL)
				{
					break;
				}
			}

			if (ChildIndex != ChildCount)
			{
				// Remove from the mesh list it is no longer a mesh
				SkelMeshes.Remove(Node);

				//replace with skeleton
				FbxSkeleton* lSkeleton = FbxSkeleton::Create(SdkManager,"");
				Node->SetNodeAttribute(lSkeleton);
				lSkeleton->SetSkeletonType(FbxSkeleton::eLimbNode);
			}
			else // this mesh may be not in skeleton mesh list. If not, add it.
			{
				if( !SkelMeshes.Contains( Node ) )
				{
					SkelMeshes.Add(Node);
				}
			}
		}
		else
		{
			// Remove from the mesh list it is no longer a mesh
			SkelMeshes.Remove(Node);
	
			//replace with skeleton
			FbxSkeleton* lSkeleton = FbxSkeleton::Create(SdkManager,"");
			Node->SetNodeAttribute(lSkeleton);
			lSkeleton->SetSkeletonType(FbxSkeleton::eLimbNode);
		}
	}
}

FbxNode* FFbxImporter::GetRootSkeleton(FbxNode* Link)
{
	FbxNode* RootBone = Link;

	// get Unreal skeleton root
	// mesh and dummy are used as bone if they are in the skeleton hierarchy
	while (RootBone && RootBone->GetParent())
	{
		bool bIsBlenderArmatureBone = false;
		if (FbxCreator == EFbxCreator::Blender)
		{
			//Hack to support armature dummy node from blender
			//Users do not want the null attribute node named armature which is the parent of the real root bone in blender fbx file
			//This is a hack since if a rigid mesh group root node is named "armature" it will be skip
			const FString RootBoneParentName(RootBone->GetParent()->GetName());
			FbxNode *GrandFather = RootBone->GetParent()->GetParent();
			bIsBlenderArmatureBone = (GrandFather == nullptr || GrandFather == Scene->GetRootNode()) && (RootBoneParentName.Compare(TEXT("armature"), ESearchCase::IgnoreCase) == 0);
		}

		FbxNodeAttribute* Attr = RootBone->GetParent()->GetNodeAttribute();
		if (Attr && 
			(Attr->GetAttributeType() == FbxNodeAttribute::eMesh || 
			 (Attr->GetAttributeType() == FbxNodeAttribute::eNull && !bIsBlenderArmatureBone) ||
			 Attr->GetAttributeType() == FbxNodeAttribute::eSkeleton) &&
			RootBone->GetParent() != Scene->GetRootNode())
		{
			// in some case, skeletal mesh can be ancestor of bones
			// this avoids this situation
			if (Attr->GetAttributeType() == FbxNodeAttribute::eMesh )
			{
				FbxMesh* Mesh = (FbxMesh*)Attr;
				if (Mesh->GetDeformerCount(FbxDeformer::eSkin) > 0)
				{
					break;
				}
			}

			RootBone = RootBone->GetParent();
		}
		else
		{
			break;
		}
	}

	return RootBone;
}

 
void FFbxImporter::ApplyTransformSettingsToFbxNode(FbxNode* Node, UFbxAssetImportData* AssetData)
{
	check(Node);
	check(AssetData);
	
	if (TransformSettingsToFbxApply.Contains(Node))
	{
		return;
	}
	TransformSettingsToFbxApply.Add(Node);

	FbxAMatrix TransformMatrix;
	BuildFbxMatrixForImportTransform(TransformMatrix, AssetData);

	FbxDouble3 ExistingTranslation = Node->LclTranslation.Get();
	FbxDouble3 ExistingRotation = Node->LclRotation.Get();
	FbxDouble3 ExistingScaling = Node->LclScaling.Get();

	// A little slower to build up this information from the matrix, but it means we get
	// the same result across all import types, as other areas can use the matrix directly
	FbxVector4 AddedTranslation = TransformMatrix.GetT();
	FbxVector4 AddedRotation = TransformMatrix.GetR();
	FbxVector4 AddedScaling = TransformMatrix.GetS();

	FbxDouble3 NewTranslation = FbxDouble3(ExistingTranslation[0] + AddedTranslation[0], ExistingTranslation[1] + AddedTranslation[1], ExistingTranslation[2] + AddedTranslation[2]);
	FbxDouble3 NewRotation = FbxDouble3(ExistingRotation[0] + AddedRotation[0], ExistingRotation[1] + AddedRotation[1], ExistingRotation[2] + AddedRotation[2]);
	FbxDouble3 NewScaling = FbxDouble3(ExistingScaling[0] * AddedScaling[0], ExistingScaling[1] * AddedScaling[1], ExistingScaling[2] * AddedScaling[2]);

	Node->LclTranslation.Set(NewTranslation);
	Node->LclRotation.Set(NewRotation);
	Node->LclScaling.Set(NewScaling);
	//Reset all the transform evaluation cache since we change some node transform
	Scene->GetAnimationEvaluator()->Reset();
}


void FFbxImporter::RemoveTransformSettingsFromFbxNode(FbxNode* Node, UFbxAssetImportData* AssetData)
{
	check(Node);
	check(AssetData);

	if (!TransformSettingsToFbxApply.Contains(Node))
	{
		return;
	}
	TransformSettingsToFbxApply.Remove(Node);

	FbxAMatrix TransformMatrix;
	BuildFbxMatrixForImportTransform(TransformMatrix, AssetData);

	FbxDouble3 ExistingTranslation = Node->LclTranslation.Get();
	FbxDouble3 ExistingRotation = Node->LclRotation.Get();
	FbxDouble3 ExistingScaling = Node->LclScaling.Get();

	// A little slower to build up this information from the matrix, but it means we get
	// the same result across all import types, as other areas can use the matrix directly
	FbxVector4 AddedTranslation = TransformMatrix.GetT();
	FbxVector4 AddedRotation = TransformMatrix.GetR();
	FbxVector4 AddedScaling = TransformMatrix.GetS();

	FbxDouble3 NewTranslation = FbxDouble3(ExistingTranslation[0] - AddedTranslation[0], ExistingTranslation[1] - AddedTranslation[1], ExistingTranslation[2] - AddedTranslation[2]);
	FbxDouble3 NewRotation = FbxDouble3(ExistingRotation[0] - AddedRotation[0], ExistingRotation[1] - AddedRotation[1], ExistingRotation[2] - AddedRotation[2]);
	FbxDouble3 NewScaling = FbxDouble3(ExistingScaling[0] / AddedScaling[0], ExistingScaling[1] / AddedScaling[1], ExistingScaling[2] / AddedScaling[2]);

	Node->LclTranslation.Set(NewTranslation);
	Node->LclRotation.Set(NewRotation);
	Node->LclScaling.Set(NewScaling);
	//Reset all the transform evaluation cache since we change some node transform
	Scene->GetAnimationEvaluator()->Reset();
}


void FFbxImporter::BuildFbxMatrixForImportTransform(FbxAMatrix& OutMatrix, UFbxAssetImportData* AssetData)
{
	if(!AssetData)
	{
		OutMatrix.SetIdentity();
		return;
	}

	FbxVector4 FbxAddedTranslation = Converter.ConvertToFbxPos(AssetData->ImportTranslation);
	FbxVector4 FbxAddedScale = Converter.ConvertToFbxScale(FVector(AssetData->ImportUniformScale));
	FbxVector4 FbxAddedRotation = Converter.ConvertToFbxRot(AssetData->ImportRotation.Euler());
	
	OutMatrix = FbxAMatrix(FbxAddedTranslation, FbxAddedRotation, FbxAddedScale);
}

/**
* Get all Fbx skeletal mesh objects which are grouped by skeleton they bind to
*
* @param Node Root node to find skeletal meshes
* @param outSkelMeshArray return Fbx meshes they are grouped by skeleton
* @param SkeletonArray
* @param ExpandLOD flag of expanding LOD to get each mesh
*/
void FFbxImporter::RecursiveFindFbxSkelMesh(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, TArray<FbxNode*>& SkeletonArray, bool ExpandLOD)
{
	FbxNode* SkelMeshNode = nullptr;
	FbxNode* NodeToAdd = Node;

    if (Node->GetMesh() && Node->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) > 0 )
	{
		SkelMeshNode = Node;
	}
	else if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
	{
		// for LODgroup, add the LODgroup to OutSkelMeshArray according to the skeleton that the first child bind to
		SkelMeshNode = FindLODGroupNode(Node, 0);
		// check if the first child is skeletal mesh
		if (SkelMeshNode != nullptr && !(SkelMeshNode->GetMesh() && SkelMeshNode->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) > 0))
		{
			SkelMeshNode = nullptr;
		}
		else if (ExpandLOD)
		{
			// if ExpandLOD is true, only add the first LODGroup level node
			NodeToAdd = SkelMeshNode;
		}		
		// else NodeToAdd = Node;
	}

	if (SkelMeshNode)
	{
		// find root skeleton

		check(SkelMeshNode->GetMesh() != nullptr);
		const int32 fbxDeformerCount = SkelMeshNode->GetMesh()->GetDeformerCount();
		FbxSkin* Deformer = static_cast<FbxSkin*>( SkelMeshNode->GetMesh()->GetDeformer(0, FbxDeformer::eSkin) );
		
		if (Deformer != NULL )
		{
			int32 ClusterCount = Deformer->GetClusterCount();
			bool bFoundCorrectLink = false;
			for (int32 ClusterId = 0; ClusterId < ClusterCount; ++ClusterId)
			{
				FbxNode* RootBoneLink = Deformer->GetCluster(ClusterId)->GetLink(); //Get the bone influences by this first cluster
				RootBoneLink = GetRootSkeleton(RootBoneLink); // Get the skeleton root itself

				if (RootBoneLink)
				{
					bool bAddedToExistingSkeleton = false;
					for (int32 SkeletonIndex = 0; SkeletonIndex < SkeletonArray.Num(); ++SkeletonIndex)
					{
						if (RootBoneLink == SkeletonArray[SkeletonIndex])
						{
							// append to existed outSkelMeshArray element
							TArray<FbxNode*>* TempArray = outSkelMeshArray[SkeletonIndex];
							TempArray->Add(NodeToAdd);
							bAddedToExistingSkeleton = true;
							break;
						}
					}

					// if there is no outSkelMeshArray element that is bind to this skeleton
					// create new element for outSkelMeshArray
					if (!bAddedToExistingSkeleton)
					{
						TArray<FbxNode*>* TempArray = new TArray<FbxNode*>();
						TempArray->Add(NodeToAdd);
						outSkelMeshArray.Add(TempArray);
						SkeletonArray.Add(RootBoneLink);
						
						if (ImportOptions->bImportScene && !ImportOptions->bTransformVertexToAbsolute)
						{
							FbxVector4 NodeScaling = NodeToAdd->EvaluateLocalScaling();
							FbxVector4 NoScale(1.0, 1.0, 1.0);
							if (NodeScaling != NoScale)
							{
								//Scene import cannot import correctly a skeletal mesh with a root node containing scale
								//Warn the user is skeletal mesh can be wrong
								AddTokenizedErrorMessage(
									FTokenizedMessage::Create(
										EMessageSeverity::Warning,
										FText::Format(LOCTEXT("FBX_ImportSceneSkeletalMeshRootNodeScaling", "Importing skeletal mesh {0} that dont have a mesh node with no scale is not supported when doing an import scene."), FText::FromString(UTF8_TO_TCHAR(NodeToAdd->GetName())))
										),
									FFbxErrors::SkeletalMesh_InvalidRoot
									);
							}
						}
					}

					bFoundCorrectLink = true;
					break;
				}
			}
			
			// we didn't find the correct link
			if (!bFoundCorrectLink)
			{
				AddTokenizedErrorMessage(
					FTokenizedMessage::Create(
						EMessageSeverity::Warning, 
						FText::Format( LOCTEXT("FBX_NoWeightsOnDeformer", "Ignoring mesh {0} because it has no weights."), FText::FromString( UTF8_TO_TCHAR(SkelMeshNode->GetName()) ) )
					), 
					FFbxErrors::SkeletalMesh_NoWeightsOnDeformer
				);
			}
		}
	}

	//Skeletalmesh node can have child so let's always iterate trough child
	{
		TArray<FbxNode*> ChildScaled;
		//Sort the node to have the one with no scaling first so we have more chance
		//to have a root skeletal mesh with no scale. Because scene import do not support
		//root skeletal mesh containing scale
		for (int32 ChildIndex = 0; ChildIndex < Node->GetChildCount(); ++ChildIndex)
		{
			FbxNode *ChildNode = Node->GetChild(ChildIndex);

			if(!Node->GetNodeAttribute() || Node->GetNodeAttribute()->GetAttributeType() != FbxNodeAttribute::eLODGroup)
			{
				FbxVector4 NoScale(1.0, 1.0, 1.0);

				if(ChildNode->EvaluateLocalScaling() == NoScale)
				{
					RecursiveFindFbxSkelMesh(ChildNode, outSkelMeshArray, SkeletonArray, ExpandLOD);
				}
				else
				{
					ChildScaled.Add(ChildNode);
				}
			}
		}

		for (FbxNode *ChildNode : ChildScaled)
		{
			RecursiveFindFbxSkelMesh(ChildNode, outSkelMeshArray, SkeletonArray, ExpandLOD);
		}
	}
}

void FFbxImporter::RecursiveFindRigidMesh(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, TArray<FbxNode*>& SkeletonArray, bool ExpandLOD)
{
	bool bRigidNodeFound = false;
	FbxNode* RigidMeshNode = nullptr;

	DEBUG_FBX_NODE("", Node);

	if (Node->GetMesh())
	{
		// ignore skeletal mesh
		if (Node->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) == 0 )
		{
			RigidMeshNode = Node;
			bRigidNodeFound = true;
		}
	}
	else if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
	{
		// for LODgroup, add the LODgroup to OutSkelMeshArray according to the skeleton that the first child bind to
		FbxNode* FirstLOD = FindLODGroupNode(Node, 0);
		// check if the first child is skeletal mesh
		if (FirstLOD != nullptr && FirstLOD->GetMesh())
		{
			if (FirstLOD->GetMesh()->GetDeformerCount(FbxDeformer::eSkin) == 0 )
			{
				bRigidNodeFound = true;
			}
		}

		if (bRigidNodeFound)
		{
			if (ExpandLOD)
			{
				RigidMeshNode = FirstLOD;
			}
			else
			{
				RigidMeshNode = Node;
			}

		}
	}

	if (bRigidNodeFound)
	{
		// find root skeleton
		FbxNode* Link = GetRootSkeleton(RigidMeshNode);

		int32 i;
		for (i = 0; i < SkeletonArray.Num(); i++)
		{
			if ( Link == SkeletonArray[i])
			{
				// append to existed outSkelMeshArray element
				TArray<FbxNode*>* TempArray = outSkelMeshArray[i];
				TempArray->Add(RigidMeshNode);
				break;
			}
		}

		// if there is no outSkelMeshArray element that is bind to this skeleton
		// create new element for outSkelMeshArray
		if ( i == SkeletonArray.Num() )
		{
			TArray<FbxNode*>* TempArray = new TArray<FbxNode*>();
			TempArray->Add(RigidMeshNode);
			outSkelMeshArray.Add(TempArray);
			SkeletonArray.Add(Link);
		}
	}

	// for LODGroup, we will not deep in.
	if (!(Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup))
	{
		int32 ChildIndex;
		for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
		{
			RecursiveFindRigidMesh(Node->GetChild(ChildIndex), outSkelMeshArray, SkeletonArray, ExpandLOD);
		}
	}
}

/**
* Get all Fbx skeletal mesh objects in the scene. these meshes are grouped by skeleton they bind to
*
* @param Node Root node to find skeletal meshes
* @param outSkelMeshArray return Fbx meshes they are grouped by skeleton
*/
void FFbxImporter::FillFbxSkelMeshArrayInScene(FbxNode* Node, TArray< TArray<FbxNode*>* >& outSkelMeshArray, bool ExpandLOD, bool bCombineSkeletalMesh, bool bForceFindRigid /*= false*/)
{
	TArray<FbxNode*> SkeletonArray;

	// a) find skeletal meshes
	
	RecursiveFindFbxSkelMesh(Node, outSkelMeshArray, SkeletonArray, ExpandLOD);
	// for skeletal mesh, we convert the skeleton system to skeleton
	// in less we recognize bone mesh as rigid mesh if they are textured
	for ( int32 SkelIndex = 0; SkelIndex < SkeletonArray.Num(); SkelIndex++)
	{
		RecursiveFixSkeleton(SkeletonArray[SkelIndex], *outSkelMeshArray[SkelIndex], ImportOptions->bImportMeshesInBoneHierarchy );
	}

	

	// b) find rigid mesh
	
	// If we are attempting to import a skeletal mesh but we have no hierarchy attempt to find a rigid mesh.
	if (bForceFindRigid || outSkelMeshArray.Num() == 0)
	{
		RecursiveFindRigidMesh(Node, outSkelMeshArray, SkeletonArray, ExpandLOD);
		if (bForceFindRigid)
		{
			//Cleanup the rigid mesh, We want to remove any real static mesh from the outSkelMeshArray
			//Any non skinned mesh that contain no animation should be part of this array.
			int32 AnimStackCount = Scene->GetSrcObjectCount<FbxAnimStack>();
			TArray<int32> SkeletalMeshArrayToRemove;
			for (int32 i = 0; i < outSkelMeshArray.Num(); i++)
			{
				bool bIsValidSkeletal = false;
				TArray<FbxNode*> NodeArray = *outSkelMeshArray[i];
				for (FbxNode *InspectedNode : NodeArray)
				{
					FbxMesh* Mesh = InspectedNode->GetMesh();

					FbxLODGroup* LodGroup = InspectedNode->GetLodGroup();
					if (LodGroup != nullptr)
					{
						FbxNode* SkelMeshNode = FindLODGroupNode(InspectedNode, 0);
						if (SkelMeshNode != nullptr)
						{
							Mesh = SkelMeshNode->GetMesh();
						}
					}

					if (Mesh == nullptr)
					{
						continue;
					}
					if (Mesh->GetDeformerCount(FbxDeformer::eSkin) > 0)
					{
						bIsValidSkeletal = true;
						break;
					}
					//If there is some anim object we count this as a valid skeletal mesh imported as rigid mesh
					for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
					{
						FbxAnimStack* CurAnimStack = Scene->GetSrcObject<FbxAnimStack>(AnimStackIndex);
						// set current anim stack

						Scene->SetCurrentAnimationStack(CurAnimStack);

						FbxTimeSpan AnimTimeSpan(FBXSDK_TIME_INFINITE, FBXSDK_TIME_MINUS_INFINITE);
						InspectedNode->GetAnimationInterval(AnimTimeSpan, CurAnimStack);

						if (AnimTimeSpan.GetDuration() > 0)
						{
							bIsValidSkeletal = true;
							break;
						}
					}
					if (bIsValidSkeletal)
					{
						break;
					}
				}
				if (!bIsValidSkeletal)
				{
					SkeletalMeshArrayToRemove.Add(i);
				}
			}
			for (int32 i = SkeletalMeshArrayToRemove.Num() - 1; i >= 0; --i)
			{
				if (!SkeletalMeshArrayToRemove.IsValidIndex(i) || !outSkelMeshArray.IsValidIndex(SkeletalMeshArrayToRemove[i]))
					continue;
				int32 IndexToRemove = SkeletalMeshArrayToRemove[i];
				outSkelMeshArray[IndexToRemove]->Empty();
				outSkelMeshArray.RemoveAt(IndexToRemove);
			}
		}
	}
	//Empty the skeleton array
	SkeletonArray.Empty();


	if (bCombineSkeletalMesh)
	{
		//Merge all the skeletal mesh arrays into one combine mesh
		TArray<FbxNode*>* CombineNodes = new TArray<FbxNode*>();
		for (TArray<FbxNode*> *Parts : outSkelMeshArray)
		{
			CombineNodes->Append(*Parts);
			delete Parts;
		}
		outSkelMeshArray.Empty(1);
		outSkelMeshArray.Add(CombineNodes);
	}
}

FbxNode* FFbxImporter::FindFBXMeshesByBone(const FName& RootBoneName, bool bExpandLOD, TArray<FbxNode*>& OutFBXMeshNodeArray)
{
	// get the root bone of Unreal skeletal mesh
	const FString BoneNameString = RootBoneName.ToString();

	// we do not need to check if the skeleton root node is a skeleton
	// because the animation may be a rigid animation
	FbxNode* SkeletonRoot = NULL;

	// find the FBX skeleton node according to name
	SkeletonRoot = Scene->FindNodeByName(TCHAR_TO_UTF8(*BoneNameString));

	// SinceFBX bone names are changed on import, it's possible that the 
	// bone name in the engine doesn't match that of the one in the FBX file and
	// would not be found by FindNodeByName().  So apply the same changes to the 
	// names of the nodes before checking them against the name of the Unreal bone
	if (!SkeletonRoot)
	{
		for (int32 NodeIndex = 0; NodeIndex < Scene->GetNodeCount(); NodeIndex++)
		{
			FbxNode* FbxNode = Scene->GetNode(NodeIndex);

			FString FbxBoneName = FSkeletalMeshImportData::FixupBoneName(MakeName(FbxNode->GetName()));

			if (FbxBoneName == BoneNameString)
			{
				SkeletonRoot = FbxNode;
				break;
			}
		}
	}


	// return if do not find matched FBX skeleton
	if (!SkeletonRoot)
	{
		return NULL;
	}
	

	// Get Mesh nodes array that bind to the skeleton system
	// 1, get all skeltal meshes in the FBX file
	TArray< TArray<FbxNode*>* > SkelMeshArray;
	FillFbxSkelMeshArrayInScene(Scene->GetRootNode(), SkelMeshArray, false, ImportOptions->bImportAsSkeletalGeometry || ImportOptions->bImportAsSkeletalSkinning, ImportOptions->bImportScene);

	// 2, then get skeletal meshes that bind to this skeleton
	for (int32 SkelMeshIndex = 0; SkelMeshIndex < SkelMeshArray.Num(); SkelMeshIndex++)
	{
		FbxNode* MeshNode = NULL;
		if((*SkelMeshArray[SkelMeshIndex]).IsValidIndex(0))
		{
			FbxNode* Node = (*SkelMeshArray[SkelMeshIndex])[0];
			if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
			{
				MeshNode = FindLODGroupNode(Node, 0);
			}
			else
			{
				MeshNode = Node;
			}
		}
		
		if( !ensure( MeshNode && MeshNode->GetMesh() ) )
		{
			return NULL;
		}

		// 3, get the root bone that the mesh bind to
		FbxSkin* Deformer = (FbxSkin*)MeshNode->GetMesh()->GetDeformer(0, FbxDeformer::eSkin);
		FbxNode* Link = nullptr;
		// If there is no deformer this is likely rigid animation
		if( Deformer )
		{
			Link = Deformer->GetCluster(0)->GetLink();
			Link = GetRootSkeleton(Link);
		}
		else
		{
			Link = GetRootSkeleton(SkeletonRoot);
		}
		// 4, fill in the mesh node
		if (Link == SkeletonRoot)
		{
			// copy meshes
			if (bExpandLOD)
			{
				TArray<FbxNode*> SkelMeshes = *SkelMeshArray[SkelMeshIndex];
				for (int32 NodeIndex = 0; NodeIndex < SkelMeshes.Num(); NodeIndex++)
				{
					FbxNode* Node = SkelMeshes[NodeIndex];
					if (Node->GetNodeAttribute() && Node->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eLODGroup)
					{
						FbxNode *InnerMeshNode = FindLODGroupNode(Node, 0);
						if (InnerMeshNode != nullptr)
							OutFBXMeshNodeArray.Add(InnerMeshNode);
					}
					else
					{
						OutFBXMeshNodeArray.Add(Node);
					}
				}
			}
			else
			{
				OutFBXMeshNodeArray.Append(*SkelMeshArray[SkelMeshIndex]);
			}
			break;
		}
	}

	for (int32 i = 0; i < SkelMeshArray.Num(); i++)
	{
		delete SkelMeshArray[i];
	}

	return SkeletonRoot;
}

/**
* Get the first Fbx mesh node.
*
* @param Node Root node
* @param bIsSkelMesh if we want a skeletal mesh
* @return FbxNode* the node containing the first mesh
*/
FbxNode* GetFirstFbxMesh(FbxNode* Node, bool bIsSkelMesh)
{
	if (Node->GetMesh())
	{
		if (bIsSkelMesh)
		{
			if (Node->GetMesh()->GetDeformerCount(FbxDeformer::eSkin)>0)
			{
				return Node;
			}
		}
		else
		{
			return Node;
		}
	}

	int32 ChildIndex;
	for (ChildIndex=0; ChildIndex<Node->GetChildCount(); ++ChildIndex)
	{
		FbxNode* FirstMesh;
		FirstMesh = GetFirstFbxMesh(Node->GetChild(ChildIndex), bIsSkelMesh);

		if (FirstMesh)
		{
			return FirstMesh;
		}
	}

	return NULL;
}

void FFbxImporter::CheckSmoothingInfo(FbxMesh* FbxMesh)
{
	if (FbxMesh && bFirstMesh)
	{
		bFirstMesh = false;	 // don't check again
		
		FbxLayer* LayerSmoothing = FbxMesh->GetLayer(0, FbxLayerElement::eSmoothing);
		if (!LayerSmoothing && !GIsAutomationTesting)
		{
			AddTokenizedErrorMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, LOCTEXT("Prompt_NoSmoothgroupForFBXScene", "No smoothing group information was found in this FBX scene.  Please make sure to enable the 'Export Smoothing Groups' option in the FBX Exporter plug-in before exporting the file.  Even for tools that don't support smoothing groups, the FBX Exporter will generate appropriate smoothing data at export-time so that correct vertex normals can be inferred while importing.")), FFbxErrors::Generic_Mesh_NoSmoothingGroup);
		}
	}
}


//-------------------------------------------------------------------------
//
//-------------------------------------------------------------------------
FbxNode* FFbxImporter::RetrieveObjectFromName(const TCHAR* ObjectName, FbxNode* Root)
{
	if (Scene)
	{
		if (!Root)
		{
			Root = Scene->GetRootNode();
		}

		for (int32 ChildIndex=0;ChildIndex<Root->GetChildCount();++ChildIndex)
		{
			FbxNode* Node = Root->GetChild(ChildIndex);
			FbxMesh* FbxMesh = Node->GetMesh();
			if (FbxMesh && 0 == FCString::Strcmp(ObjectName,UTF8_TO_TCHAR(Node->GetName())))
			{
				return Node;
			}

			if (FbxNode* NextNode = RetrieveObjectFromName(ObjectName,Node))
			{
				return NextNode;
			}
		}
	}
	return nullptr;
}

FString GetFbxPropertyStringValue(const FbxProperty& Property)
{
	FString ValueStr(TEXT("Unsupported type"));

	FbxDataType DataType = Property.GetPropertyDataType();
	switch (DataType.GetType())
	{
	case eFbxBool:
	{
		FbxBool BoolValue = Property.Get<FbxBool>();
		ValueStr = LexToString(BoolValue);
	}
	break;
	case eFbxChar:
	{
		FbxChar CharValue = Property.Get<FbxChar>();
		ValueStr = LexToString(CharValue);
	}
	break;
	case eFbxUChar:
	{
		FbxUChar UCharValue = Property.Get<FbxUChar>();
		ValueStr = LexToString(UCharValue);
	}
	break;
	case eFbxShort:
	{
		FbxShort ShortValue = Property.Get<FbxShort>();
		ValueStr = LexToString(ShortValue);
	}
	break;
	case eFbxUShort:
	{
		FbxUShort UShortValue = Property.Get<FbxUShort>();
		ValueStr = LexToString(UShortValue);
	}
	break;
	case eFbxInt:
	{
		FbxInt IntValue = Property.Get<FbxInt>();
		ValueStr = LexToString(IntValue);
	}
	break;
	case eFbxUInt:
	{
		FbxUInt UIntValue = Property.Get<FbxUInt>();
		ValueStr = LexToString(UIntValue);
	}
	break;
	case eFbxEnum:
	{
		FbxEnum EnumValue = Property.Get<FbxEnum>();
		ValueStr = LexToString(EnumValue);
	}
	break;
	case eFbxFloat:
	{
		FbxFloat FloatValue = Property.Get<FbxFloat>();
		ValueStr = LexToString(FloatValue);
	}
	break;
	case eFbxDouble:
	{
		FbxDouble DoubleValue = Property.Get<FbxDouble>();
		ValueStr = LexToString(DoubleValue);
	}
	break;
	case eFbxDouble2:
	{
		FbxDouble2 Vec = Property.Get<FbxDouble2>();
		ValueStr = FString::Printf(TEXT("(%f, %f)"), Vec[0], Vec[1]);
	}
	break;
	case eFbxDouble3:
	{
		FbxDouble3 Vec = Property.Get<FbxDouble3>();
		ValueStr = FString::Printf(TEXT("(%f, %f, %f)"), Vec[0], Vec[1], Vec[2]);
	}
	break;
	case eFbxDouble4:
	{
		FbxDouble4 Vec = Property.Get<FbxDouble4>();
		ValueStr = FString::Printf(TEXT("(%f, %f, %f, %f)"), Vec[0], Vec[1], Vec[2], Vec[3]);
	}
	break;
	case eFbxString:
	{
		FbxString StringValue = Property.Get<FbxString>();
		ValueStr = UTF8_TO_TCHAR(StringValue.Buffer());
	}
	break;
	default:
		break;
	}
	return ValueStr;
}

void FFbxImporter::ImportNodeCustomProperties(UObject* Object, FbxNode* Node, bool bPrefixTagWithNodeName)
{
	if (!Object || !Node)
	{
		return;
	}

	// Import all custom user-defined FBX properties from the FBX node to the object metadata
	FbxProperty CurrentProperty = Node->GetFirstProperty();
	FString NodeName = UTF8_TO_TCHAR(Node->GetName());
	static const FString MetadataPrefix(FBX_METADATA_PREFIX);
	while (CurrentProperty.IsValid())
	{
		if (CurrentProperty.GetFlag(FbxPropertyFlags::eUserDefined))
		{
			// Prefix the FBX metadata tag to make it distinguishable from other metadata
			// so that it can be exportable through FBX export
			FString MetadataTag = UTF8_TO_TCHAR(CurrentProperty.GetName());
			if (bPrefixTagWithNodeName && !MetadataTag.StartsWith(NodeName))
			{
				// Append the node name in the tag since all the metadata will be flattened on the Object
				MetadataTag = NodeName + TEXT(".") + MetadataTag;
			}
			MetadataTag = MetadataPrefix + MetadataTag;

			FString MetadataValue = GetFbxPropertyStringValue(CurrentProperty);
			Object->GetPackage()->GetMetaData().SetValue(Object, *MetadataTag, *MetadataValue);
		}
		CurrentProperty = Node->GetNextProperty(CurrentProperty);
	}

	int NumChildren = Node->GetChildCount();
	for (int i = 0; i < NumChildren; ++i)
	{
		ImportNodeCustomProperties(Object, Node->GetChild(i), bPrefixTagWithNodeName);
	}
}

} // namespace UnFbx

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFbxUnitTest, "Editor.Import.Fbx.UnitTests", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FFbxUnitTest::RunTest(const FString& Parameters)
{
	const ANSICHAR TestData[][64] = { "SimpleTest"
									, "SimpleTest_1"
									, "SimpleTest.1"
									, "SimpleTest,1"
									, "SimpleTest/1"
									, "SimpleTest`1"
									, "SimpleTest%1"
									, "One:SimpleTest"
									, "One::SimpleTest"
									, "One:Two:SimpleTest"
									, "One::Two::SimpleTest"
									, "O.n,e::/Two:%:Si`mpleTest" };

	const FString Results[3] = { FString(TEXT("SimpleTest"))
								, FString(TEXT("SimpleTest_1"))
								, FString(TEXT("Si_mpleTest")) };

	auto RunOneTest = [&TestData, &Results, this](int32 DataIndex, int32 ResultIndex)
	{
		FString ResultInternal = UnFbx::FFbxImporter::MakeName(TestData[DataIndex]);
		if (!ResultInternal.Equals(Results[ResultIndex]))
		{
			FStringFormatOrderedArguments OrderedArguments;
			OrderedArguments.Add(FStringFormatArg(UnFbx::FFbxImporter::MakeString(TestData[DataIndex])));
			OrderedArguments.Add(FStringFormatArg(ResultInternal));
			OrderedArguments.Add(FStringFormatArg(Results[ResultIndex]));
			FString ErrorMessage = FString::Format(TEXT("Fbx importer MakeName function transform name [{0}] into [{1}], but expected [{2}]."), OrderedArguments);
			AddError(ErrorMessage);
		}
	};

	//////////////////////////////////////////////////////////////////////////
	// String with no change test

	RunOneTest(0, 0);

	//////////////////////////////////////////////////////////////////////////
	// Special character tests return all TestData_2

	RunOneTest(1, 1);
	RunOneTest(2, 1);
	RunOneTest(3, 1);
	RunOneTest(4, 1);
	RunOneTest(5, 1);
	RunOneTest(6, 1);

	//////////////////////////////////////////////////////////////////////////
	//namespace tests return all TestData_1

	RunOneTest(7, 0);
	RunOneTest(8, 0);
	RunOneTest(9, 0);
	RunOneTest(10, 0);

	//////////////////////////////////////////////////////////////////////////
	// Mix test

	RunOneTest(11, 2);

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS


#undef LOCTEXT_NAMESPACE
