// Copyright Epic Games, Inc. All Rights Reserved.
#include "FbxMaterialConflictWindow.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "Styling/AppStyle.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Factories/FbxSceneImportFactory.h"

#include "Framework/Application/SlateApplication.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Materials/Material.h"


#define UNMATCHEDCOLOR 0.7f, 0.3f, 0.0f
#define CUSTOMMATCHEDCOLOR 0.16f, 0.82f, 0.17f
#define AUTOMATCHEDCOLOR 0.12f, 0.65f, 1.0f

#define LOCTEXT_NAMESPACE "FBXMaterialConflictWindows"

void SFbxMaterialConflictWindow::Construct(const FArguments& InArgs)
{
	ReturnOption = UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_Cancel;

	WidgetWindow = InArgs._WidgetWindow;
	check(InArgs._SourceMaterials != nullptr);
	check(InArgs._ResultMaterials != nullptr);
	check(InArgs._RemapMaterials != nullptr);
	check(InArgs._AutoRemapMaterials != nullptr);
	SourceMaterials = InArgs._SourceMaterials;
	ResultMaterials = InArgs._ResultMaterials;
	RemapMaterials = InArgs._RemapMaterials;
	AutoRemapMaterials = InArgs._AutoRemapMaterials;
	CustomRemapMaterials.AddZeroed(AutoRemapMaterials->Num());
	bIsPreviewConflict = InArgs._bIsPreviewConflict;
	
	check(RemapMaterials->Num() == AutoRemapMaterials->Num());
	check(RemapMaterials->Num() == ResultMaterials->Num());

	FillMaterialListItem();

	// Material comparison
	TSharedPtr<SWidget> MaterialCompareSection = ConstructMaterialComparison();

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(EOrientation::Orient_Vertical)
		.AlwaysShowScrollbar(false);

	this->ChildSlot
	[
		SNew(SBox)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(2.0f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(2.0f)
							[
								// Material Compare section
								MaterialCompareSection.ToSharedRef()
							]
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f)
				[
					SNew(SButton)
					.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SFbxMaterialConflictWindow::CollapsePreviewVisibility)))
					.HAlign(HAlign_Center)
					.ToolTipText(LOCTEXT("SFbxMaterialConflictWindow_Reset_Tooltip", "Change the material array to reflect the incoming FBX, match the one that fit, keep material instance from the existing data"))
					.Text(LOCTEXT("SFbxMaterialConflictWindow_Reset", "Reset To Fbx"))
					.OnClicked(this, &SFbxMaterialConflictWindow::OnReset)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("SFbxMaterialConflictWindow_Preview_Done", "Done"))
					.OnClicked(this, &SFbxMaterialConflictWindow::OnDone)
				]
			]
		]
	];	
}

TSharedPtr<SWidget> SFbxMaterialConflictWindow::ConstructMaterialComparison()
{
	FText MaterialCompareInstruction = bIsPreviewConflict ? LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareDocumentationPreview", "Material conflict preview mode") : LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareDocumentation", "To fix a material match, right click on the reimport asset material.");
	FText MaterialCompareInstructionTooltip = bIsPreviewConflict ? LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareDocumentationPreviewTooltip", "This is only a conflict preview, the material conflict dialog will show up during import to allow you to fix those conflicts.") : LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareDocumentationTooltip", "To fix a material match, right click on the reimport asset material.\nUse the [Clear] option in the context menu to clear a material match.");
	return SNew(SBox)
	.MaxDesiredHeight(500.0f)
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Text(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareHeader", "Materials"))
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(2)
			[
				SNew(SBox)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						SNew(STextBlock)
						.Text(MaterialCompareInstruction)
						.ToolTipText(MaterialCompareInstructionTooltip)
					]
					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(2)
					[
						//Show the Comparison of the meshes
						SNew(SListView< TSharedPtr<FMaterialConflictData> >)
						.ListItemsSource(&ConflictMaterialListItem)
						.OnGenerateRow(this, &SFbxMaterialConflictWindow::OnGenerateRowForCompareMaterialList)
						.HeaderRow
						(
							SNew(SHeaderRow)
							+ SHeaderRow::Column("RowIndex")
							.DefaultLabel(FText::GetEmpty())
							.FixedWidth(25)
							+ SHeaderRow::Column("Current")
							.DefaultLabel(LOCTEXT("SFbxMaterialConflictWindow_Current_ColumnHeader", "Current Asset Materials"))
							.FillWidth(0.5f)
							+ SHeaderRow::Column("Fbx")
							.DefaultLabel(LOCTEXT("SFbxMaterialConflictWindow_Fbx_ColumnHeader", "Reimport Asset Materials"))
							.FillWidth(0.5f)
						)
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SColorBlock)
							.Color(FLinearColor(UNMATCHEDCOLOR))
							.Size(FVector2D(14, 14))
						]
						+ SHorizontalBox::Slot()
						.Padding(0.0f, 0.0f, 10.0f, 0.0f)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_UnMatched", " Unmatched"))
							.ToolTipText(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_UnMatched_Tooltip", "Unmatched material are fbx material name for which we did not find any match with the existing material names."))
							.ColorAndOpacity(FSlateColor(FLinearColor(UNMATCHEDCOLOR)))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SColorBlock)
							.Color(FLinearColor(CUSTOMMATCHEDCOLOR))
							.Size(FVector2D(14, 14))
						]
						+ SHorizontalBox::Slot()
						.Padding(0.0f, 0.0f, 10.0f, 0.0f)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_CustomMatched", " Custom Matched"))
							.ToolTipText(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_CustomMatched_Tooltip", "Custom matched material are fbx material name for which you already specify a matching material name."))
							.ColorAndOpacity(FSlateColor(FLinearColor(CUSTOMMATCHEDCOLOR)))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SColorBlock)
							.Color(FLinearColor(AUTOMATCHEDCOLOR))
							.Size(FVector2D(14, 14))
						]
						+ SHorizontalBox::Slot()
						.Padding(0.0f, 0.0f, 10.0f, 0.0f)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_AutoMatched", " Auto Matched"))
							.ToolTipText(LOCTEXT("SFbxMaterialConflictWindow_MaterialCompareLegend_AutoMatched_Tooltip", "Auto matched material are fbx material name for which we found a similar enough existing material name."))
							.ColorAndOpacity(FSlateColor(FLinearColor(AUTOMATCHEDCOLOR)))
						]
					]
				]
			]
		]
	];
}

TSharedRef<ITableRow> SFbxMaterialConflictWindow::OnGenerateRowForCompareMaterialList(TSharedPtr<FMaterialConflictData> RowData, const TSharedRef<STableViewBase>& Table)
{
	TSharedRef< SCompareRowDataTableListViewRow > ReturnRow = SNew(SCompareRowDataTableListViewRow, Table)
		.CompareRowData(RowData);
	return ReturnRow;
}

FText FMaterialConflictData::GetCellString(bool IsResultData) const
{
	int32 MaterialIndex = IsResultData ? ResultMaterialIndex : SourceMaterialIndex;
	TArray<FCompMaterial>& Materials = IsResultData ? ResultMaterials : SourceMaterials;

	if (!Materials.IsValidIndex(MaterialIndex))
	{
		return FText(LOCTEXT("GetCellString_InvalidIndex", "-"));
	}

	FString CellContent = Materials[MaterialIndex].ImportedMaterialSlotName.ToString();
	//Append the remapped data in case it was remap
	if (IsResultData &&
		RemapMaterials.IsValidIndex(MaterialIndex) &&
		RemapMaterials[MaterialIndex] != INDEX_NONE &&
		RemapMaterials[MaterialIndex] != MaterialIndex &&
		SourceMaterials.IsValidIndex(RemapMaterials[MaterialIndex]))
	{
		int32 RemapToIndex = RemapMaterials[MaterialIndex];
		FCompMaterial& RemappedReferenceCompMaterial = SourceMaterials[RemapToIndex];
		FString ImportedMaterialSlotName = RemappedReferenceCompMaterial.ImportedMaterialSlotName.ToString();
		CellContent += FString::Printf(TEXT(" match with [%d]%s"), RemapToIndex, *ImportedMaterialSlotName);
	}
	return FText::FromString(CellContent);
}

FText FMaterialConflictData::GetCellTooltipString(bool IsResultData) const
{
	int32 MaterialIndex = IsResultData ? ResultMaterialIndex : SourceMaterialIndex;
	TArray<FCompMaterial>& Materials = IsResultData ? ResultMaterials : SourceMaterials;
	if (!Materials.IsValidIndex(MaterialIndex))
	{
		return FText(LOCTEXT("GetCellString_InvalidIndex", "-"));
	}

	FString CellTooltip = TEXT("Material Slot Name: ") + Materials[MaterialIndex].MaterialSlotName.ToString();
	if (IsResultData &&
		RemapMaterials.IsValidIndex(MaterialIndex) &&
		RemapMaterials[MaterialIndex] != INDEX_NONE &&
		RemapMaterials[MaterialIndex] != MaterialIndex &&
		SourceMaterials.IsValidIndex(RemapMaterials[MaterialIndex]))
	{
		//Get the remap material slot name
		//so we can query 
		CellTooltip = TEXT("Material Slot Name: ") + SourceMaterials[RemapMaterials[MaterialIndex]].MaterialSlotName.ToString();
	}
	return FText::FromString(CellTooltip);
}

FReply FMaterialConflictData::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool IsResultData)
{
	if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton || !IsResultData || !ResultMaterials.IsValidIndex(ResultMaterialIndex) || !ParentContextMenu.IsValid() || SourceMaterials.Num() == 0)
	{
		return FReply::Unhandled();
	}
	
	if (bIsPreviewConflict)
	{
		return FReply::Handled();
	}

	//Gather the possible match item
	FMenuBuilder ContextMenu(true, TSharedPtr<const FUICommandList>());
	FText EntryName = FText(LOCTEXT("OnMouseButtonDown_menuClear", "Clear"));
	if (RemapMaterials[ResultMaterialIndex] != INDEX_NONE)
	{
		//INDEX_NONE clear the remapping
		ContextMenu.AddMenuEntry(EntryName, FText::GetEmpty(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &FMaterialConflictData::AssignMaterialMatch, (int32)INDEX_NONE)));
		ContextMenu.AddMenuSeparator();
	}
	for (int32 OriginalMaterialIndex = 0; OriginalMaterialIndex < SourceMaterials.Num(); ++OriginalMaterialIndex)
	{
		EntryName = FText::FromName(SourceMaterials[OriginalMaterialIndex].ImportedMaterialSlotName);
		ContextMenu.AddMenuEntry(EntryName, FText::GetEmpty(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &FMaterialConflictData::AssignMaterialMatch, OriginalMaterialIndex)));
	}

	FSlateApplication::Get().PushMenu(
		ParentContextMenu.ToSharedRef(),
		FWidgetPath(),
		ContextMenu.MakeWidget(),
		MouseEvent.GetScreenSpacePosition(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

	return FReply::Handled();
}

void FMaterialConflictData::AssignMaterialMatch(int32 OriginalMaterialIndex)
{
	if (RemapMaterials.IsValidIndex(ResultMaterialIndex) && RemapMaterials[ResultMaterialIndex] != OriginalMaterialIndex)
	{
		//Add the assignation to the remap array
		RemapMaterials[ResultMaterialIndex] = OriginalMaterialIndex;
		CustomRemapMaterials[ResultMaterialIndex] = true;
	}
	else if (OriginalMaterialIndex == INDEX_NONE)
	{
		RemapMaterials[ResultMaterialIndex] = INDEX_NONE;
	}
}

TSharedRef<SWidget> FMaterialConflictData::ConstructCellCurrent()
{
	if (!SourceMaterials.IsValidIndex(SourceMaterialIndex))
	{
		return SNew(SBox)
			.Padding(FMargin(5.0f, 2.0f, 0.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(FText::GetEmpty())
			];
	}

	return SAssignNew(ParentContextMenu, SBorder)
		.Padding(FMargin(5.0f, 2.0f, 0.0f, 2.0f))
		.OnMouseButtonDown(this, &FMaterialConflictData::OnMouseButtonDown, false)
		[
			SNew(STextBlock)
			.Text(this, &FMaterialConflictData::GetCellString, false)
			.ToolTipText(this, &FMaterialConflictData::GetCellTooltipString, false)
			.ColorAndOpacity(this, &FMaterialConflictData::GetCellColor, false)
		];
}

TSharedRef<SWidget> FMaterialConflictData::ConstructCellFbx()
{
	if (!ResultMaterials.IsValidIndex(ResultMaterialIndex))
	{
		return SNew(SBox)
			.Padding(FMargin(5.0f, 2.0f, 0.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(FText::GetEmpty())
			];
	}

	return SAssignNew(ParentContextMenu, SBorder)
		.Padding(FMargin(5.0f, 2.0f, 0.0f, 2.0f))
		.OnMouseButtonDown(this, &FMaterialConflictData::OnMouseButtonDown, true)
		[
			SNew(STextBlock)
			.Text(this, &FMaterialConflictData::GetCellString, true)
			.ToolTipText(this, &FMaterialConflictData::GetCellTooltipString, true)
			.ColorAndOpacity(this, &FMaterialConflictData::GetCellColor, true)
		];
}

FSlateColor FMaterialConflictData::GetCellColor(bool IsResultData) const
{
	if (IsResultData)
	{
		if (RemapMaterials[ResultMaterialIndex] == INDEX_NONE)
		{
			//No match
			return FSlateColor(FLinearColor(UNMATCHEDCOLOR));
		}
		else if (CustomRemapMaterials[ResultMaterialIndex])
		{
			//Custom Match
			return FSlateColor(FLinearColor(CUSTOMMATCHEDCOLOR));
		}
		else if (AutoRemapMaterials[ResultMaterialIndex])
		{
			//Auto Match
			return FSlateColor(FLinearColor(AUTOMATCHEDCOLOR));
		}
		//Material is matched
		return FSlateColor::UseForeground();
	}
	else
	{
		if (!RemapMaterials.Contains(SourceMaterialIndex))
		{
			//No match
			return FSlateColor(FLinearColor(UNMATCHEDCOLOR));
		}
		//Material is matched
		return FSlateColor::UseForeground();
	}
}

void SFbxMaterialConflictWindow::FillMaterialListItem()
{
	//Build the compare data to show in the UI
	int32 MaterialCompareRowNumber = RemapMaterials->Num();
	for (int32 RowIndex = 0; RowIndex < MaterialCompareRowNumber; ++RowIndex)
	{
		TSharedPtr<FMaterialConflictData> CompareRowData = MakeShareable(new FMaterialConflictData(*SourceMaterials, *ResultMaterials, *RemapMaterials, *AutoRemapMaterials, CustomRemapMaterials, bIsPreviewConflict));
		CompareRowData->SourceMaterialIndex = RowIndex;
		CompareRowData->ResultMaterialIndex = RowIndex;
		CompareRowData->RowIndex = RowIndex;
		ConflictMaterialListItem.Add(CompareRowData);
	}
}

FReply SFbxMaterialConflictWindow::OnCancel()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	
	ReturnOption = UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_Cancel;

	return FReply::Handled();
}

FReply SFbxMaterialConflictWindow::OnReset()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}

	ReturnOption = UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_ResetToFbx;

	return FReply::Handled();
}

FReply SFbxMaterialConflictWindow::OnDone()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	
	ReturnOption = UnFbx::EFBXReimportDialogReturnOption::FBXRDRO_Ok;

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
